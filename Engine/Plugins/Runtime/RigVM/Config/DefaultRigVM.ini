[CoreRedirects]

; renaming decorator to trait
+StructRedirects=(OldName="RigVMDecorator", NewName="/Script/RigVM.RigVMTrait")
+PropertyRedirects=(OldName="RigVMNode.DecoratorRootPinNames",NewName="RigVMNode.TraitRootPinNames")

; graph function layout renaming
+StructRedirects=(OldName="RigVMGraphFunctionLayout", NewName="/Script/RigVM.RigVMNodeLayout")
+StructRedirects=(OldName="RigVMGraphFunctionCategory", NewName="/Script/RigVM.RigVMPinCategory")