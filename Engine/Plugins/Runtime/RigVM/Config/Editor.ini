[/Script/RigVM.RigVMProjectSettings]
+VariantTags=(Name="Outdated",Label="Outdated",ToolTip=NSLOCTEXT("RigVMSettings", "OutdatedToolTip", "This item is outdated and should no longer be used."),Color=(R=1.0,G=0.0,B=0.0,A=1.0),bShowInUserInterface=True,bMarksSubjectAsInvalid=True)
+VariantTags=(Name="Stable",Label="Stable",ToolTip=NSLOCTEXT("RigVMSettings", "StableToolTip", "This item is stable and ready to use."),Color=(R=0.0,G=1.0,B=0.0,A=1.0),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
