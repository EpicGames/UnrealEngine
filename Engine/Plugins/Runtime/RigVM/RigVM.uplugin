{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "RigVM",
	"Description": "Provides frontend and backend for the RigVM visual programming language and runtime",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules" :
	[
		{
			"Name" : "RigVM",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "RigVMDeveloper",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "PreDefault"
		},
		{
			"Name" : "RigVMEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		}
	],
	"Plugins" :
	[
		{
			"Name": "GameplayInsights",
			"Enabled": true
		},
		{
			"Name": "Scribble",
			"Enabled": true
		},
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		}
	]
}