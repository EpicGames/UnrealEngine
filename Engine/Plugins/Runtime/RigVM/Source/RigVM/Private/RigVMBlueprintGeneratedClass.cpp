// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMBlueprintGeneratedClass.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "RigVMHost.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMBlueprintGeneratedClass)

URigVMBlueprintGeneratedClass::URigVMBlueprintGeneratedClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

uint8* URigVMBlueprintGeneratedClass::GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const
{
	if(!IsInGameThread())
	{
		// we cant use the persistent frame if we are executing in parallel (as we could potentially thunk to BP)
		return nullptr;
	}
	return Super::GetPersistentUberGraphFrame(Obj, FuncToCheck);
}

void URigVMBlueprintGeneratedClass::PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph)
{
	// Skip internal classes, such as skeleton and reinstancing.
	if (IsInternalClass())
	{
		return;
	}

	if (URigVMHost* Owner = Cast<URigVMHost>(InObj))
	{
		URigVMHost* CDO = nullptr;
		if (!Owner->HasAnyFlags(RF_ClassDefaultObject))
		{
			CDO = Cast<URigVMHost>(GetDefaultObject());
		}
		Owner->PostInitInstance(CDO);
	}
}

void URigVMBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("URigVMBlueprintGeneratedClass(%s)"), *GetName()));

	Super::Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMGeneratedClass)
	{
		return;
	}

	URigVM* VM = NewObject<URigVM>(GetTransientPackage());

	if (!IsInternalClass())
	{
		if (URigVMHost* CDO = Cast<URigVMHost>(GetDefaultObject(true)))
		{
			const bool bIsSerializingToDisk = URigVM::IsSerializingToDisk(Ar);
			if (Ar.IsSaving() && bIsSerializingToDisk && CDO->VM)
			{
				VM->CopyDataForSerialization(CDO->VM);
				VM->CreateLocalizedRegistryIfRequired();
			}
		}
	}
	
	VM->Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("VM"));

	if (!IsInternalClass())
	{
		if (URigVMHost* CDO = Cast<URigVMHost>(GetDefaultObject(false)))
		{
			if (Ar.IsLoading())
			{
				CDO->VM->CopyDataForSerialization(VM);
			}
		}
	}

	Ar << GraphFunctionStore;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("GraphFunctionStore"));
}

void URigVMBlueprintGeneratedClass::PostLoad()
{
	Super::PostLoad();

	GraphFunctionStore.PostLoad();
}

void URigVMBlueprintGeneratedClass::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	const FArrayProperty* HeadersProperty = CastField<FArrayProperty>(FRigVMGraphFunctionHeaderArray::StaticStruct()->FindPropertyByName(TEXT("Headers")));
	FRigVMGraphFunctionHeaderArray HeaderArray;
	
	for (const FRigVMGraphFunctionData& FunctionData : GraphFunctionStore.PublicFunctions)
	{
		if (FunctionData.CompilationData.IsValid())
		{
			HeaderArray.Headers.Add(FunctionData.Header);
		}
	}

	FString HeadersString;
	HeadersProperty->ExportText_Direct(HeadersString, &(HeaderArray.Headers), &(HeaderArray.Headers), nullptr, PPF_None, nullptr);

	Context.AddTag(UObject::FAssetRegistryTag(TEXT("PublicGraphFunctions"), HeadersString, UObject::FAssetRegistryTag::TT_Hidden));
}

UClass* URigVMBlueprintGeneratedClass::GetRigVMHostClass() const
{
	if (URigVMHost* CDO = Cast<URigVMHost>(GetDefaultObject(false)))
	{
		return CDO->GetClass();
	}
	return URigVMHost::StaticClass();
}

void URigVMBlueprintGeneratedClass::UpdateSupportedEventNames()
{
	SupportedEventNames.Reset();
	SupportedEventNames = GetDefaultObject<URigVMHost>()->GetSupportedEvents();
}

TArray<UObject*> URigVMBlueprintGeneratedClass::GetArchetypeInstances(bool bIncludeDerivedClass) const
{
	TArray<UObject*> Instances;
	GetDefaultObject()->GetArchetypeInstances(Instances);
	return Instances;
}

bool URigVMBlueprintGeneratedClass::InitializeVariables(URigVMHost* InInstance)
{
	URigVMHost* CDO = GetDefaultObject<URigVMHost>();
	TArray<FRigVMExternalVariable> CDOVariables = CDO->GetExternalVariablesImpl(false);
	
	TArray<FRigVMExternalVariable> CurrentVariables = InInstance->GetExternalVariablesImpl(false);
	if (ensure(CurrentVariables.Num() == CDOVariables.Num()))
	{
		for (int32 i=0; i<CurrentVariables.Num(); ++i)
		{
			FRigVMExternalVariable& Variable = CurrentVariables[i];
			FRigVMExternalVariable& CDOVariable = CDOVariables[i];
			Variable.GetProperty()->CopyCompleteValue(Variable.GetMemory(), CDOVariable.GetMemory());
		}
		return true;
	}
	
	return false;
}
