// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMNativized.h"
#include "UObject/Package.h"
#include "UObject/AnimObjectVersion.h"
#include "RigVMObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "RigVMObjectVersion.h"
#include "HAL/PlatformTLS.h"
#include "Async/ParallelFor.h"
#include "Engine/UserDefinedEnum.h"
#include "GenericPlatform/GenericPlatformSurvey.h"
#include "RigVMCore/RigVMStruct.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ObjectSaveContext.h"
#include "RigVMHost.h"
#include "RigVMCore/RigVMTrait.h"
#include "RigVMStringUtils.h"
#include "Misc/MTAccessDetector.h"
#include "RigVMTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVM)

#if UE_RIGVM_DEBUG_EXECUTION
static TAutoConsoleVariable<int32> CVarControlRigDebugAllVMExecutions(
	TEXT("ControlRig.DebugAllVMExecutions"),
	0,
	TEXT("If nonzero we allow to copy the execution of a VM execution."),
	ECVF_Default);
#endif

static TAutoConsoleVariable<int32> CVarControlRigCacheMemoryHandlesBatchSize(
	TEXT("ControlRig.CacheMemoryHandlesBatchSize"),
	10,
	TEXT("If greather than zero, the cache of memory handles will be processed using a parallel_for of the specified batch size, else single thread processing."),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarControlRigCacheMemoryHandlesParallelMinSize(
	TEXT("ControlRig.CacheMemoryHandlesParallelMinSize"),
	700,
	TEXT("If greather than zero, the cache of memory handles will be processed using a parallel_for if the amount of handles is greather than the specified number, else single thread processing."),
	ECVF_Default);

void FRigVMParameter::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		return;
	}

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		// remove due to FPIEFixupSerializer hitting this checkNoEntry();
	}
}

void FRigVMParameter::Save(FArchive& Ar)
{
	Ar << Type;
	Ar << Name;
	Ar << RegisterIndex;
	Ar << CPPType;
	Ar << ScriptStructPath;
}

void FRigVMParameter::Load(FArchive& Ar)
{
	Ar << Type;
	Ar << Name;
	Ar << RegisterIndex;
	Ar << CPPType;
	Ar << ScriptStructPath;

	ScriptStruct = nullptr;
}

UScriptStruct* FRigVMParameter::GetScriptStruct() const
{
	if (ScriptStruct == nullptr)
	{
		if (ScriptStructPath != NAME_None)
		{
			FRigVMParameter* MutableThis = (FRigVMParameter*)this;
			MutableThis->ScriptStruct = FindObject<UScriptStruct>(nullptr, *ScriptStructPath.ToString());
		}
	}
	return ScriptStruct;
}

#define UE_RIGVM_GET_READ_REGISTRY_CONTEXT() \
TUniquePtr<FRigVMRegistryReadLock> RegistryReadLock; \
if(!LocalizedRegistry.IsValid()) \
{ \
	RegistryReadLock = MakeUnique<FRigVMRegistryReadLock>(); \
} \
FRigVMRegistryHandle& RegistryHandle = RegistryReadLock.IsValid() ? *RegistryReadLock.Get() : LocalizedRegistry->GetHandle_NoLock();

URigVM::URigVM()
	: ByteCodePtr(&ByteCodeStorage)
    , FunctionNamesPtr(&FunctionNamesStorage)
    , FunctionsPtr(&FunctionsStorage)
    , FactoriesPtr(&FactoriesStorage)
{
}

URigVM::~URigVM()
{
	Reset_Internal();
}

void URigVM::Serialize(FArchive& Ar)
{
	const TGuardValue<bool> _(bIsSerializing, true);
	
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		return;
	}
	
	// call into the super class to serialize any uproperty
	if(Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Super::Serialize(Ar);
	}

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		// remove due to FPIEFixupSerializer hitting this checkNoEntry();
	}
}

void URigVM::Save(FArchive& Ar)
{
	const bool bIsSerializingToDisk = IsSerializingToDisk(Ar);
	
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("URigVM(%s)"), *GetName()));

	// The Save function has to be in sync with CopyDataForSerialization
	Ar << CachedVMHash;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("CachedVMHash"));

	Ar << ExternalPropertyPathDescriptions;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("ExternalPropertyPathDescriptions"));
	Ar << FunctionNamesStorage;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("FunctionNamesStorage"));
	Ar << ByteCodeStorage;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("ByteCodeStorage"));
	Ar << Parameters;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Parameters"));

	Ar << LiteralMemoryStorage;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("LiteralMemoryStorage"));
	Ar << DefaultWorkMemoryStorage;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("DefaultWorkMemoryStorage"));

	TArray<const FRigVMFunction*> TempFunctionsStorage;
	TArray<FName> TempFunctionNamesStorage; 
	TArray<const FRigVMDispatchFactory*> TempFactoriesStorage;
	bool bUsingTemporaryLocalizedRegistry = false;

	UE::TScopeLock CreateLocalizedRegistryScopeLock(CreateLocalizedRegistryMutex);

	// if we are serializing to disk we should create the localized registry temporarily
	if (bIsSerializingToDisk && !LocalizedRegistry.IsValid())
	{
		TempFunctionsStorage = FunctionsStorage;
		TempFunctionNamesStorage = FunctionNamesStorage;
		TempFactoriesStorage = FactoriesStorage;

		// we can call the no-lock flavor here since we've established the scope lock above
		if (CreateLocalizedRegistryIfRequired_NoLock(true))
		{
			bUsingTemporaryLocalizedRegistry = true;
		}
	}

	if(bIsSerializingToDisk && LocalizedRegistry.IsValid())
	{
		FRigVMMemoryStorageStruct EmptyDebugMemoryStruct;
		Ar << EmptyDebugMemoryStruct;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("DefaultDebugMemoryStorage"));

		bool bStoredLocalizedRegistry = true;
		Ar << bStoredLocalizedRegistry;

		const int64 ArchivePosBeforeSerializedRegistry = Ar.Tell();

		int64 ArchiveOffsetForSerializedRegistry = 0;
		Ar << ArchiveOffsetForSerializedRegistry;

		LocalizedRegistry->Serialize_NoLock(Ar, this);

		// store the archive positions after the registry so we
		// can skip it during load
		const int64 ArchivePosAfterSerializedRegistry = Ar.Tell();
		ArchiveOffsetForSerializedRegistry = ArchivePosAfterSerializedRegistry - ArchivePosBeforeSerializedRegistry;
		Ar.Seek(ArchivePosBeforeSerializedRegistry);
		Ar << ArchiveOffsetForSerializedRegistry;
		Ar.Seek(ArchivePosAfterSerializedRegistry);

		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("LocalizedRegistry"));
	}
	else
	{
		Ar << DefaultDebugMemoryStorage_DEPRECATED;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("DefaultDebugMemoryStorage"));

		bool bStoredLocalizedRegistry = false;
		Ar << bStoredLocalizedRegistry;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("LocalizedRegistry"));
	}

	if (bUsingTemporaryLocalizedRegistry)
	{
		// remove the localized registry
		LocalizedRegistry.Reset();
		// swap the function / factory storage back
		FunctionsStorage = TempFunctionsStorage;
		FunctionNamesStorage = TempFunctionNamesStorage;
		FactoriesStorage = TempFactoriesStorage;
	}
}

void URigVM::Load(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);
	
	Reset_Internal();

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::BeforeCustomVersionWasAdded)
	{
		int32 RigVMUClassBasedStorageDefine = 1;
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::RigVMMemoryStorageObject)
		{
			Ar << RigVMUClassBasedStorageDefine;
		}

		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMExternalExecuteContextStruct
			&& Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::RigVMSerializeExecuteContextStruct)
		{
			FString ExecuteContextPath;
			Ar << ExecuteContextPath;
			// Context is now external to the VM, just serializing the string to keep compatibility
		}

		if (RigVMUClassBasedStorageDefine == 1)
		{
			FRigVMMemoryContainer TmpWorkMemoryStorage;
			FRigVMMemoryContainer TmpLiteralMemoryStorage;

			Ar << TmpWorkMemoryStorage;
			Ar << TmpLiteralMemoryStorage;
			Ar << FunctionNamesStorage;
			Ar << ByteCodeStorage;
			Ar << Parameters;

			if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMCopyOpStoreNumBytes)
			{
				Reset_Internal();
				return;
			}

			if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMStoringUserDefinedStructMap 
				&& Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::HostStoringUserDefinedData)
			{	
				// now serialized at Host
				TMap<FString, FSoftObjectPath> UserDefinedStructGuidToPathName;
				Ar << UserDefinedStructGuidToPathName;
			}
			if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMStoringUserDefinedEnumMap
				&& Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::HostStoringUserDefinedData)
			{
				// now serialized at Host
				TMap<FString, FSoftObjectPath> UserDefinedEnumToPathName;
				Ar << UserDefinedEnumToPathName;
			}
		}

		// we only deal with virtual machines now that use the new memory infrastructure.
		ensure(UE_RIGVM_UCLASS_BASED_STORAGE_DISABLED == 0);
		if (RigVMUClassBasedStorageDefine != UE_RIGVM_UCLASS_BASED_STORAGE_DISABLED)
		{
			Reset_Internal();
			return;
		}
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::AddedVMHashChecks)
	{
		Ar << CachedVMHash;
	}
	Ar << ExternalPropertyPathDescriptions;
	Ar << FunctionNamesStorage;
	Ar << ByteCodeStorage;
	ByteCodeStorage.AlignByteCode();
	Ar << Parameters;

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::RigVMSaveDebugMapInGraphFunctionData ||
		Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::RigVMSaveDebugMapInGraphFunctionData)
	{
		if (Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::DebugOperandMappingSimplified)
		{
			TMap<FRigVMOperand, TArray<FRigVMOperand>> PreviousOperandToDebugRegisters;
			Ar << PreviousOperandToDebugRegisters;
		}
	}
		
	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMStoringUserDefinedStructMap
		&& Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::HostStoringUserDefinedData)
	{
		// now serialized at Host
		TMap<FString, FSoftObjectPath> UserDefinedStructGuidToPathName;
		Ar << UserDefinedStructGuidToPathName;
	}
	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMStoringUserDefinedEnumMap
		&& Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::HostStoringUserDefinedData)
	{
		// now serialized at Host
		TMap<FString, FSoftObjectPath> UserDefinedEnumToPathName;
		Ar << UserDefinedEnumToPathName;
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMMemoryStorageStructSerialized)
	{
		Ar << LiteralMemoryStorage;
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMMemoryStorageDefaultsGeneratedAtVM)
	{
		Ar << DefaultWorkMemoryStorage;
		Ar << DefaultDebugMemoryStorage_DEPRECATED;
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::LocalizedRegistry)
	{
		bool bStoredLocalizedRegistry = false;
		Ar << bStoredLocalizedRegistry;

		LocalizedRegistry.Reset();
		if(bStoredLocalizedRegistry)
		{
			const int64 ArchivePosBeforeSerializedRegistry = Ar.Tell();

			int64 ArchiveOffsetForSerializedRegistry = 0;
			Ar << ArchiveOffsetForSerializedRegistry;

			if(CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread() == false)
			{
				if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::LocalizedRegistryWithRelativeSeekOffset)
				{
					Ar.Seek(ArchivePosBeforeSerializedRegistry + ArchiveOffsetForSerializedRegistry);
				}
				else
				{
					// in older versions the offset is considered absolute
					Ar.Seek(ArchiveOffsetForSerializedRegistry);
				}
			}
			else
			{
				LocalizedRegistry = FRigVMRegistry_NoLock::CreateLocalizedRegistry();
				LocalizedRegistry->Serialize_NoLock(Ar, this);
			}
		}
	}
}

void URigVM::CopyDataForSerialization(URigVM* InVM)
{
	check(InVM);
	UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);

	Reset_Internal();

	CachedVMHash = InVM->CachedVMHash;
	ExternalPropertyPathDescriptions = InVM->ExternalPropertyPathDescriptions;
	FunctionNamesStorage = InVM->FunctionNamesStorage;
	ByteCodeStorage = InVM->ByteCodeStorage;
	Parameters = InVM->Parameters;

	LiteralMemoryStorage = InVM->LiteralMemoryStorage;
	DefaultWorkMemoryStorage = InVM->DefaultWorkMemoryStorage;
	DefaultDebugMemoryStorage_DEPRECATED.Reset();
	CopyExternalVariableDefs(InVM);

	LocalizedRegistry.Reset();
	if (InVM->LocalizedRegistry.IsValid())
	{
		LocalizedRegistry = FRigVMRegistry_NoLock::CloneLocalizedRegistry(InVM->LocalizedRegistry.Get());
	}
}

void URigVM::CopyExternalVariableDefs(URigVM* InVM)
{
	ExternalVariables = InVM->ExternalVariables;
}

bool URigVM::IsSerializingToDisk(const FArchive& Ar)
{
	return Ar.IsPersistent()
		&& !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE);
}

void URigVM::PostLoad()
{
	Super::PostLoad();

	// In packaged builds, initialize the CDO VM
	// In editor, the VM will be recompiled and initialized at URigVMBlueprint::HandlePackageDone::RecompileVM
#if WITH_EDITOR
	if (GetPackage()->bIsCookedForEditor)
#endif
	{
		Instructions.Reset();
		FunctionsStorage.Reset();
		FactoriesStorage.Reset();
		ParametersNameMap.Reset();

		for (int32 Index = 0; Index < Parameters.Num(); Index++)
		{
			ParametersNameMap.Add(Parameters[Index].Name, Index);
		}

		// Rebuild functions storage from serialized function names
		ResolveFunctionsIfRequired();

		// Rebuild instructions from ByteCodeStorage
		RefreshInstructionsIfRequired();

		// rebuild the bytecode to adjust for byte shifts in shipping
		RebuildByteCodeOnLoad();

		// rebuild the argument name cache, so it is already calculated during init
		RefreshArgumentNameCaches();
	}
}

void URigVM::RefreshArgumentNameCaches()
{
	// make sure that the functions cannot change at the same time as
	// the argument name caches are being updated.
	UE::TScopeLock ResolveFunctionsScopeLock(ResolveFunctionsMutex);

	int32 TotalNumFactories;
	if(LocalizedRegistry.IsValid())
	{
		TotalNumFactories = LocalizedRegistry->GetFactories_NoLock().Num();
	}
	else
	{
		TotalNumFactories = FRigVMRegistry::Get().GetFactories().Num();
	}

	TArray<const FRigVMDispatchFactory*> Factories;

	// make sure to update all argument name caches
	const TArray<const FRigVMFunction*>& Functions = GetFunctions();
	for (const FRigVMInstruction& Instruction : Instructions)
	{
		if (Instruction.OpCode == ERigVMOpCode::Execute)
		{
			const FRigVMExecuteOp& Op = ByteCodeStorage.GetOpAt<FRigVMExecuteOp>(Instruction);
			if (Functions.IsValidIndex(Op.CallableIndex))
			{
				if (!Functions[Op.CallableIndex])
				{
					continue;
				}
				
				if (const FRigVMDispatchFactory* Factory = Functions[Op.CallableIndex]->Factory)
				{
					if(Factories.IsEmpty())
					{
						Factories.AddZeroed(TotalNumFactories);
					}
					
					if(Factories[Factory->GetFactoryIndex()] == nullptr)
					{
						Factories[Factory->GetFactoryIndex()] = Factory;
						
						FRigVMOperandArray Operands = ByteCodeStorage.GetOperandsForCallableOp(Instruction);
						if(LocalizedRegistry.IsValid())
						{
							(void)Factory->UpdateArgumentNameCache_NoLock(Operands.Num(), LocalizedRegistry->GetHandle_NoLock());
						}
						else
						{
							FRigVMRegistryReadLock ReadLock;
							(void)Factory->UpdateArgumentNameCache(Operands.Num(), ReadLock);
						}
					}
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
void URigVM::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMMemoryStorage::StaticClass()));
}
#endif

bool URigVM::IsContextValidForExecution(FRigVMExtendedExecuteContext& Context) const
{
	return Context.VMHash == GetVMHash();
}

// Stores the current VM hash
void URigVM::SetVMHash(uint32 InVMHash)
{
	CachedVMHash = InVMHash;
}

uint32 URigVM::GetVMHash() const
{
	return CachedVMHash;
}

uint32 URigVM::ComputeVMHash() const
{
	return ComputeVMHash(ExternalVariables);
}

uint32 URigVM::ComputeVMHash(const TArray<FRigVMExternalVariableDef>& InExternalVariables) const
{
	uint32 Hash = 0;
	for(const FName& FunctionName : GetFunctionNames())
	{
		Hash = HashCombine(Hash, GetTypeHash(FunctionName.ToString()));
	}

	Hash = HashCombine(Hash, GetTypeHash(GetByteCode()));

	for(const FRigVMExternalVariableDef& ExternalVariable : InExternalVariables)
	{
		Hash = HashCombine(Hash, GetTypeHash(ExternalVariable.GetName().ToString()));
		Hash = HashCombine(Hash, GetTypeHash(ExternalVariable.GetExtendedCPPType().ToString()));
	}

	Hash = HashCombine(Hash, LiteralMemoryStorage.GetMemoryHash());
	Hash = HashCombine(Hash, DefaultWorkMemoryStorage.GetMemoryHash());

	return Hash;
}

uint32 URigVM::ComputeVMHash(const TArray<FRigVMExternalVariable>& InExternalVariables) const
{
	return ComputeVMHash(RigVMTypeUtils::GetExternalVariableDefs(InExternalVariables));
}

UClass* URigVM::GetNativizedClass(uint32 InHash) const
{
	return URigVMNativized::FindClassForHash(InHash);
}

bool URigVM::ValidateBytecode()
{
	// check all operands on all ops for validity
	const TArray<const FRigVMMemoryStorageStruct*> LocalMemory = { &GetDefaultWorkMemory(), &GetDefaultLiteralMemory(), &DefaultDebugMemoryStorage_DEPRECATED };
	
	auto CheckOperandValidity = [LocalMemory, this](const FRigVMOperand& InOperand) -> bool
	{
		if(InOperand.GetContainerIndex() < 0 || InOperand.GetContainerIndex() >= (int32)ERigVMMemoryType::Invalid)
		{
			return false;
		}


		const FRigVMMemoryStorageStruct* MemoryForOperand = LocalMemory[InOperand.GetContainerIndex()];
		if(InOperand.GetMemoryType() != ERigVMMemoryType::External)
		{
			if(!MemoryForOperand->IsValidIndex(InOperand.GetRegisterIndex()))
			{
				return false;
			}

			if(InOperand.GetRegisterOffset() != INDEX_NONE)
			{
				if(!MemoryForOperand->GetPropertyPaths().IsValidIndex(InOperand.GetRegisterOffset()))
				{
					return false;
				}
			}
		}
		else if (InOperand.GetMemoryType() == ERigVMMemoryType::External)
		{
			// given that external variables array is populated at runtime
			// checking for property path is the best we can do
			if(InOperand.GetRegisterOffset() != INDEX_NONE)
			{
				if (!ExternalPropertyPathDescriptions.IsValidIndex(InOperand.GetRegisterOffset()))
				{
					return false;
				}
			}
		}
		return true;
	};

	const TArray<const FRigVMFunction*>& Functions = GetFunctions();
	
	const FRigVMInstructionArray ByteCodeInstructions = ByteCodeStorage.GetInstructions();
	for(const FRigVMInstruction& ByteCodeInstruction : ByteCodeInstructions)
	{
		switch (ByteCodeInstruction.OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				const FRigVMExecuteOp& Op = ByteCodeStorage.GetOpAt<FRigVMExecuteOp>(ByteCodeInstruction);
				if (!Functions.IsValidIndex(Op.CallableIndex))
				{
					return false;
				}
				FRigVMOperandArray Operands = ByteCodeStorage.GetOperandsForCallableOp(ByteCodeInstruction);
				for (const FRigVMOperand& Arg : Operands)
				{
					if (!CheckOperandValidity(Arg))
					{
						return false;
					}
				}
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				const FRigVMInvokeCallableOp& Op = ByteCodeStorage.GetOpAt<FRigVMInvokeCallableOp>(ByteCodeInstruction);
				if (!ByteCodeStorage.IsValidCallableIndex(Op.CallableIndex))
				{
					return false;
				}
				FRigVMOperandArray Operands = ByteCodeStorage.GetOperandsForCallableOp(ByteCodeInstruction);
				for (const FRigVMOperand& Arg : Operands)
				{
					if (!CheckOperandValidity(Arg))
					{
						return false;
					}
				}
				break;
			}
			case ERigVMOpCode::Zero:
			case ERigVMOpCode::BoolFalse:
			case ERigVMOpCode::BoolTrue:
			case ERigVMOpCode::Increment:
			case ERigVMOpCode::Decrement:
			{
				const FRigVMUnaryOp& Op = ByteCodeStorage.GetOpAt<FRigVMUnaryOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.Arg))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::Copy:
			{
				const FRigVMCopyOp& Op = ByteCodeStorage.GetOpAt<FRigVMCopyOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.Source) ||
					!CheckOperandValidity(Op.Target))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::Equals:
			case ERigVMOpCode::NotEquals:
			{
				const FRigVMComparisonOp& Op = ByteCodeStorage.GetOpAt<FRigVMComparisonOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.A) ||
					!CheckOperandValidity(Op.B) ||
					!CheckOperandValidity(Op.Result))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			case ERigVMOpCode::JumpForwardIf:
			case ERigVMOpCode::JumpBackwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCodeStorage.GetOpAt<FRigVMJumpIfOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.Arg))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::BeginBlock:
			{
				const FRigVMBinaryOp& Op = ByteCodeStorage.GetOpAt<FRigVMBinaryOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.ArgA) ||
					!CheckOperandValidity(Op.ArgB))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				const FRigVMJumpToBranchOp& Op = ByteCodeStorage.GetOpAt<FRigVMJumpToBranchOp>(ByteCodeInstruction);
				if (!CheckOperandValidity(Op.Arg))
				{
					return false;
				}
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				const FRigVMRunInstructionsOp& Op = ByteCodeStorage.GetOpAt<FRigVMRunInstructionsOp>(ByteCodeInstruction);
				CheckOperandValidity(Op.Arg);
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				const FRigVMSetupTraitsOp& Op = ByteCodeStorage.GetOpAt<FRigVMSetupTraitsOp>(ByteCodeInstruction);
				CheckOperandValidity(Op.Arg);
				break;
			}
			case ERigVMOpCode::Invalid:
			{
				ensure(false);
				break;
			}
			default:
			{
				break;
			}
		}
	}

	return true;
}

const URigVMHost* URigVM::GetHostCDO() const
{
	return GetTypedOuter<const URigVMHost>();
}

void URigVM::Reset_Internal()
{
	CachedVMHash = 0;
	CachedNativizedVMHash.Reset();
	FunctionNamesStorage.Reset();
	FunctionsStorage.Reset();
	FactoriesStorage.Reset();
	ExternalPropertyPathDescriptions.Reset();
	ExternalPropertyPaths.Reset();
	ByteCodeStorage.Reset();
	Instructions.Reset();
	Parameters.Reset();
	ParametersNameMap.Reset();

	FunctionNamesPtr = &FunctionNamesStorage;
	FunctionsPtr = &FunctionsStorage;
	FactoriesPtr = &FactoriesStorage;
	ByteCodePtr = &ByteCodeStorage;

	ExternalVariables.Reset();
	LazyBranches.Reset();

	InvalidateCachedMemory_Internal();
	LocalizedRegistry.Reset();
}

void URigVM::Reset(FRigVMExtendedExecuteContext& Context)
{
	Reset_Internal();
	InvalidateCachedMemory(Context);
}

void URigVM::Empty(FRigVMExtendedExecuteContext& Context)
{
	FunctionNamesStorage.Empty();
	FunctionsStorage.Empty();
	FactoriesStorage.Empty();
	ExternalPropertyPathDescriptions.Empty();
	ExternalPropertyPaths.Empty();
	ByteCodeStorage.Empty();
	Instructions.Empty();
	Parameters.Empty();
	ParametersNameMap.Empty();
	ExternalVariables.Empty();

	InvalidateCachedMemory(Context);

	Context.ExternalVariableRuntimeData.Reset();
}

int32 URigVM::AddRigVMFunction(UScriptStruct* InRigVMStruct, const FName& InMethodName)
{
	check(InRigVMStruct);
	const FString FunctionKey = FString::Printf(TEXT("F%s::%s"), *InRigVMStruct->GetName(), *InMethodName.ToString());
	return AddRigVMFunction(FunctionKey);
}

int32 URigVM::AddRigVMFunction(const FString& InFunctionName)
{
	const FName FunctionFName = *InFunctionName;
	const int32 FunctionIndex = GetFunctionNames().Find(FunctionFName);
	if (FunctionIndex != INDEX_NONE)
	{
		return FunctionIndex;
	}
	return AddRigVMFunctionImpl(FunctionFName, InFunctionName);
}

int32 URigVM::AddRigVMFunction(const FName& InFunctionFName)
{
	const int32 FunctionIndex = GetFunctionNames().Find(InFunctionFName);
	if (FunctionIndex != INDEX_NONE)
	{
		return FunctionIndex;
	}
	return AddRigVMFunctionImpl(InFunctionFName, InFunctionFName.ToString());
}

int32 URigVM::AddRigVMFunctionImpl(const FName& InFunctionFName, const FString& InFunctionName)
{
	const FRigVMFunction* Function = nullptr;
	if(LocalizedRegistry.IsValid())
	{
		Function = LocalizedRegistry.Get()->FindFunction_NoLock(*InFunctionName);
	}
	else
	{
		Function = FRigVMRegistry_RWLock::Get().FindFunction(*InFunctionName);
	}
	if (Function == nullptr)
	{
		return INDEX_NONE;
	}

	GetFunctionNames().Add(InFunctionFName);
	GetFactories().Add(Function->Factory);
	return GetFunctions().Add(Function);
}

FString URigVM::GetRigVMFunctionName(int32 InFunctionIndex) const
{
	return GetFunctionNames()[InFunctionIndex].ToString();
}

FRigVMMemoryStorageStruct* URigVM::GetMemoryByType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType/*, bool bCreateIfNeeded*/)
{
	FRigVMMemoryStorageStruct* MemoryStorage = nullptr;

	switch(InMemoryType)
	{
		case ERigVMMemoryType::Literal:
		{
			MemoryStorage = GetLiteralMemory();
			break;
		}

		case ERigVMMemoryType::Work:
		{
			MemoryStorage = &Context.WorkMemoryStorage;
			break;
		}

		case ERigVMMemoryType::Debug:
		{
			MemoryStorage = &Context.DebugMemoryStorage;
			break;
		}

		default:
		{
			break;
		}
	}

	return MemoryStorage;
}

const FRigVMMemoryStorageStruct* URigVM::GetMemoryByType(const FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType) const
{
	return const_cast<URigVM*>(this)->GetMemoryByType(const_cast<FRigVMExtendedExecuteContext&>(Context), InMemoryType/*, false*/);
}

FRigVMMemoryStorageStruct* URigVM::CreateDebugMemory(FRigVMExtendedExecuteContext& Context)
{
#if WITH_EDITOR
	// first find all properties on the work memory which are used by any input or output operand
	// on the bytecode while skipping execute contexts.
	TMap<FRigVMOperand, FRigVMOperand> OperandMap;
	TSet<int32> WorkPropertyIndices;
	const FRigVMByteCode& ByteCode = GetByteCode();
	for (int32 Index = 0; Index < ByteCode.GetNumInstructions(); Index++)
	{
		const FRigVMOperandArray InputOperands = ByteCode.GetInputOperands(Index);
		const FRigVMOperandArray OutputOperands = ByteCode.GetOutputOperands(Index);
		for (int32 Phase = 0; Phase < 2; Phase++)
		{
			const FRigVMOperandArray& Operands = Phase == 0 ? InputOperands : OutputOperands;
			for (const FRigVMOperand& Operand : Operands)
			{
				if (Operand.GetMemoryType() != ERigVMMemoryType::Work)
				{
					continue;
				}
				
				// ignore the register offset / property paths since we need the whole operand's memory
				bool bAlreadyExistedInList = false;
				const int32 DebugOperandIndex = WorkPropertyIndices.Add(Operand.GetRegisterIndex(), &bAlreadyExistedInList).AsInteger();
				if (bAlreadyExistedInList)
				{
					continue;
				}
				
				OperandMap.Add(
					FRigVMOperand(ERigVMMemoryType::Work, Operand.GetRegisterIndex()),
					FRigVMOperand(ERigVMMemoryType::Debug, DebugOperandIndex));
			}
		}
	}

	const FRigVMMemoryStorageStruct& WorkMemory = GetDefaultWorkMemory();

	// create a debug property for each operand
	TArray<FRigVMPropertyDescription> PropertyDescriptions;
	for (const int32& WorkPropertyIndex : WorkPropertyIndices)
	{
		 if (!WorkMemory.IsValidIndex(WorkPropertyIndex))
		 {
			 continue;
		 }
		
		const FProperty* Property = WorkMemory.GetProperty(WorkPropertyIndex);
		check(Property);

		FString PropertyCPPType = RigVMTypeUtils::GetCPPTypeFromProperty(Property);
		UObject* CPPTypeObject = RigVMTypeUtils::GetCPPTypeObjectFromProperty(Property);

		// ignore any execute types
		if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			if (ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				continue;
			}
			PropertyCPPType = RigVMTypeUtils::GetUniqueStructTypeName(ScriptStruct);
		}
		const FString CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(PropertyCPPType);

		FRigVMPropertyDescription PropertyDescription(Property->GetFName(), CPPType, CPPTypeObject, TEXT("()"), true);
		PropertyDescriptions.Add(PropertyDescription);
	}

	FRigVMMemoryStorageStruct DebugMemory;
	DebugMemory.AddProperties(PropertyDescriptions);

	Context.DebugMemoryStorage = DebugMemory;
	Context.DebuggedOperands.Reset();
	Context.OperandToDebugRegister = OperandMap;
#endif
	return &Context.DebugMemoryStorage; 
}

void URigVM::GenerateMemoryType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>* InProperties)
{
	GenerateDefaultMemoryType(InMemoryType, InProperties);

	switch (InMemoryType)
	{
	case ERigVMMemoryType::Work:
	{
		Context.WorkMemoryStorage = DefaultWorkMemoryStorage;
		break;
	}

	case ERigVMMemoryType::Debug:
	{
		// debug memory is managed by the context of the host.
		break;
	}

	default:
	{
		break;
	}
	}
}

void URigVM::GenerateDefaultMemoryType(ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>* InProperties)
{
	FRigVMMemoryStorageStruct* Memory = nullptr;

	switch (InMemoryType)
	{
		case ERigVMMemoryType::Literal:
		{
			Memory = &LiteralMemoryStorage;
			break;
		}

		case ERigVMMemoryType::Work:
		{
			Memory = &DefaultWorkMemoryStorage;
			break;
		}

#if WITH_EDITOR
		case ERigVMMemoryType::Debug:
		{
			Memory = &DefaultDebugMemoryStorage_DEPRECATED;
			break;
		}
#endif

		default:
		{
			break;
		}
	}

	if (Memory != nullptr)
	{
		*Memory = (InProperties != nullptr) ? FRigVMMemoryStorageStruct(InMemoryType, *InProperties) : FRigVMMemoryStorageStruct(InMemoryType);
	}
}

FRigVMMemoryStorageStruct* URigVM::GetDefaultMemoryByType(ERigVMMemoryType InMemoryType)
{
	FRigVMMemoryStorageStruct* MemoryStorage = nullptr;

	switch (InMemoryType)
	{
		case ERigVMMemoryType::Literal:
		{
			MemoryStorage = &LiteralMemoryStorage;
			break;
		}

		case ERigVMMemoryType::Work:
		{
			MemoryStorage = &DefaultWorkMemoryStorage;
			break;
		}

		case ERigVMMemoryType::Debug:
		{
#if WITH_EDITOR
			MemoryStorage = &DefaultDebugMemoryStorage_DEPRECATED;
#endif
			break;
		}

		default:
		{
			break;
		}
	}

	return MemoryStorage;
}

const FRigVMMemoryStorageStruct* URigVM::GetDefaultMemoryByType(ERigVMMemoryType InMemoryType) const
{
	return const_cast<URigVM*>(this)->GetDefaultMemoryByType(InMemoryType/*, false*/);
}

void URigVM::ClearMemory_Internal()
{
	// At one point our memory objects were saved with RF_Public, so to truly clear them, we have to also clear the flags
	// RF_Public will make them stay around as zombie unreferenced objects, and get included in SavePackage and cooking.
	// Clear their flags so they are not included by editor or cook SavePackage calls.

	// we now make sure that only the literal memory object on the CDO is marked as RF_Public
	// and work memory objects are no longer marked as RF_Public
	// We don't do this for packaged builds, though.

#if WITH_EDITOR
	// Running with `-game` will set GIsEditor to nullptr.
	if (GIsEditor)
	{
		TArray<UObject*> SubObjects;
		GetObjectsWithOuter(GetOuter(), SubObjects);
		for (UObject* SubObject : SubObjects)
		{
			if (URigVMMemoryStorage* MemoryObject = Cast<URigVMMemoryStorage>(SubObject))
			{
				// we don't care about memory type here because
				// 
				// if "this" is not CDO, its subobjects will not include the literal memory and
				// thus only clears the flag for work mem
				// 
				// if "this" is CDO, its subobjects will include the literal memory and this allows
				// us to actually clear the literal memory
				MemoryObject->ClearFlags(RF_Public);
			}
		}
	}
#endif

	LiteralMemoryStorage = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);
	DefaultWorkMemoryStorage = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);
#if WITH_EDITOR
	DefaultDebugMemoryStorage_DEPRECATED = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);
#endif

	InvalidateCachedMemory_Internal();
}

void URigVM::ClearMemory(FRigVMExtendedExecuteContext& Context)
{
	ClearMemory_Internal();

	Context.WorkMemoryStorage = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);
	Context.DebugMemoryStorage = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);

	InvalidateCachedMemory(Context);
}

const FRigVMInstructionArray& URigVM::GetInstructions()
{
	return Instructions;
}

bool URigVM::ContainsEntry(const FName& InEntryName) const
{
	const FRigVMByteCode& ByteCode = GetByteCode();
	return ByteCode.FindEntryIndex(InEntryName) != INDEX_NONE;
}

int32 URigVM::FindEntry(const FName& InEntryName) const
{
	const FRigVMByteCode& ByteCode = GetByteCode();
	return ByteCode.FindEntryIndex(InEntryName);
}

const TArray<FName>& URigVM::GetEntryNames() const
{
	EntryNames.Reset();
	
	const FRigVMByteCode& ByteCode = GetByteCode();
	for (int32 EntryIndex = 0; EntryIndex < ByteCode.NumEntries(); EntryIndex++)
	{
		EntryNames.Add(ByteCode.GetEntry(EntryIndex).Name);
	}

	return EntryNames;
}

bool URigVM::CanExecuteEntry(const FRigVMExtendedExecuteContext& Context, const FName& InEntryName, bool bLogErrorForMissingEntry) const
{
	const int32 EntryIndex = FindEntry(InEntryName);
	if(EntryIndex == INDEX_NONE)
	{
		if(bLogErrorForMissingEntry)
		{
			static constexpr TCHAR MissingEntry[] = TEXT("Entry('%s') cannot be found.");
			Context.GetPublicData<>().Logf(EMessageSeverity::Error, MissingEntry, *InEntryName.ToString());
		}
		return false;
	}
	
	if(Context.EntriesBeingExecuted.Contains(EntryIndex))
	{
		TArray<FString> EntryNamesBeingExecuted;
		for(const int32 EntryBeingExecuted : Context.EntriesBeingExecuted)
		{
			EntryNamesBeingExecuted.Add(GetEntryNames()[EntryBeingExecuted].ToString());
		}
		EntryNamesBeingExecuted.Add(InEntryName.ToString());

		static constexpr TCHAR RecursiveEntry[] = TEXT("Entry('%s') is being invoked recursively (%s).");
		Context.GetPublicData<>().Logf(EMessageSeverity::Error, RecursiveEntry, *InEntryName.ToString(), *RigVMStringUtils::JoinStrings(EntryNamesBeingExecuted, TEXT(" -> ")));
		return false;
	}

	return true;
}

#if WITH_EDITOR

bool URigVM::ResumeExecution(FRigVMExtendedExecuteContext& Context, const FName& InEntryName)
{
	return false;
}

#endif 

const TArray<FRigVMParameter>& URigVM::GetParameters() const
{
	return Parameters;
}

FRigVMParameter URigVM::GetParameterByName(const FName& InParameterName)
{
	if (ParametersNameMap.Num() == Parameters.Num())
	{
		const int32* ParameterIndex = ParametersNameMap.Find(InParameterName);
		if (ParameterIndex)
		{
			Parameters[*ParameterIndex].GetScriptStruct();
			return Parameters[*ParameterIndex];
		}
		return FRigVMParameter();
	}

	for (FRigVMParameter& Parameter : Parameters)
	{
		if (Parameter.GetName() == InParameterName)
		{
			Parameter.GetScriptStruct();
			return Parameter;
		}
	}

	return FRigVMParameter();
}

bool URigVM::ResolveFunctionsIfRequired()
{
	// only create the function resolval lock if we are running
	// on the global registry.
#if WITH_EDITOR
	TUniquePtr<UE::TScopeLock<FTransactionallySafeCriticalSection>> ResolveFunctionsScopeLock;
	if(!LocalizedRegistry.IsValid())
	{
		ResolveFunctionsScopeLock = MakeUnique<UE::TScopeLock<FTransactionallySafeCriticalSection>>(ResolveFunctionsMutex);
	}
#endif
	
	bool bSuccess = true;
	
	if (GetFunctions().Num() != GetFunctionNames().Num())
	{
		GetFunctions().Reset();
		GetFunctions().SetNumZeroed(GetFunctionNames().Num());
		GetFactories().Reset();
		GetFactories().SetNumZeroed(GetFunctionNames().Num());

		FRigVMUserDefinedTypeResolver TypeResolver;
		if (const URigVMHost* HostCDO = GetHostCDO())
		{
			TypeResolver = FRigVMUserDefinedTypeResolver([HostCDO](const FString& InTypeName) -> UObject*
			{
				return HostCDO->ResolveUserDefinedTypeById(InTypeName);
			});
		}

		UE_RIGVM_GET_READ_REGISTRY_CONTEXT();
		
		TArray<FName>& FunctionNames = GetFunctionNames();
		for (int32 FunctionIndex = 0; FunctionIndex < FunctionNames.Num(); FunctionIndex++)
		{
			const FString FunctionNameString = FunctionNames[FunctionIndex].ToString();
			if(const FRigVMFunction* Function = RegistryHandle->FindFunction_NoLock(*FunctionNameString, TypeResolver))
			{
				GetFunctions()[FunctionIndex] = Function;
				GetFactories()[FunctionIndex] = Function->Factory;
				
				// update the name in the function name list. the resolved function
				// may differ since it may rely on a core redirect.
				if(!FunctionNameString.Equals(Function->Name, ESearchCase::CaseSensitive))
				{
					FunctionNames[FunctionIndex] = *Function->Name;
					UE_LOGF(LogRigVM, Verbose, "Redirected function '%ls' to '%ls' for VM '%ls'", *FunctionNameString, *Function->Name, *GetPathName());
				}
			}
			else
			{
				// We cannot recover from missing functions.
				UE_LOGF(LogRigVM, Error, "No handler found for function '%ls' for VM '%ls'", *FunctionNameString, *GetPathName());

				// Print more information to help with debugging
				if (LocalizedRegistry.IsValid())
				{
					for (int32 FactoryIndex = 0; FactoryIndex < LocalizedRegistry->GetFactories_NoLock().Num(); FactoryIndex++)
					{
						const FRigVMDispatchFactory* AvailableFactory = LocalizedRegistry->GetFactories_NoLock()[FactoryIndex];
						UE_LOGF(LogRigVM, Error, "Available factory [%03d] '%ls' for VM '%ls'", FactoryIndex, *AvailableFactory->GetFactoryName().ToString(), *GetPathName());
					}
					for (const FRigVMFunction& AvailableFunction : LocalizedRegistry->GetFunctions_NoLock())
					{
						UE_LOGF(LogRigVM, Error, "Available function [%03d] '%ls' for VM '%ls'", AvailableFunction.Index, *AvailableFunction.Name, *GetPathName());
					}
				}
				
				bSuccess = false;
			}
		}
	}

	return bSuccess;
}

void URigVM::RefreshInstructionsIfRequired()
{
	if (GetByteCode().Num() == 0 && Instructions.Num() > 0)
	{
		Instructions.Reset();
	}
	else if (Instructions.Num() == 0)
	{
		Instructions = GetByteCode().GetInstructions();
	}
}

void URigVM::InvalidateCachedMemory_Internal()
{
	FirstHandleForInstruction.Reset();
	MemoryHandleCount = 0;
	ExternalPropertyPaths.Reset();
	LazyBranches.Reset();
	PreCachedMemoryHandles.Reset();
	PreCachedMemoryHandlesMemoryType.Reset();
}

void URigVM::InvalidateCachedMemory(FRigVMExtendedExecuteContext& Context)
{
	InvalidateCachedMemory_Internal();
	Context.InvalidateCachedMemory();
}

void URigVM::InstructionOpEval(
	FRigVMExtendedExecuteContext& Context,
	FRigVMRegistryHandle& RegistryHandle,
	int32 InstructionIndex,
	int32 InHandleBaseIndex,
	const TFunctionRef<void(
		FRigVMExtendedExecuteContext& /*Context*/,
		FRigVMRegistryHandle& /*RegistryHandle*/,
		int32 /*InstructionIndex*/,
		int32 /*InHandleIndex*/,
		const FRigVMBranchInfoKey& /*InBranchInfoKey*/,
		const FRigVMOperand& /*InArg*/
	)>& InOpFunc)
{
	const FRigVMByteCode& ByteCode = GetByteCode();
	const TArray<const FRigVMFunction*>& Functions = GetFunctions();

	const FRigVMInstruction& Instruction = Instructions[InstructionIndex]; 
	const ERigVMOpCode OpCode = Instruction.OpCode;

	if (OpCode == ERigVMOpCode::Execute)
	{
		const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instruction);
		FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instruction);
		const FRigVMFunction* Function = Functions[Op.CallableIndex];
		if(Function == nullptr)
		{
			return;
		}

		if (Function->Factory)
		{
			checkf(Function->Arguments.Num() <= Operands.Num(), TEXT("%s: invalid number of operands (%d) for dispatch '%s' - expected at least (%d)."), *GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
		}
		else
		{
			checkf(Function->Arguments.Num() == Operands.Num(), TEXT("%s: invalid number of operands (%d) for function '%s' - expected (%d)."), *GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
		}
		
		for (int32 ArgIndex = 0; ArgIndex < Operands.Num(); ArgIndex++)
		{
			InOpFunc(
				Context,
				RegistryHandle,
				InstructionIndex,
				InHandleBaseIndex++,
				{ InstructionIndex, ArgIndex, Function->GetArgumentNameForOperandIndex(ArgIndex, Operands.Num(), RegistryHandle) },
				Operands[ArgIndex]);
		}
	}
	else if (OpCode == ERigVMOpCode::InvokeCallable)
	{
		const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instruction);
		FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instruction);
		const FRigVMCallableInfo* Callable = ByteCode.GetCallable(Op.CallableIndex);
		if(Callable == nullptr)
		{
			return;
		}

		checkf(Callable->Arguments.Num() == Operands.Num(), TEXT("%s: invalid number of operands (%d) for callable '%s' - expected (%d)."), *GetPathName(), Operands.Num(), *Callable->Name.ToString(), Callable->Arguments.Num());
		
		int32 TotalArgIndex = 0;
		for (int32 ArgIndex = 0; ArgIndex < Operands.Num(); ArgIndex++, TotalArgIndex++)
		{
			InOpFunc(
				Context,
				RegistryHandle,
				InstructionIndex,
				InHandleBaseIndex++,
				{ InstructionIndex, TotalArgIndex, Callable->Arguments[ArgIndex].Name },
				Operands[ArgIndex]);
		}
	}
	else
	{
		switch (OpCode)
		{
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		{
			const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instruction);
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Op.Arg);
			break;
		}
		case ERigVMOpCode::Copy:
		{
			const FRigVMCopyOp& Op = ByteCode.GetOpAt<FRigVMCopyOp>(Instruction);
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 0, {}, Op.Source);
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 1, {}, Op.Target);
			break;
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			const FRigVMComparisonOp& Op = ByteCode.GetOpAt<FRigVMComparisonOp>(Instruction);
			FRigVMOperand Arg = Op.A;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 0, {}, Arg);
			Arg = Op.B;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 1, {}, Arg);
			Arg = Op.Result;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 2, {}, Arg);
			break;
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		{
			break;
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);
			const FRigVMOperand& Arg = Op.Arg;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Arg);
			break;
		}
		case ERigVMOpCode::ChangeType:
		{
			const FRigVMChangeTypeOp& Op = ByteCode.GetOpAt<FRigVMChangeTypeOp>(Instruction);
			const FRigVMOperand& Arg = Op.Arg;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Arg);
			break;
		}
		case ERigVMOpCode::Exit:
		{
			break;
		}
		case ERigVMOpCode::BeginBlock:
		{
			const FRigVMBinaryOp& Op = ByteCode.GetOpAt<FRigVMBinaryOp>(Instruction);
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 0, {}, Op.ArgA);
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex + 1, {}, Op.ArgB);
			break;
		}
		case ERigVMOpCode::EndBlock:
		{
			break;
		}
		case ERigVMOpCode::InvokeEntry:
		{
			break;
		}
		case ERigVMOpCode::JumpToBranch:
		{
			const FRigVMJumpToBranchOp& Op = ByteCode.GetOpAt<FRigVMJumpToBranchOp>(Instruction);
			const FRigVMOperand& Arg = Op.Arg;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Arg);
			break;
		}
		case ERigVMOpCode::RunInstructions:
		{
			const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instruction);
			const FRigVMOperand& Arg = Op.Arg;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Arg);
			break;
		}
		case ERigVMOpCode::SetupTraits:
		{
			const FRigVMSetupTraitsOp& Op = ByteCode.GetOpAt<FRigVMSetupTraitsOp>(Instruction);
			const FRigVMOperand& Arg = Op.Arg;
			InOpFunc(Context, RegistryHandle, InstructionIndex, InHandleBaseIndex, {}, Arg);
			break;
		}
		case ERigVMOpCode::Invalid:
		default:
		{
			checkNoEntry();
			break;
		}
		}
	}
}

void URigVM::PrepareMemoryForExecution(FRigVMExtendedExecuteContext& Context)
{
	ensureMsgf(Context.ExecutingThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("RigVM::PrepareMemoryForExecution from multiple threads (%d and %d)"), Context.ExecutingThreadId, (int32)FPlatformTLS::GetCurrentThreadId());

	InvalidateCachedMemory(Context);

	if (URigVMHost* CDO = Cast<URigVMHost>(GetOuter()))
	{
		FRigVMExtendedExecuteContext& ExtendedExecuteContext = CDO->GetRigVMExtendedExecuteContext();

		// update the VM's external variables
		ClearExternalVariables(ExtendedExecuteContext);
		SetExternalVariableDefs(CDO->GetExternalVariablesImpl(false));
	}

	if (Instructions.Num() == 0)
	{
		return;
	}

	RefreshExternalPropertyPaths();

	FRigVMByteCode& ByteCode = GetByteCode();

	// force to update the map of branch infos once
	(void)ByteCode.GetBranchInfo({ 0, 0 });

	const int32 LazyBranchSize = GetByteCode().BranchInfos.Num();
	LazyBranches.Reset(LazyBranchSize);
	LazyBranches.SetNumZeroed(LazyBranchSize);

	// Make sure we have enough room to prevent repeated allocations.
	FirstHandleForInstruction.Reset(Instructions.Num() + 1);

	TUniquePtr<FRigVMRegistryReadLock> RegistryReadLock;
	if(!LocalizedRegistry.IsValid())
	{
		RegistryReadLock = MakeUnique<FRigVMRegistryReadLock>();
	}
	FRigVMRegistryHandle& RegistryHandle = RegistryReadLock.IsValid() ? *RegistryReadLock.Get() : LocalizedRegistry->GetHandle_NoLock();

	// Count how many handles we need and set up the indirection offsets for the handles.
	int32 HandleCount = 0;
	FirstHandleForInstruction.Add(0);
	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		InstructionOpEval(Context, RegistryHandle, InstructionIndex, INDEX_NONE,
			[&HandleCount](FRigVMExtendedExecuteContext&, const FRigVMRegistryHandle&, int32, int32, const FRigVMBranchInfoKey&, const FRigVMOperand& InArg)
			{
				HandleCount++;
			});
		FirstHandleForInstruction.Add(HandleCount);
	}

	MemoryHandleCount = HandleCount;
	PreCachedMemoryHandles.Reset(MemoryHandleCount);
	PreCachedMemoryHandlesMemoryType.Reset(MemoryHandleCount);
}

void URigVM::CacheMemoryHandles(FRigVMExtendedExecuteContext& Context)
{
	const int32 NumInstructions = Instructions.Num();
	if (NumInstructions == 0)
	{
		return;
	}

	check((NumInstructions + 1) == FirstHandleForInstruction.Num());

	if (PreCachedMemoryHandles.Num() == MemoryHandleCount)
	{
		return;
	}

	// Allocate all the space and zero it out to ensure all pages required for it are paged in immediately.
	PreCachedMemoryHandles.SetNumUninitialized(MemoryHandleCount);
	PreCachedMemoryHandlesMemoryType.SetNumUninitialized(MemoryHandleCount);

	UE_RIGVM_GET_READ_REGISTRY_CONTEXT();

	// Now cache the handles as needed, in several batches as needed
	const int32 MinHandlesForParallel = CVarControlRigCacheMemoryHandlesParallelMinSize->GetInt();
	const bool bUseParallel = CVarControlRigCacheMemoryHandlesBatchSize->GetInt() > 0 && MinHandlesForParallel > 0 && NumInstructions > MinHandlesForParallel;
	const int32 TaskBatchSize = bUseParallel ? CVarControlRigCacheMemoryHandlesBatchSize->GetInt() : INT32_MAX;

	ParallelFor(TEXT("CacheMemoryHandles"), NumInstructions, TaskBatchSize, [this, &Context, &RegistryHandle](int32 InstructionIndex)
		{
			InstructionOpEval(Context, RegistryHandle, InstructionIndex, FirstHandleForInstruction[InstructionIndex],
				[&](FRigVMExtendedExecuteContext& InContext, const FRigVMRegistryHandle& InRegistry, int32 InInstructionIndex, int32 InHandleIndex, const FRigVMBranchInfoKey& InBranchInfoKey, const FRigVMOperand& InOp)
				{
					PreCacheSingleMemoryHandle(InRegistry, InInstructionIndex, InHandleIndex, InBranchInfoKey, InOp);
				});
		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread
	);
}

void URigVM::RebuildByteCodeOnLoad()
{
	Instructions = GetByteCode().GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		switch(Instruction.OpCode)
		{
			case ERigVMOpCode::Copy:
			{
				FRigVMCopyOp OldCopyOp = GetByteCode().GetOpAt<FRigVMCopyOp>(Instruction);
				if((OldCopyOp.Source.GetMemoryType() == ERigVMMemoryType::External) ||
					(OldCopyOp.Target.GetMemoryType() == ERigVMMemoryType::External))
				{
					if(ExternalVariables.IsEmpty())
					{
						break;
					}
				}
					
				// create a local copy of the original op
				FRigVMCopyOp& NewCopyOp = GetByteCode().GetOpAt<FRigVMCopyOp>(Instruction);
				NewCopyOp = GetCopyOpForOperands(OldCopyOp.Source, OldCopyOp.Target);
				check(OldCopyOp.Source == NewCopyOp.Source);
				check(OldCopyOp.Target == NewCopyOp.Target);
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

bool URigVM::Initialize(FRigVMExtendedExecuteContext& Context)
{
	if (Context.ExecutingThreadId != INDEX_NONE)
	{
		ensureMsgf(Context.ExecutingThreadId == FPlatformTLS::GetCurrentThreadId(), TEXT("RigVM::Initialize from multiple threads (%d and %d)"), Context.ExecutingThreadId, (int32)FPlatformTLS::GetCurrentThreadId());
	}
	
	TGuardValue<int32> GuardThreadId(Context.ExecutingThreadId, FPlatformTLS::GetCurrentThreadId());

	RefreshInstructionsIfRequired();
	ResolveFunctionsIfRequired();

	if (Instructions.Num() == 0)
	{
		return true;
	}

	RefreshArgumentNameCaches();

	PrepareMemoryForExecution(Context);

	CacheMemoryHandles(Context);

	return true;
}

bool URigVM::InitializeInstance(FRigVMExtendedExecuteContext& Context, bool bCopyMemory)
{
	TGuardValue<int32> GuardThreadId(Context.ExecutingThreadId, FPlatformTLS::GetCurrentThreadId());

	const int32 LazyBranchSize = GetByteCode().BranchInfos.Num();
	Context.Frame->LazyBranchExecuteState.Reset(LazyBranchSize);
	Context.Frame->LazyBranchExecuteState.SetNumZeroed(LazyBranchSize);

	if (bCopyMemory)
	{
		Context.VMHash = GetVMHash();
		Context.WorkMemoryStorage = DefaultWorkMemoryStorage;
	}

	return true;
}

ERigVMExecuteResult URigVM::ExecuteVM(FRigVMExtendedExecuteContext& Context, const FName& InEntryName)
{
	// if this the first entry being executed - get ready for execution
	const bool bIsRootEntry = Context.EntriesBeingExecuted.IsEmpty();

	TGuardValue<FName> EntryNameGuard(Context.CurrentEntryName, InEntryName);
	TGuardValue<bool> RootEntryGuard(Context.bCurrentlyRunningRootEntry, bIsRootEntry);

	FRigVMByteCode& ByteCode = GetByteCode();

#if ENABLE_MT_DETECTOR
	FBaseScopedAccessDetector ScopedMTAccessDetector;
#endif

	if (bIsRootEntry)
	{
		Context.CurrentExecuteResult = ERigVMExecuteResult::Succeeded;

#if ENABLE_MT_DETECTOR
		ScopedMTAccessDetector = MakeScopedReaderAccessDetector(AccessDetector);
#endif

		if (Instructions.Num() == 0)
		{
			return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
		}

		Context.UpdateInstanceMemory(GetLiteralMemory());

		if (ByteCode.HasPublicContextAssetPath())
		{
			const FTopLevelAssetPath ContextPublicDataAssetPath(Context.GetContextPublicDataStruct());
			if (ByteCode.GetPublicContextAssetPath().IsNull() || ByteCode.GetPublicContextAssetPath() != ContextPublicDataAssetPath)
			{
				UE_LOGF(LogRigVM, Error, "Context PublicData [%ls] does not match ByteCode Public Data [%ls]. Likely a corrupt VM. Exiting.", *ContextPublicDataAssetPath.ToString(), *ByteCode.GetPublicContextAssetPath().ToString());
				return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
			}
		}
	}


	TArray<const FRigVMFunction*>& Functions = GetFunctions();
	TArray<const FRigVMDispatchFactory*>& Factories = GetFactories();

	FRigVMExecuteContext& ContextPublicData = Context.GetPublicData<>();

#if WITH_EDITOR
	TArray<FName>& FunctionNames = GetFunctionNames();

	if(bIsRootEntry)
	{
		SetupInstructionTracking(Context, Instructions.Num(), ByteCode.NumCallables(), InEntryName);
	}
#endif

	if(bIsRootEntry)
	{
		Context.ResetExecutionState();
		Context.SetVM(this);
		Context.Frame->SliceOffsetsPerInstruction.AddZeroed(Instructions.Num());
		Context.Frame->SliceOffsetsPerCallable.AddZeroed(ByteCode.NumCallables());
	}

	ensure(Context.GetVM() == this);

	if(bIsRootEntry)
	{
		ClearDebugMemory(Context);
	}

	int32 EntryIndexToPush = INDEX_NONE;
	if (!InEntryName.IsNone())
	{
		int32 EntryIndex = ByteCode.FindEntryIndex(InEntryName);
		if (EntryIndex == INDEX_NONE)
		{
			return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
		}
		
		if (!SetInstructionIndex(Context, ContextPublicData, ByteCode.GetEntry(EntryIndex).InstructionIndex))
		{
			return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
		}
		
		EntryIndexToPush = EntryIndex;

		if(bIsRootEntry)
		{
			ContextPublicData.EventName = InEntryName;
		}
	}
	else
	{
		int32 FirstInstructionIndex = 0;
		if (bIsRootEntry)
		{
			// move the instruction index to the first instruction not owned by callable
			for (int32 CallableIndex = 0; CallableIndex < ByteCode.NumCallables(); ++CallableIndex)
			{
				const FRigVMCallableInfo* Callable = ByteCode.GetCallable(CallableIndex);
				check(Callable);
				FirstInstructionIndex = FMath::Max(FirstInstructionIndex, Callable->LastInstruction + 1);
			}
			if (!SetInstructionIndex(Context, ContextPublicData, FirstInstructionIndex))
			{
				return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
			}
		}
		
		for(int32 EntryIndex = 0; EntryIndex < ByteCode.NumEntries(); EntryIndex++)
		{
			if(ByteCode.GetEntry(EntryIndex).InstructionIndex == FirstInstructionIndex)
			{
				EntryIndexToPush = EntryIndex;
				break;
			}
		}
		
		if(bIsRootEntry)
		{
			if(ByteCode.Entries.IsValidIndex(EntryIndexToPush))
			{
				ContextPublicData.EventName = ByteCode.GetEntry(EntryIndexToPush).Name;
			}
			else
			{
				ContextPublicData.EventName = NAME_None;
			}
		}
	}

	FEntryExecuteGuard EntryExecuteGuard(Context.EntriesBeingExecuted, EntryIndexToPush);

	if(bIsRootEntry)
	{
		ContextPublicData.NumExecutions++;
	}

	if(Context.bCurrentlyRunningRootEntry)
	{
		StartProfiling(Context);
	}
#if WITH_EDITOR
	
#if UE_RIGVM_DEBUG_EXECUTION
	if (CVarControlRigDebugAllVMExecutions->GetBool() || ContextPublicData.bDebugExecution)
	{
		ContextPublicData.InstanceOpCodeEnum = StaticEnum<ERigVMOpCode>();
		FRigVMMemoryStorageStruct* LiteralMemory = GetLiteralMemory();
		ContextPublicData.DebugMemoryString = FString("\n\nLiteral Memory\n\n");
		for (int32 PropertyIndex=0; PropertyIndex<LiteralMemory->Num(); ++PropertyIndex)
		{
			ContextPublicData.DebugMemoryString += FString::Printf(TEXT("%s: %s\n"), *LiteralMemory->GetProperties()[PropertyIndex]->GetFullName(), *LiteralMemory->GetDataAsString(PropertyIndex));				
		}
		ContextPublicData.DebugMemoryString += FString(TEXT("\n\nWork Memory\n\n"));
	}
	
#endif
	
#endif

	// find the last instruction
	int32 LastInstruction = Instructions.Num() - 1;
	for (int32 InstructionIndex = ContextPublicData.InstructionIndex + 1; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		if (Instructions[InstructionIndex].OpCode == ERigVMOpCode::Exit)
		{
			LastInstruction = InstructionIndex;
			break;
		}
	}

#if WITH_EDITOR
	TSharedPtr<FRigVMDebugInfo> RigVMDebugInfo = Context.GetRigVMDebugInfoWeak().Pin();

	if (bIsRootEntry && RigVMDebugInfo)
	{
		// The marker needs to make sure to record until it has passed
		// the origin to be able to step forward, into etc.
		RigVMDebugInfo->BeginExecution();
	}
#endif

	Context.CurrentExecuteResult = ExecuteInstructions(Context, ContextPublicData.InstructionIndex, LastInstruction);

#if WITH_EDITOR
	if(bIsRootEntry)
	{
		Context.CurrentVMMemory.Reset();
		
		if (RigVMDebugInfo)
		{
			if (RigVMDebugInfo->IsActive())
			{
				if (Context.CurrentExecuteResult != ERigVMExecuteResult::Failed)
				{
					Context.CurrentExecuteResult = ERigVMExecuteResult::Halted;
					RigVMDebugInfo->ExecutionHalted().Broadcast(RigVMDebugInfo->GetStepCondition().OriginInstruction, nullptr, InEntryName);
				}
			}
		}
	}
#endif

	return Context.CurrentExecuteResult;
}

struct FRigVMCallstackGuard
{
public:
	FRigVMCallstackGuard(FRigVMExtendedExecuteContext& InContext, const FRigVMByteCode& ByteCode, int32 InCallableIndex)
		: Context(InContext)
		, PublicData(InContext.GetPublicData<>())
		, PreviousInstructionIndex(INDEX_NONE)
		, PreviousCallstackHash(InContext.CallstackHash)
		, bIsValid(false)
	{
		PreviousInstructionIndex = Context.GetInstructionIndex();
		bIsValid = PreviousInstructionIndex != INDEX_NONE && ByteCode.IsValidCallableIndex(InCallableIndex);
		if (bIsValid)
		{
			const FRigVMCallableInfo* Callable = ByteCode.GetCallable(InCallableIndex);
			PublicData.InstructionIndex = Callable->FirstInstruction;
			Context.Callstack.Push({PreviousInstructionIndex, Callable});
			Context.CallstackHash = HashCombine(PreviousInstructionIndex, InCallableIndex, Context.CallstackHash);
			Context.BeginCallableSlice(Callable->Index);

#if WITH_EDITOR
			if (FRigVMInstructionVisitInfo* RigVMInstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
			{
				RigVMInstructionVisitInfo->SetCallableVisitedDuringLastRun(Callable->Index);
				RigVMInstructionVisitInfo->AddCallableIndexToVisitOrder(Callable->Index);
			}
#endif
		}
	}

	~FRigVMCallstackGuard()
	{
		if (bIsValid)
		{
			Context.EndCallableSlice();
			Context.CallstackHash = PreviousCallstackHash;
			Context.Callstack.Pop();
			PublicData.InstructionIndex = PreviousInstructionIndex;
		}
	}
		
private:
	FRigVMExtendedExecuteContext& Context;
	FRigVMExecuteContext& PublicData;
	int32 PreviousInstructionIndex;
	uint32 PreviousCallstackHash;
	bool bIsValid;
};

ERigVMExecuteResult URigVM::ExecuteInstructions(FRigVMExtendedExecuteContext& Context, int32 InFirstInstruction, int32 InLastInstruction)
{
	FRigVMExecuteContext& ContextPublicData = Context.GetPublicData<>();
	
#if RIGVM_TRACE_ENABLED
	if (ContextPublicData.IsHostPlayingRewindDebugTrace())
	{
		return ERigVMExecuteResult::Succeeded;
	}
#endif
	
	// make we are already executing this VM
	check(!Context.CurrentVMMemory.IsEmpty());
	

	FRigVMByteCode& ByteCode = GetByteCode();

	TGuardValue<int32> InstructionIndexGuard(ContextPublicData.InstructionIndex, InFirstInstruction);
	
#if WITH_EDITOR
	FInstructionBracketGuard InstructionBracket(Context, InFirstInstruction, InLastInstruction);
	if(!InstructionBracket.ErrorMessage.IsEmpty())
	{
		Context.GetPublicDataSafe<>().Log(EMessageSeverity::Error, InstructionBracket.ErrorMessage);
		return ERigVMExecuteResult::Failed;
	}

	TSharedPtr<TGuardValue<FRigVMDebugInfo*>> GuardStrongDebugInfo;
	const TWeakPtr<FRigVMDebugInfo>& WeakDebugInfo = Context.GetRigVMDebugInfoWeak();
	if (WeakDebugInfo.IsValid())
	{
		if (FRigVMDebugInfo* CurrentDebugInfo = WeakDebugInfo.Pin().Get())
		{
			GuardStrongDebugInfo = MakeShared<TGuardValue<FRigVMDebugInfo*>>(Context.DebugInfoStrong, CurrentDebugInfo);
		}
	}
#endif
	
	TArray<const FRigVMFunction*>& Functions = GetFunctions();
	TArray<const FRigVMDispatchFactory*>& Factories = GetFactories();

	TArray<FRigVMMemoryHandle, TInlineAllocator<64>> HandlesMemory; // have ready some memory to patch handles

	// have the VM instance memory by type ready for handles patching
	static_assert((int32)ERigVMMemoryType::Num == 4);
	const TStaticArray<uint8*, (int32)ERigVMMemoryType::Num> VMMemoryPtrs =
	{
		GetMemoryByType(Context, ERigVMMemoryType::Work) != nullptr ? (uint8*)GetMemoryByType(Context, ERigVMMemoryType::Work)->GetContainerPtr() : nullptr,
		GetMemoryByType(Context, ERigVMMemoryType::Literal) != nullptr ? (uint8*)GetMemoryByType(Context, ERigVMMemoryType::Literal)->GetContainerPtr() : nullptr,
		GetMemoryByType(Context, ERigVMMemoryType::External) != nullptr ? (uint8*)GetMemoryByType(Context, ERigVMMemoryType::External)->GetContainerPtr() : nullptr,
		GetMemoryByType(Context, ERigVMMemoryType::Debug) != nullptr ? (uint8*)GetMemoryByType(Context, ERigVMMemoryType::Debug)->GetContainerPtr() : nullptr,
	};
	check(PreCachedMemoryHandles.Num() == MemoryHandleCount);
	check(PreCachedMemoryHandlesMemoryType.Num() == MemoryHandleCount);
	
#if WITH_EDITOR
	TArray<FName>& FunctionNames = GetFunctionNames();
	FRigVMInstructionVisitInfo* RigVMInstructionVisitInfo = Context.GetRigVMInstructionVisitInfo();
	FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo();
#endif

	bool bMovingToNextInstructionSucceeded = true;
	
	while (Instructions.IsValidIndex(ContextPublicData.InstructionIndex))
	{
#if WITH_EDITOR
		if(Context.CurrentExecuteResult == ERigVMExecuteResult::Halted)
		{
			return Context.CurrentExecuteResult;
		}
#endif
		
		if(ContextPublicData.InstructionIndex > InLastInstruction)
		{
			return Context.CurrentExecuteResult = ERigVMExecuteResult::Succeeded;
		}

#if WITH_EDITOR

		const int32 CurrentInstructionIndex = ContextPublicData.InstructionIndex;

		if (RigVMInstructionVisitInfo != nullptr)
		{
			RigVMInstructionVisitInfo->SetInstructionVisitedDuringLastRun(ContextPublicData.InstructionIndex);
			RigVMInstructionVisitInfo->AddInstructionIndexToVisitOrder(ContextPublicData.InstructionIndex);
		}
	
#endif

		const FRigVMInstruction& Instruction = Instructions[ContextPublicData.InstructionIndex];

#if WITH_EDITOR
#if UE_RIGVM_DEBUG_EXECUTION
		if (CVarControlRigDebugAllVMExecutions->GetBool() || ContextPublicData.bDebugExecution)
		{
			if (Instruction.OpCode == ERigVMOpCode::Execute)
			{
				const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instruction);
				FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instructions[ContextPublicData.InstructionIndex]);

				TArray<FString> Labels;
				for (const FRigVMOperand& Operand : Operands)
				{
					Labels.Add(GetOperandLabel(Context, Operand));
				}

				ContextPublicData.DebugMemoryString += FString::Printf(TEXT("Instruction %d: %s(%s)\n"), ContextPublicData.InstructionIndex, *FunctionNames[Op.FunctionIndex].ToString(), *RigVMStringUtils::JoinStrings(Labels, TEXT(", ")));
			}
			else if(Instruction.OpCode == ERigVMOpCode::Copy)
			{
				static auto FormatFunction = [](const FString& RegisterName, const FString& RegisterOffsetName) -> FString
				{
					return FString::Printf(TEXT("%s%s"), *RegisterName, *RegisterOffsetName);
				};
				const FRigVMCopyOp& Op = ByteCode.GetOpAt<FRigVMCopyOp>(Instruction);
				ContextPublicData.DebugMemoryString += FString::Printf(TEXT("Instruction %d: Copy %s -> %s\n"), ContextPublicData.InstructionIndex, *GetOperandLabel(Context, Op.Source, FormatFunction), *GetOperandLabel(Context, Op.Target, FormatFunction));
			}
			else
			{
				ContextPublicData.DebugMemoryString += FString::Printf(TEXT("Instruction %d: %s\n"), ContextPublicData.InstructionIndex, *ContextPublicData.InstanceOpCodeEnum->GetNameByIndex((uint8)Instruction.OpCode).ToString());
			}
		}
#endif
#endif

		switch (Instruction.OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instruction);
				const int32 OperandCount = FirstHandleForInstruction[ContextPublicData.InstructionIndex + 1] - FirstHandleForInstruction[ContextPublicData.InstructionIndex];

				TArrayView<FRigVMMemoryHandle> Handles;

				if(OperandCount > 0)
				{
					HandlesMemory.SetNumUninitialized(OperandCount, EAllowShrinking::No);
					Handles = TArrayView<FRigVMMemoryHandle>(HandlesMemory.GetData(), OperandCount);
					GeneratePatchedHandles(Handles, Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandles, PreCachedMemoryHandlesMemoryType, FirstHandleForInstruction[ContextPublicData.InstructionIndex], OperandCount);
				}

				FRigVMPredicateBranchArray Predicates;
				if (Op.PredicateCount > 0)
				{
					Predicates = FRigVMPredicateBranchArray(&ByteCode.PredicateBranches[Op.FirstPredicateIndex], Op.PredicateCount);
				}
#if WITH_EDITOR
				ContextPublicData.FunctionName = FunctionNames[Op.CallableIndex];
#endif
				Context.Factory = Factories[Op.CallableIndex];
				const FRigVMFunction* Function = Functions[Op.CallableIndex];
				if (Function && Function->FunctionPtr)
				{
					(*Function->FunctionPtr)(Context, Handles, Predicates);
				}
				else
				{
					// Slow path: build diagnostic strings only when we actually need them.
					const FString FunctionNameStr = GetFunctionNames().IsValidIndex(Op.CallableIndex)
						? GetFunctionNames()[Op.CallableIndex].ToString() : TEXT("<oob>");
					const FString FactoryNameStr = (Function && Function->Factory)
						? Function->Factory->GetFactoryName().ToString() : TEXT("<unknown>");
					const UObject* OuterObj = GetOuter();
					const FString OuterClassPathStr = (OuterObj && OuterObj->GetClass())
						? OuterObj->GetClass()->GetPathName() : TEXT("<no outer>");
					// Skip the instruction (don't dispatch through a null pointer) but continue VM execution
					// so the rest of the rig evaluates normally. UE_LOG fires in Shipping where ensureMsgf is compiled out.
					const FString Message = FString::Printf(
						TEXT("Null FunctionPtr - skipping instruction. FunctionIndex=%d, FunctionName='%s', Factory='%s', VM='%s', OwningClass='%s'."),
						Op.CallableIndex,
						*FunctionNameStr,
						*FactoryNameStr,
						*GetPathName(),
						*OuterClassPathStr);
					static bool bLogged = false;
					if (!bLogged)
					{
						UE_LOG(LogRigVM, Error, TEXT("%s"), *Message);
						bLogged = true;
					}
					ensureMsgf(false, TEXT("%s"), *Message);
				}

#if WITH_EDITOR
				if(GetDebugMemory(Context) && GetDebugMemory(Context)->Num() > 0)
				{
					const FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instruction);
					for(int32 OperandIndex = 0, HandleIndex = 0; OperandIndex < Operands.Num() && HandleIndex < Handles.Num(); HandleIndex++)
					{
						CopyOperandForDebuggingIfNeeded(Context, Operands[OperandIndex++], Handles[HandleIndex]);
					}
				}
#endif

				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instruction);
				const FRigVMCallableInfo* Callable = ByteCode.GetCallable(Op.CallableIndex);
				check(Callable);

				// it's possible to have empty callables, so just step over this 
				if (Callable->GetNumInstructions() == 0)
				{
					bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
					break;
				}
					
				const int32 OperandCount = FirstHandleForInstruction[ContextPublicData.InstructionIndex + 1] - FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const int32 ArgumentCount = Callable->Arguments.Num();
				check(ArgumentCount == OperandCount);

				if(OperandCount > 0)
				{
					// generate patches handles for the call site memory
					HandlesMemory.SetNumUninitialized(ArgumentCount, EAllowShrinking::No);
					TArrayView<FRigVMMemoryHandle> Handles = TArrayView<FRigVMMemoryHandle>(HandlesMemory.GetData(), ArgumentCount);
					GeneratePatchedHandles(Handles, Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandles, PreCachedMemoryHandlesMemoryType, FirstHandleForInstruction[ContextPublicData.InstructionIndex], ArgumentCount);

					// bend the content of the callable's interface operand memory to point to this callsite
					for (int32 ArgumentIndex = 0; ArgumentIndex < ArgumentCount; ArgumentIndex++)
					{
						const FRigVMOperand& ForwardedOperand = Callable->Arguments[ArgumentIndex].ForwardedOperand;
#if WITH_EDITOR
						check(ForwardedOperand.IsValid() && ForwardedOperand.GetMemoryType() == ERigVMMemoryType::Work);
#endif
						FRigVMMemoryStorageStruct* PropertyBag = GetMemoryByType(Context, ForwardedOperand.GetMemoryType());
#if WITH_EDITOR
						const FProperty* ArgumentProperty = PropertyBag->GetProperty(ForwardedOperand.GetRegisterIndex());
						check(ArgumentProperty);
						check(CastFieldChecked<FStructProperty>(ArgumentProperty)->Struct == FRigVMForwardedMemoryHandle::StaticStruct());
#endif
						FRigVMForwardedMemoryHandle* ForwardHandle = PropertyBag->GetData<FRigVMForwardedMemoryHandle>(ForwardedOperand.GetRegisterIndex());
						ForwardHandle->Memory = Handles[ArgumentIndex].GetData_Internal(true, INDEX_NONE);
						ForwardHandle->Property = Handles[ArgumentIndex].GetResolvedProperty();
#if WITH_EDITOR
						const FRigVMOperand& InterfaceOperand = Callable->Arguments[ArgumentIndex].InterfaceOperand;
						check(InterfaceOperand.IsValid() && InterfaceOperand.GetMemoryType() == ERigVMMemoryType::Work);
						const FProperty* InterfaceProperty = PropertyBag->GetProperty(InterfaceOperand.GetRegisterIndex());
						check(InterfaceProperty);
						check(InterfaceProperty->SameType(ForwardHandle->Property));
#endif
					}
				}

				// invoke the range of the callable
				ERigVMExecuteResult CallableResult = ERigVMExecuteResult::Failed; 
				{
					const FRigVMCallstackGuard CallstackGuard(Context, ByteCode, Callable->Index);
					
					CallableResult = ExecuteInstructions(Context, Callable->FirstInstruction, Callable->LastInstruction);

					// reset the forwarded handles
					for (int32 ArgumentIndex = 0; ArgumentIndex < ArgumentCount; ArgumentIndex++)
					{
						const FRigVMOperand& ForwardedOperand = Callable->Arguments[ArgumentIndex].ForwardedOperand;
						FRigVMMemoryStorageStruct* PropertyBag = GetMemoryByType(Context, ForwardedOperand.GetMemoryType());
						FRigVMForwardedMemoryHandle* ForwardHandle = PropertyBag->GetData<FRigVMForwardedMemoryHandle>(ForwardedOperand.GetRegisterIndex());
						ForwardHandle->Reset();
					}
				}

				if (CallableResult != ERigVMExecuteResult::Succeeded)
				{
					return CallableResult;
				}

				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Zero:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				if(Handle.GetProperty()->IsA<FIntProperty>())
				{
					*((int32*)Handle.GetInputData()) = 0;
				}
				else if(Handle.GetProperty()->IsA<FNameProperty>())
				{
					*((FName*)Handle.GetInputData()) = NAME_None;
				}
#if WITH_EDITOR
				if(GetDebugMemory(Context) && GetDebugMemory(Context)->Num() > 0)
				{
					const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instruction);
					CopyOperandForDebuggingIfNeeded(Context, Op.Arg, Handle);
				}
#endif

				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::BoolFalse:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				*((bool*)Handle.GetInputData()) = false;
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::BoolTrue:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				*((bool*)Handle.GetInputData()) = true;
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Copy:
			{
				const FRigVMCopyOp& Op = ByteCode.GetOpAt<FRigVMCopyOp>(Instruction);

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				FRigVMMemoryHandle SourceHandle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);
				FRigVMMemoryHandle TargetHandle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex + 1], PreCachedMemoryHandles[FirstHandleIndex + 1]);

				FRigVMMemoryStorageStruct::CopyProperty(TargetHandle, SourceHandle);
					
#if WITH_EDITOR
				if(GetDebugMemory(Context) && GetDebugMemory(Context)->Num() > 0)
				{
					CopyOperandForDebuggingIfNeeded(Context, Op.Source, SourceHandle);
				}
#endif
					
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Increment:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				(*((int32*)Handle.GetInputData()))++;
#if WITH_EDITOR
				if(GetDebugMemory(Context) && GetDebugMemory(Context)->Num() > 0)
				{
					const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instruction);
					CopyOperandForDebuggingIfNeeded(Context, Op.Arg, Handle);
				}
#endif
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Decrement:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				(*((int32*)Handle.GetInputData()))--;
#if WITH_EDITOR
				if(GetDebugMemory(Context) && GetDebugMemory(Context)->Num() > 0)
				{
					const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instruction);
					CopyOperandForDebuggingIfNeeded(Context, Op.Arg, Handle);
				}
#endif
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Equals:
			case ERigVMOpCode::NotEquals:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle HandleA = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);
				const FRigVMMemoryHandle HandleB = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex + 1], PreCachedMemoryHandles[FirstHandleIndex + 1]);
				FRigVMMemoryHandle HandleR = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex + 2], PreCachedMemoryHandles[FirstHandleIndex + 2]);

				const bool Identical = HandleA.GetProperty()->Identical(HandleA.GetInputData(), HandleB.GetInputData());;
				const bool Result = Instruction.OpCode == ERigVMOpCode::Equals ? Identical : !Identical;

				*((bool*)HandleR.GetOutputData()) = Result;
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::JumpAbsolute:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				break;
			}
			case ERigVMOpCode::JumpForward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				break;
			}
			case ERigVMOpCode::JumpBackward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				const bool Condition = *(bool*)Handle.GetInputData();
				if (Condition == Op.Condition)
				{
					bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				}
				else
				{
					bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				}
				break;
			}
			case ERigVMOpCode::JumpForwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				const bool Condition = *(bool*)Handle.GetInputData();
				if (Condition == Op.Condition)
				{
					bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				}
				else
				{
					bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				}
				break;
			}
			case ERigVMOpCode::JumpBackwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				const bool Condition = *(bool*)Handle.GetInputData();
				if (Condition == Op.Condition)
				{
					bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, FRigVMJumpOp::GetTargetInstruction(Instruction.OpCode, ContextPublicData.InstructionIndex, Op.InstructionIndex));
				}
				else
				{
					bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				}
				break;
			}
			case ERigVMOpCode::ChangeType:
			{
				ensureMsgf(false, TEXT("not implemented."));
				break;
			}
			case ERigVMOpCode::Exit:
			{
				if(Context.bCurrentlyRunningRootEntry)
				{
					StopProfiling(Context);

					if (Context.OnExecutionReachedExitCallback)
					{
						Context.OnExecutionReachedExitCallback(Context.CurrentEntryName);
					}

#if WITH_EDITOR					
#if UE_RIGVM_DEBUG_EXECUTION
					if (CVarControlRigDebugAllVMExecutions->GetBool() || ContextPublicData.bDebugExecution)
					{
						ContextPublicData.Log(EMessageSeverity::Info, ContextPublicData.DebugMemoryString);
					}
#endif
#endif
				}
				return ERigVMExecuteResult::Succeeded;
			}
			case ERigVMOpCode::BeginBlock:
			{
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle CountHandle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);
				const FRigVMMemoryHandle IndexHandle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex + 1], PreCachedMemoryHandles[FirstHandleIndex + 1]);

				const int32 Count = (*((int32*)CountHandle.GetInputData()));
				const int32 Index = (*((int32*)IndexHandle.GetInputData()));
				Context.BeginLoopSlice(Count, Index);
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::EndBlock:
			{
				Context.EndLoopSlice();
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::InvokeEntry:
			{
				const FRigVMInvokeEntryOp& Op = ByteCode.GetOpAt<FRigVMInvokeEntryOp>(Instruction);

				if(!CanExecuteEntry(Context, Op.EntryName))
				{
					return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
				}
				else
				{
					// this will restore the public data after invoking the entry
					TGuardValue<FRigVMExecuteContext> PublicDataGuard(ContextPublicData, ContextPublicData);
					const ERigVMExecuteResult ExecuteResult = ExecuteVM(Context, Op.EntryName);
					if(ExecuteResult != ERigVMExecuteResult::Succeeded)
					{
						return Context.CurrentExecuteResult = ExecuteResult;
					}
				}
					
				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				const FRigVMJumpToBranchOp& Op = ByteCode.GetOpAt<FRigVMJumpToBranchOp>(Instruction);
				
				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				const FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				// BranchLabel = Op.Arg
				FName BranchLabel = *(FName*)Handle.GetInputData();

				// iterate over the branches stored in the bytecode,
				// starting at the first branch index stored in the operator.
				// look over all branches matching this instruction index and
				// find the one with the right label - then jump to the branch.
				bool bBranchFound = false;
				const TArray<FRigVMBranchInfo>& Branches = ByteCode.BranchInfos;
				if(Branches.IsEmpty())
				{
					UE_LOGF(LogRigVM, Error, "No branches in ByteCode - but JumpToBranch instruction %d found. Likely a corrupt VM. Exiting.", ContextPublicData.InstructionIndex);
					return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
				}

				for(int32 PassIndex = 0; PassIndex < 2; PassIndex++)
				{
					for(int32 BranchIndex = Op.FirstBranchInfoIndex // start at the first branch known to this jump op
						; BranchIndex < Branches.Num(); BranchIndex++)
					{
						const FRigVMBranchInfo& Branch = Branches[BranchIndex];
						if(Branch.InstructionIndex != ContextPublicData.InstructionIndex)
						{
							break;
						}
						if(Branch.Label == BranchLabel)
						{
							bMovingToNextInstructionSucceeded = SetInstructionIndex(Context, ContextPublicData, Branch.FirstInstruction);
							bBranchFound = true;
							break;
						}
					}

					// if we don't find the branch - try to jump to the completed branch
					if (!bBranchFound)
					{
						if(PassIndex == 0 && BranchLabel != FRigVMStruct::ControlFlowCompletedName)
						{
							UE_LOGF(LogRigVM, Warning, "Branch '%ls' was not found for instruction %d.", *BranchLabel.ToString(), ContextPublicData.InstructionIndex);
							BranchLabel = FRigVMStruct::ControlFlowCompletedName;
							continue;
						}
						
						UE_LOGF(LogRigVM, Error, "Branch '%ls' was not found for instruction %d.", *BranchLabel.ToString(), ContextPublicData.InstructionIndex);
						return ERigVMExecuteResult::Failed;
					}
					break;
				}
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instruction);

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				FRigVMInstructionSetExecuteState& ExecuteState = *(FRigVMInstructionSetExecuteState*)Handle.GetPrivateData(INDEX_NONE);

				if((Op.StartInstruction != INDEX_NONE) &&
					(Op.EndInstruction != INDEX_NONE) &&
					(Op.EndInstruction >= Op.StartInstruction))
				{
					if (ExecuteState.RequiresExecute(Context.GetSliceHash(), Context.GetHashForLazyBranch()))
					{
						ExecuteInstructions(Context, Op.StartInstruction, Op.EndInstruction);
					}
				}

				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				ContextPublicData.Traits.Reset();
				ContextPublicData.AdditionalTraitMemoryHandles.Reset();

				const int32 FirstHandleIndex = FirstHandleForInstruction[ContextPublicData.InstructionIndex];
				FRigVMMemoryHandle Handle = GeneratePatchedHandle(Context.ExternalVariableRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[FirstHandleIndex], PreCachedMemoryHandles[FirstHandleIndex]);

				const TArray<int32>& TraitList = *(TArray<int32>*)Handle.GetInputData();
				ContextPublicData.Traits.Reserve(TraitList.Num());
				ContextPublicData.AdditionalTraitMemoryHandles.Reserve(TraitList.Num());
				int32 AdditionalStartIndex = INDEX_NONE;
				int32 AdditionalNum = 0;
				for(const int32 TraitPropertyIndex : TraitList)
				{
					// Properties from programmatic pins that are added to the trait index list are added before their respective trait.
					// AdditionalMemoryHandles is accumulated prior to building the FRigVMTraitScope, and each FRigVMTraitScope has a view into
					// the overall ContextPublicData.AdditionalTraitMemoryHandles buffer.
					if(Context.WorkMemoryStorage.GetProperties().IsValidIndex(TraitPropertyIndex))
					{
						const FProperty* Property = Context.WorkMemoryStorage.GetProperties()[TraitPropertyIndex];
						const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
						if(StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FRigVMTrait::StaticStruct()))
						{
							TConstArrayView<FRigVMMemoryHandle> AdditionalMemoryHandles;
							if(AdditionalStartIndex != INDEX_NONE)
							{
								AdditionalMemoryHandles = TConstArrayView<FRigVMMemoryHandle>(&ContextPublicData.AdditionalTraitMemoryHandles[AdditionalStartIndex], AdditionalNum);
								AdditionalStartIndex = INDEX_NONE;
							}

							ContextPublicData.Traits.Emplace(
								Context.WorkMemoryStorage.GetData<FRigVMTrait>(TraitPropertyIndex),
								Cast<UScriptStruct>(StructProperty->Struct),
								AdditionalMemoryHandles);
						}
						else
						{
							if(AdditionalStartIndex == INDEX_NONE)
							{
								AdditionalStartIndex = ContextPublicData.AdditionalTraitMemoryHandles.Num();
								AdditionalNum = 0;
							}
							ContextPublicData.AdditionalTraitMemoryHandles.Emplace(Context.WorkMemoryStorage.GetHandle(TraitPropertyIndex));
							AdditionalNum++;
						}
					}
				}

				bMovingToNextInstructionSucceeded = IncrementInstructionIndex(Context, ContextPublicData);
				break;
			}
			case ERigVMOpCode::Invalid:
			{
				ensure(false);
				return Context.CurrentExecuteResult = ERigVMExecuteResult::Failed;
			}
		}

#if WITH_EDITOR
		if ((RigVMInstructionVisitInfo != nullptr) && (RigVMProfilingInfo != nullptr))
		{
			if (!RigVMInstructionVisitInfo->GetInstructionVisitOrder().IsEmpty())
			{
				const uint64 EndCycles = FPlatformTime::Cycles64();
				const uint64 Cycles = EndCycles - RigVMProfilingInfo->GetStartCycles();
				if (RigVMProfilingInfo->GetInstructionCyclesDuringLastRun(CurrentInstructionIndex) == UINT64_MAX)
				{
					RigVMProfilingInfo->SetInstructionCyclesDuringLastRun(CurrentInstructionIndex, Cycles);
				}
				else
				{
					RigVMProfilingInfo->AddInstructionCyclesDuringLastRun(CurrentInstructionIndex, Cycles);
				}

				if (Instruction.OpCode == ERigVMOpCode::InvokeCallable)
				{
					const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instruction);
					const FRigVMCallableInfo* Callable = ByteCode.GetCallable(Op.CallableIndex);
					check(Callable);

					if (RigVMProfilingInfo->GetCallableCyclesDuringLastRun(Callable->Index) == UINT64_MAX)
					{
						RigVMProfilingInfo->SetCallableCyclesDuringLastRun(Callable->Index, Cycles);
					}
					else
					{
						RigVMProfilingInfo->AddCallableCyclesDuringLastRun(Callable->Index, Cycles);
					}
				}

				RigVMProfilingInfo->SetStartCycles(EndCycles);
				RigVMProfilingInfo->AddOverallCycles(Cycles);
			}
		}

		if (!bMovingToNextInstructionSucceeded)
		{
			return ERigVMExecuteResult::Halted;
		}
		
#if UE_RIGVM_DEBUG_EXECUTION
		if (CVarControlRigDebugAllVMExecutions->GetBool() || ContextPublicData.bDebugExecution)
		{
			TArray<FString> CurrentWorkMemory;
			FRigVMMemoryStorageStruct* WorkMemory = GetWorkMemory(Context);
			int32 LineIndex = 0;
			for (int32 PropertyIndex=0; PropertyIndex<WorkMemory->Num(); ++PropertyIndex, ++LineIndex)
			{
				FString Line = FString::Printf(TEXT("%s: %s"), *WorkMemory->GetProperties()[PropertyIndex]->GetFullName(), *WorkMemory->GetDataAsString(PropertyIndex));
				if (ContextPublicData.PreviousWorkMemory.Num() > 0 && ContextPublicData.PreviousWorkMemory.IsValidIndex(PropertyIndex) && ContextPublicData.PreviousWorkMemory[PropertyIndex].StartsWith(TEXT(" -- ")))
				{
					ContextPublicData.PreviousWorkMemory[PropertyIndex].RightChopInline(4);
				}
				if (ContextPublicData.PreviousWorkMemory.Num() == 0 || (ContextPublicData.PreviousWorkMemory.IsValidIndex(PropertyIndex) && Line == ContextPublicData.PreviousWorkMemory[PropertyIndex]))
				{
					CurrentWorkMemory.Add(Line);
				}
				else
				{
					CurrentWorkMemory.Add(FString::Printf(TEXT(" -- %s"), *Line));
				}
			}
			for (const FRigVMExternalVariable& ExternalVariable : GetExternalVariables(Context))
			{
				if (ExternalVariable.Memory == nullptr)
				{
					continue;
				}
				
				FString Value;
				ExternalVariable.Property->ExportTextItem_Direct(Value, ExternalVariable.Memory, nullptr, nullptr, PPF_None);
				FString Line = FString::Printf(TEXT("External %s: %s"), *ExternalVariable.Name.ToString(), *Value);
				if (ContextPublicData.PreviousWorkMemory.Num() > 0 && ContextPublicData.PreviousWorkMemory.IsValidIndex(LineIndex) && ContextPublicData.PreviousWorkMemory[LineIndex].StartsWith(TEXT(" -- ")))
				{
					ContextPublicData.PreviousWorkMemory[LineIndex].RightChopInline(4);
				}
				if (ContextPublicData.PreviousWorkMemory.Num() == 0 || (ContextPublicData.PreviousWorkMemory.IsValidIndex(LineIndex) && Line == ContextPublicData.PreviousWorkMemory[LineIndex]))
				{
					CurrentWorkMemory.Add(Line);
				}
				else
				{
					CurrentWorkMemory.Add(FString::Printf(TEXT(" -- %s"), *Line));
				}
				++LineIndex;
			}
			ContextPublicData.DebugMemoryString += RigVMStringUtils::JoinStrings(CurrentWorkMemory, TEXT("\n")) + FString(TEXT("\n\n"));
			ContextPublicData.PreviousWorkMemory = CurrentWorkMemory;
		}
#endif
#endif
	}

	return Context.CurrentExecuteResult = ERigVMExecuteResult::Succeeded;
}

ERigVMExecuteResult URigVM::ExecuteBranch(FRigVMExtendedExecuteContext& Context, const FRigVMBranchInfo& InBranchToRun)
{
#if WITH_EDITOR
	const double LastExecutionMicroSecondsGuard = Context.GetRigVMProfilingInfo() ? Context.GetRigVMProfilingInfo()->GetLastExecutionMicroSeconds() : 0.0;
	ON_SCOPE_EXIT
	{
		if (Context.GetRigVMProfilingInfo())
		{
			Context.GetRigVMProfilingInfo()->SetLastExecutionMicroSeconds(LastExecutionMicroSecondsGuard);
		}
	};
#endif

	// Maintain all settings on the context - to be reset once the branch has executed. 
	FRigVMExecuteContext& PublicContext = Context.GetPublicData<>();
	TGuardValue<uint32> NumExecutionsGuard(PublicContext.NumExecutions, PublicContext.NumExecutions);
	TGuardValue<ERigVMExecuteResult> CurrentExecuteResultGuard(Context.CurrentExecuteResult, Context.CurrentExecuteResult);
	TGuardValue<FName> CurrentEntryNameGuard(Context.CurrentEntryName, Context.CurrentEntryName);
	TGuardValue<bool> bCurrentlyRunningRootEntryGuard(Context.bCurrentlyRunningRootEntry, Context.bCurrentlyRunningRootEntry);
	TGuardValue<FName> EventNameGuard(PublicContext.EventName, PublicContext.EventName);
	TGuardValue<FName> FunctionNameGuard(PublicContext.FunctionName, PublicContext.FunctionName);
	TGuardValue<int32> InstructionIndexGuard(PublicContext.InstructionIndex, PublicContext.InstructionIndex);
	TGuardValue<double> DeltaTimeGuard(PublicContext.DeltaTime, PublicContext.DeltaTime);
	TGuardValue<double> AbsoluteTimeGuard(PublicContext.AbsoluteTime, PublicContext.AbsoluteTime);
	TGuardValue<double> FramesPerSecondGuard(PublicContext.FramesPerSecond, PublicContext.FramesPerSecond);

	return ExecuteInstructions(Context, InBranchToRun.FirstInstruction, InBranchToRun.LastInstruction);
}

void URigVM::ClearExternalVariables(FRigVMExtendedExecuteContext& Context)
{
	ExternalVariables.Reset();
	Context.ExternalVariableRuntimeData.Reset();
}

FRigVMExternalVariableDef URigVM::GetExternalVariableDefByName(const FName& InExternalVariableName)
{
	for (const FRigVMExternalVariableDef& ExternalVariable : ExternalVariables)
	{
		if (ExternalVariable.GetName() == InExternalVariableName)
		{
			return ExternalVariable;
		}
	}
	return FRigVMExternalVariableDef();
}

FRigVMExternalVariable URigVM::GetExternalVariableByName(const FRigVMExtendedExecuteContext& Context, const FName& InExternalVariableName)
{
	const int32 NumExternalVariables = ExternalVariables.Num();
	for (int i=0; i<NumExternalVariables; ++i)
	{
		const FRigVMExternalVariableDef& ExternalVariableDef = ExternalVariables[i];
		if (ExternalVariableDef.GetName() == InExternalVariableName)
		{
			const FRigVMExternalVariableRuntimeData& ExternalVariableData = Context.ExternalVariableRuntimeData[i];
			return FRigVMExternalVariable(ExternalVariableDef, ExternalVariableData.Memory);;
		}
	}
	return FRigVMExternalVariable();
}

void URigVM::SetExternalVariableDefs(const TArray<FRigVMExternalVariable>& InExternalVariables)
{
	ExternalVariables.Reset(InExternalVariables.Num());
	for (const FRigVMExternalVariableDef& ExternalVariable : InExternalVariables)
	{
		ExternalVariables.Add(ExternalVariable);
	}

	RefreshExternalPropertyPaths();
}

void URigVM::SetPropertyValueFromString(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InOperand, const FString& InDefaultValue)
{
	FRigVMMemoryStorageStruct* Memory = GetMemoryByType(Context, InOperand.GetMemoryType());
	if(Memory == nullptr)
	{
		return;
	}

	Memory->SetDataFromString(InOperand.GetRegisterIndex(), InDefaultValue);
}

FRigVMRegistry_NoLock* URigVM::GetLocalizedRegistry() const
{
	UE::TScopeLock CreateLocalizedRegistryScopeLock(CreateLocalizedRegistryMutex);
	
	if(LocalizedRegistry.IsValid())
	{
		return LocalizedRegistry.Get();
	}
	return nullptr;
}

bool URigVM::CreateLocalizedRegistryIfRequired(bool bForce)
{
	UE::TScopeLock CreateLocalizedRegistryScopeLock(CreateLocalizedRegistryMutex);
	return CreateLocalizedRegistryIfRequired_NoLock(bForce);
}

bool URigVM::CreateLocalizedRegistryIfRequired_NoLock(bool bForce)
{
	if(LocalizedRegistry.IsValid())
	{
		return true;
	}

	if(!bForce && CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread() == false)
	{
		return false;
	}

	auto ResetLocalizedRegistry = [this]() -> bool
	{
		LocalizedRegistry.Reset();
		FunctionsStorage.Reset();
		FactoriesStorage.Reset();
		verify(ResolveFunctionsIfRequired());
		return false;
	};
		
	// create a new registry
	TSharedRef<FRigVMRegistry_NoLock> TemporaryRegistry = FRigVMRegistry_NoLock::CreateLocalizedRegistry();
	FRigVMRegistryHandle TemporaryHandle(&TemporaryRegistry.Get());

	RefreshInstructionsIfRequired();
	if(ExternalPropertyPaths.IsEmpty())
	{
		RefreshExternalPropertyPaths();
	}
	ResolveFunctionsIfRequired();

	const FRigVMByteCode& ByteCode = GetByteCode();

	// add all required types to it
	{
	FRigVMRegistryReadLock ReadLock;

	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		FRigVMOperandArray Operands = ByteCode.GetOperandsForOp(Instructions[InstructionIndex]);
		for(const FRigVMOperand& Operand : Operands)
		{
			const FProperty* Property = nullptr;
			switch(Operand.GetMemoryType())
			{
				case ERigVMMemoryType::Literal:
				{
					if(Operand.GetRegisterOffset() != INDEX_NONE)
					{
						if(!LiteralMemoryStorage.GetPropertyPaths().IsValidIndex(Operand.GetRegisterOffset()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve literal memory property path '%d'.", *GetPathName(), Operand.GetRegisterOffset());
							return ResetLocalizedRegistry();
						}
						Property = LiteralMemoryStorage.GetPropertyPaths()[Operand.GetRegisterOffset()].GetTailProperty();
					}
					else
					{
						if(!LiteralMemoryStorage.IsValidIndex(Operand.GetRegisterIndex()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve literal memory property '%d'.", *GetPathName(), Operand.GetRegisterIndex());
							return ResetLocalizedRegistry();
						}
						Property = LiteralMemoryStorage.GetProperty(Operand.GetRegisterIndex());
					}
					break;
				}
				case ERigVMMemoryType::Work:
				{
					if(Operand.GetRegisterOffset() != INDEX_NONE)
					{
						if(!DefaultWorkMemoryStorage.GetPropertyPaths().IsValidIndex(Operand.GetRegisterOffset()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve work memory property path '%d'.", *GetPathName(), Operand.GetRegisterOffset());
							return ResetLocalizedRegistry();
						}
						Property = DefaultWorkMemoryStorage.GetPropertyPaths()[Operand.GetRegisterOffset()].GetTailProperty();
					}
					else
					{
						if(!DefaultWorkMemoryStorage.IsValidIndex(Operand.GetRegisterIndex()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve work memory property '%d'.", *GetPathName(), Operand.GetRegisterIndex());
							return ResetLocalizedRegistry();
						}
						Property = DefaultWorkMemoryStorage.GetProperty(Operand.GetRegisterIndex());
					}
					break;
				}
				case ERigVMMemoryType::External:
				{
					if(Operand.GetRegisterOffset() != INDEX_NONE)
					{
						if(!ExternalPropertyPaths.IsValidIndex(Operand.GetRegisterOffset()))
						{
							RefreshExternalPropertyPaths();
						}
						if(!ExternalPropertyPaths.IsValidIndex(Operand.GetRegisterOffset()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve external variable property path '%d'.", *GetPathName(), Operand.GetRegisterOffset());
							return ResetLocalizedRegistry();
						}
						Property = ExternalPropertyPaths[Operand.GetRegisterOffset()].GetTailProperty();
					}
					else
					{
						if(!ExternalVariables.IsValidIndex(Operand.GetRegisterIndex()))
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve external variable '%d'.", *GetPathName(), Operand.GetRegisterIndex());
							return ResetLocalizedRegistry();
						}
						Property = ExternalVariables[Operand.GetRegisterIndex()].GetProperty();
					}
					break;
				}
				case ERigVMMemoryType::Debug:
				default:
				{
					break;
				}
			}

			if(Property == nullptr)
			{
				continue;
			}

			// set up the types - but try to avoid loading types relying on StaticFindObjectFast
			// since this may be happening during GC or serialization
			const FName CPPTypeName = *RigVMTypeUtils::GetCPPTypeFromProperty(Property);
			UObject* CPPTypeObject = RigVMTypeUtils::GetCPPTypeObjectFromProperty(Property);

			if (CPPTypeObject && ReadLock->IsSpecificallyAllowedType_NoLock(CPPTypeObject))
			{
				TemporaryHandle->AddSpecificallyAllowedType_NoLock(CPPTypeObject);
			}
			TemporaryHandle->FindOrAddType_NoLock(FRigVMTemplateArgumentType(CPPTypeName, CPPTypeObject, bIsSerializing), true /* force */);
		}
	}

	// for dispatch functions we need to register all of the argument types for the given permutations
	// otherwise we won't be able to retrieve the permutation from string again later.
	{
		for(const FRigVMFunction* Function : FunctionsStorage)
		{
			if(Function && Function->Factory)
			{
				TArray<TRigVMTypeIndex> ArgumentTypeIndices;
				Function->GetArgumentTypeIndices_NoLock(ReadLock, ArgumentTypeIndices);
			
				for(const TRigVMTypeIndex& ArgumentTypeIndex : ArgumentTypeIndices)
				{
					if (ArgumentTypeIndex != INDEX_NONE)
					{
						const FRigVMTemplateArgumentType& Type = ReadLock->GetType_NoLock(ArgumentTypeIndex);
						if (Type.IsValid())
						{
							if (Type.CPPTypeObject && ReadLock->IsSpecificallyAllowedType_NoLock(Type.CPPTypeObject))
							{
								TemporaryHandle->AddSpecificallyAllowedType_NoLock(Type.CPPTypeObject);
							}
							TemporaryRegistry->FindOrAddType_NoLock(Type, true /* force */);
						}
						else
						{
							UE_LOGF(LogRigVM, Error, "%ls: Cannot resolve Argument Type %d for function '%ls'.", *GetPathName(), (int32)ArgumentTypeIndex, *Function->Name);
							return ResetLocalizedRegistry();
						}
					}
				}
			}
		}
	}

	// register all of the factories
	TMap<const FRigVMDispatchFactory*, TArray<const FRigVMFunction*>> RequiredFactories;
	for(const FRigVMFunction* Function : FunctionsStorage)
	{
		if(Function == nullptr)
		{
			continue;
		}
		if(Function->Factory)
		{
			RequiredFactories.FindOrAdd(Function->Factory).Add(Function);
		}
	}
	for(const TPair<const FRigVMDispatchFactory*, TArray<const FRigVMFunction*>>& Pair : RequiredFactories)
	{
		const FRigVMDispatchFactory* Factory = Pair.Key;
		if (Factory == nullptr)
		{
			UE_LOGF(LogRigVM, Error, "Found an unexpected nullptr for required dispatch factories in VM '%ls'.", *GetPathName());
			return ResetLocalizedRegistry();
		}

		UScriptStruct* FactoryStruct = Factory->GetScriptStruct();
		if (FactoryStruct == nullptr)
		{
			UE_LOGF(LogRigVM, Error, "Dispatch factory '%ls' has an unexpected nullptr for its script struct in VM '%ls'.", *Factory->GetFactoryName().ToString(), *GetPathName());
			return ResetLocalizedRegistry();
		}

		const TArray<const FRigVMFunction*>& OriginalFunctionsForFactory = Pair.Value; 

		// flatten the argument infos to the permutations used only
		TArray<FRigVMTemplateArgumentInfo> TemporaryArgumentInfos;
		{
			const TArray<FRigVMTemplateArgumentInfo> FlattenedArgumentInfos = Factory->GetTemplate_NoLock(ReadLock)->GetFlattenedArgumentInfosForFunctions_NoLock(OriginalFunctionsForFactory, ReadLock);
			TemporaryArgumentInfos.Reserve(FlattenedArgumentInfos.Num());

			for(const FRigVMTemplateArgumentInfo& ArgumentInfo : FlattenedArgumentInfos)
			{
				const TArray<TRigVMTypeIndex>& OriginalTypeIndices = ArgumentInfo.GetArgument_NoLock(ReadLock).TypeIndices;
				TArray<TRigVMTypeIndex> TemporaryTypeIndices;
				TemporaryTypeIndices.Reserve(OriginalTypeIndices.Num());
				
				for(int32 TypeIndex = 0; TypeIndex < OriginalTypeIndices.Num(); TypeIndex++)
				{
					const FRigVMTemplateArgumentType OriginalType = ReadLock.GetRegistry().GetType_NoLock(OriginalTypeIndices[TypeIndex]);
					if (!OriginalType.IsValid())
					{
						UE_LOGF(LogRigVM, Error, "Cannot register dispatch dispatch '%ls' for VM '%ls'. Invalid types found.\nCannot localize registry.", *Factory->GetFactoryName().ToString(), *GetPathName());
						return ResetLocalizedRegistry();
					}

					if (OriginalType.CPPTypeObject && (OriginalType.CPPTypeObject->IsA<UClass>() || ReadLock->IsSpecificallyAllowedType_NoLock(OriginalType.CPPTypeObject)))
					{
						TemporaryRegistry->AddSpecificallyAllowedType_NoLock(OriginalType.CPPTypeObject);
					}
					TemporaryTypeIndices.Add(TemporaryRegistry->FindOrAddType_NoLock(OriginalType, true /* force */));
					if (TemporaryTypeIndices.Last() == INDEX_NONE)
					{
						UE_LOGF(LogRigVM, Error, "Cannot register type '%ls' in dispatch '%ls' for VM '%ls'\nCannot localize registry.", *OriginalType.CPPType.ToString(), *Factory->GetFactoryName().ToString(), *GetPathName());
						return ResetLocalizedRegistry();
					}

					const FRigVMTemplateArgumentType TemporaryType = TemporaryRegistry->GetType_NoLock(TemporaryTypeIndices.Last());
					if(TemporaryType != OriginalType)
					{
						UE_LOGF(LogRigVM, Error, "Registered type '%ls' in dispatch '%ls' does not match ('%ls') for VM '%ls'\nCannot localize registry.", *OriginalType.CPPType.ToString(), *Factory->GetFactoryName().ToString(), *TemporaryType.CPPType.ToString(), *GetPathName());
						return ResetLocalizedRegistry();
					}
				}
				TemporaryArgumentInfos.Add(FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(ArgumentInfo.Name, ArgumentInfo.Direction, TemporaryTypeIndices));
			}
		}

		// register the factory
		FRigVMDispatchFactory* TemporaryFactory = const_cast<FRigVMDispatchFactory*>(TemporaryRegistry->RegisterFactory_NoLock(FactoryStruct, TemporaryArgumentInfos));

		// update all argument name caches,
		// force create all function pointers
		if(FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(TemporaryFactory->GetTemplate_NoLock(TemporaryHandle)))
		{
			(void)TemporaryFactory->UpdateArgumentNameCache_NoLock(Template->NumArguments(), TemporaryHandle);

			TArray<const FRigVMFunction*> TemporaryFunctionsForFactory;
			TemporaryFunctionsForFactory.Reserve(OriginalFunctionsForFactory.Num());
			
			for (int32 PermutationIndex = 0; PermutationIndex < Template->NumPermutations_NoLock(TemporaryHandle); PermutationIndex++)
			{
				const FRigVMFunction* TemporaryPermutation = Template->GetOrCreatePermutation_NoLock(PermutationIndex, TemporaryHandle);
				if (!TemporaryPermutation)
				{
					FRigVMTemplateTypeMap Types = Template->GetTypesForPermutation_NoLock(PermutationIndex, TemporaryHandle);
					const FString TypesString = Template->GetStringFromArgumentTypes(Types, TemporaryHandle);
					UE_LOGF(LogRigVM, Error, "Cannot create permutation [%03d] of template '%ls' types '%ls' for VM '%ls'\nCannot localize registry.", PermutationIndex, *Template->GetNotation().ToString(), *TypesString, *GetPathName());
					return ResetLocalizedRegistry();
				}

				TemporaryFunctionsForFactory.Add(TemporaryPermutation);
			}

			for (const FRigVMFunction* OriginalFunction : OriginalFunctionsForFactory)
			{
				int32 TemporaryFunctionIndex = TemporaryFunctionsForFactory.IndexOfByPredicate([OriginalFunction](const FRigVMFunction* TemporaryFunction)
				{
					return OriginalFunction->Name == TemporaryFunction->Name;
				});

				if (TemporaryFunctionIndex == INDEX_NONE)
				{
					for (int32 OriginalFunctionIndex = 0; OriginalFunctionIndex < OriginalFunctionsForFactory.Num(); OriginalFunctionIndex++)
					{
						UE_LOGF(LogRigVM, Error, "Original function [%03d] '%ls' of template '%ls' for VM '%ls'.", OriginalFunctionIndex, *OriginalFunctionsForFactory[OriginalFunctionIndex]->Name, *Template->GetNotation().ToString(), *GetPathName());
					}
					for (TemporaryFunctionIndex = 0; TemporaryFunctionIndex < TemporaryFunctionsForFactory.Num(); TemporaryFunctionIndex++)
					{
						UE_LOGF(LogRigVM, Error, "Temporary function [%03d] '%ls' of template '%ls' for VM '%ls'.", TemporaryFunctionIndex, *TemporaryFunctionsForFactory[TemporaryFunctionIndex]->Name, *Template->GetNotation().ToString(), *GetPathName());
					}
					UE_LOGF(LogRigVM, Error, "Missing expected permutation '%ls' of template '%ls' for VM '%ls'.\nCannot localize registry.", *OriginalFunction->Name, *Template->GetNotation().ToString(), *GetPathName());
					return ResetLocalizedRegistry();
				}
			}
		}
	}
	}

	// register all functions
	for(const FRigVMFunction* Function : FunctionsStorage)
	{
		if(Function == nullptr)
		{
			continue;
		}
		
		if(Function->Factory)
		{
			// pulling on the function makes sure the permutation exists
			if (!TemporaryRegistry->FindFunction_NoLock(*Function->Name))
			{
				for (int32 FactoryIndex = 0; FactoryIndex < TemporaryRegistry->GetFactories_NoLock().Num(); FactoryIndex++)
				{
					const FRigVMDispatchFactory* AvailableFactory = TemporaryRegistry->GetFactories_NoLock()[FactoryIndex];
					UE_LOGF(LogRigVM, Error, "Available factory [%03d] '%ls' for VM '%ls'", FactoryIndex, *AvailableFactory->GetFactoryName().ToString(), *GetPathName());
				}
				for (const FRigVMFunction& AvailableFunction : TemporaryRegistry->GetFunctions_NoLock())
				{
					UE_LOGF(LogRigVM, Error, "Available function [%03d] '%ls' for VM '%ls'", AvailableFunction.Index, *AvailableFunction.Name, *GetPathName());
				}

				UE_LOGF(LogRigVM, Error, "Cannot find expected permutation '%ls' for VM '%ls'\nCannot localize registry.", *Function->Name, *GetPathName());
				return ResetLocalizedRegistry();
			}
		}
		else
		{
			// registering the function will also register it's potential template
			TemporaryRegistry->Register_NoLock(*Function->Name, Function->FunctionPtr, Function->Struct, Function->Arguments);
		}
	}

	// finally relate this VM to the localized registry
	LocalizedRegistry = TemporaryRegistry.ToSharedPtr();

	// now that we've registered all functions - let's re-resolve the functions
	FunctionsStorage.Reset();
	FactoriesStorage.Reset();

	// if any of the functions fail to resolve we'll revert back to the global registry
	if(!ResolveFunctionsIfRequired())
	{
		UE_LOGF(LogRigVM, Error, "Cannot localize registry due to missing function.");
		return ResetLocalizedRegistry();
	}

	return true;
}

FString URigVM::GetLocalizedRegistryAsString() const
{
	if (!LocalizedRegistry.IsValid())
	{
		return FString();
	}

	FRigVMObjectArchive Archive;
	{
		FRigVMObjectArchiveWriter Writer(Archive, nullptr);
		LocalizedRegistry->Serialize_NoLock(Writer, this);
	}
	Archive.Compress();
	return Archive.ToString();
}

void URigVM::SetLocalizedRegistryFromString(const FString& InData)
{
	LocalizedRegistry.Reset();
	
	if (InData.IsEmpty())
	{
		return;
	}
	
	FRigVMObjectArchive Archive;
	if (!Archive.SetFromString(InData))
	{
		return;
	}

	if (!Archive.Decompress())
	{
		return;
	}

	LocalizedRegistry = FRigVMRegistry_NoLock::CreateLocalizedRegistry();
	{
		FRigVMObjectArchiveReader Reader(Archive, nullptr);
		LocalizedRegistry->Serialize_NoLock(Reader, this);
	}
}

#if WITH_EDITOR

TArray<FString> URigVM::DumpByteCodeAsTextArray(FRigVMExtendedExecuteContext& Context, const TArray<int32>& InInstructionOrder, bool bIncludeLineNumbers, TFunction<FString(const FString& RegisterName, const FString& RegisterOffsetName)> OperandFormatFunction)
{
	RefreshInstructionsIfRequired();
	const FRigVMByteCode& ByteCode = GetByteCode();
	const TArray<FName>& FunctionNames = GetFunctionNames();

	TArray<int32> InstructionOrder;
	InstructionOrder.Append(InInstructionOrder);
	if (InstructionOrder.Num() == 0)
	{
		for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
		{
			InstructionOrder.Add(InstructionIndex);
		}
	}

	TArray<FString> Result;

	for (int32 InstructionIndex : InstructionOrder)
	{
		FString ResultLine;

		switch (Instructions[InstructionIndex].OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instructions[InstructionIndex]);
				FString FunctionName = FunctionNames[Op.CallableIndex].ToString();
				FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instructions[InstructionIndex]);

				TArray<FString> Labels;
				for (const FRigVMOperand& Operand : Operands)
				{
					Labels.Add(GetOperandLabel(Context, Operand, OperandFormatFunction));
				}

				ResultLine = FString::Printf(TEXT("%s(%s)"), *FunctionName, *RigVMStringUtils::JoinStrings(Labels, TEXT(",")));
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instructions[InstructionIndex]);
				const FRigVMCallableInfo* CallableInfo = ByteCode.GetCallable(Op.CallableIndex);
				check(CallableInfo);
				const FString CallableName = CallableInfo->Name.ToString();
				FRigVMOperandArray Operands = ByteCode.GetOperandsForCallableOp(Instructions[InstructionIndex]);

				TArray<FString> Labels;
				for (const FRigVMOperand& Operand : Operands)
				{
					Labels.Add(GetOperandLabel(Context, Operand, OperandFormatFunction));
				}

				ResultLine = FString::Printf(TEXT("%s(%s)"), *CallableName, *RigVMStringUtils::JoinStrings(Labels, TEXT(",")));
				break;
			}
			case ERigVMOpCode::Zero:
			{
				const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Set %s to 0"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::BoolFalse:
			{
				const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Set %s to False"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::BoolTrue:
			{
				const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Set %s to True"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::Increment:
			{
				const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Inc %s ++"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::Decrement:
			{
				const FRigVMUnaryOp& Op = ByteCode.GetOpAt<FRigVMUnaryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Dec %s --"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::Copy:
			{
				const FRigVMCopyOp& Op = ByteCode.GetOpAt<FRigVMCopyOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Copy %s to %s"), *GetOperandLabel(Context, Op.Source, OperandFormatFunction), *GetOperandLabel(Context, Op.Target, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::Equals:
			{
				const FRigVMComparisonOp& Op = ByteCode.GetOpAt<FRigVMComparisonOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Set %s to %s == %s "), *GetOperandLabel(Context, Op.Result, OperandFormatFunction), *GetOperandLabel(Context, Op.A, OperandFormatFunction), *GetOperandLabel(Context, Op.B, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::NotEquals:
			{
				const FRigVMComparisonOp& Op = ByteCode.GetOpAt<FRigVMComparisonOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Set %s to %s != %s"), *GetOperandLabel(Context, Op.Result, OperandFormatFunction), *GetOperandLabel(Context, Op.A, OperandFormatFunction), *GetOperandLabel(Context, Op.B, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::JumpAbsolute:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Jump to instruction %d"), Op.InstructionIndex);
				break;
			}
			case ERigVMOpCode::JumpForward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Jump %d instructions forwards"), Op.InstructionIndex);
				break;
			}
			case ERigVMOpCode::JumpBackward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Jump %d instructions backwards"), Op.InstructionIndex);
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instructions[InstructionIndex]);
				if (Op.Condition)
				{
					ResultLine = FString::Printf(TEXT("Jump to instruction %d if %s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				else
				{
					ResultLine = FString::Printf(TEXT("Jump to instruction %d if !%s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				break;
			}
			case ERigVMOpCode::JumpForwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instructions[InstructionIndex]);
				if (Op.Condition)
				{
					ResultLine = FString::Printf(TEXT("Jump %d instructions forwards if %s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				else
				{
					ResultLine = FString::Printf(TEXT("Jump %d instructions forwards if !%s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				break;
			}
			case ERigVMOpCode::JumpBackwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instructions[InstructionIndex]);
				if (Op.Condition)
				{
					ResultLine = FString::Printf(TEXT("Jump %d instructions backwards if %s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				else
				{
					ResultLine = FString::Printf(TEXT("Jump %d instructions backwards if !%s"), Op.InstructionIndex, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				}
				break;
			}
			case ERigVMOpCode::ChangeType:
			{
				const FRigVMChangeTypeOp& Op = ByteCode.GetOpAt<FRigVMChangeTypeOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Change type of %s"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::Exit:
			{
				ResultLine = TEXT("Exit");
				break;
			}
			case ERigVMOpCode::BeginBlock:
			{
				ResultLine = TEXT("Begin Block");
				break;
			}
			case ERigVMOpCode::EndBlock:
			{
				ResultLine = TEXT("End Block");
				break;
			}
			case ERigVMOpCode::InvokeEntry:
			{
				const FRigVMInvokeEntryOp& Op = ByteCode.GetOpAt<FRigVMInvokeEntryOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Invoke entry %s"), *Op.EntryName.ToString());
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				const FRigVMJumpToBranchOp& Op = ByteCode.GetOpAt<FRigVMJumpToBranchOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Jump To Branch %s"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Run Instructions %d-%d (%s)"), Op.StartInstruction, Op.EndInstruction, *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				const FRigVMSetupTraitsOp& Op = ByteCode.GetOpAt<FRigVMSetupTraitsOp>(Instructions[InstructionIndex]);
				ResultLine = FString::Printf(TEXT("Setup Traits (%s)"), *GetOperandLabel(Context, Op.Arg, OperandFormatFunction));
				break;
			}
			default:
			{
				ensure(false);
				break;
			}
		}

		if (bIncludeLineNumbers)
		{
			FString ResultIndexStr = FString::FromInt(InstructionIndex);
			while (ResultIndexStr.Len() < 3)
			{
				ResultIndexStr = TEXT("0") + ResultIndexStr;
			}
			Result.Add(FString::Printf(TEXT("%s. %s"), *ResultIndexStr, *ResultLine));
		}
		else
		{
			Result.Add(ResultLine);
		}
	}

	return Result;
}

FString URigVM::DumpByteCodeAsText(FRigVMExtendedExecuteContext& Context, const TArray<int32>& InInstructionOrder, bool bIncludeLineNumbers)
{
	RefreshExternalPropertyPaths();
	return RigVMStringUtils::JoinStrings(DumpByteCodeAsTextArray(Context, InInstructionOrder, bIncludeLineNumbers), TEXT("\n"));
}

FString URigVM::GetOperandLabel(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InOperand, TFunction<FString(const FString& RegisterName, const FString& RegisterOffsetName)> FormatFunction)
{
	FString RegisterName;
	FString RegisterOffsetName;
	if (InOperand.GetMemoryType() == ERigVMMemoryType::External)
	{
		const FRigVMExternalVariableDef& ExternalVariable = ExternalVariables[InOperand.GetRegisterIndex()];
		RegisterName = FString::Printf(TEXT("Variable::%s"), *ExternalVariable.GetName().ToString());
		if (InOperand.GetRegisterOffset() != INDEX_NONE)
		{
			if(ensure(ExternalPropertyPaths.IsValidIndex(InOperand.GetRegisterOffset())))
			{
				RegisterOffsetName = ExternalPropertyPaths[InOperand.GetRegisterOffset()].ToString();
			}
		}
	}
	else
	{
		FRigVMMemoryStorageStruct* Memory = GetMemoryByType(Context, InOperand.GetMemoryType());
		if(Memory == nullptr)
		{
			return FString();
		}

		if(!Memory->IsValidIndex(InOperand.GetRegisterIndex()))
		{
			if(Memory->Num() == 0)
			{
				Memory = GetDefaultMemoryByType(InOperand.GetMemoryType());
			}
		}

		if(!Memory->IsValidIndex(InOperand.GetRegisterIndex()))
		{
			static const UEnum* MemoryTypeEnum = StaticEnum<ERigVMMemoryType>();
			static constexpr TCHAR Format[] = TEXT("%s_%d");
			RegisterName = FString::Printf(Format, *MemoryTypeEnum->GetDisplayNameTextByValue((int64)InOperand.GetMemoryType()).ToString(), InOperand.GetRegisterIndex());
		}
		else
		{
			RegisterName = Memory->GetProperties()[InOperand.GetRegisterIndex()]->GetName();
			RegisterOffsetName =
				InOperand.GetRegisterOffset() != INDEX_NONE ?
				Memory->GetPropertyPaths()[InOperand.GetRegisterOffset()].ToString() :
				FString();
		}
	}
	
	FString OperandLabel = RegisterName;
	
	// caller can provide an alternative format to override the default format(optional)
	if (FormatFunction)
	{
		OperandLabel = FormatFunction(RegisterName, RegisterOffsetName);
	}

	return OperandLabel;
}

#endif

void URigVM::ClearDebugMemory(FRigVMExtendedExecuteContext& Context)
{
#if WITH_EDITOR
	if (GetDebugMemory(Context))
	{
		for (int32 PropertyIndex = 0; PropertyIndex < GetDebugMemory(Context)->Num(); PropertyIndex++)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(GetDebugMemory(Context)->GetProperties()[PropertyIndex]))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, GetDebugMemory(Context)->GetData<uint8>(PropertyIndex));
				ArrayHelper.EmptyValues();
			}
		}
	}
#endif
}

void URigVM::PreCacheSingleMemoryHandle(const FRigVMRegistryHandle& InRegistry, int32 InInstructionIndex, int32 InHandleIndex, const FRigVMBranchInfoKey& InBranchInfoKey, const FRigVMOperand& InArg, bool bForExecute)
{
	FRigVMMemoryStorageStruct* Memory = GetDefaultMemoryByType(InArg.GetMemoryType());

	if (InArg.GetMemoryType() == ERigVMMemoryType::External)
	{
		const FRigVMPropertyPath* PropertyPath = nullptr;
		if (InArg.GetRegisterOffset() != INDEX_NONE)
		{
			check(ExternalPropertyPaths.IsValidIndex(InArg.GetRegisterOffset()));
			PropertyPath = &ExternalPropertyPaths[InArg.GetRegisterOffset()];
		}

		check(ExternalVariables.IsValidIndex(InArg.GetRegisterIndex()));

		const int32 ExternalVariableIndex = InArg.GetRegisterIndex();
		FRigVMExternalVariableDef& ExternalVariable = ExternalVariables[ExternalVariableIndex];
		check(ExternalVariable.IsValid());

		// External variables store the ExternalVariableIndex as the memory offset, as it is needed to generate the final memory address
		PreCachedMemoryHandles[InHandleIndex] = { (uint8*)((PTRINT)ExternalVariableIndex), ExternalVariable.GetProperty(), PropertyPath };
		PreCachedMemoryHandlesMemoryType[InHandleIndex] = ERigVMMemoryType::External;
		return;
	}

	const FRigVMPropertyPath* PropertyPath = nullptr;
	if (InArg.GetRegisterOffset() != INDEX_NONE)
	{
		check(Memory->GetPropertyPaths().IsValidIndex(InArg.GetRegisterOffset()));
		PropertyPath = &Memory->GetPropertyPaths()[InArg.GetRegisterOffset()];
		if (PropertyPath->IsEmpty())
		{
			UE_LOGF(LogRigVM, Error, "VM '%ls' uses an invalid property path.", *GetPathName());
			PropertyPath = nullptr;
		}
	}

	// if you are hitting this it's likely that the VM was created outside of a valid
	// package. the compiler bases the memory class construction on the package the VM
	// is in - so a VM under GetTransientPackage() can be created - but not run.
	uint8* Data = Memory->GetData<uint8>(InArg.GetRegisterIndex());
	PTRINT DataOffset = ((uint8*)Data - (uint8*)Memory->GetContainerPtr());
	const FProperty* Property = Memory->GetProperties()[InArg.GetRegisterIndex()];
	FRigVMMemoryHandle& Handle = PreCachedMemoryHandles[InHandleIndex];

	Handle = { (uint8*)DataOffset, Property, PropertyPath };
	PreCachedMemoryHandlesMemoryType[InHandleIndex] = InArg.GetMemoryType();

	// if we are lazy executing update the handle to point to a lazy branch
	if (InBranchInfoKey.IsValid())
	{
		if (const FRigVMBranchInfo* BranchInfo = GetByteCode().GetBranchInfo(InBranchInfoKey))
		{
			FRigVMLazyBranch* LazyBranch = &LazyBranches[BranchInfo->Index];
			LazyBranch->VM = this;
			LazyBranch->BranchInfo = *BranchInfo;
			Handle.LazyBranch = LazyBranch;
		}
	}

#if UE_BUILD_DEBUG
	// make sure the handle points to valid memory
	if (PropertyPath)
	{
		if (!Handle.bIsForwarded)
		{
			uint8* MemoryPtr = (uint8*)Memory->GetContainerPtr() + (PTRINT)Handle.GetOutputData();
			// don't check the result - since it may be an array element
			// that doesn't exist yet. 
			PropertyPath->GetData<uint8>(MemoryPtr, Property);
		}
	}
#endif
}

void URigVM::GeneratePatchedHandles(
	const TArrayView<FRigVMMemoryHandle>& OutHandles,
	const TArray<FRigVMExternalVariableRuntimeData>& ExternalVariablesRuntimeData,
	const TStaticArray<uint8*, (int32)ERigVMMemoryType::Num>& VMMemoryPtrs,
	const TArray<FRigVMMemoryHandle>& PreCachedMemoryHandles,
	const TArray<ERigVMMemoryType>& PreCachedMemoryHandlesMemoryType,
	int32 FirstHandleIndex,
	int32 OperandCount)
{
	check(OutHandles.Num() == OperandCount);
	for (int32 HandleIndex = 0; HandleIndex < OperandCount; HandleIndex++)
	{
		const int32 PreCachedHandleIndex = FirstHandleIndex + HandleIndex;
		const FRigVMMemoryHandle& PreCachedHandle = PreCachedMemoryHandles[PreCachedHandleIndex];
		
		OutHandles[HandleIndex] = GeneratePatchedHandle(ExternalVariablesRuntimeData, VMMemoryPtrs, PreCachedMemoryHandlesMemoryType[PreCachedHandleIndex], PreCachedHandle);
	}
}

FRigVMMemoryHandle URigVM::GeneratePatchedHandle(
	const TArray<FRigVMExternalVariableRuntimeData>& ExternalVariablesRuntimeData,
	const TStaticArray<uint8*, (int32)ERigVMMemoryType::Num>& VMMemoryPtrs,
	ERigVMMemoryType HandleMemoryType,
	const FRigVMMemoryHandle& PreCachedHandle)
{
	FRigVMMemoryHandle Handle = PreCachedHandle;

	if (HandleMemoryType == ERigVMMemoryType::External)
	{
		const int32 ExternalVariableIndex = (int32)((PTRINT)Handle.Ptr); // For external variables we store the RegisterIndex in the memory pointer, so we can retrieve the correct var

		check(ExternalVariablesRuntimeData.IsValidIndex(ExternalVariableIndex));
		const FRigVMExternalVariableRuntimeData& ExternalVariableRuntimeData = ExternalVariablesRuntimeData[ExternalVariableIndex];
		check(ExternalVariableRuntimeData.Memory != nullptr);

		Handle.Ptr = ExternalVariableRuntimeData.Memory;
		return Handle;
	}

	if (const uint8* Memory = VMMemoryPtrs[(int32)HandleMemoryType])
	{
		const PTRINT BaseAddress = (PTRINT)Memory;
		const PTRINT Offset = (PTRINT)Handle.Ptr;
		Handle.Ptr = (uint8*)(BaseAddress + Offset);
	}

	return Handle;
}

void URigVM::CopyOperandForDebuggingImpl(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InArg, const FRigVMMemoryHandle& InHandle, const FRigVMOperand& InDebugOperand)
{
#if WITH_EDITOR

	FRigVMMemoryStorageStruct* TargetMemory = GetDebugMemory(Context);
	if(TargetMemory == nullptr)
	{
		return;
	}
	const FProperty* TargetProperty = TargetMemory->GetProperties()[InDebugOperand.GetRegisterIndex()];
	uint8* TargetPtr = TargetMemory->GetData<uint8>(InDebugOperand.GetRegisterIndex());

	// since debug properties are always arrays, we need to divert to the last array element's memory
	const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(TargetProperty);
	if(TargetArrayProperty == nullptr)
	{
		return;
	}

	// add an element to the end for debug watching
	FScriptArrayHelper ArrayHelper(TargetArrayProperty, TargetPtr);

	if (Context.GetSlice().GetIndex() == 0)
	{
		ArrayHelper.Resize(0);
	}
	else if(Context.GetSlice().GetIndex() == ArrayHelper.Num() - 1)
	{
		return;
	}

	const int32 AddedIndex = ArrayHelper.AddValue();
	TargetPtr = ArrayHelper.GetRawPtr(AddedIndex);
	TargetProperty = TargetArrayProperty->Inner;

	if(InArg.GetMemoryType() == ERigVMMemoryType::External)
	{
		if(ExternalVariables.IsValidIndex(InArg.GetRegisterIndex()))
		{
			const int32 ExternalVariableIndex = InArg.GetRegisterIndex();
			FRigVMExternalVariableDef& ExternalVariable = ExternalVariables[ExternalVariableIndex];
			const FProperty* SourceProperty = ExternalVariable.GetProperty();
			FRigVMExternalVariableRuntimeData& ExternalVariableRuntimeData = Context.ExternalVariableRuntimeData[ExternalVariableIndex];
			const uint8* SourcePtr = ExternalVariableRuntimeData.Memory;
			FRigVMMemoryStorageStruct::CopyProperty(TargetProperty, TargetPtr, SourceProperty, SourcePtr);
		}
		return;
	}

	FRigVMMemoryStorageStruct* SourceMemory = GetMemoryByType(Context, InArg.GetMemoryType());
	if(SourceMemory == nullptr)
	{
		return;
	}
	const FProperty* SourceProperty = SourceMemory->GetProperties()[InArg.GetRegisterIndex()];
	const uint8* SourcePtr = SourceMemory->GetData<uint8>(InArg.GetRegisterIndex());

	FRigVMMemoryStorageStruct::CopyProperty(TargetProperty, TargetPtr, SourceProperty, SourcePtr);
	
#endif
}

FRigVMCopyOp URigVM::GetCopyOpForOperands(const FRigVMOperand& InSource, const FRigVMOperand& InTarget)
{
	return FRigVMCopyOp(InSource, InTarget);
}

void URigVM::RefreshExternalPropertyPaths()
{
	ExternalPropertyPaths.Reset();

	ExternalPropertyPaths.SetNumZeroed(ExternalPropertyPathDescriptions.Num());
	for(int32 PropertyPathIndex = 0; PropertyPathIndex < ExternalPropertyPaths.Num(); PropertyPathIndex++)
	{
		ExternalPropertyPaths[PropertyPathIndex] = FRigVMPropertyPath();

		int32 PropertyIndex = ExternalPropertyPathDescriptions[PropertyPathIndex].PropertyIndex;
		if(ExternalVariables.IsValidIndex(PropertyIndex))
		{
			// it's possible for the external variables in a different order
			const FString& ExpectedHeadCPPType = ExternalPropertyPathDescriptions[PropertyPathIndex].HeadCPPType;
			if (ExternalVariables[PropertyIndex].GetExtendedCPPType() != ExpectedHeadCPPType)
			{
				PropertyIndex = ExternalVariables.IndexOfByPredicate([ExpectedHeadCPPType](const FRigVMExternalVariableDef& ExternalVariableDef) -> bool
				{
					return ExternalVariableDef.GetExtendedCPPType() == ExpectedHeadCPPType;	
				});
			}
		}
		if(ExternalVariables.IsValidIndex(PropertyIndex))
		{
			check(ExternalVariables[PropertyIndex].GetProperty());
			
			ExternalPropertyPaths[PropertyPathIndex] = FRigVMPropertyPath(
				ExternalVariables[PropertyIndex].GetProperty(),
				ExternalPropertyPathDescriptions[PropertyPathIndex].SegmentPath);
#if WITH_EDITOR
			if (!ExternalPropertyPaths[PropertyPathIndex].IsValid())
			{
#if UE_BUILD_DEBUG
				UE_LOGF(LogRigVM, Warning, "Unable to resolve property path '%ls' for property '%ls'.", *ExternalPropertyPathDescriptions[PropertyPathIndex].SegmentPath, *ExternalVariables[PropertyIndex].GetProperty()->GetPathName());
#endif
			}
#endif
		}
	}
}

TArray<const UObject*> URigVM::GetUserDefinedDependencies(const TArray<const FRigVMMemoryStorageStruct*> InMemory)
{
	TArray<const UObject*> Dependencies;
	for (const FRigVMMemoryStorageStruct* MemoryStorage : InMemory)
	{
		MemoryStorage->GetUserDefinedDependencies(Dependencies);
	}

	const TArray<const FRigVMFunction*>& Functions = GetFunctions();
	for (const FRigVMFunction* Function : Functions)
	{
		FRigVMRegistry_NoLock* Registry = GetLocalizedRegistry();

		// if the VM doesn't have a localized registry
		// then access the global one and lock it for reading.
		TUniquePtr<FRigVMRegistryReadLock> ReadLock;
		if(Registry == nullptr)
		{
			FRigVMRegistry_RWLock& RegistryRef = FRigVMRegistry_RWLock::Get();
			Registry = &RegistryRef;
			ReadLock = MakeUnique<FRigVMRegistryReadLock>(RegistryRef);
		}
		
		const TArray<TRigVMTypeIndex>& TypeIndices = Function->GetArgumentTypeIndices_NoLock(Registry->GetHandle_NoLock());
		for (const TRigVMTypeIndex& TypeIndex : TypeIndices)
		{
			const FRigVMTemplateArgumentType& Type = Registry->GetType_NoLock(TypeIndex);
			if (Cast<UUserDefinedStruct>(Type.CPPTypeObject) ||
				Cast<UUserDefinedEnum>(Type.CPPTypeObject))
			{
				Dependencies.AddUnique(Type.CPPTypeObject);
			}
		}
	}

	return Dependencies;
}

void URigVM::GetRequiredPlugins(TArray<FString>& OutPlugins)
{
	GetDefaultMemoryByType(ERigVMMemoryType::Literal)->GetRequiredPlugins(OutPlugins);
	GetDefaultMemoryByType(ERigVMMemoryType::Work)->GetRequiredPlugins(OutPlugins);

	ResolveFunctionsIfRequired();

#if WITH_EDITOR
	TUniquePtr<UE::TScopeLock<FTransactionallySafeCriticalSection>> ResolveFunctionsScopeLock;
	if(!LocalizedRegistry.IsValid())
	{
		ResolveFunctionsScopeLock = MakeUnique<UE::TScopeLock<FTransactionallySafeCriticalSection>>(ResolveFunctionsMutex);
	}
#endif

	for(const FRigVMFunction* Function : FunctionsStorage)
	{
		if (!Function)
		{
			continue;
		}
		
		FString PluginName;
		if (Function->Struct)
		{
			PluginName = RigVMTypeUtils::GetPluginName(Function->Struct);
		}
		else if (Function->Factory && Function->Factory->GetScriptStruct())
		{
			PluginName = RigVMTypeUtils::GetPluginName(Function->Factory->GetScriptStruct());
		}
		if (!PluginName.IsEmpty())
		{
			OutPlugins.AddUnique(PluginName);
		}
	}
}

void URigVM::SetupInstructionTracking(FRigVMExtendedExecuteContext& Context, int32 InInstructionCount, int32 InCallableCount, const FName& InEntryName)
{
#if WITH_EDITOR
	if (FRigVMInstructionVisitInfo* RigVMInstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
	{
		if (RigVMInstructionVisitInfo->GetFirstEntryEventInEventQueue() == NAME_None || RigVMInstructionVisitInfo->GetFirstEntryEventInEventQueue() == InEntryName)
		{
			RigVMInstructionVisitInfo->SetupInstructionTracking(InInstructionCount, InCallableCount);
		}
	}
	if (FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo())
	{
		RigVMProfilingInfo->SetupInstructionTracking(InInstructionCount, InCallableCount, true);
	}
#endif
}

void URigVM::StartProfiling(FRigVMExtendedExecuteContext& Context)
{
#if WITH_EDITOR
	if (FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo())
	{
		RigVMProfilingInfo->StartProfiling(true);
	}
#endif
	
	// if we are going to trace the VM for Rewind Debugger
	// we should set up the debug memory and prepare the host.
	TRACE_SETUP_RIGVM_EVALUATION(Context);
}

void URigVM::StopProfiling(FRigVMExtendedExecuteContext& Context)
{
#if WITH_EDITOR
	if (FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo())
	{
		RigVMProfilingInfo->StopProfiling();
	}
#endif
	TRACE_RIGVM_EVALUATION(Context);
}

#undef UE_RIGVM_GET_READ_REGISTRY_CONTEXT
