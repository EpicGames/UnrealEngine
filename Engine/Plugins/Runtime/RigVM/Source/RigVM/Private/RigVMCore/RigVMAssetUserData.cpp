// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMAssetUserData.h"
#include "RigVMTypeUtils.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMAssetUserData)

const TArray<const UNameSpacedUserData::FUserData*> UNameSpacedUserData::EmptyUserDatas;

FString UNameSpacedUserData::FUserData::GetName() const
{
	FString Left, Right;
	if(SplitAtEnd(Path, Left, Right))
	{
		return Right;
	}
	return Path;
}

#if WITH_EDITOR

FString UNameSpacedUserData::FUserData::GetDisplayName() const
{
	if(Property)
	{
		if(Property->GetTypedOwner<UUserDefinedStruct>())
		{
			return Property->GetDisplayNameText().ToString();
		}
	}
	return GetName();
}

bool UNameSpacedUserData::FUserData::IsArray() const
{
	return CastField<FArrayProperty>(Property) != nullptr;
}

bool UNameSpacedUserData::FUserData::IsArrayElement() const
{
	if(Property)
	{
		return CastField<FArrayProperty>(Property->GetOwnerProperty()) != nullptr;
	}
	return false;
}

bool UNameSpacedUserData::FUserData::IsUObject() const
{
	return CastField<FObjectProperty>(Property) != nullptr;
}

#endif

TArray<UStruct*> UNameSpacedUserData::FUserData::GetSuperStructs(UStruct* InStruct)
{
	// Create an array of structs, ordered super -> child struct
	TArray<UStruct*> SuperStructs = {InStruct};
	while(true)
	{
		if(UStruct* SuperStruct = SuperStructs[0]->GetSuperStruct())
		{
			SuperStructs.Insert(SuperStruct, 0);
		}
		else
		{
			break;
		}
	}
	return SuperStructs;
}

void UNameSpacedUserData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	InvalidateCache();
}

const UNameSpacedUserData::FUserData* UNameSpacedUserData::GetUserData(const FString& InPath, FString* OutErrorMessage) const
{
	UE::TScopeLock Lock(CacheLock);
	if(FUserData** UserDataInCache = CachedUserData.Find(InPath))
	{
		if(*UserDataInCache)
		{
			return *UserDataInCache;
		}
	}
	return nullptr;
}

const TArray<const UNameSpacedUserData::FUserData*>& UNameSpacedUserData::GetUserDataArray(const FString& InParentPath, FString* OutErrorMessage) const
{
	UE::TScopeLock Lock(CacheLock);
	if(const TArray<const FUserData*>* UserDataArrayInCache = CachedUserDataArray.Find(InParentPath))
	{
		return *UserDataArrayInCache;
	}
	return EmptyUserDatas;
}

const UNameSpacedUserData::FUserData* UNameSpacedUserData::GetUserDataWithinStruct(const UStruct* InStruct, const uint8* InMemory, const FString& InPath, const FString& InPropertyName, FString* OutErrorMessage) const
{
	check(InStruct);
	check(InMemory);

	FString Left, Right;
	if(!InPropertyName.IsEmpty() || !FUserData::SplitAtEnd(InPath, Left, Right))
	{
		const FString& PropertyName = InPropertyName.IsEmpty() ? InPath : InPropertyName; 
		if(const FProperty* Property = FindPropertyByName(InStruct, *PropertyName))
		{
			if(!IsPropertySupported(Property, InPath, true, OutErrorMessage))
			{
				return nullptr;
			}

			const uint8* PropertyMemory = Property->ContainerPtrToValuePtr<uint8>(InMemory);
			const FUserData UserData(InPath, Property, PropertyMemory);
			return StoreCacheForUserData(UserData);
		}
	}
	else
	{
		if(const FUserData* ParentUserData = GetUserData(Left, OutErrorMessage))
		{
			check(ParentUserData->GetProperty());

			const uint8* Memory = ParentUserData->GetMemory();
			if(Memory == nullptr)
			{
				if(OutErrorMessage && OutErrorMessage->IsEmpty())
				{
					(*OutErrorMessage) = FString::Printf(InvalidMemoryFormat, *InPath);
				}
				return nullptr;
			}
			
			if(const FStructProperty* StructProperty = CastField<FStructProperty>(ParentUserData->GetProperty()))
			{
				return GetUserDataWithinStruct(StructProperty->Struct, Memory, InPath, Right, OutErrorMessage);
			}

			if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentUserData->GetProperty()))
			{
				const FProperty* Property = ArrayProperty->Inner;
				if(!IsPropertySupported(Property, InPath, false, OutErrorMessage))
				{
					return nullptr;
				}

				// make sure the next bit is provided as a number and is within bounds
				if(!Right.IsNumeric())
				{
					if(OutErrorMessage && OutErrorMessage->IsEmpty())
					{
						(*OutErrorMessage) = FString::Printf(InvalidArrayIndexFormat, *Right, *InPath);
					}
					return nullptr;
				}

				FScriptArrayHelper ArrayHelper(ArrayProperty, Memory);

				const int32 Index = FCString::Atoi(*Right);
				if(!ArrayHelper.IsValidIndex(Index))
				{
					if(OutErrorMessage && OutErrorMessage->IsEmpty())
					{
						(*OutErrorMessage) = FString::Printf(OutOfBoundArrayIndexFormat, *InPath, ArrayHelper.Num());
					}
					return nullptr;
				}

				Memory = ArrayHelper.GetRawPtr(Index);
				const FUserData UserData(InPath, Property, Memory);
				return StoreCacheForUserData(UserData);
			}

			if(OutErrorMessage && OutErrorMessage->IsEmpty())
			{
				(*OutErrorMessage) = FString::Printf(UnSupportedSubPathsFormat, *ParentUserData->GetCPPType().ToString(), *Left);
			}
		}
	}

	if(OutErrorMessage && OutErrorMessage->IsEmpty())
	{
		(*OutErrorMessage) = FString::Printf(PathNotFoundFormat, *InPath);
	}
	return nullptr;
}

const TArray<const UNameSpacedUserData::FUserData*>& UNameSpacedUserData::GetUserDataArrayWithinStruct(UStruct* InStruct, const uint8* InMemory, const FString& InPath, FString* OutErrorMessage) const
{
	UE::TScopeLock Lock(CacheLock);

	TArray<const FUserData*> UserDataArray;

	const TArray<UStruct*> Structs = FUserData::GetSuperStructs(InStruct);
	for(const UStruct* Struct : Structs)
	{
		//const bool bIsUserDefinedStruct = Struct->IsA<UUserDefinedStruct>();
		for (TFieldIterator<FProperty> It(Struct, EFieldIterationFlags::None); It; ++It)
		{
			const FProperty* Property = *It;

			const FString& PropertyName = Property->GetNameCPP();
			const FString PropertyPath = InPath.IsEmpty() ? PropertyName : FUserData::Join(InPath, PropertyName);
					
			if(!IsPropertySupported(Property, PropertyPath, true, OutErrorMessage))
			{
				continue;
			}
					
			FUserData*& UserDataInCache = CachedUserData.FindOrAdd(PropertyPath);
			if(UserDataInCache == nullptr)
			{
				const uint8* PropertyMemory = Property->ContainerPtrToValuePtr<uint8>(InMemory);
				UserDataInCache = new FUserData(PropertyPath, Property, PropertyMemory);
			}

			UserDataArray.Add(UserDataInCache);
		}
	}

	if(!UserDataArray.IsEmpty())
	{
		TArray<const FUserData*>& UserDataArrayInCache = CachedUserDataArray.FindOrAdd(InPath);
		if(UserDataArrayInCache.IsEmpty())
		{
			UserDataArrayInCache = UserDataArray;
		}
		return UserDataArrayInCache;
	}

	return EmptyUserDatas;
}

const TArray<const UNameSpacedUserData::FUserData*>& UNameSpacedUserData::GetUserDataArrayWithinArray(const FArrayProperty* InArrayProperty, const uint8* InMemory, const FString& InPath, FString* OutErrorMessage) const
{
	check(!InPath.IsEmpty());
	
	UE::TScopeLock Lock(CacheLock);

	TArray<const FUserData*> UserDataArray;

	const FProperty* Property = InArrayProperty->Inner;
	if(!IsPropertySupported(Property, InPath, false, OutErrorMessage))
	{
		return EmptyUserDatas;
	}

	FScriptArrayHelper ArrayHelper(InArrayProperty, InMemory);
	for(int32 Index = 0; Index < ArrayHelper.Num(); Index++)
	{
		const FString PropertyPath = FUserData::Join(InPath, FString::FromInt(Index));

		FUserData*& UserDataInCache = CachedUserData.FindOrAdd(PropertyPath);
		if(UserDataInCache == nullptr)
		{
			const uint8* PropertyMemory = ArrayHelper.GetRawPtr(Index);
			UserDataInCache = new FUserData(PropertyPath, Property, PropertyMemory);
		}

		UserDataArray.Add(UserDataInCache);
	}

	if(!UserDataArray.IsEmpty())
	{
		TArray<const FUserData*>& UserDataArrayInCache = CachedUserDataArray.FindOrAdd(InPath);
		if(UserDataArrayInCache.IsEmpty())
		{
			UserDataArrayInCache = UserDataArray;
		}
		return UserDataArrayInCache;
	}

	return EmptyUserDatas;
}

const FProperty* UNameSpacedUserData::FindPropertyByName(const UStruct* InStruct, const FName& InName)
{
	check(InStruct);

	if(const FProperty* Property = InStruct->FindPropertyByName(InName))
	{
		return Property;
	}

	// user defined properties may actually be called quite differently
	return nullptr;
}

void UNameSpacedUserData::InvalidateCache() const
{
	UE::TScopeLock Lock(CacheLock);
	for(const TPair<FString, FUserData*>& Pair : CachedUserData)
	{
		delete(Pair.Value);
	}
	CachedUserData.Reset();
	CachedUserDataArray.Reset();
}

const UNameSpacedUserData::FUserData* UNameSpacedUserData::StoreCacheForUserData(const FUserData& InUserData) const
{
	check(InUserData.IsValid());
	UE::TScopeLock Lock(CacheLock);

	if(const FUserData* const* UserDataInCache = CachedUserData.Find(InUserData.GetPath()))
	{
		if(*UserDataInCache)
		{
			check((*UserDataInCache)->GetCPPType() == InUserData.GetCPPType());
			return *UserDataInCache;
		}
	}

	FUserData* UserDataPtr = new FUserData(InUserData);
	CachedUserData.Add(InUserData.GetPath(), UserDataPtr);
	return UserDataPtr;
}

const TArray<const UNameSpacedUserData::FUserData*>& UNameSpacedUserData::StoreCacheForUserDataArray(const FString& InPath, const TArray<const FUserData*>& InUserDataArray) const
{
	if(InUserDataArray.IsEmpty())
	{
		return EmptyUserDatas;
	}
	
	UE::TScopeLock Lock(CacheLock);

	TArray<const FUserData*>& UserDataArrayInCache = CachedUserDataArray.FindOrAdd(InPath);
	if(UserDataArrayInCache.IsEmpty())
	{
		UserDataArrayInCache = InUserDataArray;
	}
	return UserDataArrayInCache;
}

bool UNameSpacedUserData::IsPropertySupported(const FProperty* InProperty, const FString& InPath, bool bCheckPropertyFlags, FString* OutErrorMessage) const
{
	check(InProperty);

	if(bCheckPropertyFlags)
	{
		if(!InProperty->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst | CPF_BlueprintVisible))
		{
			return false;
		}
		if(InProperty->HasAnyPropertyFlags(CPF_EditorOnly))
		{
			return false;
		}
	}

	if(InProperty->IsA<FSetProperty>() ||
		InProperty->IsA<FMapProperty>())
	{
		if(OutErrorMessage && OutErrorMessage->IsEmpty())
		{
			const FString CPPTypeString = RigVMTypeUtils::GetCPPTypeFromProperty(InProperty);
			static constexpr TCHAR UnsupportedCPPTypeFormat[] = TEXT("Unsupported user data type '%s' at path '%s'.");
			(*OutErrorMessage) = FString::Printf(UnsupportedCPPTypeFormat, *CPPTypeString, *InPath);
		}
		return false;
	}

	return true;
}

void UDataAssetLink::SetDataAsset(TSoftObjectPtr<UDataAsset> InDataAsset)
{
	InvalidateCache();
	DataAsset = InDataAsset;
	DataAssetCached = InDataAsset.Get();

	if(NameSpace.IsEmpty() && DataAssetCached)
	{
		NameSpace = DataAsset->GetName();
	}
}

const UNameSpacedUserData::FUserData* UDataAssetLink::GetUserData(const FString& InPath, FString* OutErrorMessage) const
{
	// rely on super to look it up from the cache
	if(const FUserData* ResultFromSuper = Super::GetUserData(InPath))
	{
		return ResultFromSuper;
	}

	if(DataAssetCached)
	{
		// this method caches as well - so the next time around Super::GetUserData should return the cache 
		return GetUserDataWithinStruct(DataAssetCached->GetClass(), static_cast<const uint8*>(DataAssetCached), InPath, FString(), OutErrorMessage);
	}

	if(OutErrorMessage && OutErrorMessage->IsEmpty())
	{
		(*OutErrorMessage) = FString::Printf(DataAssetNullFormat, *InPath);
	}
	return nullptr;
}

const TArray<const UNameSpacedUserData::FUserData*>& UDataAssetLink::GetUserDataArray(const FString& InParentPath, FString* OutErrorMessage) const
{
	// rely on super to look it up from the cache
	const TArray<const FUserData*>& ResultFromSuper = Super::GetUserDataArray(InParentPath);
	if(!ResultFromSuper.IsEmpty())
	{
		return ResultFromSuper;
	}

	if(DataAssetCached)
	{
		// we should only get here if we haven't cached this user data array before.
		if(InParentPath.IsEmpty())
		{
			// this method caches as well - so the next time around Super::GetUserDataArray should return the cache 
			return GetUserDataArrayWithinStruct(DataAssetCached->GetClass(), static_cast<const uint8*>(DataAssetCached), InParentPath, OutErrorMessage);
		}

		if(const FUserData* ParentUserData = GetUserData(InParentPath, OutErrorMessage))
		{
			check(ParentUserData->GetProperty());

			const uint8* Memory = ParentUserData->GetMemory();
			if(Memory == nullptr)
			{
				return EmptyUserDatas;
			}

			if(const FStructProperty* StructProperty = CastField<FStructProperty>(ParentUserData->GetProperty()))
			{
				// this method caches as well - so the next time around Super::GetUserDataArray should return the cache 
				return GetUserDataArrayWithinStruct(StructProperty->Struct, Memory, InParentPath, OutErrorMessage);
			}
			
			if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentUserData->GetProperty()))
			{
				// this method caches as well - so the next time around Super::GetUserDataArray should return the cache 
				return GetUserDataArrayWithinArray(ArrayProperty, Memory, InParentPath, OutErrorMessage);
			}

			// other types of containers are not expected here due to UNameSpacedUserData::IsPropertySupported
		}
	}

	if(OutErrorMessage && OutErrorMessage->IsEmpty())
	{
		(*OutErrorMessage) = FString::Printf(DataAssetNullFormat, *InParentPath);
	}
	return EmptyUserDatas;
}

void UDataAssetLink::Serialize(FArchive& Ar)
{
	// Treat the cached ptr as transient unless we're cooking this out.
	const bool bIsSavingAssetToStorage = Ar.IsSaving() && Ar.IsPersistent() && !Ar.IsCooking(); 
	UDataAsset* SavedDataAsset = DataAssetCached; 
	if (bIsSavingAssetToStorage)
	{
		DataAssetCached = nullptr;
	}

	Super::Serialize(Ar);
	
	if (bIsSavingAssetToStorage)
	{
		DataAssetCached = SavedDataAsset;
	}
}

#if WITH_EDITOR

void UDataAssetLink::PostLoad()
{
	Super::PostLoad();

	DataAssetCached = DataAsset.Get();
	if(DataAssetCached == nullptr && !DataAsset.IsNull())
	{
		// We need to check if the mount point exists - since the data asset library link may
		// refer to an editor-only asset in a runtime game.
		const FString PackagePath = DataAsset.GetLongPackageName();
		const FName PluginMountPoint = FPackageName::GetPackageMountPoint(PackagePath, false);
		if (FPackageName::MountPointExists(PluginMountPoint.ToString()))
		{
			const FString ObjectPath = DataAsset.ToString();

			// load without throwing additional warnings / errors
			DataAssetCached = LoadObject<UDataAsset>(nullptr, *ObjectPath, nullptr, LOAD_Quiet | LOAD_NoWarn);
			SetDataAsset(DataAsset);
		}
	}
}


void UDataAssetLink::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FProperty* DataAssetProperty= FindPropertyByName(GetClass(), GET_MEMBER_NAME_CHECKED(UDataAssetLink, DataAsset));
	if(PropertyChangedEvent.Property == DataAssetProperty ||
		PropertyChangedEvent.MemberProperty == DataAssetProperty)
	{
		SetDataAsset(DataAsset);
	}
}

#endif
