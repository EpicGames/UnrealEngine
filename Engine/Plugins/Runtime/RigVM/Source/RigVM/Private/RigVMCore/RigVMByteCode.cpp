// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMByteCode.h"
#include "UObject/AnimObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMTrait.h"
#include "RigVMCore/RigVMTraitScope.h"
#include "RigVMObjectVersion.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMByteCode)

ERigVMExecuteResult FRigVMPredicateBranch::Execute(FRigVMExtendedExecuteContext& Context)
{
	check(VM);
	if(BranchInfo.IsValid())
	{
		return VM->ExecuteBranch(Context, BranchInfo);
	}
	return ERigVMExecuteResult::Failed;
}

void FRigVMCallableArgument::Serialize(FArchive& Ar)
{
	Ar << Name;
	Ar << TypeString;
	Ar << InterfaceOperand;
	Ar << ForwardedOperand;
	Ar << Direction;
}

int32 FRigVMCallableInfo::GetArgumentIndex(const FName& InName) const
{
	return Arguments.IndexOfByPredicate([InName](const FRigVMCallableArgument& Argument) -> bool
	{
		return Argument.Name == InName;
	});
}

const FRigVMCallableArgument* FRigVMCallableInfo::FindArgument(const FName& InName) const
{
	return Arguments.FindByPredicate([InName](const FRigVMCallableArgument& Argument) -> bool
	{
		return Argument.Name == InName;
	});
}

void FRigVMCallableInfo::Serialize(FArchive& Ar)
{
	Ar << Index;
	Ar << Name;
	Ar << FunctionHash;
	Ar << Arguments;
	Ar << FirstInstruction;
	Ar << LastInstruction;
}

void FRigVMInvokeCallableOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << CallableIndex;

	if(Ar.IsLoading())
	{
		// backwards compatibility for old opcodes
		if(OpCode >= ERigVMOpCode::Execute_0_Operands && OpCode <= ERigVMOpCode::Execute_64_Operands)
		{
			ArgumentCount = uint16(OpCode) - uint16(ERigVMOpCode::Execute_0_Operands);
			OpCode = ERigVMOpCode::Execute;
		}
		else
		{
			check(FRigVMByteCode::IsCallableOpCode(OpCode));
			Ar << ArgumentCount;
		}
	}
	else
	{
		Ar << ArgumentCount;
	}
}

void FRigVMExecuteOp::Serialize(FArchive& Ar)
{
	FRigVMInvokeCallableOp::Serialize(Ar);

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::PredicatesAddedToExecuteOps)
	{
		Ar << FirstPredicateIndex;
		Ar << PredicateCount;
	}
}

void FRigVMUnaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << Arg;
}

void FRigVMBinaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << ArgA;
	Ar << ArgB;
}

void FRigVMTernaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << ArgA;
	Ar << ArgB;
	Ar << ArgC;
}

void FRigVMQuaternaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << ArgA;
	Ar << ArgB;
	Ar << ArgC;
	Ar << ArgD;
}

void FRigVMQuinaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << ArgA;
	Ar << ArgB;
	Ar << ArgC;
	Ar << ArgD;
	Ar << ArgE;
}

void FRigVMSenaryOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << ArgA;
	Ar << ArgB;
	Ar << ArgC;
	Ar << ArgD;
	Ar << ArgE;
	Ar << ArgF;
}

void FRigVMCopyOp::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	
	Ar << OpCode;
	Ar << Source;
	Ar << Target;

	if(Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMCopyOpStoreNumBytes)
		{
			RegisterType = ERigVMRegisterType::Invalid;
		}
		else
		{
			if (Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::ByteCodeCleanup)
			{
				uint16 NumBytes = 0;
				Ar << NumBytes;
			}
			
			Ar << RegisterType;
		}
	}
	else
	{
		Ar << RegisterType;
	}
}

void FRigVMComparisonOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << A;
	Ar << B;
	Ar << Result;
}

void FRigVMJumpOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << InstructionIndex;
}

void FRigVMJumpIfOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << Arg;
	Ar << InstructionIndex;
	Ar << Condition;
}

void FRigVMChangeTypeOp::Serialize(FArchive& Ar)
{
	ensure(false);
}

void FRigVMInvokeEntryOp::Serialize(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		FString EntryNameString;
		Ar << EntryNameString;
		EntryName = *EntryNameString;
	}
	else
	{
		FString EntryNameString = EntryName.ToString();
		Ar << EntryNameString;
	}
}

void FRigVMJumpToBranchOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << Arg;
	Ar << FirstBranchInfoIndex;
}

void FRigVMRunInstructionsOp::Serialize(FArchive& Ar)
{
	Ar << OpCode;
	Ar << Arg;
	Ar << StartInstruction;
	Ar << EndInstruction;
}

FRigVMInstructionArray::FRigVMInstructionArray()
{
}

FRigVMInstructionArray::FRigVMInstructionArray(const FRigVMByteCode& InByteCode, bool bByteCodeIsAligned)
{
	int32 ByteIndex = 0;
	while (ByteIndex < InByteCode.Num())
	{
		const ERigVMOpCode OpCode = InByteCode.GetOpCodeAt(ByteIndex);
		if (OpCode >= ERigVMOpCode::Invalid)
		{
			checkNoEntry();
			Instructions.Reset();
			break;
		}

		uint8 OperandAlignment = 0;

		if (bByteCodeIsAligned)
		{
			int32 Alignment = InByteCode.GetOpAlignment(OpCode);
			if (Alignment > 0)
			{
				while (!IsAligned(&InByteCode[ByteIndex], Alignment))
				{
					ByteIndex++;
				}
			}

			if (FRigVMByteCode::IsCallableOpCode(OpCode))
			{
				const int32 OperandByteIndex = ByteIndex + FRigVMByteCode::GetCallableOpSize(OpCode);

				Alignment = InByteCode.GetOperandAlignment();
				if (Alignment > 0)
				{
					while (!IsAligned(&InByteCode[OperandByteIndex + OperandAlignment], Alignment))
					{
						OperandAlignment++;
					}
				}
			}
		}

		const int32 InstructionIndex = Instructions.Num(); 
		Instructions.Add(FRigVMInstruction(OpCode, InstructionIndex, ByteIndex, OperandAlignment));
		ByteIndex += InByteCode.GetOpNumBytesAt(ByteIndex, true);
	}
}

void FRigVMInstructionArray::Reset()
{
	Instructions.Reset();
}

void FRigVMInstructionArray::Empty()
{
	Instructions.Empty();
}

TArray<int32> FRigVMByteCode::EmptyInstructionIndices;

FString FRigVMByteCodeEntry::GetSanitizedName() const
{
	FString SanitizedName = Name.ToString();
	SanitizedName.ReplaceInline(TEXT(" "), TEXT("_"));
	SanitizedName.TrimStartAndEndInline();
	return SanitizedName;
}

FRigVMByteCode::FRigVMByteCode()
	: NumInstructions(0)
	, bByteCodeIsAligned(false)
{
}

void FRigVMByteCode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		return;
	}

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		checkNoEntry();
	}
}

void FRigVMByteCode::Save(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, TEXT("FRigVMByteCode"));

	FRigVMInstructionArray Instructions;

	int32 InstructionCount = 0;
	
	Instructions = GetInstructions();
	InstructionCount = Instructions.Num();
	
	Ar << InstructionCount;

	for (int32 InstructionIndex = 0; InstructionIndex < InstructionCount; InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		ERigVMOpCode OpCode = ERigVMOpCode::Invalid;
		
		OpCode = Instruction.OpCode;
		Ar << OpCode;

		switch (OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				FRigVMExecuteOp Op = GetOpAt<FRigVMExecuteOp>(Instruction.ByteCodeIndex);
				Ar << Op;

				FRigVMOperandArray Operands = GetOperandsForCallableOp(Instruction);
				int32 OperandCount = (int32)Op.GetOperandCount();
				ensure(OperandCount == Operands.Num());

				for (int32 OperandIndex = 0; OperandIndex < OperandCount; OperandIndex++)
				{
					FRigVMOperand Operand = Operands[OperandIndex];
					Ar << Operand;
				}
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				FRigVMInvokeCallableOp Op = GetOpAt<FRigVMInvokeCallableOp>(Instruction.ByteCodeIndex);
				Ar << Op;

				FRigVMOperandArray Operands = GetOperandsForCallableOp(Instruction);
				int32 OperandCount = (int32)Op.GetOperandCount();
				ensure(OperandCount == Operands.Num());
					
				for (int32 OperandIndex = 0; OperandIndex < OperandCount; OperandIndex++)
				{
					FRigVMOperand Operand = Operands[OperandIndex];
					Ar << Operand;
				}
				break;
			}
			case ERigVMOpCode::Copy:
			{
				FRigVMCopyOp Op = GetOpAt<FRigVMCopyOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::Zero:
			case ERigVMOpCode::BoolFalse:
			case ERigVMOpCode::BoolTrue:
			case ERigVMOpCode::Increment:
			case ERigVMOpCode::Decrement:
			case ERigVMOpCode::ArrayReset:
			case ERigVMOpCode::ArrayReverse:
			{
				FRigVMUnaryOp Op = GetOpAt<FRigVMUnaryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::Equals:
			case ERigVMOpCode::NotEquals:
			{
				FRigVMComparisonOp Op = GetOpAt<FRigVMComparisonOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::JumpAbsolute:
			case ERigVMOpCode::JumpForward:
			case ERigVMOpCode::JumpBackward:
			{
				FRigVMJumpOp Op = GetOpAt<FRigVMJumpOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			case ERigVMOpCode::JumpForwardIf:
			case ERigVMOpCode::JumpBackwardIf:
			{
				FRigVMJumpIfOp Op = GetOpAt<FRigVMJumpIfOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::Exit:
			{
				// nothing todo, the ExitOp has no custom data inside of it
				// so all we need is the previously saved OpCode.
				break;
			}
			case ERigVMOpCode::BeginBlock:
			case ERigVMOpCode::ArrayGetNum:
			case ERigVMOpCode::ArraySetNum:
			case ERigVMOpCode::ArrayAppend:
			case ERigVMOpCode::ArrayClone:
			case ERigVMOpCode::ArrayRemove:
			case ERigVMOpCode::ArrayUnion:
			{
				FRigVMBinaryOp Op = GetOpAt<FRigVMBinaryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::ArrayAdd:
			case ERigVMOpCode::ArrayGetAtIndex:
			case ERigVMOpCode::ArraySetAtIndex:
			case ERigVMOpCode::ArrayInsert:
			case ERigVMOpCode::ArrayDifference:
			case ERigVMOpCode::ArrayIntersection:
			{
				FRigVMTernaryOp Op = GetOpAt<FRigVMTernaryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::ArrayFind:
			{
				FRigVMQuaternaryOp Op = GetOpAt<FRigVMQuaternaryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::ArrayIterator:
			{
				FRigVMSenaryOp Op = GetOpAt<FRigVMSenaryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::EndBlock:
			{
				// nothing todo, the EndBlock has no custom data inside of it
				// so all we need is the previously saved OpCode.
				break;
			}
			case ERigVMOpCode::InvokeEntry:
			{
				FRigVMInvokeEntryOp Op = GetOpAt<FRigVMInvokeEntryOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				FRigVMJumpToBranchOp Op = GetOpAt<FRigVMJumpToBranchOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				FRigVMRunInstructionsOp Op = GetOpAt<FRigVMRunInstructionsOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				FRigVMSetupTraitsOp Op = GetOpAt<FRigVMSetupTraitsOp>(Instruction.ByteCodeIndex);
				Ar << Op;
				break;
			}
			default:
			{
				ensure(false);
			}
		}
	}

	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Operators"));

	UScriptStruct* ScriptStruct = FRigVMByteCodeEntry::StaticStruct();
	TArray<uint8, TAlignedHeapAllocator<16>> DefaultStructData;
	DefaultStructData.AddZeroed(ScriptStruct->GetStructureSize());
	ScriptStruct->InitializeDefaultValue(DefaultStructData.GetData());

	TArray<FString> View;
	for (uint16 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		FString Value;
		ScriptStruct->ExportText(Value, &Entries[EntryIndex], DefaultStructData.GetData(), nullptr, PPF_None, nullptr);
		View.Add(Value);
	}

	Ar << View;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Entries"));

	TArray<FRigVMBranchInfo> TempBranchInfos = BranchInfos;
	Ar << TempBranchInfos;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("BranchInfos"));

	Ar << CallableInfos;	
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("CallableInfos"));

	Ar << PublicContextAssetPath;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("PublicContextAssetPath"));
}

void FRigVMByteCode::Load(FArchive& Ar)
{
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RigVMByteCodeDeterminism)
	{
		Ar << ByteCode;
		return;
	}

	int32 InstructionCount = 0;
	
	ByteCode.Reset();
	bByteCodeIsAligned = false;
	Ar << InstructionCount;

	for (int32 InstructionIndex = 0; InstructionIndex < InstructionCount; InstructionIndex++)
	{
		ERigVMOpCode OpCode = ERigVMOpCode::Invalid;

		Ar << OpCode;

		// backwards compatibility
		if(OpCode >= ERigVMOpCode::Execute_0_Operands && OpCode <= ERigVMOpCode::Execute_64_Operands)
		{
			OpCode = ERigVMOpCode::Execute;
		}

		switch (OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				FRigVMExecuteOp Op;
				Ar << Op;

				int32 OperandCount = Op.GetOperandCount();
				TArray<FRigVMOperand> Operands;
				for (int32 OperandIndex = 0; OperandIndex < OperandCount; OperandIndex++)
				{
					FRigVMOperand Operand;
					Ar << Operand;
					Operands.Add(Operand);
				}

				AddExecuteOp(Op.CallableIndex, Operands, Op.FirstPredicateIndex, Op.PredicateCount);
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				FRigVMInvokeCallableOp Op;
				Ar << Op;

				int32 OperandCount = Op.GetOperandCount();
				TArray<FRigVMOperand> Operands;
				for (int32 OperandIndex = 0; OperandIndex < OperandCount; OperandIndex++)
				{
					FRigVMOperand Operand;
					Ar << Operand;
					Operands.Add(Operand);
				}

				AddInvokeCallableOpByIndex(Op.CallableIndex, Operands);
				break;
			}
			case ERigVMOpCode::Copy:
			{
				FRigVMCopyOp Op;
				Ar << Op;
				AddOp<FRigVMCopyOp>(Op);
				break;
			}
			case ERigVMOpCode::Zero:
			case ERigVMOpCode::BoolFalse:
			case ERigVMOpCode::BoolTrue:
			case ERigVMOpCode::Increment:
			case ERigVMOpCode::Decrement:
			case ERigVMOpCode::ArrayReset:
			case ERigVMOpCode::ArrayReverse:
			{
				FRigVMUnaryOp Op;
				Ar << Op;
				AddOp<FRigVMUnaryOp>(Op);
				break;
			}
			case ERigVMOpCode::Equals:
			case ERigVMOpCode::NotEquals:
			{
				FRigVMComparisonOp Op;
				Ar << Op;
				AddOp<FRigVMComparisonOp>(Op);
				break;
			}
			case ERigVMOpCode::JumpAbsolute:
			case ERigVMOpCode::JumpForward:
			case ERigVMOpCode::JumpBackward:
			{
				FRigVMJumpOp Op;
				Ar << Op;
				AddOp<FRigVMJumpOp>(Op);
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			case ERigVMOpCode::JumpForwardIf:
			case ERigVMOpCode::JumpBackwardIf:
			{
				FRigVMJumpIfOp Op;
				Ar << Op;
				AddOp<FRigVMJumpIfOp>(Op);
				break;
			}
			case ERigVMOpCode::Exit:
			{	
				AddExitOp();	
				break;
			}
			case ERigVMOpCode::BeginBlock:
			case ERigVMOpCode::ArrayGetNum:
			case ERigVMOpCode::ArraySetNum:
			case ERigVMOpCode::ArrayAppend:
			case ERigVMOpCode::ArrayClone:
			case ERigVMOpCode::ArrayRemove:
			case ERigVMOpCode::ArrayUnion:
			{
				FRigVMBinaryOp Op;
				Ar << Op;
				AddOp<FRigVMBinaryOp>(Op);
				break;
			}
			case ERigVMOpCode::ArrayAdd:
			case ERigVMOpCode::ArrayGetAtIndex:
			case ERigVMOpCode::ArraySetAtIndex:
			case ERigVMOpCode::ArrayInsert:
			case ERigVMOpCode::ArrayDifference:
			case ERigVMOpCode::ArrayIntersection:
			{
				FRigVMTernaryOp Op;
				Ar << Op;
				AddOp<FRigVMTernaryOp>(Op);
				break;
			}
			case ERigVMOpCode::ArrayFind:
			{
				FRigVMQuaternaryOp Op;
				Ar << Op;
				AddOp<FRigVMQuaternaryOp>(Op);
				break;
			}
			case ERigVMOpCode::ArrayIterator:
			{
				FRigVMSenaryOp Op;
				Ar << Op;
				AddOp<FRigVMSenaryOp>(Op);
				break;
			}
			case ERigVMOpCode::EndBlock:
			{
				AddEndBlockOp();
				break;
			}
			case ERigVMOpCode::InvokeEntry:
			{
				FRigVMInvokeEntryOp Op;
				Ar << Op;
				AddOp<FRigVMInvokeEntryOp>(Op);
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				FRigVMJumpToBranchOp Op;
				Ar << Op;
				AddOp<FRigVMJumpToBranchOp>(Op);
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				FRigVMRunInstructionsOp Op;
				Ar << Op;
				AddOp<FRigVMRunInstructionsOp>(Op);
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				FRigVMSetupTraitsOp Op;
				Ar << Op;
				AddOp<FRigVMSetupTraitsOp>(Op);
				break;
			}
			default:
			{
				ensure(false);
			}
		}
	}
	
	Entries.Reset();
	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeRigVMEntries)
	{
		UScriptStruct* ScriptStruct = FRigVMByteCodeEntry::StaticStruct();

		TArray<FString> View;
		Ar << View;

		for (int32 EntryIndex = 0; EntryIndex < View.Num(); EntryIndex++)
		{
			FRigVMByteCodeEntry Entry;
			ScriptStruct->ImportText(*View[EntryIndex], &Entry, nullptr, PPF_None, nullptr, ScriptStruct->GetName());
			Entries.Add(Entry);
		}
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::RigVMLazyEvaluation)
	{
		Ar << BranchInfos;

		// make sure the lookup table is up 2 date
		BranchInfoLookup.Reset();
		(void)GetBranchInfo({0,0});
	}
	else
	{
		BranchInfos.Reset();
		BranchInfoLookup.Reset();
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::RigVMCallables)
	{
		Ar << CallableInfos;
	}
	else
	{
		CallableInfos.Reset();
	}

	if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMBytecodeStorePublicContextPathAsTopLevelAssetPath)
	{
		Ar << PublicContextAssetPath;
		bHasPublicContextAssetPath = true;
	}
	else if (Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::VMBytecodeStorePublicContextPath)
	{
		FString PublicContextPathName; 
		Ar << PublicContextPathName;

		PublicContextAssetPath = FTopLevelAssetPath(PublicContextPathName);
		bHasPublicContextAssetPath = true;
	}
}

void FRigVMByteCode::Reset()
{
	ByteCode.Reset();
	bByteCodeIsAligned = false;
	NumInstructions = 0;
	Entries.Reset();
	BranchInfos.Reset();
	BranchInfoLookup.Reset();
	CallableInfos.Reset();
	PublicContextAssetPath.Reset();

#if WITH_EDITORONLY_DATA
	SubjectPerInstruction.Reset();
	SubjectToInstructions.Reset();
	CallPathPerInstruction.Reset();
	CallPathToInstructions.Reset();
	CallstackPerInstruction.Reset();
	CallstackHashToInstructions.Reset();
	CallstackHashPerInstruction.Reset();
	InputOperandsPerInstruction.Reset();
	OutputOperandsPerInstruction.Reset();
#endif
}

void FRigVMByteCode::Empty()
{
	ByteCode.Empty();
	bByteCodeIsAligned = false;
	NumInstructions = 0;
	Entries.Empty();
	BranchInfos.Empty();
	BranchInfoLookup.Empty();
	CallableInfos.Empty();
	PublicContextAssetPath.Reset();

#if WITH_EDITORONLY_DATA
	SubjectPerInstruction.Empty();
	SubjectToInstructions.Empty();
	CallPathPerInstruction.Empty();
	CallPathToInstructions.Empty();
	CallstackPerInstruction.Empty();
	CallstackHashToInstructions.Empty();
	CallstackHashPerInstruction.Empty();
	InputOperandsPerInstruction.Empty();
	OutputOperandsPerInstruction.Empty();
#endif
}

void FRigVMByteCode::Shrink()
{
	ByteCode.Shrink();
	Entries.Shrink();
	BranchInfos.Shrink();
	BranchInfoLookup.Shrink();
	CallableInfos.Shrink();

#if WITH_EDITORONLY_DATA
	SubjectPerInstruction.Shrink();
	SubjectToInstructions.Shrink();
	CallPathPerInstruction.Shrink();
	CallPathToInstructions.Shrink();
	CallstackPerInstruction.Shrink();
	CallstackHashToInstructions.Shrink();
	CallstackHashPerInstruction.Shrink();
	InputOperandsPerInstruction.Shrink();
	OutputOperandsPerInstruction.Shrink();
#endif
}

uint32 FRigVMByteCode::GetByteCodeHash() const
{
	uint32 Hash = 0;

	for(int32 EntryIndex = 0; EntryIndex < NumEntries(); EntryIndex++)
	{
		Hash = HashCombine(Hash, GetTypeHash(GetEntry(EntryIndex).Name.ToString()));
		Hash = HashCombine(Hash, GetTypeHash(GetEntry(EntryIndex).InstructionIndex));
	}

	for(const FRigVMInstruction& Instruction : GetInstructions())
	{
		Hash = HashCombine(Hash, GetOperatorHash(Instruction));
	}

	for(const FRigVMBranchInfo& BranchInfo : BranchInfos)
	{
		Hash = HashCombine(Hash, GetTypeHash(BranchInfo));
	}

	for(const FRigVMCallableInfo& CallableInfo : CallableInfos)
	{
		Hash = HashCombine(Hash, GetTypeHash(CallableInfo));
	}

	return Hash;
}

uint32 FRigVMByteCode::GetOperatorHash(const FRigVMInstruction& InInstruction) const
{
	switch (GetOpCodeAt(InInstruction.ByteCodeIndex))
	{
		case ERigVMOpCode::Execute:
		{
			const FRigVMExecuteOp& Op = GetOpAt<FRigVMExecuteOp>(InInstruction);
			uint32 Hash = GetTypeHash(Op);
			for(const FRigVMOperand& Operand : GetOperandsForCallableOp(InInstruction))
			{
				Hash = HashCombine(Hash, GetTypeHash(Operand));
			}
			return Hash;
		}
		case ERigVMOpCode::InvokeCallable:
		{
			const FRigVMInvokeCallableOp& Op = GetOpAt<FRigVMInvokeCallableOp>(InInstruction);
			uint32 Hash = GetTypeHash(Op);
			for(const FRigVMOperand& Operand : GetOperandsForCallableOp(InInstruction))
			{
				Hash = HashCombine(Hash, GetTypeHash(Operand));
			}
			return Hash;
		}
		case ERigVMOpCode::Copy:
		{
			return GetTypeHash(GetOpAt<FRigVMCopyOp>(InInstruction));
		}
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayReverse:
		{
			return GetTypeHash(GetOpAt<FRigVMUnaryOp>(InInstruction));
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			return GetTypeHash(GetOpAt<FRigVMComparisonOp>(InInstruction));
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		{
			return GetTypeHash(GetOpAt<FRigVMJumpOp>(InInstruction));
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			return GetTypeHash(GetOpAt<FRigVMJumpIfOp>(InInstruction));
		}
		case ERigVMOpCode::ChangeType:
		{
			checkNoEntry();
			return 0;
		}
		case ERigVMOpCode::Exit:
		{
			return GetTypeHash(GetOpAt<FRigVMBaseOp>(InInstruction));
		}
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayUnion:
		{
			return GetTypeHash(GetOpAt<FRigVMBinaryOp>(InInstruction));
		}
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		{				
			return GetTypeHash(GetOpAt<FRigVMTernaryOp>(InInstruction));
		}
		case ERigVMOpCode::ArrayFind:
		{				
			return GetTypeHash(GetOpAt<FRigVMQuaternaryOp>(InInstruction));
		}
		case ERigVMOpCode::ArrayIterator:
		{				
			return GetTypeHash(GetOpAt<FRigVMSenaryOp>(InInstruction));
		}
		case ERigVMOpCode::EndBlock:
		{
			return GetTypeHash(GetOpAt<FRigVMBaseOp>(InInstruction));
		}
		case ERigVMOpCode::InvokeEntry:
		{
			return GetTypeHash(GetOpAt<FRigVMInvokeEntryOp>(InInstruction));
		}
		case ERigVMOpCode::JumpToBranch:
		{
			return GetTypeHash(GetOpAt<FRigVMJumpToBranchOp>(InInstruction));
		}
		case ERigVMOpCode::RunInstructions:
		{
			return GetTypeHash(GetOpAt<FRigVMRunInstructionsOp>(InInstruction));
		}
		case ERigVMOpCode::SetupTraits:
		{
			return GetTypeHash(GetOpAt<FRigVMSetupTraitsOp>(InInstruction));
		}
		case ERigVMOpCode::Invalid:
		{
			ensure(false);
			return 0;
		}
		default:
		{
			checkNoEntry();
		}
	}
	return 0;
}

int32 FRigVMByteCode::Num() const
{
	return ByteCode.Num();
}

int32 FRigVMByteCode::NumEntries() const
{
	return Entries.Num();
}

const FRigVMByteCodeEntry& FRigVMByteCode::GetEntry(int32 InEntryIndex) const
{
	return Entries[InEntryIndex];
}

int32 FRigVMByteCode::FindEntryIndex(const FName& InEntryName) const
{
	for (int32 EntryIndex = 0; EntryIndex < Entries.Num(); EntryIndex++)
	{
		if (Entries[EntryIndex].Name == InEntryName)
		{
			return EntryIndex;
		}
	}
	return INDEX_NONE;
}

int32 FRigVMByteCode::NumCallables() const
{
	return CallableInfos.Num();
}

bool FRigVMByteCode::IsValidCallableIndex(int32 InCallableIndex) const
{
	return CallableInfos.IsValidIndex(InCallableIndex);
}

bool FRigVMByteCode::IsValidCallableIndex(uint16 InCallableIndex) const
{
	return IsValidCallableIndex(IntCastChecked<int32, uint16>(InCallableIndex));
}

const FRigVMCallableInfo* FRigVMByteCode::GetCallable(int32 InCallableIndex) const
{
	if (ensure(CallableInfos.IsValidIndex(InCallableIndex)))
	{
		ensure(CallableInfos[InCallableIndex].Index == InCallableIndex);
		return &CallableInfos[InCallableIndex];
	}
	return nullptr;
}

FRigVMCallableInfo* FRigVMByteCode::GetCallable(int32 InCallableIndex)
{
	if (ensure(CallableInfos.IsValidIndex(InCallableIndex)))
	{
		ensure(CallableInfos[InCallableIndex].Index == InCallableIndex);
		return &CallableInfos[InCallableIndex];
	}
	return nullptr;
}

const FRigVMCallableInfo* FRigVMByteCode::GetCallable(uint16 InCallableIndex) const
{
	return GetCallable(IntCastChecked<int32, uint16>(InCallableIndex));
}

FRigVMCallableInfo* FRigVMByteCode::GetCallable(uint16 InCallableIndex)
{
	return GetCallable(IntCastChecked<int32, uint16>(InCallableIndex));
}

int32 FRigVMByteCode::GetCallableIndex(const uint32& InHash) const
{
	return CallableInfos.IndexOfByPredicate([InHash](const FRigVMCallableInfo& InCallableInfo) -> bool
	{
		return InCallableInfo.FunctionHash == InHash;
	});
}

const FRigVMCallableInfo* FRigVMByteCode::FindCallable(const uint32& InHash) const
{
	return CallableInfos.FindByPredicate([InHash](const FRigVMCallableInfo& InCallableInfo) -> const FRigVMCallableInfo*
	{
		if (InCallableInfo.FunctionHash == InHash)
		{
			return &InCallableInfo;
		}
		return nullptr;
	});
}

const FRigVMCallableInfo* FRigVMByteCode::FindCallable(const FName& InName) const
{
	return CallableInfos.FindByPredicate([InName](const FRigVMCallableInfo& InCallableInfo) -> const FRigVMCallableInfo*
	{
		if (InCallableInfo.Name == InName)
		{
			return &InCallableInfo;
		}
		return nullptr;
	});
}

const FRigVMCallableInfo* FRigVMByteCode::FindCallableForInstruction(int32 InInstructionIndex) const
{
	return CallableInfos.FindByPredicate([InInstructionIndex](const FRigVMCallableInfo& InCallableInfo) -> const FRigVMCallableInfo*
	{
		if (FMath::IsWithinInclusive(InInstructionIndex, InCallableInfo.FirstInstruction, InCallableInfo.LastInstruction))
		{
			return &InCallableInfo;
		}
		return nullptr;
	});
}

int32 FRigVMByteCode::GetOpNumBytesAt(int32 InByteCodeIndex, bool bIncludeOperands) const
{
	const ERigVMOpCode OpCode = GetOpCodeAt(InByteCodeIndex);
	switch (OpCode)
	{
		case ERigVMOpCode::Execute_0_Operands:
		case ERigVMOpCode::Execute_1_Operands:
		case ERigVMOpCode::Execute_2_Operands:
		case ERigVMOpCode::Execute_3_Operands:
		case ERigVMOpCode::Execute_4_Operands:
		case ERigVMOpCode::Execute_5_Operands:
		case ERigVMOpCode::Execute_6_Operands:
		case ERigVMOpCode::Execute_7_Operands:
		case ERigVMOpCode::Execute_8_Operands:
		case ERigVMOpCode::Execute_9_Operands:
		case ERigVMOpCode::Execute_10_Operands:
		case ERigVMOpCode::Execute_11_Operands:
		case ERigVMOpCode::Execute_12_Operands:
		case ERigVMOpCode::Execute_13_Operands:
		case ERigVMOpCode::Execute_14_Operands:
		case ERigVMOpCode::Execute_15_Operands:
		case ERigVMOpCode::Execute_16_Operands:
		case ERigVMOpCode::Execute_17_Operands:
		case ERigVMOpCode::Execute_18_Operands:
		case ERigVMOpCode::Execute_19_Operands:
		case ERigVMOpCode::Execute_20_Operands:
		case ERigVMOpCode::Execute_21_Operands:
		case ERigVMOpCode::Execute_22_Operands:
		case ERigVMOpCode::Execute_23_Operands:
		case ERigVMOpCode::Execute_24_Operands:
		case ERigVMOpCode::Execute_25_Operands:
		case ERigVMOpCode::Execute_26_Operands:
		case ERigVMOpCode::Execute_27_Operands:
		case ERigVMOpCode::Execute_28_Operands:
		case ERigVMOpCode::Execute_29_Operands:
		case ERigVMOpCode::Execute_30_Operands:
		case ERigVMOpCode::Execute_31_Operands:
		case ERigVMOpCode::Execute_32_Operands:
		case ERigVMOpCode::Execute_33_Operands:
		case ERigVMOpCode::Execute_34_Operands:
		case ERigVMOpCode::Execute_35_Operands:
		case ERigVMOpCode::Execute_36_Operands:
		case ERigVMOpCode::Execute_37_Operands:
		case ERigVMOpCode::Execute_38_Operands:
		case ERigVMOpCode::Execute_39_Operands:
		case ERigVMOpCode::Execute_40_Operands:
		case ERigVMOpCode::Execute_41_Operands:
		case ERigVMOpCode::Execute_42_Operands:
		case ERigVMOpCode::Execute_43_Operands:
		case ERigVMOpCode::Execute_44_Operands:
		case ERigVMOpCode::Execute_45_Operands:
		case ERigVMOpCode::Execute_46_Operands:
		case ERigVMOpCode::Execute_47_Operands:
		case ERigVMOpCode::Execute_48_Operands:
		case ERigVMOpCode::Execute_49_Operands:
		case ERigVMOpCode::Execute_50_Operands:
		case ERigVMOpCode::Execute_51_Operands:
		case ERigVMOpCode::Execute_52_Operands:
		case ERigVMOpCode::Execute_53_Operands:
		case ERigVMOpCode::Execute_54_Operands:
		case ERigVMOpCode::Execute_55_Operands:
		case ERigVMOpCode::Execute_56_Operands:
		case ERigVMOpCode::Execute_57_Operands:
		case ERigVMOpCode::Execute_58_Operands:
		case ERigVMOpCode::Execute_59_Operands:
		case ERigVMOpCode::Execute_60_Operands:
		case ERigVMOpCode::Execute_61_Operands:
		case ERigVMOpCode::Execute_62_Operands:
		case ERigVMOpCode::Execute_63_Operands:
		case ERigVMOpCode::Execute_64_Operands:
		case ERigVMOpCode::Execute:
		case ERigVMOpCode::InvokeCallable:
		{
			const int32 OpSize = GetCallableOpSize(OpCode); 
			int32 NumBytes = OpSize;
			if(bIncludeOperands)
			{
				FRigVMInvokeCallableOp CallableOp;
				uint8* CallableOpPtr = reinterpret_cast<uint8*>(&CallableOp);
				for (int32 ByteIndex = 0; ByteIndex < sizeof(FRigVMInvokeCallableOp); ByteIndex++)
				{
					CallableOpPtr[ByteIndex] = ByteCode[InByteCodeIndex + ByteIndex];
				}

				if (bByteCodeIsAligned)
				{
					static const uint64 OperandAlignment = GetOperandAlignment();
					if (OperandAlignment > 0)
					{
						while (!IsAligned(&ByteCode[InByteCodeIndex + NumBytes], OperandAlignment))
						{
							NumBytes++;
						}
					}
				}
				NumBytes += CallableOp.GetOperandCount() * sizeof(FRigVMOperand);
			}
			return NumBytes;
		}
		case ERigVMOpCode::Copy:
		{
			return sizeof(FRigVMCopyOp);
		}
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayReverse:
		{
			return sizeof(FRigVMUnaryOp);
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			return sizeof(FRigVMComparisonOp);
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		{
			return sizeof(FRigVMJumpOp);
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			return sizeof(FRigVMJumpIfOp);
		}
		case ERigVMOpCode::ChangeType:
		{
			checkNoEntry();
			return 0;
		}
		case ERigVMOpCode::Exit:
		{
			return sizeof(FRigVMBaseOp);
		}
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayUnion:
		{
			return sizeof(FRigVMBinaryOp);
		}
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		{				
			return sizeof(FRigVMTernaryOp);
		}
		case ERigVMOpCode::ArrayFind:
		{				
			return sizeof(FRigVMQuaternaryOp);
		}
		case ERigVMOpCode::ArrayIterator:
		{				
			return sizeof(FRigVMSenaryOp);
		}
		case ERigVMOpCode::EndBlock:
		{
			return sizeof(FRigVMBaseOp);
		}
		case ERigVMOpCode::InvokeEntry:
		{
			return sizeof(FRigVMInvokeEntryOp);
		}
		case ERigVMOpCode::JumpToBranch:
		{
			return sizeof(FRigVMJumpToBranchOp);
		}
		case ERigVMOpCode::RunInstructions:
		{
			return sizeof(FRigVMRunInstructionsOp);
		}
		case ERigVMOpCode::SetupTraits:
		{
			return sizeof(FRigVMSetupTraitsOp);
		}
		case ERigVMOpCode::Invalid:
		{
			ensure(false);
			return 0;
		}
	}
	return 0;
}

int32 FRigVMByteCode::AddZeroOp(const FRigVMOperand& InArg)
{
	FRigVMUnaryOp Op(ERigVMOpCode::Zero, InArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddFalseOp(const FRigVMOperand& InArg)
{
	FRigVMUnaryOp Op(ERigVMOpCode::BoolFalse, InArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddTrueOp(const FRigVMOperand& InArg)
{
	FRigVMUnaryOp Op(ERigVMOpCode::BoolTrue, InArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddCopyOp(const FRigVMOperand& InSource, const FRigVMOperand& InTarget)
{
	check(InTarget.GetMemoryType() != ERigVMMemoryType::Literal);
	check(InSource != InTarget);

	const FRigVMCopyOp Op(InSource, InTarget);
	const int32 ByteIndex = AddOp(Op);

#if WITH_EDITORONLY_DATA
	const FRigVMInstruction Instruction(Op.OpCode, NumInstructions - 1, ByteIndex);
	const FRigVMOperandArray InputOperands(&InSource, 1);
	const FRigVMOperandArray OutputOperands(&InTarget, 1);
	SetOperandsForInstruction(Instruction, InputOperands, OutputOperands);
#endif
	
	return ByteIndex;
}

int32 FRigVMByteCode::AddCopyOp(const FRigVMCopyOp& InCopyOp)
{
	return AddCopyOp(InCopyOp.Source, InCopyOp.Target);
}

int32 FRigVMByteCode::AddIncrementOp(const FRigVMOperand& InArg)
{
	ensure(InArg.GetMemoryType() != ERigVMMemoryType::Literal);
	FRigVMUnaryOp Op(ERigVMOpCode::Increment, InArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddDecrementOp(const FRigVMOperand& InArg)
{
	ensure(InArg.GetMemoryType() != ERigVMMemoryType::Literal);
	FRigVMUnaryOp Op(ERigVMOpCode::Decrement, InArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddEqualsOp(const FRigVMOperand& InA, const FRigVMOperand& InB, const FRigVMOperand& InResult)
{
	const FRigVMComparisonOp Op(ERigVMOpCode::Equals, InA, InB, InResult);
	const int32 ByteIndex = AddOp(Op);
	
#if WITH_EDITORONLY_DATA
	const FRigVMInstruction Instruction(Op.OpCode, NumInstructions - 1, ByteIndex);
	TArray<FRigVMOperand> Inputs;
	Inputs.Add(InA);
	Inputs.Add(InB);
	const FRigVMOperandArray InputOperands(&Inputs[0], Inputs.Num());
	const FRigVMOperandArray OutputOperands(&InResult, 1);
	SetOperandsForInstruction(Instruction, InputOperands, OutputOperands);
#endif

	return ByteIndex;
}

int32 FRigVMByteCode::AddNotEqualsOp(const FRigVMOperand& InA, const FRigVMOperand& InB, const FRigVMOperand& InResult)
{
	const FRigVMComparisonOp Op(ERigVMOpCode::NotEquals, InA, InB, InResult);
	const int32 ByteIndex = AddOp(Op);

#if WITH_EDITORONLY_DATA
	const FRigVMInstruction Instruction(Op.OpCode, NumInstructions - 1, ByteIndex);
	TArray<FRigVMOperand> Inputs;
	Inputs.Add(InA);
	Inputs.Add(InB);
	const FRigVMOperandArray InputOperands(&Inputs[0], Inputs.Num());
	const FRigVMOperandArray OutputOperands(&InResult, 1);
	SetOperandsForInstruction(Instruction, InputOperands, OutputOperands);
#endif

	return ByteIndex;
}

int32 FRigVMByteCode::AddJumpOp(ERigVMOpCode InOpCode, int32 InInstructionIndex)
{
	FRigVMJumpOp Op(InOpCode, InInstructionIndex);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddJumpIfOp(ERigVMOpCode InOpCode, int32 InInstructionIndex, const FRigVMOperand& InConditionArg, bool bJumpWhenConditionIs)
{
	FRigVMJumpIfOp Op(InOpCode, InConditionArg, InInstructionIndex, bJumpWhenConditionIs);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddExecuteOp(int32 InFunctionIndex, const FRigVMOperandArray& InOperands, const int32 InStartPredicateIndex, const int32 InPredicateCount)
{
	FRigVMExecuteOp Op(IntCastChecked<uint16>(InFunctionIndex), IntCastChecked<uint16>(InOperands.Num()));
	if (InPredicateCount)
	{
		Op.FirstPredicateIndex = IntCastChecked<uint16>(InStartPredicateIndex);
		Op.PredicateCount = IntCastChecked<uint16>(InPredicateCount);
	}
	int32 OpByteIndex = AddOp(Op);

#if WITH_EDITOR
	for (const FRigVMOperand& Operand : InOperands)
	{
		verify(Operand.IsValid());
	}
#endif

	const int32 OperandsByteIndex = ByteCode.AddZeroed(sizeof(FRigVMOperand) * InOperands.Num());
	FMemory::Memcpy(ByteCode.GetData() + OperandsByteIndex, InOperands.GetData(), sizeof(FRigVMOperand) * InOperands.Num());

	for(int32 Index = 0; Index < InOperands.Num(); Index++)
	{
		FRigVMOperand* Operand = reinterpret_cast<FRigVMOperand*>(ByteCode.GetData() + OperandsByteIndex + sizeof(FRigVMOperand) * Index);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(Operand);
	}
	
	return OpByteIndex;
}

int32 FRigVMByteCode::InlineFunction(const FRigVMByteCode* FunctionByteCode, int32 InFirstIndex, int32 InLastIndex)
{
	check(FunctionByteCode);
	check(!FunctionByteCode->bByteCodeIsAligned);
	const int32 OpByteIndex = ByteCode.Num();
	if (FunctionByteCode->Num() == 0)
	{
		return OpByteIndex;
	}

	if (InFirstIndex == INDEX_NONE && InLastIndex == INDEX_NONE)
	{
		ByteCode.Append(FunctionByteCode->ByteCode);
		NumInstructions += FunctionByteCode->NumInstructions;
	}
	else
	{
		const FRigVMInstructionArray FunctionInstructions = FunctionByteCode->GetInstructions();
		check(FunctionInstructions.IsValidIndex(InFirstIndex) && FunctionInstructions.IsValidIndex(InLastIndex));

		const TArrayView<const uint8> BytesView(FunctionByteCode->ByteCode);
		const int32 NumInstructionsToAdd = InLastIndex - InFirstIndex + 1;
		const int32 FirstByteIndex = FunctionInstructions[InFirstIndex].ByteCodeIndex;
		const int32 LastByteIndex =
			InLastIndex == FunctionInstructions.Num() - 1 ?
			FunctionByteCode->Num() - 1 :
			FunctionInstructions[InLastIndex + 1].ByteCodeIndex - 1;
		const int32 NumBytesToAdd = LastByteIndex - FirstByteIndex + 1;
		
		const TArrayView<const uint8> BytesToAdd(BytesView.Slice(FirstByteIndex, NumBytesToAdd));
		ByteCode.Append(BytesToAdd);
		NumInstructions += NumInstructionsToAdd;
	}
	
	return OpByteIndex;
}

int32 FRigVMByteCode::AddExitOp()
{
	FRigVMBaseOp Op(ERigVMOpCode::Exit);
	return AddOp(Op);
}

FString FRigVMByteCode::DumpToText() const
{
	TArray<FString> Lines;

	FRigVMInstructionArray Instructions = GetInstructions();
	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		ERigVMOpCode OpCode = Instruction.OpCode;

		FString Line = StaticEnum<ERigVMOpCode>()->GetNameByValue(static_cast<int64>(OpCode)).ToString();

		switch (OpCode)
		{
			case ERigVMOpCode::Execute:
			{
				const FRigVMExecuteOp& Op = GetOpAt<FRigVMExecuteOp>(Instruction.ByteCodeIndex);
				Line += FString::Printf(TEXT(", FunctionIndex %u"), Op.CallableIndex);

				FRigVMOperandArray Operands = GetOperandsForCallableOp(Instruction);
				if (Operands.Num() > 0)
				{
					TArray<FString> OperandsContent;
					for (const FRigVMOperand& Operand : Operands)
					{
						FString OperandContent;
						FRigVMOperand::StaticStruct()->ExportText(OperandContent, &Operand, &Operand, nullptr, PPF_None, nullptr);
						OperandsContent.Add(FString::Printf(TEXT("\t%s"), *OperandContent));
					}

					Line += RigVMStringUtils::JoinStrings(OperandsContent, TEXT("(\n"), TEXT("\n"), TEXT("\n)"));
				}
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				const FRigVMInvokeCallableOp& Op = GetOpAt<FRigVMInvokeCallableOp>(Instruction.ByteCodeIndex);
				const FRigVMCallableInfo* CallableInfo = GetCallable(Op.CallableIndex);
				check(CallableInfo);
				Line += FString::Printf(TEXT(", CallableIndex %u"), Op.CallableIndex);
				Line += FString::Printf(TEXT(", Function '%s'"), *CallableInfo->Name.ToString());

				FRigVMOperandArray Operands = GetOperandsForCallableOp(Instruction);
				if (Operands.Num() > 0)
				{
					TArray<FString> OperandsContent;
					for (const FRigVMOperand& Operand : Operands)
					{
						FString OperandContent;
						FRigVMOperand::StaticStruct()->ExportText(OperandContent, &Operand, &Operand, nullptr, PPF_None, nullptr);
						OperandsContent.Add(FString::Printf(TEXT("\t%s"), *OperandContent));
					}

					Line += RigVMStringUtils::JoinStrings(OperandsContent, TEXT("(\n"), TEXT("\n"), TEXT("\n)"));
				}
				break;
			}
			case ERigVMOpCode::Copy:
			{
				const FRigVMCopyOp& Op = GetOpAt<FRigVMCopyOp>(Instruction.ByteCodeIndex);
				FString SourceContent;
				FRigVMOperand::StaticStruct()->ExportText(SourceContent, &Op.Source, &Op.Source, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", Source %s"), *SourceContent);
				FString TargetContent;
				FRigVMOperand::StaticStruct()->ExportText(TargetContent, &Op.Target, &Op.Target, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", Target %s"), *TargetContent);
				break;
			}
			case ERigVMOpCode::Zero:
			case ERigVMOpCode::BoolFalse:
			case ERigVMOpCode::BoolTrue:
			case ERigVMOpCode::Increment:
			case ERigVMOpCode::Decrement:
			{
				const FRigVMUnaryOp& Op = GetOpAt<FRigVMUnaryOp>(Instruction.ByteCodeIndex);
				FString ArgContent;
				FRigVMOperand::StaticStruct()->ExportText(ArgContent, &Op.Arg, &Op.Arg, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", Source %s"), *ArgContent);
				break;
			}
			case ERigVMOpCode::Equals:
			case ERigVMOpCode::NotEquals:
			{
				const FRigVMComparisonOp& Op = GetOpAt<FRigVMComparisonOp>(Instruction.ByteCodeIndex);
				FString AContent;
				FRigVMOperand::StaticStruct()->ExportText(AContent, &Op.A, &Op.A, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", A %s"), *AContent);
				FString BContent;
				FRigVMOperand::StaticStruct()->ExportText(BContent, &Op.B, &Op.B, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", B %s"), *BContent);
				FString ResultContent;
				FRigVMOperand::StaticStruct()->ExportText(ResultContent, &Op.Result, &Op.Result, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", Result %s"), *ResultContent);
				break;
			}
			case ERigVMOpCode::JumpAbsolute:
			case ERigVMOpCode::JumpForward:
			case ERigVMOpCode::JumpBackward:
			{
				const FRigVMJumpOp& Op = GetOpAt<FRigVMJumpOp>(Instruction.ByteCodeIndex);
				Line += FString::Printf(TEXT(", InstructionIndex %d"), (int32)Op.InstructionIndex);
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			case ERigVMOpCode::JumpForwardIf:
			case ERigVMOpCode::JumpBackwardIf:
			{
				const FRigVMJumpIfOp& Op = GetOpAt<FRigVMJumpIfOp>(Instruction.ByteCodeIndex);
				Line += FString::Printf(TEXT(", InstructionIndex %d"), (int32)Op.InstructionIndex);
				FString ArgContent;
				FRigVMOperand::StaticStruct()->ExportText(ArgContent, &Op.Arg, &Op.Arg, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", Source %s"), *ArgContent);
				Line += FString::Printf(TEXT(", Condition %d"), (int32)(Op.Condition ? 0 : 1));
				break;
			}
			case ERigVMOpCode::Exit:
			{
				break;
			}
			case ERigVMOpCode::BeginBlock:
			{
				const FRigVMBinaryOp& Op = GetOpAt<FRigVMBinaryOp>(Instruction.ByteCodeIndex);
				FString ArgA, ArgB;
				FRigVMOperand::StaticStruct()->ExportText(ArgA, &Op.ArgA, &Op.ArgA, nullptr, PPF_None, nullptr);
				FRigVMOperand::StaticStruct()->ExportText(ArgB, &Op.ArgB, &Op.ArgB, nullptr, PPF_None, nullptr);
				Line += FString::Printf(TEXT(", ArgA %s"), *ArgA);
				Line += FString::Printf(TEXT(", ArgB %s"), *ArgB);
				break;
			}
			case ERigVMOpCode::InvokeEntry:
			{
				const FRigVMInvokeEntryOp& Op = GetOpAt<FRigVMInvokeEntryOp>(Instruction.ByteCodeIndex);
				Line += FString::Printf(TEXT(", Entry '%s'"), *Op.EntryName.ToString());
				break;
			}
			case ERigVMOpCode::JumpToBranch:
			{
				const FRigVMJumpToBranchOp& Op = GetOpAt<FRigVMJumpToBranchOp>(Instruction.ByteCodeIndex);
				FString Arg;
				FRigVMOperand::StaticStruct()->ExportText(Arg, &Op.Arg, &Op.Arg, nullptr, PPF_None, nullptr);
				Line += TEXT(" BlockToRun ");
				Line += Arg;
				Line += TEXT(" for branches ");

				TArray<FString> BranchInfoTexts;
				for(int32 BranchIndex = Op.FirstBranchInfoIndex; BranchIndex < BranchInfos.Num(); BranchIndex++)
				{
					const FRigVMBranchInfo& BranchInfo = BranchInfos[BranchIndex];
					if(BranchInfo.InstructionIndex != InstructionIndex)
					{
						break;
					}
					BranchInfoTexts.Add(FString::Printf(TEXT("%s (%d)"), *BranchInfo.Label.ToString(), BranchInfo.FirstInstruction));
				}
				Line += RigVMStringUtils::JoinStrings(BranchInfoTexts, TEXT(", "));
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				const FRigVMRunInstructionsOp& Op = GetOpAt<FRigVMRunInstructionsOp>(Instruction.ByteCodeIndex);
				FString Arg;
				FRigVMOperand::StaticStruct()->ExportText(Arg, &Op.Arg, &Op.Arg, nullptr, PPF_None, nullptr);
				Line += TEXT(" Instructions ");
				Line += FString::FromInt(Op.StartInstruction);
				Line += TEXT("-");
				Line += FString::FromInt(Op.EndInstruction);
				break;
			}
			case ERigVMOpCode::SetupTraits:
			{
				const FRigVMSetupTraitsOp& Op = GetOpAt<FRigVMSetupTraitsOp>(Instruction.ByteCodeIndex);
				FString Arg;
				FRigVMOperand::StaticStruct()->ExportText(Arg, &Op.Arg, &Op.Arg, nullptr, PPF_None, nullptr);
				Line += Arg;
				break;
			}
			case ERigVMOpCode::Invalid:
			{
				ensure(false);
			}
			default:
			{
				break;
			}
		}

		Lines.Add(Line);
	}

	if (Lines.Num() == 0)
	{
		return FString();
	}

	return RigVMStringUtils::JoinStrings(Lines, TEXT("\n"));
}

int32 FRigVMByteCode::AddBeginBlockOp(FRigVMOperand InCountArg, FRigVMOperand InIndexArg)
{
	FRigVMBinaryOp Op(ERigVMOpCode::BeginBlock, InCountArg, InIndexArg);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddEndBlockOp()
{
	FRigVMBaseOp Op(ERigVMOpCode::EndBlock);
	return AddOp(Op);
}

int32 FRigVMByteCode::AddInvokeEntryOp(const FName& InEntryName)
{
	return AddOp(FRigVMInvokeEntryOp(InEntryName));
}

int32 FRigVMByteCode::AddInvokeCallableOp(const uint32& InCallableHash, const FRigVMOperandArray& InOperands)
{
	const FRigVMCallableInfo* CallableInfo = FindCallable(InCallableHash);
	check(CallableInfo);
	return AddInvokeCallableOpByIndex(IntCastChecked<uint16>(CallableInfo->Index), InOperands);
}

int32 FRigVMByteCode::AddInvokeCallableOpByIndex(const uint16& InCallableIndex, const FRigVMOperandArray& InOperands)
{
	FRigVMInvokeCallableOp Op(InCallableIndex, IntCastChecked<uint16>(InOperands.Num()));
	int32 OpByteIndex = AddOp(Op);

#if WITH_EDITOR
	for (const FRigVMOperand& Operand : InOperands)
	{
		verify(Operand.IsValid());
	}
#endif

	const int32 OperandsByteIndex = ByteCode.AddZeroed(sizeof(FRigVMOperand) * InOperands.Num());
	FMemory::Memcpy(ByteCode.GetData() + OperandsByteIndex, InOperands.GetData(), sizeof(FRigVMOperand) * InOperands.Num());

	for(int32 Index = 0; Index < InOperands.Num(); Index++)
	{
		FRigVMOperand* Operand = reinterpret_cast<FRigVMOperand*>(ByteCode.GetData() + OperandsByteIndex + sizeof(FRigVMOperand) * Index);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(Operand);
	}

	return OpByteIndex;
}

int32 FRigVMByteCode::AddJumpToBranchOp(FRigVMOperand InBranchNameArg, int32 InFirstBranchInfoIndex)
{
	return AddOp(FRigVMJumpToBranchOp(InBranchNameArg, InFirstBranchInfoIndex));
}

int32 FRigVMByteCode::AddRunInstructionsOp(FRigVMOperand InExecuteStateArg, int32 InStartInstruction, int32 InEndInstruction)
{
	return AddOp(FRigVMRunInstructionsOp(InExecuteStateArg, InStartInstruction, InEndInstruction));
}

int32 FRigVMByteCode::AddSetupTraitsOp(FRigVMOperand InTraitListArg)
{
	return AddOp(FRigVMSetupTraitsOp(InTraitListArg));
}

int32 FRigVMByteCode::AddBranchInfo(const FRigVMBranchInfo& InBranchInfo)
{
	FRigVMBranchInfo BranchInfo = InBranchInfo;
	BranchInfo.Index = BranchInfos.Num();
	BranchInfos.Add(BranchInfo);
	BranchInfoLookup.Reset();
	return BranchInfo.Index;
}

int32 FRigVMByteCode::AddBranchInfo(const FName& InBranchLabel, int32 InInstructionIndex, int32 InArgumentIndex,
	int32 InFirstBranchInstruction, int32 InLastBranchInstruction)
{
	FRigVMBranchInfo BranchInfo;
	BranchInfo.Label = InBranchLabel;
	BranchInfo.InstructionIndex = InInstructionIndex;
	BranchInfo.ArgumentIndex = InArgumentIndex;
	BranchInfo.FirstInstruction = InFirstBranchInstruction;
	BranchInfo.LastInstruction = InLastBranchInstruction;
	return AddBranchInfo(BranchInfo);
}

int32 FRigVMByteCode::AddPredicateBranch(const FRigVMPredicateBranch& InPredicateBranch)
{
	return PredicateBranches.Add(InPredicateBranch);
}

const FRigVMBranchInfo* FRigVMByteCode::FindBranch(int32 InInstructionIndex, int32 InArgumentIndex) const
{
	for(const FRigVMBranchInfo& BranchInfo : BranchInfos)
	{
		if (BranchInfo.InstructionIndex == InInstructionIndex && BranchInfo.ArgumentIndex == InArgumentIndex)
		{
			return &BranchInfo;
		}
	}
	return nullptr;
}

const FRigVMBranchInfo* FRigVMByteCode::FindBranch(int32 InInstructionIndex, const FName& InLabel) const
{
	for(const FRigVMBranchInfo& BranchInfo : BranchInfos)
	{
		if (BranchInfo.InstructionIndex == InInstructionIndex && BranchInfo.Label == InLabel)
		{
			return &BranchInfo;
		}
	}
	return nullptr;
}

FRigVMOperandArray FRigVMByteCode::GetOperandsForOp(const FRigVMInstruction& InInstruction) const
{
	switch(InInstruction.OpCode)
	{
		case ERigVMOpCode::Execute:
		case ERigVMOpCode::InvokeCallable:
		{
			return GetOperandsForCallableOp(InInstruction);
		}
		case ERigVMOpCode::Copy:
		{
			const FRigVMCopyOp& Op = GetOpAt<FRigVMCopyOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Source, 2);
		}
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayReverse:
		{
			const FRigVMUnaryOp& Op = GetOpAt<FRigVMUnaryOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Arg, 1);
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			const FRigVMComparisonOp& Op = GetOpAt<FRigVMComparisonOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.A, 3);
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			const FRigVMJumpIfOp& Op = GetOpAt<FRigVMJumpIfOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Arg, 1);
		}
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayUnion:
		{
			const FRigVMBinaryOp& Op = GetOpAt<FRigVMBinaryOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.ArgA, 2);
		}
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		{
			const FRigVMTernaryOp& Op = GetOpAt<FRigVMTernaryOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.ArgA, 3);
		}
		case ERigVMOpCode::ArrayFind:
		{
			const FRigVMQuaternaryOp& Op = GetOpAt<FRigVMQuaternaryOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.ArgA, 4);
		}
		case ERigVMOpCode::ArrayIterator:
		{
			const FRigVMSenaryOp& Op = GetOpAt<FRigVMSenaryOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.ArgA, 6);
		}
		case ERigVMOpCode::JumpToBranch:
		{
			const FRigVMJumpToBranchOp& Op = GetOpAt<FRigVMJumpToBranchOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Arg, 1);
		}
		case ERigVMOpCode::RunInstructions:
		{
			const FRigVMRunInstructionsOp& Op = GetOpAt<FRigVMRunInstructionsOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Arg, 1);
		}
		case ERigVMOpCode::SetupTraits:
		{
			const FRigVMSetupTraitsOp& Op = GetOpAt<FRigVMSetupTraitsOp>(InInstruction.ByteCodeIndex);
			return FRigVMOperandArray(&Op.Arg, 1);
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		case ERigVMOpCode::ChangeType:
		case ERigVMOpCode::Exit:
		case ERigVMOpCode::EndBlock:
		case ERigVMOpCode::Invalid:
		default:
		{
			break;
		}
	}

	return FRigVMOperandArray();
}

int32 FRigVMByteCode::GetFirstOperandByteIndex(const FRigVMInstruction& InInstruction) const
{
	if (IsCallableOpCode(InInstruction.OpCode))
	{
		const int32 OpSize = GetCallableOpSize(InInstruction.OpCode);
		const int32 ByteCodeIndex = InInstruction.ByteCodeIndex;
		// if the bytecode is not aligned the OperandAlignment needs to be 0
		ensure(bByteCodeIsAligned || InInstruction.OperandAlignment == 0);
		return ByteCodeIndex + OpSize + InInstruction.OperandAlignment;
	}

	switch(InInstruction.OpCode)
	{
		case ERigVMOpCode::Copy:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMCopyOp, Source);
		}
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayReverse:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMUnaryOp, Arg);
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMComparisonOp, A);
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMJumpIfOp, Arg);
		}
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayUnion:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMBinaryOp, ArgA);
		}
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMTernaryOp, ArgA);
		}
		case ERigVMOpCode::ArrayFind:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMQuaternaryOp, ArgA);
		}
		case ERigVMOpCode::ArrayIterator:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMSenaryOp, ArgA);
		}
		case ERigVMOpCode::JumpToBranch:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMJumpToBranchOp, Arg);
		}
		case ERigVMOpCode::RunInstructions:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMRunInstructionsOp, Arg);
		}
		case ERigVMOpCode::SetupTraits:
		{
			return InInstruction.ByteCodeIndex + offsetof(FRigVMSetupTraitsOp, Arg);
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		case ERigVMOpCode::ChangeType:
		case ERigVMOpCode::Exit:
		case ERigVMOpCode::EndBlock:
		case ERigVMOpCode::Invalid:
		default:
		{
			break;
		}
	}

	return INDEX_NONE;
}

TArray<int32> FRigVMByteCode::GetInstructionsForOperand(const FRigVMOperand& InOperand) const
{
	TArray<int32> InstructionIndices;
	
	const FRigVMInstructionArray Instructions = GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < GetNumInstructions(); InstructionIndex++)
	{
		if(GetOperandsForOp(Instructions[InstructionIndex]).Contains(InOperand))
		{
			InstructionIndices.Add(InstructionIndex);
		}

	}
	return InstructionIndices;
}

#if WITH_EDITOR

TArray<int32> FRigVMByteCode::GetInstructionsForInputOperand(const FRigVMOperand& InOperand) const
{
	TArray<int32> InstructionIndices;
	
	const FRigVMInstructionArray Instructions = GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < GetNumInstructions(); InstructionIndex++)
	{
		if(InputOperandsPerInstruction[InstructionIndex].Contains(InOperand))
		{
			InstructionIndices.Add(InstructionIndex);
		}

	}
	return InstructionIndices;
}

TArray<int32> FRigVMByteCode::GetInstructionsForOutputOperand(const FRigVMOperand& InOperand) const
{
	TArray<int32> InstructionIndices;
	
	const FRigVMInstructionArray Instructions = GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < GetNumInstructions(); InstructionIndex++)
	{
		if(OutputOperandsPerInstruction[InstructionIndex].Contains(InOperand))
		{
			InstructionIndices.Add(InstructionIndex);
		}

	}
	return InstructionIndices;
}

#endif

int32 FRigVMByteCode::GetOpAlignment(ERigVMOpCode InOpCode)
{
	switch (InOpCode)
	{
		case ERigVMOpCode::Execute:
		{
			static const int32 Alignment = FRigVMExecuteOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::InvokeCallable:
		{
			static const int32 Alignment = FRigVMInvokeCallableOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::Copy:
		{
			static const int32 Alignment = FRigVMCopyOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayReverse:
		{
			static const int32 Alignment = FRigVMUnaryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		{
			static const int32 Alignment = FRigVMComparisonOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::JumpAbsolute:
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		{
			static const int32 Alignment = FRigVMJumpOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::JumpAbsoluteIf:
		case ERigVMOpCode::JumpForwardIf:
		case ERigVMOpCode::JumpBackwardIf:
		{
			static const int32 Alignment = FRigVMJumpIfOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::ChangeType:
		{
			checkNoEntry();
			return 0;
		}
		case ERigVMOpCode::Exit:
		{
			static const int32 Alignment = FRigVMBaseOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayUnion:
		{
			static const int32 Alignment = FRigVMBinaryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		{				
			static const int32 Alignment = FRigVMTernaryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::ArrayFind:
		{				
			static const int32 Alignment = FRigVMQuaternaryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::ArrayIterator:
		{				
			static const int32 Alignment = FRigVMSenaryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::EndBlock:
		{
			static const int32 Alignment = FRigVMBaseOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::InvokeEntry:
		{
			static const int32 Alignment = FRigVMInvokeEntryOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::JumpToBranch:
		{
			static const int32 Alignment = FRigVMJumpToBranchOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::RunInstructions:
		{
			static const int32 Alignment = FRigVMRunInstructionsOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::SetupTraits:
		{
			static const int32 Alignment = FRigVMSetupTraitsOp::StaticStruct()->GetCppStructOps()->GetAlignment();
			return Alignment;
		}
		case ERigVMOpCode::Invalid:
		{
			ensure(false);
			return 0;
		}
		default:
			checkNoEntry();
			break;
	}
	return 0;
}

int32 FRigVMByteCode::GetOperandAlignment()
{
	static const int32 OperandAlignment = FRigVMOperand::StaticStruct()->GetCppStructOps()->GetAlignment();
	return OperandAlignment;
}

void FRigVMByteCode::AlignByteCode()
{
	if (bByteCodeIsAligned)
	{
		return;
	}

	if (ByteCode.Num() == 0)
	{
		return;
	}

	const FRigVMInstructionArray Instructions(*this, false);
	int32 BytesToReserve = ByteCode.Num();

	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		BytesToReserve += GetOpAlignment(Instruction.OpCode);

		if (IsCallableOpCode(Instruction.OpCode))
		{
			BytesToReserve += GetOperandAlignment();
		}
	}

	TArray<uint8> AlignedByteCode;
	AlignedByteCode.Reserve(BytesToReserve);
	AlignedByteCode.AddZeroed(ByteCode.Num());

	int32 ShiftedBytes = 0;
	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		int32 OriginalByteCodeIndex = Instruction.ByteCodeIndex;
		int32 AlignedByteCodeIndex = OriginalByteCodeIndex + ShiftedBytes;
		int32 OpAlignment = GetOpAlignment(Instruction.OpCode);

		if (OpAlignment > 0)
		{
			while (!IsAligned(&AlignedByteCode[AlignedByteCodeIndex], OpAlignment))
			{
				AlignedByteCode[AlignedByteCodeIndex] = static_cast<uint8>(Instruction.OpCode);
				AlignedByteCodeIndex++;
				ShiftedBytes++;
				AlignedByteCode.AddZeroed(1);
			}
		}

		int32 NumBytesToCopy = GetOpNumBytesAt(OriginalByteCodeIndex, false);
		for (int32 ByteIndex = 0; ByteIndex < NumBytesToCopy; ByteIndex++)
		{
			AlignedByteCode[AlignedByteCodeIndex + ByteIndex] = ByteCode[OriginalByteCodeIndex + ByteIndex];
		}

		if (IsCallableOpCode(Instruction.OpCode))
		{
			AlignedByteCodeIndex += NumBytesToCopy;

			static const int32 OperandAlignment = GetOperandAlignment();
			if (OperandAlignment > 0)
			{
				while (!IsAligned(&AlignedByteCode[AlignedByteCodeIndex], OperandAlignment))
				{
					AlignedByteCodeIndex++;
					ShiftedBytes++;
					AlignedByteCode.AddZeroed(1);
				}
			}

			const int32 OperandCount = GetOpAt<FRigVMInvokeCallableOp>(OriginalByteCodeIndex).GetOperandCount();
			OriginalByteCodeIndex += NumBytesToCopy;
			NumBytesToCopy = sizeof(FRigVMOperand) * OperandCount;

			for (int32 ByteIndex = 0; ByteIndex < NumBytesToCopy; ByteIndex++)
			{
				AlignedByteCode[AlignedByteCodeIndex + ByteIndex] = ByteCode[OriginalByteCodeIndex + ByteIndex];
			}
		}
	}

	Swap(ByteCode, AlignedByteCode);
	bByteCodeIsAligned = true;
}

#if WITH_EDITOR

UObject* FRigVMByteCode::GetSubjectForInstruction(int32 InInstructionIndex) const
{
	if (SubjectPerInstruction.IsValidIndex(InInstructionIndex))
	{
		if (SubjectPerInstruction[InInstructionIndex].IsValid())
		{
			return SubjectPerInstruction[InInstructionIndex].Get();
		}
	}
	return nullptr;
}

int32 FRigVMByteCode::GetFirstInstructionIndexForSubject(UObject* InSubject) const
{
	const TArray<int32>& InstructionIndices = GetAllInstructionIndicesForSubject(InSubject);
	if (InstructionIndices.Num() > 0)
	{
		return InstructionIndices[0];
	}
	return INDEX_NONE;
}

const TArray<int32>& FRigVMByteCode::GetAllInstructionIndicesForSubject(UObject* InSubject) const
{
	if (const TArray<int32>* InstructionIndices = SubjectToInstructions.Find(InSubject))
	{
		return *InstructionIndices;
	}
	return EmptyInstructionIndices;
}

FString FRigVMByteCode::GetCallPathForInstruction(int32 InInstructionIndex) const
{
	if (CallPathPerInstruction.IsValidIndex(InInstructionIndex))
	{
		return CallPathPerInstruction[InInstructionIndex];
	}
	return FString();
}

int32 FRigVMByteCode::GetFirstInstructionIndexForCallPath(const FString& InCallPath, bool bStartsWith, bool bEndsWith) const
{
	const TArray<int32> InstructionIndices = GetAllInstructionIndicesForCallPath(InCallPath, bStartsWith, bEndsWith);
	if (InstructionIndices.Num() > 0)
	{
		return InstructionIndices[0];
	}
	return INDEX_NONE;
}

TArray<int32> FRigVMByteCode::GetAllInstructionIndicesForCallPath(const FString& InCallPath, bool bStartsWith, bool bEndsWith) const
{
	if(InCallPath.IsEmpty())
	{
		return EmptyInstructionIndices;
	}

	TArray<int32> MatchedInstructions;
	if (const TArray<int32>* InstructionIndices = CallPathToInstructions.Find(InCallPath))
	{
		MatchedInstructions.Append(*InstructionIndices);
	}
	
	if(bStartsWith || bEndsWith)
	{
		const FString CallPathStart = FString::Printf(TEXT("%s|"), *InCallPath);
		const FString CallPathEnd = FString::Printf(TEXT("|%s"), *InCallPath);
		for(int32 InstructionIndex = 0; InstructionIndex < CallPathPerInstruction.Num(); InstructionIndex++)
		{
			if(bStartsWith)
			{
				if(CallPathPerInstruction[InstructionIndex].StartsWith(CallPathStart))
				{
					MatchedInstructions.Add(InstructionIndex);
				}
			}
			else if(bEndsWith)
			{
				if(CallPathPerInstruction[InstructionIndex].EndsWith(CallPathEnd))
				{
					MatchedInstructions.Add(InstructionIndex);
				}
			}
		}
	}

	return MatchedInstructions;
}

int32 FRigVMByteCode::GetFirstInstructionIndexForCallstack(const TArray<TWeakObjectPtr<UObject>>& InCallstack) const
{
	const TArray<int32>& InstructionIndices = GetAllInstructionIndicesForCallstack(InCallstack);
	if (InstructionIndices.Num() > 0)
	{
		return InstructionIndices[0];
	}
	return INDEX_NONE;
}

const TArray<int32>& FRigVMByteCode::GetAllInstructionIndicesForCallstack(const TArray<TWeakObjectPtr<UObject>>& InCallstack) const
{
	if(InCallstack.IsEmpty())
	{
		return EmptyInstructionIndices;
	}

	const uint32 Hash = GetCallstackHash(InCallstack);
	if(const TArray<int32>* Instructions = CallstackHashToInstructions.Find(Hash))
	{
		return *Instructions;
	}
	
	return EmptyInstructionIndices;
}

void FRigVMByteCode::SetSubject(int32 InInstructionIndex, const FString& InCallPath, const TArray<TWeakObjectPtr<UObject>>& InCallstack)
{
	TWeakObjectPtr<UObject> Subject = InCallstack.Last();
	if (SubjectPerInstruction.Num() <= InInstructionIndex)
	{
		SubjectPerInstruction.AddZeroed(1 + InInstructionIndex - SubjectPerInstruction.Num());
	}
	SubjectPerInstruction[InInstructionIndex] = Subject;
	SubjectToInstructions.FindOrAdd(Subject).AddUnique(InInstructionIndex);

	if (CallPathPerInstruction.Num() <= InInstructionIndex)
	{
		CallPathPerInstruction.AddZeroed(1 + InInstructionIndex - CallPathPerInstruction.Num());
	}
	CallPathPerInstruction[InInstructionIndex] = InCallPath;
	CallPathToInstructions.FindOrAdd(InCallPath).AddUnique(InInstructionIndex);

	if (CallstackPerInstruction.Num() <= InInstructionIndex)
	{
		CallstackPerInstruction.AddZeroed(1 + InInstructionIndex - CallstackPerInstruction.Num());
	}
	CallstackPerInstruction[InInstructionIndex] = InCallstack;

	if (CallstackHashPerInstruction.Num() <= InInstructionIndex)
	{
		CallstackHashPerInstruction.AddZeroed(1 + InInstructionIndex - CallstackHashPerInstruction.Num());
	}
	CallstackHashPerInstruction[InInstructionIndex] = GetCallstackHash(InCallstack);

	for(int32 CallstackLength = InCallstack.Num(); CallstackLength > 0; CallstackLength--)
	{
		TWeakObjectPtr<UObject> const* DataPtr = &InCallstack[InCallstack.Num() - CallstackLength];
		TArrayView<TWeakObjectPtr<UObject> const> View(DataPtr, CallstackLength);
		uint32 Hash = GetCallstackHash(View);
		CallstackHashToInstructions.FindOrAdd(Hash).AddUnique(InInstructionIndex);
	}
}

void FRigVMByteCode::AddInstructionForSubject(UObject* InSubject, int32 InInstructionIndex)
{
	check(InSubject);
	TWeakObjectPtr<UObject> WeakSubject(InSubject);
	SubjectToInstructions.FindOrAdd(WeakSubject).AddUnique(InInstructionIndex);
}

const TArray<TWeakObjectPtr<UObject>>* FRigVMByteCode::GetCallstackForInstruction(int32 InInstructionIndex) const
{
	if (CallstackPerInstruction.IsValidIndex(InInstructionIndex))
	{
		return &CallstackPerInstruction[InInstructionIndex];
	}
	return nullptr;
}

uint32 FRigVMByteCode::GetCallstackHashForInstruction(int32 InInstructionIndex) const
{
	if (CallstackHashPerInstruction.IsValidIndex(InInstructionIndex))
	{
		return CallstackHashPerInstruction[InInstructionIndex];
	}
	return 0;
}

uint32 FRigVMByteCode::GetCallstackHash(const TArray<TWeakObjectPtr<UObject>>& InCallstack)
{
	TWeakObjectPtr<UObject> const * DataPtr = nullptr;
	if(InCallstack.Num() > 0)
	{
		DataPtr = &InCallstack[0];
	}
	TArrayView<TWeakObjectPtr<UObject> const> View(DataPtr, InCallstack.Num());
	return GetCallstackHash(View);
}

uint32 FRigVMByteCode::GetCallstackHash(const TArrayView<TWeakObjectPtr<UObject> const>& InCallstack)
{
	uint32 Hash = GetTypeHash(InCallstack.Num());
	for(const TWeakObjectPtr<UObject> Object : InCallstack)
	{
		Hash = HashCombine(Hash, GetTypeHash(Object));
	}
	return Hash;
}

void FRigVMByteCode::SetOperandsForInstruction(const FRigVMInstruction& InInstruction, const FRigVMOperandArray& InputOperands,
	const FRigVMOperandArray& OutputOperands)
{
	const FRigVMOperandArray OperandsForInstruction = GetOperandsForOp(InInstruction);
	const int32 InstructionIndex = InInstruction.Index;
	check(InInstruction.Index != INDEX_NONE && InInstruction.Index < NumInstructions);
	
	if (InputOperandsPerInstruction.Num() <= InstructionIndex)
	{
		InputOperandsPerInstruction.AddZeroed(1 + InstructionIndex - InputOperandsPerInstruction.Num());
	}
	InputOperandsPerInstruction[InstructionIndex].Reset();
	InputOperandsPerInstruction[InstructionIndex].Reserve(InputOperands.Num());

	for(int32 OperandIndex = 0; OperandIndex < InputOperands.Num(); OperandIndex++)
	{
		// we are only interested in memory here which can change over time
		if(InputOperands[OperandIndex].GetMemoryType() == ERigVMMemoryType::Literal)
		{
			continue;;
		}
		check(OperandsForInstruction.Contains(InputOperands[OperandIndex]));
		InputOperandsPerInstruction[InstructionIndex].Add(InputOperands[OperandIndex]);
	}

	if (OutputOperandsPerInstruction.Num() <= InstructionIndex)
	{
		OutputOperandsPerInstruction.AddZeroed(1 + InstructionIndex - OutputOperandsPerInstruction.Num());
	}
	OutputOperandsPerInstruction[InstructionIndex].Reset();
	OutputOperandsPerInstruction[InstructionIndex].Reserve(OutputOperands.Num());

	for(int32 OperandIndex = 0; OperandIndex < OutputOperands.Num(); OperandIndex++)
	{
		// we are only interested in memory here which can change over time
		if(OutputOperands[OperandIndex].GetMemoryType() == ERigVMMemoryType::Literal)
		{
			continue;;
		}
		check(OperandsForInstruction.Contains(OutputOperands[OperandIndex]));
		OutputOperandsPerInstruction[InstructionIndex].Add(OutputOperands[OperandIndex]);
	}
}

#endif

TMap<int32, TArray<FRigVMTraitScope>> FRigVMByteCode::GetTraits(
	FRigVMMemoryStorageStruct& InLiteralMemory,
	FRigVMMemoryStorageStruct& InWorkMemory,
	const UScriptStruct* InScriptStruct) const
{
	TMap<int32, TArray<FRigVMTraitScope>> Result;
	const FRigVMInstructionArray Instructions = GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if(Instruction.OpCode == ERigVMOpCode::SetupTraits)
		{
			const TArray<FRigVMTraitScope> Traits = GetTraitsForInstruction(Instruction, InLiteralMemory, InWorkMemory, InScriptStruct);
			if(!Traits.IsEmpty())
			{
				Result.Add(InstructionIndex, Traits);
			}
		}
	}
	return Result;
}

TMap<int32, TArray<FRigVMTraitScope>> FRigVMByteCode::GetTraits(
	FRigVMMemoryStorageStruct& InLiteralMemory,
	FRigVMMemoryStorageStruct& InWorkMemory,
	TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles,
	const UScriptStruct* InScriptStruct) const
{
	// Count memory handles we will add and reserve to ensure we don't reallocate and invalidate
	// handle views in returned FRigVMTraitScopes
	int32 NumAdditionalMemoryHandles = 0;
	const FRigVMInstructionArray Instructions = GetInstructions();
	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if(Instruction.OpCode == ERigVMOpCode::SetupTraits)
		{
			TArray<FRigVMMemoryHandle> AdditionalMemoryHandles;
			const TArray<FRigVMTraitScope> Traits = GetTraitsForInstruction(Instruction, InLiteralMemory, InWorkMemory, AdditionalMemoryHandles, InScriptStruct);
			for(const FRigVMTraitScope& Trait : Traits)
			{
				NumAdditionalMemoryHandles += Trait.GetAdditionalMemoryHandles().Num();
			}
		}
	}

	OutAdditionalMemoryHandles.Reserve(NumAdditionalMemoryHandles);

	TMap<int32, TArray<FRigVMTraitScope>> Result;
	for(int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if(Instruction.OpCode == ERigVMOpCode::SetupTraits)
		{
			const TArray<FRigVMTraitScope> Traits = GetTraitsForInstruction(Instruction, InLiteralMemory, InWorkMemory, OutAdditionalMemoryHandles, InScriptStruct);
			if(!Traits.IsEmpty())
			{
				Result.Add(InstructionIndex, Traits);
			}
		}
	}
	return Result;
}

TArray<FRigVMTraitScope> FRigVMByteCode::GetTraitsForInstruction(
	const FRigVMInstruction& InInstruction,
	FRigVMMemoryStorageStruct& InLiteralMemory,
	FRigVMMemoryStorageStruct& InWorkMemory,
	const UScriptStruct* InScriptStruct) const
{
	TArray<FRigVMTraitScope> Traits;

	if(InInstruction.OpCode == ERigVMOpCode::SetupTraits)
	{
		if(InScriptStruct == nullptr)
		{
			InScriptStruct = FRigVMTrait::StaticStruct();
		}

		const FRigVMSetupTraitsOp& Op = GetOpAt<FRigVMSetupTraitsOp>(InInstruction);
		check(Op.Arg.GetMemoryType() == ERigVMMemoryType::Literal);

		if(InLiteralMemory.GetProperties().IsValidIndex(Op.Arg.GetRegisterIndex()))
		{
			if(const FArrayProperty* TraitIndicesProperty = CastField<FArrayProperty>(InLiteralMemory.GetProperty(Op.Arg.GetRegisterIndex())))
			{
				if(TraitIndicesProperty->Inner->IsA<FIntProperty>())
				{
					const TArray<int32>& TraitIndices = *InLiteralMemory.GetData<TArray<int32>>(TraitIndicesProperty);
					for(const int32 TraitIndex : TraitIndices)
					{
						const FProperty* Property = InWorkMemory.GetProperties()[TraitIndex];
						if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
						{
							if(StructProperty->Struct && StructProperty->Struct->IsChildOf(InScriptStruct))
							{
								Traits.Emplace(
									InWorkMemory.GetData<FRigVMTrait>(StructProperty),
									Cast<UScriptStruct>(StructProperty->Struct));
							}
						}
					}
				}
			}
		}
	}

	return Traits;
}

TArray<FRigVMTraitScope> FRigVMByteCode::GetTraitsForInstruction(
	const FRigVMInstruction& InInstruction,
	FRigVMMemoryStorageStruct& InLiteralMemory,
	FRigVMMemoryStorageStruct& InWorkMemory,
	TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles,
	const UScriptStruct* InScriptStruct) const
{
	TArray<FRigVMTraitScope> Traits;

	if(InInstruction.OpCode == ERigVMOpCode::SetupTraits)
	{
		if(InScriptStruct == nullptr)
		{
			InScriptStruct = FRigVMTrait::StaticStruct();
		}

		const FRigVMSetupTraitsOp& Op = GetOpAt<FRigVMSetupTraitsOp>(InInstruction);
		check(Op.Arg.GetMemoryType() == ERigVMMemoryType::Literal);

		if(InLiteralMemory.GetProperties().IsValidIndex(Op.Arg.GetRegisterIndex()))
		{
			if(const FArrayProperty* TraitIndicesProperty = CastField<FArrayProperty>(InLiteralMemory.GetProperty(Op.Arg.GetRegisterIndex())))
			{
				if(TraitIndicesProperty->Inner->IsA<FIntProperty>())
				{
					const TArray<int32>& TraitIndices = *InLiteralMemory.GetData<TArray<int32>>(TraitIndicesProperty);

					// Count memory handles we will add and reserve to ensure we don't reallocate and invalidate
					// handle views in returned FRigVMTraitScopes
					int32 NumAdditionalMemoryHandles = 0;
					for(const int32 TraitIndex : TraitIndices)
					{
						const FProperty* Property = InWorkMemory.GetProperties()[TraitIndex];
						const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
						if(Property && !(StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FRigVMTrait::StaticStruct())))
						{
							NumAdditionalMemoryHandles++;
						}
					}

					OutAdditionalMemoryHandles.Reserve(NumAdditionalMemoryHandles);

					int32 AdditionalStartIndex = OutAdditionalMemoryHandles.Num();
					int32 AdditionalNum = 0;

					for(const int32 TraitIndex : TraitIndices)
					{
						const FProperty* Property = InWorkMemory.GetProperties()[TraitIndex];
						const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
						if(StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(InScriptStruct))
						{
							TConstArrayView<FRigVMMemoryHandle> AdditionalMemoryHandles;
							if(AdditionalStartIndex != INDEX_NONE)
							{
								AdditionalMemoryHandles = TConstArrayView<FRigVMMemoryHandle>(&OutAdditionalMemoryHandles[AdditionalStartIndex], AdditionalNum);
								AdditionalStartIndex = INDEX_NONE;
							}

							Traits.Emplace(
								InWorkMemory.GetData<FRigVMTrait>(StructProperty),
								Cast<UScriptStruct>(StructProperty->Struct),
								AdditionalMemoryHandles);
						}
						else if(Property && !(StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FRigVMTrait::StaticStruct())))
						{
							if(AdditionalStartIndex == INDEX_NONE)
							{
								AdditionalStartIndex = OutAdditionalMemoryHandles.Num();
								AdditionalNum = 0;
							}
							OutAdditionalMemoryHandles.Emplace(InWorkMemory.GetHandle(TraitIndex));
							AdditionalNum++;
						}
					}
				}
			}
		}
	}

	return Traits;
}

const FRigVMBranchInfo* FRigVMByteCode::GetBranchInfo(const FRigVMBranchInfoKey& InBranchInfoKey) const
{
	if(BranchInfos.IsEmpty())
	{
		return nullptr;
	}

	if(BranchInfoLookup.IsEmpty())
	{
		for(const FRigVMBranchInfo& BranchInfo : BranchInfos)
		{
			const FRigVMBranchInfoKey Key(BranchInfo.InstructionIndex, BranchInfo.ArgumentIndex, BranchInfo.Label);
			BranchInfoLookup.FindOrAdd(Key) = &BranchInfo;
		}
	}
	
	if(const FRigVMBranchInfo** BranchInfoPtr = BranchInfoLookup.Find(InBranchInfoKey))
	{
		return *BranchInfoPtr;
	}

	return nullptr;
}
