// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMDebugInfo.h"
#include "RigVMCore/RigVMExecuteContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDebugInfo)

bool FRigVMDebugInfo::ShouldStop(const FRigVMExtendedExecuteContext& Context, int32 CurrentInstruction)
{
	const int32 CallstackCompare = CompareCallstackPath(Context, StepCondition.OriginCallstackPath);
	const int32 IterCompare = CompareIterationPath(Context, StepCondition.OriginIterationPath);

	// Update bHasPassedOrigin for stepping modes
	if (!StepCondition.bHasPassedOrigin)
	{
		if (StepCondition.Mode == ERigVMStepMode::Into
			|| StepCondition.Mode == ERigVMStepMode::Forward
			|| StepCondition.Mode == ERigVMStepMode::Out)
		{
			if (CallstackCompare == 0
				&& CurrentInstruction == StepCondition.OriginInstruction
				&& IterCompare == 0)
			{
				StepCondition.bHasPassedOrigin = true;
			}
		}
	}

	switch (StepCondition.Mode)
	{
	case ERigVMStepMode::None:
		return false;

	case ERigVMStepMode::Stopped:
		// Stopped state - halt if we're at the exact same location
		return CurrentInstruction == StepCondition.OriginInstruction
			&& CallstackCompare == 0
			&& IterCompare == 0;

	case ERigVMStepMode::Into:
		{
			if (!StepCondition.bHasPassedOrigin)
			{
				return false;
			}

			// Check if we've entered a callable (callstack is deeper)
			if (CallstackCompare < 0)
			{
				TransitionToStopped(Context, CurrentInstruction);
				return true;
			}

			// If we haven't entered a callable, check for forward progress at same callstack
			if (CallstackCompare == 0)
			{
				if (IterCompare > 0)
				{
					// Advanced in iteration or exited loop - stop
					TransitionToStopped(Context, CurrentInstruction);
					return true;
				}

				if (IterCompare == 0 && CurrentInstruction > StepCondition.OriginInstruction)
				{
					// Same iteration, advanced instruction - stop
					TransitionToStopped(Context, CurrentInstruction);
					return true;
				}
			}

			// Exited a callable (callstack is shallower) - stop at next instruction
			if (CallstackCompare > 0)
			{
				TransitionToStopped(Context, CurrentInstruction);
				return true;
			}

			return false;
		}

	case ERigVMStepMode::Forward:
		{
			if (!StepCondition.bHasPassedOrigin)
			{
				return false;
			}

			// Exited a callable (callstack is shallower) - stop
			if (CallstackCompare > 0)
			{
				TransitionToStopped(Context, CurrentInstruction);
				return true;
			}

			if (CallstackCompare == 0)
			{
				if (IterCompare > 0)
				{
					// Advanced in iteration or exited loop - stop
					TransitionToStopped(Context, CurrentInstruction);
					return true;
				}

				if (IterCompare == 0 && CurrentInstruction > StepCondition.OriginInstruction)
				{
					// Same iteration, advanced instruction - stop
					TransitionToStopped(Context, CurrentInstruction);
					return true;
				}
			}

			return false;
		}

	case ERigVMStepMode::Out:
		{
			if (!StepCondition.bHasPassedOrigin)
			{
				return false;
			}

			// We've passed the origin, check if we've exited to a shallower callstack
			if (CallstackCompare > 0)
			{
				TransitionToStopped(Context, CurrentInstruction);
				return true;
			}

			return false;
		}

	case ERigVMStepMode::RunTo:
		if (CurrentInstruction == StepCondition.TargetInstruction)
		{
			TransitionToStopped(Context, CurrentInstruction);
			return true;
		}
		return false;
	}

	return false;
}

void FRigVMDebugInfo::TransitionToStopped(const FRigVMExtendedExecuteContext& Context, int32 CurrentInstruction)
{
	UE_LOGF(LogRigVM, Display, "<-- Step Condition %ls", *GetStepCondition().ToString());

	StepCondition.Mode = ERigVMStepMode::Stopped;
	StepCondition.OriginInstruction = CurrentInstruction;
	StepCondition.TargetInstruction = INDEX_NONE;

	// Capture current callstack and iteration paths
	CaptureCallstackPath(Context, StepCondition.OriginCallstackPath);
	CaptureIterationPath(Context, StepCondition.OriginIterationPath);

	UE_LOGF(LogRigVM, Display, "--> Step Condition %ls", *GetStepCondition().ToString());
}

void FRigVMDebugInfo::CaptureCallstackPath(const FRigVMExtendedExecuteContext& Context, TArray<int32, TInlineAllocator<16>>& OutPath)
{
	OutPath.Reset();
	const FRigVMExecuteCallstack& Callstack = Context.GetCallstack();
	for (int32 i = 0; i < Callstack.Num(); i++)
	{
		OutPath.Add(Callstack[i].InstructionIndex);
	}
}

int32 FRigVMDebugInfo::CompareCallstackPath(const FRigVMExtendedExecuteContext& Context, const TArray<int32, TInlineAllocator<16>>& InPath)
{
	const FRigVMExecuteCallstack& Callstack = Context.GetCallstack();
	const int32 MinLen = FMath::Min(Callstack.Num(), InPath.Num());

	// Compare common prefix
	for (int32 i = 0; i < MinLen; i++)
	{
		const int32 CurrentInstr = Callstack[i].InstructionIndex;
		const int32 OriginInstr = InPath[i];

		if (CurrentInstr > OriginInstr)
			return +1;  // Current is past origin at this level
		if (CurrentInstr < OriginInstr)
			return -1;  // Current is before origin at this level
	}

	// Prefixes match - compare by depth
	if (Callstack.Num() > InPath.Num())
		return -1;  // Current is deeper (entered a callable)
	if (Callstack.Num() < InPath.Num())
		return +1;  // Current is shallower (exited a callable)

	return 0;  // Exact match
}

void FRigVMDebugInfo::CaptureIterationPath(const FRigVMExtendedExecuteContext& Context, TArray<int32, TInlineAllocator<16>>& OutPath)
{
	OutPath.Reset();
	for (const FRigVMSlice& Slice : Context.Frame->Slices)
	{
		if (Slice.IsLoop())
		{
			OutPath.Add(Slice.GetRelativeIndex());
		}
	}
}

int32 FRigVMDebugInfo::CompareIterationPath(const FRigVMExtendedExecuteContext& Context, const TArray<int32, TInlineAllocator<16>>& InPath)
{
	int32 PathIndex = 0;
	for (const FRigVMSlice& Slice : Context.Frame->Slices)
	{
		if (Slice.IsLoop())
		{
			if (PathIndex >= InPath.Num())
			{
				// Origin path is shorter - we've compared all we can
				// Current is deeper in loops, but at same point up to origin's depth
				break;
			}

			const int32 CurrentIter = Slice.GetRelativeIndex();
			const int32 OriginIter = InPath[PathIndex];

			if (CurrentIter > OriginIter)
				return +1;  // Advanced in this loop
			if (CurrentIter < OriginIter)
				return -1;  // Before origin in this loop

			PathIndex++;
		}
	}

	// If we get here, all compared elements matched
	// PathIndex tells us how many loops we compared

	if (PathIndex < InPath.Num())
	{
		// Current has fewer loops than origin - we've exited a loop scope
		return +1;
	}

	return 0;  // Same path (up to min length)
}
