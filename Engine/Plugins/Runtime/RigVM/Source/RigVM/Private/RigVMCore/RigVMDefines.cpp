// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMDefines.h"

#if UE_RIGVM_ARCHIVETRACE_ENABLE

#include "RigVMModule.h"

TMap<FArchive*, TSharedPtr<FRigVMArchiveTrace>> FRigVMArchiveTrace::ActiveTraces;
FCriticalSection FRigVMArchiveTrace::AddRemoveTracesMutex;

FRigVMArchiveTrace* FRigVMArchiveTrace::AddRefTrace(FArchive* InArchive)
{
	const FScopeLock Lock(&AddRemoveTracesMutex);
	if(const TSharedPtr<FRigVMArchiveTrace>* ExistingTrace = ActiveTraces.Find(InArchive))
	{
		ExistingTrace->Get()->Counter++;
		return ExistingTrace->Get();
	}

	const TSharedPtr<FRigVMArchiveTrace> NewTrace = MakeShared<FRigVMArchiveTrace>();
	ActiveTraces.Add(InArchive, NewTrace);
	NewTrace->Archive = InArchive;
	return NewTrace.Get();
}

void FRigVMArchiveTrace::DecRefTrace(FRigVMArchiveTrace* InTrace)
{
	InTrace->Counter--;
	if(InTrace->Counter == 0)
	{
		const FScopeLock Lock(&AddRemoveTracesMutex);
		verify(ActiveTraces.Remove(InTrace->Archive) == 1);
	}
}

FRigVMArchiveTraceBracket::FRigVMArchiveTraceBracket(FArchive& InArchive, const FString& InScope)
: Trace(nullptr)
, Indentation(0)
, ArchivePos(0)
, LastArchivePos(0)
, bEnabled(true)
{
	Trace = FRigVMArchiveTrace::AddRefTrace(&InArchive);
	Indentation = Trace->Counter - 1;

	bEnabled = InArchive.IsSaving() && !InArchive.IsObjectReferenceCollector() && !InArchive.IsTransacting() && InArchive.IsPersistent();

	if(bEnabled)
	{
		ArchivePos = InArchive.GetArchiveState().Tell();
		LastArchivePos = ArchivePos;
		
		ArchiveName = InArchive.GetArchiveName();
		ArchiveWhiteSpace = WhiteSpace(Indentation * 2);
		ArchivePrefix = TEXT("  ") + ArchiveWhiteSpace + InScope;

		const FString ArchiveOffset = ArchiveOffsetToString(ArchivePos);
		static const FString EntrySize = WhiteSpace(14);
		UE_LOGF(LogRigVM, Display, "%ls %ls%ls %ls%ls", *ArchiveName, *ArchiveOffset, *EntrySize, *ArchiveWhiteSpace, *InScope);
	}
}

FRigVMArchiveTraceBracket::~FRigVMArchiveTraceBracket()
{
	FRigVMArchiveTrace::DecRefTrace(Trace);
}

void FRigVMArchiveTraceBracket::AddEntry(FArchive& InArchive, const FString& InScope)
{
	if(bEnabled)
	{
		ArchivePos = InArchive.GetArchiveState().Tell();
		const FString ArchiveOffset = FRigVMArchiveTraceBracket::ArchiveOffsetToString(LastArchivePos);
		const FString EntrySize = FRigVMArchiveTraceBracket::ArchiveOffsetToString(ArchivePos - LastArchivePos);
		UE_LOGF(LogRigVM, Display, "%ls %ls,%ls %ls %ls", *ArchiveName, *ArchiveOffset, *EntrySize, *ArchivePrefix, *InScope);
		Swap(ArchivePos, LastArchivePos);
	}
}

FString FRigVMArchiveTraceBracket::WhiteSpace(int32 InCount)
{
	if(InCount <= 0)
	{
		return FString();
	}
	return FString(TEXT("\t")).ConvertTabsToSpaces(InCount);
}

FString FRigVMArchiveTraceBracket::ArchiveOffsetToString(int64 InOffset)
{
	const FString OffsetStr = FString::FromInt(InOffset);
	FString FormattedOffsetStr = WhiteSpace(FMath::Max(11, int32(OffsetStr.Len() / 3) * 4 + 3)); 
	int32 Input = OffsetStr.Len() - 1;
	int32 Output = FormattedOffsetStr.Len() - 1;
	int32 CharsLeft = 3;
	while(Input >= 0)
	{
		FormattedOffsetStr[Output] = OffsetStr[Input];
		Input--;
		Output--;
		CharsLeft--;

		if(CharsLeft == 0 && Input >= 0)
		{
			FormattedOffsetStr[Output] = TCHAR('.');
			Output--;
			CharsLeft = 3;
		}
	}
	static const FString Prefix = TEXT("[");
	static const FString Suffix = TEXT("]");
	return Prefix + FormattedOffsetStr + Suffix;
}

#endif
