// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatchFactory)

FCriticalSection FRigVMDispatchFactory::GetTemplateMutex;

FName FRigVMDispatchFactory::GetFactoryName() const
{
#if WITH_EDITOR
	// for unit testing we create factories temporarily and they don't
	// have the factory name / nor factoryname string set
	if (FactoryName.IsNone())
	{
		FactoryName = GetFactoryName(GetScriptStruct());
	}
#endif
	return FactoryName;
}

FName FRigVMDispatchFactory::GetFactoryName(const UScriptStruct* InFactoryStruct)
{
	check(InFactoryStruct);
	check(InFactoryStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()));
	return *(DispatchPrefix + InFactoryStruct->GetName());
}

#if WITH_EDITOR

FName FRigVMDispatchFactory::GetNextAggregateName(const FName& InLastAggregatePinName) const
{
	return FRigVMStruct().GetNextAggregateName(InLastAggregatePinName);
}

FName FRigVMDispatchFactory::GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const
{
	if(InArgumentName == FRigVMStruct::ExecuteContextName)
	{
		return FRigVMStruct::ExecuteName;
	}
	return NAME_None;
}

FString FRigVMDispatchFactory::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == FRigVMStruct::ControlFlowBlockToRunName &&
		InMetaDataKey == FRigVMStruct::SingletonMetaName)
	{
		return TrueString;
	}
	return FString();
}

FLinearColor FRigVMDispatchFactory::GetNodeColor() const
{
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		FString NodeColor;
		if (ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::NodeColorMetaName, &NodeColor))
		{
			return FRigVMTemplate::GetColorFromMetadata(NodeColor);
		}
	}
	return FLinearColor::White;
}

FText FRigVMDispatchFactory::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return GetScriptStruct()->GetToolTipText();
}

FString FRigVMDispatchFactory::GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::Bool)
	{
		static const FString FalseString = TEXT("False");
		return FalseString;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::Float || InTypeIndex == RigVMTypeUtils::TypeIndex::Double)
	{
		static const FString ZeroString = TEXT("0.000000");
		return ZeroString;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		static const FString ZeroString = TEXT("0");
		return ZeroString;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return FName(NAME_None).ToString();
	}

	FRigVMRegistry_RWLock::FConditionalReadScopeLock Lock(FRigVMRegistry_RWLock::Get());
	FRigVMRegistry_NoLock& Registry = Lock.GetRegistry();
	if(Registry.IsArrayType_NoLock(InTypeIndex))
	{
		static const FString EmptyArrayString = TEXT("()");
		return EmptyArrayString;
	}
	
	if (Registry.IsGlobalRegistry())
	{
		// GetTypeIndex_NoLock checks are expensive, so we can use a cached type index for the global registry here
		static const TRigVMTypeIndex GlobalVector2DIndex = Registry.GetTypeIndex_NoLock<FVector2D>();
		if(InTypeIndex == GlobalVector2DIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FVector2D::ZeroVector);
			return DefaultValueString;
		}
		static const TRigVMTypeIndex GlobalVectorIndex = Registry.GetTypeIndex_NoLock<FVector>();
		if(InTypeIndex == GlobalVectorIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FVector::ZeroVector); 
			return DefaultValueString;
		}
		static const TRigVMTypeIndex GlobalRotatorIndex = Registry.GetTypeIndex_NoLock<FRotator>();
		if(InTypeIndex == GlobalRotatorIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FRotator::ZeroRotator); 
			return DefaultValueString;
		}
		static const TRigVMTypeIndex GlobalQuatIndex = Registry.GetTypeIndex_NoLock<FQuat>();
		if(InTypeIndex == GlobalQuatIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FQuat::Identity); 
			return DefaultValueString;
		}
		static const TRigVMTypeIndex GlobalTransformIndex = Registry.GetTypeIndex_NoLock<FTransform>();
		if(InTypeIndex == GlobalTransformIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FTransform::Identity); 
			return DefaultValueString;
		}
		static const TRigVMTypeIndex GlobalLinearColorIndex = Registry.GetTypeIndex_NoLock<FLinearColor>();
		if(InTypeIndex == GlobalLinearColorIndex)
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FLinearColor::White); 
			return DefaultValueString;
		}
	}
	else
	{
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FVector2D>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FVector2D::ZeroVector);
			return DefaultValueString;
		}
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FVector>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FVector::ZeroVector); 
			return DefaultValueString;
		}
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FRotator>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FRotator::ZeroRotator); 
			return DefaultValueString;
		}
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FQuat>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FQuat::Identity); 
			return DefaultValueString;
		}
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FTransform>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FTransform::Identity); 
			return DefaultValueString;
		}
		if(InTypeIndex == Registry.GetTypeIndex_NoLock<FLinearColor>())
		{
			static FString DefaultValueString = GetDefaultValueForStruct(FLinearColor::White); 
			return DefaultValueString;
		}
	}
	return FString();
}

FString FRigVMDispatchFactory::GetCategory() const
{
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		FString Category;
		if (ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &Category))
		{
			return Category;
		}
	}
	return FString();
}

FString FRigVMDispatchFactory::GetKeywords() const
{
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		FString Keywords;
		if (ScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::KeywordsMetaName, &Keywords))
		{
			return Keywords;
		}
	}
	return FString();
}

bool FRigVMDispatchFactory::IsLazyInputArgument(const FName& InArgumentName) const
{
	return HasArgumentMetaData(InArgumentName, FRigVMStruct::ComputeLazilyMetaName);
}

#endif

FName FRigVMDispatchFactory::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	check(GetArgumentInfos(InRegistry).Num() == InTotalOperands);
	return GetArgumentInfos(InRegistry)[InOperandIndex].Name;
}

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMDispatchFactory::GetControlFlowBlocks(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const
{
	const TArray<FName>& Blocks = GetControlFlowBlocks_Impl(InContext, InTypeMap);
#if WITH_EDITOR
	FRigVMStruct::ValidateControlFlowBlocks(Blocks);
#endif
	return Blocks;
}
#endif

bool FRigVMDispatchFactory::SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const
{
	return InExecuteContextStruct->IsChildOf(GetExecuteContextStruct());
}

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMDispatchFactory::GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const
{
	static const TArray<FName> EmptyArray;
	return EmptyArray;
}
#endif

const TArray<FName>* FRigVMDispatchFactory::UpdateArgumentNameCache(int32 InNumberOperands, FRigVMRegistryHandle& InRegistry) const
{
	FScopeLock ArgumentNamesLock(ArgumentNamesMutex);
	return UpdateArgumentNameCache_NoLock(InNumberOperands, InRegistry);
}

const TArray<FName>* FRigVMDispatchFactory::UpdateArgumentNameCache_NoLock(int32 InNumberOperands, FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);			

	TSharedPtr<TArray<FName>>& ArgumentNames = ArgumentNamesMap.FindOrAdd(InNumberOperands);
	if(!ArgumentNames.IsValid())
	{
		ArgumentNames = MakeShareable(new TArray<FName>()); 
		ArgumentNames->Reserve(InNumberOperands);
		for(int32 OperandIndex = 0; OperandIndex < InNumberOperands; OperandIndex++)
		{
			ArgumentNames->Add(GetArgumentNameForOperandIndex(OperandIndex, InNumberOperands, InRegistry));
		}
	}
	return ArgumentNames.Get();
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatchFactory::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	static const TArray<FRigVMTemplateArgumentInfo> EmptyArguments;
	return EmptyArguments;
}

TArray<FRigVMExecuteArgument> FRigVMDispatchFactory::GetExecuteArguments_NoLock(const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	TArray<FRigVMExecuteArgument> Arguments = GetExecuteArguments_Impl(InContext);
	for(FRigVMExecuteArgument& Argument : Arguments)
	{
		if(Argument.TypeIndex != INDEX_NONE && InRegistry->IsArrayType_NoLock(Argument.TypeIndex))
		{
			Argument.TypeIndex = RigVMTypeUtils::TypeIndex::ExecuteArray;
		}
		else
		{
			Argument.TypeIndex = RigVMTypeUtils::TypeIndex::Execute;
		}
	}
	return Arguments;
}

const TArray<FRigVMExecuteArgument>& FRigVMDispatchFactory::GetExecuteArguments_Impl(
	const FRigVMDispatchContext& InContext) const
{
	static TArray<FRigVMExecuteArgument> EmptyArguments;
	return EmptyArguments;
}

FRigVMFunctionPtr FRigVMDispatchFactory::GetOrCreateDispatchFunction(const FRigVMTemplateTypeMap& InTypes) const
{
	return GetOrCreateDispatchFunction_NoLock(InTypes, FRigVMRegistryReadLock());
}

const FRigVMTemplate* FRigVMDispatchFactory::CreateTemplateForArgumentInfos_NoLock(const TArray<FRigVMTemplateArgumentInfo>& InArguments, FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	check(CachedTemplate == nullptr);

	const FName CopyOfFactoryName = GetFactoryName();
	FRigVMTemplateDelegates Delegates;
	Delegates.GetDispatchFactoryDelegate = FRigVMTemplate_GetDispatchFactoryDelegate::CreateLambda(
	[CopyOfFactoryName](const FRigVMRegistryHandle& InRegistry)
	{
		return InRegistry->FindDispatchFactory_NoLock(CopyOfFactoryName);
	});
	
	CachedTemplate = InRegistry->AddTemplateFromArguments_NoLock(GetFactoryName(), InArguments, Delegates);
	(void)UpdateArgumentNameCache_NoLock(InArguments.Num(), OwnerRegistry->ThisHandle);

	CachedArgumentInfos = InArguments;
	
	return CachedTemplate;
}

FRigVMFunctionPtr FRigVMDispatchFactory::GetOrCreateDispatchFunction_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	const FString PermutationName = GetPermutationNameImpl(InTypes, InRegistry);
	if(const FRigVMFunction* ExistingFunction = InRegistry->FindFunction_NoLock(*PermutationName))
	{
		return ExistingFunction->FunctionPtr;
	}
	
	return CreateDispatchFunction_NoLock(InTypes, InRegistry);
}

FRigVMFunctionPtr FRigVMDispatchFactory::CreateDispatchFunction_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	return GetDispatchFunctionImpl(InTypes, InRegistry);
}

TArray<FRigVMFunction> FRigVMDispatchFactory::CreateDispatchPredicates_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	return GetDispatchPredicatesImpl(InTypes, InRegistry);
}

FString FRigVMDispatchFactory::GetPermutationName(const FRigVMTemplateTypeMap& InTypes) const
{
	FRigVMRegistryReadLock ReadLock;
	return GetPermutationName_NoLock(InTypes, ReadLock);
}

FString FRigVMDispatchFactory::GetPermutationName_NoLock(const FRigVMTemplateTypeMap& InTypes, FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
#if WITH_EDITOR
	const TArray<FRigVMTemplateArgumentInfo>& Arguments = GetArgumentInfos(InRegistry);
	
	checkf(InTypes.Num() == Arguments.Num(), TEXT("Failed getting permutation names for '%s' "), *FactoryNameString);
	
	for(const FRigVMTemplateArgumentInfo& Argument : Arguments)
	{
		check(InTypes.Contains(Argument.Name));
	}
#endif
	return GetPermutationNameImpl(InTypes, InRegistry);
}

TArray<FRigVMTemplateArgumentInfo> FRigVMDispatchFactory::BuildArgumentListFromPrimaryArgument(const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FName& InPrimaryArgumentName, FRigVMRegistryHandle& InRegistry) const
{
	TArray<FRigVMTemplateArgumentInfo> NewInfos;
	
	const FRigVMTemplateArgumentInfo* PrimaryInfo = InInfos.FindByPredicate([InPrimaryArgumentName](const FRigVMTemplateArgumentInfo& Arg)
	{
		return Arg.Name == InPrimaryArgumentName;
	});

	if (!PrimaryInfo)
	{
		return NewInfos;
	}

	const int32 NumInfos = InInfos.Num();
	TArray< TArray<TRigVMTypeIndex> > TypeIndicesArray;
	TypeIndicesArray.SetNum(NumInfos);

	const FRigVMTemplateArgument PrimaryArgument = PrimaryInfo->GetArgument_NoLock(InRegistry);
	bool bFoundArg = true;
	PrimaryArgument.ForEachType([&](const TRigVMTypeIndex Type)
	{
		if (bFoundArg)
		{
			TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>> Permutations;
			GetPermutationsFromArgumentType(InPrimaryArgumentName, Type, Permutations, InRegistry);
			for (const FRigVMTemplateTypeMap& Permutation : Permutations)
			{
				for (int32 Index=0; Index < InInfos.Num(); ++Index)
				{
					if (const TRigVMTypeIndex* PermutationArg = Permutation.Find(InInfos[Index].Name))
					{
						TypeIndicesArray[Index].Add(*PermutationArg);
					}
					else
					{
						bFoundArg = false;
						break;
					}
				}
				if (!bFoundArg)
				{
					break;
				}
			}
		}
		return true;
	}, InRegistry);

	if (!bFoundArg)
	{
		return NewInfos;	
	}

	NewInfos.Reserve(NumInfos);
	for (int32 Index=0; Index < NumInfos; ++Index)
	{
		const FRigVMTemplateArgument Argument = InInfos[Index].GetArgument_NoLock(InRegistry);
		const TArray<TRigVMTypeIndex>& TypeIndices = TypeIndicesArray[Index];
		if (TypeIndices.IsEmpty())
		{
			NewInfos.Emplace(InInfos[Index].Name, Argument.Direction, Argument.TypeCategories, nullptr);
		}
		else
		{
			NewInfos.Emplace(InInfos[Index].Name, Argument.Direction, TypeIndices);
		}
	}
	
	return NewInfos;
}

FString FRigVMDispatchFactory::GetPermutationNameImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const FString TypePairStrings = FRigVMTemplate::GetStringFromArgumentTypes(InTypes, InRegistry);
	return RigVMStringUtils::JoinStrings(FactoryNameString, TypePairStrings, TEXT("::"));
}

bool FRigVMDispatchFactory::CopyProperty(const FProperty* InTargetProperty, uint8* InTargetPtr,
                                         const FProperty* InSourceProperty, const uint8* InSourcePtr)
{
	return URigVMMemoryStorage::CopyProperty(InTargetProperty, InTargetPtr, InSourceProperty, InSourcePtr);
}

const FRigVMTemplate* FRigVMDispatchFactory::GetTemplate() const
{
	FRigVMRegistryReadLock Lock;
	return GetTemplate_NoLock(Lock);
}

const FRigVMTemplate* FRigVMDispatchFactory::GetTemplate_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	// make sure to rely on the instance of this factory that's stored under the registry
	const FRigVMDispatchFactory* ThisFactory = InRegistry->FindDispatchFactory_NoLock(GetFactoryName());
	if(ThisFactory != this)
	{
		return ThisFactory->GetTemplate_NoLock(InRegistry);
	}

	check(InRegistry.GetRegistry() == OwnerRegistry);

	FScopeLock GetTemplateScopeLock(&GetTemplateMutex);
	
	if(CachedTemplate)
	{
		if (!CachedTemplate->IsValid())
		{
			return nullptr;
		}
		return CachedTemplate;
	}
	
	// we don't allow execute types on arguments	
	const TArray<FRigVMTemplateArgumentInfo>& Infos = GetArgumentInfos(InRegistry);
	for (const FRigVMTemplateArgumentInfo& Info : Infos)
	{
		const FRigVMTemplateArgument Argument = Info.GetArgument_NoLock(InRegistry);
		const int32 Index = Argument.IndexOfByPredicate([&](const TRigVMTypeIndex TypeIndex)
		{
			return InRegistry->IsExecuteType_NoLock(TypeIndex);
		}, InRegistry);
		
		if (Index != INDEX_NONE)
		{
			UE_LOGF(LogRigVM, Error, "Failed to add template for dispatch '%ls'. Argument '%ls' is an execute type.",
				*FactoryNameString, *Info.Name.ToString());
			return nullptr;			
		}
	}

	const FName CopyOfFactoryName = GetFactoryName();
	FRigVMTemplateDelegates Delegates;
	Delegates.GetDispatchFactoryDelegate = FRigVMTemplate_GetDispatchFactoryDelegate::CreateLambda(
	[CopyOfFactoryName](const FRigVMRegistryHandle& InRegistry)
	{
		return InRegistry->FindDispatchFactory_NoLock(CopyOfFactoryName);
	});

	CachedTemplate = InRegistry->AddTemplateFromArguments_NoLock(GetFactoryName(), Infos, Delegates); 
	return CachedTemplate;
}

FName FRigVMDispatchFactory::GetTemplateNotation() const
{
	FRigVMRegistryWriteLock Lock;
	return GetTemplateNotation_NoLock(Lock);
}

FName FRigVMDispatchFactory::GetTemplateNotation_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	if(const FRigVMTemplate* Template = GetTemplate_NoLock(InRegistry))
	{
		return Template->GetNotation();
	}
	return NAME_None;
}
