// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMDrawContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDrawContainer)

int32 FRigVMDrawContainer::GetIndex(const FName& InName) const
{
	for (int32 Index = 0; Index < Instructions.Num(); ++Index)
	{
		if (Instructions[Index].Name == InName)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

void FRigVMDrawContainer::Reset()
{
	Instructions.Reset();
}

