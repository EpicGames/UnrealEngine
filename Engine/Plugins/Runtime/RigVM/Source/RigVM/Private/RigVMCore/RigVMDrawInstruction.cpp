// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMDrawInstruction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDrawInstruction)

