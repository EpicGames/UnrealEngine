// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMDrawInterface.h"
#include "DynamicMeshBuilder.h"
#include "SceneManagement.h" 

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDrawInterface)

TAutoConsoleVariable<int32> CVarEnableRigVMDrawInterfaceInShipping(TEXT("RigVM.EnableDrawInterfaceInShipping"), 0, TEXT("Set to 1 to enable control rig draw interface in shipping"));

void FRigVMDrawInterface::DrawInstruction(const FRigVMDrawInstruction& InInstruction)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	Instructions.Add(InInstruction);
}

void FRigVMDrawInterface::DrawText(const FTransform& WorldOffset, const FVector& Position, const FString& Text, const FLinearColor& Color, float FontScale, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled() || Text.IsEmpty())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Text, Color, 0.f, WorldOffset, SDPG_Foreground, Lifetime);
	Instruction.Positions.Add(Position);
	Instruction.Text = Text;
	Instruction.FontScale = FontScale;
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawPoint(const FTransform& WorldOffset, const FVector& Position, float Size, const FLinearColor& Color, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Points, Color, Size, WorldOffset, DepthPriority, Lifetime);
	Instruction.Positions.Add(Position);
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawPoints(const FTransform& WorldOffset, const TArrayView<const FVector>& Points, float Size, const FLinearColor& Color, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Points, Color, Size, WorldOffset, DepthPriority, Lifetime);
	Instruction.Positions.Append(Points.GetData(), Points.Num());
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawLine(const FTransform& WorldOffset, const FVector& LineStart, const FVector& LineEnd, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, Color, Thickness, WorldOffset, DepthPriority, Lifetime);
	Instruction.Positions.Add(LineStart);
	Instruction.Positions.Add(LineEnd);
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawLines(const FTransform& WorldOffset, const TArrayView<const FVector>& Positions, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, Color, Thickness, WorldOffset, DepthPriority, Lifetime);
	Instruction.Positions.Append(Positions.GetData(), Positions.Num());
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawLineStrip(const FTransform& WorldOffset, const TArrayView<const FVector>& Positions, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, Color, Thickness, WorldOffset, DepthPriority, Lifetime);
	Instruction.Positions.Append(Positions.GetData(), Positions.Num());
	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawBox(const FTransform& WorldOffset, const FTransform& Transform, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FTransform DrawTransform = Transform * WorldOffset;

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, Color, Thickness, DrawTransform, DepthPriority, Lifetime);

	Instruction.Positions.Add(FVector(0.5f, 0.5f, 0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, 0.5f));
	Instruction.Positions.Add(FVector(0.5f, 0.5f, 0.5f));

	Instruction.Positions.Add(FVector(0.5f, 0.5f, -0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, -0.5f));
	Instruction.Positions.Add(FVector(0.5f, 0.5f, -0.5f));

	Instruction.Positions.Add(FVector(0.5f, 0.5f, 0.5f));
	Instruction.Positions.Add(FVector(0.5f, 0.5f, -0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, -0.5f, -0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, 0.5f));
	Instruction.Positions.Add(FVector(-0.5f, 0.5f, -0.5f));

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawAxes(const FTransform& WorldOffset, const FTransform& Transform, float Size, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	DrawLine(WorldOffset, Transform.GetLocation(), Transform.TransformPosition(FVector(Size, 0.f, 0.f)), FLinearColor::Red, Thickness);
	DrawLine(WorldOffset, Transform.GetLocation(), Transform.TransformPosition(FVector(0.f, Size, 0.f)), FLinearColor::Green, Thickness);
	DrawLine(WorldOffset, Transform.GetLocation(), Transform.TransformPosition(FVector(0.f, 0.f, Size)), FLinearColor::Blue, Thickness);
}

void FRigVMDrawInterface::DrawAxes(const FTransform& WorldOffset, const TArrayView<const FTransform>& Transforms, float Size, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	if (Transforms.Num() == 0)
	{
		return;
	}

	FRigVMDrawInstruction InstructionX(ERigVMDrawSettings::Lines, FLinearColor::Red, Thickness, WorldOffset, DepthPriority, Lifetime);
	FRigVMDrawInstruction InstructionY(ERigVMDrawSettings::Lines, FLinearColor::Green, Thickness, WorldOffset, DepthPriority, Lifetime);
	FRigVMDrawInstruction InstructionZ(ERigVMDrawSettings::Lines, FLinearColor::Blue, Thickness, WorldOffset, DepthPriority, Lifetime);

	InstructionX.Positions.Reserve(Transforms.Num() * 2);
	InstructionY.Positions.Reserve(Transforms.Num() * 2);
	InstructionZ.Positions.Reserve(Transforms.Num() * 2);

	for (const FTransform& Transform : Transforms)
	{
		InstructionX.Positions.Add(Transform.GetLocation());
		InstructionX.Positions.Add(Transform.TransformPosition(FVector(Size, 0.f, 0.f)));
		InstructionY.Positions.Add(Transform.GetLocation());
		InstructionY.Positions.Add(Transform.TransformPosition(FVector(0.f, Size, 0.f)));
		InstructionZ.Positions.Add(Transform.GetLocation());
		InstructionZ.Positions.Add(Transform.TransformPosition(FVector(0.f, 0.f, Size)));
	}

	Instructions.Add(InstructionX);
	Instructions.Add(InstructionY);
	Instructions.Add(InstructionZ);
}

void FRigVMDrawInterface::DrawAxes(const FTransform& WorldOffset, const FTransform& Transform,
	const FLinearColor& InColor, float Size, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, InColor, Thickness, WorldOffset, DepthPriority, Lifetime);

	Instruction.Positions.Reserve(6);
	Instruction.Positions.Add(Transform.GetLocation());
	Instruction.Positions.Add(Transform.TransformPosition(FVector(Size, 0.f, 0.f)));
	Instruction.Positions.Add(Transform.GetLocation());
	Instruction.Positions.Add(Transform.TransformPosition(FVector(0.f, Size, 0.f)));
	Instruction.Positions.Add(Transform.GetLocation());
	Instruction.Positions.Add(Transform.TransformPosition(FVector(0.f, 0.f, Size)));

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawAxes(const FTransform& WorldOffset, const TArrayView<const FTransform>& Transforms,
	const FLinearColor& InColor, float Size, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	if (Transforms.Num() == 0)
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, InColor, Thickness, WorldOffset, DepthPriority, Lifetime);

	Instruction.Positions.Reserve(Transforms.Num() * 6);

	for (const FTransform& Transform : Transforms)
	{
		Instruction.Positions.Add(Transform.GetLocation());
		Instruction.Positions.Add(Transform.TransformPosition(FVector(Size, 0.f, 0.f)));
		Instruction.Positions.Add(Transform.GetLocation());
		Instruction.Positions.Add(Transform.TransformPosition(FVector(0.f, Size, 0.f)));
		Instruction.Positions.Add(Transform.GetLocation());
		Instruction.Positions.Add(Transform.TransformPosition(FVector(0.f, 0.f, Size)));
	}

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawRectangle(const FTransform& WorldOffset, const FTransform& Transform, float Size, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FTransform DrawTransform = Transform * WorldOffset;

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, Color, Thickness, DrawTransform, DepthPriority, Lifetime);

	float Extent = Size * 0.5f;
	Instruction.Positions.Reserve(5);
	Instruction.Positions.Add(FVector(-Extent, -Extent, 0.f));
	Instruction.Positions.Add(FVector(-Extent, Extent, 0.f));
	Instruction.Positions.Add(FVector(Extent, Extent, 0.f));
	Instruction.Positions.Add(FVector(Extent, -Extent, 0.f));
	Instruction.Positions.Add(FVector(-Extent, -Extent, 0.f));

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawArc(const FTransform& WorldOffset, const FTransform& Transform, float Radius, float MinimumAngle, float MaximumAngle, const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	int32 Count = FMath::Clamp<int32>(Detail, 4, 32);
	
	FTransform DrawTransform = Transform * WorldOffset;

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::LineStrip, Color, Thickness, DrawTransform, DepthPriority, Lifetime);
	Instruction.Positions.Reserve(Count);

	FVector V = FVector(Radius, 0.f, 0.f);
	FQuat Rotation(FVector(0.f, 0.f, 1.f), MinimumAngle);
	V = Rotation.RotateVector(V);
	Instruction.Positions.Add(V);
	float StepAngle = (MaximumAngle - MinimumAngle) / float(Count);
	if (FMath::Abs<float>(MaximumAngle - MinimumAngle) >= TWO_PI)
	{
		StepAngle = TWO_PI / float(Count);
	}
	Rotation = FQuat(FVector(0.f, 0.f, 1.f), StepAngle);
	for(int32 Index=0;Index<Count;Index++)
	{
		V = Rotation.RotateVector(V);
		Instruction.Positions.Add(V);
	}

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawCircle(const FTransform& WorldOffset, const FTransform& Transform, float Radius,
	const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	DrawArc(WorldOffset, Transform, Radius, 0, TWO_PI, Color, Thickness, Detail);
}

void FRigVMDrawInterface::DrawSphere(const FTransform& WorldOffset, const FTransform& Transform, float Radius, 
	const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	// Circles are drawn in the XY plane
	FTransform R1(FQuat(FVector(1.f, 0.f, 0.f), HALF_PI));
	FTransform R2(FQuat(FVector(0.f, 1.f, 0.f), HALF_PI));
	DrawCircle(WorldOffset, Transform, Radius, Color, Thickness, Detail);
	DrawCircle(WorldOffset, R1 * Transform, Radius, Color, Thickness, Detail);
	DrawCircle(WorldOffset, R2 * Transform, Radius, Color, Thickness, Detail);
}

void FRigVMDrawInterface::DrawHemisphere(const FTransform& WorldOffset, const FTransform& Transform, float Radius,
	const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	// Circles are drawn in the XY plane
	FTransform R1(FQuat(FVector(1.f, 0.f, 0.f), HALF_PI));
	FTransform R2(FQuat(FVector(0.f, 1.f, 0.f), -HALF_PI));
	DrawCircle(WorldOffset, Transform, Radius, Color, Thickness, Detail);
	DrawArc(WorldOffset, R1 * Transform, Radius, 0, PI, Color, Thickness, Detail);
	DrawArc(WorldOffset, R2 * Transform, Radius, -HALF_PI, HALF_PI, Color, Thickness, Detail);
}

void FRigVMDrawInterface::DrawCapsule(const FTransform& WorldOffset, const FTransform& Transform,
	float Radius, float Length, const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	FTransform T1(FVector(0, 0, Length * 0.5));
	FTransform T2(FQuat(FVector(0.f, 1.f, 0.f), PI), FVector(0, 0, -Length * 0.5));

	DrawHemisphere(WorldOffset, T1 * Transform, Radius, Color, Thickness, Detail);
	DrawHemisphere(WorldOffset, T2 * Transform, Radius, Color, Thickness, Detail);

	DrawLine(WorldOffset, 
		Transform.TransformPosition(FVector(Radius, 0, Length * 0.5)), 
		Transform.TransformPosition(FVector(Radius, 0, -Length * 0.5)), Color, Thickness);
	DrawLine(WorldOffset,
		Transform.TransformPosition(FVector(-Radius, 0, Length * 0.5)),
		Transform.TransformPosition(FVector(-Radius, 0, -Length * 0.5)), Color, Thickness);
	DrawLine(WorldOffset,
		Transform.TransformPosition(FVector(0, Radius, Length * 0.5)),
		Transform.TransformPosition(FVector(0, Radius, -Length * 0.5)), Color, Thickness);
	DrawLine(WorldOffset,
		Transform.TransformPosition(FVector(0, -Radius, Length * 0.5)),
		Transform.TransformPosition(FVector(0, -Radius, -Length * 0.5)), Color, Thickness);
}


void FRigVMDrawInterface::DrawCone(const FTransform& WorldOffset, const FTransform& ConeOffset, float Angle1, float Angle2, uint32 NumSides,
	bool bDrawSideLines, const FLinearColor& SideLineColor, FMaterialRenderProxy* const MaterialRenderProxy, float SideLineThickness,
	ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	BuildConeVerts(Angle1, Angle2, 1.f, 0.f, NumSides, MeshVerts, MeshIndices);

	FTransform ConeTransform = ConeOffset * WorldOffset;
	FRigVMDrawInstruction MeshInstruction;
	MeshInstruction.PrimitiveType = ERigVMDrawSettings::DynamicMesh;
	MeshInstruction.Transform = ConeTransform;
	MeshInstruction.MeshVerts = MeshVerts;
	MeshInstruction.MeshIndices = MeshIndices;
	MeshInstruction.MaterialRenderProxy = MaterialRenderProxy;
	MeshInstruction.DepthPriority = DepthPriority;
	MeshInstruction.Lifetime = Lifetime;
	Instructions.Add(MeshInstruction);

	if (bDrawSideLines)
	{
		TArray<FVector> ConeVerts;
		ConeVerts.AddUninitialized(NumSides);

		// Draw lines down major directions
		for (int32 i = 0; i < 4; i++)
		{
			float Fraction = (float)i / (float)(4);
			float Azi = 2.f * PI * Fraction;
			FVector ConeVert = CalcConeVert(Angle1, Angle2, Azi);
			DrawLine(WorldOffset, ConeOffset.GetLocation(), ConeOffset.TransformPosition(ConeVert), SideLineColor, SideLineThickness);
		}
	}
}

void FRigVMDrawInterface::DrawArrow(const FTransform& WorldOffset, const FVector& Direction, const FVector& Side,
	const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	FRigVMDrawInstruction Instruction(ERigVMDrawSettings::Lines, Color, Thickness, WorldOffset, DepthPriority, Lifetime);

	Instruction.Positions.SetNum(6);
	Instruction.Positions[0] = FVector::ZeroVector;
	Instruction.Positions[1] = Direction;
	Instruction.Positions[2] = Instruction.Positions[1];
	Instruction.Positions[3] = Direction - Direction.GetSafeNormal() * Side.Size() + Side;
	Instruction.Positions[4] = Instruction.Positions[1];
	Instruction.Positions[5] = Direction - Direction.GetSafeNormal() * Side.Size() - Side;

	Instructions.Add(Instruction);
}

void FRigVMDrawInterface::DrawPlane(const FTransform& WorldOffset, const FVector2D& Scale, const FLinearColor& MeshColor,
	bool bDrawLines, const FLinearColor& LineColor, FMaterialRenderProxy* const MaterialRenderProxy, ESceneDepthPriorityGroup DepthPriority, float Lifetime)
{
	LLM_SCOPE_BYNAME(TEXT("RigVM/Debug"));

	if (!IsEnabled())
	{
		return;
	}

	const FVector Corner(Scale.X, Scale.Y, 0);

	FRigVMDrawInstruction MeshInstruction;
	MeshInstruction.PrimitiveType = ERigVMDrawSettings::DynamicMesh;
	MeshInstruction.Transform = WorldOffset;
	MeshInstruction.MeshVerts = {
		(FVector3f)(Corner * FVector(-1, 1, 0)),
		(FVector3f)(Corner * FVector(1, 1, 0)),
		(FVector3f)(Corner * FVector(1, -1, 0)),
		(FVector3f)(Corner * FVector(-1, -1, 0))
	};
	MeshInstruction.MeshIndices = { 0, 1, 2, 2, 3, 0 };
	MeshInstruction.MaterialRenderProxy = MaterialRenderProxy;
	MeshInstruction.DepthPriority = DepthPriority;
	MeshInstruction.Lifetime = Lifetime;

	Instructions.Add(MeshInstruction);

	if (bDrawLines)
	{
		FRigVMDrawInstruction LinesInstruction(ERigVMDrawSettings::LineStrip, LineColor, 0.f, WorldOffset, DepthPriority, Lifetime);

		LinesInstruction.Positions.Reserve(5);
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[0].Position));
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[1].Position));
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[2].Position));
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[3].Position));
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[0].Position));
		LinesInstruction.Positions.Add(FVector(MeshInstruction.MeshVerts[2].Position));

		Instructions.Add(LinesInstruction);
	}
}

bool FRigVMDrawInterface::IsEnabled() const
{
	bool bIsEnabled = true;

#if UE_BUILD_TEST
	bIsEnabled = CVarEnableRigVMDrawInterfaceInShipping.GetValueOnAnyThread() == 1; 
#endif

	return bIsEnabled; 
}

