// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVM.h"
#include "RigVMHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMExecuteContext)

TAutoConsoleVariable<bool> CVarRigVMReportAllMessages(
	TEXT("RigVM.ReportAllMessages"),
	false,
	TEXT("Report all log messages, even when no logging procedure has been setup."));

void FRigVMExecuteContext::SetOwningObject(const UObject* InOwningObject)
{
	OwningObject = InOwningObject;
	OwningActor = nullptr;
	World = nullptr;
	ToWorldSpaceTransform = FTransform::Identity;

	if(InOwningObject)
	{
		if(const UActorComponent* ActorComponent = Cast<UActorComponent>(InOwningObject))
		{
			SetOwningActor(ActorComponent->GetOwner());
		}
		else
		{
			SetWorld(OwningObject->GetWorld());
		}
	}
}

void FRigVMExecuteContext::SetOwningComponent(const USceneComponent* InOwningComponent)
{
	OwningObject = InOwningComponent;
	OwningActor = nullptr;
	World = nullptr;
	ToWorldSpaceTransform = FTransform::Identity;
	
	if(InOwningComponent)
	{
		ToWorldSpaceTransform = InOwningComponent->GetComponentToWorld();
		SetOwningActor(InOwningComponent->GetOwner());
	}
}

void FRigVMExecuteContext::SetOwningActor(const AActor* InActor)
{
	OwningActor = InActor;
	World = nullptr;
	if(OwningActor)
	{
		World = OwningActor->GetWorld();
	}
}

void FRigVMExecuteContext::SetWorld(const UWorld* InWorld)
{
	World = InWorld;
}

bool FRigVMExecuteContext::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FLazyName ControlRigExecuteContextName("ControlRigExecuteContext");
	if (Tag.GetType().IsStruct(ControlRigExecuteContextName))
	{
		static const FString CRExecuteContextPath = TEXT("/Script/ControlRig.ControlRigExecuteContext");
		UScriptStruct* OldStruct = FindFirstObject<UScriptStruct>(*CRExecuteContextPath, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
		checkf(OldStruct, TEXT("FControlRigExecuteContext was not found."));

		TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(OldStruct));
		OldStruct->SerializeItem(Slot, StructOnScope->GetStructMemory(), nullptr);		
		return true;
	}

	return false;
}

const FRigVMFunction* FRigVMExecuteContext::FindDispatchFunction(const FName& InDispatchFunctionName) const
{
	check(ExtendedExecuteContext);
	return ExtendedExecuteContext->FindDispatchFunction(InDispatchFunctionName);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FRigVMExtendedExecuteContext::~FRigVMExtendedExecuteContext()
{
	Reset();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

// --- FRigVMExtendedExecuteContext ---

void FRigVMExtendedExecuteContext::Reset()
{
	VMHash = 0;

	ResetExecutionState();

	WorkMemoryStorage = FRigVMMemoryStorageStruct();
	DebugMemoryStorage = FRigVMMemoryStorageStruct();

	CurrentVMMemory.Reset();
	
	OnExecutionReachedExitCallback.Reset();

	Frame->LazyBranchExecuteState.Reset();
	ExternalVariableRuntimeData.Reset();

	if(PublicDataScope.IsValid())
	{
		GetPublicData<>().NumExecutions = 0;
	}

	Callstack.Reset();
	CallstackHash = 0;

	ExecutingThreadId = INDEX_NONE;

	EntriesBeingExecuted.Reset();

	CurrentExecuteResult = ERigVMExecuteResult::Failed;
	CurrentEntryName = NAME_None;
	bCurrentlyRunningRootEntry = false;

#if WITH_EDITOR
	if (TSharedPtr<FRigVMDebugInfo> DebugInfoShared = DebugInfoWeak.Pin())
	{
		DebugInfoShared->Reset();
		DebugInfoWeak.Reset();
	}
#endif // WITH_EDITOR
}

/** Resets VM execution state */
void FRigVMExtendedExecuteContext::ResetExecutionState()
{
	if (FRigVMExecuteContext* ExecuteContext = reinterpret_cast<FRigVMExecuteContext*>(PublicDataScope.GetStructMemory()))
	{
		ExecuteContext->Reset();
		ExecuteContext->ExtendedExecuteContext = this;
	}

	VM = nullptr;
	Frame = &FrameStorage;
	Frame->Reset();
	Factory = nullptr;
	DispatchFunctionMap.Reset();
}

void FRigVMExtendedExecuteContext::CopyMemoryStorage(const FRigVMExtendedExecuteContext& Other)
{
	VMHash = Other.VMHash;
	WorkMemoryStorage = Other.WorkMemoryStorage;
	DebugMemoryStorage = Other.DebugMemoryStorage;
}

FRigVMExtendedExecuteContext& FRigVMExtendedExecuteContext::operator =(const FRigVMExtendedExecuteContext& Other)
{
	CopyMemoryStorage(Other);

	const UScriptStruct* OtherPublicDataStruct = Cast<UScriptStruct>(Other.PublicDataScope.GetStruct());
	check(OtherPublicDataStruct);
	if(PublicDataScope.GetStruct() != OtherPublicDataStruct)
	{
		PublicDataScope = FStructOnScope(OtherPublicDataStruct);
	}

	FRigVMExecuteContext* ThisPublicContext = (FRigVMExecuteContext*)PublicDataScope.GetStructMemory();
	const FRigVMExecuteContext* OtherPublicContext = (const FRigVMExecuteContext*)Other.PublicDataScope.GetStructMemory();
	ThisPublicContext->Copy(OtherPublicContext);
	ThisPublicContext->ExtendedExecuteContext = this;

	if(OtherPublicContext->GetNameCache() == &Other.NameCache)
	{
		SetDefaultNameCache();
	}

	FrameStorage.Slices = Other.FrameStorage.Slices;
	FrameStorage.SliceOffsetsPerInstruction = Other.FrameStorage.SliceOffsetsPerInstruction;
	FrameStorage.SliceOffsetsPerCallable = Other.FrameStorage.SliceOffsetsPerCallable;
	FrameStorage.LazyBranchExecuteState = Other.FrameStorage.LazyBranchExecuteState;
	Frame = &FrameStorage;
	
	ExternalVariableRuntimeData = Other.ExternalVariableRuntimeData;

	return *this;
}

void FRigVMExtendedExecuteContext::Initialize(const UScriptStruct* InScriptStruct)
{
	check(InScriptStruct->IsChildOf(FRigVMExecuteContext::StaticStruct()));
	PublicDataScope = FStructOnScope(InScriptStruct);
	((FRigVMExecuteContext*)PublicDataScope.GetStructMemory())->ExtendedExecuteContext = this;
	SetDefaultNameCache();
}

bool FRigVMExtendedExecuteContext::StepForward()
{
#if WITH_EDITOR
	if (TSharedPtr<FRigVMDebugInfo> CurrentDebugInfo = DebugInfoWeak.Pin())
	{
		CurrentDebugInfo->StepForward();
		return true;
	}
#endif
	return false;
}

bool FRigVMExtendedExecuteContext::StepInto()
{
#if WITH_EDITOR
	if (TSharedPtr<FRigVMDebugInfo> CurrentDebugInfo = DebugInfoWeak.Pin())
	{
		CurrentDebugInfo->StepInto();
		return true;
	}
#endif
	return false;
}

bool FRigVMExtendedExecuteContext::StepOut()
{
#if WITH_EDITOR
	if (TSharedPtr<FRigVMDebugInfo> CurrentDebugInfo = DebugInfoWeak.Pin())
	{
		CurrentDebugInfo->StepOut();
		return true;
	}
#endif
	return false;
}

int32 FRigVMExtendedExecuteContext::GetInstructionIndex() const
{
	return GetPublicData<>().InstructionIndex;
}

const URigVM* FRigVMExtendedExecuteContext::GetVM() const
{
	URigVM* MutableVM = const_cast<FRigVMExtendedExecuteContext*>(this)->GetVM();
	return MutableVM;
}

URigVM* FRigVMExtendedExecuteContext::GetVM()
{
	if (VM)
	{
		return VM;
	}
	if (Host)
	{
		return Host->GetVM();
	}
	return nullptr;
}

FString FRigVMExtendedExecuteContext::GetVMPathName() const
{
	return GetVM() ? GetVM()->GetPathName() : TEXT("Unknown VM");
}

void FRigVMExtendedExecuteContext::SetVM(URigVM* InVM)
{
	VM = InVM;
}

const FRigVMFunction* FRigVMExtendedExecuteContext::FindDispatchFunction(const FName& InDispatchFunctionName) const
{
	if (const FRigVMFunction* const* FunctionPtr = DispatchFunctionMap.Find(InDispatchFunctionName))
	{
		return *FunctionPtr;
	}

	if (const URigVM* CurrentVM = GetVM())
	{
		const FRigVMFunction* Function = nullptr;
		if (FRigVMRegistry_NoLock* Registry = CurrentVM->GetLocalizedRegistry())
		{
			Function = Registry->FindFunction_NoLock(*InDispatchFunctionName.ToString());
		}
		else
		{
			Function = FRigVMRegistry::Get().FindFunction(*InDispatchFunctionName.ToString());
		}
		DispatchFunctionMap.Add(InDispatchFunctionName, Function);
		return Function;
	}
	return nullptr;
}

const FRigVMOperand* FRigVMExtendedExecuteContext::GetDebugOperandForOperand(const FRigVMOperand& InOperand) const
{
	const FRigVMOperand KeyOperand(InOperand.GetMemoryType(), InOperand.GetRegisterIndex()); // no register offset
	if (DebuggedOperands.Num() == OperandToDebugRegister.Num() || DebuggedOperands.Contains(KeyOperand))
	{
		if(const FRigVMOperand* DebugOperandPtr = OperandToDebugRegister.Find(KeyOperand))
		{
			return DebugOperandPtr;
		}
	}

	return nullptr;
}

void FRigVMExtendedExecuteContext::MarkOperandForDebugging(const FRigVMOperand& InOperand, bool bEnableDebugging)
{
	const FRigVMOperand KeyOperand(InOperand.GetMemoryType(), InOperand.GetRegisterIndex(), INDEX_NONE /* remove register offset */);
	if (bEnableDebugging && OperandToDebugRegister.Contains(KeyOperand))
	{
		DebuggedOperands.Add(KeyOperand);
	}
	else
	{
		DebuggedOperands.Remove(KeyOperand);
	}
}

void FRigVMExtendedExecuteContext::UnmarkOperandForDebugging(const FRigVMOperand& InOperand)
{
	MarkOperandForDebugging(InOperand, false);
}

void FRigVMExtendedExecuteContext::MarkAllOperandsForDebugging(bool bEnableDebugging)
{
	if (bEnableDebugging)
	{
		// if the count matches we'll assume for now that it's already mapped.
		if (DebuggedOperands.Num() == OperandToDebugRegister.Num())
		{
			return;
		}
		
		DebuggedOperands.Reset();
		DebuggedOperands.Reserve(OperandToDebugRegister.Num());
		OperandToDebugRegister.GetKeys(DebuggedOperands);
	}
	else
	{
		DebuggedOperands.Reset();
	}
}

void FRigVMExtendedExecuteContext::UnmarkAllOperandsForDebugging()
{
	DebuggedOperands.Reset();
}
