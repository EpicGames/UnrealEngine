// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMExternalVariable.h"

#include "RigVMObjectVersion.h"
#include "RigVMCore/RigVMRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMExternalVariable)

#if WITH_EDITOR

#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"

FRigVMExternalVariable FRigVMExternalVariable::Make(const FGuid InGuid, const FName& InName, const FEdGraphPinType& InPinType, bool bInPublic, bool bInReadonly)
{
	FRigVMExternalVariable ExternalVariable;
	ExternalVariable.Guid = InGuid;
	ExternalVariable.Name = InName;
	ExternalVariable.bIsPublic = bInPublic;
	ExternalVariable.bIsReadOnly = bInReadonly;

	if (InPinType.ContainerType == EPinContainerType::None)
	{
		ExternalVariable.bIsArray = false;
	}
	else if (InPinType.ContainerType == EPinContainerType::Array)
	{
		ExternalVariable.bIsArray = true;
	}
	else
	{
		return FRigVMExternalVariable();
	}

	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		ExternalVariable.BaseCPPType = RigVMTypeUtils::BoolTypeName;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		ExternalVariable.BaseCPPType = RigVMTypeUtils::Int32TypeName;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Enum ||
		InPinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		if (UEnum* Enum = Cast<UEnum>(InPinType.PinSubCategoryObject))
		{
			ExternalVariable.CPPTypeObject = Enum;
		}
		else
		{
			ExternalVariable.BaseCPPType = RigVMTypeUtils::UInt8TypeName;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
		{
			ExternalVariable.BaseCPPType = RigVMTypeUtils::FloatTypeName;
		}
		else if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
		{
			ExternalVariable.BaseCPPType = RigVMTypeUtils::DoubleTypeName;
		}
		else
		{
			checkf(false, TEXT("Unexpected subcategory for PC_Real pin type."));
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		ExternalVariable.BaseCPPType = RigVMTypeUtils::FNameTypeName;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		ExternalVariable.BaseCPPType = RigVMTypeUtils::FStringTypeName;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		ExternalVariable.BaseCPPType = RigVMTypeUtils::FTextTypeName;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (UScriptStruct* Struct = Cast<UScriptStruct>(InPinType.PinSubCategoryObject))
		{
			ExternalVariable.CPPTypeObject = Struct;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		if (UClass* Class = Cast<UClass>(InPinType.PinSubCategoryObject))
		{
			ExternalVariable.CPPTypeObject = Class;
		}
	}

	if (ExternalVariable.CPPTypeObject)
	{
		ExternalVariable.BaseCPPType = *RigVMTypeUtils::CPPTypeFromObject(ExternalVariable.CPPTypeObject, RigVMTypeUtils::EClassArgType::AsObject);
	}

	ExternalVariable.CheckCPPTypeIntegrity();
	return ExternalVariable;
}

#endif

FRigVMExternalVariable FRigVMExternalVariable::Make(const FGuid InGuid, const FName& InName, const FString& InCPPTypePath, bool bInPublic, bool bInReadonly)
{
	FString BaseCPPTypePath = InCPPTypePath;
	if (RigVMTypeUtils::IsArrayType(InCPPTypePath))
	{
		BaseCPPTypePath = RigVMTypeUtils::BaseTypeFromArrayType(InCPPTypePath);
	}

	UObject* CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(BaseCPPTypePath);
	return Make(InGuid, InName, InCPPTypePath, CPPTypeObject, bInPublic, bInReadonly);
}

FRigVMExternalVariable FRigVMExternalVariable::Make(const FGuid InGuid, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, bool bInPublic, bool bInReadonly)
{
	FRigVMExternalVariable Variable;
	if (InCPPType.StartsWith(TEXT("TMap<")))
	{
		return Variable;
	}
	
	Variable.Guid = InGuid;
	Variable.Name = InName;
	Variable.bIsPublic = bInPublic;
	Variable.bIsReadOnly = bInReadonly;

	FString CPPType = InCPPType;
	Variable.bIsArray = RigVMTypeUtils::IsArrayType(CPPType);
	if (Variable.bIsArray)
	{
		CPPType = RigVMTypeUtils::BaseTypeFromArrayType(CPPType);
	}

	Variable.BaseCPPType = *CPPType;
	Variable.CPPTypeObject = InCPPTypeObject;

	if (Variable.CPPTypeObject)
	{
		Variable.BaseCPPType = *RigVMTypeUtils::CPPTypeFromObject(Variable.CPPTypeObject, RigVMTypeUtils::EClassArgType::AsObject);
	}

	Variable.CheckCPPTypeIntegrity();
	return Variable;
}

#if WITH_EDITOR

FRigVMExternalVariable RigVMTypeUtils::ExternalVariableFromBPVariableDescription(
	const FBPVariableDescription& InVariableDescription)
{
	const bool bIsPublic = !((InVariableDescription.PropertyFlags & CPF_DisableEditOnInstance) == CPF_DisableEditOnInstance);
	const bool bIsReadOnly = ((InVariableDescription.PropertyFlags & CPF_BlueprintReadOnly) == CPF_BlueprintReadOnly);
	FRigVMExternalVariable Variable =  ExternalVariableFromPinType(InVariableDescription.VarGuid, InVariableDescription.VarName, InVariableDescription.VarType, bIsPublic, bIsReadOnly);
	return Variable;
}

FRigVMExternalVariable RigVMTypeUtils::ExternalVariableFromBPVariableDescription(const FBPVariableDescription& InVariableDescription, const UStruct* VariablesStruct, void* Container)
{
	const bool bIsPublic = !((InVariableDescription.PropertyFlags & CPF_DisableEditOnInstance) == CPF_DisableEditOnInstance);
	const bool bIsReadOnly = ((InVariableDescription.PropertyFlags & CPF_BlueprintReadOnly) == CPF_BlueprintReadOnly);
	
	FRigVMExternalVariable ExternalVariable = ExternalVariableFromPinType(InVariableDescription.VarGuid, InVariableDescription.VarName, InVariableDescription.VarType, bIsPublic, bIsReadOnly);
	
	if (VariablesStruct)
	{
		if (FProperty* Property = VariablesStruct->FindPropertyByName(InVariableDescription.VarName))
		{
			ExternalVariable.SetProperty(Property);
		}
		
		if (Container != nullptr && ExternalVariable.GetProperty() != nullptr)
		{
			ExternalVariable.SetMemory(ExternalVariable.GetProperty()->ContainerPtrToValuePtr<uint8>(Container));
		}
	}

	return ExternalVariable;
}

FRigVMExternalVariable RigVMTypeUtils::ExternalVariableFromPinType(const FGuid InGuid, const FName& InName, const FEdGraphPinType& InPinType, bool bInPublic, bool bInReadonly)
{
	return FRigVMExternalVariable::Make(InGuid, InName, InPinType, bInPublic, bInReadonly);
}

FRigVMExternalVariable RigVMTypeUtils::ExternalVariableFromCPPTypePath(const FGuid InGuid, const FName& InName, const FString& InCPPTypePath, bool bInPublic, bool bInReadonly)
{
	return FRigVMExternalVariable::Make(InGuid, InName, InCPPTypePath, bInPublic, bInReadonly);
}

FRigVMExternalVariable RigVMTypeUtils::ExternalVariableFromCPPType(const FGuid InGuid, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, bool bInPublic, bool
                                                                   bInReadonly)
{
	return FRigVMExternalVariable::Make(InGuid, InName, InCPPType, InCPPTypeObject, bInPublic, bInReadonly);
}

#endif // WITH_EDITOR

TRigVMTypeIndex FRigVMExternalVariable::GetTypeIndex() const
{
	if(IsValid(true))
	{
		return FRigVMRegistry_RWLock::Get().GetTypeIndexFromCPPType(GetExtendedCPPType().ToString());
	}
	return INDEX_NONE;
}

RIGVM_API TArray<FRigVMExternalVariableDef> RigVMTypeUtils::GetExternalVariableDefs(const TArray<FRigVMExternalVariable>& ExternalVariables)
{
	TArray<FRigVMExternalVariableDef> VariableDefs;

	for (const FRigVMExternalVariable& Var : ExternalVariables)
	{
		VariableDefs.Add(Var);
	}

	return VariableDefs;
}

FArchive& operator<<(FArchive& Ar, FRigVMExternalVariableDef& Variable)
{
	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::GuidForVariables)
	{
		Variable.Guid.Invalidate();
	}
	else
	{
		Ar << Variable.Guid;
	}
	
	Ar << Variable.Name;
	Ar << Variable.BaseCPPType;

	if (Ar.IsSaving())
	{
		FSoftObjectPath TypeObjectPath(Variable.CPPTypeObject);
		Ar << TypeObjectPath;
	}
	else if (Ar.IsLoading())
	{
		FSoftObjectPath TypeObjectPath;
		Ar << TypeObjectPath;
		Variable.CPPTypeObject = TypeObjectPath.ResolveObject();

		if (Variable.CPPTypeObject)
		{
			Variable.BaseCPPType = *RigVMTypeUtils::CPPTypeFromObject(Variable.CPPTypeObject, RigVMTypeUtils::EClassArgType::AsObject);
		}
	}

	Ar << Variable.bIsArray;
	Ar << Variable.bIsPublic;
	Ar << Variable.bIsReadOnly;

	// we need to keep serializing this until we increment the object version
	int32 DummySize = 0;
	Ar << DummySize;
	return Ar;
}

void FRigVMExternalVariableDef::CheckCPPTypeIntegrity() const
{
#if WITH_EDITOR
	if (BaseCPPType.IsNone())
	{
		return;
	}
	const FString BaseCPPTypeString = BaseCPPType.ToString();
	check(!RigVMTypeUtils::IsArrayType(BaseCPPTypeString));

	if (RigVMTypeUtils::RequiresCPPTypeObject(BaseCPPTypeString))
	{
		check(CPPTypeObject);
	}
#endif
}
