// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMPropertyUtils.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction)

void FRigVMFunctionArgument::Serialize_NoLock(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		NameString = MakeShared<FString>();
		TypeString = MakeShared<FString>();
		Ar << *NameString.Get();
		Ar << *TypeString.Get();
		
		Name = NameString->operator*();
		Type = TypeString->operator*();

		int32 DirectionAsInt32 = 0;
		Ar << DirectionAsInt32;
		Direction = (ERigVMFunctionArgumentDirection)DirectionAsInt32;
	}
	else
	{
		FString LocalNameString = Name;
		FString LocalTypeString = Type;
		Ar << LocalNameString;
		Ar << LocalTypeString;

		int32 DirectionAsInt32 = (int32)(Direction);
		Ar << DirectionAsInt32;
	}
}

bool FRigVMFunctionArgument::operator==(const FRigVMFunctionArgument& InOther) const
{
	if(FCString::Strcmp(Name, InOther.Name) != 0)
	{
		return false;
	}
	if(FCString::Strcmp(Type, InOther.Type) != 0)
	{
		return false;
	}
	return Direction == InOther.Direction;
}

FRigVMFunction::FRigVMFunction(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct, int32 InIndex, const TArray<FRigVMFunctionArgument>& InArguments)
: Name(InName), Struct(InStruct)
, Factory(nullptr)
, OwnerRegistry(nullptr)
, FunctionPtr(InFunctionPtr)
, Index(InIndex)
, TemplateIndex(INDEX_NONE)
, PermutationIndex(INDEX_NONE)
, Arguments(InArguments)
, ArgumentNames()
{
	ArgumentNames.SetNumZeroed(Arguments.Num());
	for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		ArgumentNames[ArgumentIndex] = Arguments[ArgumentIndex].Name;
	}
}

FString FRigVMFunction::GetName() const
{
	return Name;
}

FName FRigVMFunction::GetMethodName() const
{
	FString FullName(Name);
	FString Right;
	if (FullName.Split(TEXT("::"), nullptr, &Right))
	{
		return *Right;
	}
	return NAME_None;
}

FString FRigVMFunction::GetModuleName() const
{
#if WITH_EDITOR
	if (Struct)
	{
		if (UPackage* Package = Struct->GetPackage())
		{
			return Package->GetName();
		}
	}
	if (Factory)
	{
		if (UPackage* Package = Factory->GetScriptStruct()->GetPackage())
		{
			return Package->GetName();
		}
	}
#endif
	return FString();
}

FString FRigVMFunction::GetModuleRelativeHeaderPath() const
{
#if WITH_EDITOR
	if (Struct)
	{
		FString ModuleRelativePath;
		if (Struct->GetStringMetaDataHierarchical(TEXT("ModuleRelativePath"), &ModuleRelativePath))
		{
			return ModuleRelativePath;
		}
	}
#endif
	return FString();
}

int32 FRigVMFunction::GetArgumentIndex(const FName& InArgumentName) const
{
	for (int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		if (Arguments[ArgumentIndex].Name == InArgumentName)
		{
			return ArgumentIndex;
		}
	}
	return INDEX_NONE;
}

const TArray<TRigVMTypeIndex>& FRigVMFunction::GetArgumentTypeIndices() const
{
	FRigVMRegistryWriteLock Lock;
	return GetArgumentTypeIndices_NoLock(Lock);
}

const TArray<TRigVMTypeIndex>& FRigVMFunction::GetArgumentTypeIndices_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	if(ArgumentTypeIndices.IsEmpty() && !Arguments.IsEmpty())
	{
		GetArgumentTypeIndices_NoLock(InRegistry, ArgumentTypeIndices);
	}
	return ArgumentTypeIndices;
}

void FRigVMFunction::GetArgumentTypeIndices_NoLock(FRigVMRegistryHandle& InRegistry, TArray<TRigVMTypeIndex>& OutTypeIndices) const
{
	OutTypeIndices.Reset();
	
	if(Struct)
	{
		for(const FRigVMFunctionArgument& Argument : Arguments)
		{
			if(const FProperty* Property = Struct->FindPropertyByName(Argument.Name))
			{
				FName CPPType = NAME_None;
				UObject* CPPTypeObject = nullptr;
				RigVMPropertyUtils::GetTypeFromProperty(Property, CPPType, CPPTypeObject);

				const FRigVMTemplateArgumentType Type(CPPType, CPPTypeObject);
				OutTypeIndices.Add(InRegistry->FindOrAddType_NoLock(Type));
			}
			else
			{
				OutTypeIndices.Add(INDEX_NONE);
			}
		}
	}
	else if(InRegistry->GetTemplates_NoLock().IsValidIndex(TemplateIndex))
	{
		const FRigVMTemplate* Template = &InRegistry->GetTemplates_NoLock()[TemplateIndex];
			
		check(PermutationIndex != INDEX_NONE);
		check(PermutationIndex == Template->FindPermutation_NoLock(this, InRegistry));

		for(const FRigVMFunctionArgument& FunctionArgument : Arguments)
		{
			TRigVMTypeIndex TypeIndex = InRegistry->GetTypeIndexFromCPPType_NoLock(FunctionArgument.Type);
			
			// fall back on the template just in case
			if(TypeIndex == INDEX_NONE)
			{
				if (const FRigVMTemplateArgument* TemplateArgument = Template->FindArgument(FunctionArgument.Name))
				{
					TypeIndex = TemplateArgument->GetTypeIndex_NoLock(PermutationIndex, InRegistry);
				}
			}
#if WITH_EDITOR
			if(TypeIndex != INDEX_NONE)
			{
				const FRigVMTemplateArgumentType& Type = InRegistry->GetType_NoLock(TypeIndex); 
				check(FunctionArgument.Type == Type.CPPType.ToString());
			}
#endif 
			OutTypeIndices.Add(TypeIndex);
		}
	}
	else
	{
		// checkNoEntry();
	}
}

const FRigVMTemplate* FRigVMFunction::GetTemplate() const
{
	return GetTemplate_NoLock(FRigVMRegistryReadLock());
}

const FRigVMTemplate* FRigVMFunction::GetTemplate_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	if(TemplateIndex == INDEX_NONE)
	{
		return nullptr;
	}

	const FRigVMTemplate* Template = &InRegistry->GetTemplates_NoLock()[TemplateIndex];
	if (!Template->IsValid())
	{
		return nullptr;
	}

	if (!Template->UsesDispatch())
	{
		if(Template->NumPermutations_NoLock(InRegistry) <= 1)
		{
			return nullptr;
		}
	}

	return Template;
}

const UScriptStruct* FRigVMFunction::GetExecuteContextStruct() const
{
	return GetExecuteContextStruct_NoLock(FRigVMRegistryReadLock());
}

const UScriptStruct* FRigVMFunction::GetExecuteContextStruct_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if(Factory)
	{
		return Factory->GetExecuteContextStruct();
	}
	if(Struct && Struct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		FStructOnScope StructOnScope(Struct);
		const FRigVMStruct* StructOnScopeMemory = reinterpret_cast<const FRigVMStruct*>(StructOnScope.GetStructMemory());
		return StructOnScopeMemory->GetExecuteContextStruct();
		
	}
	return FRigVMExecuteContext::StaticStruct();
}

bool FRigVMFunction::SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const
{
	return SupportsExecuteContextStruct_NoLock(InExecuteContextStruct, FRigVMRegistryReadLock());
}

bool FRigVMFunction::SupportsExecuteContextStruct_NoLock(const UScriptStruct* InExecuteContextStruct, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	return InExecuteContextStruct->IsChildOf(GetExecuteContextStruct_NoLock(InRegistry));
}

const FName& FRigVMFunction::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if(Factory)
	{
		const TArray<FName>* FactoryArgumentNames = Factory->UpdateArgumentNameCache_NoLock(InTotalOperands, InRegistry);
		check(FactoryArgumentNames);
		check(FactoryArgumentNames->IsValidIndex(InOperandIndex));
		return (*FactoryArgumentNames)[InOperandIndex];
	}

	checkf(ArgumentNames.IsValidIndex(InOperandIndex), TEXT("RigVMFunction %s: Invalid operand index %d/%d"), 
		*Name, InOperandIndex, InTotalOperands);
	return ArgumentNames[InOperandIndex];
}

void FRigVMFunction::Serialize_NoLock(FArchive& Ar, FRigVMRegistryHandle& InRegistry, const UObject* Owner)
{
	if(Ar.IsSaving())
	{
		verify(InRegistry.GetRegistry() == OwnerRegistry);
	}
	
	Ar << Name;
	Ar << Struct;

	if(Ar.IsLoading())
	{
		Index = INDEX_NONE;
		TemplateIndex = INDEX_NONE;
		FunctionPtr = nullptr;
		Arguments.Reset();
		ArgumentTypeIndices.Reset();
		ArgumentNames.Reset();

		UScriptStruct* FactoryStruct = nullptr;
		Ar << FactoryStruct;

		int32 NumArguments = 0;
		Ar << NumArguments;
		Arguments.AddZeroed(NumArguments);
		for(int32 ArgumentIndex = 0; ArgumentIndex < NumArguments; ArgumentIndex++)
		{
			Arguments[ArgumentIndex].Serialize_NoLock(Ar);
		}

		if(FactoryStruct)
		{
			Factory = InRegistry->FindOrAddDispatchFactory_NoLock(FactoryStruct);
		}
		else
		{
			Factory = nullptr;

			// try to retrieve the function pointer from the global registry.
			// in non-editor environments the global registry will still contain
			// function pointers (but not the types and other expensive storage).
			if(const FRigVMFunction* GlobalFunction = FRigVMRegistry::Get().FindFunction(*Name))
			{
				FunctionPtr = GlobalFunction->FunctionPtr;
#if WITH_EDITOR
				checkf(GlobalFunction->Struct == Struct, TEXT("Struct mismatch when looking for function '%s'. GlobalFunction->Struct:%s Struct:%s"), *Name, *GetPathNameSafe(GlobalFunction->Struct), *GetPathNameSafe(Struct));

				auto CompareArguments = [GlobalFunction, Owner](const FRigVMFunctionArgument& SerializedArg, const FRigVMFunctionArgument& CurrentArg)
				{
					if (SerializedArg == CurrentArg)
					{
						return;
					}
					UE_LOGF(LogRigVM, Error, "Localized Registry Function '%ls' mismatch for argument '%ls'. Please outdate function instead of changing them in the code and roll back potential changes. Owner: [%ls]", *GlobalFunction->Struct->GetStructCPPName(), CurrentArg.Name, Owner ? *Owner->GetPathName() : TEXT("null"));
				};

				// do a rough pass to see of all arguments match.
				bool bAllArgumentsMatch = true;
				if (GlobalFunction->Arguments.Num() == Arguments.Num())
				{
					for(int32 Arg = 0; Arg < Arguments.Num(); Arg++)
					{
						if(Arguments[Arg] != GlobalFunction->Arguments[Arg])
						{
							bAllArgumentsMatch = false;
						}
					}
				}
				
				// if we have more arguments in the serialized file compared
				// to what we have now - we need to let users know about the missing arguments.
				if(GlobalFunction->Arguments.Num() < Arguments.Num())
				{
					TMap<FName, int32> GlobalArgumentToIndex;
					for(int32 GlobalArgIndex = 0; GlobalArgIndex < GlobalFunction->Arguments.Num(); GlobalArgIndex++)
					{
						GlobalArgumentToIndex.Add(GlobalFunction->Arguments[GlobalArgIndex].Name, GlobalArgIndex);
					}
					for(int32 SerializedArgIndex = 0; SerializedArgIndex < Arguments.Num(); SerializedArgIndex++)
					{
						const FName ArgName = Arguments[SerializedArgIndex].Name;
						const int32* GlobalArgIndex = GlobalArgumentToIndex.Find(ArgName);
						if (!GlobalArgIndex)
						{
							UE_LOGF(LogRigVM, Error, "Localized Registry Function '%ls' is missing formerly existing argument '%ls'. Please outdate function instead of changing them in the code and roll back potential changes. Owner: [%ls]", *GlobalFunction->Struct->GetStructCPPName(), *ArgName.ToString(), Owner ? *Owner->GetPathName() : TEXT("null"));
							continue;
						}
						CompareArguments(Arguments[SerializedArgIndex], GlobalFunction->Arguments[*GlobalArgIndex]);
					}
				}

				// if we have more or sufficient arguments now,
				// we still need to check mismatches.
				else if (!bAllArgumentsMatch)
				{
					TMap<FName, int32> SerializedArgumentToIndex;
					for(int32 SerializedArgIndex = 0; SerializedArgIndex < Arguments.Num(); SerializedArgIndex++)
					{
						SerializedArgumentToIndex.Add(Arguments[SerializedArgIndex].Name, SerializedArgIndex);
					}
					for(int32 GlobalArgIndex = 0; GlobalArgIndex < GlobalFunction->Arguments.Num(); GlobalArgIndex++)
					{
						const FName ArgName = GlobalFunction->Arguments[GlobalArgIndex].Name;
						const int32* SerializedArgIndex = SerializedArgumentToIndex.Find(ArgName);
						if (!SerializedArgIndex)
						{
							// if the function has more arguments now - that's not in the file - that's ok.
							continue;
						}
						CompareArguments(Arguments[*SerializedArgIndex], GlobalFunction->Arguments[GlobalArgIndex]);
					}
				}
#endif
			}
		}
	}
	else if(Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector())
	{
		UScriptStruct* FactoryStruct = nullptr;
		if(Factory)
		{
			FactoryStruct = Factory->GetScriptStruct();
		}
		Ar << FactoryStruct;

		int32 NumArguments = Arguments.Num();
		Ar << NumArguments;
		for(FRigVMFunctionArgument& Argument : Arguments)
		{
			Argument.Serialize_NoLock(Ar);
		}
	}
}

#if WITH_EDITOR

bool FRigVMFunction::IsFixedSizeArray(const FName& InArgumentName) const
{
	if (Factory)
	{
		static const FString TrueString = TEXT("True");
		return Factory->GetArgumentMetaData(InArgumentName, FRigVMStruct::FixedSizeArrayMetaName) == TrueString;
	}
	if (Struct)
	{
		const FProperty* Property = Struct->FindPropertyByName(InArgumentName);
		check(Property);
		return Property->HasMetaData(FRigVMStruct::FixedSizeArrayMetaName); 
	}
	return false;
}

bool FRigVMFunction::IsLazyValue(const FName& InArgumentName) const
{
	if (Factory)
	{
		static const FString TrueString = TEXT("True");
		return Factory->GetArgumentMetaData(InArgumentName, FRigVMStruct::ComputeLazilyMetaName) == TrueString;
	}
	if (Struct)
	{
		const FProperty* Property = Struct->FindPropertyByName(InArgumentName);
		check(Property);
		return Property->HasMetaData(FRigVMStruct::ComputeLazilyMetaName); 
	}
	return false;
}

#endif