// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMGraphFunctionHost.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMGraphFunctionHost)

const FRigVMGraphFunctionData* FRigVMGraphFunctionStore::FindFunctionImpl(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic) const
{
	const FRigVMGraphFunctionData* Info = PublicFunctions.FindByPredicate([InLibraryPointer](const FRigVMGraphFunctionData& Info)
	{
		return Info.Header.LibraryPointer == InLibraryPointer;
	});
	if (Info)
	{
		if (bOutIsPublic)
		{
			(*bOutIsPublic) = true; 
		}
		return Info;
	}

	Info = PrivateFunctions.FindByPredicate([InLibraryPointer](const FRigVMGraphFunctionData& Info)
	{
		return Info.Header.LibraryPointer == InLibraryPointer;
	});
	if (Info)
	{
		if (bOutIsPublic)
		{
			(*bOutIsPublic) = false; 
		}
		return Info;
	}
	return nullptr;
}

const FRigVMGraphFunctionData* FRigVMGraphFunctionStore::FindFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic) const
{
	return FindFunctionImpl(InLibraryPointer, bOutIsPublic);
}

FRigVMGraphFunctionData* FRigVMGraphFunctionStore::FindFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic)
{
	return const_cast<FRigVMGraphFunctionData*>(FindFunctionImpl(InLibraryPointer, bOutIsPublic));
}

FRigVMGraphFunctionData* FRigVMGraphFunctionStore::FindFunctionByName(const FName& Name, bool* bOutIsPublic)
{
	FRigVMGraphFunctionData* Info = PublicFunctions.FindByPredicate([Name](const FRigVMGraphFunctionData& Info)
	{
		return Info.Header.Name == Name;
	});
	if (Info)
	{
		if (bOutIsPublic)
		{
			(*bOutIsPublic) = true; 
		}
		return Info;
	}

	Info = PrivateFunctions.FindByPredicate([Name](const FRigVMGraphFunctionData& Info)
	{
		return Info.Header.Name == Name;
	});
	if (Info)
	{
		if (bOutIsPublic)
		{
			(*bOutIsPublic) = false; 
		}
		return Info;
	}
	return nullptr;
}

bool FRigVMGraphFunctionStore::ContainsFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer) const
{
	if(FindFunction(InLibraryPointer))
	{
		return true;
	}
	return false;
}

bool FRigVMGraphFunctionStore::IsFunctionPublic(const FRigVMGraphFunctionIdentifier& InLibraryPointer) const
{
	bool bIsPublic;
	if (FindFunction(InLibraryPointer, &bIsPublic))
	{
		return bIsPublic;
	}
	return false;
}

FRigVMGraphFunctionData* FRigVMGraphFunctionStore::AddFunction(const FRigVMGraphFunctionHeader& FunctionHeader, bool bIsPublic)
{
	// Fail if the function already exists
	if (ContainsFunction(FunctionHeader.LibraryPointer))
	{
		return nullptr;
	}

	FRigVMGraphFunctionData NewInfo(FunctionHeader);
	if(bIsPublic)
	{
		int32 Index = PublicFunctions.Add(NewInfo);
		return &PublicFunctions[Index];
	}
	else
	{
		int32 Index =  PrivateFunctions.Add(NewInfo);
		return &PrivateFunctions[Index];
	}
}

bool FRigVMGraphFunctionStore::RemoveFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bIsPublic)
{
	int32 NumRemoved = PublicFunctions.RemoveAll([InLibraryPointer](const FRigVMGraphFunctionData& Info)
	{
		return Info.Header.LibraryPointer == InLibraryPointer;
	});
	if (NumRemoved > 0 && bIsPublic)
	{
		*bIsPublic = true;
	}
		
	if (NumRemoved == 0)
	{
		NumRemoved = PrivateFunctions.RemoveAll([InLibraryPointer](const FRigVMGraphFunctionData& Info)
		{
		   return Info.Header.LibraryPointer == InLibraryPointer;
		});
		if (NumRemoved > 0 && bIsPublic)
		{
			*bIsPublic = false;
		}
	}

	return NumRemoved > 0;
}

bool FRigVMGraphFunctionStore::MarkFunctionAsPublic(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool bIsPublic)
{
	TArray<FRigVMGraphFunctionData>* OldContainer = nullptr;
	TArray<FRigVMGraphFunctionData>* NewContainer = nullptr;
	if(bIsPublic)
	{
		OldContainer = &PrivateFunctions;
		NewContainer = &PublicFunctions;
	}
	else
	{
		OldContainer = &PublicFunctions;
		NewContainer = &PrivateFunctions;
	}
	
	int32 Index = OldContainer->IndexOfByPredicate([InLibraryPointer](const FRigVMGraphFunctionData& FunctionData)
		{
			return FunctionData.Header.LibraryPointer == InLibraryPointer;
		});
	if(Index == INDEX_NONE)
	{
		return false;
	}
	const int32 NewIndex = NewContainer->Add(OldContainer->operator[](Index));
	OldContainer->RemoveAt(Index);
	FRigVMGraphFunctionData& FunctionData = NewContainer->operator[](NewIndex);
	if(!bIsPublic)
	{
		FunctionData.SerializedCollapsedNode_DEPRECATED.Empty();
		FunctionData.CollapseNodeArchive.Empty();
	}
	return true;
}

FRigVMGraphFunctionData* FRigVMGraphFunctionStore::UpdateFunctionInterface(const FRigVMGraphFunctionHeader& Header)
{
	bool bIsPublic;

	if (const FRigVMGraphFunctionData* OldData = FindFunction(Header.LibraryPointer))
	{
		TMap<FRigVMGraphFunctionIdentifier, uint32> Dependencies = OldData->Header.Dependencies;
		TArray<FRigVMExternalVariable> ExternalVariables = OldData->Header.ExternalVariables;
		if (RemoveFunction(Header.LibraryPointer, &bIsPublic))
		{
			if (FRigVMGraphFunctionData* NewData = AddFunction(Header, bIsPublic))
			{
				NewData->Header.Variant = Header.Variant;
				NewData->Header.Dependencies = Dependencies;
				NewData->Header.ExternalVariables = ExternalVariables;
				NewData->PatchExternalVariablesToArguments();
				return NewData;
			}
		}
	}
	return nullptr;
}

bool FRigVMGraphFunctionStore::UpdateDependencies(const FRigVMGraphFunctionIdentifier& InLibraryPointer, TMap<FRigVMGraphFunctionIdentifier, uint32>& Dependencies)
{
	if (FRigVMGraphFunctionData* Data = FindFunction(InLibraryPointer))
	{
		FRigVMGraphFunctionHeader& Header = Data->Header;
		
		// Check if they are the same
		if (Header.Dependencies.Num() == Dependencies.Num())
		{
			bool bFoundDifference = false;
			for (const TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : Dependencies)
			{
				if (uint32* Hash = Header.Dependencies.Find(Pair.Key))
				{
					if (*Hash != Pair.Value)
					{
						bFoundDifference = true;
						break;
					}
				}
				else
				{
					bFoundDifference = true;
					break;
				}
			}
			if (!bFoundDifference)
			{
				return false;
			}
		}
		
		Data->Header.Dependencies = Dependencies;		
		return true;
	}
	return false;
}

bool FRigVMGraphFunctionStore::UpdateExternalVariables(const FRigVMGraphFunctionIdentifier& InLibraryPointer, TArray<FRigVMExternalVariable> InExternalVariables)
{
	if (FRigVMGraphFunctionData* Data = FindFunction(InLibraryPointer))
	{
		Data->PatchExternalVariablesToArguments();
		
		FRigVMGraphFunctionHeader& Header = Data->Header;

		// Check if they are the same
		const TArray<FRigVMExternalVariable>& HeaderExternalVariables = Header.GetExternalVariables();
		if (HeaderExternalVariables.Num() == InExternalVariables.Num())
		{
			bool bFoundDifference = false;
			for (const FRigVMExternalVariable& Variable : InExternalVariables)
			{
				if (!HeaderExternalVariables.ContainsByPredicate([Variable](const FRigVMExternalVariable& ExternalVariable)
				{
					return Variable.GetName() == ExternalVariable.GetName() &&
							Variable.IsSameType(ExternalVariable) &&
							Variable.IsPublic() == ExternalVariable.IsPublic() &&
							Variable.IsReadOnly() == ExternalVariable.IsReadOnly();
				}))
				{
					bFoundDifference = true;
					break;
				}
			}
			if (!bFoundDifference)
			{
				return false;
			}
		}

		if (!IsFunctionPublic(InLibraryPointer))
		{
			bool bFoundDifference = false;
			Header.ExternalVariables.Reset();
			// add missing variable based arguments
			for (const FRigVMExternalVariable& ExternalVariable : InExternalVariables)
			{
				if (Header.Arguments.ContainsByPredicate([&ExternalVariable](const FRigVMGraphFunctionArgument& Argument) -> bool
				{
					return Argument.Name == ExternalVariable.GetName();
				}))
				{
					continue;
				}
				FRigVMExternalVariable::MergeExternalVariable(Header.ExternalVariables, ExternalVariable);
				bFoundDifference = true;
			}
			return bFoundDifference;
		}
		
		// remove obsolete variable based arguments
		Header.Arguments.RemoveAll([InExternalVariables](const FRigVMGraphFunctionArgument& Argument)
		{
			if (!Argument.bIsInputVariable)
			{
				return false;
			}
			if(Argument.Direction != ERigVMPinDirection::Input)
			{
				return false;
			}
			// remove arguments for which we can't find the matching variable 
			return !InExternalVariables.ContainsByPredicate([&Argument](const FRigVMExternalVariable& ExternalVariable) -> bool
			{
				return Argument.Name == ExternalVariable.GetName();
			});
		});
		
		// add missing variable based arguments
		for (const FRigVMExternalVariable& ExternalVariable : InExternalVariables)
		{
			if (Header.Arguments.ContainsByPredicate([&ExternalVariable](const FRigVMGraphFunctionArgument& Argument) -> bool
			{
				return Argument.Name == ExternalVariable.GetName();
			}))
			{
				continue;
			}
			Header.Arguments.Emplace(ExternalVariable);
		}
		
		// update existing variables
		for (const FRigVMExternalVariable& ExternalVariable : InExternalVariables)
		{
			FRigVMGraphFunctionArgument* Argument = Header.Arguments.FindByPredicate([&ExternalVariable](const FRigVMGraphFunctionArgument& Argument) -> bool
			{
				return Argument.Name == ExternalVariable.GetName();
			});
			if (!Argument)
			{
				continue;
			}
			// update the content of the argument
			*Argument = FRigVMGraphFunctionArgument(ExternalVariable);
		}
		
		return true;
	}
	return false;
}

bool FRigVMGraphFunctionStore::UpdateFunctionCompilationData(const FRigVMGraphFunctionIdentifier& InLibraryPointer, const FRigVMFunctionCompilationData& CompilationData)
{
	FRigVMGraphFunctionData* Info = FindFunction(InLibraryPointer);
	if (Info)
	{
		if (Info->CompilationData.Hash == CompilationData.Hash)
		{
			return false;
		}
		
		Info->CompilationData = CompilationData;
		return true;
	}

	return false;
}

bool FRigVMGraphFunctionStore::RemoveFunctionCompilationData(const FRigVMGraphFunctionIdentifier& InLibraryPointer)
{
	FRigVMGraphFunctionData* FunctionData = FindFunction(InLibraryPointer);
	if (FunctionData)
	{
		FunctionData->ClearCompilationData();
		return true;
	}
	return false;
}

bool FRigVMGraphFunctionStore::RemoveAllCompilationData()
{
	for (FRigVMGraphFunctionData& Data : PublicFunctions)
	{
		Data.ClearCompilationData();
	}

	for (FRigVMGraphFunctionData& Data : PrivateFunctions)
	{
		Data.ClearCompilationData();
	}

	return true;
}
