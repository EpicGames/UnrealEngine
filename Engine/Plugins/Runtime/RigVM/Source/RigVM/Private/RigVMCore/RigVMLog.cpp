// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMLog.h"

#if WITH_EDITOR
TArray<FRigVMLog::FLogEntry> FRigVMLog::GetEntries(EMessageSeverity::Type InSeverity, bool bIncludeHigherSeverity)
{
	return Entries.FilterByPredicate([InSeverity, bIncludeHigherSeverity](const FRigVMLog::FLogEntry& Entry)
	{
		if (bIncludeHigherSeverity)
		{
			return Entry.Severity <= InSeverity;
		}
		return Entry.Severity == InSeverity;
	});
}
#endif

void FRigVMLog::Reset()
{
#if WITH_EDITOR
	Entries.Reset();
	KnownMessages.Reset();
#endif
}

void FRigVMLog::Report(const FRigVMLogSettings& InLogSettings, const FName& InFunctionName, int32 InInstructionIndex, const FString& InMessage)
{
#if WITH_EDITOR

	if (!InMessage.IsEmpty() && InLogSettings.bLogOnce)
	{
		if (KnownMessages.Contains(InMessage))
		{
			return;
		}
	}

	int32 EntryIndex = Entries.Add(FLogEntry(InLogSettings.Severity, InFunctionName, InInstructionIndex, InMessage));

	if(InLogSettings.bLogOnce)
	{
		KnownMessages.Add(InMessage, true);
	}
#endif
}

#if WITH_EDITOR

void FRigVMLog::RemoveRedundantEntries()
{
	if(Entries.IsEmpty())
	{
		return;
	}

	TArray<FLogEntry> OldEntries;
	Swap(OldEntries, Entries);

	// find a single entry for each severity index per instruction.
	// this prefers the latest (most recent) entries over old entries.
	TMap<int32, FIntVector3> EntriesPerInstruction;
	for(int32 EntryIndex = 0; EntryIndex < OldEntries.Num(); EntryIndex++)
	{
		const FLogEntry& OldEntry = OldEntries[EntryIndex];
		if(OldEntry.InstructionIndex == INDEX_NONE)
		{
			Entries.Add(OldEntry);
			continue;
		}
		
		const int32 SeverityIndex = OldEntry.Severity == EMessageSeverity::Error ? 2 :
			(OldEntry.Severity == EMessageSeverity::Warning ? 1 : 0);
		
		FIntVector3& EntryPerInstruction =
			EntriesPerInstruction.FindOrAdd(OldEntry.InstructionIndex, {INDEX_NONE, INDEX_NONE, INDEX_NONE});

		EntryPerInstruction[SeverityIndex] = EntryIndex;
	}

	// add the entries back - culling redundant / least recent entries
	for(const TPair<int32, FIntVector3>& Pair : EntriesPerInstruction)
	{
		for(int32 SeverityIndex = 0; SeverityIndex < 3; SeverityIndex++)
		{
			const int32 EntryIndex = Pair.Value[SeverityIndex];
			if(EntryIndex != INDEX_NONE)
			{
				Entries.Add(OldEntries[EntryIndex]);
			}
		}
	}
}

#endif