// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMMemoryCommon.h"
#include "RigVMModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMMemoryCommon)

#if DEBUG_RIGVMMEMORY
	DEFINE_LOG_CATEGORY(LogRigVMMemory);
#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRigVMInstructionSetExecuteState::RequiresExecute(const uint32& InSliceHash, const uint32& InHashForLazyBranch)
{
	uint32& StoredHash = SliceHashToCallStackHash.FindOrAdd(InSliceHash, UINT32_MAX);
	if(StoredHash != InHashForLazyBranch)
	{
		StoredHash = InHashForLazyBranch;
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMOperand::Serialize(FArchive& Ar)
{
	Ar << MemoryType;
	Ar << RegisterIndex;
	Ar << RegisterOffset;
}

FString FRigVMOperand::ToString() const
{
	const FString MemoryPrefix = StaticEnum<ERigVMMemoryType>()->GetDisplayNameTextByValue((int64)GetMemoryType()).ToString();
	return FString::Printf(TEXT("%s(%d,%d)"), *MemoryPrefix, GetRegisterIndex(), GetRegisterOffset());
}

void FRigVMMemoryStorageImportErrorContext::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (bLogErrors)
	{
#if WITH_EDITOR
		UE_LOGF(LogRigVM, Display, "Skipping Importing To MemoryStorage: %ls", V);
#else
		UE_LOGF(LogRigVM, Error, "Error Importing To MemoryStorage: %ls", V);
#endif
	}
	NumErrors++;
}
