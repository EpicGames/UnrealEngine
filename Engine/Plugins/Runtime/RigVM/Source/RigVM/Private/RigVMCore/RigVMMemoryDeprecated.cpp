// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMMemoryDeprecated.h"
#include "UObject/AnimObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Package.h"
#include "RigVMModule.h"
#include "UObject/CoreRedirects.h"
#include "RigVMTypeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMMemoryDeprecated)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMRegister::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		return;
	}

	uint16 SliceIndex = 0;

	Ar << Type;
	Ar << ByteIndex;
	Ar << ElementSize;
	Ar << ElementCount;
	Ar << SliceIndex;
	Ar << SliceCount;
	Ar << AlignmentBytes;
	Ar << TrailingBytes;
	Ar << Name;
	Ar << ScriptStructIndex;

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		checkNoEntry();
	}
}

void FRigVMRegister::Save(FArchive& Ar)
{
	Ar << bIsArray;
	Ar << bIsDynamic;
}

void FRigVMRegister::Load(FArchive& Ar)
{
	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeRigVMRegisterArrayState)
	{
		Ar << bIsArray;
	}
	else
	{
		bIsArray = false;
	}

	if (Ar.CustomVer(FAnimObjectVersion::GUID) >= FAnimObjectVersion::SerializeRigVMRegisterDynamicState)
	{
		Ar << bIsDynamic;
	}
	else
	{
		bIsDynamic = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMRegisterOffset::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
	{
		return;
	}

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		checkNoEntry();
	}
}

void FRigVMRegisterOffset::Save(FArchive& Ar)
{
	Ar << Segments;
	Ar << Type;
	Ar << CPPType;

	Ar << ScriptStruct;
	
	Ar << ElementSize;
	Ar << ParentScriptStruct;
	Ar << CachedSegmentPath;
	Ar << ArrayIndex;
}

void FRigVMRegisterOffset::Load(FArchive& Ar)
{
	Ar << Segments;
	Ar << Type;
	Ar << CPPType;

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::SerializeRigVMOffsetSegmentPaths)
	{
		FName ScriptStructPath;
		Ar << ScriptStructPath;

		if(ScriptStructPath.IsNone())
		{
			ScriptStruct = nullptr;
		}
		else
		{
			ScriptStruct = UClass::TryFindTypeSlow<UScriptStruct>(ScriptStructPath.ToString());
		}
	}
	else
	{
		Ar << ScriptStruct;
	}

	Ar << ElementSize;

	if (Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::SerializeRigVMOffsetSegmentPaths)
	{
		FString SegmentPath;
		Ar << ParentScriptStruct;
		Ar << SegmentPath;
		Ar << ArrayIndex;

		if (Ar.IsTransacting())
		{
			CachedSegmentPath = SegmentPath;
		}
		else if (ParentScriptStruct != nullptr)
		{
			// if segment path is empty, it implies that the register offset refers to an element in a struct array
			// so segments also need to be recalculated
			int32 InitialOffset = ArrayIndex * ParentScriptStruct->GetStructureSize();
			FRigVMRegisterOffset TempOffset(ParentScriptStruct, SegmentPath, InitialOffset, ElementSize);
			if(TempOffset.Type == ERigVMRegisterType::Invalid)
			{
				Type = ERigVMRegisterType::Invalid;
				CachedSegmentPath = SegmentPath;
				return;
			}
			
			if (TempOffset.GetSegments().Num() == Segments.Num())
			{
				Segments = TempOffset.GetSegments();
				CachedSegmentPath = SegmentPath;
			}
			else
			{
				checkNoEntry();
			}
		}
	}
}

FRigVMRegisterOffset::FRigVMRegisterOffset(UScriptStruct* InScriptStruct, const FString& InSegmentPath, int32 InInitialOffset, uint16 InElementSize, const FName& InCPPType)
	: Segments()
	, Type(ERigVMRegisterType::Plain)
	, CPPType(InCPPType)
	, ScriptStruct(nullptr)
	, ParentScriptStruct(nullptr)
	, ArrayIndex(0)
	, ElementSize(InElementSize)
	, CachedSegmentPath(InSegmentPath)
{
	ParentScriptStruct = InScriptStruct;

	if (ParentScriptStruct)
	{
		ArrayIndex = InInitialOffset / InScriptStruct->GetStructureSize();
	}
	else
	{
		ArrayIndex = InInitialOffset / InElementSize;
	}

	struct FRigVMRegisterOffsetBuilder
	{
		static bool WalkStruct(UStruct* InStruct, const FString& InPath, FRigVMRegisterOffset& Offset)
		{
			FString Left, Right;
			if (!InPath.Split(TEXT("."), &Left, &Right))
			{
				Left = InPath;
			}

			FProperty* Property = InStruct->FindPropertyByName(*Left);
			if(Property == nullptr)
			{
				return false;
			}

			int32 SegmentIndex = Property->GetOffset_ReplaceWith_ContainerPtrToValuePtr();
			if (Offset.Segments.Num() > 0)
			{
				if (Offset.Segments.Last() >= 0)
				{
					Offset.Segments[Offset.Segments.Num() - 1] += SegmentIndex;
				}
				else
				{
					Offset.Segments.Add(SegmentIndex);
				}
			}
			else
			{
				Offset.Segments.Add(SegmentIndex);
			}

			if (!Right.IsEmpty())
			{
				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					return WalkStruct(StructProperty->Struct, Right, Offset);
				}
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					return WalkArray(ArrayProperty, Right, Offset);
				}
			}
			else
			{
				Offset.CPPType = *Property->GetCPPType();
				Offset.ElementSize = static_cast<uint16>(Property->GetElementSize());

				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					Offset.Segments.Add(-1);
					Property = ArrayProperty->Inner;

					Offset.CPPType = *ArrayProperty->Inner->GetCPPType();
					Offset.ElementSize = static_cast<uint16>(ArrayProperty->Inner->GetElementSize());
				}

				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					Offset.ScriptStruct = StructProperty->Struct;
					Offset.Type = ERigVMRegisterType::Struct;
				}
				else
				{
					Offset.Type = ERigVMRegisterType::Plain;
				}
			}

			return true;
		}

		static bool WalkArray(FArrayProperty* InArrayProperty, const FString& InPath, FRigVMRegisterOffset& Offset)
		{
			FString Left, Right;
			if (!InPath.Split(TEXT("."), &Left, &Right))
			{
				Left = InPath;
			}

			int32 ArrayIndex = FCString::Atoi(*Left);
			int32 SegmentIndex = -1 - InArrayProperty->Inner->GetElementSize() * ArrayIndex;

			if (Offset.Segments.Num() > 0)
			{
				if (Offset.Segments.Last() == 0)
				{
					Offset.Segments[Offset.Segments.Num() - 1] = SegmentIndex;
				}
				else
				{
					Offset.Segments.Add(SegmentIndex);
				}
			}
			else
			{
				Offset.Segments.Add(SegmentIndex);
			}

			if (!Right.IsEmpty())
			{
				if (FStructProperty* StructProperty = CastField<FStructProperty>(InArrayProperty->Inner))
				{
					return WalkStruct(StructProperty->Struct, Right, Offset);
				}
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InArrayProperty->Inner))
				{
					return WalkArray(ArrayProperty, Right, Offset);
				}
			}
			else
			{
				Offset.CPPType = *InArrayProperty->Inner->GetCPPType();
				Offset.ElementSize = static_cast<uint16>(InArrayProperty->Inner->GetElementSize());

				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InArrayProperty->Inner))
				{
					Offset.Segments.Add(-1);
					InArrayProperty = ArrayProperty;
				}

				if (FStructProperty* StructProperty = CastField<FStructProperty>(InArrayProperty->Inner))
				{
					Offset.ScriptStruct = StructProperty->Struct;
					Offset.Type = ERigVMRegisterType::Struct;
				}
				else
				{
					Offset.Type = ERigVMRegisterType::Plain;
				}
			}

			return true;
		}
	};

	Segments.Add(InInitialOffset);

	// if segment path is not empty, it implies that the register offset refers to a sub-property in a struct pin
	if (!InSegmentPath.IsEmpty())
	{
		ensure(!InSegmentPath.IsEmpty());
		check(InScriptStruct)
		FString SegmentPath = InSegmentPath;
		SegmentPath = SegmentPath.Replace(TEXT("["), TEXT("."));
		SegmentPath = SegmentPath.Replace(TEXT("]"), TEXT("."));
		if(!FRigVMRegisterOffsetBuilder::WalkStruct(InScriptStruct, SegmentPath, *this))
		{
			Type = ERigVMRegisterType::Invalid;
			return;
		}
		
		if (Type == ERigVMRegisterType::Plain)
		{
			if (CPPType == TEXT("FName"))
			{
				Type = ERigVMRegisterType::Name;
			}
			else if (CPPType == TEXT("FString"))
			{
				Type = ERigVMRegisterType::String;
			}
		}
	}
	else
	{
		// if segment path is empty, it implies that the register offset refers to an element in a struct array
		if (ParentScriptStruct)
		{
			ScriptStruct = ParentScriptStruct;
			Type = ERigVMRegisterType::Struct;
			CPPType = *ScriptStruct->GetStructCPPName();
		}
	}

	ensure(ElementSize > 0);
}

uint8* FRigVMRegisterOffset::GetData(uint8* InContainer) const
{
	uint8* Data = InContainer;
	for(int32 SegmentIndex : Segments)
	{
		if (SegmentIndex < 0)
		{
			int32 ArrayOffset = (-SegmentIndex) - 1;
			FRigVMByteArray* ArrayPtr = (FRigVMByteArray*)Data;
			Data = ArrayPtr->GetData() + ArrayOffset;
		}
		else
		{
			Data = Data + SegmentIndex;
		}
	}
	return Data;
}

bool FRigVMRegisterOffset::ContainsArraySegment() const
{
	for (int32 SegmentIndex : Segments)
	{
		if (SegmentIndex < 0) // Segement indices less than zero represent array element offsets.
		{
			return true;
		}
	}
	return false;
}

bool FRigVMRegisterOffset::operator == (const FRigVMRegisterOffset& InOther) const
{
	if (Segments.Num() != InOther.Segments.Num())
	{
		return false;
	}
	if (ElementSize != InOther.ElementSize)
	{
		return false;
	}
	if (GetScriptStruct() != InOther.GetScriptStruct())
	{
		return false;
	}
	if (ParentScriptStruct != InOther.ParentScriptStruct)
	{
		return false;
	}
	if (CachedSegmentPath != InOther.CachedSegmentPath)
	{
		return false;
	}
	for (int32 Index = 0; Index < Segments.Num(); Index++)
	{
		if (Segments[Index] != InOther.Segments[Index])
		{
			return false;
		}
	}
	return true;
}

UScriptStruct* FRigVMRegisterOffset::GetScriptStruct() const
{
	return ScriptStruct;
}

uint16 FRigVMRegisterOffset::GetElementSize() const
{
	return ElementSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FRigVMMemoryContainerImportErrorContext : public FOutputDevice
{
public:

	int32 NumErrors;

	FRigVMMemoryContainerImportErrorContext()
		: FOutputDevice()
		, NumErrors(0)
	{
	}

	void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
#if WITH_EDITOR
		UE_LOGF(LogRigVM, Display, "Skipping Importing To MemoryContainer: %ls", V);
#else
		UE_LOGF(LogRigVM, Error, "Error Importing To MemoryContainer: %ls", V);
#endif
		NumErrors++;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FRigVMMemoryContainer::FRigVMMemoryContainer(bool bInUseNames)
	: bUseNameMap(bInUseNames)
	, MemoryType(ERigVMMemoryType::Work)
	, bEncounteredErrorDuringLoad(false)
{
}

FRigVMMemoryContainer::FRigVMMemoryContainer(const FRigVMMemoryContainer& Other)
{
	*this = Other;
}

FRigVMMemoryContainer::~FRigVMMemoryContainer()
{
	Reset();
}

bool FRigVMMemoryContainer::CopyRegisters(const FRigVMMemoryContainer &InOther)
{
	ensure(Registers.Num() == InOther.Registers.Num());
	return true;
}

FRigVMMemoryContainer& FRigVMMemoryContainer::operator= (const FRigVMMemoryContainer &InOther)
{
	Reset();

	MemoryType = InOther.MemoryType;
	bUseNameMap = InOther.bUseNameMap;
	bEncounteredErrorDuringLoad = false;
	Data.Append(InOther.Data);
	Registers.Append(InOther.Registers);
	RegisterOffsets.Append(InOther.RegisterOffsets);
	ScriptStructs.Append(InOther.ScriptStructs);

	UpdateRegisters();

	for (int32 Index = 0; Index < Registers.Num(); Index++)
	{
		const FRigVMRegister& Register = Registers[Index];
		if (Register.IsNestedDynamic())
		{
			FMemory::Memzero(&Data[static_cast<int32>(Register.GetWorkByteIndex())], sizeof(FRigVMNestedByteArray));
			*(FRigVMNestedByteArray*)&Data[static_cast<int32>(Register.GetWorkByteIndex())] = FRigVMNestedByteArray();
		}
		else if(Register.IsDynamic())
		{
			FMemory::Memzero(&Data[static_cast<int32>(Register.GetWorkByteIndex())], sizeof(FRigVMByteArray));
			*(FRigVMByteArray*)&Data[static_cast<int32>(Register.GetWorkByteIndex())] = FRigVMByteArray();
		}

		Construct(Index);
	}

	return *this;
}

void FRigVMMemoryContainer::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAnimObjectVersion::GUID);

 	if (Ar.CustomVer(FAnimObjectVersion::GUID) < FAnimObjectVersion::StoreMarkerNamesOnSkeleton)
 	{
 		return;
 	}

	if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
	{
		Save(Ar);
	}
	else if (Ar.IsLoading())
	{
		Load(Ar);
	}
	else
	{
		checkNoEntry();
	}
}

void FRigVMMemoryContainer::Save(FArchive& Ar)
{
}

void FRigVMMemoryContainer::Load(FArchive& Ar)
{
	for (int32 RegisterIndex = 0; RegisterIndex < Registers.Num(); RegisterIndex++)
	{
		Destroy(RegisterIndex);
	}
	
	Ar << bUseNameMap;
	Ar << MemoryType;
	Ar << Registers;
	Ar << RegisterOffsets;

#if DEBUG_RIGVMMEMORY
	UE_LOG_RIGVMMEMORY(TEXT("%d Memory - Begin Loading..."), (int32)GetMemoryType());
#endif

	bEncounteredErrorDuringLoad = false;

	for(const FRigVMRegisterOffset& RegisterOffset : RegisterOffsets)
	{
		if(RegisterOffset.GetType() == ERigVMRegisterType::Invalid)
		{
			FString PackagePath = Ar.GetArchiveName();
			UE_LOGF(LogRigVM, Error, "RegisterOffset '%ls' cannot be found. Asset '%ls' no longer functional.", *RegisterOffset.GetCachedSegmentPath(), *PackagePath);
			bEncounteredErrorDuringLoad = true;
		}
	}

	ScriptStructs.Reset();
	TArray<FString> ScriptStructPaths;
	Ar << ScriptStructPaths;

	for (const FString& ScriptStructPath : ScriptStructPaths)
	{
		UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(*ScriptStructPath);

		// try to find a redirect
		if (ScriptStruct == nullptr)
		{
			const FCoreRedirectObjectName OldObjectName(ScriptStructPath);
			const FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Struct, OldObjectName);
			if (OldObjectName != NewObjectName)
			{
				ScriptStruct = FindObject<UScriptStruct>(nullptr, *NewObjectName.ToString());
			}
		}

		// this might have happened if a given script struct no longer 
		// exists or cannot be loaded.
		if (ScriptStruct == nullptr)
		{
			FString PackagePath = Ar.GetArchiveName();
			UE_LOGF(LogRigVM, Error, "Struct '%ls' cannot be found. Asset '%ls' no longer functional.", *ScriptStructPath, *PackagePath);
			bEncounteredErrorDuringLoad = true;
		}

		ScriptStructs.Add(ScriptStruct);
	}

	uint64 TotalBytes = 0;
	Ar << TotalBytes;

	Data.Empty();

	if (!bEncounteredErrorDuringLoad)
	{

		// during load we'll recreate the memory for all registers.
		// the size for structs might have changed, so we need to reallocate.
		for (int32 RegisterIndex = 0; RegisterIndex < Registers.Num(); RegisterIndex++)
		{
			FRigVMRegister& Register = Registers[RegisterIndex];

			if (UScriptStruct* ScriptStruct = GetScriptStruct(Register))
			{
				Register.ElementSize = static_cast<uint16>(ScriptStruct->GetStructureSize());
			}
			else if (Register.Type == ERigVMRegisterType::Name)
			{
				Register.ElementSize = sizeof(FName);
			}
			else if (Register.Type == ERigVMRegisterType::String)
			{
				Register.ElementSize = sizeof(FString);
			}

			Register.AlignmentBytes = 0;
			Register.TrailingBytes = 0;

			if (Registers[RegisterIndex].IsDynamic())
			{
				Register.ByteIndex = Data.AddZeroed(sizeof(FRigVMByteArray));
			}
			else
			{
				Register.ByteIndex = Data.AddZeroed(Register.GetNumBytesAllSlices());
			}
		}

		UpdateRegisters();

		for (int32 RegisterOffsetIndex = 0; RegisterOffsetIndex < RegisterOffsets.Num(); RegisterOffsetIndex++)
		{
			FRigVMRegisterOffset& RegisterOffset = RegisterOffsets[RegisterOffsetIndex];

			if (UScriptStruct* ScriptStruct = RegisterOffset.GetScriptStruct())
			{
				RegisterOffset.SetElementSize(static_cast<uint16>(ScriptStruct->GetStructureSize()));
			}
			if (RegisterOffset.GetType() == ERigVMRegisterType::Name)
			{
				RegisterOffset.SetElementSize(sizeof(FName));
			}
			else if (RegisterOffset.GetType() == ERigVMRegisterType::String)
			{
				RegisterOffset.SetElementSize(sizeof(FString));
			}
		}

		// once the register memory is allocated we can construt its contents.
		for (int32 RegisterIndex = 0; RegisterIndex < Registers.Num(); RegisterIndex++)
		{
			const FRigVMRegister& Register = Registers[RegisterIndex];
			if (!Registers[RegisterIndex].IsDynamic())
			{
				Construct(RegisterIndex);
			}
		}
	}

	for (int32 RegisterIndex = 0; RegisterIndex < Registers.Num(); RegisterIndex++)
	{
		FRigVMRegister& Register = Registers[RegisterIndex];

		if (Register.ElementCount == 0 && !Register.IsDynamic())
		{
			continue;
		}

		// literal mem container should not have any dynamic registers,
		// if we ever added them, we might need to re-consider if we need to serialize them
		// by default dynamic registers' memory are not serialized
		if (MemoryType == ERigVMMemoryType::Literal)
		{
			check(!Register.IsDynamic());
		}
		
		if (!Register.IsDynamic())
		{
			switch (Register.Type)
			{
				case ERigVMRegisterType::Plain:
				{
					FRigVMByteArray View;
					Ar << View;

					if (!bEncounteredErrorDuringLoad)
					{
						ensure(View.Num() <= Register.GetAllocatedBytes());
						FMemory::Memcpy(&Data[static_cast<int32>(Register.GetWorkByteIndex())], View.GetData(), View.Num());
					}
					break;
				}
				case ERigVMRegisterType::Name:
				{
					TArray<FName> View;
					Ar << View;

					if (!bEncounteredErrorDuringLoad)
					{
						ensure(View.Num() == Register.GetTotalElementCount());
						RigVMCopy<FName>(&Data[static_cast<int32>(Register.GetWorkByteIndex())], View.GetData(), View.Num());
					}
					break;
				}
				case ERigVMRegisterType::String:
				{
					TArray<FString> View;
					Ar << View;

					if (!bEncounteredErrorDuringLoad)
					{
						ensure(View.Num() == Register.GetTotalElementCount());
						RigVMCopy<FString>(&Data[static_cast<int32>(Register.GetWorkByteIndex())], View.GetData(), View.Num());
					}
					break;
				}
				case ERigVMRegisterType::Struct:
				{
					TArray<FString> View;
					Ar << View;

					if (!bEncounteredErrorDuringLoad)
					{
						ensure(View.Num() == Register.GetTotalElementCount());

						uint8* DataPtr = &Data[static_cast<int32>(Register.GetWorkByteIndex())];
						UScriptStruct* ScriptStruct = GetScriptStruct(Register);
						if (ScriptStruct && !bEncounteredErrorDuringLoad)
						{
							ensure(ScriptStruct->GetStructureSize() == Register.ElementSize);

							for (uint16 ElementIndex = 0; ElementIndex < Register.GetTotalElementCount(); ElementIndex++)
							{
								FRigVMMemoryContainerImportErrorContext ErrorPipe;
								ScriptStruct->ImportText(*View[ElementIndex], DataPtr, nullptr, PPF_None, &ErrorPipe, ScriptStruct->GetName());
								if (ErrorPipe.NumErrors > 0)
								{
									bEncounteredErrorDuringLoad = true;
									break;
								}
								DataPtr += Register.ElementSize;
							}
						}
					}
					break;
				}
			}
		}
		else if (!Register.IsNestedDynamic())
		{
			// notice that all dynamic registers' memory is not really serialized
			// serialize empty arrays here just to avoid having to increment serialization version
			// the loaded data is not getting used at all, it will be recomputed at execution time
			switch (Register.Type)
			{
				case ERigVMRegisterType::Plain:
				{
					FRigVMByteArray View;
					Ar << View;
					break;
				}
				case ERigVMRegisterType::Name:
				{
					TArray<FName> View;
					Ar << View;
					break;
				}
				case ERigVMRegisterType::String:
				{
					TArray<FString> View;
					Ar << View;
					break;
				}
				case ERigVMRegisterType::Struct:
				{
					TArray<FString> View;
					Ar << View;
					break;
				}
			}
		}
		else
		{
			
			FRigVMNestedByteArray* NestedArrayStorage = nullptr;
			if (!bEncounteredErrorDuringLoad)
			{
				NestedArrayStorage = (FRigVMNestedByteArray*)&Data[static_cast<int32>(Register.GetWorkByteIndex())];
				NestedArrayStorage->Reset();
				NestedArrayStorage->SetNumZeroed(Register.SliceCount);
			}
			
			for (int32 SliceIndex = 0; SliceIndex < Register.SliceCount; SliceIndex++)
			{
				// notice that all dynamic registers' memory is not really serialized
				// serialize empty arrays here just to avoid having to increment serialization version
				// the loaded data is not getting used at all, it will be recomputed at execution time
				switch (Register.Type)
				{
					case ERigVMRegisterType::Plain:
					{
						FRigVMByteArray View;
						Ar << View;
						break;
					}
					case ERigVMRegisterType::Name:
					{
						TArray<FName> View;
						Ar << View;
						break;
					}
					case ERigVMRegisterType::String:
					{
						TArray<FString> View;
						Ar << View;
						break;
					}
					case ERigVMRegisterType::Struct:
					{
						TArray<FString> View;
						Ar << View;
						break;
					}
				}
			}
		}
	}
}

void FRigVMMemoryContainer::Reset()
{
	if (Data.Num() > 0)
	{
		for (int32 Index = 0; Index < Registers.Num(); Index++)
		{
			Destroy(Index);
		}
	}

	Data.Reset();
	Registers.Reset();
	RegisterOffsets.Reset();
	ScriptStructs.Reset();
	NameMap.Reset();
}

void FRigVMMemoryContainer::Empty()
{
	Data.Empty();
	Registers.Empty();
	RegisterOffsets.Empty();
	ScriptStructs.Empty();
	NameMap.Empty();
}

int32 FRigVMMemoryContainer::Allocate(const FName& InNewName, int32 InElementSize, int32 InElementCount, int32 InSliceCount, const uint8* InDataPtr, bool bUpdateRegisters)
{
	FName Name = InNewName;
	if (bUseNameMap && InNewName == NAME_None)
	{
		const TCHAR* RegisterPrefix = TEXT("Register");
		int32 RegisterSuffix = 0;
		do
		{
			Name = FName(*FString::Printf(TEXT("%s_%d"), RegisterPrefix, RegisterSuffix++));
		} while (!IsNameAvailable(Name));
	}

	ensure(InElementSize > 0 && InElementCount >= 0 && InSliceCount > 0);

	if (bUseNameMap)
	{
		if (!IsNameAvailable(Name))
		{
			return INDEX_NONE;
		}
	}

	FRigVMRegister NewRegister;
	NewRegister.ByteIndex = Data.Num();
	if (bUseNameMap)
	{
		NewRegister.Name = Name;
	}
	NewRegister.ElementSize = (uint16)InElementSize;
	NewRegister.ElementCount = (uint16)InElementCount;
	NewRegister.SliceCount = (uint16)InSliceCount;
	NewRegister.Type = ERigVMRegisterType::Plain;

	Data.AddZeroed(NewRegister.GetAllocatedBytes());

#if DEBUG_RIGVMMEMORY
	if (NewRegister.ElementCount > 0)
	{
		UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Allocated %04d bytes at %04d (%s)."), (int32)GetMemoryType(), Registers.Num(), NewRegister.GetAllocatedBytes(), (int32)reinterpret_cast<long long>(&Data[NewRegister.ByteIndex]), *NewRegister.Name.ToString());
	}
#endif

	if (InDataPtr != nullptr)
	{
		for (uint16 SliceIndex = 0; SliceIndex < NewRegister.SliceCount; SliceIndex++)
		{
			FMemory::Memcpy(&Data[static_cast<int32>(NewRegister.GetWorkByteIndex(SliceIndex))], InDataPtr, NewRegister.GetNumBytesPerSlice());
		}
	}

	int32 RegisterIndex = Registers.Num();
	Registers.Add(NewRegister);

	if (bUpdateRegisters)
	{
		UpdateRegisters();
	}
	return RegisterIndex;
}

int32 FRigVMMemoryContainer::Allocate(int32 InElementSize, int32 InElementCount, int32 InSliceCount, const uint8* InDataPtr, bool bUpdateRegisters)
{
	return Allocate(NAME_None, InElementSize, InElementCount, InSliceCount, InDataPtr, bUpdateRegisters);
}

bool FRigVMMemoryContainer::Construct(int32 InRegisterIndex, int32 InElementIndex, int32 InSliceIndex)
{
	ensure(Registers.IsValidIndex(InRegisterIndex));

	const FRigVMRegister& Register = Registers[InRegisterIndex];

	if (Register.ElementCount == 0 || (Register.Type == ERigVMRegisterType::Plain && !Register.IsDynamic()))
	{
		return true;
	}

	int32 ElementIndex = InElementIndex == INDEX_NONE ? 0 : InElementIndex;
	int32 ElementCount = InElementIndex == INDEX_NONE ? Register.GetTotalElementCount() : 1;
	int32 NumSlices = 1;

	uint8* TopDataPtr = &Data[static_cast<int32>(Register.GetWorkByteIndex())];
	if (Register.IsDynamic())
	{
		if (Register.IsNestedDynamic())
		{
			FRigVMNestedByteArray& Storage = *(FRigVMNestedByteArray*)TopDataPtr;
			NumSlices = Storage.Num();
			TopDataPtr = (uint8*)Storage.GetData();
		}
		else
		{
			FRigVMByteArray& Storage = *(FRigVMByteArray*)TopDataPtr;
			ElementCount = Storage.Num() / Register.ElementSize;
			if (ElementCount == 0)
			{
				Storage.AddZeroed(Register.ElementSize);
				ElementCount = 1;
			}
			ensure(ElementCount * Register.ElementSize == Storage.Num());
			TopDataPtr = Storage.GetData();
		}
	}

	for (int32 SliceIndex = 0; SliceIndex < NumSlices; SliceIndex++)
	{
		if (InSliceIndex != INDEX_NONE)
		{
			SliceIndex = InSliceIndex;

			if (Register.IsNestedDynamic())
			{
				TopDataPtr = TopDataPtr + SliceIndex * sizeof(FRigVMByteArray);
			}
		}

		uint8* NestedDataPtr = TopDataPtr;
		if (Register.IsNestedDynamic())
		{
			FRigVMByteArray& Storage = *(FRigVMByteArray*)TopDataPtr;
			ElementCount = Storage.Num() / Register.ElementSize;
			ensure(ElementCount * Register.ElementSize == Storage.Num());
			NestedDataPtr = Storage.GetData();
		}

		switch (Register.Type)
		{
			case ERigVMRegisterType::Struct:
			{
				uint8* DataPtr = NestedDataPtr + ElementIndex * Register.ElementSize;
				UScriptStruct* ScriptStruct = GetScriptStruct(InRegisterIndex);

				if (ScriptStruct)
				{
					if (Register.ElementSize == ScriptStruct->GetStructureSize())
					{
						ScriptStruct->InitializeStruct(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
						UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Initialized struct, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ElementCount * ScriptStruct->GetStructureSize(), (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
					}
				}
				break;
			}
			case ERigVMRegisterType::String:
			{
				FString* DataPtr = (FString*)(NestedDataPtr + ElementIndex * Register.ElementSize);
				RigVMInitialize<FString>(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
				UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Initialized string, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ElementCount * Register.ElementSize, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
				break;
			}
			case ERigVMRegisterType::Name:
			{
				FName* DataPtr = (FName*)(NestedDataPtr + ElementIndex * Register.ElementSize);
				RigVMInitialize<FName>(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
				UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Initialized name, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ElementCount * Register.ElementSize, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
				break;
			}
			default:
			{
				return false;
			}
		}

		if (Register.IsNestedDynamic())
		{
			TopDataPtr = TopDataPtr + sizeof(FRigVMByteArray);
		}

		if (InSliceIndex != INDEX_NONE)
		{
			break;
		}
	}

	return true;
}

bool FRigVMMemoryContainer::Destroy(int32 InRegisterIndex, int32 InElementIndex, int32 InSliceIndex)
{
	FRigVMRegister& Register = Registers[InRegisterIndex];

	if (Register.ElementCount == 0 || (Register.Type == ERigVMRegisterType::Plain && !Register.IsDynamic()))
	{
		return true;
	}

	int32 ElementIndex = InElementIndex == INDEX_NONE ? 0 : InElementIndex;
	int32 ElementCount = InElementIndex == INDEX_NONE ? Register.GetTotalElementCount() : 1;
	int32 NumSlices = 1;

	uint8* TopDataPtr = &Data[static_cast<int32>(Register.GetWorkByteIndex())];
	if (Register.IsDynamic())
	{
		if (Register.IsNestedDynamic())
		{
			FRigVMNestedByteArray& Storage = *(FRigVMNestedByteArray*)TopDataPtr;
			NumSlices = Storage.Num();
			TopDataPtr = (uint8*)Storage.GetData();
		}
		else
		{
			FRigVMByteArray& Storage = *(FRigVMByteArray*)TopDataPtr;
			ElementCount = Storage.Num() / Register.ElementSize;
			ensure(ElementCount * Register.ElementSize == Storage.Num());
			TopDataPtr = Storage.GetData();
		}
	}

	for (int32 SliceIndex = 0; SliceIndex < NumSlices; SliceIndex++)
	{
		if (InSliceIndex != INDEX_NONE) 
		{
			SliceIndex = InSliceIndex;

			if (Register.IsNestedDynamic())
			{
				TopDataPtr = TopDataPtr + SliceIndex * sizeof(FRigVMByteArray);
			}
		}

		uint8* NestedDataPtr = TopDataPtr;
		if (Register.IsNestedDynamic())
		{
			FRigVMByteArray& Storage = *(FRigVMByteArray*)TopDataPtr;
			ElementCount = Storage.Num() / Register.ElementSize;
			ensure(ElementCount * Register.ElementSize == Storage.Num());
			NestedDataPtr = Storage.GetData();
		}

		switch (Register.Type)
		{
			case ERigVMRegisterType::Struct:
			{
				uint8* DataPtr = NestedDataPtr + ElementIndex * Register.ElementSize;

				UScriptStruct* ScriptStruct = GetScriptStruct(InRegisterIndex);
				if (ScriptStruct)
				{
					if (Register.ElementSize != ScriptStruct->GetStructureSize())
					{
						FMemory::Memzero(DataPtr, Register.ElementSize * ElementCount);
#if DEBUG_RIGVMMEMORY
						UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Zeroed struct, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, Register.ElementSize * ElementCount, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
					}
					else
					{
						ScriptStruct->DestroyStruct(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
						UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Destroyed struct, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ScriptStruct->GetStructureSize() * ElementCount, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
					}
				}
				else
				{
					FMemory::Memzero(DataPtr, Register.ElementSize * ElementCount);
#if DEBUG_RIGVMMEMORY
					UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Zeroed struct, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, Register.ElementSize * ElementCount, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
				}
				break;
			}
			case ERigVMRegisterType::String:
			{
				FString* DataPtr = (FString*)(NestedDataPtr + ElementIndex * Register.ElementSize);
				RigVMDestroy<FString>(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
				UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Destroyed string, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ElementCount * Register.ElementSize, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
				break;
			}
			case ERigVMRegisterType::Name:
			{
				FName* DataPtr = (FName*)(NestedDataPtr + ElementIndex * Register.ElementSize);
				RigVMDestroy<FName>(DataPtr, ElementCount);
#if DEBUG_RIGVMMEMORY
				UE_LOG_RIGVMMEMORY(TEXT("%d.%04d: Destroyed name, %04d bytes at %04d (%s)."), (int32)GetMemoryType(), InRegisterIndex, ElementCount * Register.ElementSize, (int32)reinterpret_cast<long long>(DataPtr), *Register.Name.ToString());
#endif
				break;
			}
			default:
			{
				if (!Register.IsDynamic())
				{
					return false;
				}
			}
		}

		if (Register.IsNestedDynamic())
		{
			((FRigVMByteArray*)TopDataPtr)->Empty();
			TopDataPtr = TopDataPtr + sizeof(FRigVMByteArray);
		}

		if (InSliceIndex != INDEX_NONE)
		{
			break;
		}
	}

	if (Register.IsDynamic())
	{
		TopDataPtr = &Data[static_cast<int32>(Register.GetWorkByteIndex())];

		if (Register.IsNestedDynamic())
		{
			if (InSliceIndex != INDEX_NONE)
			{
				((FRigVMNestedByteArray*)TopDataPtr)->Empty();
			}
		}
		else
		{
			((FRigVMByteArray*)TopDataPtr)->Empty();
		}
	}

	return true;
}

int32 FRigVMMemoryContainer::GetOrAddRegisterOffset(int32 InRegisterIndex, int32 InArrayElement)
{
	return GetOrAddRegisterOffset(InRegisterIndex, FString(), InArrayElement);
}

int32 FRigVMMemoryContainer::GetOrAddRegisterOffset(int32 InRegisterIndex, const FString& InSegmentPath, int32 InArrayElement)
{
	ensure(Registers.IsValidIndex(InRegisterIndex));

	UScriptStruct* ScriptStruct = GetScriptStruct(InRegisterIndex);
	int32 ElementSize = Registers[InRegisterIndex].ElementSize;
	int32 InitialOffset = InArrayElement * ElementSize;
	return GetOrAddRegisterOffset(InRegisterIndex, ScriptStruct, InSegmentPath, InitialOffset);
}

int32 FRigVMMemoryContainer::GetOrAddRegisterOffset(int32 InRegisterIndex, UScriptStruct* InScriptStruct, const FString& InSegmentPath, int32 InInitialOffset, int32 InElementSize)
{
	if ((InScriptStruct == nullptr || InSegmentPath.IsEmpty()) && InInitialOffset == 0)
	{
		return INDEX_NONE;
	}

	// if this is a register offset for a external variable
	// the register index is expected to be INDEX_NONE
	// and it is also expected that InElementSize != 0
	if (InElementSize == 0)
	{
		ensure(Registers.IsValidIndex(InRegisterIndex)); 
		InElementSize = Registers[InRegisterIndex].ElementSize;
	}
	
	FName BaseCPPType = NAME_None;
	if(Registers.IsValidIndex(InRegisterIndex))
	{
		if(InElementSize == 0)
		{
			InElementSize = Registers[InRegisterIndex].ElementSize;
		}
#if WITH_EDITORONLY_DATA
		BaseCPPType = Registers[InRegisterIndex].BaseCPPType;
#endif
	}

	FRigVMRegisterOffset Offset(InScriptStruct, InSegmentPath, InInitialOffset, static_cast<uint16>(InElementSize), BaseCPPType);
	int32 ExistingIndex = RegisterOffsets.Find(Offset);
	if (ExistingIndex == INDEX_NONE)
	{
		return RegisterOffsets.Add(Offset);
	}
	return ExistingIndex;
}

void FRigVMMemoryContainer::UpdateRegisters()
{
	int32 AlignmentShift = 0;
	for (int32 RegisterIndex = 0; RegisterIndex < Registers.Num(); RegisterIndex++)
	{
		FRigVMRegister& Register = Registers[RegisterIndex];
		Register.ByteIndex += AlignmentShift;

		int32 Alignment = 4;

		if (Register.IsDynamic() ||
			Register.Type == ERigVMRegisterType::Name ||
			Register.Type == ERigVMRegisterType::String)
		{
			Alignment = 8;
		}
		else if (UScriptStruct* ScriptStruct = GetScriptStruct(RegisterIndex))
		{
			if (UScriptStruct::ICppStructOps* TheCppStructOps = ScriptStruct->GetCppStructOps())
			{
				Alignment = TheCppStructOps->GetAlignment();
			}
		}

		if (Alignment != 0)
		{
			// no need to adjust for alignment if nothing is allocated
			if (!Register.IsDynamic() && Register.ElementCount == 0)
			{
				continue;
			}

			if (ensure(Data.IsValidIndex(static_cast<int32>(Register.GetWorkByteIndex()))))
			{
				uint8* Pointer = (&(Data[static_cast<int32>(Register.GetWorkByteIndex())]));
			
				if (Register.AlignmentBytes > 0)
				{
					if (!IsAligned(Pointer, Alignment))
					{
						Data.RemoveAt(static_cast<int32>(Register.GetFirstAllocatedByte()), Register.AlignmentBytes);
						AlignmentShift -= Register.AlignmentBytes;
						Register.ByteIndex -= Register.AlignmentBytes;
						Register.AlignmentBytes = 0;
						Pointer = (&(Data[static_cast<int32>(Register.GetWorkByteIndex())]));
					}
				}

				while (!IsAligned(Pointer, Alignment))
				{
					Data.InsertZeroed(static_cast<int32>(Register.GetFirstAllocatedByte()), 1);
					Register.AlignmentBytes++;
					Register.ByteIndex++;
					AlignmentShift++;
					Pointer = (&(Data[static_cast<int32>(Register.GetWorkByteIndex())]));
				}
			}
		}
	}

	if (bUseNameMap)
	{
		NameMap.Reset();
		for (int32 Index = 0; Index < Registers.Num(); Index++)
		{
			NameMap.Add(Registers[Index].Name, Index);
		}
	}
}

int32 FRigVMMemoryContainer::FindOrAddScriptStruct(UScriptStruct* InScriptStruct)
{
	if (InScriptStruct == nullptr)
	{
		return INDEX_NONE;
	}

	int32 StructIndex = INDEX_NONE;
	if (ScriptStructs.Find(InScriptStruct, StructIndex))
	{
		return StructIndex;
	}
	return ScriptStructs.Add(InScriptStruct);
}

