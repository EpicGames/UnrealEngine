// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMModule.h"
#include "RigVMObjectVersion.h"
#include "RigVMTypeUtils.h"
#include "UObject/Interface.h"
#include "AssetRegistry/AssetData.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMExecuteContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMMemoryStorage)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Generator class should be parented to the asset object, instead of the package
// because the engine no longer supports multiple 'assets' per package
static UObject* GetGeneratorClassOuter(UPackage* InPackage)
{
	if (!InPackage)
	{
		return nullptr;
	}

	// special case handling for transient package
	if (InPackage == GetTransientPackage())
	{
		return InPackage;
	}

	// find the asset object to use as the class outer
	// not using FindAssetInPackage here because
	// in ControlRigEditor::UpdateControlRig
	// we set the BP object to transient before VM recompilation
	// so FindAssetInPackage would fail in finding the BP object
	// the transient flag was needed such that recompile VM does not dirty BP
	TArray<UObject*> TopLevelObjects;
	GetObjectsWithOuter(InPackage, TopLevelObjects, EGetObjectsFlags::None);
	
	UObject* AssetObject = nullptr;
	for (UObject* Object : TopLevelObjects)
	{
		if (FAssetData::IsUAsset(Object))
		{
			AssetObject = Object;
			break;
		}
	}

	return AssetObject;	
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMBranchInfo::Serialize(FArchive& Ar)
{
	Ar << Index;

	if(Ar.IsLoading())
	{
		FString LabelString;
		Ar << LabelString;
		Label = *LabelString;
	}
	else
	{
		FString LabelString = Label.ToString();
		Ar << LabelString;
	}
	
	Ar << InstructionIndex;
	Ar << ArgumentIndex;

	if (Ar.IsLoading() && Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::ByteCodeCleanup)
	{
		uint16 FirstInstructionTruncated = 0, LastInstructionTruncated = 0;
		Ar << FirstInstructionTruncated;
		Ar << LastInstructionTruncated;

		FirstInstruction = FirstInstructionTruncated;
		LastInstruction = LastInstructionTruncated;
	}
	else
	{
		Ar << FirstInstruction;
		Ar << LastInstruction;
	}
}

ERigVMExecuteResult FRigVMLazyBranch::Execute(FRigVMExtendedExecuteContext& Context)
{
	if(BranchInfo.IsValid())
	{
		check(VM);
		return VM->ExecuteBranch(Context, BranchInfo);
	}

	check(FunctionPtr);
	return FunctionPtr();
}

ERigVMExecuteResult FRigVMLazyBranch::ExecuteIfRequired(FRigVMExtendedExecuteContext& Context, uint32 InSliceHash)
{
	FRigVMInstructionSetExecuteState& ExecuteState = Context.Frame->LazyBranchExecuteState[BranchInfo.Index];
	const uint32 HashForLazyBranch = Context.GetHashForLazyBranch();
	if(ExecuteState.RequiresExecute(InSliceHash, HashForLazyBranch))
	{
		return Execute(Context);
	}
	
	return ERigVMExecuteResult::Succeeded;
}

const uint8* TRigVMLazyValueBase::GetData(const FRigVMExecuteContext& Context) const
{
	if(MemoryHandle)
	{
		if(MemoryHandle->IsLazy())
		{
			check(Context.ExtendedExecuteContext != nullptr);
			MemoryHandle->ComputeLazyValueIfNecessary(*Context.ExtendedExecuteContext, SliceHash);
		}
		return MemoryHandle->GetData_Internal(bFollowPropertyPath, INDEX_NONE);
	}
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRigVMMemoryHandle::ComputeLazyValueIfNecessary(FRigVMExtendedExecuteContext& Context, uint32 InSliceHash)
{
	check(IsLazy());
	return LazyBranch->ExecuteIfRequired(Context, InSliceHash) != ERigVMExecuteResult::Failed;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void URigVMMemoryStorageGeneratorClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);

	// clear our state as well
	CachedMemoryHash = 0;
	LinkedProperties.Reset();
	PropertyPaths.Reset();
	PropertyPathDescriptions.Reset();
}

void URigVMMemoryStorageGeneratorClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	// Force assembly of the reference token stream so that we can be properly handled by the
	// garbage collector.
	AssembleReferenceTokenStream(/*bForce=*/true);

	// rebuild property list and property path list
	RefreshLinkedProperties();
	RefreshPropertyPaths();
}

void URigVMMemoryStorageGeneratorClass::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading() && ChildProperties != nullptr)
	{
		// if there are already properties in this class
		// it means that the VM has regenerated the class before
		// the class' own deserialization took place
		// in that case, the VM generated class should take priority
		// so a dummy class is used here to consume/throw away outdated serialized data
		URigVMMemoryStorageGeneratorClass* AuxStorageClass = NewObject<URigVMMemoryStorageGeneratorClass>(
			GetTransientPackage(),
			TEXT("URigVMMemoryStorageGeneratorClass_Auxiliary"));
		AuxStorageClass->CppClassStaticFunctions.SetAddReferencedObjects(&this->AddReferencedObjects);
		AuxStorageClass->Serialize(Ar);
		return;
	}

	Super::Serialize(Ar);

	if(Ar.IsLoading() || Ar.IsSaving())
	{
		Ar << PropertyPathDescriptions;
		Ar << MemoryType;
	}
}

void URigVMMemoryStorageGeneratorClass::PostLoad()
{
	Super::PostLoad();

	// rebuild property list and property path list
	RefreshLinkedProperties();
	RefreshPropertyPaths();

	// Mark the class as deprecated so it doesn't show up in asset searches
	// and UI elements. This also avoids the call into
	// GetStructPathName / GetClassPathName during asset registry searches.
	ClassFlags |= CLASS_Deprecated;
}

const FString& URigVMMemoryStorageGeneratorClass::GetClassName(ERigVMMemoryType InMemoryType)
{
	static TArray<FString> ClassNames;
	if(ClassNames.IsEmpty())
	{
		for(int32 MemoryTypeIndex=0;MemoryTypeIndex<(int32)ERigVMMemoryType::Invalid;MemoryTypeIndex++)
		{
			ClassNames.Add(TEXT("RigVMMemory_") + StaticEnum<ERigVMMemoryType>()->GetDisplayNameTextByValue((int64)MemoryTypeIndex).ToString());
		}
	}
	return ClassNames[(int32)InMemoryType];
}

URigVMMemoryStorageGeneratorClass* URigVMMemoryStorageGeneratorClass::GetStorageClass(UObject* InOuter, ERigVMMemoryType InMemoryType)
{
	check(InOuter);

	TArray<UObject*> PotentialClassContainers;
	PotentialClassContainers.Add(InOuter);
	
	UObject* Outer = InOuter->GetOuter();
	do
	{
		if(!Outer || Outer->IsA<UPackage>())
		{
			break;
		}

		PotentialClassContainers.Add(Outer);
		PotentialClassContainers.Add(Outer->GetClass());

		Outer = Outer->GetOuter();
	}
	while (Outer);

	const FString& ClassName = GetClassName(InMemoryType);

	for(UObject* PotentialClassContainer : PotentialClassContainers)
	{
		UPackage* Package = PotentialClassContainer->GetOutermost();
		if(URigVMMemoryStorageGeneratorClass* Class = FindObject<URigVMMemoryStorageGeneratorClass>(Package, *ClassName))
		{
			return Class;
		}
	}

	return nullptr;
}

URigVMMemoryStorageGeneratorClass* URigVMMemoryStorageGeneratorClass::CreateStorageClass(
	UObject* InOuter,
	ERigVMMemoryType InMemoryType,
	const TArray<FRigVMPropertyDescription>& InProperties,
	const TArray<FRigVMPropertyPathDescription>& InPropertyPaths)
{
	check(InOuter);
	UPackage* Package = InOuter->GetOutermost();
	UClass *SuperClass = URigVMMemoryStorage::StaticClass();

	const FString& ClassName = GetClassName(InMemoryType);

	// if there's an old class - remove it from the package and mark it to destroy
	//
	// we need to use StaticFindObjectFastInternal with ExclusiveInternalFlags = EInternalObjectFlags::None
	// here because objects with RF_NeedPostLoad cannot be found by regular FindObject calls as they will have
	// ExclusiveInternalFlags = EInternalObjectFlags::AsyncLoading when called from game thread.
	//
	// in theory, whenever this function is called as part of OnEndLoadPackage -> CRBP::HandlePackageDone,
	// all objects within the package should have been fully loaded, but it turns out that OnEndLoadPackage
	// does not guarantee that condition. As a result we will do OldClass->ConditionalPostLoad() to ensure the OldClass
	// is fully loaded, ready to be replaced.
	
	UObject* OldClassObject = StaticFindObjectFastInternal( /*Class=*/ NULL, Package, *ClassName, EFindObjectFlags::ExactClass, RF_NoFlags, EInternalObjectFlags::None);
	if(OldClassObject)
	{
		auto FindUniqueName = [](const TCHAR* TemplateName, int32& TemplateIndex)
		{
			FName UniqueName;
			do
			{
				UniqueName = FName(TemplateName, TemplateIndex++);
				if(StaticFindObjectFast(nullptr, GetTransientPackage(), UniqueName) == nullptr)
				{
					break;
				}
			}
			while (TemplateIndex < INT_MAX);
 
			return UniqueName;
		};
		
		auto RenameAndMarkGarbage = [](UObject* InObject, FName NewName)
		{
			InObject->Rename(*NewName.ToString(), GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			InObject->MarkAsGarbage();
		};
 
		// ensure the OldClass is completely loaded so that we can remove it and replace it with the new class
		// otherwise we get an assertion trying to replace objects currently being loaded
		OldClassObject->ConditionalPostLoad();
		UClass* OldClass = Cast<UClass>(OldClassObject);
 
		// using an FName to avoid formatting a string and then immediately turning around to parse it.
		static const TCHAR* DiscardedMemoryClassTemplate(TEXT("DiscardedMemoryClassTemplate"));
		static int32 DiscardedMemoryClassIndex = 0;
		const FName DiscardedMemoryClassName = FindUniqueName(DiscardedMemoryClassTemplate, DiscardedMemoryClassIndex);
		
		OldClass->ClassFlags |= CLASS_NewerVersionExists;
		RenameAndMarkGarbage(OldClass, DiscardedMemoryClassName);
 
		if (OldClass->GetDefaultObject(false))
		{
			static const TCHAR* DiscardedMemoryCDOTemplate(TEXT("DiscardedMemoryCDOTemplate"));
			static int32 DiscardedMemoryCDOIndex = 0;
			RenameAndMarkGarbage(OldClass->GetDefaultObject(false), FindUniqueName(DiscardedMemoryCDOTemplate, DiscardedMemoryCDOIndex));
		}
	}

	// create the new class
	URigVMMemoryStorageGeneratorClass* Class = NewObject<URigVMMemoryStorageGeneratorClass>(
		Package,
		*ClassName,
		RF_Standalone | RF_Public
	);

	// clear the class (sets relevant flags)
	Class->PurgeClass(false);

	// setup class to inherit from the super class
	Class->SetSuperStruct(SuperClass);
	Class->PropertyLink = SuperClass->PropertyLink;

	// allow to create this class's instances everywhere
	Class->ClassWithin = UObject::StaticClass();

	// make sure that it cannot be placed / used for class selectors
	Class->ClassFlags |= CLASS_Hidden;

	// store our custom state
	Class->MemoryType = InMemoryType;

	// Generate properties by iterating
	FField::FLinkedListBuilder PropertyListBuilder(&Class->ChildProperties);
	for(const FRigVMPropertyDescription& PropertyDescription : InProperties)
	{
		FProperty* CachedProperty = AddProperty(Class, PropertyDescription, PropertyListBuilder);
		check(CachedProperty);
	}

	// Store the property path descriptions
	Class->PropertyPathDescriptions = InPropertyPaths;

	// Update the class's data and link it / build it
	Class->Bind();
	Class->StaticLink(true);
	
	// Create default object
	URigVMMemoryStorage* CDO = Cast<URigVMMemoryStorage>(Class->GetDefaultObject(true));

	// and store default values.
	const TArray<const FProperty*>& LinkedProperties = CDO->GetProperties();
	for(int32 PropertyIndex = 0; PropertyIndex < LinkedProperties.Num(); PropertyIndex++)
	{
		const FString& DefaultValue = InProperties[PropertyIndex].DefaultValue;

		static const FString EmptyBraces = TEXT("()");
		if(DefaultValue.IsEmpty() || DefaultValue == EmptyBraces)
		{
			continue;
		}

		const FProperty* Property = LinkedProperties[PropertyIndex];
		uint8* ValuePtr = Property->ContainerPtrToValuePtr<uint8>(CDO);

		FRigVMMemoryStorageImportErrorContext ErrorPipe(false);
		Property->ImportText_Direct(*DefaultValue, ValuePtr, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
		if(ErrorPipe.NumErrors > 0)
		{
			// check if the default value was provided as a single element
			if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				static constexpr TCHAR BraceFormat[] = TEXT("(%s)");
				const FString DefaultValueWithBraces = FString::Printf(BraceFormat, *DefaultValue);
				ErrorPipe = FRigVMMemoryStorageImportErrorContext(false);
				Property->ImportText_Direct(*DefaultValueWithBraces, ValuePtr, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
			}
		}
	}

	return Class;
}

bool URigVMMemoryStorageGeneratorClass::RemoveStorageClass(UObject* InOuter, ERigVMMemoryType InMemoryType)
{
	check(InOuter);
	UPackage* Package = InOuter->GetOutermost();

	const FString ClassName = GetClassName(InMemoryType);

	// if there's an old class - remove it from the package and mark it to destroy
	URigVMMemoryStorageGeneratorClass* OldClass = FindObject<URigVMMemoryStorageGeneratorClass>(Package, *ClassName);
	if(OldClass)
	{
		OldClass->ClassFlags |= CLASS_NewerVersionExists;
		OldClass->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		OldClass->MarkAsGarbage();
		return true;
	}

	return false;
}

uint32 URigVMMemoryStorageGeneratorClass::GetMemoryHash() const
{
 	if(CachedMemoryHash != 0)
	{
		return CachedMemoryHash;
	}
	
	CachedMemoryHash = 0;

	for(const FProperty* Property : LinkedProperties)
	{
		CachedMemoryHash = HashCombineFast(CachedMemoryHash, GetTypeHash(Property->GetFName().ToString()));
		CachedMemoryHash = HashCombineFast(CachedMemoryHash, GetTypeHash(Property->GetCPPType()));
	}
	
	// for literals we also hash the content / defaults for each property
	if(GetMemoryType() == ERigVMMemoryType::Literal)
	{
		if(const URigVMMemoryStorage* CDO = Cast<URigVMMemoryStorage>(GetDefaultObject(true)))
		{
			for(const FProperty* Property : LinkedProperties)
			{
				const uint32 PropertyHash =
					RigVMPropertyUtils::GetPropertyHashFast(Property, reinterpret_cast<const uint8*>(CDO));
 
				CachedMemoryHash = HashCombineFast(CachedMemoryHash, PropertyHash);
			}
		}
	}
	return CachedMemoryHash;
}

FRigVMMemoryStatistics URigVMMemoryStorageGeneratorClass::GetStatistics() const
{
	FRigVMMemoryStatistics Statistics;
	Statistics.RegisterCount = GetProperties().Num();
	Statistics.TotalBytes = Statistics.DataBytes + sizeof(URigVMMemoryStorage);
	Statistics.TotalBytes += sizeof(FProperty) * Statistics.RegisterCount;
	return Statistics;
}

FProperty* URigVMMemoryStorageGeneratorClass::AddProperty(URigVMMemoryStorageGeneratorClass* InClass, const FRigVMPropertyDescription& InProperty, FField::FLinkedListBuilder& PropertyListBuilder)
{
	UClass *SuperClass = URigVMMemoryStorage::StaticClass();
	
	check(InClass);
	check(InClass->GetSuperClass() == SuperClass);

	PropertyListBuilder.MoveToEnd();

	FProperty* Result = nullptr;
	if(InProperty.Property)
	{
		// if the property description provides a property simply duplicate that property
		FProperty* NewProperty = CastFieldChecked<FProperty>(FField::Duplicate(InProperty.Property, InClass, InProperty.Name));
		check(NewProperty);

		Result = NewProperty;
		PropertyListBuilder.AppendNoTerminate(*Result);
	}
	else
	{
		FFieldVariant PropertyOwner = InClass;
		FProperty** ValuePropertyPtr = &Result;

		for(EPinContainerType Container : InProperty.Containers)
		{
			switch(Container)
			{
				case EPinContainerType::Array:
				{
					// create an array property as a container for the tail
					FArrayProperty* ArrayProperty = new FArrayProperty(PropertyOwner, InProperty.Name);
					*ValuePropertyPtr = ArrayProperty;
					ValuePropertyPtr = &ArrayProperty->Inner;
					PropertyOwner = ArrayProperty;
					break;
				}
				case EPinContainerType::Map:
				{
					// create a map property as a container for the tail
					checkNoEntry(); // this is not implemented yet
					FMapProperty* MapProperty = new FMapProperty(PropertyOwner, InProperty.Name);
					*ValuePropertyPtr = MapProperty;
					ValuePropertyPtr = &MapProperty->ValueProp;
					PropertyOwner = MapProperty;
					break;
				}
				case EPinContainerType::Set:
				{
					checkNoEntry();
					break;
				}
				case EPinContainerType::None:
				default:
				{
					break;
				}
			}
		}

		// create properties for types which are provided through the CPPTypeObject
		if(InProperty.CPPTypeObject != nullptr)
		{
			if(UEnum* Enum = Cast<UEnum>(InProperty.CPPTypeObject))
			{
				FByteProperty* EnumProperty = new FByteProperty(PropertyOwner, InProperty.Name);
				EnumProperty->Enum = Enum;
				(*ValuePropertyPtr) = EnumProperty;
			}
			else if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InProperty.CPPTypeObject))
			{
				FStructProperty* StructProperty = new FStructProperty(PropertyOwner, InProperty.Name);
				StructProperty->Struct = ScriptStruct;
				(*ValuePropertyPtr) = StructProperty;
			}
			else if(UClass* PropertyClass = Cast<UClass>(InProperty.CPPTypeObject))
			{
				check(RigVMCore::SupportsUObjects());

				const bool bIsClass = RigVMTypeUtils::IsUClassType(InProperty.CPPType);
				if(bIsClass)
				{
					FClassProperty* ClassProperty = new FClassProperty(PropertyOwner, InProperty.Name);
					ClassProperty->SetMetaClass(PropertyClass);
					(*ValuePropertyPtr) = ClassProperty;
				}
				else
				{
					FObjectProperty* ObjectProperty = new FObjectProperty(PropertyOwner, InProperty.Name);
					ObjectProperty->SetPropertyClass(PropertyClass);
					(*ValuePropertyPtr) = ObjectProperty;
				}
			}
			else
			{
				checkNoEntry();
			}
		}
		else // take care of simple types...
		{
			static const FString IntString = TEXT("int");

			const FString BaseCPPType = InProperty.GetTailCPPType();
			if(BaseCPPType.Equals(RigVMTypeUtils::BoolType, ESearchCase::IgnoreCase))
			{
				FBoolProperty* BoolProperty = new FBoolProperty(PropertyOwner, InProperty.Name); 
				BoolProperty->SetBoolSize(sizeof(bool), true);
				(*ValuePropertyPtr) = BoolProperty;
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::Int32Type, ESearchCase::IgnoreCase) ||
				BaseCPPType.Equals(IntString, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FIntProperty(PropertyOwner, InProperty.Name);
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::UInt32Type, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FUInt32Property(PropertyOwner, InProperty.Name);
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::FloatType, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FFloatProperty(PropertyOwner, InProperty.Name);
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::DoubleType, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FDoubleProperty(PropertyOwner, InProperty.Name);
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::FStringType, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FStrProperty(PropertyOwner, InProperty.Name);
			}
			else if(BaseCPPType.Equals(RigVMTypeUtils::FNameType, ESearchCase::IgnoreCase))
			{
				(*ValuePropertyPtr) = new FNameProperty(PropertyOwner, InProperty.Name);
			}
			else
			{
				// we might have hit a base structure
				static TArray<UScriptStruct*> BaseStructures;
				if(BaseStructures.IsEmpty())
				{
					BaseStructures.Add(TBaseStructure<FVector>::Get());
					BaseStructures.Add(TBaseStructure<FVector2D>::Get());
					BaseStructures.Add(TBaseStructure<FRotator>::Get());
					BaseStructures.Add(TBaseStructure<FQuat>::Get());
					BaseStructures.Add(TBaseStructure<FTransform>::Get());
					BaseStructures.Add(TBaseStructure<FLinearColor>::Get());
				}

				for(UScriptStruct* BaseStructure : BaseStructures)
				{
					if(RigVMTypeUtils::GetUniqueStructTypeName(BaseStructure) == BaseCPPType)
					{
						FStructProperty* StructProperty = new FStructProperty(PropertyOwner, InProperty.Name);
						StructProperty->Struct = BaseStructure;
						(*ValuePropertyPtr) = StructProperty;
						break;
					}
				}

				if(*ValuePropertyPtr == nullptr)
				{
					return nullptr;
				}
			}
		}

		// mark the property as visible only for details panels
		// also mark it as non transactional, which means the undo / redo system will ignore changes to it
		Result->SetPropertyFlags(CPF_Edit | CPF_EditConst | CPF_NonTransactional);

#if WITH_EDITORONLY_DATA

		// store some additional meta data,
		// mainly for inspecting things in the details panel
		static const FLazyName NAME_DisplayName(TEXT("DisplayName"));
		static const FLazyName NAME_ToolTipName(TEXT("ToolTip"));

		FString DisplayName = Result->GetName();
		while(DisplayName.ReplaceInline(TEXT("__"), TEXT("_")) > 0)
		{}
		DisplayName.ReplaceInline(TEXT("_"), TEXT(" "));

		DisplayName = FString::Printf(TEXT("[%d] %s"), InClass->LinkedProperties.Num(), *DisplayName);
		Result->SetMetaData(NAME_DisplayName, *DisplayName);
		Result->SetMetaData(NAME_ToolTipName, *FString::Printf(TEXT("Name %s\nCPPType %s"), *Result->GetName(), *InProperty.CPPType));
		
#endif

		// store the property in the cached list
		InClass->LinkedProperties.Add(Result);
		PropertyListBuilder.AppendNoTerminate(*Result);
	}

	return Result;
}

void URigVMMemoryStorageGeneratorClass::RefreshLinkedProperties()
{
	CachedMemoryHash = 0;
	LinkedProperties.Reset();
	const FProperty* Property = CastField<FProperty>(ChildProperties);
	while(Property)
	{
		LinkedProperties.Add(Property);
		Property = CastField<FProperty>(Property->Next);
	}
}

void URigVMMemoryStorageGeneratorClass::RefreshPropertyPaths()
{
	// Update the property paths based on the descriptions
	PropertyPaths.SetNumZeroed(PropertyPathDescriptions.Num());
	for(int32 PropertyPathIndex = 0; PropertyPathIndex < PropertyPaths.Num(); PropertyPathIndex++)
	{
		PropertyPaths[PropertyPathIndex] = FRigVMPropertyPath();

		const int32 PropertyIndex = PropertyPathDescriptions[PropertyPathIndex].PropertyIndex;
		if(LinkedProperties.IsValidIndex(PropertyIndex))
		{
			PropertyPaths[PropertyPathIndex] = FRigVMPropertyPath(
				LinkedProperties[PropertyIndex],
				PropertyPathDescriptions[PropertyPathIndex].SegmentPath);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<const FProperty*> URigVMMemoryStorage::EmptyProperties;
const TArray<FRigVMPropertyPath> URigVMMemoryStorage::EmptyPropertyPaths;

FString URigVMMemoryStorage::GetDataAsString(int32 InPropertyIndex, int32 PortFlags)
{
	check(IsValidIndex(InPropertyIndex));
	const uint8* Data = GetData<uint8>(InPropertyIndex);

	FString Value;
	GetProperties()[InPropertyIndex]->ExportTextItem_Direct(Value, Data, nullptr, nullptr, PortFlags);
	return Value;
}

FString URigVMMemoryStorage::GetDataAsString(const FRigVMOperand& InOperand, int32 PortFlags)
{
	const int32 PropertyIndex = InOperand.GetRegisterIndex();
	check(IsValidIndex(PropertyIndex));
	return GetDataAsString(PropertyIndex, PortFlags);
}

FString URigVMMemoryStorage::GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags)
{
	if(!IsValidIndex(InPropertyIndex))
	{
		return FString();
	}
	return GetDataAsString(InPropertyIndex, PortFlags);
}

FString URigVMMemoryStorage::GetDataAsStringSafe(const FRigVMOperand& InOperand, int32 PortFlags)
{
	const int32 PropertyIndex = InOperand.GetRegisterIndex();
	return GetDataAsStringSafe(PropertyIndex, PortFlags);
}

bool URigVMMemoryStorage::SetDataFromString(int32 InPropertyIndex, const FString& InValue)
{
	check(IsValidIndex(InPropertyIndex));
	uint8* Data = GetData<uint8>(InPropertyIndex);
	
	FRigVMMemoryStorageImportErrorContext ErrorPipe;
	GetProperties()[InPropertyIndex]->ImportText_Direct(*InValue, Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
	return ErrorPipe.NumErrors == 0;
}

FRigVMMemoryHandle URigVMMemoryStorage::GetHandle(int32 InPropertyIndex, const FRigVMPropertyPath* InPropertyPath)
{
	check(IsValidIndex(InPropertyIndex));

	const FProperty* Property = GetProperties()[InPropertyIndex];
	uint8* Data = GetData<uint8>(InPropertyIndex);

	return FRigVMMemoryHandle(Data, Property, InPropertyPath);
}

bool URigVMMemoryStorage::CopyProperty(
	const FProperty* InTargetProperty,
	uint8* InTargetPtr,
	const FProperty* InSourceProperty,
	const uint8* InSourcePtr)
{
	check(InTargetProperty != nullptr);
	check(InSourceProperty != nullptr);
	check(InTargetPtr != nullptr);
	check(InSourcePtr != nullptr);

	// This block below is there to support Large World Coordinates (LWC).
	// We allow to link float and double pins (single and arrays) so we need
	// to support copying values between those as well.
	if(!InTargetProperty->SameType(InSourceProperty))
	{
		if(const FFloatProperty* TargetFloatProperty = CastField<FFloatProperty>(InTargetProperty))
		{
			if(const FDoubleProperty* SourceDoubleProperty = CastField<FDoubleProperty>(InSourceProperty))
			{
				if(TargetFloatProperty->ArrayDim == SourceDoubleProperty->ArrayDim)
				{
					float* TargetFloats = (float*)InTargetPtr;
					double* SourceDoubles = (double*)InSourcePtr;
					for(int32 Index=0;Index<TargetFloatProperty->ArrayDim;Index++)
					{
						TargetFloats[Index] = (float)SourceDoubles[Index];
					}
					return true;
				}
			}
		}
		else if(const FDoubleProperty* TargetDoubleProperty = CastField<FDoubleProperty>(InTargetProperty))
		{
			if(const FFloatProperty* SourceFloatProperty = CastField<FFloatProperty>(InSourceProperty))
			{
				if(TargetDoubleProperty->ArrayDim == SourceFloatProperty->ArrayDim)
				{
					double* TargetDoubles = (double*)InTargetPtr;
					float* SourceFloats = (float*)InSourcePtr;
					for(int32 Index=0;Index<TargetDoubleProperty->ArrayDim;Index++)
					{
						TargetDoubles[Index] = (double)SourceFloats[Index];
					}
					return true;
				}
			}
		}
		else if (const FByteProperty* TargetByteProperty = CastField<FByteProperty>(InTargetProperty))
		{
			if (const FEnumProperty* SourceEnumProperty = CastField<FEnumProperty>(InSourceProperty))
			{
				if (TargetByteProperty->Enum == SourceEnumProperty->GetEnum())
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if (const FEnumProperty* TargetEnumProperty = CastField<FEnumProperty>(InTargetProperty))
		{
			if (const FByteProperty* SourceByteProperty = CastField<FByteProperty>(InSourceProperty))
			{
				if (TargetEnumProperty->GetEnum() == SourceByteProperty->Enum)
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if(const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(InTargetProperty))
		{
			if(const FArrayProperty* SourceArrayProperty = CastField<FArrayProperty>(InSourceProperty))
			{
				FScriptArrayHelper TargetArray(TargetArrayProperty, InTargetPtr);
				FScriptArrayHelper SourceArray(SourceArrayProperty, InSourcePtr);
				
				if(TargetArrayProperty->Inner->IsA<FFloatProperty>())
				{
					if(SourceArrayProperty->Inner->IsA<FDoubleProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for(int32 Index=0;Index<TargetArray.Num();Index++)
						{
							float* TargetFloat = (float*)TargetArray.GetRawPtr(Index);
							const double* SourceDouble = (const double*)SourceArray.GetRawPtr(Index);
							*TargetFloat = (float)*SourceDouble;
						}
						return true;
					}
				}
				else if(TargetArrayProperty->Inner->IsA<FDoubleProperty>())
				{
					if(SourceArrayProperty->Inner->IsA<FFloatProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for(int32 Index=0;Index<TargetArray.Num();Index++)
						{
							double* TargetDouble = (double*)TargetArray.GetRawPtr(Index);
							const float* SourceFloat = (const float*)SourceArray.GetRawPtr(Index);
							*TargetDouble = (double)*SourceFloat;
						}
						return true;
					}
				}
				else if(FByteProperty* TargetArrayInnerByteProperty = CastField<FByteProperty>(TargetArrayProperty->Inner))
				{
					if(FEnumProperty* SourceArrayInnerEnumProperty = CastField<FEnumProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerByteProperty->Enum == SourceArrayInnerEnumProperty->GetEnum())
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
				else if(FEnumProperty* TargetArrayInnerEnumProperty = CastField<FEnumProperty>(TargetArrayProperty->Inner))
				{
					if(FByteProperty* SourceArrayInnerByteProperty = CastField<FByteProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerEnumProperty->GetEnum() == SourceArrayInnerByteProperty->Enum)
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
			}
		}

		// if we reach this we failed since we are trying to copy
		// between two properties which are not compatible
		if(!InTargetProperty->SameType(InSourceProperty))
		{
			// Only log the issue once, rather than spam.
			static TSet<FString> ReportedErrors;
			
			const FString TargetType = RigVMTypeUtils::GetCPPTypeFromProperty(InTargetProperty);
			const FString SourceType = RigVMTypeUtils::GetCPPTypeFromProperty(InSourceProperty);
			
			UPackage* Package = InTargetProperty->GetOutermost();
			
			FString Message = FString::Printf(TEXT("Failed to copy %s (%s) to %s (%s) in package %s"),
				*InSourceProperty->GetName(),
				*SourceType,
				*InTargetProperty->GetName(), 
				*TargetType,
				Package ? *Package->GetName() : TEXT("<Unknown Package>"));

			if (!ReportedErrors.Contains(Message))
			{
				UE_LOGF(LogRigVM, Error, "%ls", *Message);
				ReportedErrors.Add(Message);
			}
			
			return false;
		}
	}

	// rely on the core to copy the property contents
	InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
	return true;
}

bool URigVMMemoryStorage::CopyProperty(
	const FProperty* InTargetProperty,
	uint8* InTargetPtr,
	const FRigVMPropertyPath& InTargetPropertyPath,
	const FProperty* InSourceProperty,
	const uint8* InSourcePtr,
	const FRigVMPropertyPath& InSourcePropertyPath)
{
	check(InTargetProperty != nullptr);
	check(InSourceProperty != nullptr);
	check(InTargetPtr != nullptr);
	check(InSourcePtr != nullptr);

	auto TraversePropertyPath = [](const FProperty*& Property, uint8*& MemoryPtr, const FRigVMPropertyPath& PropertyPath)
	{
		if(PropertyPath.IsEmpty())
		{
			return;
		}

		MemoryPtr = PropertyPath.GetData<uint8>(MemoryPtr, Property);
		Property = PropertyPath.GetTailProperty();
	};

	uint8* SourcePtr = (uint8*)InSourcePtr;
	TraversePropertyPath(InTargetProperty, InTargetPtr, InTargetPropertyPath);
	TraversePropertyPath(InSourceProperty, SourcePtr, InSourcePropertyPath);

	if(InTargetPtr == nullptr)
	{
		check(!InTargetPropertyPath.IsEmpty());
		return false;
	}
	if(InSourcePtr == nullptr)
	{
		check(!InSourcePropertyPath.IsEmpty());
		return false;
	}

	return CopyProperty(InTargetProperty, InTargetPtr, InSourceProperty, SourcePtr);
}

bool URigVMMemoryStorage::CopyProperty(
	URigVMMemoryStorage* InTargetStorage,
	int32 InTargetPropertyIndex,
	const FRigVMPropertyPath& InTargetPropertyPath,
	URigVMMemoryStorage* InSourceStorage,
	int32 InSourcePropertyIndex,
	const FRigVMPropertyPath& InSourcePropertyPath)
{
	check(InTargetStorage != nullptr);
	check(InSourceStorage != nullptr);

	const FProperty* TargetProperty = InTargetStorage->GetProperties()[InTargetPropertyIndex];
	const FProperty* SourceProperty = InSourceStorage->GetProperties()[InSourcePropertyIndex];
	uint8* TargetPtr = TargetProperty->ContainerPtrToValuePtr<uint8>(InTargetStorage);
	uint8* SourcePtr = SourceProperty->ContainerPtrToValuePtr<uint8>(InSourceStorage);

	return CopyProperty(TargetProperty, TargetPtr, InTargetPropertyPath, SourceProperty, SourcePtr, InSourcePropertyPath);
}

bool URigVMMemoryStorage::CopyProperty(
	FRigVMMemoryHandle& InTargetHandle,
	FRigVMMemoryHandle& InSourceHandle)
{
	return CopyProperty(
		InTargetHandle.GetProperty(),
		InTargetHandle.GetData_Internal(false, INDEX_NONE),
		InTargetHandle.GetPropertyPathRef(),
		InSourceHandle.GetProperty(),
		InSourceHandle.GetData_Internal(false, INDEX_NONE),
		InSourceHandle.GetPropertyPathRef());
}

void URigVMMemoryStorage::CopyFrom(URigVMMemoryStorage* InSourceMemory)
{
	check(InSourceMemory);
	check(GetClass() == InSourceMemory->GetClass());

	for(int32 PropertyIndex = 0; PropertyIndex < Num(); PropertyIndex++)
	{
		URigVMMemoryStorage::CopyProperty(
			this,
			PropertyIndex,
			FRigVMPropertyPath::Empty,
			InSourceMemory,
			PropertyIndex,
			FRigVMPropertyPath::Empty
		);
	}
}

const TArray<const FProperty*>& URigVMMemoryStorage::GetProperties() const
{
	if(const URigVMMemoryStorageGeneratorClass* Class = Cast<URigVMMemoryStorageGeneratorClass>(GetClass()))
	{
		return Class->GetProperties();
	}
	return EmptyProperties;
}

const TArray<FRigVMPropertyPath>& URigVMMemoryStorage::GetPropertyPaths() const
{
	if(const URigVMMemoryStorageGeneratorClass* Class = Cast<URigVMMemoryStorageGeneratorClass>(GetClass()))
	{
		return Class->GetPropertyPaths();
	}
	return EmptyPropertyPaths;
}

int32 URigVMMemoryStorage::GetPropertyIndex(const FProperty* InProperty) const
{
	const TArray<const FProperty*>& Properties = GetProperties();
	for(int32 PropertyIndex = 0; PropertyIndex < Properties.Num(); PropertyIndex++)
	{
		if(Properties[PropertyIndex] == InProperty)
		{
			return PropertyIndex;
		}
	}
	return INDEX_NONE;
}

int32 URigVMMemoryStorage::GetPropertyIndexByName(const FName& InName) const
{
	const FProperty* Property = FindPropertyByName(InName);
	return GetPropertyIndex(Property);
}

FProperty* URigVMMemoryStorage::FindPropertyByName(const FName& InName) const
{
	const FName SanitizedName = FRigVMPropertyDescription::SanitizeName(InName);
	return GetClass()->FindPropertyByName(SanitizedName);
}

FRigVMOperand URigVMMemoryStorage::GetOperand(int32 InPropertyIndex, int32 InPropertyPathIndex) const
{
	if(GetProperties().IsValidIndex(InPropertyIndex))
	{
		if(InPropertyPathIndex != INDEX_NONE)
		{
			check(GetPropertyPaths().IsValidIndex(InPropertyPathIndex));
			return FRigVMOperand(GetMemoryType(), InPropertyIndex, InPropertyPathIndex);
		}
		return FRigVMOperand(GetMemoryType(), InPropertyIndex);
	}
	return FRigVMOperand();
}

FRigVMOperand URigVMMemoryStorage::GetOperandByName(const FName& InName, int32 InPropertyPathIndex) const
{
	const int32 PropertyIndex = GetPropertyIndexByName(InName);
	return GetOperand(PropertyIndex, InPropertyPathIndex);
}

