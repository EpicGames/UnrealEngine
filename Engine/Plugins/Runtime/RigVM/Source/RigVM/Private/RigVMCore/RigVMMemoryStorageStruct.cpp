// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMTypeUtils.h"
#include "RigVMModule.h"
#include "Misc/Guid.h"
#include "Hash/Blake3.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMMemoryStorageStruct)

FRigVMMemoryStorageStruct::FRigVMMemoryStorageStruct()
	: FInstancedPropertyBag()
	, MemoryType(ERigVMMemoryType::Invalid)
{
}

FRigVMMemoryStorageStruct::FRigVMMemoryStorageStruct(ERigVMMemoryType InMemoryType)
	: FInstancedPropertyBag()
	, MemoryType(InMemoryType)
{
}

FRigVMMemoryStorageStruct::FRigVMMemoryStorageStruct(ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>& InPropertyDescriptions, const TArray<FRigVMPropertyPathDescription>& InPropertyPaths)
	: FInstancedPropertyBag()
	, MemoryType(InMemoryType)
{
	AddProperties(InPropertyDescriptions, InPropertyPaths);
}

FRigVMMemoryStorageStruct::~FRigVMMemoryStorageStruct()
{
}

void FRigVMMemoryStorageStruct::InitializeFromNativeStruct(const UScriptStruct* InStruct)
{
	Value.InitializeAs(InStruct);
	RefreshLinkedProperties();
}

bool FRigVMMemoryStorageStruct::Serialize(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, TEXT("FRigVMMemoryStorageStruct"));
	Super::Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	Ar << MemoryType;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("MemoryType"));
	Ar << PropertyPathDescriptions;
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("PropertyPathDescriptions"));

	if (Ar.IsLoading())
	{
		// rebuild property list and property path list
		RefreshLinkedProperties();
		RefreshPropertyPaths();

		CachedMemoryHash = 0;
	}

	return true;
}

void FRigVMMemoryStorageStruct::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddStructReferencedObjects(Collector);
}

void FRigVMMemoryStorageStruct::GetUserDefinedDependencies(TArray<const UObject*>& OutDependencies) const
{
	ForEachObjectDependency([&OutDependencies](const UObject* Dependency)
	{
		OutDependencies.AddUnique(Dependency);
	});
}

void FRigVMMemoryStorageStruct::GetRequiredPlugins(TArray<FString>& OutPlugins) const
{
	ForEachObjectDependency([&OutPlugins](const UObject* Dependency)
	{
		const FString PluginName = RigVMTypeUtils::GetPluginName(Dependency);
		if (!PluginName.IsEmpty())
		{
			OutPlugins.AddUnique(PluginName);
		}
	});
}

void FRigVMMemoryStorageStruct::ForEachObjectDependency(const TFunction<void(const UObject*)>& InCallBack) const
{
	if (!InCallBack)
	{
		return;
	}
	
	const TArray<const FProperty*>& Properties = GetProperties();
	for (const FProperty* Property : Properties)
	{
		const FProperty* PropertyToVisit = Property;
		while (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyToVisit))
		{
			PropertyToVisit = ArrayProperty->Inner;
		}
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyToVisit))
		{
			if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(StructProperty->Struct))
			{
				InCallBack(UserDefinedStruct);
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyToVisit))
		{
			if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(EnumProperty->GetEnum()))
			{
				InCallBack(UserDefinedEnum);
			}
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyToVisit))
		{
			if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(ByteProperty->Enum))
			{
				InCallBack(UserDefinedEnum);
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyToVisit))
		{
			// if we are a root property - just get the value
			if (PropertyToVisit == Property)
			{
				const UObject* Dependency = ObjectProperty->GetObjectPropertyValue_InContainer(Value.GetMemory());
				if (Dependency && FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Dependency))
				{
					InCallBack(Dependency);
				}
			}
			else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property)) // if we are within an array property
			{
				if (const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, Value.GetMemory());
					for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
					{
						UObject* Dependency = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(ElementIndex));
						if (Dependency && FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Dependency))
						{
							InCallBack(Dependency);
						}
					}
				}
			}
		}
	}
}

void FRigVMMemoryStorageStruct::AddProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions, const TArray<FRigVMPropertyPathDescription>& InPropertyPathDescriptions)
{
	TArray<FPropertyBagPropertyDesc> BagDescriptors;
	BagDescriptors.Reserve(InPropertyDescriptions.Num());

	// Generate PropertyBag descriptors
	for (const FRigVMPropertyDescription& RigVMDescriptor : InPropertyDescriptions)
	{
		if (!ensure(BagDescriptors.FindByPredicate([&RigVMDescriptor](const FPropertyBagPropertyDesc& Desc) { return Desc.Name == RigVMDescriptor.Name; }) == nullptr))
		{
			UE_LOGF(LogRigVM, Error, "Detected duplicated name in the generated properties : [%ls]", *RigVMDescriptor.Name.ToString());
		}

		const FPropertyBagPropertyDesc& PropertyBagDescriptor = GeneratePropertyBagDescriptor(RigVMDescriptor);
		if (PropertyBagDescriptor.ValueType != EPropertyBagPropertyType::None)
		{
			BagDescriptors.Add(PropertyBagDescriptor);
		}
	}

	// Generate the struct with the passed types
	FInstancedPropertyBag::AddProperties(BagDescriptors);

	PropertyPathDescriptions = InPropertyPathDescriptions;

	RefreshLinkedProperties();
	RefreshPropertyPaths();

	// And set the passed text DefaultValues
	SetDefaultValues(InPropertyDescriptions);

	CachedMemoryHash = 0;
}

int32 FRigVMMemoryStorageStruct::GetPropertyIndex(const FProperty* InProperty) const
{
	if (InProperty != nullptr)
	{
		const TArray<const FProperty*>& Properties = GetProperties();

		return Properties.IndexOfByKey(InProperty);
	}

	return INDEX_NONE;
}

int32 FRigVMMemoryStorageStruct::GetPropertyIndexByName(const FName& InName) const
{
	const FProperty* Property = FindPropertyByName(InName);
	return GetPropertyIndex(Property);
}

FProperty* FRigVMMemoryStorageStruct::FindPropertyByName(const FName& InName) const
{
	const FName SanitizedName = FRigVMPropertyDescription::SanitizeName(InName);
	return Value.GetScriptStruct() != nullptr ? Value.GetScriptStruct()->FindPropertyByName(SanitizedName) : nullptr;
}

FRigVMOperand FRigVMMemoryStorageStruct::GetOperand(int32 InPropertyIndex, int32 InPropertyPathIndex) const
{
	if (GetProperties().IsValidIndex(InPropertyIndex))
	{
		if (InPropertyPathIndex != INDEX_NONE)
		{
			check(GetPropertyPaths().IsValidIndex(InPropertyPathIndex));
			return FRigVMOperand(GetMemoryType(), InPropertyIndex, InPropertyPathIndex);
		}
		return FRigVMOperand(GetMemoryType(), InPropertyIndex);
	}
	return FRigVMOperand();
}

FRigVMOperand FRigVMMemoryStorageStruct::GetOperandByName(const FName& InName, int32 InPropertyPathIndex) const
{
	const int32 PropertyIndex = GetPropertyIndexByName(InName);
	return GetOperand(PropertyIndex, InPropertyPathIndex);
}

void* FRigVMMemoryStorageStruct::GetContainerPtr() const
{
	return (void*)GetValue().GetMemory();
}

uint32 FRigVMMemoryStorageStruct::GetMemoryHash() const
{
	if (CachedMemoryHash != 0)
	{
		return CachedMemoryHash;
	}

	for (const FProperty* Property : LinkedProperties)
	{
		CachedMemoryHash = HashCombine(CachedMemoryHash, GetTypeHash(Property->GetFName().ToString()));
		CachedMemoryHash = HashCombine(CachedMemoryHash, GetTypeHash(Property->GetCPPType()));
	}

	// for literals we also hash the content / defaults for each property
	if (GetMemoryType() == ERigVMMemoryType::Literal)
	{
		for (const FProperty* Property : LinkedProperties)
		{
			// Note that it is using Export Direct and also passing Data as the default value forces the text exporter to generate all the elements
			// If we pass nullptr as default, values that are default initialized to a value that is different from the type default value will not be correctly exported
			FString DefaultValue;
			const uint8* Data = GetData<uint8>(Property);
			Property->ExportTextItem_Direct(DefaultValue, Data, Data, nullptr, PPF_None, nullptr);
			CachedMemoryHash = HashCombine(CachedMemoryHash, GetTypeHash(DefaultValue));
		}
	}

	return CachedMemoryHash;
}

//****************************************************************************

const TArray<const FProperty*> FRigVMMemoryStorageStruct::EmptyProperties;
const TArray<FRigVMPropertyPath> FRigVMMemoryStorageStruct::EmptyPropertyPaths;

FString FRigVMMemoryStorageStruct::GetDataAsString(int32 InPropertyIndex, int32 PortFlags) const
{
	check(IsValidIndex(InPropertyIndex));
	// Note that it is using Export Direct and also passing Data as the default value, so it forces the text exporter to generate all the elements
	// If we pass nullptr, values that are default initialized to a value that is different from the type default value will not be correctly exported
	FString RetValue;
	const uint8* Data = GetData<uint8>(InPropertyIndex);
	GetProperty(InPropertyIndex)->ExportTextItem_Direct(RetValue, Data, Data, nullptr, PortFlags);
	return RetValue;
}

FString FRigVMMemoryStorageStruct::GetDataAsString(const FRigVMOperand& InOperand, int32 PortFlags) const
{
	const int32 PropertyIndex = InOperand.GetRegisterIndex();
	check(IsValidIndex(PropertyIndex));
	return GetDataAsString(PropertyIndex, PortFlags);
}

FString FRigVMMemoryStorageStruct::GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags) const
{
	if (!IsValidIndex(InPropertyIndex))
	{
		return FString();
	}
	return GetDataAsString(InPropertyIndex, PortFlags);
}

FString FRigVMMemoryStorageStruct::GetDataAsStringSafe(const FRigVMOperand& InOperand, int32 PortFlags) const
{
	const int32 PropertyIndex = InOperand.GetRegisterIndex();
	return GetDataAsStringSafe(PropertyIndex, PortFlags);
}

bool FRigVMMemoryStorageStruct::SetDataFromString(int32 InPropertyIndex, const FString& InValue)
{
	check(IsValidIndex(InPropertyIndex));
	uint8* Data = GetData<uint8>(InPropertyIndex);

	const FProperty* Property = GetProperty(InPropertyIndex);

	FRigVMMemoryStorageImportErrorContext ErrorPipe(false);
	Property->ImportText_Direct(*InValue, Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);

	if (ErrorPipe.NumErrors > 0)
	{
		// check if the value was provided as a single element
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			const FString ValueWithBraces = FString::Printf(UE::RigVM::RigVMCore::Private::BraceFormat, *InValue);

			ErrorPipe = FRigVMMemoryStorageImportErrorContext(false);
			Property->ImportText_Direct(*ValueWithBraces, Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
		}
	}

	return ErrorPipe.NumErrors == 0;
}

//****************************************************************************

FRigVMMemoryHandle FRigVMMemoryStorageStruct::GetHandle(int32 InPropertyIndex, const FRigVMPropertyPath* InPropertyPath)
{
	check(IsValidIndex(InPropertyIndex));

	const FProperty* Property = GetProperty(InPropertyIndex);
	uint8* Data = GetData<uint8>(InPropertyIndex);

	return FRigVMMemoryHandle(Data, Property, InPropertyPath);
}

//****************************************************************************

bool FRigVMMemoryStorageStruct::CopyProperty(
	const FProperty* InTargetProperty,
	uint8* InTargetPtr,
	const FProperty* InSourceProperty,
	const uint8* InSourcePtr)
{
	check(InTargetProperty != nullptr);
	check(InSourceProperty != nullptr);
	check(InTargetPtr != nullptr);
	check(InSourcePtr != nullptr);

	// This block below is there to support Large World Coordinates (LWC).
	// We allow to link float and double pins (single and arrays) so we need
	// to support copying values between those as well.
	if (!InTargetProperty->SameType(InSourceProperty))
	{
		if (const FFloatProperty* TargetFloatProperty = CastField<FFloatProperty>(InTargetProperty))
		{
			if (const FDoubleProperty* SourceDoubleProperty = CastField<FDoubleProperty>(InSourceProperty))
			{
				if (TargetFloatProperty->ArrayDim == SourceDoubleProperty->ArrayDim)
				{
					float* TargetFloats = (float*)InTargetPtr;
					double* SourceDoubles = (double*)InSourcePtr;
					for (int32 Index = 0; Index < TargetFloatProperty->ArrayDim; Index++)
					{
						TargetFloats[Index] = (float)SourceDoubles[Index];
					}
					return true;
				}
			}
		}
		else if (const FDoubleProperty* TargetDoubleProperty = CastField<FDoubleProperty>(InTargetProperty))
		{
			if (const FFloatProperty* SourceFloatProperty = CastField<FFloatProperty>(InSourceProperty))
			{
				if (TargetDoubleProperty->ArrayDim == SourceFloatProperty->ArrayDim)
				{
					double* TargetDoubles = (double*)InTargetPtr;
					float* SourceFloats = (float*)InSourcePtr;
					for (int32 Index = 0; Index < TargetDoubleProperty->ArrayDim; Index++)
					{
						TargetDoubles[Index] = (double)SourceFloats[Index];
					}
					return true;
				}
			}
		}
		else if (const FByteProperty* TargetByteProperty = CastField<FByteProperty>(InTargetProperty))
		{
			if (const FEnumProperty* SourceEnumProperty = CastField<FEnumProperty>(InSourceProperty))
			{
				if (TargetByteProperty->Enum == SourceEnumProperty->GetEnum())
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if (const FEnumProperty* TargetEnumProperty = CastField<FEnumProperty>(InTargetProperty))
		{
			if (const FByteProperty* SourceByteProperty = CastField<FByteProperty>(InSourceProperty))
			{
				if (TargetEnumProperty->GetEnum() == SourceByteProperty->Enum)
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if (const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(InTargetProperty))
		{
			if (const FArrayProperty* SourceArrayProperty = CastField<FArrayProperty>(InSourceProperty))
			{
				FScriptArrayHelper TargetArray(TargetArrayProperty, InTargetPtr);
				FScriptArrayHelper SourceArray(SourceArrayProperty, InSourcePtr);

				if (TargetArrayProperty->Inner->IsA<FFloatProperty>())
				{
					if (SourceArrayProperty->Inner->IsA<FDoubleProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for (int32 Index = 0; Index < TargetArray.Num(); Index++)
						{
							float* TargetFloat = (float*)TargetArray.GetRawPtr(Index);
							const double* SourceDouble = (const double*)SourceArray.GetRawPtr(Index);
							*TargetFloat = (float)*SourceDouble;
						}
						return true;
					}
				}
				else if (TargetArrayProperty->Inner->IsA<FDoubleProperty>())
				{
					if (SourceArrayProperty->Inner->IsA<FFloatProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for (int32 Index = 0; Index < TargetArray.Num(); Index++)
						{
							double* TargetDouble = (double*)TargetArray.GetRawPtr(Index);
							const float* SourceFloat = (const float*)SourceArray.GetRawPtr(Index);
							*TargetDouble = (double)*SourceFloat;
						}
						return true;
					}
				}
				else if (FByteProperty* TargetArrayInnerByteProperty = CastField<FByteProperty>(TargetArrayProperty->Inner))
				{
					if (FEnumProperty* SourceArrayInnerEnumProperty = CastField<FEnumProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerByteProperty->Enum == SourceArrayInnerEnumProperty->GetEnum())
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
				else if (FEnumProperty* TargetArrayInnerEnumProperty = CastField<FEnumProperty>(TargetArrayProperty->Inner))
				{
					if (FByteProperty* SourceArrayInnerByteProperty = CastField<FByteProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerEnumProperty->GetEnum() == SourceArrayInnerByteProperty->Enum)
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
			}
		}

		// if we reach this we failed since we are trying to copy
		// between two properties which are not compatible
		if (!InTargetProperty->SameType(InSourceProperty))
		{
			// Only log the issue once, rather than spam.
			static TSet<FString> ReportedErrors;

			const FString TargetType = RigVMTypeUtils::GetCPPTypeFromProperty(InTargetProperty);
			const FString SourceType = RigVMTypeUtils::GetCPPTypeFromProperty(InSourceProperty);

			UPackage* Package = InTargetProperty->GetOutermost();

			FString Message = FString::Printf(TEXT("Failed to copy %s (%s) to %s (%s) in package %s"),
				*InSourceProperty->GetName(),
				*SourceType,
				*InTargetProperty->GetName(),
				*TargetType,
				Package ? *Package->GetName() : TEXT("<Unknown Package>"));

			if (!ReportedErrors.Contains(Message))
			{
				UE_LOGF(LogRigVM, Error, "%ls", *Message);
				ReportedErrors.Add(Message);
			}

			return false;
		}
	}

	// rely on the core to copy the property contents
	InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
	return true;
}

bool FRigVMMemoryStorageStruct::CopyProperty(
	const FProperty* InTargetProperty,
	uint8* InTargetPtr,
	const FRigVMPropertyPath& InTargetPropertyPath,
	const FProperty* InSourceProperty,
	const uint8* InSourcePtr,
	const FRigVMPropertyPath& InSourcePropertyPath)
{
	check(InTargetProperty != nullptr);
	check(InSourceProperty != nullptr);
	check(InTargetPtr != nullptr);
	check(InSourcePtr != nullptr);

	auto TraversePropertyPath = [](const FProperty*& Property, uint8*& MemoryPtr, const FRigVMPropertyPath& PropertyPath)
	{
		if (PropertyPath.IsEmpty())
		{
			return;
		}

		MemoryPtr = PropertyPath.GetData<uint8>(MemoryPtr, Property);
		Property = PropertyPath.GetTailProperty();
	};

	uint8* SourcePtr = (uint8*)InSourcePtr;
	TraversePropertyPath(InTargetProperty, InTargetPtr, InTargetPropertyPath);
	TraversePropertyPath(InSourceProperty, SourcePtr, InSourcePropertyPath);

	if (InTargetPtr == nullptr)
	{
		check(!InTargetPropertyPath.IsEmpty());
		return false;
	}
	if (InSourcePtr == nullptr)
	{
		check(!InSourcePropertyPath.IsEmpty());
		return false;
	}

	return CopyProperty(InTargetProperty, InTargetPtr, InSourceProperty, SourcePtr);
}

bool FRigVMMemoryStorageStruct::CopyProperty(
	FRigVMMemoryStorageStruct* InTargetStorage,
	int32 InTargetPropertyIndex,
	const FRigVMPropertyPath& InTargetPropertyPath,
	FRigVMMemoryStorageStruct* InSourceStorage,
	int32 InSourcePropertyIndex,
	const FRigVMPropertyPath& InSourcePropertyPath)
{
	check(InTargetStorage != nullptr);
	check(InSourceStorage != nullptr);

	const FProperty* TargetProperty = InTargetStorage->GetProperty(InTargetPropertyIndex);
	const FProperty* SourceProperty = InSourceStorage->GetProperty(InSourcePropertyIndex);
	uint8* TargetPtr = InTargetStorage->GetData<uint8>(TargetProperty);
	uint8* SourcePtr = InSourceStorage->GetData<uint8>(SourceProperty);

	return CopyProperty(TargetProperty, TargetPtr, InTargetPropertyPath, SourceProperty, SourcePtr, InSourcePropertyPath);
}

bool FRigVMMemoryStorageStruct::CopyProperty(
	FRigVMMemoryHandle& InTargetHandle,
	FRigVMMemoryHandle& InSourceHandle)
{
	return CopyProperty(
		InTargetHandle.GetProperty(),
		InTargetHandle.GetData_Internal(false, INDEX_NONE),
		InTargetHandle.GetPropertyPathRef(),
		InSourceHandle.GetProperty(),
		InSourceHandle.GetData_Internal(false, INDEX_NONE),
		InSourceHandle.GetPropertyPathRef());
}

const UStruct* FRigVMMemoryStorageStruct::GetStruct() const
{
	return Value.IsValid() ? Value.GetScriptStruct() : nullptr;
}

//****************************************************************************

void FRigVMMemoryStorageStruct::RefreshLinkedProperties()
{
	LinkedProperties.Reset();

	if (const UStruct* Struct = GetStruct())
	{
		const FProperty* Property = CastField<FProperty>(Struct->ChildProperties);
		while (Property)
		{
			LinkedProperties.Add(Property);
			Property = CastField<FProperty>(Property->Next);
		}
	}
}

void FRigVMMemoryStorageStruct::Refresh()
{
	RefreshLinkedProperties();
	RefreshPropertyPaths();
}

void FRigVMMemoryStorageStruct::RefreshPropertyPaths()
{
	// Update the property paths based on the descriptions
	PropertyPaths.SetNumZeroed(PropertyPathDescriptions.Num());
	for (int32 PropertyPathIndex = 0; PropertyPathIndex < PropertyPaths.Num(); PropertyPathIndex++)
	{
		PropertyPaths[PropertyPathIndex] = FRigVMPropertyPath();

		const int32 PropertyIndex = PropertyPathDescriptions[PropertyPathIndex].PropertyIndex;
		if (LinkedProperties.IsValidIndex(PropertyIndex))
		{
			PropertyPaths[PropertyPathIndex] = FRigVMPropertyPath(
				LinkedProperties[PropertyIndex],
				PropertyPathDescriptions[PropertyPathIndex].SegmentPath);
#if WITH_EDITOR
			if (!PropertyPaths[PropertyPathIndex].IsValid())
			{
#if UE_BUILD_DEBUG
				UE_LOGF(LogRigVM, Warning, "Unable to resolve property path '%ls' for property '%ls'.", *PropertyPathDescriptions[PropertyPathIndex].SegmentPath, *LinkedProperties[PropertyIndex]->GetPathName());
#endif
			}
#endif
		}
	}
}

void FRigVMMemoryStorageStruct::SetDefaultValues(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions)
{
	// and store default values.
	const TArray<const FProperty*>& Properties = GetProperties();
	
	const int32 NumProperties = Properties.Num();
	if (!ensure(NumProperties == InPropertyDescriptions.Num()))
	{
		UE_LOGF(LogRigVM, Error, "Number of properties in storage is different from descriptors, unsupported type or duplicated name in the generated properties");
		return;
	}

	for (int32 PropertyIndex = 0; PropertyIndex < NumProperties; PropertyIndex++)
	{
		const FRigVMPropertyDescription& RigVMDesc = InPropertyDescriptions[PropertyIndex];
		const FString& DefaultValue = RigVMDesc.DefaultValue;

		if (DefaultValue.IsEmpty() || DefaultValue == UE::RigVM::RigVMCore::Private::EmptyBraces)
		{
			continue;
		}

		SetDataFromString(PropertyIndex, DefaultValue);
	}
}

//****************************************************************************

/*static*/ FPropertyBagPropertyDesc FRigVMMemoryStorageStruct::GeneratePropertyBagDescriptor(const FRigVMPropertyDescription& RigVMDescriptor)
{
	FPropertyBagPropertyDesc Result;

	EPropertyBagPropertyType PropertyBagType = EPropertyBagPropertyType::None;
	FPropertyBagContainerTypes PropertyBagContainerTypes;

	if (GetPropertyTypeDataFromVMDescriptor(RigVMDescriptor, PropertyBagType, PropertyBagContainerTypes))
	{
		if (PropertyBagContainerTypes.Num() > 0)
		{
			Result = FPropertyBagPropertyDesc(RigVMDescriptor.Name, PropertyBagContainerTypes, PropertyBagType, RigVMDescriptor.CPPTypeObject);
		}
		else
		{
			Result = FPropertyBagPropertyDesc(RigVMDescriptor.Name, PropertyBagType, RigVMDescriptor.CPPTypeObject);
		}

		const FString Name = RigVMDescriptor.Name.ToString();

		FBlake3 Builder;
		Builder.Update(*Name, Name.Len() * sizeof(TCHAR));
		Builder.Update(*RigVMDescriptor.CPPType, RigVMDescriptor.CPPType.Len()  * sizeof(TCHAR));
		FBlake3Hash Hash = Builder.Finalize();
		Result.ID = FGuid::NewGuidFromHash(Hash);
	}

	return Result;
}

/*static*/ bool FRigVMMemoryStorageStruct::GetPropertyTypeDataFromVMDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, EPropertyBagPropertyType & OutBagPropertyType, FPropertyBagContainerTypes& OutBagContainerTypes)
{
	EPropertyBagPropertyType BagPropertyType = EPropertyBagPropertyType::None;

	FString VMTypeString = *RigVMDescriptor.CPPType;

	OutBagContainerTypes.Reset(); // in case it is reused by the caller

	if (RigVMTypeUtils::IsArrayType(VMTypeString))
	{
		const int32 NumContainers = RigVMDescriptor.Containers.Num();
		for (int i= 0; i < NumContainers; ++i)
		{
			switch (RigVMDescriptor.Containers[i])
			{
			case EPinContainerType::Array:
				OutBagContainerTypes.Add(EPropertyBagContainerType::Array);
				break;
			case EPinContainerType::Set:
				ensureMsgf(false, TEXT("Unsuported Set type container : %s"), *VMTypeString);
				break;
			case EPinContainerType::Map:
				ensureMsgf(false, TEXT("Unsuported Map type container : %s"), *VMTypeString);
				break;
			default:
				break;
			}
		}

		VMTypeString = RigVMDescriptor.GetTailCPPType();
	}

	FName VMType = *VMTypeString;

	if (VMType == RigVMTypeUtils::BoolTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Bool;
	}
	else if (VMType == RigVMTypeUtils::UInt8TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Byte;
	}
	else if (VMType == RigVMTypeUtils::Int32TypeName || VMType == RigVMTypeUtils::IntTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Int32;
	}
	else if (VMType == RigVMTypeUtils::UInt32TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::UInt32;
	}
	else if (VMType == RigVMTypeUtils::Int64TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Int64;
	}
	else if (VMType == RigVMTypeUtils::UInt64TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::UInt64;
	}
	else if (VMType == RigVMTypeUtils::FloatTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Float;
	}
	else if (VMType == RigVMTypeUtils::DoubleTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Double;
	}
	else if (VMType == RigVMTypeUtils::FNameTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Name;
	}
	else if (VMType == RigVMTypeUtils::FStringTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::String;
	}
	else if (VMType == RigVMTypeUtils::FTextTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Text;
	}
	else if (Cast<UScriptStruct>(RigVMDescriptor.CPPTypeObject))
	{
		OutBagPropertyType = EPropertyBagPropertyType::Struct;
	}
	else if (Cast<UEnum>(RigVMDescriptor.CPPTypeObject))
	{
		OutBagPropertyType = EPropertyBagPropertyType::Enum;
	}
	else if (Cast<UObject>(RigVMDescriptor.CPPTypeObject))
	{
		const bool bIsClass = RigVMTypeUtils::IsUClassType(RigVMDescriptor.CPPType);
		if(bIsClass)
		{
			OutBagPropertyType = EPropertyBagPropertyType::Class;
		}
		else
		{
			OutBagPropertyType = EPropertyBagPropertyType::Object;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported type : %s"), *VMTypeString);
		OutBagPropertyType = EPropertyBagPropertyType::None;
	}

	return OutBagPropertyType != EPropertyBagPropertyType::None;
}
