// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMNameCache.h"
#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "RigVMCore/RigVM.h"
TAutoConsoleVariable<int32> CVarRigVMNameCacheMaxSize(TEXT("RigVM.NameCacheMaxSize"), 4096, TEXT("Change to control how many names are cached per VM instance."));
#endif

////////////////////////////////////////////////////////////////////////////////
// FRigVMNameOp
////////////////////////////////////////////////////////////////////////////////

FRigVMNameOp FRigVMNameOp::Concat(const FName& InA, const FName& InB)
{
	FRigVMNameOp Op;
	Op.Type = ERigVMNameOp::Concat;
	Op.A = GetTypeHash(InA);
	Op.B = GetTypeHash(InB);
	return Op;
}

FRigVMNameOp FRigVMNameOp::Left(const FName& InA, const uint32 InCount)
{
	FRigVMNameOp Op;
	Op.Type = ERigVMNameOp::Left;
	Op.A = GetTypeHash(InA);
	Op.B = InCount;
	return Op;
}

FRigVMNameOp FRigVMNameOp::Right(const FName& InA, const uint32 InCount)
{
	FRigVMNameOp Op;
	Op.Type = ERigVMNameOp::Right;
	Op.A = GetTypeHash(InA);
	Op.B = InCount;
	return Op;
}

FRigVMNameOp FRigVMNameOp::LeftChop(const FName& InA, const uint32 InCount)
{
	FRigVMNameOp Op;
	Op.Type = ERigVMNameOp::LeftChop;
	Op.A = GetTypeHash(InA);
	Op.B = InCount;
	return Op;
}

FRigVMNameOp FRigVMNameOp::RightChop(const FName& InA, const uint32 InCount)
{
	FRigVMNameOp Op;
	Op.Type = ERigVMNameOp::RightChop;
	Op.A = GetTypeHash(InA);
	Op.B = InCount;
	return Op;
}

FRigVMNameOp FRigVMNameOp::Replace(const FName& InA, const FName& InB, const FName& InC, const ESearchCase::Type InSearchCase)
{
	FRigVMNameOp Op;
	Op.Type = InSearchCase == ESearchCase::CaseSensitive ? ERigVMNameOp::ReplaceCase : ERigVMNameOp::ReplaceNoCase;
	Op.A = GetTypeHash(InA);
	Op.B = GetTypeHash(InB);
	Op.C = GetTypeHash(InC);
	return Op;
}

FRigVMNameOp FRigVMNameOp::EndsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	FRigVMNameOp Op;
	Op.Type = InSearchCase == ESearchCase::CaseSensitive ? ERigVMNameOp::EndsWithCase : ERigVMNameOp::EndsWithNoCase;
	Op.A = GetTypeHash(InA);
	Op.B = GetTypeHash(InB);
	return Op;
}

FRigVMNameOp FRigVMNameOp::StartsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	FRigVMNameOp Op;
	Op.Type = InSearchCase == ESearchCase::CaseSensitive ? ERigVMNameOp::StartsWithCase : ERigVMNameOp::StartsWithNoCase;
	Op.A = GetTypeHash(InA);
	Op.B = GetTypeHash(InB);
	return Op;
}

FRigVMNameOp FRigVMNameOp::Contains(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	FRigVMNameOp Op;
	Op.Type = InSearchCase == ESearchCase::CaseSensitive ? ERigVMNameOp::ContainsCase : ERigVMNameOp::ContainsNoCase;
	Op.A = GetTypeHash(InA);
	Op.B = GetTypeHash(InB);
	return Op;
}

////////////////////////////////////////////////////////////////////////////////
// FRigVMNameCache
////////////////////////////////////////////////////////////////////////////////

FName FRigVMNameCache::Concat(const FName& InA, const FName& InB)
{
	if(InA.IsNone())
	{
		return InB;
	}

	if(InB.IsNone())
	{
		return InA;
	}
	
	const FRigVMNameOp Op = FRigVMNameOp::Concat(InA, InB);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}
	const FName NewName = *(InA.ToString() + InB.ToString());

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	return NewName;
}

FName FRigVMNameCache::Left(const FName& InA, const uint32 InCount)
{
	if(InCount == 0 || InA.IsNone())
	{
		return NAME_None;
	}

	if(InCount >= InA.GetStringLength())
	{
		return InA;
	}
	
	const FRigVMNameOp Op = FRigVMNameOp::Left(InA, InCount);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}
	const FName NewName = *InA.ToString().Left(InCount);

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	
	return NewName;
}

FName FRigVMNameCache::Right(const FName& InA, const uint32 InCount)
{
	if(InCount == 0 || InA.IsNone())
	{
		return NAME_None;
	}

	if(InCount >= InA.GetStringLength())
	{
		return InA;
	}

	const FRigVMNameOp Op = FRigVMNameOp::Right(InA, InCount);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}
	const FName NewName = *InA.ToString().Right(InCount);

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	
	return NewName;
}

FName FRigVMNameCache::LeftChop(const FName& InA, const uint32 InCount)
{
	if(InCount == 0 || InA.IsNone())
	{
		return InA;
	}
	
	const FRigVMNameOp Op = FRigVMNameOp::LeftChop(InA, InCount);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}
	const FName NewName = *InA.ToString().LeftChop(InCount);

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	
	return NewName;
}

FName FRigVMNameCache::RightChop(const FName& InA, const uint32 InCount)
{
	if(InCount == 0 || InA.IsNone())
	{
		return InA;
	}

	const FRigVMNameOp Op = FRigVMNameOp::RightChop(InA, InCount);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}
	const FName NewName = *InA.ToString().RightChop(InCount);

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	
	return NewName;
}

FName FRigVMNameCache::Replace(const FName& InA, const FName& InB, const FName& InC, const ESearchCase::Type InSearchCase)
{
	if(InA.IsNone() || InB.IsNone())
	{
		return InA;
	}

	const FRigVMNameOp Op = FRigVMNameOp::Replace(InA, InB, InC, InSearchCase);
	if(const FName* ExistingName = NameCache.Find(Op))
	{
		return *ExistingName;
	}

	FString NewString(TEXT(""));
	if (!InC.IsNone())
	{
		NewString = *InC.ToString();
	}
	const FName NewName = *InA.ToString().Replace(*InB.ToString(), *NewString, InSearchCase);

	if(CheckCacheSize())
	{
		NameCache.Add(Op, NewName);
	}
	
	return NewName;
}

bool FRigVMNameCache::EndsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	if(InA.IsNone() || InB.IsNone())
	{
		return false;
	}
	
	const FRigVMNameOp Op = FRigVMNameOp::EndsWith(InA, InB, InSearchCase);
	if(const bool* ExistingBool = BoolCache.Find(Op))
	{
		return *ExistingBool;
	}
	const bool bResult = InA.ToString().EndsWith(InB.ToString(), InSearchCase);

	if(CheckCacheSize())
	{
		BoolCache.Add(Op, bResult);
	}
	
	return bResult;
}

bool FRigVMNameCache::StartsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	if(InA.IsNone() || InB.IsNone())
	{
		return false;
	}

	const FRigVMNameOp Op = FRigVMNameOp::StartsWith(InA, InB, InSearchCase);
	if(const bool* ExistingBool = BoolCache.Find(Op))
	{
		return *ExistingBool;
	}
	const bool bResult = InA.ToString().StartsWith(InB.ToString(), InSearchCase);

	if(CheckCacheSize())
	{
		BoolCache.Add(Op, bResult);
	}
	
	return bResult;
}

bool FRigVMNameCache::Contains(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase)
{
	if(InA.IsNone() || InB.IsNone())
	{
		return false;
	}

	const FRigVMNameOp Op = FRigVMNameOp::Contains(InA, InB, InSearchCase);
	if(const bool* ExistingBool = BoolCache.Find(Op))
	{
		return *ExistingBool;
	}
	const bool bResult = InA.ToString().Contains(InB.ToString(), InSearchCase);

	if(CheckCacheSize())
	{
		BoolCache.Add(Op, bResult);
	}
	
	return bResult;
}

TArray<FRigVMNameOp> FRigVMNameCache::GetNameOps() const
{
	TArray<FRigVMNameOp> Result;
	NameCache.GenerateKeyArray(Result);
	return Result;
}

TArray<FName> FRigVMNameCache::GetNameValues() const
{
	TArray<FName> Result;
	NameCache.GenerateValueArray(Result);
	return Result;
}

TArray<FRigVMNameOp> FRigVMNameCache::GetBoolOps() const
{
	TArray<FRigVMNameOp> Result;
	BoolCache.GenerateKeyArray(Result);
	return Result;
}

TArray<bool> FRigVMNameCache::GetBoolValues() const
{
	TArray<bool> Result;
	BoolCache.GenerateValueArray(Result);
	return Result;
}

#if WITH_EDITOR
bool FRigVMNameCache::CheckCacheSize() const
{
	if(CVarRigVMNameCacheMaxSize.GetValueOnAnyThread() == NameCache.Num() || CVarRigVMNameCacheMaxSize.GetValueOnAnyThread() == BoolCache.Num())
	{
		UE_LOGF(LogRigVM, Warning, "FRigVMNameCache exceeded maximum size of %d. You can change it using the 'RigVM.NameCacheMaxSize' console variable.", CVarRigVMNameCacheMaxSize.GetValueOnAnyThread());
		return false;
	}
	return true;
}
#endif
