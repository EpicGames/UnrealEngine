// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMNativized.h"
#include "RigVMModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMNativized)

FRigVMNativizedContext::FRigVMNativizedContext(FRigVMExtendedExecuteContext& InExecuteContext, FRigVMExecuteContext& InPublicContext)
: ExecuteContext(InExecuteContext)
, PublicContextBase(InPublicContext)
{
}

URigVMNativized::URigVMNativized()
	: URigVM()
{
}

URigVMNativized::~URigVMNativized()
{
}

UClass* URigVMNativized::FindClassForHash(uint32 InVMHash)
{
	if (UClass** ClassPtr = GetNativizedClassMap().Find(InVMHash))
	{
		return *ClassPtr;
	}
	return nullptr;
}

void URigVMNativized::RegisterClassForHash(uint32 InVMHash, UClass* InClass)
{
	GetNativizedClassMap().FindOrAdd(InVMHash, InClass);
}

TMap<uint32, UClass*>& URigVMNativized::GetNativizedClassMap()
{
	static TMap<uint32, UClass*> Lookup;
	return Lookup;
}

void URigVMNativized::Serialize(FArchive& Ar)
{
	UObject::Serialize(Ar);
}

void URigVMNativized::Reset(FRigVMExtendedExecuteContext& Context)
{
	// don't call super on purpose
	ByteCodeStorage.Reset();
}

bool URigVMNativized::Initialize(FRigVMExtendedExecuteContext& Context)
{
	// nothing to do here 
	return true;
}

ERigVMExecuteResult URigVMNativized::ExecuteVM(FRigVMExtendedExecuteContext& Context, const FName& InEntryName)
{
	// to be implemented by the generated code
	return ERigVMExecuteResult::Failed;
}

FRigVMMemoryStorageStruct* URigVMNativized::GetMemoryByType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType)
{
	return Super::GetMemoryByType(Context, InMemoryType);
}

const FRigVMMemoryStorageStruct* URigVMNativized::GetMemoryByType(const FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType) const
{
	return Super::GetMemoryByType(Context, InMemoryType);
}

const FRigVMInstructionArray& URigVMNativized::GetInstructions()
{
	if(URigVM* SourceVM = GetSourceVM())
	{
		return SourceVM->GetInstructions();
	}
	static const FRigVMInstructionArray EmptyInstructions;
	return EmptyInstructions;
}

const FRigVMByteCode& URigVMNativized::GetByteCode() const
{
	if(const URigVM* SourceVM = GetSourceVM())
	{
		return SourceVM->GetByteCode();
	}
	return Super::GetByteCode();
}

const TArray<const FRigVMFunction*>& URigVMNativized::GetFunctions() const
{
	if(const URigVM* SourceVM = GetSourceVM())
	{
		return SourceVM->GetFunctions();
	}
	return Super::GetFunctions();
}

const TArray<FName>& URigVMNativized::GetFunctionNames() const
{
	if(const URigVM* SourceVM = GetSourceVM())
	{
		return SourceVM->GetFunctionNames();
	}
	return Super::GetFunctionNames();
}

const TArray<const FRigVMDispatchFactory*>& URigVMNativized::GetFactories() const
{
	if(const URigVM* SourceVM = GetSourceVM())
	{
		return SourceVM->GetFactories();
	}
	return Super::GetFactories();
}

const FRigVMFunction* URigVMNativized::FindDispatchFunction(const FString& InIdentifier) const
{
	if (LocalizedRegistry.IsValid())
	{
		return LocalizedRegistry->FindFunction_NoLock(*InIdentifier);
	}
	return FRigVMRegistry::Get().FindFunction(*InIdentifier);
}

URigVM* URigVMNativized::GetSourceVM() const
{
#if WITH_EDITOR
	if (WeakSourceVM.IsValid())
	{
		return WeakSourceVM.Get();
	}
#endif
	return nullptr;
}

#if WITH_EDITOR

void URigVMNativized::SetSourceVM(URigVM* InSourceVM)
{
	WeakSourceVM = InSourceVM;
}

void URigVMNativized::FDefaultValueImportErrorPipe::Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Warning)
	{
		UE_LOGF(LogRigVM, Error, "%ls: Error Importing Default Value: %ls", *Owner, V);
		NumErrors++;
	}
}

#endif
