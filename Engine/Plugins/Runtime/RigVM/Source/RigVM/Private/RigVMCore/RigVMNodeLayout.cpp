// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMNodeLayout.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMNodeLayout)

FString FRigVMPinCategory::GetName() const
{
	FString Left, Right;
	if(!RigVMStringUtils::SplitNodePathAtEnd(Path, Left, Right))
	{
		Right = Path;
	}
	return Right;
}

bool FRigVMPinCategory::IsDefaultCategory() const
{
	return Path.Equals(GetDefaultCategoryName(), ESearchCase::IgnoreCase);
}

const FString& FRigVMPinCategory::GetDefaultCategoryName()
{
	static const FString DefaultCategoryName = TEXT("Default");
	return DefaultCategoryName;
}

bool FRigVMNodeLayout::IsValid() const
{
	for(const FRigVMPinCategory& Category : Categories)
	{
		if(!Category.Elements.IsEmpty())
		{
			return true;
		}
	}
	return false;
}

const FString* FRigVMNodeLayout::FindCategory(const FString& InElement) const
{
	for(const FRigVMPinCategory& Category : Categories)
	{
		if(Category.Elements.Contains(InElement))
		{
			return &Category.Path;
		}
	}
	return nullptr;
}

const FString* FRigVMNodeLayout::FindDisplayName(const FString& InElement) const
{
	return DisplayNames.Find(InElement);
}