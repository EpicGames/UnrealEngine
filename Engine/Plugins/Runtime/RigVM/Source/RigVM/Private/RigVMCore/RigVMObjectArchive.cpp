// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMObjectArchive.h"
#include "Misc/Compression.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "RigVMObjectVersion.h"
#include "RigVMStringUtils.h"
#include "Misc/Base64.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMObjectArchive)

void FRigVMObjectArchive::Reset()
{
	Buffer.Reset();
	bIsCompressed = false;
	CompressedSize = UncompressedSize = INDEX_NONE;
}

void FRigVMObjectArchive::Empty()
{
	Buffer.Empty();
	bIsCompressed = false;
	CompressedSize = UncompressedSize = INDEX_NONE;
}

bool FRigVMObjectArchive::IsEmpty() const
{
	return Buffer.IsEmpty();
}

void FRigVMObjectArchive::Compress()
{
	if(IsCompressed())
	{
		return;
	}

	UncompressedSize = Buffer.Num();

	if(Buffer.IsEmpty())
	{
		return;
	}
	
	// It is possible for compression to actually increase the size of the data, so we over allocate here to handle that.
	CompressedSize = FCompression::CompressMemoryBound(NAME_Zlib, UncompressedSize);

	TArray<uint8> CompressedBuffer;
	CompressedBuffer.SetNumUninitialized(CompressedSize);

	if (FCompression::CompressMemory(NAME_Zlib, CompressedBuffer.GetData(), CompressedSize, Buffer.GetData(), Buffer.Num(), COMPRESS_BiasMemory))
	{
		// In the case that compressing it actually increases the size, we leave it uncompressed
		if (CompressedSize < UncompressedSize)
		{
			CompressedBuffer.SetNum(CompressedSize);
			Buffer = MoveTemp(CompressedBuffer);
			Buffer.Shrink();
			bIsCompressed = true;
		}
		else
		{
			CompressedSize = INDEX_NONE;
		}
	}

}

bool FRigVMObjectArchive::Decompress()
{
	if(!IsCompressed())
	{
		return true;
	}

	CompressedSize = Buffer.Num();
	
	TArray<uint8> UncompressedBuffer;
	UncompressedBuffer.SetNumUninitialized(UncompressedSize);

	if (FCompression::UncompressMemory(NAME_Zlib, UncompressedBuffer.GetData(), UncompressedSize, Buffer.GetData(), Buffer.Num()))
	{
		Buffer = MoveTemp(UncompressedBuffer);
		bIsCompressed = false;
		return true;
	}
	return false;
}

FString FRigVMObjectArchive::ToString()
{
	if (IsEmpty())
	{
		return FString();
	}
	
	Compress();

	TArray<FString> VersionStrings;
	for (const TPair<FGuid,int32>& Pair : Versions)
	{
		VersionStrings.Add(FString::Printf(TEXT("%s=%d"), *Pair.Key.ToString(EGuidFormats::Base36Encoded), Pair.Value));
	}
	const FString VersionsString = RigVMStringUtils::JoinStrings(VersionStrings, TEXT("|"));
	const FString Base64Header = FString::Printf(TEXT("%d,%d,%s,Base64:"), CompressedSize, UncompressedSize, *VersionsString);
	return Base64Header + FBase64::Encode(Buffer.GetData(), Buffer.Num());
}

bool FRigVMObjectArchive::SetFromString(const FString& InString)
{
	Reset();

	check(!InString.IsEmpty());
	check(InString.Contains(TEXT(",Base64:"), ESearchCase::CaseSensitive));

	FStringView CompleteView(InString);
	int32 Separator = INDEX_NONE;
	if(!CompleteView.FindChar(TEXT(':'), Separator))
	{
		Reset();
		return false;
	}
	FStringView Header = CompleteView.Left(Separator);
	FStringView Data = CompleteView.Mid(Separator + 1);

	if(!Header.FindChar(TEXT(','), Separator))
	{
		Reset();
		return false;
	}
	CompressedSize = FCString::Atoi(*FString(Header.Left(Separator)));
	Header = Header.Mid(Separator + 1);
	if(!Header.FindChar(TEXT(','), Separator))
	{
		Reset();
		return false;
	}
	UncompressedSize = FCString::Atoi(*FString(Header.Left(Separator)));
	Header = Header.Mid(Separator + 1);
	if(!Header.FindChar(TEXT(','), Separator))
	{
		Reset();
		return false;
	}
	const FString VersionsString = FString(Header.Left(Separator));
	TArray<FString> VersionStrings;
	RigVMStringUtils::SplitString(VersionsString, TEXT("|"), VersionStrings);
	for (const FString& VersionString: VersionStrings)
	{
		FString Guid,Value;
		if(!VersionString.Split(TEXT("="), &Guid, &Value))
		{
			Reset();
			return false;
		}
		Versions.Add(FGuid(Guid), FCString::Atoi(*Value));
	}
	Header = Header.Mid(Separator + 1);
	if(Header != TEXT("Base64"))
	{
		Reset();
		return false;		
	}

	if (!FBase64::Decode(FString(Data), Buffer))
	{
		Reset();
		return false;
	}

	bIsCompressed = CompressedSize != INDEX_NONE && CompressedSize < UncompressedSize;
	return true;
}

void FRigVMObjectArchive::SetVersionsFromArchive(const FArchive& Ar)
{
	const FCustomVersionContainer& VersionContainer = Ar.GetCustomVersions();
	const FCustomVersionArray& CustomVersions = VersionContainer.GetAllVersions();

	Versions.Reset();
	Versions.Reserve(CustomVersions.Num());
	for (const FCustomVersion& CustomVersion : CustomVersions)
	{
		Versions.FindOrAdd(CustomVersion.Key, 0) = CustomVersion.Version;
	}
}

FRigVMObjectArchiveWriter::FRigVMObjectArchiveWriter(FRigVMObjectArchive& InArchive, const UObject* InRoot)
: Archive( InArchive )
, Offset(0)
, Root( InRoot )
, RootPathName( InRoot ? InRoot->GetPathName() : FString() )
{
	SetIsSaving(true);
	SetIsLoading(false);
	UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	UsingCustomVersion(FRigVMObjectVersion::GUID);
	ArIgnoreOuterRef = 1;
}

FRigVMObjectArchiveWriter::~FRigVMObjectArchiveWriter()
{
	Archive.SetVersionsFromArchive(*this);
}

void FRigVMObjectArchiveWriter::Serialize(void* V, int64 Length)
{
	if(Length == 0)
	{
		check(V == nullptr);
	}
	else
	{
		check(V);
		int32 Start = IntCastChecked<int32, int64>(Offset);

		if(Archive.Buffer.IsValidIndex(Start))
		{
			const int32 MissingBytes = IntCastChecked<int32, int64>(Length) - (Archive.Buffer.Num() - Start);
			if(MissingBytes > 0)
			{
				(void)Archive.Buffer.AddUninitialized(MissingBytes);
			}
		}
		else if(Start == Archive.Buffer.Num())
		{
			Start = Archive.Buffer.AddUninitialized(IntCastChecked<int32, int64>(Length));
		}
		else
		{
			checkNoEntry();
		}
		
		FMemory::Memcpy( &(Archive.Buffer[Start]), V, Length );
		Offset += Length;
	}
}

int64 FRigVMObjectArchiveWriter::Tell()
{
	return Offset;
}

int64 FRigVMObjectArchiveWriter::TotalSize()
{
	return Archive.Buffer.Num();
}

void FRigVMObjectArchiveWriter::Seek(int64 InPos)
{
	check((InPos >= 0) && (InPos <= Archive.Buffer.Num()));
	Offset = InPos;
}

FArchive& FRigVMObjectArchiveWriter::operator<<(UObject*& Obj)
{
	if (Obj)
	{
		bool bStorePath = true;
		if(!VisitedObjects.Contains(Obj))
		{
			if((Obj == Root) || Obj->GetPathName().StartsWith(RootPathName, ESearchCase::CaseSensitive))
			{
				VisitedObjects.Add(Obj);
				bStorePath = false;

				uint8 State = StoringFullObject;
				*this << State;
				FName ClassName = *Obj->GetClass()->GetPathName();
				*this << ClassName;
				FName ObjectName = *Obj->GetName();
				*this << ObjectName;
				int32 ObjectFlags = (int32)Obj->GetFlags();
				*this << ObjectFlags;

				FName OuterPathName = FName(NAME_None);
				if(UObject* Outer = Obj->GetOuter())
				{
					if(VisitedObjects.Contains(Outer))
					{
						FString OuterPathNameString = Outer->GetPathName();
						if(OuterPathNameString.StartsWith(RootPathName, ESearchCase::CaseSensitive))
						{
							OuterPathNameString = OuterPathNameString.Mid(RootPathName.Len());
						}
						OuterPathName = OuterPathNameString.IsEmpty() ? FName(NAME_None) : *OuterPathNameString;
					}
				}
				
				*this << OuterPathName;

				Obj->Serialize(*this);
			}
		}

		if(bStorePath)
		{
			FString PathNameString = Obj->GetPathName();
			if(PathNameString.StartsWith(RootPathName, ESearchCase::CaseSensitive))
			{
				PathNameString = PathNameString.Mid(RootPathName.Len());
			}
			
			uint8 State = StoringArchiveLocalPath;
			*this << State;
			FName PathName = *PathNameString;
			*this << PathName;
		}
	}
	else
	{
		uint8 State = StoringNullptr;
		*this << State;
	}

	return *this;
}

FArchive& FRigVMObjectArchiveWriter::operator<<(FName& Value)
{
	if(const int64* NameOffsetPtr = NameToOffset.Find(Value))
	{
		uint8 State = StoringNameAsOffset;
		*this << State;
		
		int64 NameOffset = *NameOffsetPtr;
		*this << NameOffset;
	}
	else
	{
		uint8 State = StoringNameAsString;
		*this << State;
		NameToOffset.Add(Value, Tell());
		FString NameAsString = Value.IsNone() ? FString() : Value.ToString();
		*this << NameAsString;
	}

	return *this;
}

FArchive& FRigVMObjectArchiveWriter::operator<<(FText& Value)
{
	FString ValueString = Value.ToString();
	*this << ValueString;
	return *this;
}

FRigVMObjectArchiveReader::FRigVMObjectArchiveReader(FRigVMObjectArchive& InArchive, const UObject* InRoot)
: FRigVMObjectArchiveWriter(InArchive, InRoot)
{
	SetIsSaving(false);
	SetIsLoading(true);
	Archive.Decompress();

	ReadVersions();
}

void FRigVMObjectArchiveReader::Serialize(void* V, int64 Length)
{
	if(Length == 0)
	{
		check(V == nullptr);
	}
	else
	{
		check(V);
		check(Archive.Buffer.Num() >= Offset + Length);
		FMemory::Memcpy( V, &(Archive.Buffer[IntCastChecked<int32, int64>(Offset)]), Length );
		Offset += Length;
	}
}

FArchive& FRigVMObjectArchiveReader::operator<<(UObject*& Obj)
{
	uint8 State = UINT8_MAX;
	*this << State;

	if (State == StoringNullptr)
	{
		Obj = nullptr;
	}
	else
	{
		if(State == StoringFullObject)
		{
			const FObjectHeader Header = ReadObjectHeader();
			check(Header.IsValid());

			const EObjectFlags Flags = Header.Flags | RF_NeedPostLoad;

			if(Obj != Root)
			{
				UObject* Outer = const_cast<UObject*>(Root);
				if(!Header.OuterPathName.IsNone())
				{
					Outer = ReadObjects.FindChecked(Header.OuterPathName);
				}

				Obj = StaticFindObjectFast(Header.Class, Outer, Header.Name);
				if(Obj == nullptr || !IsValid(Obj))
				{
					Obj = NewObject<UObject>(Outer, Header.Class, Header.Name, Flags);
				}
				else
				{
					Obj->SetFlags(Flags);
				}
			}
			else
			{
				check(IsValid(Obj));
			}

			FString PathNameString = Obj->GetPathName();
			if(PathNameString.StartsWith(RootPathName, ESearchCase::CaseSensitive))
			{
				PathNameString = PathNameString.Mid(RootPathName.Len());
			}
			const FName PathName = *PathNameString;

			check(!ReadObjects.Contains(PathName));
			ReadObjects.Add(PathName, Obj);
			
			Obj->Serialize(*this);
			DeserializedObjects.Add(Obj);
		}
		else if(State == StoringArchiveLocalPath)
		{
			FName PathName;
			*this << PathName;

			if(UObject** PreviouslyReadObj = ReadObjects.Find(PathName))
			{
				Obj = *PreviouslyReadObj;
				check(Obj);
			}
			else
			{
				Obj = FindObject<UObject>(nullptr, *PathName.ToString(), EFindObjectFlags::None);
				check(Obj);
				ReadObjects.Add(PathName, Obj);
			}
		}
		else
		{
			checkNoEntry();
		}
	}

	if(Obj == Root)
	{
		// perform post load
		for(UObject* DeserializedObject : DeserializedObjects)
		{
			DeserializedObject->PostLoad();
			DeserializedObject->ClearFlags(RF_NeedPostLoad);
		}
	}
	
	return *this;
}

FArchive& FRigVMObjectArchiveReader::operator<<(FName& Value)
{
	uint8 State = UINT8_MAX;
	*this << State;

	auto ReadNameString = [this, &Value]()
	{
		const int64 OffsetOfString = Tell();
		FString NameAsString;
		*this << NameAsString;
		if(NameAsString.IsEmpty())
		{
			Value = FName(NAME_None);
		}
		else
		{
			Value = *NameAsString;
		}
		OffsetToName.FindOrAdd(OffsetOfString) = Value;
	};

	if(State == StoringNameAsString)
	{
		// read the string at the current position
		ReadNameString();
	}
	else if(State == StoringNameAsOffset)
	{
		int64 OffsetOfString = -1;
		*this << OffsetOfString;
		check(Archive.Buffer.IsValidIndex(IntCastChecked<int32, int64>(OffsetOfString)));
		
		if(const FName* NamePtr = OffsetToName.Find(OffsetOfString))
		{
			Value = *NamePtr;
		}
		else
		{
			TGuardValue<int64> OffsetGuard(Offset, Offset);
			Seek(OffsetOfString);
			ReadNameString();
		}
	}
	else
	{
		checkNoEntry();
	}

	return *this;
}

FArchive& FRigVMObjectArchiveReader::operator<<(FText& Value)
{
	FString ValueString;
	*this << ValueString;
	Value = FText::FromString(ValueString);
	return *this;
}

FRigVMObjectArchiveReader::FObjectHeader FRigVMObjectArchiveReader::ReadObjectHeader(int64 InPosition)
{
	const int64 CurrentPosition = Tell();
	if(InPosition != INDEX_NONE)
	{
		Seek(InPosition);
	}
	
	FObjectHeader Header;
	FName ClassName = FName(NAME_None);
	*this << ClassName;

	if(UClass** ClassPtr = ReadClasses.Find(ClassName))
	{
		Header.Class = *ClassPtr;
	}
	else
	{
		Header.Class = FindObject<UClass>(nullptr, *ClassName.ToString(), EFindObjectFlags::None);
		ReadClasses.Add(ClassName, Header.Class);
	}

	*this << Header.Name;
	int32 ObjectFlags = (int32)Header.Flags;
	*this << ObjectFlags;
	Header.Flags = (EObjectFlags)ObjectFlags;
	*this << Header.OuterPathName;

	if(InPosition != INDEX_NONE)
	{
		Seek(CurrentPosition);
	}

	return Header;
}

FRigVMObjectArchiveReader::FObjectHeader FRigVMObjectArchiveReader::GetRootObjectHeader()
{
	TGuardValue<int64> OffsetGuard(Offset, Offset);

	Seek(0);
	
	uint8 State = UINT8_MAX;
	*this << State;

	if(State == StoringFullObject)
	{
		return ReadObjectHeader();
	}

	return FObjectHeader();
}

void FRigVMObjectArchiveReader::ReadVersions()
{
	if (Archive.Versions.IsEmpty())
	{
		return;
	}

	FCustomVersionContainer VersionContainer = GetCustomVersions();
	for (const TPair<FGuid,int32>& Pair : Archive.Versions)
	{
		VersionContainer.SetVersion(Pair.Key, Pair.Value, VersionContainer.GetFriendlyName(Pair.Key));
	}
	SetCustomVersions(VersionContainer);
}

FArchive& operator<<(FArchive& Ar, FRigVMObjectArchive& Data)
{
	if (!Ar.IsLoading() || Ar.CustomVer(FRigVMObjectVersion::GUID) >= FRigVMObjectVersion::ObjectArchiveVersionContainerSerialization)
	{
		Ar << Data.Versions;
	}
	else
	{
		Data.SetVersionsFromArchive(Ar);
	}
	Ar << Data.Buffer;
	Ar << Data.UncompressedSize;
	Ar << Data.CompressedSize;
	Ar << Data.bIsCompressed;
	return Ar;
}
