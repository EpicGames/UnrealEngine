// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMProfilingInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMProfilingInfo)

void FRigVMInstructionVisitInfo::SetupInstructionTracking(int32 InInstructionCount, int32 InCallableCount)
{
#if WITH_EDITOR
	ResetInstructionVisitedDuringLastRun(InInstructionCount);
	SetNumInstructionVisitedDuringLastRunZeroed(InInstructionCount);
	ResetInstructionVisitOrder(InInstructionCount);
	ResetCallableVisitedDuringLastRun(InCallableCount);
	SetNumCallableVisitedDuringLastRunZeroed(InCallableCount);
	ResetCallableVisitOrder(InCallableCount);
#endif
}

void FRigVMProfilingInfo::SetupInstructionTracking(int32 InInstructionCount, int32 InCallableCount, bool bEnableProfiling)
{
#if WITH_EDITOR
	ResetInstructionCyclesDuringLastRun(InInstructionCount);
	ResetCallableCyclesDuringLastRun(InCallableCount);
	if (bEnableProfiling)
	{
		InitInstructionCyclesDuringLastRunValues(InInstructionCount, UINT64_MAX);
		InitCallableCyclesDuringLastRunValues(InCallableCount, UINT64_MAX);
	}
#endif
}

void FRigVMProfilingInfo::StartProfiling(bool bEnableProfiling)
{
#if WITH_EDITOR
		SetStartCycles(0);
		SetOverallCycles(0);
		if (bEnableProfiling)
		{
			SetStartCycles(FPlatformTime::Cycles64());
		}
#endif
}

void FRigVMProfilingInfo::StopProfiling()
{
#if WITH_EDITOR
	const uint64 Cycles = GetOverallCycles() > 0 ? GetOverallCycles() : (FPlatformTime::Cycles64() - GetStartCycles());
	LastExecutionMicroSeconds = static_cast<double>(Cycles) * FPlatformTime::GetSecondsPerCycle() * 1000.0 * 1000.0;
#endif
}
