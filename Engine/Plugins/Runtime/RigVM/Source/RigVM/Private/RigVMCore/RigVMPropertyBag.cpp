// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMPropertyBag.h"
#include "RigVMTypeUtils.h"
#include "RigVMModule.h"
#include "Misc/Guid.h"
#include "Hash/Blake3.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMPropertyBag)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const FName FRigVMPropertyDescription::MD_DisplayName(TEXT("DisplayName"));

FRigVMPropertyDescription::FRigVMPropertyDescription(const FProperty* InProperty, const FString& InDefaultValue, const FName& InName, const bool bAllowSpacesInName)
	: Name(InName)
	, Property(InProperty)
	, CPPType()
	, CPPTypeObject(nullptr)
	, Containers()
	, DefaultValue(InDefaultValue)
{
	if(InName.IsNone() && Property != nullptr)
	{
		Name = Property->GetFName();
	}

	if(CPPType.IsEmpty() && Property != nullptr)
	{
		CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(Property);

		if(CPPType == RigVMTypeUtils::UInt8Type)
		{
			if(CastField<FBoolProperty>(Property))
			{
				CPPType = RigVMTypeUtils::BoolType;
			}
		}
		else if(CPPType == RigVMTypeUtils::UInt8ArrayType)
		{
			if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				if(CastField<FBoolProperty>(ArrayProperty->Inner))
				{
					CPPType = RigVMTypeUtils::BoolArrayType;
				}
			}
		}
	}

	if(CPPTypeObject == nullptr && Property != nullptr)
	{
		const FProperty* ValueProperty = Property;
		while(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ValueProperty))
		{
			ValueProperty = ArrayProperty->Inner;
		}
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(ValueProperty))
		{
			CPPTypeObject = StructProperty->Struct;
		}
#if UE_RIGVM_UOBJECT_PROPERTIES_ENABLED
		else if(const FClassProperty* ClassProperty = CastField<FClassProperty>(ValueProperty))
		{
			CPPTypeObject = ClassProperty->MetaClass;
		}
		else if(const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ValueProperty))
		{
			CPPTypeObject = ObjectProperty->PropertyClass;
		}
#endif
#if UE_RIGVM_UINTERFACE_PROPERTIES_ENABLED
		else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(ValueProperty))
		{
			CPPTypeObject = InterfaceProperty->InterfaceClass;
		}
#endif
		else if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ValueProperty))
		{
			CPPTypeObject = EnumProperty->GetEnum();
		}
		else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(ValueProperty))
		{
			CPPTypeObject = ByteProperty->Enum;
		}
	}
	
	// make sure to use valid names only
	SanitizeName(bAllowSpacesInName);

	// walk the property until you reach the tail
	// and build up the container array along the way
	const FProperty* ChildProperty = InProperty;
	do
	{
		if(ChildProperty->IsA<FObjectProperty>() ||
			ChildProperty->IsA<FSoftObjectProperty>())
		{
			checkf(RigVMCore::SupportsUObjects(), TEXT("UClass types are not supported."));
		}

		if (ChildProperty->IsA<FInterfaceProperty>())
		{
			checkf(RigVMCore::SupportsUInterfaces(), TEXT("UInterface types are not supported."));
		}

		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ChildProperty))
		{
			Containers.Add(EPinContainerType::Array);
			ChildProperty = ArrayProperty->Inner;
		}
		else if(const FSetProperty* SetProperty = CastField<FSetProperty>(ChildProperty))
		{
			checkNoEntry();
		}
		else if(const FMapProperty* MapProperty = CastField<FMapProperty>(ChildProperty))
		{
			Containers.Add(EPinContainerType::Map);
			ChildProperty = MapProperty->ValueProp;
		}
		else
		{
			ChildProperty = nullptr;
		}
	}
	while (ChildProperty);
}

FRigVMPropertyDescription::FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const bool bAllowSpacesInName)
	: Name(InName)
	, Property(nullptr)
	, CPPType(InCPPType)
	, CPPTypeObject(InCPPTypeObject)
	, Containers()
	, DefaultValue(InDefaultValue)
{
	if(CPPTypeObject)
	{
		if(CPPTypeObject->IsA<UClass>())
		{
			if (!RigVMCore::SupportsUInterfaces() || !Cast<UClass>(CPPTypeObject)->IsChildOf<UInterface>())
			{
				checkf(RigVMCore::SupportsUObjects(), TEXT("UClass types are not supported."));
			}
		}
	}

	// only allow valid names
	SanitizeName(bAllowSpacesInName);

	FString TailCPPType = CPPType;

	// build the containers and validate the expected string
	// for the tail CPP type
	do
	{
		if(TailCPPType.RemoveFromStart(ArrayPrefix))
		{
			Containers.Add(EPinContainerType::Array);
			verify(TailCPPType.RemoveFromEnd(ContainerSuffix));
		}
		else if(TailCPPType.RemoveFromStart(MapPrefix))
		{
			Containers.Add(EPinContainerType::Map);
			verify(TailCPPType.RemoveFromEnd(ContainerSuffix));
		}
		else
		{
			break;
		}
	}
	while(!TailCPPType.IsEmpty());

	// sanity check the CPPType matches the provided struct
	if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
	{
		checkf(TailCPPType == RigVMTypeUtils::GetUniqueStructTypeName(ScriptStruct), TEXT("CPPType '%s' doesn't match provided Struct '%s'"),
			*TailCPPType, *RigVMTypeUtils::GetUniqueStructTypeName(ScriptStruct));
	}
}

#if WITH_EDITOR
FRigVMPropertyDescription::FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const EPropertyFlags InPropertyFlags, const TMap<FName, FString>& InMetaData, const bool bAllowSpacesInName)
#else
FRigVMPropertyDescription::FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const EPropertyFlags InPropertyFlags, const bool bAllowSpacesInName)
#endif
	: FRigVMPropertyDescription(InName, InCPPType, InCPPTypeObject, InDefaultValue, bAllowSpacesInName)
{
	PropertyFlags = InPropertyFlags;
#if WITH_EDITOR
	MetaData = InMetaData;
#endif
}

FName FRigVMPropertyDescription::SanitizeName(const FName& InName, const bool bAllowSpaces)
{
	FString NameString = InName.ToString();
	SanitizeName(NameString, bAllowSpaces);

	if (NameString != InName.ToString())
	{
		return *NameString;
	}

	return InName;

}

void FRigVMPropertyDescription::SanitizeName(FString& InString, const bool bAllowSpaces)
{
	// Sanitize the name
	for (int32 i = 0; i < InString.Len(); ++i)
	{
		TCHAR& C = InString[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||							// Any letter
			(C == '_') || 													// _ anytime
			(bAllowSpaces && (i > 0) && (C == ' ')) ||						// spaces after the first character
			((i > 0) && (FChar::IsDigit(C)));								// 0-9 after the first character

		if (!bGoodChar)
		{
			C = '_';
		}
	}
}

void FRigVMPropertyDescription::SanitizeName(const bool bAllowSpaces)
{
	Name = SanitizeName(Name, bAllowSpaces);
}

FString FRigVMPropertyDescription::GetTailCPPType() const
{
	FString TailCPPType = CPPType;

	// walk the containers and remove prefix and
	// suffix from the path. Turns 'TArray<TArray<float>>' to 'float'
	for(EPinContainerType Container : Containers)
	{
		switch(Container)
		{
			case EPinContainerType::Array:
			{
				verify(TailCPPType.RemoveFromStart(ArrayPrefix));
				verify(TailCPPType.RemoveFromEnd(ContainerSuffix));
				break;
			}		
			case EPinContainerType::Map:
			{
				verify(TailCPPType.RemoveFromStart(MapPrefix));
				verify(TailCPPType.RemoveFromEnd(ContainerSuffix));
				break;
			}		
			case EPinContainerType::None:
			default:
			{
				break;
			}
		}
	}
	
	return TailCPPType;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FRigVMPropertyBag::FRigVMPropertyBag()
	: FInstancedPropertyBag()
{
}

bool FRigVMPropertyBag::Serialize(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, TEXT("FRigVMPropertyBag"));
	Super::Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	if (Ar.IsLoading())
	{
		// rebuild property list and property path list
		Refresh();

		CachedMemoryHash = 0;
	}

	return true;
}

void FRigVMPropertyBag::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddStructReferencedObjects(Collector);
}

void FRigVMPropertyBag::GetUserDefinedDependencies(TArray<const UObject*>& OutDependencies) const
{
	ForEachObjectDependency([&OutDependencies](const UObject* Dependency)
	{
		OutDependencies.AddUnique(Dependency);
	});
}

void FRigVMPropertyBag::GetRequiredPlugins(TArray<FString>& OutPlugins) const
{
	ForEachObjectDependency([&OutPlugins](const UObject* Dependency)
	{
		const FString PluginName = RigVMTypeUtils::GetPluginName(Dependency);
		if (!PluginName.IsEmpty())
		{
			OutPlugins.AddUnique(PluginName);
		}
	});
}

void FRigVMPropertyBag::ForEachObjectDependency(const TFunction<void(const UObject*)>& InCallBack) const
{
	if (!InCallBack)
	{
		return;
	}
	
	const TArray<const FProperty*>& Properties = GetProperties();
	for (const FProperty* Property : Properties)
	{
		const FProperty* PropertyToVisit = Property;
		while (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyToVisit))
		{
			PropertyToVisit = ArrayProperty->Inner;
		}
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyToVisit))
		{
			if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(StructProperty->Struct))
			{
				InCallBack(UserDefinedStruct);
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyToVisit))
		{
			if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(EnumProperty->GetEnum()))
			{
				InCallBack(UserDefinedEnum);
			}
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyToVisit))
		{
			if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(ByteProperty->Enum))
			{
				InCallBack(UserDefinedEnum);
			}
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyToVisit))
		{
			// if we are a root property - just get the value
			if (PropertyToVisit == Property)
			{
				const UObject* Dependency = ObjectProperty->GetObjectPropertyValue_InContainer(Value.GetMemory());
				if (Dependency && FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Dependency))
				{
					InCallBack(Dependency);
				}
			}
			else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property)) // if we are within an array property
			{
				if (const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(ArrayProperty->Inner))
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, Value.GetMemory());
					for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
					{
						UObject* Dependency = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetRawPtr(ElementIndex));
						if (Dependency && FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Dependency))
						{
							InCallBack(Dependency);
						}
					}
				}
			}
		}
	}
}

void FRigVMPropertyBag::AddProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions)
{
	TArray<FPropertyBagPropertyDesc> BagDescriptors;
	BagDescriptors.Reserve(InPropertyDescriptions.Num());

	// Generate PropertyBag descriptors
	for (const FRigVMPropertyDescription& RigVMDescriptor : InPropertyDescriptions)
	{
		if (!ensure(BagDescriptors.FindByPredicate([&RigVMDescriptor](const FPropertyBagPropertyDesc& Desc) { return Desc.Name == RigVMDescriptor.Name; }) == nullptr))
		{
			UE_LOGF(LogRigVM, Error, "Detected duplicated name in the generated properties : [%ls]", *RigVMDescriptor.Name.ToString());
		}

		const FPropertyBagPropertyDesc& PropertyBagDescriptor = GeneratePropertyBagDescriptor(RigVMDescriptor, !RigVMDescriptor.Guid.IsValid());
		if (PropertyBagDescriptor.ValueType != EPropertyBagPropertyType::None)
		{
			BagDescriptors.Add(PropertyBagDescriptor);
		}
	}

	// Generate the struct with the passed types
	Super::AddProperties(BagDescriptors);

	Refresh();

	// And set the passed text DefaultValues
	SetDefaultValuesOnTailProperties(InPropertyDescriptions);
	
	CachedMemoryHash = 0;
}

EPropertyBagAlterationResult FRigVMPropertyBag::RemovePropertyByName(const FName& InPropertyName)
{
	const FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(InPropertyName);
	const EPropertyBagAlterationResult Result = Super::RemovePropertyByName(SanitizedName);
	if (Result == EPropertyBagAlterationResult::Success)
	{
		Refresh();
	}

	return Result;
}

int32 FRigVMPropertyBag::GetPropertyIndex(const FProperty* InProperty) const
{
	if (InProperty != nullptr)
	{
		const TArray<const FProperty*>& Properties = GetProperties();

		return Properties.IndexOfByKey(InProperty);
	}

	return INDEX_NONE;
}

int32 FRigVMPropertyBag::GetPropertyIndexByName(const FName& InName) const
{
	const FProperty* Property = FindPropertyByName(InName);
	return GetPropertyIndex(Property);
}

FProperty* FRigVMPropertyBag::FindPropertyByName(const FName& InName) const
{
	const FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(InName);
	return Value.GetScriptStruct() != nullptr ? Value.GetScriptStruct()->FindPropertyByName(SanitizedName) : nullptr;
}

void* FRigVMPropertyBag::GetContainerPtr() const
{
	return (void*)GetValue().GetMemory();
}

//****************************************************************************

const TArray<const FProperty*> FRigVMPropertyBag::EmptyProperties;

FString FRigVMPropertyBag::GetDataAsString(int32 InPropertyIndex, int32 PortFlags) const
{
	check(IsValidIndex(InPropertyIndex));
	// Note that it is using Export Direct and also passing Data as the default value, so it forces the text exporter to generate all the elements
	// If we pass nullptr, values that are default initialized to a value that is different from the type default value will not be correctly exported
	FString RetValue;
	const uint8* Data = GetData<uint8>(InPropertyIndex);
	GetProperty(InPropertyIndex)->ExportTextItem_Direct(RetValue, Data, Data, nullptr, PortFlags);
	return RetValue;
}

FString FRigVMPropertyBag::GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags) const
{
	if (!IsValidIndex(InPropertyIndex))
	{
		return FString();
	}
	return GetDataAsString(InPropertyIndex, PortFlags);
}

bool FRigVMPropertyBag::SetDataFromString(int32 InPropertyIndex, const FString& InValue)
{
	check(IsValidIndex(InPropertyIndex));
	uint8* Data = GetData<uint8>(InPropertyIndex);

	const FProperty* Property = GetProperty(InPropertyIndex);

	FRigVMMemoryStorageImportErrorContext ErrorPipe(false);
	Property->ImportText_Direct(*InValue, Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);

	if (ErrorPipe.NumErrors > 0)
	{
		// check if the value was provided as a single element
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			const FString ValueWithBraces = FString::Printf(UE::RigVM::RigVMCore::Private::BraceFormat, *InValue);

			ErrorPipe = FRigVMMemoryStorageImportErrorContext(false);
			Property->ImportText_Direct(*ValueWithBraces, Data, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
		}
	}

	return ErrorPipe.NumErrors == 0;
}

//****************************************************************************

bool FRigVMPropertyBag::CopyProperty(
	const FProperty* InTargetProperty,
	uint8* InTargetPtr,
	const FProperty* InSourceProperty,
	const uint8* InSourcePtr)
{
	check(InTargetProperty != nullptr);
	check(InSourceProperty != nullptr);
	check(InTargetPtr != nullptr);
	check(InSourcePtr != nullptr);

	// This block below is there to support Large World Coordinates (LWC).
	// We allow to link float and double pins (single and arrays) so we need
	// to support copying values between those as well.
	if (!InTargetProperty->SameType(InSourceProperty))
	{
		if (const FFloatProperty* TargetFloatProperty = CastField<FFloatProperty>(InTargetProperty))
		{
			if (const FDoubleProperty* SourceDoubleProperty = CastField<FDoubleProperty>(InSourceProperty))
			{
				if (TargetFloatProperty->ArrayDim == SourceDoubleProperty->ArrayDim)
				{
					float* TargetFloats = (float*)InTargetPtr;
					double* SourceDoubles = (double*)InSourcePtr;
					for (int32 Index = 0; Index < TargetFloatProperty->ArrayDim; Index++)
					{
						TargetFloats[Index] = (float)SourceDoubles[Index];
					}
					return true;
				}
			}
		}
		else if (const FDoubleProperty* TargetDoubleProperty = CastField<FDoubleProperty>(InTargetProperty))
		{
			if (const FFloatProperty* SourceFloatProperty = CastField<FFloatProperty>(InSourceProperty))
			{
				if (TargetDoubleProperty->ArrayDim == SourceFloatProperty->ArrayDim)
				{
					double* TargetDoubles = (double*)InTargetPtr;
					float* SourceFloats = (float*)InSourcePtr;
					for (int32 Index = 0; Index < TargetDoubleProperty->ArrayDim; Index++)
					{
						TargetDoubles[Index] = (double)SourceFloats[Index];
					}
					return true;
				}
			}
		}
		else if (const FByteProperty* TargetByteProperty = CastField<FByteProperty>(InTargetProperty))
		{
			if (const FEnumProperty* SourceEnumProperty = CastField<FEnumProperty>(InSourceProperty))
			{
				if (TargetByteProperty->Enum == SourceEnumProperty->GetEnum())
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if (const FEnumProperty* TargetEnumProperty = CastField<FEnumProperty>(InTargetProperty))
		{
			if (const FByteProperty* SourceByteProperty = CastField<FByteProperty>(InSourceProperty))
			{
				if (TargetEnumProperty->GetEnum() == SourceByteProperty->Enum)
				{
					InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
					return true;
				}
			}
		}
		else if (const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(InTargetProperty))
		{
			if (const FArrayProperty* SourceArrayProperty = CastField<FArrayProperty>(InSourceProperty))
			{
				FScriptArrayHelper TargetArray(TargetArrayProperty, InTargetPtr);
				FScriptArrayHelper SourceArray(SourceArrayProperty, InSourcePtr);

				if (TargetArrayProperty->Inner->IsA<FFloatProperty>())
				{
					if (SourceArrayProperty->Inner->IsA<FDoubleProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for (int32 Index = 0; Index < TargetArray.Num(); Index++)
						{
							float* TargetFloat = (float*)TargetArray.GetRawPtr(Index);
							const double* SourceDouble = (const double*)SourceArray.GetRawPtr(Index);
							*TargetFloat = (float)*SourceDouble;
						}
						return true;
					}
				}
				else if (TargetArrayProperty->Inner->IsA<FDoubleProperty>())
				{
					if (SourceArrayProperty->Inner->IsA<FFloatProperty>())
					{
						TargetArray.Resize(SourceArray.Num());
						for (int32 Index = 0; Index < TargetArray.Num(); Index++)
						{
							double* TargetDouble = (double*)TargetArray.GetRawPtr(Index);
							const float* SourceFloat = (const float*)SourceArray.GetRawPtr(Index);
							*TargetDouble = (double)*SourceFloat;
						}
						return true;
					}
				}
				else if (FByteProperty* TargetArrayInnerByteProperty = CastField<FByteProperty>(TargetArrayProperty->Inner))
				{
					if (FEnumProperty* SourceArrayInnerEnumProperty = CastField<FEnumProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerByteProperty->Enum == SourceArrayInnerEnumProperty->GetEnum())
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
				else if (FEnumProperty* TargetArrayInnerEnumProperty = CastField<FEnumProperty>(TargetArrayProperty->Inner))
				{
					if (FByteProperty* SourceArrayInnerByteProperty = CastField<FByteProperty>(SourceArrayProperty->Inner))
					{
						if (TargetArrayInnerEnumProperty->GetEnum() == SourceArrayInnerByteProperty->Enum)
						{
							InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
							return true;
						}
					}
				}
			}
		}

		// if we reach this we failed since we are trying to copy
		// between two properties which are not compatible
		if (!InTargetProperty->SameType(InSourceProperty))
		{
			// Only log the issue once, rather than spam.
			static TSet<FString> ReportedErrors;

			const FString TargetType = RigVMTypeUtils::GetCPPTypeFromProperty(InTargetProperty);
			const FString SourceType = RigVMTypeUtils::GetCPPTypeFromProperty(InSourceProperty);

			UPackage* Package = InTargetProperty->GetOutermost();

			FString Message = FString::Printf(TEXT("Failed to copy %s (%s) to %s (%s) in package %s"),
				*InSourceProperty->GetName(),
				*SourceType,
				*InTargetProperty->GetName(),
				*TargetType,
				Package ? *Package->GetName() : TEXT("<Unknown Package>"));

			if (!ReportedErrors.Contains(Message))
			{
				UE_LOGF(LogRigVM, Error, "%ls", *Message);
				ReportedErrors.Add(Message);
			}

			return false;
		}
	}

	// rely on the core to copy the property contents
	InTargetProperty->CopyCompleteValue(InTargetPtr, InSourcePtr);
	return true;
}




//****************************************************************************

void FRigVMPropertyBag::Refresh()
{
	RefreshLinkedProperties();
}

void FRigVMPropertyBag::RefreshLinkedProperties()
{
	LinkedProperties.Reset();

	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		const FProperty* Property = CastField<FProperty>(CurrentBagStruct->ChildProperties);
		while (Property)
		{
			LinkedProperties.Add(Property);
			Property = CastField<FProperty>(Property->Next);
		}
	}
}

void FRigVMPropertyBag::SetDefaultValues(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions)
{
	// and store default values.
	const TArray<const FProperty*>& Properties = GetProperties();
	
	const int32 NumProperties = Properties.Num();
	if (!ensure(NumProperties == InPropertyDescriptions.Num()))
	{
		UE_LOGF(LogRigVM, Error, "Number of properties in storage is different from descriptors, unsupported type or duplicated name in the generated properties");
		return;
	}

	SetDefaultValuesOnTailProperties(InPropertyDescriptions);
}

void FRigVMPropertyBag::SetDefaultValuesOnTailProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions)
{
	// and store default values.
	const TArray<const FProperty*>& Properties = GetProperties();
	
	const int32 NumProperties = Properties.Num();
	const int32 StartIndex = NumProperties - InPropertyDescriptions.Num();

	for (int32 PropertyIndex = StartIndex; PropertyIndex < NumProperties; PropertyIndex++)
	{
		const FRigVMPropertyDescription& RigVMDesc = InPropertyDescriptions[PropertyIndex-StartIndex];
		const FString& DefaultValue = RigVMDesc.DefaultValue;

		if (DefaultValue.IsEmpty() || DefaultValue == UE::RigVM::RigVMCore::Private::EmptyBraces)
		{
			continue;
		}

		SetDataFromString(PropertyIndex, DefaultValue);
	}
}
//****************************************************************************

/*static*/ FPropertyBagPropertyDesc FRigVMPropertyBag::GeneratePropertyBagDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, bool bComputeID)
{
	FPropertyBagPropertyDesc Result;

	EPropertyBagPropertyType PropertyBagType = EPropertyBagPropertyType::None;
	FPropertyBagContainerTypes PropertyBagContainerTypes;

	if (GetPropertyTypeDataFromVMDescriptor(RigVMDescriptor, PropertyBagType, PropertyBagContainerTypes))
	{
		const FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(RigVMDescriptor.Name);
		if (PropertyBagContainerTypes.Num() > 0)
		{
			Result = FPropertyBagPropertyDesc(SanitizedName, PropertyBagContainerTypes, PropertyBagType, RigVMDescriptor.CPPTypeObject);
		}
		else
		{
			Result = FPropertyBagPropertyDesc(SanitizedName, PropertyBagType, RigVMDescriptor.CPPTypeObject);
		}

		Result.PropertyFlags = RigVMDescriptor.PropertyFlags;
#if WITH_EDITOR
		for (const TTuple<FName, FString>& MetaData : RigVMDescriptor.MetaData)
		{
			Result.SetMetaData(MetaData.Key, MetaData.Value);
		}
		
		Result.SetMetaData(FRigVMPropertyDescription::MD_DisplayName, RigVMDescriptor.Name.ToString());
#endif

		if (bComputeID)
		{
			const FString SanitizedNameStr = SanitizedName.ToString();
			FBlake3 Builder;
			Builder.Update(*SanitizedNameStr, SanitizedNameStr.Len() * sizeof(TCHAR));
			Builder.Update(*RigVMDescriptor.CPPType, RigVMDescriptor.CPPType.Len()  * sizeof(TCHAR));
			Builder.Update(&RigVMDescriptor.PropertyFlags, sizeof(RigVMDescriptor.PropertyFlags));
#if WITH_EDITORONLY_DATA
			for (FPropertyBagPropertyDescMetaData& MetaData : Result.MetaData)
			{
				const FString Key = MetaData.Key.ToString();
				const FString Value = MetaData.Value;
				Builder.Update(*Key, Key.Len() * sizeof(TCHAR));
				Builder.Update(*Value, Value.Len() * sizeof(TCHAR));
			}
#endif
		
			FBlake3Hash Hash = Builder.Finalize();
			Result.ID = FGuid::NewGuidFromHash(Hash);
		}
		else
		{
			Result.ID = RigVMDescriptor.Guid;
		}
	}

	return Result;
}

/*static*/ bool FRigVMPropertyBag::GetPropertyTypeDataFromVMDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, EPropertyBagPropertyType & OutBagPropertyType, FPropertyBagContainerTypes& OutBagContainerTypes)
{
	EPropertyBagPropertyType BagPropertyType = EPropertyBagPropertyType::None;

	FString VMTypeString = *RigVMDescriptor.CPPType;

	OutBagContainerTypes.Reset(); // in case it is reused by the caller

	if (RigVMTypeUtils::IsArrayType(VMTypeString))
	{
		const int32 NumContainers = RigVMDescriptor.Containers.Num();
		for (int i= 0; i < NumContainers; ++i)
		{
			switch (RigVMDescriptor.Containers[i])
			{
			case EPinContainerType::Array:
				OutBagContainerTypes.Add(EPropertyBagContainerType::Array);
				break;
			case EPinContainerType::Set:
				ensureMsgf(false, TEXT("Unsuported Set type container : %s"), *VMTypeString);
				break;
			case EPinContainerType::Map:
				ensureMsgf(false, TEXT("Unsuported Map type container : %s"), *VMTypeString);
				break;
			default:
				break;
			}
		}

		VMTypeString = RigVMDescriptor.GetTailCPPType();
	}

	FName VMType = *VMTypeString;

	if (VMType == RigVMTypeUtils::BoolTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Bool;
	}
	else if (VMType == RigVMTypeUtils::UInt8TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Byte;
	}
	else if (VMType == RigVMTypeUtils::Int32TypeName || VMType == RigVMTypeUtils::IntTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Int32;
	}
	else if (VMType == RigVMTypeUtils::UInt32TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::UInt32;
	}
	else if (VMType == RigVMTypeUtils::Int64TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Int64;
	}
	else if (VMType == RigVMTypeUtils::UInt64TypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::UInt64;
	}
	else if (VMType == RigVMTypeUtils::FloatTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Float;
	}
	else if (VMType == RigVMTypeUtils::DoubleTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Double;
	}
	else if (VMType == RigVMTypeUtils::FNameTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Name;
	}
	else if (VMType == RigVMTypeUtils::FStringTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::String;
	}
	else if (VMType == RigVMTypeUtils::FTextTypeName)
	{
		OutBagPropertyType = EPropertyBagPropertyType::Text;
	}
	else if (Cast<UScriptStruct>(RigVMDescriptor.CPPTypeObject))
	{
		OutBagPropertyType = EPropertyBagPropertyType::Struct;
	}
	else if (Cast<UEnum>(RigVMDescriptor.CPPTypeObject))
	{
		OutBagPropertyType = EPropertyBagPropertyType::Enum;
	}
	else if (Cast<UObject>(RigVMDescriptor.CPPTypeObject))
	{
		const bool bIsClass = RigVMTypeUtils::IsUClassType(RigVMDescriptor.CPPType);
		if(bIsClass)
		{
			OutBagPropertyType = EPropertyBagPropertyType::Class;
		}
		else
		{
			OutBagPropertyType = EPropertyBagPropertyType::Object;
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported type : %s"), *VMTypeString);
		OutBagPropertyType = EPropertyBagPropertyType::None;
	}

	return OutBagPropertyType != EPropertyBagPropertyType::None;
}
