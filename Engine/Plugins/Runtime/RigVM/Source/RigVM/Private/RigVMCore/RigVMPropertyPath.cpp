// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMPropertyPath.h"
#include "RigVMModule.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FRigVMPropertyPath FRigVMPropertyPath::Empty;

FRigVMPropertyPath::FRigVMPropertyPath()
: Path()
, Segments()
{
}

FRigVMPropertyPath::FRigVMPropertyPath(const FProperty* InProperty, const FString& InSegmentPath)
: Path()
, Segments()
{
	check(InProperty);
	check(!InSegmentPath.IsEmpty())

	// Reformat the path. We'll remove any brackets and clean up
	// double periods. Turns '[2].Translation.X' to '2.Translation.X'
	FString WorkPath = InSegmentPath;
	WorkPath = WorkPath.Replace(TEXT("["), TEXT("."));
	WorkPath = WorkPath.Replace(TEXT("]"), TEXT("."));
	WorkPath = WorkPath.Replace(TEXT(".."), TEXT("."));
	WorkPath.TrimCharInline('.', nullptr);

	// Traverse the provided segment path and build up the segments for
	// the property path/
	const FProperty* Property = InProperty;
	while(!WorkPath.IsEmpty())
	{
		// split off the head
		FString PathSegment, PathRemainder;
		if(!WorkPath.Split(TEXT("."), &PathSegment, &PathRemainder))
		{
			PathSegment = WorkPath;
			PathRemainder.Empty();
		}

		FRigVMPropertyPathSegment Segment;

		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			Segment.Type = ERigVMPropertyPathSegmentType::StructMember;

			if(const FProperty* MemberProperty = StructProperty->Struct->FindPropertyByName(*PathSegment))
			{
				Segment.Name = MemberProperty->GetFName();
				Segment.Index = MemberProperty->GetOffset_ForInternal();
				Property = Segment.Property = MemberProperty;
			}
			else
			{
				Segments.Empty();
#if UE_BUILD_DEBUG
				UE_LOGF(LogRigVM, Warning, "Unable to resolve property path '%ls' for property '%ls'.", *InSegmentPath, *InProperty->GetPathName())
#endif
				return;
			}
		}
		else if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			Segment.Type = ERigVMPropertyPathSegmentType::ArrayElement;
			const FProperty* ElementProperty = ArrayProperty->Inner;
			Segment.Name = ElementProperty->GetFName();
			Segment.Index = FCString::Atoi(*PathSegment);
			Segment.Property = ArrayProperty;
			Property = ElementProperty;
		}
		else if(const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			Segment.Type = ERigVMPropertyPathSegmentType::MapValue;
			const FProperty* ValueProperty = MapProperty->ValueProp;
			Segment.Name = *PathSegment;
			Segment.Index = INDEX_NONE;
			Segment.Property = MapProperty;
			Property = ValueProperty;
		}
		else if(CastField<FSetProperty>(Property))
		{
			Segments.Empty();
#if UE_BUILD_DEBUG
			UE_LOGF(LogRigVM, Warning, "Unable to resolve property path '%ls' for property '%ls'. FSetProperty is not supported.", *InSegmentPath, *InProperty->GetPathName())
#endif
			return;
		}
		else
		{
			Segments.Empty();
			return;
		}

		Segments.Add(Segment);
		WorkPath = PathRemainder;
	}

	// rebuild the path again from the retrieved segments 
	Path.Empty();
	for(const FRigVMPropertyPathSegment& Segment : Segments)
	{
		switch(Segment.Type)
		{
			case ERigVMPropertyPathSegmentType::StructMember:
			{
				if(!Path.IsEmpty())
				{
					Path += TEXT(".");
				}
				Path += Segment.Name.ToString();
				break;
			}
			case ERigVMPropertyPathSegmentType::ArrayElement:
			{
				Path += TEXT("[") + FString::FromInt(Segment.Index) + TEXT("]");
				break;
			}
			case ERigVMPropertyPathSegmentType::MapValue:
			{
				Path += TEXT("[") + Segment.Name.ToString() + TEXT("]");
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

FString FRigVMPropertyPath::GetName() const
{
	TArray<FString> Parts;
	Parts.Reset(Segments.Num() + 2);
	Parts.Add(TEXT("PropertyPath"));
	Parts.Add(GetHeadCPPType().Replace(TEXT("<"), TEXT("_")).Replace(TEXT(">"), TEXT("_")));
	for(const FRigVMPropertyPathSegment& Segment : Segments)
	{
		if (Segment.Type == ERigVMPropertyPathSegmentType::ArrayElement)
		{
			Parts.Add(FString::FromInt(Segment.Index));
		}
		else
		{
			Parts.Add(Segment.Name.ToString());
		}
	}
	return RigVMStringUtils::JoinStrings(Parts, TEXT("_"));
}

FString FRigVMPropertyPath::GetHeadCPPType() const
{
	if(Segments.IsEmpty())
	{
		return FString();
	}
	
	const FRigVMPropertyPathSegment& FirstSegment = Segments[0];
	switch(FirstSegment.Type)
	{
		case ERigVMPropertyPathSegmentType::ArrayElement:
		case ERigVMPropertyPathSegmentType::MapValue:
		{
			// for array elements map values the segment refers to the array itself
			FString ExtendedType;
			FString BaseType = FirstSegment.Property->GetCPPType(&ExtendedType);  
			return BaseType + ExtendedType;
		}
		case ERigVMPropertyPathSegmentType::StructMember:
		{
			UScriptStruct* OwnerStruct = Cast<UScriptStruct>(FirstSegment.Property->GetOwnerStruct());
			check(OwnerStruct);
			return OwnerStruct->GetStructCPPName();
		}
		default:
		{
			checkNoEntry();
			break;
		}
	}
	return FString();
}

const FProperty* FRigVMPropertyPath::GetTailProperty() const
{
	if(Segments.IsEmpty())
	{
		return nullptr;
	}
	
	const FRigVMPropertyPathSegment& LastSegment = Segments[Num() - 1];
	switch(LastSegment.Type)
	{
		case ERigVMPropertyPathSegmentType::ArrayElement:
		{
			const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(LastSegment.Property);
			return ArrayProperty->Inner;
		}
		case ERigVMPropertyPathSegmentType::MapValue:
		{
			const FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(LastSegment.Property);
			return MapProperty->ValueProp;
		}
		case ERigVMPropertyPathSegmentType::StructMember:
		{
			return LastSegment.Property;
		}
		default:
		{
			checkNoEntry();
			break;
		}
	}

	return nullptr;
}

uint8* FRigVMPropertyPath::GetData_Internal(uint8* InPtr, const FProperty* InProperty) const
{
	for(const FRigVMPropertyPathSegment& Segment : Segments)
	{
		switch(Segment.Type)
		{
			case ERigVMPropertyPathSegmentType::StructMember:
			{
#if UE_BUILD_DEBUG
				// make sure to only allow jumps within the same data structure.
				// we check this to avoid heap corruption.
				const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InProperty);
				// check(StructProperty->Struct == Segment.Property->GetOwnerStruct());
				InProperty = Segment.Property;
#endif

				InPtr += Segment.Property->GetOffset_ForInternal();
				break;
			}
			case ERigVMPropertyPathSegmentType::ArrayElement:
			{
#if UE_BUILD_DEBUG
				// we check this to avoid heap corruption.
				check(InProperty->SameType(Segment.Property));
				InProperty = CastFieldChecked<FArrayProperty>(Segment.Property)->Inner;
#endif
				FScriptArrayHelper ArrayHelper(CastFieldChecked<FArrayProperty>(Segment.Property), InPtr);
				if(!ArrayHelper.IsValidIndex(Segment.Index))
				{
					return nullptr;
				}
				InPtr = ArrayHelper.GetRawPtr(Segment.Index);
				break;
			}
			case ERigVMPropertyPathSegmentType::MapValue:
			{
#if UE_BUILD_DEBUG
				// we check this to avoid heap corruption.
				check(InProperty->SameType(Segment.Property));
				InProperty = CastFieldChecked<FMapProperty>(Segment.Property)->ValueProp;
#endif
				FScriptMapHelper MapHelper(CastFieldChecked<FMapProperty>(Segment.Property), InPtr);
				InPtr = MapHelper.FindValueFromHash(&Segment.Name);;
				break;
			}
			default:
			{
				break;
			}
		}
	}

	return InPtr;
}

