// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/StaticBitArray.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/ScopeLock.h"
#include "RigVMCore/RigVMStruct.h"
#include <RigVMCore/RigVMTrait.h>
#include "RigVMTypeUtils.h"
#include "RigVMStringUtils.h"
#include "RigVMModule.h"
#include "Async/ParallelFor.h"
#include "Animation/AttributeTypes.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectIterator.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DelayedAutoRegister.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"
#include "Interfaces/IPluginManager.h"
#include <atomic>

// When the object system has been completely loaded, load in all the engine types that we haven't registered already in InitializeIfNeeded 
static FDelayedAutoRegisterHelper GRigVMRegistrySingletonHelper(EDelayedRegisterRunPhase::EndOfEngineInit, &FRigVMRegistry_NoLock::OnEngineInit);

static TAutoConsoleVariable<bool> CVarRigVMUpdateDispatchFactoriesGreedily(TEXT("RigVM.UpdateDispatchFactoriesGreedily"), true, TEXT("Set this to false to avoid loading dispatch factories during engine init / plugin mount."));
TAutoConsoleVariable<bool> CVarRigVMEnableLocalizedRegistry(TEXT("RigVM.EnableLocalizedRegistry"), true, TEXT("Set this to true to allow VMs to serialize their own localized registries."));

#if UE_WITH_CONSTINIT_UOBJECT
static FRegisterRigVMStructs* GRigVMStructsToRegister = nullptr;
FRegisterRigVMStructs::FRegisterRigVMStructs(TConstArrayView<FRigVMCompiledInStruct> InStructs)
	: Structs(InStructs)
{
	Next = GRigVMStructsToRegister;
	GRigVMStructsToRegister = this;
}

static void RegisterRigVMStructs()
{
	FRegisterRigVMStructs* Head = GRigVMStructsToRegister;
	GRigVMStructsToRegister = nullptr;
	FRigVMRegistry& Registry = FRigVMRegistry::Get();
	for (; Head != nullptr; Head = Head->Next)
	{
		for (const FRigVMCompiledInStruct& Struct : Head->Structs)
		{
			Registry.RegisterCompiledInStruct(Struct.Struct, Struct.Functions);
		}
	}
}

static FDelayedAutoRegisterHelper GRigVMRegistry_PostUObjectInit(EDelayedRegisterRunPhase::ObjectSystemReady, []()
{
#if IS_MONOLITHIC
	// Doing this in modular builds means we may try and register the structes declared in this module before their UObjects have been initialized
	// because we don't know the order the dynamic initializers for this helper, the FRegisterRigVMStructs and the UObject registration will happen in.
	// We may also be able to remove this call in monolithic builds if we can guarantee there will be another call to ProcessNewlyLoadedUObjects after 
	// EDelayedRegisterRunPhase::ObjectSystemReady
	RegisterRigVMStructs();
#endif
	// Register newly added structs as new modules create their UObjects
	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.AddLambda([](FName ModuleName, ECompiledInUObjectsRegisteredStatus Status)
	{
		if (Status != ECompiledInUObjectsRegisteredStatus::Delayed)
		{
			RegisterRigVMStructs();
		}
	});
});
#endif // UE_WITH_CONSTINIT_UOBJECT

FRigVMRegistry_NoLock::FRigVMRegistry_NoLock(bool InIsGlobalRegistry)
	: FGCObject(EFlags::RegisterLater)
	, ThisHandle(this)
	, bIsGlobalRegistry(InIsGlobalRegistry)
	, bAvoidTypePropagation(false)
	, bEverRefreshedEngineTypes(false)
	, bEverRefreshedDispatchFactoriesAfterEngineInit(false)
{
	// Needs to be called in the most derived constructor to avoid still
	// having pure virtual calls in the virtual table at registration time.
	RegisterGCObject();
}

TSharedRef<FRigVMRegistry_NoLock> FRigVMRegistry_NoLock::CreateLocalizedRegistry()
{
	TSharedRef<FRigVMRegistry_NoLock> LocalizedRegistry = MakeShareable(new FRigVMRegistry_NoLock(false));
	LocalizedRegistry->InitializeBaseTypes_NoLock();
	return LocalizedRegistry;
}

TSharedRef<FRigVMRegistry_NoLock> FRigVMRegistry_NoLock::CloneLocalizedRegistry(FRigVMRegistry_NoLock* InRegistry)
{
	TSharedRef<FRigVMRegistry_NoLock> ClonedRegistry = MakeShareable(new FRigVMRegistry_NoLock(false));
	ClonedRegistry->Types = InRegistry->Types;
	ClonedRegistry->TypeToIndex = InRegistry->TypeToIndex;

	ClonedRegistry->Factories.Reserve(InRegistry->Factories.Num());
	ClonedRegistry->FactoryNameToFactory.Reserve(InRegistry->Factories.Num());
	ClonedRegistry->FactoryStructToFactory.Reserve(InRegistry->Factories.Num());
	
	for (const FRigVMDispatchFactory* Factory : InRegistry->Factories)
	{
		const FRigVMTemplate* Template = Factory->GetTemplate_NoLock(InRegistry->GetHandle_NoLock());
		const TArray<FRigVMTemplateArgumentInfo> FlattenedArguments = Template->GetFlattenedArgumentInfos_NoLock(InRegistry->GetHandle_NoLock());

		const FRigVMDispatchFactory* ClonedFactory = ClonedRegistry->RegisterFactory_NoLock(Factory->GetScriptStruct(), FlattenedArguments);
		const FRigVMTemplate* ClonedTemplate = ClonedFactory->GetTemplate_NoLock(ClonedRegistry->GetHandle_NoLock());
		const_cast<FRigVMTemplate*>(ClonedTemplate)->Permutations = Template->Permutations;
	}

	for (FRigVMFunction& Function : InRegistry->Functions)
	{
		if(Function.Struct)
		{
			ClonedRegistry->Register_NoLock(*Function.Name, Function.FunctionPtr, Function.Struct, Function.Arguments);
		}
		else if(Function.Factory)
		{
			// finding function will pull on the factory to create it
			(void)ClonedRegistry->FindFunction_NoLock(*Function.Name);
		}
	}

	ClonedRegistry->AllowedClasses = InRegistry->AllowedClasses;
	ClonedRegistry->AllowedStructs = InRegistry->AllowedStructs;

	return ClonedRegistry;
}

FRigVMRegistry_NoLock::~FRigVMRegistry_NoLock()
{
	// Needs to be called in the most derived destructor to avoid leaving
	// a registered object with half its virtual table destroyed.
	UnregisterGCObject();

	FRigVMRegistry_NoLock::Reset_NoLock();
}

void FRigVMRegistry_NoLock::AddReferencedObjects(FReferenceCollector& Collector)
{
	auto AddReferenceLambda = [&Collector](TObjectPtr<UObject>& Object)
	{
		// the Object needs to be checked for validity since it may be a user defined type (struct or enum)
		// which is about to get removed. 
		if (Object)
		{
#if !UE_BUILD_SHIPPING
			// in non shipping builds, immediately run IsValidLowLevelFast such that
			// we can catch invalid types earlier via a direct crash more often
			if (Object->IsValidLowLevelFast())
			{
				// By design, hold strong references only to non-native types
				if (!Object->IsNative())
				{
					Collector.AddReferencedObject(Object);
				}
			}
#else
			// in shipping builds, try to be as safe as possible
			if(IsValid(Object))
			{
				if(Object->GetClass())
				{
					if(Object->IsValidLowLevelFast() &&
						!Object->IsNative() &&
						!Object->IsUnreachable())
					{
						// make sure the object is part of the GUObjectArray and can be retrieved
						// so that GC doesn't crash after receiving the referenced object
						const int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
						if(ObjectIndex != INDEX_NONE)
						{
							if(const FUObjectItem* Item = GUObjectArray.IndexToObject(ObjectIndex))
							{
								if(Item->GetObject() == Object)
								{
									Collector.AddReferencedObject(Object);
								}
							}
						}
					}
				}
			}
#endif
		}
	};
	
	// registry should hold strong references to these type objects
	// otherwise GC may remove them without the registry known it
	// which can happen during cook time.
	for (FTypeInfo& Type : Types)
	{
		AddReferenceLambda(Type.Type.CPPTypeObject);
	}

	for (const FRigVMFunction& Function : Functions)
	{
		if (Function.IsValid())
		{
			TObjectPtr<UObject> StructPtr(Function.Struct);
			AddReferenceLambda(StructPtr);
		}
	}

	for (const FRigVMDispatchFactory* Factory : Factories)
	{
		if (Factory)
		{
			TObjectPtr<UObject> FactoryPtr(Factory->GetScriptStruct());
			AddReferenceLambda(FactoryPtr);
		}
	}
}

FString FRigVMRegistry_NoLock::GetReferencerName() const
{
	return TEXT("FRigVMRegistry");
}

const TArray<UScriptStruct*>& FRigVMRegistry_NoLock::GetMathTypes()
{
	// The list of base math types to automatically register 
	static const TArray<UScriptStruct*> MathTypes = { 
		TBaseStructure<FRotator>::Get(),
		TBaseStructure<FQuat>::Get(),
		TBaseStructure<FTransform>::Get(),
		TBaseStructure<FLinearColor>::Get(),
		TBaseStructure<FColor>::Get(),
		TBaseStructure<FPlane>::Get(),
		TBaseStructure<FVector>::Get(),
		TBaseStructure<FVector2D>::Get(),
		TBaseStructure<FVector4>::Get(),
		TBaseStructure<FBox2D>::Get()
	};

	return MathTypes;
}

uint32 FRigVMRegistry_NoLock::GetHashForType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(!Types.IsValidIndex(InTypeIndex))
	{
		return UINT32_MAX;
	}

	const FTypeInfo& TypeInfo = Types[InTypeIndex];
	
	if(TypeInfo.Hash != UINT32_MAX)
	{
		return TypeInfo.Hash;
	}

	uint32 Hash;
	if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(TypeInfo.Type.CPPTypeObject))
	{
		Hash = GetHashForScriptStruct_NoLock(ScriptStruct, false);
	}
	else if(const UStruct* Struct = Cast<UStruct>(TypeInfo.Type.CPPTypeObject))
	{
		Hash = GetHashForStruct_NoLock(Struct);
	}
	else if(const UEnum* Enum = Cast<UEnum>(TypeInfo.Type.CPPTypeObject))
    {
    	Hash = GetHashForEnum_NoLock(Enum, false);
    }
    else
    {
    	Hash = GetTypeHash(TypeInfo.Type.CPPType.ToString());
    }

	// for used defined structs - always recompute it
	if(Cast<UUserDefinedStruct>(TypeInfo.Type.CPPTypeObject))
	{
		return Hash;
	}

	TypeInfo.Hash = Hash;
	return Hash;
}

uint32 FRigVMRegistry_NoLock::GetHashForScriptStruct_NoLock(const UScriptStruct* InScriptStruct, bool bCheckTypeIndex) const
{
	if(bCheckTypeIndex)
	{
		const TRigVMTypeIndex TypeIndex = GetTypeIndex_NoLock(*InScriptStruct->GetStructCPPName(), (UObject*)InScriptStruct);
		if(TypeIndex != INDEX_NONE)
		{
			return GetHashForType_NoLock(TypeIndex);
		}
	}
	
	const uint32 NameHash = GetTypeHash(InScriptStruct->GetStructCPPName());
	return HashCombine(NameHash, GetHashForStruct_NoLock(InScriptStruct));
}

uint32 FRigVMRegistry_NoLock::GetHashForStruct_NoLock(const UStruct* InStruct) const
{
	uint32 Hash = GetTypeHash(InStruct->GetPathName());
	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		const FProperty* Property = *It;
		if(IsAllowedType_NoLock(Property))
		{
			Hash = HashCombine(Hash, GetHashForProperty_NoLock(Property));
		}
	}
	return Hash;
}

uint32 FRigVMRegistry_NoLock::GetHashForEnum_NoLock(const UEnum* InEnum, bool bCheckTypeIndex) const
{
	if(bCheckTypeIndex)
	{
		const TRigVMTypeIndex TypeIndex = GetTypeIndex_NoLock(*InEnum->CppType, (UObject*)InEnum);
		if(TypeIndex != INDEX_NONE)
		{
			return GetHashForType_NoLock(TypeIndex);
		}
	}
	
	uint32 Hash = GetTypeHash(InEnum->GetName());
	for(int32 Index = 0; Index < InEnum->NumEnums(); Index++)
	{
		Hash = HashCombine(Hash, GetTypeHash(InEnum->GetValueByIndex(Index)));
		Hash = HashCombine(Hash, GetTypeHash(InEnum->GetDisplayNameTextByIndex(Index).ToString()));
	}
	return Hash;
}

uint32 FRigVMRegistry_NoLock::GetHashForProperty_NoLock(const FProperty* InProperty) const
{
	uint32 Hash = GetTypeHash(InProperty->GetName());

	const FString CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(InProperty);
	Hash = HashCombine(Hash, GetTypeHash(CPPType));
	
	if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		InProperty = ArrayProperty->Inner;
	}
	
	if(const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		Hash = HashCombine(Hash, GetHashForStruct_NoLock(StructProperty->Struct));
	}
	else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		if(ByteProperty->Enum)
		{
			Hash = HashCombine(Hash, GetHashForEnum_NoLock(ByteProperty->Enum));
		}
	}
	else if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		Hash = HashCombine(Hash, GetHashForEnum_NoLock(EnumProperty->GetEnum()));
	}
	
	return Hash;
}

void FRigVMRegistry_NoLock::RebuildRegistry_NoLock()
{
	Reset_NoLock();
	
	Types.Reset();
	TypeToIndex.Reset();
	Functions.Empty();
	Templates.Empty();
	DeprecatedTemplates.Empty();
	Factories.Reset();
	FactoryNameToFactory.Reset();
	FactoryStructToFactory.Reset();
	FunctionNameToIndex.Reset();
	StructNameToPredicates.Reset();
	TemplateNotationToIndex.Reset();
	DeprecatedTemplateNotationToIndex.Reset();
	TypesPerCategory.Reset();
	TemplatesPerCategory.Reset();
	SoftObjectPathToTypeIndex.Reset();
	AllowedClasses.Reset();

	InitializeBaseTypes_NoLock();
}

void FRigVMRegistry_NoLock::FTypeInfo::Serialize_NoLock(FArchive& Ar)
{
	Type.Serialize_NoLock(Ar);
	
	if(Ar.IsLoading())
	{
		BaseTypeIndex = INDEX_NONE;
		ArrayTypeIndex = INDEX_NONE;
		bIsArray = false;
		bIsExecute = false;
		Hash = UINT32_MAX;
	}
}

const FRigVMRegistryHandle& FRigVMRegistry_NoLock::GetHandle_NoLock() const
{
	return ThisHandle;
}

FRigVMRegistryHandle& FRigVMRegistry_NoLock::GetHandle_NoLock()
{
	return ThisHandle;
}

void FRigVMRegistry_NoLock::InitializeBaseTypes_NoLock()
{
	Types.Reserve(bIsGlobalRegistry ? 512 : 0);
	TypeToIndex.Reserve(bIsGlobalRegistry ? 512 : 0);
	TypesPerCategory.Reserve(19);
	TemplatesPerCategory.Reserve(19);
	
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_Execute, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 8 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 256 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 256 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 256 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleSimpleValue, TArray<TRigVMTypeIndex>()).Reserve(8);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArraySimpleValue, TArray<TRigVMTypeIndex>()).Reserve(8);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArraySimpleValue, TArray<TRigVMTypeIndex>()).Reserve(8);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue, TArray<TRigVMTypeIndex>()).Reserve(GetMathTypes().Num());
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayMathStructValue, TArray<TRigVMTypeIndex>()).Reserve(GetMathTypes().Num());
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayMathStructValue, TArray<TRigVMTypeIndex>()).Reserve(GetMathTypes().Num());
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayScriptStructValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayScriptStructValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleEnumValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayEnumValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayEnumValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayObjectValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);
	TypesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayObjectValue, TArray<TRigVMTypeIndex>()).Reserve(bIsGlobalRegistry ? 128 : 0);

	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_Execute, TArray<int32>()).Reserve(bIsGlobalRegistry ? 8 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleSimpleValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArraySimpleValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArraySimpleValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayMathStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayMathStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayScriptStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayScriptStructValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleEnumValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayEnumValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayEnumValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayObjectValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);
	TemplatesPerCategory.Add(FRigVMTemplateArgument::ETypeCategory_ArrayArrayObjectValue, TArray<int32>()).Reserve(bIsGlobalRegistry ? 64 : 0);

	// register the simple types the same between all registries.
	// we rely on the static RigVMTypeUtils::TypeIndex lookup - so all registries have to use the same type indices for this.
	TRigVMTypeIndex LocalExecuteIndex, LocalExecuteArrayIndex, LocalBoolIndex, LocalFloatIndex, LocalDoubleIndex, LocalInt32Index, LocalUInt32Index,
		LocalUInt8Index, LocalFNameIndex, LocalFStringIndex, LocalWildCardIndex, LocalBoolArrayIndex, LocalFloatArrayIndex, LocalDoubleArrayIndex,
		LocalInt32ArrayIndex, LocalUInt32ArrayIndex, LocalUInt8ArrayIndex, LocalFNameArrayIndex, LocalFStringArrayIndex, LocalWildCardArrayIndex;
	{
		const TGuardValue<bool> EnableAllArrayDimensions(bIsGlobalRegistry, true);
		LocalExecuteIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(FRigVMExecuteContext::StaticStruct()), false);
		LocalExecuteArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(FRigVMExecuteContext::StaticStruct()).ConvertToArray(), false);
		LocalBoolIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::BoolTypeName, nullptr), false);
		LocalFloatIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FloatTypeName, nullptr), false);
		LocalDoubleIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::DoubleTypeName, nullptr), false);
		LocalInt32Index = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::Int32TypeName, nullptr), false);
		LocalUInt32Index = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::UInt32TypeName, nullptr), false);
		LocalUInt8Index = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::UInt8TypeName, nullptr), false);
		LocalFNameIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FNameTypeName, nullptr), false);
		LocalFStringIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FStringTypeName, nullptr), false);
		LocalWildCardIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::GetWildCardCPPTypeName(), RigVMTypeUtils::GetWildCardCPPTypeObject()), false);
		LocalBoolArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::BoolArrayTypeName, nullptr), false);
		LocalFloatArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FloatArrayTypeName, nullptr), false);
		LocalDoubleArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::DoubleArrayTypeName, nullptr), false);
		LocalInt32ArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::Int32ArrayTypeName, nullptr), false);
		LocalUInt32ArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::UInt32ArrayTypeName, nullptr), false);
		LocalUInt8ArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::UInt8ArrayTypeName, nullptr), false);
		LocalFNameArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FNameArrayTypeName, nullptr), false);
		LocalFStringArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::FStringArrayTypeName, nullptr), false);
		LocalWildCardArrayIndex = FindOrAddType_NoLock(FRigVMTemplateArgumentType(RigVMTypeUtils::GetWildCardArrayCPPTypeName(), RigVMTypeUtils::GetWildCardCPPTypeObject()), false);
	}

	if(bIsGlobalRegistry)
	{
		RigVMTypeUtils::TypeIndex::Execute = LocalExecuteIndex;
		RigVMTypeUtils::TypeIndex::ExecuteArray = LocalExecuteArrayIndex;
		RigVMTypeUtils::TypeIndex::Bool = LocalBoolIndex;
		RigVMTypeUtils::TypeIndex::Float = LocalFloatIndex;
		RigVMTypeUtils::TypeIndex::Double = LocalDoubleIndex;
		RigVMTypeUtils::TypeIndex::Int32 = LocalInt32Index;
		RigVMTypeUtils::TypeIndex::UInt32 = LocalUInt32Index;
		RigVMTypeUtils::TypeIndex::UInt8 = LocalUInt8Index;
		RigVMTypeUtils::TypeIndex::FName = LocalFNameIndex;
		RigVMTypeUtils::TypeIndex::FString = LocalFStringIndex;
		RigVMTypeUtils::TypeIndex::WildCard = LocalWildCardIndex;
		RigVMTypeUtils::TypeIndex::BoolArray = LocalBoolArrayIndex;
		RigVMTypeUtils::TypeIndex::FloatArray = LocalFloatArrayIndex;
		RigVMTypeUtils::TypeIndex::DoubleArray = LocalDoubleArrayIndex;
		RigVMTypeUtils::TypeIndex::Int32Array = LocalInt32ArrayIndex;
		RigVMTypeUtils::TypeIndex::UInt32Array = LocalUInt32ArrayIndex;
		RigVMTypeUtils::TypeIndex::UInt8Array = LocalUInt8ArrayIndex;
		RigVMTypeUtils::TypeIndex::FNameArray = LocalFNameArrayIndex;
		RigVMTypeUtils::TypeIndex::FStringArray = LocalFStringArrayIndex;
		RigVMTypeUtils::TypeIndex::WildCardArray = LocalWildCardArrayIndex;
		
		// register the default math types
		for(UScriptStruct* MathType : GetMathTypes())
		{
			FindOrAddType_NoLock(FRigVMTemplateArgumentType(MathType), false);
		}

		// hook the registry to prepare for engine shutdown
		FCoreDelegates::OnExit.AddLambda([&]()
		{
			Reset_NoLock();

			if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
			{
				if (AssetRegistryModule->TryGet())
				{
					AssetRegistryModule->Get().OnAssetRemoved().RemoveAll(this);
					AssetRegistryModule->Get().OnAssetRenamed().RemoveAll(this);
				}
			}

			IPluginManager::Get().OnNewPluginMounted().RemoveAll(this);
			IPluginManager::Get().OnPluginUnmounted().RemoveAll(this);
			FModuleManager::Get().OnModulesUnloaded().RemoveAll(this);

			UE::Anim::AttributeTypes::GetOnAttributeTypesChanged().RemoveAll(this);

			FCoreDelegates::CleanupUnloadingObjects.RemoveAll(this);
		});
	}
	else
	{
		check(RigVMTypeUtils::TypeIndex::Execute == LocalExecuteIndex);
		check(RigVMTypeUtils::TypeIndex::ExecuteArray == LocalExecuteArrayIndex);
		check(RigVMTypeUtils::TypeIndex::Bool == LocalBoolIndex);
		check(RigVMTypeUtils::TypeIndex::Float == LocalFloatIndex);
		check(RigVMTypeUtils::TypeIndex::Double == LocalDoubleIndex);
		check(RigVMTypeUtils::TypeIndex::Int32 == LocalInt32Index);
		check(RigVMTypeUtils::TypeIndex::UInt32 == LocalUInt32Index);
		check(RigVMTypeUtils::TypeIndex::UInt8 == LocalUInt8Index);
		check(RigVMTypeUtils::TypeIndex::FName == LocalFNameIndex);
		check(RigVMTypeUtils::TypeIndex::FString == LocalFStringIndex);
		check(RigVMTypeUtils::TypeIndex::WildCard == LocalWildCardIndex);
		check(RigVMTypeUtils::TypeIndex::BoolArray == LocalBoolArrayIndex);
		check(RigVMTypeUtils::TypeIndex::FloatArray == LocalFloatArrayIndex);
		check(RigVMTypeUtils::TypeIndex::DoubleArray == LocalDoubleArrayIndex);
		check(RigVMTypeUtils::TypeIndex::Int32Array == LocalInt32ArrayIndex);
		check(RigVMTypeUtils::TypeIndex::UInt32Array == LocalUInt32ArrayIndex);
		check(RigVMTypeUtils::TypeIndex::UInt8Array == LocalUInt8ArrayIndex);
		check(RigVMTypeUtils::TypeIndex::FNameArray == LocalFNameArrayIndex);
		check(RigVMTypeUtils::TypeIndex::FStringArray == LocalFStringArrayIndex);
		check(RigVMTypeUtils::TypeIndex::WildCardArray == LocalWildCardArrayIndex);
	}
}

void FRigVMRegistry_NoLock::Serialize_NoLock(FArchive& Ar, const UObject* Owner)
{
	check(!bIsGlobalRegistry);

	if(Ar.IsLoading())
	{
		RebuildRegistry_NoLock();
		Ar << AllowedClasses;

		// types
		{
			int32 NumTypes = 0;
			Ar << NumTypes;
			Types.Reserve(NumTypes * 3); // 3 times due to array dimensions [] and [][]
		
			for(int32 TypeIndex = 0; TypeIndex < NumTypes; TypeIndex++)
			{
				FTypeInfo TypeInfo;
				TypeInfo.Serialize_NoLock(Ar);
				(void)FindOrAddType_NoLock(TypeInfo.Type, true /* force loading of the type */);
			}
		}

		// factories
		{
			int32 NumFactories = 0;
			Ar << NumFactories;
			
			for(int32 FactoryIndex = 0; FactoryIndex < NumFactories; FactoryIndex++)
			{
				UScriptStruct* FactoryScriptStruct = nullptr;
				Ar << FactoryScriptStruct;

				int32 NumFlattenedArguments = 0;
				Ar << NumFlattenedArguments;

				TArray<FRigVMTemplateArgumentInfo> FlattenedArguments;
				FlattenedArguments.Reserve(NumFlattenedArguments);

				for(int32 ArgumentIndex = 0; ArgumentIndex < NumFlattenedArguments; ArgumentIndex++)
				{
					FName ArgumentName(NAME_None);
					Ar << ArgumentName;

					int32 Direction = 0;
					Ar << Direction;

					int32 NumTypes = 0;
					Ar << NumTypes;

					TArray<TRigVMTypeIndex> TypeIndices;
					TypeIndices.Reserve(NumTypes);
					
					for(int32 TypeIndex = 0; TypeIndex < NumTypes; TypeIndex++)
					{
						FName CPPType(NAME_None);
						Ar << CPPType;

						UObject* CPPTypeObject = nullptr;
						Ar << CPPTypeObject;
						
						TypeIndices.Add(FindOrAddType_NoLock({CPPType, CPPTypeObject}));
					}

					if(TypeIndices.Num() == 1)
					{
						FlattenedArguments.Emplace(ArgumentName, static_cast<ERigVMPinDirection>(Direction), TypeIndices[0]);
					}
					else
					{
						FlattenedArguments.Emplace(ArgumentName, static_cast<ERigVMPinDirection>(Direction), TypeIndices);
					}
				}

				if(const FRigVMDispatchFactory* Factory = RegisterFactory_NoLock(FactoryScriptStruct, FlattenedArguments))
				{
					// pulling on the caches the template
					Factory->GetTemplate_NoLock(ThisHandle);
				}
				else
				{
					UE_LOGF(LogRigVM, Error, "Failed to register dispatch factory struct '%ls' when loading '%ls'", *FactoryScriptStruct->GetName(), *Ar.GetArchiveState().GetArchiveName());
				}
			}
		}

		// functions
		{
			int32 NumFunctions = 0;
			Ar << NumFunctions;
			for(int32 FunctionIndex = 0; FunctionIndex < NumFunctions; FunctionIndex++)
			{
				FRigVMFunction Function;
				Function.Serialize_NoLock(Ar, ThisHandle, Owner);

				if(Function.Struct)
				{
					Register_NoLock(*Function.Name, Function.FunctionPtr, Function.Struct, Function.Arguments);
				}
				else if(Function.Factory)
				{
					// finding function will pull on the factory to create it
					(void)FindFunction_NoLock(*Function.Name);
				}
			}
		}
	}
	else if(Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector())
	{
		Ar << AllowedClasses;
		
		// types
		{
			int32 NumTypes = Types.Num();
			Ar << NumTypes;
			for(FTypeInfo& TypeInfo : Types)
			{
				TypeInfo.Serialize_NoLock(Ar);
			}
		}

		// factories
		{
			int32 NumFactories = Factories.Num();
			Ar << NumFactories;
			for(FRigVMDispatchFactory* Factory : Factories)
			{
				UScriptStruct* FactoryScriptStruct = Factory->GetScriptStruct();
				Ar << FactoryScriptStruct;

				// find all used permutations of the factory and store their types
				TArray<FRigVMTemplateArgumentInfo> FlattenedArguments = Factory->GetTemplate_NoLock(ThisHandle)->GetFlattenedArgumentInfos_NoLock(ThisHandle);

				int32 NumFlattenedArguments = FlattenedArguments.Num();
				Ar << NumFlattenedArguments;

				for(int32 ArgumentIndex = 0; ArgumentIndex < NumFlattenedArguments; ArgumentIndex++)
				{
					FRigVMTemplateArgumentInfo& ArgumentInfo = FlattenedArguments[ArgumentIndex];
					FRigVMTemplateArgument Argument = ArgumentInfo.GetArgument_NoLock(ThisHandle);
					check(!Argument.bUseCategories);
					
					Ar << Argument.Name;

					int32 Direction = static_cast<int32>(Argument.Direction);
					Ar << Direction;

					int32 NumTypes = Argument.TypeIndices.Num(); 
					Ar << NumTypes;

					for(int32 TypeIndex = 0; TypeIndex < NumTypes; TypeIndex++)
					{
						FRigVMTemplateArgumentType Type = GetType_NoLock(Argument.TypeIndices[TypeIndex]);
						Ar << Type.CPPType;
						Ar << Type.CPPTypeObject;
					}
				}
			}
		}

		// functions
		{
			int32 NumFunctions = Functions.Num();
			Ar << NumFunctions;
			for(FRigVMFunction& Function : Functions)
			{
				Function.Serialize_NoLock(Ar, ThisHandle, Owner);
			}
		}
	}
}

void FRigVMRegistry_NoLock::RefreshEngineTypesIfRequired_NoLock()
{
	if(bEverRefreshedEngineTypes)
	{
		return;
	}

	RefreshEngineTypes_NoLock();
}

void FRigVMRegistry_NoLock::RefreshEngineTypes_NoLock()
{
#if !WITH_EDITOR
	// if we are not in editor and VMs rely on their own localized registry
	// we don't have to initialize all of the types here.
	if(CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread() == true)
	{
		return;
	}
#endif

	if(!bIsGlobalRegistry)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMRegistry::RefreshEngineTypes);

	TGuardValue<bool> AvoidTypePropagationGuard(bAvoidTypePropagation, true);

	const int32 NumTypesBefore = Types.Num(); 
	
	// Register all user-defined types that the engine knows about. Enumerating over the entire object hierarchy is
	// slow, so we do it for structs, enums and dispatch factories in one shot.
	TArray<UScriptStruct*> DispatchFactoriesToRegister;
	DispatchFactoriesToRegister.Reserve(32);

	for (TObjectIterator<UScriptStruct> ScriptStructIt; ScriptStructIt; ++ScriptStructIt)
	{
		UScriptStruct* ScriptStruct = *ScriptStructIt;
		
		// if this is a C++ type - skip it
		if(ScriptStruct->IsA<UUserDefinedStruct>() || ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
		{
			// this check for example makes sure we don't add structs defined in verse
			if(IsAllowedType_NoLock(ScriptStruct))
			{
				FindOrAddType_NoLock(FRigVMTemplateArgumentType(ScriptStruct), false);
			}
		}
		else if (ScriptStruct != FRigVMDispatchFactory::StaticStruct() &&
				 ScriptStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()))
		{
			DispatchFactoriesToRegister.Add(ScriptStruct);
		}
		else if(AllowedStructs.Contains(ScriptStruct))
		{
			FindOrAddType_NoLock(FRigVMTemplateArgumentType(ScriptStruct));
		}
	}

	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		UEnum* Enum = *EnumIt;
		if(IsAllowedType_NoLock(Enum))
		{
			const FString CPPType = Enum->CppType.IsEmpty() ? Enum->GetName() : Enum->CppType;
			FindOrAddType_NoLock(FRigVMTemplateArgumentType(*CPPType, Enum), false);
		}
	}
	
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		if (IsAllowedType_NoLock(Class))
		{
			// Register both the class and the object type for use
			FindOrAddType_NoLock(FRigVMTemplateArgumentType(Class, RigVMTypeUtils::EClassArgType::AsClass), false);
			FindOrAddType_NoLock(FRigVMTemplateArgumentType(Class, RigVMTypeUtils::EClassArgType::AsObject), false);
		}
	}

	// Register all dispatch factories only after all other types have been registered.
	for (UScriptStruct* DispatchFactoryStruct: DispatchFactoriesToRegister)
	{
		RegisterFactory_NoLock(DispatchFactoryStruct);
	}

	const int32 NumTypesNow = Types.Num();
	if(NumTypesBefore != NumTypesNow)
	{
		// update all of the templates once
		TArray<bool> TemplateProcessed;
		TemplateProcessed.AddZeroed(Templates.Num());
		for(const TPair<FRigVMTemplateArgument::ETypeCategory, TArray<int32>>& Pair : TemplatesPerCategory)
		{
			for(const int32 TemplateIndex : Pair.Value)
			{
				if(!TemplateProcessed[TemplateIndex])
				{
					FRigVMTemplate& Template = Templates[TemplateIndex];
					(void)Template.UpdateAllArgumentTypesSlow(ThisHandle);
					TemplateProcessed[TemplateIndex] = true;
				}
			}
		}
	}

	// also refresh the functions and dispatches
	(void)RefreshFunctionsAndDispatches_NoLock();
	
	bEverRefreshedEngineTypes = true;
}

bool FRigVMRegistry_NoLock::RefreshFunctionsAndDispatches_NoLock()
{
	if(!CVarRigVMUpdateDispatchFactoriesGreedily->GetBool())
	{
		return false;
	}

	// nothing to do for functions for now - they are registered by their static initialize

	bool bRegistryChanged = false;

	// factories are also registered by RegisterFactory_NoLock, so we don't need to visit all
	// currently known UScriptStructs. By the time we get here the factories are registered.
	for(const FRigVMDispatchFactory* Factory : Factories)
	{
		// pulling on the template will cause the template to be initialized.
		// that may introduce a certain cost - which we don't want to experience during
		// the game.
		if(Factory->CachedTemplate == nullptr)
		{
			(void)Factory->GetTemplate_NoLock(ThisHandle);
			bRegistryChanged = true;
		}
	}
	return bRegistryChanged; 
}

bool FRigVMRegistry_NoLock::OnCleanupUnloadingObjects_NoLock(const TArrayView<UObject*> InObjects)
{
	bool bRemoved = false;
	for (UObject* Obj : InObjects)
	{
		const UObject* ObjectOuter = Obj ? Obj->GetOuter() : nullptr;
		if (ObjectOuter && (ObjectOuter->GetClass() == UPackage::StaticClass()))
		{
			if (RemoveType_NoLock(FSoftObjectPath(Obj)))
			{
				bRemoved = true;
			}
		}
	}

	return bRemoved;
}

void FRigVMRegistry_NoLock::OnAssetRenamed_NoLock(const FAssetData& InAssetData, const FString& InOldObjectPath)
{
	const FSoftObjectPath OldPath(InOldObjectPath);
	
	if (const TRigVMTypeIndexTriple* TypeIndicesPtr = SoftObjectPathToTypeIndex.Find(OldPath))
	{
		const TRigVMTypeIndexTriple TypeIndices = *TypeIndicesPtr;
		SoftObjectPathToTypeIndex.Remove(OldPath);
		SoftObjectPathToTypeIndex.Add(InAssetData.ToSoftObjectPath()) = TypeIndices;
	}
}

bool FRigVMRegistry_NoLock::OnAssetRemoved_NoLock(const FAssetData& InAssetData)
{
	return RemoveType_NoLock(InAssetData.ToSoftObjectPath());
}

bool FRigVMRegistry_NoLock::OnPluginLoaded_NoLock(IPlugin& InPlugin)
{
	// only update the functions / dispatches once the engine has initialized.
	if(!bEverRefreshedDispatchFactoriesAfterEngineInit)
	{
		return false;
	}
	return RefreshFunctionsAndDispatches_NoLock();
}

TArray<FString> FRigVMRegistry_NoLock::GetPluginModulePrefixes(const IPlugin& InPlugin)
{
	return RigVMTypeUtils::GetPluginModulePrefixes(InPlugin);
}

bool FRigVMRegistry_NoLock::IsWithinPlugin(const IPlugin& InPlugin, const UObject* InObject)
{
	return IsWithinPlugin(GetPluginModulePrefixes(InPlugin), InObject);
}

bool FRigVMRegistry_NoLock::IsWithinPlugin(const TArray<FString>& InModulePrefixes, const UObject* InObject)
{
	if (InObject == nullptr || !IsValid(InObject) || InModulePrefixes.IsEmpty())
	{
		return false;
	}

	FNameBuilder PackagePathStr;
	FSoftObjectPath(InObject).AppendString(PackagePathStr);
	const FStringView PackagePath = PackagePathStr.ToView();
	for (const FString& Prefix : InModulePrefixes)
	{
		if (PackagePath.StartsWith(Prefix))
		{
			return true;
		}
	}

	return false;
}

bool FRigVMRegistry_NoLock::OnPluginUnloaded_NoLock(IPlugin& InPlugin)
{
	const TArray<FString> ModulePrefixes = GetPluginModulePrefixes(InPlugin);
	return OnModulesUnloadedInternal(ModulePrefixes);
}

bool FRigVMRegistry_NoLock::OnModulesUnloaded_NoLock(TConstArrayView<FName> ModuleNames)
{
	TArray<FString> ModulePrefixes;

	for (FName ModuleName : ModuleNames)
	{
		static constexpr TCHAR Format[] = TEXT("/Script/%s.");
		ModulePrefixes.Add(FString::Printf(Format, *ModuleName.ToString()));
	}

	return OnModulesUnloadedInternal(ModulePrefixes);
}

struct alignas(PLATFORM_CACHE_LINE_SIZE) FNoFalseSharingBoolSet
{
	static constexpr int32 NumBitsPerBucket = PLATFORM_CACHE_LINE_SIZE * 8;
	static int32 CalcNumBuckets(int32 TotalNumberOfElements)
	{
		return FMath::DivideAndRoundUp(TotalNumberOfElements, NumBitsPerBucket);
	}
	static int32 CalcNumBitsInTheBucket(int32 BucketIdx, int32 TotalNumberOfElements)
	{
		return FMath::Min(NumBitsPerBucket, TotalNumberOfElements - BucketIdx * NumBitsPerBucket);
	}

	TStaticBitArray<NumBitsPerBucket> Set;
};
static_assert(sizeof(FNoFalseSharingBoolSet) == PLATFORM_CACHE_LINE_SIZE, "Make sure that the FNoFalseSharingBoolSet fits inside the cache line");

bool FRigVMRegistry_NoLock::OnModulesUnloadedInternal(const TArray<FString>& ModulePrefixes)
{
	if (ModulePrefixes.IsEmpty())
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMRegistry_NoLock::OnModulesUnloadedInternal);

	bool bRegistryChanged = false;
	FCriticalSection ParallelForDataLock;

	// remove the dispatch factories defined in the plugin
	TArray<UScriptStruct*> FactoriesToRemove;
	for (int32 Index = 0; Index < Factories.Num(); Index++)
	{
		if (!IsWithinPlugin(ModulePrefixes, Factories[Index]->GetScriptStruct()))
		{
			continue;
		}
		FactoriesToRemove.AddUnique(Factories[Index]->GetScriptStruct());
	}
	for (UScriptStruct* FactoryToRemove : FactoriesToRemove)
	{
		if (RemoveFactory_NoLock(FactoryToRemove))
		{
			bRegistryChanged = true;
		}
	}

	// remove all templates defined by solely structs within the plugin
	for (int32 Index = 0; Index < Templates.Num(); Index++)
	{
		if (Templates[Index].UsesDispatch())
		{
			continue;
		}

		bool bHasFunctionOutsideOfPlugin = false;
		for (int32 FunctionIndex = 0; FunctionIndex < Templates[Index].NumPermutations_NoLock(ThisHandle); FunctionIndex++)
		{
			const FRigVMFunction* Function = Templates[Index].GetPermutation_NoLock(FunctionIndex, ThisHandle);
			if (Function == nullptr)
			{
				continue;
			}
			
			check(Function->Struct);
			check(Function->Factory == nullptr);

			if (!IsWithinPlugin(ModulePrefixes, Function->Struct))
			{
				bHasFunctionOutsideOfPlugin = true;
			}
		}

		if (bHasFunctionOutsideOfPlugin)
		{
			continue;
		}

		if (RemoveTemplate_NoLock(Index))
		{
			bRegistryChanged = true;
		}
	}

	// remove the functions defined in the plugin - parellize the search as it can last for a significant time
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMRegistry_NoLock::OnModulesUnloadedInternal::Functions);

		TArray<FNoFalseSharingBoolSet> FunctionIndicesToRemove;
		const int32 TotalNumberOfElements = Functions.Num();
		const int32 NumBuckets = FNoFalseSharingBoolSet::CalcNumBuckets(TotalNumberOfElements);
		FunctionIndicesToRemove.SetNum(NumBuckets);
		ParallelFor(NumBuckets, [this, &ModulePrefixes, &FunctionIndicesToRemove, TotalNumberOfElements](int32 BucketIdx)
		{
			const int32 NumBitsInThisBucket = FNoFalseSharingBoolSet::CalcNumBitsInTheBucket(BucketIdx, TotalNumberOfElements);
			for (int32 Index = 0; Index < NumBitsInThisBucket; ++Index)
			{
				const int32 FuncIdx = Index + BucketIdx * FNoFalseSharingBoolSet::NumBitsPerBucket;
				FunctionIndicesToRemove[BucketIdx].Set[Index] =
					Functions[FuncIdx].IsValid()
					&& Functions[FuncIdx].Struct != nullptr
					&& IsWithinPlugin(ModulePrefixes, Functions[FuncIdx].Struct);
			}
		});

		for (int32 BucketIdx = 0; BucketIdx < NumBuckets; ++BucketIdx)
		{
			const int32 NumBitsInThisBucket = FNoFalseSharingBoolSet::CalcNumBitsInTheBucket(BucketIdx, TotalNumberOfElements);
			for (int32 Index = 0; Index < NumBitsInThisBucket; ++Index)
			{
				if (FunctionIndicesToRemove[BucketIdx].Set[Index])
				{
					const int32 FuncIdx = Index + BucketIdx * FNoFalseSharingBoolSet::NumBitsPerBucket;
					if (RemoveFunction_NoLock(FuncIdx))
					{
						bRegistryChanged = true;
					}
				}
			}
		}
	}

	// remove the types defined in the plugin - parellize the search as it can last for a very significant time
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMRegistry_NoLock::OnModulesUnloadedInternal::Types);

		TArray<FNoFalseSharingBoolSet> TypeIndicesToRemove;
		const int32 TotalNumberOfElements = Types.Num();
		const int32 NumBuckets = FNoFalseSharingBoolSet::CalcNumBuckets(TotalNumberOfElements);
		TypeIndicesToRemove.SetNum(NumBuckets);
		ParallelFor(NumBuckets, [this, &ModulePrefixes, &TypeIndicesToRemove, TotalNumberOfElements](int32 BucketIdx)
		{
			const int32 NumBitsInThisBucket = FNoFalseSharingBoolSet::CalcNumBitsInTheBucket(BucketIdx, TotalNumberOfElements);
			for (int32 Index = 0; Index < NumBitsInThisBucket; ++Index)
			{
				const int32 TypeIdx = Index + BucketIdx * FNoFalseSharingBoolSet::NumBitsPerBucket;
				TypeIndicesToRemove[BucketIdx].Set[Index] = 
					!Types[TypeIdx].IsUnknownType()
					// avoid array types, the base type removal will clean up related array types.
					// For example removing FMyStruct will also clean up TArray<FMyStruct>.
					// Note: There's always a base type registered for an array type being present.
					&& !Types[TypeIdx].bIsArray
					&& IsWithinPlugin(ModulePrefixes, Types[TypeIdx].Type.CPPTypeObject);
			}
		});

		for (int32 BucketIdx = 0; BucketIdx < NumBuckets; ++BucketIdx)
		{
			const int32 NumBitsInThisBucket = FNoFalseSharingBoolSet::CalcNumBitsInTheBucket(BucketIdx, TotalNumberOfElements);
			for (int32 Index = 0; Index < NumBitsInThisBucket; ++Index)
			{
				if (TypeIndicesToRemove[BucketIdx].Set[Index])
				{
					const int32 TypeIdx = Index + BucketIdx * FNoFalseSharingBoolSet::NumBitsPerBucket;
					if (RemoveType_NoLock(TypeIdx))
					{
						bRegistryChanged = true;
					}
				}
			}
		}
	}

	return bRegistryChanged;
}

void FRigVMRegistry_NoLock::OnAnimationAttributeTypesChanged_NoLock(const UScriptStruct* InStruct, bool bIsAdded)
{
	if (!ensure(InStruct))
	{
		return;
	}

	if (bIsAdded)
	{
		FindOrAddType_NoLock(FRigVMTemplateArgumentType(const_cast<UScriptStruct*>(InStruct)));
	}
}


void FRigVMRegistry_NoLock::Reset_NoLock()
{
	for(FRigVMDispatchFactory* Factory : Factories)
	{
		if(const UScriptStruct* ScriptStruct = Factory->GetScriptStruct())
		{
			ScriptStruct->DestroyStruct(Factory, 1);
		}
		FMemory::Free(Factory);
	}
	Factories.Reset();
	FactoryNameToFactory.Reset();
	FactoryStructToFactory.Reset();
}

TRigVMTypeIndex FRigVMRegistry_NoLock::FindOrAddType_NoLock(const FRigVMTemplateArgumentType& InType, bool bForce)
{
	// we don't use a mutex here since by the time the engine relies on worker
	// thread for execution or async loading all types will have been registered.
	
	TRigVMTypeIndex Index = GetTypeIndex_NoLock(InType);
	if(Index == INDEX_NONE)
	{
		const bool bRegisterAllArrayDimensions = bIsGlobalRegistry;
		
		static constexpr int32 MaxArrayDimension = 3;
		
		FRigVMTemplateArgumentType ElementType = InType;
		while(ElementType.IsArray())
		{
			ElementType.ConvertToBaseElement();
		}

		const UObject* CPPTypeObject = ElementType.CPPTypeObject;
		if(!bForce && (CPPTypeObject != nullptr))
		{
			if(const UClass* Class = Cast<UClass>(CPPTypeObject))
			{
				if(!IsAllowedType_NoLock(Class))
				{
					return Index;
				}	
			}
			else if(const UEnum* Enum = Cast<UEnum>(CPPTypeObject))
			{
				if(!IsAllowedType_NoLock(Enum))
				{
					return Index;
				}
			}
			else if(const UStruct* Struct = Cast<UStruct>(CPPTypeObject))
			{
				if(!IsAllowedType_NoLock(Struct))
				{					
					return Index;
				}
			}
		}

		bool bIsExecute = false;
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			bIsExecute = ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct());
		}

		// 3 is the max array dimension allowed within RigVM
		TArray<TRigVMTypeIndex, TInlineAllocator<3>> Indices;

		// we need to look up the type indices to reuse if this is a type that we've encountered before
		// from a plugin that got unloaded - and is being loaded again now.
		if (ElementType.CPPTypeObject)
		{
			const FSoftObjectPath SoftObjectPath(ElementType.CPPTypeObject);
			if (const TRigVMTypeIndexTriple* ExistingIndices = SoftObjectPathToTypeIndex.Find(SoftObjectPath))
			{
				// this can potentially happen multiple times - it's ok
				// if the types at the existing locations are valid.
				check(Types.IsValidIndex(ExistingIndices->Get<0>()));
				check(Types.IsValidIndex(ExistingIndices->Get<1>()));
				check(Types.IsValidIndex(ExistingIndices->Get<2>()));
				Indices.Add(ExistingIndices->Get<0>());
				Indices.Add(ExistingIndices->Get<1>());
				Indices.Add(ExistingIndices->Get<2>());
			}
		}
		
		TArray<bool, TInlineAllocator<3>> AddedTypeIndex;
		for (int32 ArrayDimension=0; ArrayDimension<MaxArrayDimension; ++ArrayDimension)
		{
			if (bIsExecute && ArrayDimension > 1)
			{
				break;
			}
			
			FRigVMTemplateArgumentType CurType = ElementType;
			for (int32 j=0; j<ArrayDimension; ++j)
			{
				CurType.ConvertToArray();
			}

			FTypeInfo Info;
			Info.Type = CurType;
			Info.bIsArray = ArrayDimension > 0;
			Info.bIsExecute = bIsExecute;

			// if this is a localized registry we may already have this type.
			if(!bRegisterAllArrayDimensions)
			{
				if(const TRigVMTypeIndex* ExistingIndex = TypeToIndex.Find(CurType))
				{
					Indices.Add(*ExistingIndex);
					AddedTypeIndex.Add(false);
					continue;
				}
			}

			if (Indices.IsValidIndex(ArrayDimension))
			{
				Index = Indices[ArrayDimension];
				Types[Index] = Info;
			}
			else
			{
				Index = Types.Add(Info);
			}
#if UE_RIGVM_DEBUG_TYPEINDEX
			Index.Name = Info.Type.CPPType;
#endif
			TypeToIndex.Add(CurType, Index);

			if (!Indices.IsValidIndex(ArrayDimension))
			{
				Indices.Add(Index);
			}
			AddedTypeIndex.Add(true);
		}

		if(Indices.Num() > 1)
		{
			Types[Indices[1]].BaseTypeIndex = Indices[0];
			Types[Indices[0]].ArrayTypeIndex = Indices[1];

			if (!bIsExecute && Indices.Num() > 2)
			{
				Types[Indices[2]].BaseTypeIndex = Indices[1];
				Types[Indices[1]].ArrayTypeIndex = Indices[2];
			}
		}

		// update the categories first then propagate to TemplatesPerCategory once all categories up to date
		TArray<TPair<FRigVMTemplateArgument::ETypeCategory, int32>> ToPropagate;
		auto RegisterNewType = [&](FRigVMTemplateArgument::ETypeCategory InCategory, TRigVMTypeIndex NewIndex)
		{
			RegisterTypeInCategory_NoLock(InCategory, NewIndex);
			ToPropagate.Emplace(InCategory, NewIndex);
		}; 

		for (int32 ArrayDimension=0; ArrayDimension<MaxArrayDimension; ++ArrayDimension)
		{
			if(bIsExecute && ArrayDimension > 1)
			{
				break;
			}
			if(!AddedTypeIndex[ArrayDimension])
			{
				continue;
			}
			
			Index = Indices[ArrayDimension];

			// Add to category
			// simple types
			if(CPPTypeObject == nullptr)
			{
				switch(ArrayDimension)
				{
					default:
					case 0:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleSimpleValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, Index);
						break;
					}
					case 1:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArraySimpleValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, Index);
						break;
					}
					case 2:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArraySimpleValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, Index);
						break;
					}
				}
			}
			else if(CPPTypeObject->IsA<UClass>())
			{
				switch(ArrayDimension)
				{
					default:
					case 0:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, Index);
						break;
					}
					case 1:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayObjectValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, Index);
						break;
					}
					case 2:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayObjectValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, Index);
						break;
					}
				}
			}
			else if(CPPTypeObject->IsA<UEnum>())
			{
				switch(ArrayDimension)
				{
					default:
					case 0:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleEnumValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, Index);
						break;
					}
					case 1:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayEnumValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, Index);
						break;
					}
					case 2:
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayEnumValue, Index);
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, Index);
						break;
					}
				}
			}
			else if(const UStruct* Struct = Cast<UStruct>(CPPTypeObject))
			{
				if(Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					if(ArrayDimension == 0)
					{
						RegisterNewType(FRigVMTemplateArgument::ETypeCategory_Execute, Index);
					}
				}
				else
				{
					if(GetMathTypes().Contains(CPPTypeObject))
					{
						switch(ArrayDimension)
						{
							default:
							case 0:
							{
								RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue, Index);
								break;
							}
							case 1:
							{
								RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayMathStructValue, Index);
								break;
							}
							case 2:
							{
								RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayMathStructValue, Index);
								break;
							}
						}
					}
					
					switch(ArrayDimension)
					{
						default:
						case 0:
						{
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue, Index);
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, Index);
							break;
						}
						case 1:
						{
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayScriptStructValue, Index);
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, Index);
							break;
						}
						case 2:
						{
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayScriptStructValue, Index);
							RegisterNewType(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, Index);
							break;
						}
					}
				}
			}
		}

		// propagate new type to templates once they have all been added to the categories
		for (const auto& [Category, NewIndex]: ToPropagate)
		{
			PropagateTypeAddedToCategory_NoLock(Category, NewIndex);
		}

		// if the type is a structure
		// then add all of its sub property types
		if(const UStruct* Struct = Cast<UStruct>(CPPTypeObject))
		{
			for (TFieldIterator<FProperty> It(Struct); It; ++It)
			{
				FProperty* Property = *It;
				if(IsAllowedType_NoLock(Property))
				{
					// by creating a template argument for the child property
					// the type will be added by calling ::FindOrAddType_Internal recursively.
					(void)FRigVMTemplateArgument::Make_NoLock(Property, ThisHandle);
				}
#if WITH_EDITOR
				else
				{
					// If the subproperty is not allowed, let's make sure it's hidden. Otherwise we end up with
					// subpins with invalid types 
					check(FRigVMStruct::GetPinDirectionFromProperty(Property) == ERigVMPinDirection::Hidden);
				}
#endif
			}			
		}
		
		Index = GetTypeIndex_NoLock(InType);
		if (IsValid(CPPTypeObject))
		{
			if (CPPTypeObject->IsA<UUserDefinedStruct>() || CPPTypeObject->IsA<UUserDefinedEnum>())
			{
				TRigVMTypeIndex ElementTypeIndex = GetTypeIndex_NoLock(ElementType);

				TRigVMTypeIndexTriple Triple;
				Triple.Get<0>() = ElementTypeIndex;
				Triple.Get<1>() = GetArrayTypeFromBaseTypeIndex_NoLock(Triple.Get<0>());
				Triple.Get<2>() = GetArrayTypeFromBaseTypeIndex_NoLock(Triple.Get<1>());
				
				// used to track name changes to user defined types, stores the element type indices, see RemoveType().
				SoftObjectPathToTypeIndex.FindOrAdd(CPPTypeObject) = Triple;
			}
		}
		
		return Index;
	}
	
	return Index;
}

void FRigVMRegistry_NoLock::RegisterTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory, TRigVMTypeIndex InTypeIndex)
{
	check(InCategory != FRigVMTemplateArgument::ETypeCategory_Invalid);

	TypesPerCategory.FindChecked(InCategory).AddUnique(InTypeIndex);
}

void FRigVMRegistry_NoLock::PropagateTypeAddedToCategory_NoLock(const FRigVMTemplateArgument::ETypeCategory InCategory, const TRigVMTypeIndex InTypeIndex)
{
	if(bAvoidTypePropagation)
	{
		return;
	}
	
	check(InCategory != FRigVMTemplateArgument::ETypeCategory_Invalid);
	if ( ensure(TypesPerCategory.FindChecked(InCategory).Contains(InTypeIndex)) )
	{
		// when adding a new type - we need to update template arguments which expect to have access to that type 
		const TArray<int32>& TemplatesToUseType = TemplatesPerCategory.FindChecked(InCategory);
		for(const int32 TemplateIndex : TemplatesToUseType)
		{
			FRigVMTemplate& Template = Templates[TemplateIndex];
			(void)Template.HandlePropagatedArgumentType(InTypeIndex, ThisHandle);
		}
	}
}

bool FRigVMRegistry_NoLock::RemoveType_NoLock(const FSoftObjectPath& InObjectPath)
{
	if (const TRigVMTypeIndexTriple* TypeIndicesPtr = SoftObjectPathToTypeIndex.Find(InObjectPath))
	{
		const TRigVMTypeIndexTriple& TypeIndices = *TypeIndicesPtr;
		if(TypeIndices.Get<0>() == INDEX_NONE)
		{
			return false;
		}

		if (RemoveType_NoLock(TypeIndices.Get<0>()))
		{
			return true;
		}
	}
	else if (const UObject* ResolvedObject = InObjectPath.ResolveObject())
	{
		for (int32 Index = 0; Index < Types.Num(); Index++)
		{
			if (Types[Index].Type.CPPTypeObject == ResolvedObject)
			{
				if (RemoveType_NoLock(Index))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FRigVMRegistry_NoLock::RemoveType_NoLock(TRigVMTypeIndex InTypeIndex)
{
	if (!Types.IsValidIndex(InTypeIndex))
	{
		return false;
	}
	
	if (Types[InTypeIndex].IsUnknownType())
	{
		return false;
	}

	check(!IsArrayType_NoLock(InTypeIndex));

	TStaticArray<TRigVMTypeIndex, 3> Indices {InPlace, INDEX_NONE};
	Indices[0] = InTypeIndex;
	Indices[1] = GetArrayTypeFromBaseTypeIndex_NoLock(Indices[0]);

	// any type that can be removed should have 3 entries in the registry
	if (ensure(Indices[1] != INDEX_NONE))
	{
		Indices[2] = GetArrayTypeFromBaseTypeIndex_NoLock(Indices[1]);
	}
	
	for (int32 ArrayDimension=0; ArrayDimension<3; ++ArrayDimension)
	{
		const TRigVMTypeIndex Index = Indices[ArrayDimension];
		
		if (Index == INDEX_NONE)
		{
			break;
		}
		
		switch(ArrayDimension)
		{
			default:
			case 0:
			{
				// we remove all user defined types from the table - but we keep the softobject paths for 
				// C++ defined types. that way we can look up their type indices later in case the types
				// come back into view. 
				if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Types[Index].Type.CPPTypeObject))
				{
					SoftObjectPathToTypeIndex.Remove(UserDefinedStruct);
				}
				else if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(Types[Index].Type.CPPTypeObject))
				{
					SoftObjectPathToTypeIndex.Remove(UserDefinedEnum);
				}
				else if (Types[Index].Type.CPPTypeObject && IsValid(Types[Index].Type.CPPTypeObject))
				{
					// for C++ based types - let's remember where the types where in the first place
					// so we can put them back.
					SoftObjectPathToTypeIndex.FindOrAdd(Types[Index].Type.CPPTypeObject) = {Indices[0], Indices[1], Indices[2]};
				}
					
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleSimpleValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleEnumValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue, Index);
				break;
			}
			case 1:
			{
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArraySimpleValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayMathStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayScriptStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayEnumValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayObjectValue, Index);
				break;
			}
			case 2:
			{
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArraySimpleValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArrayMathStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArrayScriptStructValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArrayEnumValue, Index);
				RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_ArrayArrayObjectValue, Index);
				break;
			}
		}

		// remove the type from the registry entirely
		TypeToIndex.Remove(Types[Index].Type);
		Types[Index] = FTypeInfo();
	}

	return true;	
}

void FRigVMRegistry_NoLock::RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory, TRigVMTypeIndex InTypeIndex)
{
	check(InCategory != FRigVMTemplateArgument::ETypeCategory_Invalid);

	if (!TypesPerCategory.FindChecked(InCategory).Contains(InTypeIndex))
	{
		return;
	}

	const TArray<int32>& TemplatesToUseType = TemplatesPerCategory.FindChecked(InCategory);
	for (const int32 TemplateIndex : TemplatesToUseType)
	{
		FRigVMTemplate& Template = Templates[TemplateIndex];
		Template.HandleTypeRemoval(InTypeIndex, ThisHandle);
	}
}

void FRigVMRegistry_NoLock::OnEngineInit()
{
	FRigVMRegistry_RWLock& Registry = FRigVMRegistry_RWLock::Get();
	Registry.RefreshEngineTypes();
	Registry.bEverRefreshedDispatchFactoriesAfterEngineInit = true;
}

// This function needs to be in cpp file instead of header file
// to avoid confusing certain compilers into creating multiple copies of the registry
FRigVMRegistry_RWLock& FRigVMRegistry_RWLock::Get()
{
	static FRigVMRegistry_RWLock s_RigVMRegistry(true /* global */);
	return s_RigVMRegistry;
}

void FRigVMRegistry_RWLock::OnCleanupUnloadingObjects(const TArrayView<UObject*> InObjects)
{
	bool bAssetRemoved;
	{
		FConditionalWriteScopeLock _(*this);
		bAssetRemoved = Super::OnCleanupUnloadingObjects_NoLock(InObjects);
	}

	if (bAssetRemoved)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();
	}
}

void FRigVMRegistry_RWLock::OnAssetRemoved(const FAssetData& InAssetData)
{
	bool bAssetRemoved;
	{
		FConditionalWriteScopeLock _(*this);
		bAssetRemoved = Super::OnAssetRemoved_NoLock(InAssetData);
	}

	if (bAssetRemoved)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();
	}
}

void FRigVMRegistry_RWLock::OnPluginLoaded(IPlugin& InPlugin)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigVMRegistry_RWLock::OnPluginLoaded");
	bool bRegistryChanged;
	{
		FConditionalWriteScopeLock _(*this);
		bRegistryChanged = Super::OnPluginLoaded_NoLock(InPlugin);
	}
		
	if (bRegistryChanged)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();
	}
}

static std::atomic_int32_t GUnmountSuspendedCounter = 0;
static FCriticalSection GListToUnmountSync;
static TArray<FString> GListOfModules;

void FRigVMRegistry_RWLock::SuspendGlobalUnmount()
{
	GUnmountSuspendedCounter.fetch_add(1, std::memory_order_acq_rel);
}

void FRigVMRegistry_RWLock::ResumeGlobalUnmount()
{
	if (GUnmountSuspendedCounter.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		TArray<FString> LocalListOfModules;
		{
			FScopeLock Guard(&GListToUnmountSync);
			LocalListOfModules = MoveTemp(GListOfModules);
		}
		Get().OnModulesUnloadedInternal(LocalListOfModules);
	}
}

void FRigVMRegistry_RWLock::OnPluginUnloaded(IPlugin& InPlugin)
{
	if (IsGlobalRegistry() && GUnmountSuspendedCounter.load(std::memory_order_acquire) > 0)
	{
		TArray<FString> ModulesPrefixesList = GetPluginModulePrefixes(InPlugin);
		FScopeLock Guard(&GListToUnmountSync);
		GListOfModules.Append(MoveTemp(ModulesPrefixesList));
		return;
	}

	bool bRegistryChanged;
	{
		FConditionalWriteScopeLock _(*this);
		bRegistryChanged = Super::OnPluginUnloaded_NoLock(InPlugin);
	}
		
	if (bRegistryChanged)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();
	}
}

void FRigVMRegistry_RWLock::OnModulesUnloaded(TConstArrayView<FName> ModuleNames)
{
	bool bRegistryChanged;
	{
		FConditionalWriteScopeLock _(*this);
		bRegistryChanged = Super::OnModulesUnloaded_NoLock(ModuleNames);
	}
		
	if (bRegistryChanged)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();
	}
}

void FRigVMRegistry_RWLock::OnAnimationAttributeTypesChanged(const UScriptStruct* InStruct, bool bIsAdded)
{
	{
		FConditionalWriteScopeLock _(*this);
		Super::OnAnimationAttributeTypesChanged_NoLock(InStruct, bIsAdded);
	}

	if (bIsAdded)
	{
		OnRigVMRegistryChangedDelegate.Broadcast();		
	}
}

TRigVMTypeIndex FRigVMRegistry_NoLock::GetTypeIndex_NoLock(const FRigVMTemplateArgumentType& InType) const
{
	if(const TRigVMTypeIndex* Index = TypeToIndex.Find(InType))
	{
		return *Index;
	}
	return INDEX_NONE;
}

const FRigVMTemplateArgumentType& FRigVMRegistry_NoLock::GetType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if((Types.IsValidIndex(InTypeIndex)))
	{
		return Types[InTypeIndex].Type;
	}
	static FRigVMTemplateArgumentType EmptyType;
	return EmptyType;
}

const FRigVMTemplateArgumentType& FRigVMRegistry_NoLock::FindTypeFromCPPType_NoLock(const FString& InCPPType) const
{
	const int32 TypeIndex = GetTypeIndexFromCPPType_NoLock(InCPPType);
	if(ensure(Types.IsValidIndex(TypeIndex)))
	{
		return Types[TypeIndex].Type;
	}

	static FRigVMTemplateArgumentType EmptyType;
	return EmptyType;
}

TRigVMTypeIndex FRigVMRegistry_NoLock::GetTypeIndexFromCPPType_NoLock(const FString& InCPPType) const
{
	TRigVMTypeIndex Result = INDEX_NONE;
	if(!InCPPType.IsEmpty())
	{
		const FName CPPTypeName = *InCPPType;

		auto Predicate = [CPPTypeName](const FTypeInfo& Info) -> bool
		{
			return Info.Type.CPPType == CPPTypeName;
		};

		Result = Types.IndexOfByPredicate(Predicate);

		// in game / non-editor it's possible that a user defined struct or enum 
		// has not been registered. thus we'll try to find it and if not,
		// we will call RefreshEngineTypes to bring things up to date here.
		if (Result == INDEX_NONE)
		{
			const FName BaseCPPTypeName = RigVMTypeUtils::IsArrayType(InCPPType) ? *RigVMTypeUtils::BaseTypeFromArrayType(InCPPType) : *InCPPType;

			for (TObjectIterator<UUserDefinedStruct> ScriptStructIt; ScriptStructIt; ++ScriptStructIt)
			{
				UUserDefinedStruct* ScriptStruct = *ScriptStructIt;
				const FRigVMTemplateArgumentType ArgumentType(ScriptStruct);
				if (ArgumentType.CPPType == BaseCPPTypeName)
				{
					// this check for example makes sure we don't add structs defined in verse
					if (IsAllowedType_NoLock(ScriptStruct))
					{
						const_cast<FRigVMRegistry_NoLock*>(this)->FindOrAddType_NoLock(ArgumentType, false);
						Result = Types.IndexOfByPredicate(Predicate);
						break;
					}
				}
			}
			if (Result == INDEX_NONE) // if we can not find a struct, lets try an enum
			{
				for (TObjectIterator<UUserDefinedEnum> EnumIt; EnumIt; ++EnumIt)
				{
					UUserDefinedEnum* Enum = *EnumIt;
					const FRigVMTemplateArgumentType ArgumentType(Enum);
					if (ArgumentType.CPPType == BaseCPPTypeName)
					{
						// this check for example makes sure we don't add enums defined in verse
						if (IsAllowedType_NoLock(Enum))
						{
							const_cast<FRigVMRegistry_NoLock*>(this)->FindOrAddType_NoLock(ArgumentType, false);
							Result = Types.IndexOfByPredicate(Predicate);
							break;
						}
					}
				}
			}
			if (Result == INDEX_NONE) // else a full scan
			{
				// we may need to update the types again to registry potentially
				// missing predicate types 
				const_cast<FRigVMRegistry_NoLock*>(this)->RefreshEngineTypes_NoLock();
				Result = Types.IndexOfByPredicate(Predicate);
			}
		}

		// If not found, try to find a redirect
		if (Result == INDEX_NONE)
		{
			const FString NewCPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType);
			Result = Types.IndexOfByPredicate([NewCPPType](const FTypeInfo& Info) -> bool
			{
				return Info.Type.CPPType == *NewCPPType;
			});
		}
	}
	return Result;
}

bool FRigVMRegistry_NoLock::IsArrayType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if((Types.IsValidIndex(InTypeIndex)))
	{
		return Types[InTypeIndex].bIsArray;
	}
	return false;
}

bool FRigVMRegistry_NoLock::IsExecuteType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(InTypeIndex == INDEX_NONE)
	{
		return false;
	}
	
	if(ensure(Types.IsValidIndex(InTypeIndex)))
	{
		return Types[InTypeIndex].bIsExecute;
	}
	return false;
}

bool FRigVMRegistry_NoLock::ConvertExecuteContextToBaseType_NoLock(TRigVMTypeIndex& InOutTypeIndex) const
{
	if(InOutTypeIndex == INDEX_NONE)
	{
		return false;
	}
		
	if(InOutTypeIndex == RigVMTypeUtils::TypeIndex::Execute) 
	{
		return true;
	}

	if(!IsExecuteType_NoLock(InOutTypeIndex))
	{
		return false;
	}

	// execute arguments can have various execute context types. but we always
	// convert them to the base execute type to make matching types easier later.
	// this means that the execute argument in every permutations shares 
	// the same type index of RigVMTypeUtils::TypeIndex::Execute
	if(IsArrayType_NoLock(InOutTypeIndex))
	{
		InOutTypeIndex = GetArrayTypeFromBaseTypeIndex_NoLock(RigVMTypeUtils::TypeIndex::Execute);
	}
	else
	{
		InOutTypeIndex = RigVMTypeUtils::TypeIndex::Execute;
	}

	return true;
}

int32 FRigVMRegistry_NoLock::GetArrayDimensionsForType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(ensure(Types.IsValidIndex(InTypeIndex)))
	{
		const FTypeInfo& Info = Types[InTypeIndex];
		if(Info.bIsArray)
		{
			return 1 + GetArrayDimensionsForType_NoLock(Info.BaseTypeIndex);
		}
	}
	return 0;
}

bool FRigVMRegistry_NoLock::IsWildCardType_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	return RigVMTypeUtils::TypeIndex::WildCard == InTypeIndex ||
		RigVMTypeUtils::TypeIndex::WildCardArray == InTypeIndex;
}

bool FRigVMRegistry_NoLock::CanMatchTypes_NoLock(TRigVMTypeIndex InTypeIndexA, TRigVMTypeIndex InTypeIndexB, bool bAllowFloatingPointCasts) const
{
	if(!Types.IsValidIndex(InTypeIndexA) || !Types.IsValidIndex(InTypeIndexB))
	{
		return false;
	}

	if(InTypeIndexA == InTypeIndexB)
	{
		return true;
	}

	// execute types can always be connected
	if(IsExecuteType_NoLock(InTypeIndexA) && IsExecuteType_NoLock(InTypeIndexB))
	{
		return GetArrayDimensionsForType_NoLock(InTypeIndexA) == GetArrayDimensionsForType_NoLock(InTypeIndexB);
	}

	if(bAllowFloatingPointCasts)
	{
		// swap order since float is known to registered before double
		if(InTypeIndexA > InTypeIndexB)
		{
			Swap(InTypeIndexA, InTypeIndexB);
		}
		if(InTypeIndexA == RigVMTypeUtils::TypeIndex::Float && InTypeIndexB == RigVMTypeUtils::TypeIndex::Double)
		{
			return true;
		}
		if(InTypeIndexA == RigVMTypeUtils::TypeIndex::FloatArray && InTypeIndexB == RigVMTypeUtils::TypeIndex::DoubleArray)
		{
			return true;
		}
	}
	return false;
}

const TArray<TRigVMTypeIndex>& FRigVMRegistry_NoLock::GetCompatibleTypes_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		static const TArray<TRigVMTypeIndex> CompatibleTypes = {RigVMTypeUtils::TypeIndex::Double};
		return CompatibleTypes;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::Double)
	{
		static const TArray<TRigVMTypeIndex> CompatibleTypes = {RigVMTypeUtils::TypeIndex::Float};
		return CompatibleTypes;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
	{
		static const TArray<TRigVMTypeIndex> CompatibleTypes = {RigVMTypeUtils::TypeIndex::DoubleArray};
		return CompatibleTypes;
	}
	if(InTypeIndex == RigVMTypeUtils::TypeIndex::DoubleArray)
	{
		static const TArray<TRigVMTypeIndex> CompatibleTypes = {RigVMTypeUtils::TypeIndex::FloatArray};
		return CompatibleTypes;
	}

	static const TArray<TRigVMTypeIndex> EmptyTypes;
	return EmptyTypes;
}

const TArray<TRigVMTypeIndex>& FRigVMRegistry_NoLock::GetTypesForCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory) const
{
	check(InCategory != FRigVMTemplateArgument::ETypeCategory_Invalid);
	return TypesPerCategory.FindChecked(InCategory);
}

TRigVMTypeIndex FRigVMRegistry_NoLock::GetArrayTypeFromBaseTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(ensure(Types.IsValidIndex(InTypeIndex)))
	{
#if UE_RIGVM_DEBUG_TYPEINDEX
		TRigVMTypeIndex Result = Types[InTypeIndex].ArrayTypeIndex;
		if(!InTypeIndex.Name.IsNone())
		{
			Result.Name = *RigVMTypeUtils::ArrayTypeFromBaseType(InTypeIndex.Name.ToString());
		}
		return Result;
#else
		return Types[InTypeIndex].ArrayTypeIndex;
#endif
	}
	return INDEX_NONE;
}

TRigVMTypeIndex FRigVMRegistry_NoLock::GetBaseTypeFromArrayTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const
{
	if(ensure(Types.IsValidIndex(InTypeIndex)))
	{
#if UE_RIGVM_DEBUG_TYPEINDEX
		TRigVMTypeIndex Result = Types[InTypeIndex].BaseTypeIndex;
		if(!InTypeIndex.Name.IsNone())
		{
			Result.Name = *RigVMTypeUtils::BaseTypeFromArrayType(InTypeIndex.Name.ToString());
		}
		return Result;
#else
		return Types[InTypeIndex].BaseTypeIndex;
#endif
	}
	return INDEX_NONE;
}

bool FRigVMRegistry_NoLock::IsSpecificallyAllowedType_NoLock(const UObject* InObject) const
{
	if (const UClass* Class = Cast<UClass>(InObject))
	{
		return AllowedClasses.Contains(Class);
	}
	if (const UScriptStruct* Struct = Cast<UScriptStruct>(InObject))
	{
		return AllowedStructs.Contains(Struct);
	}
	return false;
}

void FRigVMRegistry_NoLock::AddSpecificallyAllowedType_NoLock(const UObject* InObject)
{
	if (const UClass* Class = Cast<UClass>(InObject))
	{
		AllowedClasses.Add(Class);
	}
	if (const UScriptStruct* Struct = Cast<UScriptStruct>(InObject))
	{
		AllowedStructs.Add(Struct);
	}
}

bool FRigVMRegistry_NoLock::IsAllowedType_NoLock(const FProperty* InProperty) const
{
	if(InProperty->IsA<FBoolProperty>() ||
		InProperty->IsA<FUInt32Property>() ||
		InProperty->IsA<FInt8Property>() ||
		InProperty->IsA<FInt16Property>() ||
		InProperty->IsA<FIntProperty>() ||
		InProperty->IsA<FInt64Property>() ||
		InProperty->IsA<FFloatProperty>() ||
		InProperty->IsA<FDoubleProperty>() ||
		InProperty->IsA<FNumericProperty>() ||
		InProperty->IsA<FNameProperty>() ||
		InProperty->IsA<FStrProperty>())
	{
		return true;
	}

	if(const FArrayProperty* ArrayProperty  = CastField<FArrayProperty>(InProperty))
	{
		if (ArrayProperty->Inner)
		{
			return IsAllowedType_NoLock(ArrayProperty->Inner);
		}
	}
	if(const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		return IsAllowedType_NoLock(StructProperty->Struct);
	}
	if(const FClassProperty* ClassProperty = CastField<FClassProperty>(InProperty))
	{
		return IsAllowedType_NoLock(ClassProperty->MetaClass);
	}
	if(const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		return IsAllowedType_NoLock(ObjectProperty->PropertyClass);
	}
	if(const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InProperty))
	{
		return IsAllowedType_NoLock(SoftObjectProperty->PropertyClass);
	}
	if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		return IsAllowedType_NoLock(EnumProperty->GetEnum());
	}
	if(const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		if(const UEnum* Enum = ByteProperty->Enum)
		{
			return IsAllowedType_NoLock(Enum);
		}
		return true;
	}
	return false;
}

bool FRigVMRegistry_NoLock::IsAllowedType_NoLock(const UEnum* InEnum) const
{
	if(!InEnum)
	{
		return false;
	}
	
	// disallow verse based enums for now
	if (FPackageName::IsVersePackage(InEnum->GetPackage()->GetName()))
	{
		return false;
	}

	static const FName VerseEnumName(TEXT("VerseEnum"));
	if(IsTypeOfByName(InEnum, VerseEnumName))
	{
		return false;
	}

	return !InEnum->HasAnyFlags(DisallowedFlags()) && InEnum->HasAllFlags(NeededFlags());
}

bool FRigVMRegistry_NoLock::IsAllowedType_NoLock(const UStruct* InStruct) const
{
	if(!InStruct || InStruct->HasAnyFlags(DisallowedFlags()) || !InStruct->HasAllFlags(NeededFlags()))
	{
		return false;
	}
	if(InStruct->IsChildOf(FRigVMStruct::StaticStruct()) &&
		!InStruct->IsChildOf(FRigVMTrait::StaticStruct()))
	{
		return false;
	}
	if(InStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()))
	{
		return false;
	}
	
	// disallow verse data structures for now
	if (FPackageName::IsVersePackage(InStruct->GetPackage()->GetName()))
	{
		return false;
	}
	
	static const FName VerseStructName(TEXT("VerseStruct"));
	if(IsTypeOfByName(InStruct, VerseStructName))
	{
		return false;
	}

	// allow all user defined structs since they can always be changed to be compliant with RigVM restrictions
	if (InStruct->IsA<UUserDefinedStruct>())
	{
		return true;
	}

	// Allow structs we have explicitly opted into
	// This is on the understanding that if they have invalid sub-members that any pins representing them will need to be hidden
	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InStruct))
	{
		if(AllowedStructs.Contains(ScriptStruct))
		{
			return true;
		}
	}

	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		if(!IsAllowedType_NoLock(*It))
		{
			return false;
		}
	}
	return true;
}

bool FRigVMRegistry_NoLock::IsAllowedType_NoLock(const UClass* InClass) const
{
	if(!InClass || InClass->HasAnyClassFlags(CLASS_Hidden))
	{
		return false;
	}

	// Only allow native object types
	if (!InClass->HasAnyClassFlags(CLASS_Native))
	{
		return false;
	}

	// disallow verse based classes for now
	if (FPackageName::IsVersePackage(InClass->GetPackage()->GetName()))
	{
		return false;
	}

	static const FName VerseClassName(TEXT("VerseClass"));
	if(IsTypeOfByName(InClass, VerseClassName))
	{
		return false;
	}

	return AllowedClasses.Contains(InClass);
}

bool FRigVMRegistry_NoLock::IsTypeOfByName(const UObject* InObject, const FName& InName)
{
	if(!InObject || InName.IsNone())
	{
		return false;
	}
	
	const UClass* Class = InObject->GetClass();
	while(Class)
	{
		if(Class->GetFName().IsEqual(InName, ENameCase::CaseSensitive))
		{
			return true;
		}
		Class = Class->GetSuperClass();
	}
	
	return false;
}

void FRigVMRegistry_NoLock::Register_NoLock(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct, const TArray<FRigVMFunctionArgument>& InArguments)
{
	if (FindFunction_NoLock(InName) != nullptr)
	{
		return;
	}

#if WITH_EDITOR
	FString StructureError;
	if (!FRigVMStruct::ValidateStruct(InStruct, &StructureError))
	{
		UE_LOGF(LogRigVM, Error, "Failed to validate struct '%ls': %ls", *InStruct->GetName(), *StructureError);
		return;
	}
#endif

	const FName FunctionName = InName;
	int32 FunctionIndex = Functions.Num();
	if (const int32* ExistingFunctionIndex = PreviousFunctionNameToIndex.Find(FunctionName))
	{
		check(Functions.IsValidIndex(*ExistingFunctionIndex));
		FunctionIndex = *ExistingFunctionIndex;
	}

	const FRigVMFunction Function(InName, InFunctionPtr, InStruct, FunctionIndex, InArguments);
	check(Function.Index == FunctionIndex);

	if (FunctionIndex == Functions.Num())
	{
		Functions.AddElement(Function);
	}
	else
	{
		Functions[FunctionIndex] = Function;
	}
	Functions[FunctionIndex].OwnerRegistry = this;
	FunctionNameToIndex.Add(FunctionName, FunctionIndex);

#if !WITH_EDITOR
	// if we are not in editor and VMs rely on their own localized registry
	// let's only register the function pointer - but not the types etc... keep it as shallow as possible.
	if(bIsGlobalRegistry && CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread() == true)
	{
		return;
	}
#endif

	// register all of the types used by the function
	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		// creating the argument causes the registration
		(void)FRigVMTemplateArgument::Make_NoLock(*It, ThisHandle);
	}

#if WITH_EDITOR
	
	FString TemplateMetadata;
	if (InStruct->GetStringMetaDataHierarchical(TemplateNameMetaName, &TemplateMetadata))
	{
		bool bIsDeprecated = InStruct->HasMetaData(FRigVMStruct::DeprecatedMetaName);
		TChunkedArray<FRigVMTemplate>& TemplateArray = (bIsDeprecated) ? DeprecatedTemplates : Templates;
		TMap<FName, int32>& NotationToIndex = (bIsDeprecated) ? DeprecatedTemplateNotationToIndex : TemplateNotationToIndex;
		
		FString MethodName;
		if (FString(InName).Split(TEXT("::"), nullptr, &MethodName))
		{
			const FString TemplateName = RigVMStringUtils::JoinStrings(TemplateMetadata, MethodName, TEXT("::"));
			FRigVMTemplate Template(InStruct, TemplateName, Function.Index, ThisHandle);
			if (Template.IsValid())
			{
				Functions[Function.Index].PermutationIndex = Template.NumPermutations_NoLock(ThisHandle) - 1;
				
				bool bWasMerged = false;

				const int32* ExistingTemplateIndexPtr = NotationToIndex.Find(Template.GetNotation());
				if(ExistingTemplateIndexPtr)
				{
					FRigVMTemplate& ExistingTemplate = TemplateArray[*ExistingTemplateIndexPtr];
					if (ExistingTemplate.Merge(Template, ThisHandle))
					{
						if (!bIsDeprecated)
						{
							Functions[Function.Index].TemplateIndex = ExistingTemplate.Index;
							Functions[Function.Index].PermutationIndex = ExistingTemplate.NumPermutations_NoLock(ThisHandle) - 1;
#if WITH_EDITOR
							int32 MaxNumPermutations = 0;
							for(const FRigVMTemplateArgument& Argument : ExistingTemplate.Arguments)
							{
								MaxNumPermutations = FMath::Max(MaxNumPermutations, Argument.GetNumTypes_NoLock(ThisHandle));
							}
							verify(MaxNumPermutations == ExistingTemplate.NumPermutations_NoLock(ThisHandle));
#endif
						}
						bWasMerged = true;
					}
				}

				if (!bWasMerged)
				{
					Template.Index = TemplateArray.Num();
					if (!bIsDeprecated)
					{
						Functions[Function.Index].TemplateIndex = Template.Index;
					}
					TemplateArray.AddElement(Template);
					
					if(ExistingTemplateIndexPtr == nullptr)
					{
						NotationToIndex.Add(Template.GetNotation(), Template.Index);
					}
				}
			}
		}
	}

#endif
}

void FRigVMRegistry_NoLock::RegisterCompiledInStruct_NoLock(UScriptStruct* InStruct, TConstArrayView<FRigVMCompiledInFunction> InFunctions)
{
	check(InStruct);
	for (const FRigVMCompiledInFunction& Function : InFunctions)
	{
		// Predicates have no function ptr
		if (Function.Function != nullptr)
		{
			Register_NoLock(Function.MethodName, Function.Function, InStruct, TArray<FRigVMFunctionArgument>(Function.Parameters));
		}
		else 
		{
			RegisterPredicate_NoLock(InStruct, Function.MethodName, TArray<FRigVMFunctionArgument>(Function.Parameters));
		}
	}
}

bool FRigVMRegistry_NoLock::RemoveFunction_NoLock(int32 InFunctionIndex)
{
	check(Functions.IsValidIndex(InFunctionIndex));

	if(!Functions[InFunctionIndex].IsValid())
	{
		return false;
	}
	
	const FName PermutationName = *Functions[InFunctionIndex].GetName();
	FunctionNameToIndex.Remove(PermutationName);
	PreviousFunctionNameToIndex.FindOrAdd(PermutationName, INDEX_NONE) = InFunctionIndex;
	Functions[InFunctionIndex] = FRigVMFunction();

	return true;
}

const FRigVMDispatchFactory* FRigVMRegistry_NoLock::RegisterFactory_NoLock(UScriptStruct* InFactoryStruct, const TArray<FRigVMTemplateArgumentInfo>& InArgumentInfos)
{
	check(InFactoryStruct);
	check(InFactoryStruct != FRigVMDispatchFactory::StaticStruct());
	check(InFactoryStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()));

	// ensure to register factories only once
	if(const FRigVMDispatchFactory* const* ExistingFactory = FactoryStructToFactory.Find(InFactoryStruct))
	{
		return *ExistingFactory;
	}

#if WITH_EDITOR
	if(InFactoryStruct->HasMetaData(TEXT("Abstract")))
	{
		return nullptr;
	}
#endif

	FRigVMDispatchFactory* Factory = (FRigVMDispatchFactory*)FMemory::Malloc(InFactoryStruct->GetStructureSize());
	InFactoryStruct->InitializeStruct(Factory, 1);
	Factory->FactoryScriptStruct = InFactoryStruct;
	Factory->FactoryIndex = Factories.Add(Factory);
	Factory->FactoryName = FRigVMDispatchFactory::GetFactoryName(InFactoryStruct);
	Factory->FactoryNameString = Factory->FactoryName.ToString();
	Factory->OwnerRegistry = this;
	FactoryNameToFactory.Add(Factory->FactoryName, Factory);
	FactoryStructToFactory.Add(InFactoryStruct, Factory);
	Factory->RegisterDependencyTypes_NoLock(ThisHandle);

	if(!InArgumentInfos.IsEmpty())
	{
		Factory->CreateTemplateForArgumentInfos_NoLock(InArgumentInfos, ThisHandle);
	}
	
	return Factory;
}

bool FRigVMRegistry_NoLock::RemoveFactory_NoLock(UScriptStruct* InFactoryStruct)
{
	const int32 FactoryIndex = Factories.IndexOfByPredicate([InFactoryStruct](const FRigVMDispatchFactory* Factory) -> bool
	{
		return Factory->GetScriptStruct() == InFactoryStruct;
	});

	if (FactoryIndex == INDEX_NONE)
	{
		return false;
	}

	check(Factories.IsValidIndex(FactoryIndex));

	const int32 TemplateIndex = Factories[FactoryIndex]->GetTemplate_NoLock(ThisHandle)->Index;
	
	if (!RemoveTemplate_NoLock(TemplateIndex))
	{
		return false;
	}

	const FName FactoryName = Factories[FactoryIndex]->GetFactoryName();
	InFactoryStruct->DestroyStruct(Factories[FactoryIndex], 1);
	FMemory::Free(Factories[FactoryIndex]);
	
	Factories.RemoveAt(FactoryIndex);
	FactoryNameToFactory.Remove(FactoryName);
	FactoryStructToFactory.Remove(InFactoryStruct);
	return true;
}

bool FRigVMRegistry_NoLock::RemoveTemplate_NoLock(int32 InTemplateIndex)
{
	check(Templates.IsValidIndex(InTemplateIndex));

	if(!Templates[InTemplateIndex].IsValid())
	{
		return false;
	}

	for (const int32& FunctionIndex : Templates[InTemplateIndex].Permutations)
	{
		if (FunctionIndex == INDEX_NONE)
		{
			continue;
		}
		if (!RemoveFunction_NoLock(FunctionIndex))
		{
			return false;
		}
	}
	
	const FName Notation = Templates[InTemplateIndex].GetNotation();
	TemplateNotationToIndex.Remove(Notation);
	PreviousTemplateNotationToIndex.FindOrAdd(Notation, INDEX_NONE) = InTemplateIndex;
	Templates[InTemplateIndex] = FRigVMTemplate();

	return true;

}

void FRigVMRegistry_NoLock::RegisterPredicate_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const TArray<FRigVMFunctionArgument>& InArguments)
{
#if !WITH_EDITOR
	// if we are not in editor and VMs rely on their own localized registry
	// let's not register any predicates
	if(bIsGlobalRegistry && CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread() == true)
	{
		return;
	}
#endif

	// Make sure the predicate does not already exist
	TArray<FRigVMFunction>& Predicates = StructNameToPredicates.FindOrAdd(InStruct->GetFName());
	if (Predicates.ContainsByPredicate([InName](const FRigVMFunction& Predicate)
	{
		return Predicate.Name == InName;
	}))
	{
		
		return;
	}

	FRigVMFunction Function(InName, nullptr, InStruct, Predicates.Num(), InArguments);
	Predicates.Add(Function);
}

void FRigVMRegistry_NoLock::RegisterObjectTypes_NoLock(TConstArrayView<TPair<UClass*, ERegisterObjectOperation>> InClasses)
{
	for (TPair<UClass*, ERegisterObjectOperation> ClassOpPair : InClasses)
	{
		UClass* Class = ClassOpPair.Key;
		ERegisterObjectOperation Operation = ClassOpPair.Value;

		// Only allow native object types
		if (Class->HasAnyClassFlags(CLASS_Native))
		{
			switch (Operation)
			{
			case ERegisterObjectOperation::Class:
				AllowedClasses.Add(Class);
				break;
			case ERegisterObjectOperation::ClassAndParents:
				{
					// Add all parent classes
					do
					{
						AllowedClasses.Add(Class);
						Class = Class->GetSuperClass();
					} while (Class);
					break;
				}
			case ERegisterObjectOperation::ClassAndChildren:
				{
					// Add all child classes
					TArray<UClass*> DerivedClasses({ Class });
					GetDerivedClasses(Class, DerivedClasses, /*bRecursive=*/true);
					for (UClass* DerivedClass : DerivedClasses)
					{
						AllowedClasses.Add(DerivedClass);
					}
					break;
				}
			}

		}
	}
}

void FRigVMRegistry_NoLock::RegisterStructTypes_NoLock(TConstArrayView<UScriptStruct*> InStructs)
{
	for (UScriptStruct* Struct : InStructs)
	{
		if(!Struct->IsA<UUserDefinedStruct>())
		{
			AllowedStructs.Add(Struct);
		}
	}
}

const FRigVMFunction* FRigVMRegistry_NoLock::FindFunction_NoLock(const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InTypeResolver) const
{
	// Check first if the function is provided by internally registered rig units. 
	if(const int32* FunctionIndexPtr = FunctionNameToIndex.Find(InName))
	{
		return &Functions[*FunctionIndexPtr];
	}

	// Otherwise ask the associated dispatch factory for a function matching this signature.
	const FString NameString(InName);
	FString StructOrFactoryName, SuffixString;
	if(NameString.Split(TEXT("::"), &StructOrFactoryName, &SuffixString))
	{
		// if the factory has never been registered - FindDispatchFactory will try to look it up and register
		if(const FRigVMDispatchFactory* Factory = FindDispatchFactory_NoLock(*StructOrFactoryName))
		{
			if(const FRigVMTemplate* Template = Factory->GetTemplate_NoLock(ThisHandle))
			{
				const FRigVMTemplateTypeMap ArgumentTypes = Template->GetArgumentTypesFromString_NoLock(SuffixString, &InTypeResolver, ThisHandle);
				if(ArgumentTypes.Num() == Template->NumArguments())
				{
					const int32 PermutationIndex = Template->FindPermutation_NoLock(ArgumentTypes, ThisHandle);
					if(PermutationIndex != INDEX_NONE)
					{
						return ((FRigVMTemplate*)Template)->GetOrCreatePermutation_NoLock(PermutationIndex, ThisHandle);
					}
				}
			}
		}
	}

	// if we haven't been able to find the function - try to see if we can get the dispatch or rigvmstruct
	// from a core redirect
	if(!StructOrFactoryName.IsEmpty())
	{
		static const FString StructPrefix = TEXT("F");
		const bool bIsDispatchFactory = StructOrFactoryName.StartsWith(FRigVMDispatchFactory::DispatchPrefix, ESearchCase::CaseSensitive);
		if(bIsDispatchFactory)
		{
			StructOrFactoryName = StructOrFactoryName.Mid(FCString::Strlen(FRigVMDispatchFactory::DispatchPrefix));
		}
		else if(StructOrFactoryName.StartsWith(StructPrefix, ESearchCase::CaseSensitive))
		{
			StructOrFactoryName = StructOrFactoryName.Mid(StructPrefix.Len());
		}
		
		const FCoreRedirectObjectName OldObjectName(StructOrFactoryName);
		TArray<const FCoreRedirect*> Redirects;
		if(FCoreRedirects::GetMatchingRedirects(ECoreRedirectFlags::Type_Struct, OldObjectName, Redirects, ECoreRedirectMatchFlags::AllowPartialMatch))
		{
			for(const FCoreRedirect* Redirect : Redirects)
			{
				FString NewStructOrFactoryName = Redirect->NewName.ObjectName.ToString();

				// Check name differs - this could just be a struct that moved package
				if(NewStructOrFactoryName != StructOrFactoryName)
				{
					if(bIsDispatchFactory)
					{
						NewStructOrFactoryName = FRigVMDispatchFactory::DispatchPrefix + NewStructOrFactoryName;
					}
					else
					{
						NewStructOrFactoryName = StructPrefix + NewStructOrFactoryName;
					}
					const FRigVMFunction* RedirectedFunction = FindFunction_NoLock(*(NewStructOrFactoryName + TEXT("::") + SuffixString), InTypeResolver);
					if(RedirectedFunction)
					{
						FRigVMRegistry_NoLock* MutableRegistry = const_cast<FRigVMRegistry_NoLock*>(this);
						MutableRegistry->FunctionNameToIndex.Add(InName, RedirectedFunction->Index);
						return RedirectedFunction;
					}
				}
			}
		}
	}
	
	return nullptr;
}

const FRigVMFunction* FRigVMRegistry_NoLock::FindFunction_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InResolvalInfo) const
{
	check(InStruct);
	check(InName);

	const FString FunctionName = RigVMStringUtils::JoinStrings(InStruct->GetStructCPPName(), InName, TEXT("::"));
	return FindFunction_NoLock(*FunctionName, InResolvalInfo);
}

const TChunkedArray<FRigVMFunction>& FRigVMRegistry_NoLock::GetFunctions_NoLock() const
{
	return Functions;
}

const FRigVMTemplate* FRigVMRegistry_NoLock::FindTemplate_NoLock(const FName& InNotation, bool bIncludeDeprecated, bool bRemapToCoreDispatch) const
{
	if (InNotation.IsNone())
	{
		return nullptr;
	}

	if(const int32* TemplateIndexPtr = TemplateNotationToIndex.Find(InNotation))
	{
		const FRigVMTemplate* Template = &Templates[*TemplateIndexPtr];
		if (!Template->IsValid())
		{
			return nullptr;
		}
		return Template;
	}

	const FString NotationString(InNotation.ToString());
	FString FactoryName, ArgumentsString;
	if(NotationString.Split(TEXT("("), &FactoryName, &ArgumentsString))
	{
		FRigVMRegistry_NoLock* MutableThis = const_cast<FRigVMRegistry_NoLock*>(this);
		
		// deal with a couple of custom cases
		static const TMap<FString, FString> CoreDispatchMap =
		{
			{
				TEXT("Equals::Execute"),
				MutableThis->FindOrAddDispatchFactory_NoLock<FRigVMDispatch_CoreEquals>()->GetFactoryName().ToString()
			},
			{
				TEXT("NotEquals::Execute"),
				MutableThis->FindOrAddDispatchFactory_NoLock<FRigVMDispatch_CoreNotEquals>()->GetFactoryName().ToString()
			},
		};

		if (bRemapToCoreDispatch)
		{
			if(const FString* RemappedDispatch = CoreDispatchMap.Find(FactoryName))
			{
				FactoryName = *RemappedDispatch;
			}
		}
		
		if(const FRigVMDispatchFactory* Factory = FindDispatchFactory_NoLock(*FactoryName))
		{
			return Factory->GetTemplate_NoLock(ThisHandle);
		}
	}

	if (bIncludeDeprecated)
	{
		if(const int32* TemplateIndexPtr = DeprecatedTemplateNotationToIndex.Find(InNotation))
		{
			return &DeprecatedTemplates[*TemplateIndexPtr];
		}
	}

	const FString OriginalNotation = InNotation.ToString();

	// we may have a dispatch factory which has to be redirected
#if WITH_EDITOR
	if(OriginalNotation.StartsWith(FRigVMDispatchFactory::DispatchPrefix))
	{
		static const int32 PrefixLen = FCString::Strlen(FRigVMDispatchFactory::DispatchPrefix);
		const int32 BraceIndex = OriginalNotation.Find(TEXT("("));
		const FString OriginalDispatchFactoryName = OriginalNotation.Mid(PrefixLen, BraceIndex - PrefixLen); 

		const FCoreRedirectObjectName OldObjectName(OriginalDispatchFactoryName);
		TArray<const FCoreRedirect*> Redirects;
		if(FCoreRedirects::GetMatchingRedirects(ECoreRedirectFlags::Type_Struct, OldObjectName, Redirects, ECoreRedirectMatchFlags::AllowPartialMatch))
		{
			for(const FCoreRedirect* Redirect : Redirects)
			{
				const FString NewDispatchFactoryName = FRigVMDispatchFactory::DispatchPrefix + Redirect->NewName.ObjectName.ToString();
				if(const FRigVMDispatchFactory* NewDispatchFactory = FindDispatchFactory_NoLock(*NewDispatchFactoryName))
				{
					return NewDispatchFactory->GetTemplate_NoLock(ThisHandle);
				}
			}
		}
	}
#endif

	// if we still arrive here we may have a template that used to contain an executecontext.
	{
		FString SanitizedNotation = OriginalNotation;

		static const TArray<TPair<FString, FString>> ExecuteContextArgs = {
			{ TEXT("FRigUnit_SequenceExecution::Execute(in ExecuteContext,out A,out B,out C,out D)"), TEXT("FRigUnit_SequenceExecution::Execute()") },
			{ TEXT("FRigUnit_SequenceAggregate::Execute(in ExecuteContext,out A,out B)"), TEXT("FRigUnit_SequenceAggregate::Execute()") },
			{ TEXT(",io ExecuteContext"), TEXT("") },
			{ TEXT("io ExecuteContext,"), TEXT("") },
			{ TEXT("(io ExecuteContext)"), TEXT("()") },
			{ TEXT(",out ExecuteContext"), TEXT("") },
			{ TEXT("out ExecuteContext,"), TEXT("") },
			{ TEXT("(out ExecuteContext)"), TEXT("()") },
			{ TEXT(",out Completed"), TEXT("") },
			{ TEXT("out Completed,"), TEXT("") },
			{ TEXT("(out Completed)"), TEXT("()") },
		};

		for(int32 Index = 0; Index < ExecuteContextArgs.Num(); Index++)
		{
			const TPair<FString, FString>& Pair = ExecuteContextArgs[Index];
			if(SanitizedNotation.Contains(Pair.Key))
			{
				SanitizedNotation = SanitizedNotation.Replace(*Pair.Key, *Pair.Value);
			}
		}

		if(SanitizedNotation != OriginalNotation)
		{
			return FindTemplate_NoLock(*SanitizedNotation, bIncludeDeprecated);
		}
	}

	return nullptr;
}

const TChunkedArray<FRigVMTemplate>& FRigVMRegistry_NoLock::GetTemplates_NoLock() const
{
	return Templates;
}

const FRigVMTemplate* FRigVMRegistry_NoLock::GetOrAddTemplateFromArguments_NoLock(const FName& InName, const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FRigVMTemplateDelegates& InDelegates)
{
	// avoid reentry in FindTemplate. try to find an existing
	// template only if we are not yet in ::FindTemplate.
	const FName Notation = FRigVMTemplateArgumentInfo::ComputeTemplateNotation(InName, InInfos);
	if(const FRigVMTemplate* ExistingTemplate = FindTemplate_NoLock(Notation))
	{
		return ExistingTemplate;
	}

	return AddTemplateFromArguments_NoLock(InName, InInfos, InDelegates);
}

const FRigVMTemplate* FRigVMRegistry_NoLock::AddTemplateFromArguments_NoLock(const FName& InName, const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FRigVMTemplateDelegates& InDelegates)
{
	// we only support to ask for templates here which provide singleton types
	int32 NumPermutations = 0;
	FRigVMTemplate Template(InName, InInfos, ThisHandle);
	for(const FRigVMTemplateArgument& Argument : Template.Arguments)
	{
		const int32 NumIndices = Argument.GetNumTypes_NoLock(ThisHandle);
		if(!Argument.IsSingleton_NoLock({}, ThisHandle) && NumPermutations > 1)
		{
			if(NumIndices != NumPermutations)
			{
				UE_LOGF(LogRigVM, Error, "Failed to add template '%ls' since the arguments' types counts don't match.", *InName.ToString());
				return nullptr;
			}
		}
		NumPermutations = FMath::Max(NumPermutations, NumIndices); 
	}

	// if any of the arguments are wildcards we'll need to update the types
	for(FRigVMTemplateArgument& Argument : Template.Arguments)
	{
		const int32 NumTypes = Argument.GetNumTypes_NoLock(ThisHandle);
		if(NumTypes == 1)
		{
			const TRigVMTypeIndex FirstTypeIndex = Argument.GetTypeIndex_NoLock(0, ThisHandle);
			if(IsWildCardType_NoLock(FirstTypeIndex))
			{
#if WITH_EDITOR
				Argument.InvalidatePermutations(FirstTypeIndex);
#endif
				if(IsArrayType_NoLock(FirstTypeIndex))
				{
					Argument.TypeCategories.Add(FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue);
				}
				else
				{
					Argument.TypeCategories.Add(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue);
				}
				Argument.bUseCategories = true;
				Argument.TypeIndices.Reset();
		
				NumPermutations = FMath::Max(NumPermutations, Argument.GetNumTypes_NoLock(ThisHandle));
			}
		}
	}

	// Remove duplicate permutations
	/*
	 * we'll disable this for now since it's not a valid approach.
	 * most arguments use type indices by categories, so we can't just remove
	 * single type indices.
	 */

	TArray<FRigVMTypeCacheScope_NoLock> TypeCaches;
	if(!Template.Arguments.IsEmpty())
	{
		//TArray<bool> ToRemove;
		//int32 NumToRemove = 0;
		//ToRemove.AddZeroed(NumPermutations);

		//TSet<uint32> EncounteredPermutations;
		//EncounteredPermutations.Reserve(NumPermutations);

		const int32 NumArguments = Template.Arguments.Num();
		TypeCaches.SetNum(NumArguments);
		
		bool bAnyArgumentWithZeroTypes = false;
		for(int32 ArgIndex = 0; ArgIndex < NumArguments; ArgIndex++)
		{
			(void)TypeCaches[ArgIndex].UpdateIfRequired(ThisHandle, Template.Arguments[ArgIndex]);
			bAnyArgumentWithZeroTypes = bAnyArgumentWithZeroTypes || TypeCaches[ArgIndex].GetNumTypes_NoLock() == 0;
		}

		/*
		if(!bAnyArgumentWithZeroTypes)
		{
			for(int32 Index = 0; Index < NumPermutations; Index++)
			{
				uint32 Hash = GetTypeHash(TypeCaches[0].GetTypeIndex_NoLock(Index));
				for(int32 ArgIndex = 1; ArgIndex < NumArguments; ArgIndex++)
				{
					Hash = HashCombine(Hash, GetTypeHash(TypeCaches[ArgIndex].GetTypeIndex_NoLock(Index)));
				}

				if (EncounteredPermutations.Contains(Hash))
				{
					ToRemove[Index] = true;
					NumToRemove++;
				}
				else
				{
					EncounteredPermutations.Add(Hash);
				}
			}
		}
		
		if(NumToRemove > 0)
		{
			for(FRigVMTemplateArgument& Argument : Template.Arguments)
			{
				// this is not enough - we may have arguments where the type indices
				// array is empty...
				// if(Argument.IsSingleton_NoLock())
				//{
				//	continue;
				//}
				
				TArray<TRigVMTypeIndex> NewTypeIndices;
				NewTypeIndices.Reserve(Argument.TypeIndices.Num() - NumToRemove);

				for(int32 Index = 0; Index < Argument.TypeIndices.Num(); Index++)
				{
					if(!ToRemove[Index])
					{
						NewTypeIndices.Add(Argument.TypeIndices[Index]);
					}
				}
				Argument.TypeIndices = MoveTemp(NewTypeIndices);
			}
			NumPermutations -= NumToRemove;
		}
		*/
	}

#if WITH_EDITOR
	for(FRigVMTemplateArgument& Argument : Template.Arguments)
	{
		Argument.UpdateTypeToPermutationsSlow(ThisHandle);
	}
#endif

	Template.Permutations.Init(INDEX_NONE, NumPermutations);
	Template.RecomputeTypesHashToPermutations(TypeCaches, ThisHandle);

	int32 TemplateIndex = Templates.Num();
	const FName Notation = Template.GetNotation();
	if (const int32* ExistingTemplateIndex = PreviousTemplateNotationToIndex.Find(Notation))
	{
		TemplateIndex = *ExistingTemplateIndex;
	}

	if (TemplateIndex == Templates.Num())
	{
		Templates.AddElement(Template);
	}
	else
	{
		Templates[TemplateIndex] = Template;
	}
	Templates[TemplateIndex].Index = TemplateIndex;
	Templates[TemplateIndex].OwnerRegistry = this;
	Templates[TemplateIndex].Delegates = InDelegates;
	TemplateNotationToIndex.Add(Notation, TemplateIndex);

	for(int32 ArgumentIndex=0; ArgumentIndex < Templates[TemplateIndex].Arguments.Num(); ArgumentIndex++)
	{
		for(const FRigVMTemplateArgument::ETypeCategory& ArgumentTypeCategory : Templates[TemplateIndex].Arguments[ArgumentIndex].TypeCategories)
		{
			TemplatesPerCategory.FindChecked(ArgumentTypeCategory).AddUnique(TemplateIndex);
		}
	}
	
	return &Templates[TemplateIndex];
}

FRigVMDispatchFactory* FRigVMRegistry_NoLock::FindDispatchFactory_NoLock(const FName& InFactoryName) const
{
	if (FRigVMDispatchFactory* const* ExistingFactory = FactoryNameToFactory.Find(InFactoryName))
	{
		return *ExistingFactory;
	}
	
	if (bIsGlobalRegistry)
	{
		if(InFactoryName.ToString().StartsWith(FRigVMDispatchFactory::DispatchPrefix))
		{
			FRigVMDispatchFactory* FoundFactory = nullptr;
		
			// if the factory has never been registered - we should try to look it up	
			for (TObjectIterator<UScriptStruct> ScriptStructIt; ScriptStructIt; ++ScriptStructIt)
			{
				static const UScriptStruct* SuperStruct = FRigVMDispatchFactory::StaticStruct();

				UScriptStruct* FactoryStruct = *ScriptStructIt;
				if ((FactoryStruct == SuperStruct) || !FactoryStruct->IsChildOf(SuperStruct))
				{
					continue;
				}

				// for the global registry make sure to register all factories
				FRigVMRegistry_NoLock* MutableThis = const_cast<FRigVMRegistry_NoLock*>(this);
				FRigVMDispatchFactory* Factory = const_cast<FRigVMDispatchFactory*>(MutableThis->RegisterFactory_NoLock(FactoryStruct));

				if (Factory && !FoundFactory && Factory->GetFactoryName().IsEqual(InFactoryName, ENameCase::IgnoreCase))
				{
					FoundFactory = Factory;
				}
			}

			return FoundFactory;
		}
	}
	
	return nullptr;
}

FRigVMDispatchFactory* FRigVMRegistry_NoLock::FindOrAddDispatchFactory_NoLock(UScriptStruct* InFactoryStruct)
{
	return const_cast<FRigVMDispatchFactory*>(RegisterFactory_NoLock(InFactoryStruct));
}

FString FRigVMRegistry_NoLock::FindOrAddSingletonDispatchFunction_NoLock(UScriptStruct* InFactoryStruct)
{
	if(const FRigVMDispatchFactory* Factory = FindOrAddDispatchFactory_NoLock(InFactoryStruct))
	{
		if(Factory->IsSingleton())
		{
			if(const FRigVMTemplate* Template = Factory->GetTemplate_NoLock(ThisHandle))
			{
				// use the types for the first permutation - since we don't care
				// for a singleton dispatch
				const FRigVMTemplateTypeMap TypesForPrimaryPermutation = Template->GetTypesForPermutation_NoLock(0, ThisHandle);
				const FString Name = Factory->GetPermutationName_NoLock(TypesForPrimaryPermutation, ThisHandle);
				if(const FRigVMFunction* Function = FindFunction_NoLock(*Name))
				{
					return Function->Name;
				}
			}
		}
	}
	return FString();
}

const TArray<FRigVMDispatchFactory*>& FRigVMRegistry_NoLock::GetFactories_NoLock() const
{	return Factories;
}

const TArray<FRigVMFunction>* FRigVMRegistry_NoLock::GetPredicatesForStruct_NoLock(const FName& InStructName) const
{
	return StructNameToPredicates.Find(InStructName);
}

FRigVMRegistry_RWLock::FRigVMRegistry_RWLock(bool bInIsGlobalRegistry)
	: FRigVMRegistry_NoLock(bInIsGlobalRegistry)
{
	FRigVMRegistryWriteLock WriteLock(*this);
	Initialize_NoLock();
}

void FRigVMRegistry_RWLock::Initialize_NoLock()
{
	FRigVMRegistry_NoLock::InitializeBaseTypes_NoLock();
	
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FRigVMRegistry_RWLock::OnAssetRemoved);
	AssetRegistryModule.Get().OnAssetRenamed().AddRaw(this, &FRigVMRegistry_RWLock::OnAssetRenamed);

	if (bIsGlobalRegistry)
	{
		IPluginManager::Get().OnNewPluginMounted().AddRaw(this, &FRigVMRegistry_RWLock::OnPluginLoaded);
		IPluginManager::Get().OnPluginUnmounted().AddRaw(this, &FRigVMRegistry_RWLock::OnPluginUnloaded);
		FModuleManager::Get().OnModulesUnloaded().AddRaw(this, &FRigVMRegistry_RWLock::OnModulesUnloaded);
	}
	
	UE::Anim::AttributeTypes::GetOnAttributeTypesChanged().AddRaw(this, &FRigVMRegistry_RWLock::OnAnimationAttributeTypesChanged);

	FCoreDelegates::CleanupUnloadingObjects.AddRaw(this, &FRigVMRegistry_RWLock::OnCleanupUnloadingObjects);
}

void FRigVMRegistry_RWLock::EnsureLocked(ELockType InLockType)
{
	check(InLockType != LockType_Invalid);

	const FRigVMRegistry_RWLock& Registry = Get();
	ELockType CurrentLockType = LockType_Invalid;

	UE_AUTORTFM_OPEN
	{
		CurrentLockType = Registry.LockType.load();
	};

	switch (InLockType)
	{
		case LockType_Read:
		{
			ensureMsgf(
				(CurrentLockType == LockType_Read) ||
				(CurrentLockType == LockType_Write),
				TEXT("The Registry is not locked for reading yet - access to the NoLock registry is only possible after locking the RWLock registry (by using its public API calls)."));
			break;
		}
		case LockType_Write:
		{
			ensureMsgf(
				(CurrentLockType == LockType_Write),
				TEXT("The Registry is not locked for writing yet - access to the NoLock registry is only possible after locking the RWLock registry (by using its public API calls)."));
			break;
		}
		default:
		{
			break;
		}
	}
}
