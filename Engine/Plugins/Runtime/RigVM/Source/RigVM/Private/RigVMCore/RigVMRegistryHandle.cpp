// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMRegistryHandle.h"
#include "RigVMCore/RigVMRegistry.h"

FRigVMRegistryHandle::FRigVMRegistryHandle(const FRigVMRegistry_NoLock* InRegistry)
	: Registry(const_cast<FRigVMRegistry_NoLock*>(InRegistry))
{
}

FRigVMRegistryHandle::FRigVMRegistryHandle()
	: Registry(nullptr)
{
}

bool FRigVMRegistryHandle::IsValid() const
{
	return Registry != nullptr;
}

FRigVMRegistry_NoLock* FRigVMRegistryHandle::operator->()
{
	return Registry;
}

const FRigVMRegistry_NoLock* FRigVMRegistryHandle::operator->() const
{
	return Registry;
}

FRigVMRegistry_NoLock* FRigVMRegistryHandle::GetRegistry()
{
	return Registry;
}

const FRigVMRegistry_NoLock* FRigVMRegistryHandle::GetRegistry() const
{
	return Registry;
}
