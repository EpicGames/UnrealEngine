// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMStringUtils.h"
#include "UObject/StructOnScope.h"
#include "RigVMModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMStruct)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRigVMUnitNodeCreatedContext::IsValid() const
{
	return Reason != ERigVMNodeCreatedReason::Unknown &&
		AllExternalVariablesDelegate.IsBound() &&
		CreateExternalVariableDelegate.IsBound() &&
		BindPinToExternalVariableDelegate.IsBound();
}

TArray<FRigVMExternalVariable> FRigVMUnitNodeCreatedContext::GetExternalVariables() const
{
	TArray<FRigVMExternalVariable> ExternalVariables;

	if (AllExternalVariablesDelegate.IsBound())
	{
		ExternalVariables = AllExternalVariablesDelegate.Execute();
	}

	return ExternalVariables;
}

FName FRigVMUnitNodeCreatedContext::AddExternalVariable(const FRigVMExternalVariable& InVariableToCreate, FString InDefaultValue)
{
	if (CreateExternalVariableDelegate.IsBound())
	{
		return CreateExternalVariableDelegate.Execute(InVariableToCreate, InDefaultValue);
	}
	return NAME_None;
}

bool FRigVMUnitNodeCreatedContext::BindPinToExternalVariable(FString InPinPath, FString InVariablePath)
{
	if (BindPinToExternalVariableDelegate.IsBound())
	{
		const FString NodePinPath = FString::Printf(TEXT("%s.%s"), *NodeName.ToString(), *InPinPath);
		return BindPinToExternalVariableDelegate.Execute(NodePinPath, InVariablePath);
	}
	return false;
}

FRigVMExternalVariable FRigVMUnitNodeCreatedContext::FindVariable(FName InVariableName) const
{
	TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
	for (FRigVMExternalVariable ExternalVariable : ExternalVariables)
	{
		if (ExternalVariable.GetName() == InVariableName)
		{
			return ExternalVariable;
		}
	}
	return FRigVMExternalVariable();
}

FName FRigVMUnitNodeCreatedContext::FindFirstVariableOfType(FName InCPPTypeName) const
{
	TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
	for (FRigVMExternalVariable ExternalVariable : ExternalVariables)
	{
		if (ExternalVariable.GetExtendedCPPType() == InCPPTypeName)
		{
			return ExternalVariable.GetName();
		}
	}
	return NAME_None;
}

FName FRigVMUnitNodeCreatedContext::FindFirstVariableOfType(UObject* InCPPTypeObject) const
{
	TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
	for (FRigVMExternalVariable ExternalVariable : ExternalVariables)
	{
		if (ExternalVariable.GetCPPTypeObject() == InCPPTypeObject)
		{
			return ExternalVariable.GetName();
		}
	}
	return NAME_None;
}

FRigVMStructUpgradeInfo::FRigVMStructUpgradeInfo()
	: NodePath()
	, OldStruct(nullptr)
	, NewStruct(nullptr)
{
}

FRigVMStructUpgradeInfo FRigVMStructUpgradeInfo::MakeFromStructToFactory(UScriptStruct* InRigVMStruct, UScriptStruct* InFactoryStruct, const TMap<FName,FName>& InArgumentNameMap)
{
	check(InRigVMStruct);
	check(InFactoryStruct);

	FRigVMRegistry& Registry = FRigVMRegistry::Get();
	const FName FactoryName = FRigVMDispatchFactory::GetFactoryName(InFactoryStruct);
	const FRigVMDispatchFactory* Factory = Registry.FindDispatchFactory(FactoryName);
	if (!Factory)
	{
		Factory = Registry.FindOrAddDispatchFactory(InFactoryStruct);
		check(Factory);
	}
	FRigVMTemplateTypeMap Types = GetTypeMapFromStruct(InRigVMStruct);
	{
		TArray<FRigVMTemplateArgumentInfo> Infos;
		{
			FRigVMRegistryReadLock ReadLock(Registry);
			Infos = Factory->GetArgumentInfos(ReadLock);
		}

		// if we have only one argument - it's quite possible we'll have to remap the argument name.
		if(Types.Num() == 1 && Infos.Num() == 1)
		{
			FRigVMTemplateTypeMap NewTypes;
			NewTypes.Add(Infos[0].Name, Types.FindArbitraryElement()->Value);
			Swap(NewTypes, Types);
		}

		// if we have more arguments or a mismatch - we need to rely on the name map
		else
		{
			FRigVMTemplateTypeMap NewTypes;
			for(const FRigVMTemplateArgumentInfo& Info : Infos)
			{
				FName ArgumentName = Info.Name;
				if(const FName* MappedArgumentName = InArgumentNameMap.FindKey(ArgumentName))
				{
					ArgumentName = *MappedArgumentName;
				}
				
				if(const TRigVMTypeIndex* TypeIndex = Types.Find(ArgumentName))
				{
					NewTypes.Add(Info.Name, *TypeIndex);
				}
				else
				{
					UE_LOGF(
						LogRigVM,
						Error, 
						"Cannot upgrade node using '%ls' to dispatch '%ls' - type for argument '%ls' cannot be determined.\nYou may need to provide the InArgumentNameMap.",
						*InRigVMStruct->GetName(),
						*InFactoryStruct->GetName(),
						*Info.Name.ToString()
					);
					return FRigVMStructUpgradeInfo();
				}
			}
			Swap(NewTypes, Types);
		}
	}
	
	const FString PermutationName = Factory->GetPermutationName(Types);

	FRigVMStructUpgradeInfo Info;
	Info.OldStruct = InRigVMStruct;
	Info.NewStruct = InFactoryStruct;
	Info.NewDispatchFunction = *PermutationName;

	for(const TPair<FName, FName>& Pair : InArgumentNameMap)
	{
		Info.AddRemappedPin(Pair.Key.ToString(), Pair.Value.ToString());
	}
	
	return Info;
}

FRigVMTemplateTypeMap FRigVMStructUpgradeInfo::GetTypeMapFromStruct(UScriptStruct* InScriptStruct)
{
	FRigVMTemplateTypeMap Types;
#if WITH_EDITOR
	for (TFieldIterator<FProperty> It(InScriptStruct); It; ++It)
	{
		if(It->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		if(!It->HasMetaData(FRigVMStruct::InputMetaName) &&
			!It->HasMetaData(FRigVMStruct::OutputMetaName) &&
			!It->HasMetaData(FRigVMStruct::VisibleMetaName))
		{
			continue;
		}
		
		const FName PropertyName = It->GetFName();
		const TRigVMTypeIndex TypeIndex = FRigVMTemplateArgument::Make(*It).GetTypeIndex(0);
		Types.Add(PropertyName, TypeIndex);
	}
#endif
	return Types;
}

bool FRigVMStructUpgradeInfo::IsValid() const
{
	return NodePath.IsEmpty() && (OldStruct != nullptr) && (NewStruct != nullptr);
}

const FString& FRigVMStructUpgradeInfo::GetDefaultValueForPin(const FName& InPinName) const
{
	static const FString EmptyString = FString();
	if(const FString* DefaultValue = DefaultValues.Find(InPinName))
	{
		return *DefaultValue;
	}
	return EmptyString;
}

void FRigVMStructUpgradeInfo::SetDefaultValueForPin(const FName& InPinName, const FString& InDefaultValue)
{
	DefaultValues.FindOrAdd(InPinName) = InDefaultValue;
}

void FRigVMStructUpgradeInfo::AddRemappedPin(const FString& InOldPinPath, const FString& InNewPinPath, bool bAsInput,
	bool bAsOutput)
{
	if(bAsInput)
	{
		InputLinkMap.Add(InOldPinPath, InNewPinPath);
	}
	if(bAsOutput)
	{
		OutputLinkMap.Add(InOldPinPath, InNewPinPath);
	}
}

FString FRigVMStructUpgradeInfo::RemapPin(const FString& InPinPath, bool bIsInput, bool bContainsNodeName) const
{
	FString NodeName;
	FString PinPath = InPinPath;

	if(bContainsNodeName)
	{
		if(!PinPath.Split(TEXT("."), &NodeName, &PinPath, ESearchCase::IgnoreCase, ESearchDir::FromStart))
		{
			return InPinPath;
		}
	}

	const TMap<FString, FString>& LinkMap = bIsInput ? InputLinkMap : OutputLinkMap;

	int32 FoundMaxLength = 0;
	FString FoundReplacement;

	for(const TPair<FString, FString>& Pair : LinkMap)
	{
		if(Pair.Key == PinPath)
		{
			PinPath = Pair.Value;
			FoundReplacement.Reset();
			break;
		}

		const FString KeyWithPeriod = Pair.Key + TEXT(".");
		if(PinPath.StartsWith(KeyWithPeriod))
		{
			if(FoundMaxLength < KeyWithPeriod.Len())
			{
				FoundMaxLength = KeyWithPeriod.Len();
				FoundReplacement = Pair.Value + PinPath.RightChop(Pair.Key.Len());
			}
		}
	}

	if(!FoundReplacement.IsEmpty())
	{
		PinPath = FoundReplacement;
	}

	if(bContainsNodeName)
	{
		PinPath = FString::Printf(TEXT("%s.%s"), *NodeName, *PinPath);
	}

	return PinPath;
}

FString FRigVMStructUpgradeInfo::AddAggregatePin(FString InPinName)
{
	if(InPinName.IsEmpty())
	{
		FString LastPinName;
		
		FStructOnScope StructOnScope(GetNewStruct());
		const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope.GetStructMemory();

		if(AggregatePins.IsEmpty())
		{
#if WITH_EDITOR
			FString LastInputPinName;
			FString LastOutputPinName;
			for (TFieldIterator<FProperty> It(GetNewStruct()); It; ++It)
			{
				if(It->HasMetaData(FRigVMStruct::AggregateMetaName))
				{
					FString& LastDeterminedPinName = It->HasMetaData(FRigVMStruct::InputMetaName) ?
						LastInputPinName : LastOutputPinName;

					if(!LastDeterminedPinName.IsEmpty())
					{
						LastPinName = It->GetName();
						break;
					}

					LastDeterminedPinName = It->GetName();
				}
			}
#else
			LastPinName = TEXT("B");
#endif
		}
		else
		{
			LastPinName = AggregatePins.Last();
		}

		InPinName = StructMemory->GetNextAggregateName(*LastPinName).ToString();
	}
	AggregatePins.Add(InPinName);
	return AggregatePins.Last();
}

void FRigVMStructUpgradeInfo::SetDefaultValues(const void* InNewStructMemory)
{
	check(NewStruct);
	check(InNewStructMemory);
	DefaultValues = FRigVMStruct::GetDefaultValues(InNewStructMemory, NewStruct);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

float FRigVMStruct::GetRatioFromIndex(int32 InIndex, int32 InCount)
{
	if (InCount <= 1)
	{
		return 0.f;
	}
	return ((float)FMath::Clamp<int32>(InIndex, 0, InCount - 1)) / ((float)(InCount - 1));
}

#if WITH_EDITORONLY_DATA

bool FRigVMStruct::IsForLoop() const
{
	UE_LOGF(LogRigVM, Warning, "FRigVMStruct::IsForLoop is deprecated. Please call FRigVMStruct::IsControlFlowNode instead.");
	
	if(IsControlFlowNode())
	{
		static const TArray<FName> ExpectedLoopBlocks = {ExecuteContextName, ForLoopCompletedPinName};
		const TArray<FName>& Blocks = GetControlFlowBlocks();
		if(Blocks.Num() == ExpectedLoopBlocks.Num())
		{
			return Blocks[0] == ExpectedLoopBlocks[0] && Blocks[1] == ExpectedLoopBlocks[1];
		}
	}
	
	return false;
}

bool FRigVMStruct::IsControlFlowNode() const
{
	return !GetControlFlowBlocks().IsEmpty();
}

const TArray<FName>& FRigVMStruct::GetControlFlowBlocks() const
{
	const TArray<FName>& Blocks = GetControlFlowBlocks_Impl();
#if WITH_EDITOR
	ValidateControlFlowBlocks(Blocks);
#endif
	return Blocks;
}
#endif

TArray<FRigVMUserWorkflow> FRigVMStruct::GetWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const
{
	return GetSupportedWorkflows(InSubject).FilterByPredicate([InType](const FRigVMUserWorkflow& InWorkflow) -> bool
	{
		return uint32(InWorkflow.GetType()) & uint32(InType) &&
			InWorkflow.IsValid();
	});
}

#if WITH_EDITOR

bool FRigVMStruct::ValidateStruct(UScriptStruct* InStruct, FString* OutErrorMessage)
{
	if (!InStruct->IsChildOf(StaticStruct()))
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = TEXT("Not a child of FRigVMStruct.");
		}
		return false;
	}

	FStructOnScope StructOnScope(InStruct);
	FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope.GetStructMemory();

	if (StructMemory->IsControlFlowNode())
	{
		// first check that the Block To Run pin exists
		if (!CheckPinExists(InStruct, ControlFlowBlockToRunName, TEXT("FName"), OutErrorMessage))
		{
			return false;
		}

		// check that the block to run name is marked up as a singleton
		if (!CheckMetadata(InStruct, ControlFlowBlockToRunName, SingletonMetaName, OutErrorMessage))
		{
			return false;
		}

		if (!CheckPinDirection(InStruct, ControlFlowBlockToRunName, HiddenMetaName))
		{
			if (OutErrorMessage)
			{
				*OutErrorMessage = FString::Printf(TEXT("The '%s' pin needs to be hidden (not an input nor an output)."), *ControlFlowBlockToRunName.ToString());
			}
			return false;
		}

		// the call to GetControlFlowBlocks will already ensure that the block names
		// themselves are valid.
		const TArray<FName>& Blocks = StructMemory->GetControlFlowBlocks();
		for(const FName& Block : Blocks)
		{
			if (!CheckPinExists(InStruct, Block, FRigVMExecuteContext::StaticStruct()->GetStructCPPName(), OutErrorMessage))
			{
				return false;
			}
			if (!CheckPinDirection(InStruct, Block, OutputMetaName) &&
				!CheckPinDirection(InStruct, Block, IOMetaName))
			{
				if (OutErrorMessage)
				{
					*OutErrorMessage = FString::Printf(TEXT("The '%s' pin needs to be either an IO or an output."), *Block.ToString());
				}
				return false;
			}

			if(StructMemory->IsControlFlowBlockSliced(Block))
			{
				if(Block == ControlFlowCompletedName)
				{
					if (OutErrorMessage)
					{
						*OutErrorMessage = FString::Printf(TEXT("The '%s' control flow block cannot be sliced."), *Block.ToString());
					}
					return false;
				}
				if (!CheckPinExists(InStruct, ControlFlowCountName, TEXT("int32"), OutErrorMessage))
				{
					return false;
				}
				if (!CheckPinDirection(InStruct, ControlFlowCountName, OutputMetaName) &&
					!CheckPinDirection(InStruct, ControlFlowCountName, IOMetaName) &&
					!CheckPinDirection(InStruct, ControlFlowCountName, HiddenMetaName) &&
					!CheckPinDirection(InStruct, ControlFlowCountName, InputMetaName))
				{
					if (OutErrorMessage)
					{
						*OutErrorMessage = FString::Printf(TEXT("The '%s' pin needs to be either an IO, input, output or hidden."), *ControlFlowCountName.ToString());
					}
					return false;
				}

				if (!CheckPinExists(InStruct, ControlFlowIndexName, TEXT("int32"), OutErrorMessage))
				{
					return false;
				}
				if (!CheckPinDirection(InStruct, ControlFlowIndexName, OutputMetaName) &&
					!CheckPinDirection(InStruct, ControlFlowIndexName, IOMetaName) &&
					!CheckPinDirection(InStruct, ControlFlowIndexName, HiddenMetaName))
				{
					if (OutErrorMessage)
					{
						*OutErrorMessage = FString::Printf(TEXT("The '%s' pin needs to be either an IO, an output or hidden."), *ControlFlowIndexName.ToString());
					}
					return false;
				}
			}
		}
	}

	return true;
}

bool FRigVMStruct::CheckPinDirection(UScriptStruct* InStruct, const FName& PinName, const FName& InDirectionMetaName)
{
	if (FProperty* Property = InStruct->FindPropertyByName(PinName))
	{
		if (InDirectionMetaName == IOMetaName)
		{
			return Property->HasMetaData(InputMetaName) && Property->HasMetaData(OutputMetaName);
		}
		else if (InDirectionMetaName == HiddenMetaName)
		{
			return !Property->HasMetaData(InputMetaName) && !Property->HasMetaData(OutputMetaName);
		}
		return Property->HasMetaData(InDirectionMetaName);
	}
	return true;
}

bool FRigVMStruct::CheckPinType(UScriptStruct* InStruct, const FName& PinName, const FString& ExpectedType, FString* OutErrorMessage)
{
	if (const FProperty* Property = InStruct->FindPropertyByName(PinName))
	{
		if (Property->GetCPPType() != ExpectedType)
		{
			if(ExpectedType == FRigVMExecuteContext::StaticStruct()->GetStructCPPName() ||
				ExpectedType == FRigVMExecutePin::StaticStruct()->GetStructCPPName())
			{
				if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					if(StructProperty->Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
					{
						return true;
					}
				}
			}
			if (OutErrorMessage)
			{
				*OutErrorMessage = FString::Printf(TEXT("The '%s' property needs to be of type '%s'."), *PinName.ToString(), *ExpectedType);;
			}
			return false;
		}
	}

	return true;
}

bool FRigVMStruct::CheckPinExists(UScriptStruct* InStruct, const FName& PinName, const FString& ExpectedType, FString* OutErrorMessage)
{
	if (InStruct->FindPropertyByName(PinName))
	{
		if (!ExpectedType.IsEmpty())
		{
			if(!CheckPinType(InStruct, PinName, ExpectedType, OutErrorMessage))
			{
				return false;
			}
		}
	}
	else
	{
		if (OutErrorMessage)
		{
			if (ExpectedType.IsEmpty())
			{
				*OutErrorMessage = FString::Printf(TEXT("Struct requires a '%s' property."), *PinName.ToString());;
			}
			else
			{
				*OutErrorMessage = FString::Printf(TEXT("Struct requires a '%s' property of type '%s'."), *PinName.ToString(), *ExpectedType);;
			}
		}
		return false;
	}

	return true;
}

bool FRigVMStruct::CheckMetadata(UScriptStruct* InStruct, const FName& PinName, const FName& InMetadataKey, FString* OutErrorMessage)
{
	if (FProperty* Property = InStruct->FindPropertyByName(PinName))
	{
		if (!Property->HasMetaData(InMetadataKey))
		{
			if (OutErrorMessage)
			{
				*OutErrorMessage = FString::Printf(TEXT("Property '%s' requires a '%s' metadata tag."), *PinName.ToString(), *InMetadataKey.ToString());;
			}
			return false;
		}
	}
	else
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = FString::Printf(TEXT("Struct requires a '%s' property."), *PinName.ToString());;
		}
		return false;
	}

	return true;
}

bool FRigVMStruct::CheckFunctionExists(UScriptStruct* InStruct, const FName& FunctionName, FString* OutErrorMessage)
{
	FString Key = FString::Printf(TEXT("%s::%s"), *InStruct->GetStructCPPName(), *FunctionName.ToString());
	if (FRigVMRegistry::Get().FindFunction(*Key) == nullptr)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = FString::Printf(TEXT("Function '%s' not found, required for this type of struct."), *Key);
		}
		return false;
	}
	return true;
}

bool FRigVMStruct::ShouldCreatePinForProperty(const FProperty* InProperty) const
{
	return true;
}

ERigVMPinDirection FRigVMStruct::GetPinDirectionFromProperty(FProperty* InProperty)
{
	const bool bIsInput = InProperty->HasMetaData(InputMetaName);
	const bool bIsOutput = InProperty->HasMetaData(OutputMetaName);
	const bool bIsIO = InProperty->HasMetaData(IOMetaName);
	const bool bIsVisible = InProperty->HasMetaData(VisibleMetaName);
	const bool bIsExternalVariable = InProperty->HasMetaData(ExternalVariableMetaName);

	// Validate ExternalVariable usage: only valid as (Input, ExternalVariable) — no other combinations.
	if (bIsExternalVariable)
	{
		if (bIsIO)
		{
			// IO already implies mutability; ExternalVariable is redundant and confusing.
			UE_LOGF(LogRigVM, Warning,
				"Property '%ls' has ExternalVariable meta combined with IO — IO already implies mutability. Treating as Hidden.",
				*InProperty->GetName());
			return ERigVMPinDirection::Hidden;
		}
		if (bIsOutput)
		{
			// Output (or Input+Output) is incompatible with ExternalVariable.
			UE_LOGF(LogRigVM, Warning,
				"Property '%ls' has ExternalVariable meta combined with Output — ExternalVariable is not compatible with Output. Treating as Hidden.",
				*InProperty->GetName());
			return ERigVMPinDirection::Hidden;
		}
		if (!bIsInput)
		{
			// ExternalVariable requires Input — standalone ExternalVariable is invalid.
			UE_LOGF(LogRigVM, Warning,
				"Property '%ls' has ExternalVariable meta without Input — ExternalVariable requires Input. Treating as Hidden.",
				*InProperty->GetName());
			return ERigVMPinDirection::Hidden;
		}
		if (InProperty->HasMetaData(ConstantMetaName))
		{
			// Constant forces immutability, which contradicts ExternalVariable's mutable-reference purpose.
			UE_LOGF(LogRigVM, Warning,
				"Property '%ls' has ExternalVariable meta combined with Constant — these are contradictory. Treating as Hidden.",
				*InProperty->GetName());
			return ERigVMPinDirection::Hidden;
		}

		// Valid case: (Input, ExternalVariable) — mutable input passed by non-const reference.
		return ERigVMPinDirection::Input;
	}

	if (bIsVisible)
	{
		return ERigVMPinDirection::Visible;
	}

	if (bIsInput)
	{
		return bIsOutput ? ERigVMPinDirection::IO : ERigVMPinDirection::Input;
	}

	if(bIsOutput)
	{
		return ERigVMPinDirection::Output;
	}

	return ERigVMPinDirection::Hidden;
}

#endif

FString FRigVMStruct::ExportToFullyQualifiedText(const FProperty* InMemberProperty, const uint8* InMemberMemoryPtr, bool bUseQuotes)
{
	check(InMemberProperty);
	check(InMemberMemoryPtr);

	FString DefaultValue;

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InMemberProperty))
	{
		DefaultValue = ExportToFullyQualifiedText(StructProperty->Struct, InMemberMemoryPtr);
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InMemberProperty))
	{
		FScriptArrayHelper ScriptArrayHelper(ArrayProperty, InMemberMemoryPtr);

		TArray<FString> ElementValues;
		for (int32 ElementIndex = 0; ElementIndex < ScriptArrayHelper.Num(); ElementIndex++)
		{
			const uint8* ElementMemoryPtr = ScriptArrayHelper.GetRawPtr(ElementIndex);
			ElementValues.Add(ExportToFullyQualifiedText(ArrayProperty->Inner, ElementMemoryPtr));
		}

		DefaultValue = RigVMStringUtils::JoinStrings(ElementValues, TEXT(","), TEXT("("), TEXT(")"));
	}
	else
	{
		// Note we pass PPF_Delimited here to correctly escape strings
		InMemberProperty->ExportTextItem_Direct(DefaultValue, InMemberMemoryPtr, nullptr, nullptr, PPF_Delimited);

		if (CastField<FNameProperty>(InMemberProperty) != nullptr ||
			CastField<FStrProperty>(InMemberProperty) != nullptr ||
			CastField<FTextProperty>(InMemberProperty) != nullptr)
		{
			if(bUseQuotes)
			{
				if (DefaultValue.IsEmpty())
				{
					DefaultValue = TEXT("\"\"");
				}
				else if (DefaultValue[0] != TCHAR('\"') && DefaultValue[DefaultValue.Len() - 1] != TCHAR('\"'))
				{
					DefaultValue = FString::Printf(TEXT("\"%s\""), *DefaultValue);
				}
			}
			else
			{
				DefaultValue = DefaultValue.TrimQuotes();
			}
		}
	}

	return DefaultValue;
}

FString FRigVMStruct::ExportToFullyQualifiedText(const UScriptStruct* InStruct, const uint8* InStructMemoryPtr, bool bUseQuotes)
{
	check(InStruct);
	check(InStructMemoryPtr);

	// If a struct has a custom text import/export, we must use its ExportText call rather than going per-property
	if((InStruct->StructFlags & (STRUCT_ImportTextItemNative | STRUCT_ExportTextItemNative)) != 0)
	{
		FInstancedStruct Defaults;
		Defaults.InitializeAs(InStruct);
		FString Value;
		InStruct->ExportText(Value, InStructMemoryPtr, Defaults.GetMemory(), nullptr, PPF_None, nullptr);
		return Value;
	}
	else
	{
		TArray<FString> FieldValues;
		for (TFieldIterator<FProperty> It(InStruct); It; ++It)
		{
			if(It->HasAnyPropertyFlags(CPF_Transient))
			{
				continue;
			}
		
			FString PropertyName = It->GetName();
			const uint8* StructMemberMemoryPtr = It->ContainerPtrToValuePtr<uint8>(InStructMemoryPtr);
			FString DefaultValue = ExportToFullyQualifiedText(*It, StructMemberMemoryPtr, bUseQuotes);
			FieldValues.Add(FString::Printf(TEXT("%s=%s"), *PropertyName, *DefaultValue));
		}

		return RigVMStringUtils::JoinStrings(FieldValues, TEXT(","), TEXT("("), TEXT(")"));
	}
}

FString FRigVMStruct::ExportToFullyQualifiedText(const UScriptStruct* InScriptStruct, const FName& InPropertyName, const uint8* InStructMemoryPointer, bool bUseQuotes) const
{
	check(InScriptStruct);
	if(InStructMemoryPointer == nullptr)
	{
		InStructMemoryPointer = (const uint8*)this;
	}
	
	const FProperty* Property = InScriptStruct->FindPropertyByName(InPropertyName);
	if(Property == nullptr)
	{
		return FString();
	}

	const uint8* StructMemberMemoryPtr = Property->ContainerPtrToValuePtr<uint8>(InStructMemoryPointer);
	return ExportToFullyQualifiedText(Property, StructMemberMemoryPtr, bUseQuotes);
}

FName FRigVMStruct::GetNextAggregateName(const FName& InLastAggregatePinName) const
{
	if(InLastAggregatePinName.IsNone())
	{
		return InLastAggregatePinName;
	}

	const FString PinName = InLastAggregatePinName.ToString();
	if(PinName.IsNumeric())
	{
		const int32 Index = FCString::Atoi(*PinName);
		return *FString::FormatAsNumber(Index+1);
	}

	if(PinName.Len() == 1)
	{
		const TCHAR C = PinName[0];
		if((C >='a' && C < 'z') || (C >='A' && C <'Z'))
		{
			FString Result;
			Result.AppendChar(C + 1);
			return *Result;
		}
	}

	if(PinName.Contains(TEXT("_")))
	{
		const FString Left = PinName.Left(PinName.Find(TEXT("_"), ESearchCase::IgnoreCase, ESearchDir::FromEnd));
		const FString Right = PinName.Mid(PinName.Find(TEXT("_"), ESearchCase::IgnoreCase, ESearchDir::FromEnd)+1);
		return *FString::Printf(TEXT("%s_%s"), *Left, *GetNextAggregateName(*Right).ToString());
	}

	return *FString::Printf(TEXT("%s_1"), *PinName);
}

TMap<FName, FString> FRigVMStruct::GetDefaultValues(UScriptStruct* InScriptStruct) const
{
	return GetDefaultValues(this, InScriptStruct);
}

TMap<FName, FString> FRigVMStruct::GetDefaultValues(const void* InValue, UScriptStruct* InScriptStruct)
{
	check(InScriptStruct);

	TMap<FName, FString> DefaultValues;
	// If a struct has a custom text import/export, we cannot extract its property values individually
	if((InScriptStruct->StructFlags & (STRUCT_ImportTextItemNative | STRUCT_ExportTextItemNative)) == 0)
	{
		for (TFieldIterator<FProperty> It(InScriptStruct); It; ++It)
		{
			if(It->HasAnyPropertyFlags(CPF_Transient))
			{
				continue;
			}

#if WITH_EDITOR
			if(!It->HasMetaData(FRigVMStruct::InputMetaName) &&
				!It->HasMetaData(FRigVMStruct::OutputMetaName) &&
				!It->HasMetaData(FRigVMStruct::VisibleMetaName))
			{
				continue;
			}
#endif
			
			const FName PropertyName = It->GetFName();
			const uint8* StructMemberMemoryPtr = It->ContainerPtrToValuePtr<uint8>(InValue);
			const FString DefaultValue = ExportToFullyQualifiedText(*It, StructMemberMemoryPtr, false);
			DefaultValues.Add(PropertyName, DefaultValue);
		}
	}

	return DefaultValues;
}

class FRigVMStructApplyUpgradeInfoErrorContext : public FOutputDevice
{
public:

	int32 NumErrors;
	UScriptStruct* OldStruct;
	UScriptStruct* NewStruct;
	FName PropertyName;

	FRigVMStructApplyUpgradeInfoErrorContext(UScriptStruct* InOldStruct, UScriptStruct* InNewStruct, const FName& InPropertyName)
		: FOutputDevice()
		, NumErrors(0)
		, OldStruct(InOldStruct)
		, NewStruct(InNewStruct)
		, PropertyName(InPropertyName)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if(Verbosity == ELogVerbosity::Error)
		{
			UE_LOGF(LogRigVM, Warning, "Error when applying upgrade data from %ls to %ls, property %ls: %ls",
				*OldStruct->GetStructCPPName(),
				*NewStruct->GetStructCPPName(),
				*PropertyName.ToString(),
				V);
			NumErrors++;
		}
		else if(Verbosity == ELogVerbosity::Warning)
		{
			UE_LOGF(LogRigVM, Warning, "Warning when applying upgrade data from %ls to %ls, property %ls: %ls",
				*OldStruct->GetStructCPPName(),
				*NewStruct->GetStructCPPName(),
				*PropertyName.ToString(),
				V);
		}
	}
};

bool FRigVMStruct::ApplyUpgradeInfo(const FRigVMStructUpgradeInfo& InUpgradeInfo)
{
	check(InUpgradeInfo.IsValid());

	for(const TPair<FName, FString>& Pair : InUpgradeInfo.GetDefaultValues())
	{
		const FName& PropertyName = Pair.Key;
		const FString& DefaultValue = Pair.Value;
		
		const FProperty* Property = InUpgradeInfo.GetNewStruct()->FindPropertyByName(PropertyName);
		if(Property == nullptr)
		{
			return false;
		}

		uint8* MemberMemory = Property->ContainerPtrToValuePtr<uint8>(this);

		FRigVMStructApplyUpgradeInfoErrorContext ErrorPipe(InUpgradeInfo.GetOldStruct(), InUpgradeInfo.GetNewStruct(), PropertyName);
        Property->ImportText_Direct(*DefaultValue, MemberMemory, nullptr, PPF_None, &ErrorPipe);

		if(ErrorPipe.NumErrors > 0)
		{
			return false;
		}
	}

	return true;
}

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMStruct::GetControlFlowBlocks_Impl() const
{
	static const TArray<FName> EmptyArray;
	return EmptyArray;
}
#endif

#if WITH_EDITOR

void FRigVMStruct::ValidateControlFlowBlocks(const TArray<FName>& InBlocks)
{
	if(!InBlocks.IsEmpty())
	{
		check(InBlocks.Num() > 0); // At least the Completed block
		check(InBlocks.Last() == ControlFlowCompletedName);

		TArray<FName> UniqueBlocks;
		UniqueBlocks.Reserve(InBlocks.Num());
		for(const FName& Block : InBlocks)
		{
			UniqueBlocks.AddUnique(Block);
		}
		check(UniqueBlocks.Num() == InBlocks.Num());
	}
}

#endif