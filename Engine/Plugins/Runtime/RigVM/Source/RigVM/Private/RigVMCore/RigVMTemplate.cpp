// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMTemplate.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMModule.h"
#include "RigVMStringUtils.h"
#include "Algo/Accumulate.h"
#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Async/ParallelFor.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTemplate)

////////////////////////////////////////////////////////////////////////////////////////////////

FRigVMTemplateArgumentType::FRigVMTemplateArgumentType(const FName& InCPPType, UObject* InCPPTypeObject, bool bIsSerializing)
		: CPPType(InCPPType)
		, CPPTypeObject(InCPPTypeObject)
{
	// InCppType is unreliable because not all caller knows that
	// we use generated unique names for user defined structs
	// so here we override the CppType name with the actual name used in the registry
	CPPType = *RigVMTypeUtils::PostProcessCPPType(CPPType.ToString(), CPPTypeObject, nullptr, bIsSerializing);
#if WITH_EDITOR
	if (CPPType.IsNone())
	{
		const FString InCPPTypeString = CPPType.ToString();
		if (const UClass* ObjectClass = InCPPTypeObject ? InCPPTypeObject->GetClass() : nullptr)
		{
			UE_LOGF(LogRigVM, Warning, "FRigVMTemplateArgumentType(): Input CPPType '%ls' (Input Object '%ls') could not be resolved.",
				*InCPPTypeString, *ObjectClass->GetName());
		}
		else
		{
			UE_LOGF(LogRigVM, Warning, "FRigVMTemplateArgumentType(): Input CPPType '%ls' could not be resolved.", *InCPPTypeString);
		}
	}
#endif
}

void FRigVMTemplateArgumentType::Serialize_NoLock(FArchive& Ar)
{
	Ar << CPPType;
	Ar << CPPTypeObject;
}

FRigVMTemplateArgument::FRigVMTemplateArgument()
{}

FRigVMTemplateArgument::FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection)
	: Name(InName)
	, Direction(InDirection)
{}

FRigVMTemplateArgument::FRigVMTemplateArgument(FProperty* InProperty, FRigVMRegistryHandle& InRegistry)
	: Name(InProperty->GetFName())
{
#if WITH_EDITOR
	Direction = FRigVMStruct::GetPinDirectionFromProperty(InProperty);
#endif

	const FName CPPTypeName = *RigVMTypeUtils::GetCPPTypeFromProperty(InProperty);
	UObject* CPPTypeObject = RigVMTypeUtils::GetCPPTypeObjectFromProperty(InProperty);
	
	const FRigVMTemplateArgumentType Type(CPPTypeName, CPPTypeObject);
	const TRigVMTypeIndex TypeIndex = InRegistry->FindOrAddType_NoLock(Type, true); 

	TypeIndices.Add(TypeIndex);
	EnsureValidExecuteType_NoLock(InRegistry);
#if WITH_EDITOR
	UpdateTypeToPermutationsSlow(InRegistry);
#endif
}

FRigVMTemplateArgument FRigVMTemplateArgument::Make(FProperty* InProperty)
{
	FRigVMRegistryReadLock Lock;;
	return Make_NoLock(InProperty, Lock);
}

FRigVMTemplateArgument FRigVMTemplateArgument::Make_NoLock(FProperty* InProperty, FRigVMRegistryHandle& InRegistry)
{
	return FRigVMTemplateArgument(InProperty, InRegistry);
}

FRigVMTemplateArgument::FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, TRigVMTypeIndex InTypeIndex, const FRigVMRegistryHandle& InRegistry)
	: Name(InName)
	, Direction(InDirection)
	, TypeIndices({InTypeIndex})
{
	EnsureValidExecuteType_NoLock(InRegistry);
#if WITH_EDITOR
	UpdateTypeToPermutationsSlow(InRegistry);
#endif
}

FRigVMTemplateArgument::FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices, const FRigVMRegistryHandle& InRegistry)
	: Name(InName)
	, Direction(InDirection)
	, TypeIndices(InTypeIndices)
{
	check(TypeIndices.Num() > 0);
	EnsureValidExecuteType_NoLock(InRegistry);
#if WITH_EDITOR
	UpdateTypeToPermutationsSlow(InRegistry);
#endif
}

FRigVMTemplateArgument::FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, const TArray<ETypeCategory>& InTypeCategories, TFunction<bool(const TRigVMTypeIndex&, const FRigVMRegistryHandle&)> InFilterType, const FRigVMRegistryHandle& InRegistry)
	: Name(InName)
	, Direction(InDirection)
	, TypeCategories(InTypeCategories)
	, FilterType(InFilterType)
{
	const int32 NumCategories = TypeCategories.Num();
	if (NumCategories > 0)
	{
		TSet<TRigVMTypeIndex> AllTypes;

		TArray<int32> NumTypesByCategory, TypesAddedByCategory;
		TypesAddedByCategory.Reserve(NumCategories);
		NumTypesByCategory.Reserve(NumCategories);

		bUseCategories = NumCategories == 1;
		if (!bUseCategories)
		{
			// preallocate AllTypes since it can be a large array
			{
				int32 NumTypes = 0;
				for (const ETypeCategory TypeCategory : TypeCategories)
				{
					NumTypes += InRegistry->GetTypesForCategory_NoLock(TypeCategory).Num();
				}
				AllTypes.Reserve(NumTypes);
			}

			for (const ETypeCategory TypeCategory : TypeCategories)
			{
				const TArray<TRigVMTypeIndex>& Types = InRegistry->GetTypesForCategory_NoLock(TypeCategory);
				
				for (const TRigVMTypeIndex Type: Types)
				{
					AllTypes.Add(Type);
				}
			
				NumTypesByCategory.Add(Types.Num());
				TypesAddedByCategory.Add(AllTypes.Num());
			}
			bUseCategories = NumTypesByCategory[0] == TypesAddedByCategory[0] && Algo::Accumulate(NumTypesByCategory, 0) == AllTypes.Num();
		}

		if (bUseCategories)
		{
			TypeIndices.Reset();
		}
		else
		{
			TArray<TRigVMTypeIndex> Indices = AllTypes.Array();
			if (FilterType)
			{
				Indices = Indices.FilterByPredicate([this, &InRegistry](const TRigVMTypeIndex& Type)
				{
					return FilterType(Type, InRegistry);
				});
			}
			TypeIndices = MoveTemp(Indices);
			EnsureValidExecuteType_NoLock(InRegistry);
		}

#if WITH_EDITOR
		UpdateTypeToPermutationsSlow(InRegistry);
#endif
	}
}

void FRigVMTemplateArgument::EnsureValidExecuteType_NoLock(const FRigVMRegistryHandle& InRegistry)
{
	for(TRigVMTypeIndex& TypeIndex : TypeIndices)
	{
		InRegistry->ConvertExecuteContextToBaseType_NoLock(TypeIndex);
	}
}

#if WITH_EDITOR

void FRigVMTemplateArgument::UpdateTypeToPermutationsSlow(const FRigVMRegistryHandle& InRegistry)
{
	TypeToPermutations.Reset();
	TypeToPermutations.Reserve(GetNumTypes_NoLock(InRegistry));

	int32 TypeIndex = 0;
	ForEachType([&](const TRigVMTypeIndex Type)
	{
		TypeToPermutations.FindOrAdd(Type).Add(TypeIndex++);
		return true;
	}, InRegistry);
}

bool FRigVMTemplateArgument::SupportsTypeIndex(TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex) const
{
	FRigVMRegistryReadLock Lock;;
	return SupportsTypeIndex_NoLock(Lock, InTypeIndex, OutTypeIndex);
}

bool FRigVMTemplateArgument::SupportsTypeIndex_NoLock(const FRigVMRegistryHandle& InRegistry, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex) const
{
	if(InTypeIndex == INDEX_NONE)
	{
		return false;
	}
	
	// convert any execute type into the base execute
	if(InRegistry->IsExecuteType_NoLock(InTypeIndex))
	{
		const bool bIsArray = InRegistry->IsArrayType_NoLock(InTypeIndex);
		InTypeIndex = RigVMTypeUtils::TypeIndex::Execute;
		if(bIsArray)
		{
			InTypeIndex = InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex);
		}
	}

	const TArray<int32>& Permutations = GetPermutations_NoLock(InTypeIndex, InRegistry);
	if (!Permutations.IsEmpty())
	{
		if(OutTypeIndex)
		{
			(*OutTypeIndex) = GetTypeIndex_NoLock(Permutations[0], InRegistry);
		}
		return true;
	}

	// Try to find compatible type
	const TArray<TRigVMTypeIndex>& CompatibleTypes = InRegistry->GetCompatibleTypes_NoLock(InTypeIndex);
	for (const TRigVMTypeIndex& CompatibleTypeIndex : CompatibleTypes)
	{
		const TArray<int32>& CompatiblePermutations = GetPermutations_NoLock(CompatibleTypeIndex, InRegistry);
		if (!CompatiblePermutations.IsEmpty())
		{
			if(OutTypeIndex)
			{
				(*OutTypeIndex) = GetTypeIndex_NoLock(CompatiblePermutations[0], InRegistry);
			}
			return true;
		}
	}

	return false;
}

#endif

bool FRigVMTemplateArgument::IsSingleton(const TArray<int32>& InPermutationIndices) const
{
	FRigVMRegistryReadLock Lock;;
	return IsSingleton_NoLock(InPermutationIndices, Lock);
}

bool FRigVMTemplateArgument::IsSingleton_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const
{
#if WITH_EDITOR
	if (TypeToPermutations.Num() == 1)
	{
		return true;
	}
#endif

	// if a type is using categories it can't be singleton
	// since categories provide more than one type.
	if(bUseCategories)
	{
		return false;
	}

	const bool bUsesPermutations = !InPermutationIndices.IsEmpty(); 
	const int32 NumPermutations = bUsesPermutations ? InPermutationIndices.Num() : GetNumTypes_NoLock(InRegistry);
	const TRigVMTypeIndex InType0 = GetTypeIndex_NoLock(bUsesPermutations ? InPermutationIndices[0] : 0, InRegistry);
	for (int32 PermutationIndex = 1; PermutationIndex < NumPermutations; PermutationIndex++)
	{
		if (GetTypeIndex_NoLock(bUsesPermutations ? InPermutationIndices[PermutationIndex] : PermutationIndex, InRegistry) != InType0)
		{
			return false;
		}
	}

	return true;
}

bool FRigVMTemplateArgument::IsExecute() const
{
	return IsExecute_NoLock(FRigVMRegistryReadLock());
}

bool FRigVMTemplateArgument::IsExecute_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	const int32 FoundAnyNotExec = IndexOfByPredicate([&InRegistry](const TRigVMTypeIndex Type)
	{
		return !InRegistry->IsExecuteType_NoLock(Type);
	}, InRegistry);
	return FoundAnyNotExec == INDEX_NONE;
}

FRigVMTemplateArgument::EArrayType FRigVMTemplateArgument::GetArrayType() const
{
	return GetArrayType_NoLock(FRigVMRegistryReadLock());
}

FRigVMTemplateArgument::EArrayType FRigVMTemplateArgument::GetArrayType_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArrayType.IsSet())
	{
		return CachedArrayType.GetValue();
	}
	
	TArray<TRigVMTypeIndex> Types; GetAllTypes_NoLock(Types, InRegistry);

	const int32 NumTypes = Types.Num();
	if (NumTypes > 0)
	{
		const EArrayType ArrayType = InRegistry->IsArrayType_NoLock(Types[0]) ? EArrayType_ArrayValue : EArrayType_SingleValue;
		
		if(IsSingleton_NoLock({}, InRegistry))
		{
			CachedArrayType = ArrayType;
			return CachedArrayType.GetValue();
		}

		for(int32 PermutationIndex=1; PermutationIndex<NumTypes;PermutationIndex++)
		{
			const TRigVMTypeIndex TypeIndex = Types[PermutationIndex];
			// INDEX_NONE indicates deleted permutation
			if (TypeIndex == INDEX_NONE)
			{
				continue;
			}
			
			const EArrayType OtherArrayType = InRegistry->IsArrayType_NoLock(TypeIndex) ? EArrayType_ArrayValue : EArrayType_SingleValue;
			if(OtherArrayType != ArrayType)
			{
				CachedArrayType = EArrayType_Mixed;
				return CachedArrayType.GetValue();
			}
		}

		CachedArrayType = ArrayType;
		return CachedArrayType.GetValue();
	}

	return EArrayType_Invalid;
}

#if WITH_EDITOR

const TArray<int32>& FRigVMTemplateArgument::GetPermutations(const TRigVMTypeIndex InType) const
{
	return GetPermutations_NoLock(InType, FRigVMRegistryReadLock());
}

const TArray<int32>& FRigVMTemplateArgument::GetPermutations_NoLock(const TRigVMTypeIndex InType, const FRigVMRegistryHandle& InRegistry) const
{
	if (const TArray<int32>* Found = TypeToPermutations.Find(InType))
	{
		return *Found;
	}

	int32 IndexInTypes = 0;
	TArray<int32> Permutations;
	ForEachType([&](const TRigVMTypeIndex Type)
	{
		if (Type == InType)
		{
			Permutations.Add(IndexInTypes);
		}
		IndexInTypes++;
		return true;
	}, InRegistry);

	if (!Permutations.IsEmpty())
	{
		return TypeToPermutations.Emplace(InType, MoveTemp(Permutations));
	}

	static const TArray<int32> Dummy;
	return Dummy;
}

void FRigVMTemplateArgument::InvalidatePermutations(const TRigVMTypeIndex InType)
{
	TypeToPermutations.Remove(InType);
}

#endif

void FRigVMTemplateArgument::GetAllTypes(TArray<TRigVMTypeIndex>& OutTypes) const
{
	GetAllTypes_NoLock(OutTypes, FRigVMRegistryReadLock());
}

void FRigVMTemplateArgument::GetAllTypes_NoLock(TArray<TRigVMTypeIndex>& OutTypes, const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		OutTypes = TypeIndices;
		return;
	}
	
	OutTypes.Reset();
	for (const ETypeCategory Category: TypeCategories)
	{
		if (FilterType == nullptr)
		{
			OutTypes.Append(InRegistry->GetTypesForCategory_NoLock(Category));
		}
		else
		{
			const TArray<TRigVMTypeIndex>& CategoryTypes = InRegistry->GetTypesForCategory_NoLock(Category);
			for (const TRigVMTypeIndex& Type : CategoryTypes)
			{
				if (FilterType(Type, InRegistry))
				{
					OutTypes.Add(Type);
				}
			}
		}
	}
}

TRigVMTypeIndex FRigVMTemplateArgument::GetTypeIndex(const int32 InIndex) const
{
	return GetTypeIndex_NoLock(InIndex, FRigVMRegistryReadLock());
}

TRigVMTypeIndex FRigVMTemplateArgument::GetTypeIndex_NoLock(const int32 InIndex, const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		if (TypeIndices.IsEmpty())
		{
			return INDEX_NONE;
		}
		return TypeIndices.IsValidIndex(InIndex) ? TypeIndices[InIndex] : TypeIndices[0];		
	}

	if (FilterType)
	{
		TRigVMTypeIndex ValidType = INDEX_NONE;
		int32 ValidIndex = 0;
		CategoryViews(TypeCategories, InRegistry).ForEachType([this, &ValidIndex, InIndex, &ValidType, &InRegistry](const TRigVMTypeIndex& Type) -> bool
		{
			if (FilterType(Type, InRegistry))
			{
				if (ValidIndex == InIndex)
				{
					ValidType = Type;
					return false;
				}
				ValidIndex++;
			}
			return true;
		});
		return ValidType;
	}

	return CategoryViews(TypeCategories, InRegistry).GetTypeIndex(InIndex);
}

TOptional<TRigVMTypeIndex> FRigVMTemplateArgument::TryToGetTypeIndex(const int32 InIndex) const
{
	return TryToGetTypeIndex_NoLock(InIndex, FRigVMRegistryReadLock());
}

TOptional<TRigVMTypeIndex> FRigVMTemplateArgument::TryToGetTypeIndex_NoLock(const int32 InIndex, const FRigVMRegistryHandle& InRegistry) const
{
	if(IsSingleton_NoLock({}, InRegistry))
	{
		return GetTypeIndex_NoLock(0, InRegistry);
	}
	const TRigVMTypeIndex TypeIndex = GetTypeIndex_NoLock(InIndex, InRegistry);
	if(TypeIndex != INDEX_NONE)
	{
		return TypeIndex;
	}
	return TOptional<TRigVMTypeIndex>();
}

int32 FRigVMTemplateArgument::FindTypeIndex(const TRigVMTypeIndex InTypeIndex, const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		return TypeIndices.IndexOfByKey(InTypeIndex);
	}

	if (FilterType)
	{
		bool bFound = false;
		int32 ValidIndex = 0;
		CategoryViews(TypeCategories, InRegistry).ForEachType([this, &ValidIndex, &bFound, InTypeIndex, &InRegistry](const TRigVMTypeIndex& Type) -> bool
		{
			if (Type == InTypeIndex)
			{
				bFound = true;
				return false;
			}
			if (FilterType(Type, InRegistry))
			{
				ValidIndex++;
			}
			return true;
		});
		if (bFound)
		{
			return ValidIndex;
		}
		return INDEX_NONE;
	}

	return CategoryViews(TypeCategories, InRegistry).FindIndex(InTypeIndex);
}

int32 FRigVMTemplateArgument::GetNumTypes(const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		return TypeIndices.Num();
	}

	if (FilterType)
	{
		int32 NumTypes = 0;
		CategoryViews(TypeCategories, InRegistry).ForEachType([this, &NumTypes, &InRegistry](const TRigVMTypeIndex& Type) -> bool
		{
			if (FilterType(Type, InRegistry))
			{
				NumTypes++;
			}
			return true;
		});
		return NumTypes;
	}
	
	return Algo::Accumulate(TypeCategories, 0, [&InRegistry](int32 Sum, const ETypeCategory Category)
	{
		return Sum + InRegistry->GetTypesForCategory_NoLock(Category).Num();
	});
}

int32 FRigVMTemplateArgument::GetNumTypes_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		return TypeIndices.Num();
	}

	
	if (FilterType)
	{
		int32 NumTypes = 0;
		CategoryViews(TypeCategories, InRegistry).ForEachType([this, &NumTypes, &InRegistry](const TRigVMTypeIndex& Type) -> bool
		{
			if (FilterType(Type, InRegistry))
			{
				NumTypes++;
			}
			return true;
		});
		return NumTypes;
	}

	return Algo::Accumulate(TypeCategories, 0, [&InRegistry](int32 Sum, const ETypeCategory Category)
	{
		return Sum + InRegistry->GetTypesForCategory_NoLock(Category).Num();
	});
}

void FRigVMTemplateArgument::AddTypeIndex(const TRigVMTypeIndex InTypeIndex)
{
	ensure( TypeCategories.IsEmpty() );
	TypeIndices.AddUnique(InTypeIndex);
}

void FRigVMTemplateArgument::RemoveType(const int32 InIndex)
{
	ensure( TypeCategories.IsEmpty() );
	TypeIndices.RemoveAt(InIndex);
}

void FRigVMTemplateArgument::ForEachType(TFunction<bool(const TRigVMTypeIndex InType)>&& InCallback, const FRigVMRegistryHandle& InRegistry) const
{
	if (!bUseCategories)
	{
		return Algo::ForEach(TypeIndices, InCallback);
	}

	if (FilterType)
	{
		CategoryViews(TypeCategories, InRegistry).ForEachType([this, InCallback, &InRegistry](const TRigVMTypeIndex Type)
		{
			if (FilterType(Type, InRegistry))
			{
				return InCallback(Type);
			}
			return true;
		});
		return;
	}
	
	return CategoryViews(TypeCategories, InRegistry).ForEachType(MoveTemp(InCallback));
}

TArray<TRigVMTypeIndex> FRigVMTemplateArgument::GetSupportedTypeIndices(const TArray<int32>& InPermutationIndices) const
{
	return GetSupportedTypeIndices_NoLock(InPermutationIndices, FRigVMRegistryReadLock());
}

TArray<TRigVMTypeIndex> FRigVMTemplateArgument::GetSupportedTypeIndices_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const
{
#if UE_RIGVM_DEBUG_TYPEINDEX
	auto UpdateTypeIndex = [&InRegistry](const TRigVMTypeIndex& TypeIndex) -> TRigVMTypeIndex
	{
		if(TypeIndex.Name.IsNone())
		{
			return InRegistry->GetTypeIndex_NoLock(InRegistry->GetType_NoLock(TypeIndex));
		}
		return TypeIndex;
	};
#endif

	TSet<TRigVMTypeIndex> SupportedTypes;
	if(InPermutationIndices.IsEmpty())
	{
		ForEachType([&](const TRigVMTypeIndex TypeIndex)
		{
			// INDEX_NONE indicates deleted permutation
			if (TypeIndex != INDEX_NONE)
			{
#if UE_RIGVM_DEBUG_TYPEINDEX
				SupportedTypes.Add(UpdateTypeIndex(TypeIndex));
#else
				SupportedTypes.Add(TypeIndex);
#endif
			}
			return true;
		}, InRegistry);
	}
	else
	{
		SupportedTypes.Reserve(InPermutationIndices.Num());
		for(const int32 PermutationIndex : InPermutationIndices)
		{
			// INDEX_NONE indicates deleted permutation
			const TRigVMTypeIndex TypeIndex = GetTypeIndex_NoLock(PermutationIndex, InRegistry);
			if (TypeIndex != INDEX_NONE)
			{
#if UE_RIGVM_DEBUG_TYPEINDEX
				SupportedTypes.Add(UpdateTypeIndex(TypeIndex));
#else
				SupportedTypes.Add(TypeIndex);
#endif
			}
		}
	}
	return SupportedTypes.Array();
}

#if WITH_EDITOR

TArray<FString> FRigVMTemplateArgument::GetSupportedTypeStrings(const TArray<int32>& InPermutationIndices) const
{
	return GetSupportedTypeStrings_NoLock(InPermutationIndices, FRigVMRegistryReadLock());
}

TArray<FString> FRigVMTemplateArgument::GetSupportedTypeStrings_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const
{
	TArray<FString> SupportedTypes;
	if(InPermutationIndices.IsEmpty())
	{
		ForEachType([&InRegistry, &SupportedTypes](const TRigVMTypeIndex TypeIndex)
		{
			// INDEX_NONE indicates deleted permutation
			if (TypeIndex != INDEX_NONE)
			{
				const FString TypeString = InRegistry->GetType_NoLock(TypeIndex).CPPType.ToString();
				SupportedTypes.AddUnique(TypeString);
			}
			return true;
		}, InRegistry);
	}
	else
	{
		for(const int32 PermutationIndex : InPermutationIndices)
		{
			const TRigVMTypeIndex TypeIndex = GetTypeIndex_NoLock(PermutationIndex, InRegistry);
			// INDEX_NONE indicates deleted permutation
			if (TypeIndex != INDEX_NONE)
			{
				const FString TypeString = InRegistry->GetType_NoLock(TypeIndex).CPPType.ToString();
				SupportedTypes.AddUnique(TypeString);
			}
		}
	}
	return SupportedTypes;
}

#endif


FRigVMTemplateArgument::CategoryViews::CategoryViews(const TArray<ETypeCategory>& InCategories, const FRigVMRegistryHandle& InRegistry)
{
	Types.Reserve(InCategories.Num());
	for (const ETypeCategory Category: InCategories)
	{
		Types.Emplace(InRegistry->GetTypesForCategory_NoLock(Category));
	}
}

void FRigVMTemplateArgument::CategoryViews::ForEachType(TFunction<bool(const TRigVMTypeIndex InType)>&& InCallback) const
{
	for (const TArrayView<const TRigVMTypeIndex>& TypeView: Types)
	{
		for (const TRigVMTypeIndex& TypeIndex : TypeView)
		{
			if (!InCallback(TypeIndex))
			{
				return;
			}
		}
	}
}

TRigVMTypeIndex FRigVMTemplateArgument::CategoryViews::GetTypeIndex(int32 InIndex) const
{
	for (const TArrayView<const TRigVMTypeIndex>& TypeView: Types)
	{
		if (TypeView.IsValidIndex(InIndex))
		{
			return TypeView[InIndex];  
		}
		InIndex -= TypeView.Num();
	}
	return INDEX_NONE;
}
	
int32 FRigVMTemplateArgument::CategoryViews::FindIndex(const TRigVMTypeIndex InTypeIndex) const
{
	int32 Offset = 0;
	for (const TArrayView<const TRigVMTypeIndex>& TypeView: Types)
	{
		const int32 Found = TypeView.IndexOfByKey(InTypeIndex);
		if (Found != INDEX_NONE)
		{
			return Found + Offset;
		}
		Offset += TypeView.Num();
	}
	return INDEX_NONE;
}

/**
 * FRigVMTemplateArgumentInfo 
 */

FRigVMTemplateArgumentInfo::FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices)
	: Name(InName)
	, Direction(InDirection)
	, FactoryCallback([InTypeIndices](const FName InName, ERigVMPinDirection InDirection, const FRigVMRegistryHandle& InRegistry){ return FRigVMTemplateArgument(InName, InDirection, InTypeIndices, InRegistry); } )
{}

FRigVMTemplateArgumentInfo::FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, TRigVMTypeIndex InTypeIndex)
	: Name(InName)
	, Direction(InDirection)
	, FactoryCallback( [InTypeIndex](const FName InName, ERigVMPinDirection InDirection, const FRigVMRegistryHandle& InRegistry){ return FRigVMTemplateArgument(InName, InDirection, InTypeIndex, InRegistry); } )
{}

FRigVMTemplateArgumentInfo::FRigVMTemplateArgumentInfo(
	const FName InName, ERigVMPinDirection InDirection,
	const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories,
	TypeFilterCallback InTypeFilter)
	: Name(InName)
	, Direction(InDirection)
	, FactoryCallback([InTypeCategories, InTypeFilter](const FName InName, ERigVMPinDirection InDirection, const FRigVMRegistryHandle& InRegistry){ return FRigVMTemplateArgument(InName, InDirection, InTypeCategories, InTypeFilter, InRegistry); })
{}

FRigVMTemplateArgumentInfo::FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection)
	: Name(InName)
	, Direction(InDirection)
{}

FRigVMTemplateArgumentInfo::FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, ArgumentCallback&& InCallback)
	: Name(InName)
	, Direction(InDirection)
	, FactoryCallback(InCallback)
{}

FRigVMTemplateArgumentInfo FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(const FName InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices)
{
	FRigVMTemplateArgumentInfo Info(InName, InDirection);
	Info.TypeIndices = InTypeIndices;
	return Info;
}

FRigVMTemplateArgument FRigVMTemplateArgumentInfo::GetArgument() const
{
	return GetArgument_NoLock(FRigVMRegistryReadLock());
}

FRigVMTemplateArgument FRigVMTemplateArgumentInfo::GetArgument_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	if (!TypeIndices.IsEmpty())
	{
		return FRigVMTemplateArgument(Name, Direction, TypeIndices, InRegistry);
	}
	
	check (FactoryCallback)
	FRigVMTemplateArgument Argument = FactoryCallback(Name, Direction, InRegistry);
	return MoveTemp(Argument);
}

FName FRigVMTemplateArgumentInfo::ComputeTemplateNotation(const FName InTemplateName, const TArray<FRigVMTemplateArgumentInfo>& InInfos)
{
	if (InInfos.IsEmpty())
	{
		return NAME_None;	
	}
		
	TArray<FString> ArgumentNotations;
	Algo::TransformIf(
		InInfos,
		ArgumentNotations,
		[](const FRigVMTemplateArgumentInfo& Info){ return Info.Direction != ERigVMPinDirection::Invalid && Info.Direction != ERigVMPinDirection::Hidden; },
		[](const FRigVMTemplateArgumentInfo& Info){ return FRigVMTemplate::GetArgumentNotation(Info.Name, Info.Direction); }
	);

	if (ArgumentNotations.IsEmpty())
	{
		return NAME_None;
	}

	return *RigVMStringUtils::MakeTemplateNotation(InTemplateName.ToString(), ArgumentNotations);
}

TArray<TRigVMTypeIndex> FRigVMTemplateArgumentInfo::GetTypesFromCategories(
	const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories,
	const FRigVMTemplateArgument::FTypeFilter& InTypeFilter)
{
	return GetTypesFromCategories_NoLock(InTypeCategories, InTypeFilter, FRigVMRegistryReadLock());
}

TArray<TRigVMTypeIndex> FRigVMTemplateArgumentInfo::GetTypesFromCategories_NoLock(
	const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories,
	const FRigVMTemplateArgument::FTypeFilter& InTypeFilter,
	const FRigVMRegistryHandle& InRegistry)
{
	TSet<TRigVMTypeIndex> AllTypes;

	// preallocate AllTypes since it can be a large array
	{
		int32 NumTypes = 0;
		for (const FRigVMTemplateArgument::ETypeCategory TypeCategory : InTypeCategories)
		{
			NumTypes += InRegistry->GetTypesForCategory_NoLock(TypeCategory).Num();
		}
		AllTypes.Reserve(NumTypes);
	}

	for (const FRigVMTemplateArgument::ETypeCategory TypeCategory : InTypeCategories)
	{
		AllTypes.Append(InRegistry->GetTypesForCategory_NoLock(TypeCategory));
	}

	TArray<TRigVMTypeIndex> Types;
	if (!InTypeFilter.IsBound())
	{
		Types = AllTypes.Array();
	}
	else
	{
		Types.Reserve(AllTypes.Num());
		for (const TRigVMTypeIndex& Type : AllTypes)
		{
			if (InTypeFilter.Execute(Type, InRegistry))
			{
				Types.Add(Type);
			}
		}
	}
	
	return MoveTemp(Types);
}

////////////////////////////////////////////////////////////////////////////////////////////////

FRigVMTemplate::FRigVMTemplate()
	: Index(INDEX_NONE)
	, Notation(NAME_None)
	, Hash(UINT32_MAX)
	, OwnerRegistry(nullptr)
{

}

FRigVMTemplate::FRigVMTemplate(UScriptStruct* InStruct, const FString& InTemplateName, int32 InFunctionIndex, FRigVMRegistryHandle& InRegistry)
	: Index(INDEX_NONE)
	, Notation(NAME_None)
	, Hash(UINT32_MAX)
	, OwnerRegistry(InRegistry.GetRegistry())
{
	TArray<FString> ArgumentNotations;

	// create the arguments sorted by super -> child struct.
	TArray<UStruct*> Structs = GetSuperStructs(InStruct, true);
	for(UStruct* Struct : Structs)
	{
		// only iterate on this struct's fields, not the super structs'
		for (TFieldIterator<FProperty> It(Struct, EFieldIterationFlags::None); It; ++It)
		{
			FRigVMTemplateArgument Argument = FRigVMTemplateArgument::Make_NoLock(*It, InRegistry);
			Argument.Index = Arguments.Num();

			if(!Argument.IsExecute_NoLock(InRegistry) && IsValidArgumentForTemplate(Argument.GetDirection()) && Argument.GetDirection() != ERigVMPinDirection::Hidden)
			{
				Arguments.Add(Argument);
			}
		}
	}

	// the template notation needs to be in the same order as the C++ implementation,
	// which is the order of child -> super class members
	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		if(const FRigVMTemplateArgument* Argument = FindArgument(It->GetFName()))
		{
			if(!Argument->IsExecute_NoLock(InRegistry) && Argument->GetDirection() != ERigVMPinDirection::Hidden)
			{
				ArgumentNotations.Add(GetArgumentNotation(Argument->Name, Argument->Direction));
			}
		}
	}

	if (ArgumentNotations.Num() > 0)
	{
		const FString NotationStr = RigVMStringUtils::MakeTemplateNotation(InTemplateName, ArgumentNotations);
		Notation = *NotationStr;
		if (InFunctionIndex != INDEX_NONE)
		{
			Permutations.Add(InFunctionIndex);
			for (const FRigVMTemplateArgument& Argument : Arguments)
			{
				check(Argument.TypeIndices.Num() == 1);
			}
		}

		UpdateTypesHashToPermutation(Permutations.Num()-1, InRegistry);
	}
}

FRigVMTemplate::FRigVMTemplate(const FName& InTemplateName, const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FRigVMRegistryHandle& InRegistry)
	: Index(INDEX_NONE)
	, Notation(NAME_None)
	, Hash(UINT32_MAX)
	, OwnerRegistry(const_cast<FRigVMRegistry_NoLock*>(InRegistry.GetRegistry()))
{
	for (const FRigVMTemplateArgumentInfo& InInfo : InInfos)
	{
		if(IsValidArgumentForTemplate(InInfo.Direction))
		{
			FRigVMTemplateArgument Argument = InInfo.GetArgument_NoLock(InRegistry);
			Argument.Index = Arguments.Num();
			Arguments.Emplace(MoveTemp(Argument));
		}
	}
	
	Notation = FRigVMTemplateArgumentInfo::ComputeTemplateNotation(InTemplateName, InInfos);
	UpdateTypesHashToPermutation(Permutations.Num()-1, InRegistry);
}

FLinearColor FRigVMTemplate::GetColorFromMetadata(FString InMetadata)
{
	FLinearColor Color = FLinearColor::Black;

	FString Metadata = InMetadata;
	Metadata.TrimStartAndEndInline();
	static const FString SplitString(TEXT(" "));
	FString Red, Green, Blue, GreenAndBlue;
	if (Metadata.Split(SplitString, &Red, &GreenAndBlue))
	{
		Red.TrimEndInline();
		GreenAndBlue.TrimStartInline();
		if (GreenAndBlue.Split(SplitString, &Green, &Blue))
		{
			Green.TrimEndInline();
			Blue.TrimStartInline();

			const float RedValue = FCString::Atof(*Red);
			const float GreenValue = FCString::Atof(*Green);
			const float BlueValue = FCString::Atof(*Blue);
			Color = FLinearColor(RedValue, GreenValue, BlueValue);
		}
	}

	return Color;
}

bool FRigVMTemplate::IsValidArgumentForTemplate(const ERigVMPinDirection InDirection)
{
	return InDirection != ERigVMPinDirection::Invalid;
}


const FString& FRigVMTemplate::GetDirectionPrefix(const ERigVMPinDirection InDirection)
{
	static const FString EmptyPrefix = FString();
	static const FString InPrefix = TEXT("in ");
	static const FString OutPrefix = TEXT("out ");
	static const FString IOPrefix = TEXT("io ");

	switch(InDirection)
	{
	case ERigVMPinDirection::Input:
	case ERigVMPinDirection::Visible:
		{
			return InPrefix;
		}
	case ERigVMPinDirection::Output:
		{
			return OutPrefix;
		}
	case ERigVMPinDirection::IO:
		{
			return IOPrefix;
		}
	default:
		{
			break;
		}
	}

	return EmptyPrefix;
}

FString FRigVMTemplate::GetArgumentNotation(const FName InName, const ERigVMPinDirection InDirection)
{
	return GetDirectionPrefix(InDirection) + InName.ToString();
}

void FRigVMTemplate::ComputeNotationFromArguments(const FString& InTemplateName)
{
	TArray<FString> ArgumentNotations;			
	for (const FRigVMTemplateArgument& Argument : Arguments)
	{
		if(IsValidArgumentForTemplate(Argument.GetDirection()))
		{
			ArgumentNotations.Add(GetArgumentNotation(Argument.Name, Argument.Direction));
		}
	}

	const FString NotationStr = RigVMStringUtils::MakeTemplateNotation(InTemplateName, ArgumentNotations);
	Notation = *NotationStr;
}

TArray<UStruct*> FRigVMTemplate::GetSuperStructs(UStruct* InStruct, bool bIncludeLeaf)
{
	// Create an array of structs, ordered super -> child struct
	TArray<UStruct*> SuperStructs = {InStruct};
	while(true)
	{
		if(UStruct* SuperStruct = SuperStructs[0]->GetSuperStruct())
		{
			SuperStructs.Insert(SuperStruct, 0);
		}
		else
		{
			break;
		}
	}

	if(!bIncludeLeaf)
	{
		SuperStructs.Remove(SuperStructs.Last());
	}

	return SuperStructs;
}

FRigVMTemplate::FTypeMap FRigVMTemplate::GetArgumentTypesFromString(const FString& InTypeString, const FRigVMUserDefinedTypeResolver* InTypeResolver) const
{
	FRigVMRegistryWriteLock Lock;
	return GetArgumentTypesFromString_NoLock(InTypeString, InTypeResolver, Lock);
}

FRigVMTemplate::FTypeMap FRigVMTemplate::GetArgumentTypesFromString_NoLock(const FString& InTypeString, const FRigVMUserDefinedTypeResolver* InTypeResolver, FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	FTypeMap Types;
	if(!InTypeString.IsEmpty())
	{
		FString Left, Right = InTypeString;
		while(!Right.IsEmpty())
		{
			if(!Right.Split(TEXT(","), &Left, &Right))
			{
				Left = Right;
				Right.Reset();
			}

			FString ArgumentName, TypeName;
			if(Left.Split(TEXT(":"), &ArgumentName, &TypeName))
			{
				if(const FRigVMTemplateArgument* Argument = FindArgument(*ArgumentName))
				{
					const FName TypeFName = *TypeName;
					if (IsIgnoredType(TypeFName))
					{
						Types.Reset();
						return Types;
					}

					TRigVMTypeIndex TypeIndex = InRegistry->GetTypeIndexFromCPPType_NoLock(TypeName);

					// If the type was not found, check if it's a user-defined type that hasn't been registered yet.
					if (TypeIndex == INDEX_NONE && RigVMTypeUtils::RequiresCPPTypeObject(TypeName))
					{
						UObject* CPPTypeObject = RigVMTypeUtils::ObjectFromCPPType(TypeName, true, InTypeResolver);
						if (CPPTypeObject != nullptr)
						{
							FRigVMTemplateArgumentType ArgType(TypeFName, CPPTypeObject);
							TypeIndex = InRegistry->FindOrAddType_NoLock(ArgType, false);
						}
						else
						{
							UE_LOGF(LogRigVM, Display, "RigVMTemplate '%ls' cannot resolve type '%ls' when solving permutation for '%ls'.",
									*GetNotation().ToString(),
									*TypeName,
									*InTypeString);
						}
					}
					
					if(TypeIndex != INDEX_NONE)
					{
						Types.Add(Argument->GetName(), TypeIndex);
					}
				}
			}
		}
	}
	return Types;
}

FString FRigVMTemplate::GetStringFromArgumentTypes(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry)
{
	TArray<FString> TypePairStrings;
	for(const TPair<FName,TRigVMTypeIndex>& Pair : InTypes)
	{
		const FRigVMTemplateArgumentType& Type = InRegistry->GetType_NoLock(Pair.Value);
		static constexpr TCHAR Format[] = TEXT("%s:%s");
		FString PairString;
		const FString KeyString = Pair.Key.ToString();
		const FString CPPTypeString = Type.CPPType.ToString();
		PairString.Reserve(KeyString.Len() + CPPTypeString.Len() + 1);
		PairString.Append(KeyString);
		PairString.AppendChar(TEXT(':'));
		PairString.Append(CPPTypeString);
		TypePairStrings.Add(PairString);
	}
			
	return RigVMStringUtils::JoinStrings(TypePairStrings, TEXT(","));
}

bool FRigVMTemplate::IsValid() const
{
	return !Notation.IsNone();
}

const FName& FRigVMTemplate::GetNotation() const
{
	return Notation;
}

FName FRigVMTemplate::GetName() const
{
	FString Left;
	if (GetNotation().ToString().Split(TEXT("::"), &Left, nullptr))
	{
		return *Left;
	}
	if (GetNotation().ToString().Split(TEXT("("), &Left, nullptr))
	{
		return *Left;
	}
	return NAME_None;
}

FName FRigVMTemplate::GetNodeName() const
{
#if WITH_EDITOR
	if(UsesDispatch())
	{
		if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
		{
			if(const UScriptStruct* FactoryStruct = Factory->GetScriptStruct())
			{
				FString DisplayName = FactoryStruct->GetDisplayNameText().ToString();
				RigVMStringUtils::SanitizeName(DisplayName, false, false, 100);
				if(!DisplayName.IsEmpty())
				{
					return *DisplayName;
				}
			}
		}
	}
#endif
	return GetName();
}

#if WITH_EDITOR

FLinearColor FRigVMTemplate::GetColor(const TArray<int32>& InPermutationIndices) const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		return Factory->GetNodeColor();
	}

	bool bFirstColorFound = false;
	FLinearColor ResolvedColor = FLinearColor::White;

	auto VisitPermutation = [&bFirstColorFound, &ResolvedColor, this](int32 InPermutationIndex) -> bool
	{
		static const FLazyName NodeColorName = TEXT("NodeColor");
		FString NodeColorMetadata;

		// if we can't find one permutation we are not going to find any, so it's ok to return false here
		const FRigVMFunction* ResolvedFunction = GetPermutation(InPermutationIndex);
		if(ResolvedFunction == nullptr)
		{
			return false;
		}

		ResolvedFunction->Struct->GetStringMetaDataHierarchical(NodeColorName, &NodeColorMetadata);
		if (!NodeColorMetadata.IsEmpty())
		{
			if(bFirstColorFound)
			{
				const FLinearColor NodeColor = GetColorFromMetadata(NodeColorMetadata);
				if(!ResolvedColor.Equals(NodeColor, 0.01f))
				{
					ResolvedColor = FLinearColor::White;
					return false;
				}
			}
			else
			{
				ResolvedColor = GetColorFromMetadata(NodeColorMetadata);
				bFirstColorFound = true;
			}
		}
		return true;
	};

	if(InPermutationIndices.IsEmpty())
	{
		for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
		{
			if(!VisitPermutation(PermutationIndex))
			{
				break;
			}
		}
	}
	else
	{
		for(const int32 PermutationIndex : InPermutationIndices)
		{
			if(!VisitPermutation(PermutationIndex))
			{
				break;
			}
		}
	}
	return ResolvedColor;
}

FText FRigVMTemplate::GetTooltipText(const TArray<int32>& InPermutationIndices) const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		FTypeMap Types;
		if(InPermutationIndices.Num() == 1)
		{
			Types = GetTypesForPermutation(InPermutationIndices[0]);
		}
		return Factory->GetNodeTooltip(Types);
	}

	FText ResolvedTooltipText;

	auto VisitPermutation = [&ResolvedTooltipText, this](int32 InPermutationIndex) -> bool
	{
		if (InPermutationIndex >= NumPermutations())
		{
			return false;
		}
		
		// if we can't find one permutation we are not going to find any, so it's ok to return false here
		const FRigVMFunction* ResolvedFunction = GetPermutation(InPermutationIndex);
		if(ResolvedFunction == nullptr)
		{
			return false;
		}
		
		const FText TooltipText = ResolvedFunction->Struct->GetToolTipText();
		
		if (!ResolvedTooltipText.IsEmpty())
		{
			if(!ResolvedTooltipText.EqualTo(TooltipText))
			{
				ResolvedTooltipText = FText::FromName(GetName());
				return false;
			}
		}
		else
		{
			ResolvedTooltipText = TooltipText;
		}
		return true;
	};

	for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
	{
		if(!VisitPermutation(PermutationIndex))
		{
			break;
		}
	}

	return ResolvedTooltipText;
}

FText FRigVMTemplate::GetDisplayNameForArgument(const FName& InArgumentName, const TArray<int32>& InPermutationIndices) const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		FRigVMTemplateTypeMap Types;
		if (InPermutationIndices.Num() == 1)
		{
			Types = GetTypesForPermutation_NoLock(InPermutationIndices[0], FRigVMRegistryReadLock());
		}
		
		const FName DisplayName = Factory->GetDisplayNameForArgument(InArgumentName, Types);
		if(DisplayName.IsNone())
		{
			return FText();
		}
		return FText::FromName(DisplayName);
	}

	if(const FRigVMTemplateArgument* Argument = FindArgument(InArgumentName))
	{
		FText ResolvedDisplayName;

		auto VisitPermutation = [InArgumentName, &ResolvedDisplayName, this](const int32 InPermutationIndex) -> bool
		{
			// if we can't find one permutation we are not going to find any, so it's ok to return false here
			const FRigVMFunction* ResolvedFunction = GetPermutation(InPermutationIndex);
			if(ResolvedFunction == nullptr)
			{
				return false;
			}
			
			if(const FProperty* Property = ResolvedFunction->Struct->FindPropertyByName(InArgumentName))
			{
				const FText DisplayName = Property->GetDisplayNameText();
				if (!ResolvedDisplayName.IsEmpty())
				{
					if(!ResolvedDisplayName.EqualTo(DisplayName))
					{
						ResolvedDisplayName = FText::FromName(InArgumentName);
						return false;
					}
				}
				else
				{
					ResolvedDisplayName = DisplayName;
				}
			}
			return true;
		};

		if(InPermutationIndices.IsEmpty())
		{
			for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
			{
				if(!VisitPermutation(PermutationIndex))
				{
					break;
				}
			}
		}
		else
		{
			for(const int32 PermutationIndex : InPermutationIndices)
			{
				if(!VisitPermutation(PermutationIndex))
				{
					break;
				}
			}
		}

		return ResolvedDisplayName;
	}
	return FText();
}

FString FRigVMTemplate::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey, const TArray<int32>& InPermutationIndices) const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		return Factory->GetArgumentMetaData(InArgumentName, InMetaDataKey);
	}

	if(const FRigVMTemplateArgument* Argument = FindArgument(InArgumentName))
	{
		FString ResolvedMetaData;

		auto VisitPermutation = [InArgumentName, &ResolvedMetaData, InMetaDataKey, this](const int32 InPermutationIndex) -> bool
		{
			// if we can't find one permutation we are not going to find any, so it's ok to return false here
			const FRigVMFunction* ResolvedFunction = GetPermutation(InPermutationIndex);
			if(ResolvedFunction == nullptr)
			{
				return false;
			}
			
			if(const FProperty* Property = ResolvedFunction->Struct->FindPropertyByName(InArgumentName))
			{
				// Flag-style metadata (no value) returns empty from GetMetaData; use sentinel.
				FString MetaData = Property->GetMetaData(InMetaDataKey);
				if (MetaData.IsEmpty() && Property->HasMetaData(InMetaDataKey))
				{
					MetaData = FRigVMDispatchFactory::TrueString;
				}
				if (!ResolvedMetaData.IsEmpty())
				{
					if(!ResolvedMetaData.Equals(MetaData))
					{
						ResolvedMetaData = FString();
						return false;
					}
				}
				else
				{
					ResolvedMetaData = MetaData;
				}
			}
			return true;
		};

		if(InPermutationIndices.IsEmpty())
		{
			for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
			{
				if(!VisitPermutation(PermutationIndex))
				{
					break;
				}
			}
		}
		else
		{
			for(const int32 PermutationIndex : InPermutationIndices)
			{
				if(!VisitPermutation(PermutationIndex))
				{
					break;
				}
			}
		}

		return ResolvedMetaData;
	}
	return FString();
}

#endif

bool FRigVMTemplate::Merge(const FRigVMTemplate& InOther, const FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if (!IsValid() || !InOther.IsValid())
	{
		return false;
	}

	if(Notation != InOther.Notation)
	{
		return false;
	}

	check(OwnerRegistry == InOther.OwnerRegistry);

	if(InOther.GetExecuteContextStruct_NoLock(InRegistry) != GetExecuteContextStruct_NoLock(InRegistry))
	{
		// find the previously defined permutation.
		UE_LOGF(LogRigVM, Display, "RigVMFunction '%ls' cannot be merged into the '%ls' template. ExecuteContext Types differ ('%ls' vs '%ls' from '%ls').",
			*InOther.GetPrimaryPermutation_NoLock(InRegistry)->Name,
			*GetNotation().ToString(),
			*InOther.GetExecuteContextStruct_NoLock(InRegistry)->GetStructCPPName(),
			*GetExecuteContextStruct_NoLock(InRegistry)->GetStructCPPName(),
			*GetPrimaryPermutation_NoLock(InRegistry)->Name);
		return false;
	}

	if (InOther.Permutations.Num() != 1)
	{
		return false;
	}

	// find colliding permutations
	for(int32 PermutationIndex = 0;PermutationIndex<NumPermutations_NoLock(InRegistry);PermutationIndex++)
	{
		int32 MatchingArguments = 0;
		for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
		{
			if (Arguments[ArgumentIndex].GetTypeIndex_NoLock(PermutationIndex, InRegistry) == InOther.Arguments[ArgumentIndex].GetTypeIndex_NoLock(0, InRegistry))
			{
				MatchingArguments++;
			}
		}
		if(MatchingArguments == Arguments.Num())
		{
			// find the previously defined permutation.
			UE_LOGF(LogRigVM, Display, "RigVMFunction '%ls' cannot be merged into the '%ls' template. It collides with '%ls'.",
				*InOther.GetPrimaryPermutation_NoLock(InRegistry)->Name,
				*GetNotation().ToString(),
				*GetPermutation_NoLock(PermutationIndex, InRegistry)->Name);
			return false;
		}
	}

	TArray<FRigVMTemplateArgument> NewArgs;

	for (int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		const FRigVMTemplateArgument& OtherArg = InOther.Arguments[ArgumentIndex];
		if (OtherArg.GetNumTypes_NoLock(InRegistry) != 1)
		{
			return false;
		}

		// Add Other argument information into the TypeToPermutations map
		{
			FRigVMTemplateArgument& NewArg = NewArgs.Add_GetRef(Arguments[ArgumentIndex]);
			const TRigVMTypeIndex OtherTypeIndex = OtherArg.GetTypeIndex_NoLock(0, InRegistry);
#if WITH_EDITOR
			const int32 NewPermutationIndex = NewArg.GetNumTypes_NoLock(InRegistry);
			if (TArray<int32>* ArgTypePermutations = NewArg.TypeToPermutations.Find(OtherTypeIndex))
			{
				ArgTypePermutations->Add(NewPermutationIndex);
			}
			else
			{
				NewArg.TypeToPermutations.Add(OtherTypeIndex, {NewPermutationIndex});
			}
#endif
			NewArg.TypeIndices.Add(OtherTypeIndex);
		}
	}

	Arguments = NewArgs;

	Permutations.Add(InOther.Permutations[0]);

	UpdateTypesIndicesToPermutationIndices(Permutations.Num() - 1, InRegistry);
	UpdateTypesHashToPermutation(Permutations.Num()-1, InRegistry);

	return true;
}

const FRigVMTemplateArgument* FRigVMTemplate::FindArgument(const FName& InArgumentName) const
{
	return Arguments.FindByPredicate([InArgumentName](const FRigVMTemplateArgument& Argument) -> bool
	{
		return Argument.GetName() == InArgumentName;
	});
}

int32 FRigVMTemplate::NumExecuteArguments(const FRigVMDispatchContext& InContext) const
{
	return GetExecuteArguments(InContext).Num();
}

const FRigVMExecuteArgument* FRigVMTemplate::GetExecuteArgument(int32 InIndex, const FRigVMDispatchContext& InContext) const
{
	const TArray<FRigVMExecuteArgument>& Args = GetExecuteArguments(InContext);
	if(Args.IsValidIndex(InIndex))
	{
		return &Args[InIndex];
	}
	return nullptr;
}

const FRigVMExecuteArgument* FRigVMTemplate::FindExecuteArgument(const FName& InArgumentName, const FRigVMDispatchContext& InContext) const
{
	return FindExecuteArgument_NoLock(InArgumentName, InContext, FRigVMRegistryReadLock());
}

const FRigVMExecuteArgument* FRigVMTemplate::FindExecuteArgument_NoLock(const FName& InArgumentName, const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const
{
	const TArray<FRigVMExecuteArgument>& Args = GetExecuteArguments_NoLock(InContext, InRegistry);
	return Args.FindByPredicate([InArgumentName](const FRigVMExecuteArgument& Arg) -> bool
	{
		return Arg.Name == InArgumentName;
	});
}

const TArray<FRigVMExecuteArgument>& FRigVMTemplate::GetExecuteArguments(const FRigVMDispatchContext& InContext) const
{
	return GetExecuteArguments_NoLock(InContext, FRigVMRegistryReadLock());
}

const TArray<FRigVMExecuteArgument>& FRigVMTemplate::GetExecuteArguments_NoLock(const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if(ExecuteArguments.IsEmpty())
	{
		if(UsesDispatch())
		{
			const FRigVMDispatchFactory* Factory = Delegates.GetDispatchFactoryDelegate.Execute(InRegistry);
			check(Factory);

			ExecuteArguments = Factory->GetExecuteArguments_NoLock(InContext, InRegistry);
		}
		else if(const FRigVMFunction* PrimaryPermutation = GetPrimaryPermutation_NoLock(InRegistry))
		{
			if(PrimaryPermutation->Struct)
			{
				FRigVMRegistryHandle& MutableRegistryHandle = *const_cast<FRigVMRegistryHandle*>(&InRegistry);
				TArray<UStruct*> Structs = GetSuperStructs(PrimaryPermutation->Struct, true);
				for(const UStruct* Struct : Structs)
				{
					// only iterate on this struct's fields, not the super structs'
					for (TFieldIterator<FProperty> It(Struct, EFieldIterationFlags::None); It; ++It)
					{
						FRigVMTemplateArgument Argument = FRigVMTemplateArgument::Make_NoLock(*It, MutableRegistryHandle);
						if(Argument.IsExecute_NoLock(InRegistry))
						{
							ExecuteArguments.Emplace(Argument.Name, Argument.Direction, Argument.GetTypeIndex_NoLock(0, InRegistry));
						}
					}
				}
			}
		}
	}
	return ExecuteArguments;
}

const UScriptStruct* FRigVMTemplate::GetExecuteContextStruct() const
{
	return GetExecuteContextStruct_NoLock(FRigVMRegistryReadLock());
}

const UScriptStruct* FRigVMTemplate::GetExecuteContextStruct_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory_NoLock(InRegistry))
	{
		return Factory->GetExecuteContextStruct();
	}
	check(!Permutations.IsEmpty());
	return GetPrimaryPermutation_NoLock(InRegistry)->GetExecuteContextStruct_NoLock(InRegistry);
}

bool FRigVMTemplate::SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const
{
	return SupportsExecuteContextStruct_NoLock(InExecuteContextStruct, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::SupportsExecuteContextStruct_NoLock(const UScriptStruct* InExecuteContextStruct, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	return InExecuteContextStruct->IsChildOf(GetExecuteContextStruct_NoLock(InRegistry));
}

#if WITH_EDITOR

bool FRigVMTemplate::ArgumentSupportsTypeIndex(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex) const
{
	if (const FRigVMTemplateArgument* Argument = FindArgument(InArgumentName))
	{
		return Argument->SupportsTypeIndex(InTypeIndex, OutTypeIndex);
	}
	return false;
}

#endif

int32 FRigVMTemplate::NumPermutations() const
{
	return NumPermutations_NoLock(FRigVMRegistryReadLock());
}

const FRigVMFunction* FRigVMTemplate::GetPrimaryPermutation() const
{
	return GetPrimaryPermutation_NoLock(FRigVMRegistryReadLock());
}

const FRigVMFunction* FRigVMTemplate::GetPrimaryPermutation_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if (NumPermutations_NoLock(InRegistry) > 0)
	{
		return GetPermutation_NoLock(0, InRegistry);
	}
	return nullptr;
}

const FRigVMFunction* FRigVMTemplate::GetPermutation(int32 InIndex) const
{
	return GetPermutation_NoLock(InIndex, FRigVMRegistryReadLock());
}

const FRigVMFunction* FRigVMTemplate::GetPermutation_NoLock(int32 InIndex, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	const int32 FunctionIndex = Permutations[InIndex];
	if(InRegistry->GetFunctions_NoLock().IsValidIndex(FunctionIndex))
	{
		const FRigVMFunction* Function = &InRegistry->GetFunctions_NoLock()[Permutations[InIndex]];
		if (Function && Function->TemplateIndex != INDEX_NONE)
		{
			check(Function->PermutationIndex == InIndex);
		}
		return Function;
	}
	return nullptr;
}

const FRigVMFunction* FRigVMTemplate::GetOrCreatePermutation(int32 InIndex)
{
	FRigVMRegistryWriteLock Lock;
	return GetOrCreatePermutation_NoLock(InIndex, Lock);
}

const FRigVMFunction* FRigVMTemplate::GetOrCreatePermutation_NoLock(int32 InIndex, FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if(const FRigVMFunction* Function = GetPermutation_NoLock(InIndex, InRegistry))
	{
		return Function;
	}

	if(Permutations[InIndex] == INDEX_NONE && UsesDispatch())
	{
		FTypeMap Types;
		for(const FRigVMTemplateArgument& Argument : Arguments)
		{
			const TRigVMTypeIndex TypeIndex = Argument.GetTypeIndex_NoLock(InIndex, InRegistry);
			if (IsIgnoredType(TypeIndex))
			{
				return nullptr;
			}
			Types.Add(Argument.GetName(), TypeIndex);
		}

		FRigVMDispatchFactory* Factory = Delegates.GetDispatchFactoryDelegate.Execute(InRegistry);
		if (ensure(Factory))
		{
			const FRigVMFunctionPtr DispatchFunction = Factory->CreateDispatchFunction_NoLock(Types, InRegistry);

			TArray<FRigVMFunctionArgument> FunctionArguments;
			for(const FRigVMTemplateArgument& Argument : Arguments)
			{
				const FRigVMTemplateArgumentType& Type = InRegistry->GetType_NoLock(Argument.GetTypeIndex_NoLock(InIndex, InRegistry));
				FunctionArguments.Add(FRigVMFunctionArgument(Argument.Name.ToString(), Type.CPPType.ToString()));
			}
			
			static constexpr TCHAR Format[] = TEXT("%s::%s");
			const FString PermutationName = Factory->GetPermutationNameImpl(Types, InRegistry);

			int32 FunctionIndex = InRegistry->Functions.Num();
			if (const int32* ExistingFunctionIndex = InRegistry->PreviousFunctionNameToIndex.Find(*PermutationName))
			{
				FunctionIndex = *ExistingFunctionIndex;
			}

			Permutations[InIndex] = FunctionIndex;
			UpdateTypesIndicesToPermutationIndices(InIndex, InRegistry);

			if (FunctionIndex  == InRegistry->Functions.Num())
			{
				InRegistry->Functions.AddElement(
					FRigVMFunction(
						PermutationName,
						DispatchFunction,
						Factory,
						FunctionIndex,
						FunctionArguments
					)
				);
			}
			else
			{
				InRegistry->Functions[FunctionIndex] = FRigVMFunction(
					PermutationName,
					DispatchFunction,
					Factory,
					FunctionIndex,
					FunctionArguments
				);
			}
			
			InRegistry->Functions[FunctionIndex].TemplateIndex = Index;
			InRegistry->Functions[FunctionIndex].PermutationIndex = InIndex;
			InRegistry->Functions[FunctionIndex].OwnerRegistry = InRegistry.GetRegistry();
			InRegistry->FunctionNameToIndex.Add(*PermutationName, FunctionIndex);

			TArray<FRigVMFunction> Predicates = Factory->CreateDispatchPredicates_NoLock(Types, InRegistry);
			InRegistry->StructNameToPredicates.Add(*PermutationName, MoveTemp(Predicates));

			return &InRegistry->Functions[FunctionIndex];
		}
	}

	return nullptr;
}

bool FRigVMTemplate::ContainsPermutation(const FRigVMFunction* InPermutation) const
{
	return ContainsPermutation_NoLock(InPermutation, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::ContainsPermutation_NoLock(const FRigVMFunction* InPermutation, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	return FindPermutation_NoLock(InPermutation, InRegistry) != INDEX_NONE;
}

int32 FRigVMTemplate::FindPermutation(const FRigVMFunction* InPermutation) const
{
	return FindPermutation_NoLock(InPermutation, FRigVMRegistryReadLock());
}

int32 FRigVMTemplate::FindPermutation_NoLock(const FRigVMFunction* InPermutation, const FRigVMRegistryHandle& InRegistry) const
{
	check(InPermutation);
	check(InPermutation->TemplateIndex == Index);
	check(InRegistry.GetRegistry() == OwnerRegistry);
	check(Permutations.Find(InPermutation->Index) == InPermutation->PermutationIndex);
	return InPermutation->PermutationIndex;
}

int32 FRigVMTemplate::FindPermutation(const FTypeMap& InTypes) const
{
	return FindPermutation_NoLock(InTypes, FRigVMRegistryReadLock());
}

int32 FRigVMTemplate::FindPermutation_NoLock(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	FTypeMap Types = InTypes;
	int32 PermutationIndex = INDEX_NONE;
	if(FullyResolve_NoLock(Types, PermutationIndex, InRegistry))
	{
		return PermutationIndex;
	}
	return INDEX_NONE;
}

bool FRigVMTemplate::FullyResolve(FRigVMTemplate::FTypeMap& InOutTypes, int32& OutPermutationIndex) const
{
	return FullyResolve_NoLock(InOutTypes, OutPermutationIndex, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::FullyResolve_NoLock(FTypeMap& InOutTypes, int32& OutPermutationIndex, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	TArray<int32> PermutationIndices;
	Resolve_NoLock(InOutTypes, PermutationIndices, false, InRegistry);
	if(PermutationIndices.Num() == 1)
	{
		OutPermutationIndex = PermutationIndices[0];
	}
	else
	{
		OutPermutationIndex = INDEX_NONE;
	}
	return OutPermutationIndex != INDEX_NONE;
}

bool FRigVMTemplate::Resolve(FTypeMap& InOutTypes, TArray<int32>& OutPermutationIndices, bool bAllowFloatingPointCasts) const
{
	return Resolve_NoLock(InOutTypes, OutPermutationIndices, bAllowFloatingPointCasts, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::Resolve_NoLock(FTypeMap& InOutTypes, TArray<int32>& OutPermutationIndices, bool bAllowFloatingPointCasts, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);
	
	FTypeMap InputTypes = InOutTypes;
	InOutTypes.Reset();

	const int32 CurrentNumPermutations = NumPermutations_NoLock(InRegistry);
	OutPermutationIndices.Reset();
	OutPermutationIndices.Reserve(CurrentNumPermutations);
	
	for (int32 PermutationIndex = 0; PermutationIndex < CurrentNumPermutations; PermutationIndex++)
	{
		OutPermutationIndices.Add(PermutationIndex);
	}
	
	for (const FRigVMTemplateArgument& Argument : Arguments)
	{
		const TRigVMTypeIndex* InputType = InputTypes.Find(Argument.Name);
		
		if (Argument.IsSingleton_NoLock({}, InRegistry))
		{
			// if we are singleton - we still need to check if the potentially provided
			// type is compatible with the singleton type.
			const TRigVMTypeIndex SingleTypeIndex = Argument.GetTypeIndex_NoLock(0, InRegistry);
			if (InputType)
			{
				if (!InRegistry->IsWildCardType_NoLock(*InputType))
				{
					if (!InRegistry->CanMatchTypes_NoLock(*InputType, SingleTypeIndex, bAllowFloatingPointCasts))
					{
						OutPermutationIndices.Reset();
						return false;
					}
				}
			}
			InOutTypes.Add(Argument.Name, SingleTypeIndex);
			continue;
		}
		else if (InputType)
		{
			TArray<TRigVMTypeIndex> AllTypes; Argument.GetAllTypes_NoLock(AllTypes, InRegistry);

			TRigVMTypeIndex MatchedType = *InputType;
			bool bFoundMatch = false;
			bool bFoundPerfectMatch = false;

			// Using a map to collect all permutations that we can keep/remove
			// instead of removing them one by one, which can be costly
			TMap<int32, bool> PermutationsToKeep;
			const int32 NumAllTypes = AllTypes.Num();
			PermutationsToKeep.Reserve(NumAllTypes);

			for (int32 PermutationIndex = 0; PermutationIndex < NumAllTypes; PermutationIndex++)
			{
				const TRigVMTypeIndex& Type = AllTypes[PermutationIndex];
				if(!InRegistry->CanMatchTypes_NoLock(Type, *InputType, bAllowFloatingPointCasts))
				{
					PermutationsToKeep.Add(PermutationIndex, false);
				}
				else
				{
					PermutationsToKeep.Add(PermutationIndex, true);
					bFoundMatch = true;

					// if the type matches - but it's not the exact same
					if(!bFoundPerfectMatch)
					{
						MatchedType = Type;

						// if we found the perfect match - let's stop here
						if(Type == *InputType)
						{
							bFoundPerfectMatch = true;
						}
					}
				}
			}

			OutPermutationIndices = OutPermutationIndices.FilterByPredicate([&PermutationsToKeep](int32 PermutationIndex) -> bool
			{
				const bool* Value = PermutationsToKeep.Find(PermutationIndex);
				return Value ? *Value : false;
			});
			
			if(bFoundMatch)
			{
				InOutTypes.Add(Argument.Name,MatchedType);

				continue;
			}
		}

		const FRigVMTemplateArgument::EArrayType ArrayType = Argument.GetArrayType_NoLock(InRegistry);
		if(ArrayType == FRigVMTemplateArgument::EArrayType_Mixed)
		{
			InOutTypes.Add(Argument.Name, RigVMTypeUtils::TypeIndex::WildCard);

			if(InputType)
			{
				if(InRegistry->IsArrayType_NoLock(*InputType))
				{
					InOutTypes.FindChecked(Argument.Name) = RigVMTypeUtils::TypeIndex::WildCardArray;
				}
			}
		}
		else if(ArrayType == FRigVMTemplateArgument::EArrayType_ArrayValue)
		{
			InOutTypes.Add(Argument.Name, RigVMTypeUtils::TypeIndex::WildCardArray);
		}
		else
		{
			InOutTypes.Add(Argument.Name, RigVMTypeUtils::TypeIndex::WildCard);
		}
	}

	if (OutPermutationIndices.Num() == 1)
	{
		InOutTypes.Reset();
		for (const FRigVMTemplateArgument& Argument : Arguments)
		{
			InOutTypes.Add(Argument.Name, Argument.GetTypeIndex_NoLock(OutPermutationIndices[0], InRegistry));
		}
	}
	else if (OutPermutationIndices.Num() > 1)
	{
		for (const FRigVMTemplateArgument& Argument : Arguments)
		{
			if (Argument.IsSingleton_NoLock(OutPermutationIndices, InRegistry))
			{
				InOutTypes.FindChecked(Argument.Name) = Argument.GetTypeIndex_NoLock(OutPermutationIndices[0], InRegistry);
			}
		}
	}

	return !OutPermutationIndices.IsEmpty();
}

uint32 FRigVMTemplate::GetTypesHashFromTypes(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	// It is only a valid type map if it includes all arguments, and non of the types is a wildcard
	
	uint32 TypeHash = 0;
	if (InTypes.Num() != NumArguments())
	{
		return TypeHash;
	}

	for (const TPair<FName, TRigVMTypeIndex>& Pair : InTypes)
	{
		if (InRegistry->IsWildCardType_NoLock(Pair.Value))
		{
			return TypeHash;
		}
	}

	for (const FRigVMTemplateArgument& Argument : Arguments)
	{
		const TRigVMTypeIndex* ArgType = InTypes.Find(Argument.Name);
		if (!ArgType)
		{
			return 0;
		}
		TypeHash = HashCombine(TypeHash, GetTypeHash(*ArgType));
	}
	return TypeHash;
}

bool FRigVMTemplate::ContainsPermutation(const FTypeMap& InTypes) const
{
	return ContainsPermutation_NoLock(InTypes, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::ContainsPermutation_NoLock(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	// If they type map is valid (full description of arguments), then we can rely on
	// the TypesHashToPermutation cache. Otherwise, we will have to search for a specific permutation
	// by filtering types
	const uint32 TypesHash = GetTypesHashFromTypes(InTypes, InRegistry);
	if (TypesHash != 0)
	{
		return TypesHashToPermutation.Contains(TypesHash);
	}

#if WITH_EDITOR
	TArray<int32> PossiblePermutations;
	for (const TPair<FName, TRigVMTypeIndex>& Pair : InTypes)
	{
		if (const FRigVMTemplateArgument* Argument = FindArgument(Pair.Key))
		{
			const TArray<int32>& ArgumentPermutations = Argument->GetPermutations_NoLock(Pair.Value, InRegistry);
			if (!ArgumentPermutations.IsEmpty())
			{
				// If possible permutations is empty, initialize it
				if (PossiblePermutations.IsEmpty())
				{
					PossiblePermutations = ArgumentPermutations;
				}
				else
				{
					// Intersect possible permutations and the permutations found for this argument
					PossiblePermutations = ArgumentPermutations.FilterByPredicate([PossiblePermutations](const int32& ArgPermutation)
					{
						return PossiblePermutations.Contains(ArgPermutation);
					});
					if (PossiblePermutations.IsEmpty())
					{
						return false;
					}
				}
			}
			else
			{
				// The argument does not support the given type
				return false;
			}
		}
		else
		{
			// The argument cannot be found
			return false;
		}
	}
	
	return true;
#else
	return false;
#endif
}

bool FRigVMTemplate::ResolveArgument(const FName& InArgumentName, const TRigVMTypeIndex InTypeIndex,
	FTypeMap& InOutTypes) const
{
	return ResolveArgument_NoLock(InArgumentName, InTypeIndex, InOutTypes, FRigVMRegistryReadLock());
}

bool FRigVMTemplate::ResolveArgument_NoLock(const FName& InArgumentName, const TRigVMTypeIndex InTypeIndex,
	FTypeMap& InOutTypes, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	auto RemoveWildCardTypes = [&InRegistry](const FTypeMap& InTypes)
	{
		FTypeMap FilteredTypes;
		for(const FTypePair& Pair: InTypes)
		{
			if(!InRegistry->IsWildCardType_NoLock(Pair.Value))
			{
				FilteredTypes.Add(Pair);
			}
		}
		return FilteredTypes;
	};

	// remove all wildcards from map
	InOutTypes = RemoveWildCardTypes(InOutTypes);

	// first resolve with no types given except for the new argument type
	FTypeMap ResolvedTypes;
	ResolvedTypes.Add(InArgumentName, InTypeIndex);
	TArray<int32> PermutationIndices;
	FTypeMap RemainingTypesToResolve;
	
	if(Resolve_NoLock(ResolvedTypes, PermutationIndices, true, InRegistry))
	{
		// let's see if the input argument resolved into the expected type
		const TRigVMTypeIndex ResolvedInputType = ResolvedTypes.FindChecked(InArgumentName);
		if(!InRegistry->CanMatchTypes_NoLock(ResolvedInputType, InTypeIndex, true))
		{
			return false;
		}
		
		ResolvedTypes = RemoveWildCardTypes(ResolvedTypes);
		
		// remove all argument types from the reference list
		// provided from the outside. we cannot resolve these further
		auto RemoveResolvedTypesFromRemainingList = [](
			FTypeMap& InOutTypes, const FTypeMap& InResolvedTypes, FTypeMap& InOutRemainingTypesToResolve)
		{
			InOutRemainingTypesToResolve = InOutTypes;
			for(const FTypePair& Pair: InOutTypes)
			{
				if(InResolvedTypes.Contains(Pair.Key))
				{
					InOutRemainingTypesToResolve.Remove(Pair.Key);
				}
			}
			InOutTypes = InResolvedTypes;
		};

		RemoveResolvedTypesFromRemainingList(InOutTypes, ResolvedTypes, RemainingTypesToResolve);

		// if the type hasn't been specified we need to slowly resolve the template
		// arguments until we hit a match. for this we'll create a list of arguments
		// to resolve and reduce the list slowly.
		bool bSuccessFullyResolvedRemainingTypes = true;
		while(!RemainingTypesToResolve.IsEmpty())
		{
			PermutationIndices.Reset();

			const FTypePair TypeToResolve = *RemainingTypesToResolve.begin();
			FTypeMap NewResolvedTypes = RemoveWildCardTypes(ResolvedTypes);
			NewResolvedTypes.FindOrAdd(TypeToResolve.Key) = TypeToResolve.Value;

			if(Resolve_NoLock(NewResolvedTypes, PermutationIndices, true, InRegistry))
			{
				ResolvedTypes = NewResolvedTypes;
				RemoveResolvedTypesFromRemainingList(InOutTypes, ResolvedTypes, RemainingTypesToResolve);
			}
			else
			{
				// we were not able to resolve this argument, remove it from the resolved types list.
				RemainingTypesToResolve.Remove(TypeToResolve.Key);
				bSuccessFullyResolvedRemainingTypes = false;
			}
		}

		// if there is nothing left to resolve we were successful
		return RemainingTypesToResolve.IsEmpty() && bSuccessFullyResolvedRemainingTypes;
	}

	return false;
}

FRigVMTemplateTypeMap FRigVMTemplate::GetTypesForPermutation(const int32 InPermutationIndex) const
{
	return GetTypesForPermutation_NoLock(InPermutationIndex, FRigVMRegistryReadLock());
}

FRigVMTemplateTypeMap FRigVMTemplate::GetTypesForPermutation_NoLock(const int32 InPermutationIndex, const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	FTypeMap TypeMap;
	for (int32 ArgIndex = 0; ArgIndex < NumArguments(); ++ArgIndex)
	{
		const FRigVMTemplateArgument* Argument = GetArgument(ArgIndex);
		const TOptional<TRigVMTypeIndex> TypeIndex = Argument->TryToGetTypeIndex_NoLock(InPermutationIndex, InRegistry);
		if(TypeIndex.IsSet())
		{
			if (IsIgnoredType(TypeIndex.GetValue()))
			{
				TypeMap.Reset();
				return TypeMap;
			}
			TypeMap.Add(Argument->GetName(), TypeIndex.GetValue());
		}
		else
		{
			TypeMap.Reset();
			return TypeMap;
		}
	}
	return TypeMap;
}

#if WITH_EDITOR

FString FRigVMTemplate::GetCategory() const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		return Factory->GetCategory();
	}
	
	FString Category;
	GetPrimaryPermutation()->Struct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &Category);

	if (Category.IsEmpty())
	{
		return Category;
	}

	for (int32 PermutationIndex = 1; PermutationIndex < NumPermutations(); PermutationIndex++)
	{
		FString OtherCategory;
		if (GetPermutation(PermutationIndex)->Struct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &OtherCategory))
		{
			while (!OtherCategory.StartsWith(Category, ESearchCase::IgnoreCase))
			{
				FString Left;
				if (Category.Split(TEXT("|"), &Left, nullptr, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
				{
					Category = Left;
				}
				else
				{
					return FString();
				}

			}
		}
	}

	return Category;
}

FString FRigVMTemplate::GetKeywords() const
{
	if(const FRigVMDispatchFactory* Factory = GetDispatchFactory())
	{
		return Factory->GetKeywords();
	}

	TArray<FString> KeywordsMetadata;
	KeywordsMetadata.Add(GetName().ToString());

	for (int32 PermutationIndex = 0; PermutationIndex < NumPermutations(); PermutationIndex++)
	{
		if (const FRigVMFunction* Function = GetPermutation(PermutationIndex))
		{
			KeywordsMetadata.Add(Function->Struct->GetDisplayNameText().ToString());
			
			FString FunctionKeyWordsMetadata;
			Function->Struct->GetStringMetaDataHierarchical(FRigVMStruct::KeywordsMetaName, &FunctionKeyWordsMetadata);
			if (!FunctionKeyWordsMetadata.IsEmpty())
			{
				KeywordsMetadata.Add(FunctionKeyWordsMetadata);
			}
		}
	}

	return RigVMStringUtils::JoinStrings(KeywordsMetadata, TEXT(","));
}

#endif

bool FRigVMTemplate::UpdateAllArgumentTypesSlow(FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	const int32 PrimaryArgumentIndex = Arguments.IndexOfByPredicate([](const FRigVMTemplateArgument& Argument)
	{
		return Argument.bUseCategories;
	});

	// this template may not be affected at all by this
	if(PrimaryArgumentIndex == INDEX_NONE)
	{
		return true;
	}

	InvalidateHash();

	for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		FRigVMTemplateArgument& Argument = Arguments[ArgumentIndex];
		if(Argument.bUseCategories || Argument.IsSingleton_NoLock({}, InRegistry))
		{
			continue;
		}

		Argument.TypeIndices.Reset();
#if WITH_EDITOR
		Argument.TypeToPermutations.Reset();
#endif
	}

	const FRigVMTemplateArgument& PrimaryArgument = Arguments[PrimaryArgumentIndex];
	TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>> TypesArray;
	bool bResult = true;

	const FRigVMDispatchFactory* Factory = nullptr;
	if(Delegates.GetDispatchFactoryDelegate.IsBound())
	{
		Factory = Delegates.GetDispatchFactoryDelegate.Execute(InRegistry);
		ensure(Factory);
	}

	PrimaryArgument.ForEachType([&](const TRigVMTypeIndex PrimaryTypeIndex)
	{
		if(!UpdateArgumentTypes_Impl(PrimaryArgument, PrimaryTypeIndex, InRegistry, Factory, TypesArray))
		{
			bResult = false;
		}
		return true;
	}, InRegistry);

#if WITH_EDITOR
	for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		FRigVMTemplateArgument& Argument = Arguments[ArgumentIndex];
		if(Argument.bUseCategories)
		{
			Argument.UpdateTypeToPermutationsSlow(InRegistry);
		}
	}
#endif

	return bResult;
}

bool FRigVMTemplate::UpdateArgumentTypes_Impl(
	const FRigVMTemplateArgument& InPrimaryArgument,
	const TRigVMTypeIndex InPrimaryTypeIndex,
	FRigVMRegistryHandle& InRegistry, 
	const FRigVMDispatchFactory* InFactory,
	TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& InOutTypesArray)
{
	InOutTypesArray.Reset();
	if(InFactory)
	{
		InFactory->GetPermutationsFromArgumentType(InPrimaryArgument.Name, InPrimaryTypeIndex, InOutTypesArray, InRegistry);
	}
	else if(OnNewArgumentType().IsBound())
	{
		FRigVMTemplateTypeMap Types = OnNewArgumentType().Execute(InPrimaryArgument.Name, InPrimaryTypeIndex, InRegistry);
		InOutTypesArray.Add(Types);
	}

	if (!InOutTypesArray.IsEmpty())
	{
		for (FRigVMTemplateTypeMap& Types : InOutTypesArray)
		{
			if(Types.Num() == Arguments.Num())
			{
				for (TPair<FName, TRigVMTypeIndex>& ArgumentAndType : Types)
				{
					// similar to FRigVMTemplateArgument::EnsureValidExecuteType
					InRegistry->ConvertExecuteContextToBaseType_NoLock(ArgumentAndType.Value);
				}

				// Find if these types were already registered
				if (ContainsPermutation_NoLock(Types, InRegistry))
				{
					return true;
				}

				uint32 TypeHash=0;
				for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
				{
					FRigVMTemplateArgument& Argument = Arguments[ArgumentIndex];

					const TRigVMTypeIndex* TypeIndexPtr = Types.Find(Argument.Name);
					if(TypeIndexPtr == nullptr)
					{
						return false;
					}
					if(*TypeIndexPtr == INDEX_NONE)
					{
						return false;
					}

					const TRigVMTypeIndex& TypeIndex = *TypeIndexPtr;
					TypeHash = HashCombine(TypeHash, GetTypeHash(TypeIndex));
#if WITH_EDITOR
					Argument.TypeToPermutations.FindOrAdd(TypeIndex).Add(Permutations.Num());
#endif
					if(Argument.bUseCategories || Argument.IsSingleton_NoLock({}, InRegistry))
					{
						continue;
					}
					Argument.TypeIndices.Add(TypeIndex);
				}

				Permutations.Add(INDEX_NONE);
				TypesHashToPermutation.Add(TypeHash, Permutations.Num()-1);
			}
			else
			{
				return false;
			}
		}
	}
	else
	{
		return false;
	}
	
	return true;
}

bool FRigVMTemplate::IsIgnoredType(const TRigVMTypeIndex& InTypeIndex) const
{
	return TypeIndicesToIgnore.Contains(InTypeIndex);
}

bool FRigVMTemplate::IsIgnoredType(const FName& InTypeName) const
{
	return TypeNamesToIgnore.Contains(InTypeName);
}

bool FRigVMTemplate::HandlePropagatedArgumentType(const TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	const int32 PrimaryArgumentIndex = Arguments.IndexOfByPredicate([](const FRigVMTemplateArgument& Argument)
	{
		return Argument.bUseCategories;
	});

	// this template may not be affected at all by this
	if(PrimaryArgumentIndex == INDEX_NONE)
	{
		return true;
	}

	// clear out the type to ignore as it is being added back
	if (TypeIndicesToIgnore.Remove(InTypeIndex) > 0)
	{
		TypeNamesToIgnore.Remove(InRegistry->GetType_NoLock(InTypeIndex).CPPType);
	}

	InvalidateHash();

	const FRigVMTemplateArgument& PrimaryArgument = Arguments[PrimaryArgumentIndex];
	TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>> TypesArray;

	const FRigVMDispatchFactory* Factory = nullptr;
	if(Delegates.GetDispatchFactoryDelegate.IsBound())
	{
		Factory = Delegates.GetDispatchFactoryDelegate.Execute(InRegistry);
		ensure(Factory);
	}

	const bool bResult = UpdateArgumentTypes_Impl(PrimaryArgument, InTypeIndex, InRegistry, Factory, TypesArray);
#if WITH_EDITOR
	if(bResult)
	{
		for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
		{
			FRigVMTemplateArgument& Argument = Arguments[ArgumentIndex];
			if(Argument.bUseCategories)
			{
				Argument.UpdateTypeToPermutationsSlow(InRegistry);
			}
		}
	}
#endif

	// we may need to update the template permutations since we have have registered a new type
	// in between other types - especially if the template uses multiple categories somewhere
	bool bUsesMultipleCategories = false; 
	for(int32 ArgumentIndex = 0; ArgumentIndex < Arguments.Num(); ArgumentIndex++)
	{
		FRigVMTemplateArgument& Argument = Arguments[ArgumentIndex];
		if(Argument.bUseCategories && Argument.TypeCategories.Num() > 1)
		{
			bUsesMultipleCategories = true;
			break;
		}
	}
	if(bUsesMultipleCategories)
	{
		TArray<int32> ValidFunctionIndices;
		bool bHasChangedPermutations = false;

		for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
		{
			const int32& FunctionIndex = Permutations[PermutationIndex];
			if(FunctionIndex == INDEX_NONE)
			{
				continue;
			}
			ValidFunctionIndices.Add(FunctionIndex);

			FRigVMTemplateTypeMap Types;
			Types.Reserve(OwnerRegistry->Functions[FunctionIndex].Arguments.Num());
			for(const FRigVMFunctionArgument& Argument : OwnerRegistry->Functions[FunctionIndex].Arguments)
			{
				const TRigVMTypeIndex TypeIndex = OwnerRegistry->GetTypeIndexFromCPPType_NoLock(Argument.Type);
				check(TypeIndex != INDEX_NONE);
				Types.Add(Argument.Name, TypeIndex);
			}
			
			const int32 NewPermutationIndex = FindPermutation_NoLock(Types, InRegistry);
			if(NewPermutationIndex != PermutationIndex)
			{
				OwnerRegistry->Functions[FunctionIndex].PermutationIndex = NewPermutationIndex;
				bHasChangedPermutations = true;
			}
		}

		if(bHasChangedPermutations)
		{
			for(int32 PermutationIndex = 0; PermutationIndex < Permutations.Num(); PermutationIndex++)
			{
				Permutations[PermutationIndex] = INDEX_NONE;
				UpdateTypesIndicesToPermutationIndices(PermutationIndex, InRegistry);
			}
			for(int32 FunctionIndex : ValidFunctionIndices)
			{
				int32 PermutationIndex = OwnerRegistry->Functions[FunctionIndex].PermutationIndex;
				Permutations[PermutationIndex] = FunctionIndex;
				UpdateTypesIndicesToPermutationIndices(PermutationIndex, InRegistry);
			}
		}
	}

	return bResult;
}

void FRigVMTemplate::UpdateTypesIndicesToPermutationIndices(int32 InPermutationIndex, const FRigVMRegistryHandle& InRegistry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMTemplate::UpdateTypesIndicesToPermutationIndices);

	const FRigVMTemplateTypeMap TypeMap = GetTypesForPermutation_NoLock(InPermutationIndex, InRegistry);

	for (const TPair<FName, TRigVMTypeIndex>& Pair : TypeMap)
	{
		TArray<int32>& PermutationIndices = TypesIndicesToPermutationIndices.FindOrAdd(Pair.Value);
		PermutationIndices.AddUnique(InPermutationIndex);
	}
}

void FRigVMTemplate::HandleTypeRemoval(TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMTemplate::HandleTypeRemoval);

	check(InRegistry.GetRegistry() == OwnerRegistry);
	if (IsIgnoredType(InTypeIndex))
	{
		return;
	}

	InvalidateHash();

	TArray<int32> PermutationIndices;
	TArray<int32>* CachedPermutationIndices = TypesIndicesToPermutationIndices.Find(InTypeIndex);
	if (CachedPermutationIndices)
	{
		for (int32 PermutationIndex : *CachedPermutationIndices)
		{
			if (!RemovedPermutationIndices.Contains(PermutationIndex))
			{
				PermutationIndices.Add(PermutationIndex);
			}
		}
	}

#if UE_BUILD_DEBUG

	// The fast path above is tricky, so we'll also run the slow path, and confirm we found the right indices
	TArray<int32> SlowPermutationIndices;
	const int32 CurrentNumPermutations = NumPermutations_NoLock(InRegistry);
	for (int32 PermutationIndex = 0; PermutationIndex < CurrentNumPermutations; PermutationIndex++)
	{
		const FRigVMTemplateTypeMap TypeMap = GetTypesForPermutation_NoLock(PermutationIndex, InRegistry);

		for (const TPair<FName, TRigVMTypeIndex>& Pair : TypeMap)
		{
			if (Pair.Value == InTypeIndex)
			{
				if (const FRigVMFunction* Permutation = GetPermutation_NoLock(PermutationIndex, InRegistry))
				{
					SlowPermutationIndices.Add(PermutationIndex);
				}
				break;
			}
		}
	}

	// Verify our indices
	if (PermutationIndices.Num() != SlowPermutationIndices.Num())
	{
		UE_LOGF(LogRigVM, Error, "HandleTypeRemoval: bad permutation count %d v %d (%d)",
		PermutationIndices.Num(), SlowPermutationIndices.Num(), PermutationIndices.Num() > 0 ? PermutationIndices[0] : SlowPermutationIndices[0]);
	}
	else
	{
		for (int32 TestIndex = 0; TestIndex < PermutationIndices.Num(); TestIndex++)
		{
			if (PermutationIndices[TestIndex] != SlowPermutationIndices[TestIndex])
			{
				UE_LOGF(LogRigVM, Error, "HandleTypeRemoval: bad permutation index %d v %d",
					PermutationIndices[TestIndex], SlowPermutationIndices[TestIndex]);
			}
		}
	}

	PermutationIndices = SlowPermutationIndices;

#endif // UE_BUILD_DEBUG

	for (int32 PermutationIndex : PermutationIndices)
	{
		if (const FRigVMFunction* Permutation = GetPermutation_NoLock(PermutationIndex, InRegistry))
		{
			const int32 FunctionIndex = Permutation->Index;
			InRegistry->RemoveFunction_NoLock(FunctionIndex);
			Permutations[PermutationIndex] = INDEX_NONE;
		}

		RemovedPermutationIndices.Add(PermutationIndex);
	}

	TypeIndicesToIgnore.Add(InTypeIndex);
	TypeNamesToIgnore.Add(InRegistry->GetType_NoLock(InTypeIndex).CPPType);

#if WITH_EDITOR
	for (FRigVMTemplateArgument& Argument : Arguments)
	{
		Argument.TypeToPermutations.Remove(InTypeIndex);
	}
#endif
}

const FRigVMDispatchFactory* FRigVMTemplate::GetDispatchFactory() const
{
	FRigVMRegistryReadLock Lock;
	return GetDispatchFactory_NoLock(Lock);
}

const FRigVMDispatchFactory* FRigVMTemplate::GetDispatchFactory_NoLock(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	return UsesDispatch() ? Delegates.GetDispatchFactoryDelegate.Execute(InRegistry) : nullptr;
}

void FRigVMTemplate::RecomputeTypesHashToPermutations(const FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	TArray<FRigVMTypeCacheScope_NoLock> TypeCaches;
	TypeCaches.SetNum(NumArguments());
	
	for(int32 ArgIndex = 0; ArgIndex < NumArguments(); ArgIndex++)
	{
		(void)TypeCaches[ArgIndex].UpdateIfRequired(InRegistry, Arguments[ArgIndex]);
	}
	
	RecomputeTypesHashToPermutations(TypeCaches, InRegistry);
}

void FRigVMTemplate::RecomputeTypesHashToPermutations(const TArray<FRigVMTypeCacheScope_NoLock>& InTypeCaches, const FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	TypesHashToPermutation.Reset();

	bool bAnyArgumentWithZeroTypes = false;
	for(int32 ArgIndex = 0; ArgIndex < InTypeCaches.Num(); ArgIndex++)
	{
		bAnyArgumentWithZeroTypes = bAnyArgumentWithZeroTypes || InTypeCaches[ArgIndex].GetNumTypes_NoLock() == 0;
	}

	if(!bAnyArgumentWithZeroTypes)
	{
		for (int32 PermutationIndex=0; PermutationIndex<NumPermutations_NoLock(InRegistry); ++PermutationIndex)
		{
			uint32 TypesHash=0;
			for (int32 ArgIndex=0; ArgIndex < InTypeCaches.Num(); ++ArgIndex)
			{
				TypesHash = HashCombine(TypesHash, GetTypeHash(InTypeCaches[ArgIndex].GetTypeIndex_NoLock(PermutationIndex)));
			}
			TypesHashToPermutation.Add(TypesHash, PermutationIndex);
		}
	}
}

void FRigVMTemplate::UpdateTypesHashToPermutation(const int32 InPermutation, const FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	if (!Permutations.IsValidIndex(InPermutation))
	{
		return;
	}

	uint32 TypeHash=0;
	for (const FRigVMTemplateArgument& Argument : Arguments)
	{
		TypeHash = HashCombine(TypeHash, GetTypeHash(Argument.GetTypeIndex_NoLock(InPermutation, InRegistry)));
	}
	TypesHashToPermutation.Add(TypeHash, InPermutation);
}

TArray<FRigVMTemplateArgumentInfo> FRigVMTemplate::GetFlattenedArgumentInfos_NoLock(FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	TArray<FRigVMTemplateArgumentInfo> Infos;
	for (const FRigVMTemplateArgument& Argument : Arguments)
	{
		TArray<TRigVMTypeIndex> TypeIndices;
		Argument.GetAllTypes_NoLock(TypeIndices, InRegistry);
		check(!TypeIndices.IsEmpty());
		
		if (Argument.IsSingleton_NoLock({}, InRegistry))
		{
			Infos.Add(FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(Argument.Name, Argument.Direction, {TypeIndices[0]}));
		}
		else
		{
			Infos.Add(FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(Argument.Name, Argument.Direction, TypeIndices));
		}
	}
	return Infos;
}

TArray<FRigVMTemplateArgumentInfo> FRigVMTemplate::GetFlattenedArgumentInfosForFunctions_NoLock(const TArray<const FRigVMFunction*>& InFunctions, FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	TMap<FName, TArray<TRigVMTypeIndex>> TypeIndices;
	
	for(const FRigVMFunction* Function : InFunctions)
	{
		check(Function->OwnerRegistry == this->OwnerRegistry);
		check(Function->TemplateIndex == this->Index);

		TArray<TRigVMTypeIndex> TypeIndicesForFunction;
		Function->GetArgumentTypeIndices_NoLock(InRegistry, TypeIndicesForFunction);
		check(TypeIndicesForFunction.Num() == Function->Arguments.Num());

		for (int32 ArgumentIndex = 0; ArgumentIndex < Function->Arguments.Num(); ArgumentIndex++)
		{
			TypeIndices.FindOrAdd(Function->Arguments[ArgumentIndex].Name).Add(TypeIndicesForFunction[ArgumentIndex]);
		}
	}

	TArray<FRigVMTemplateArgumentInfo> Infos;
	for(const FRigVMTemplateArgument& Argument : Arguments)
	{
		TArray<TRigVMTypeIndex> ArgumentTypeIndices = TypeIndices.FindChecked(Argument.Name);
		check(!ArgumentTypeIndices.IsEmpty());

		bool bIsSingleton = false;
		if(ArgumentTypeIndices.Num() > 1)
		{
			bIsSingleton = true;
			const TRigVMTypeIndex FirstTypeIndex = ArgumentTypeIndices[0];
			for (const TRigVMTypeIndex& TypeIndex : ArgumentTypeIndices)
			{
				if (FirstTypeIndex != TypeIndex)
				{
					bIsSingleton = false;
					break;
				}
			}
		}
	
		if(bIsSingleton)
		{
			Infos.Add(FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(Argument.Name, Argument.Direction, {ArgumentTypeIndices[0]}));
		}
		else
		{
			Infos.Add(FRigVMTemplateArgumentInfo::MakeFlattenedFromTypeIndices(Argument.Name, Argument.Direction, ArgumentTypeIndices));
		}
	}

	return Infos;
}

/*
bool FRigVMTemplate::SetPermutationsOverride_NoLock(const TArray<FRigVMTemplateTypeMap>& InPermutations, FRigVMRegistryHandle& InRegistry)
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	for(const FRigVMTemplateTypeMap& Permutation : InPermutations)
	{
		if(!Permutation.Num() == Template->NumArguments())
		{
			UE_LOGF(LogRigVM, Error, "%ls - SetPermutationsOverride_NoLock: Provided permutation has incorrect number of types (%d), expected %d.",
				*GetFactoryName().ToString(), Permutation.Num(), Template->NumArguments());
			return false;
		}
		for(const TPair<FName, TRigVMTypeIndex>& TypePair : Permutation)
		{
			const FRigVMTemplateArgument* Argument = Template->FindArgument(TypePair.Key);
			if(Argument == nullptr)
			{
				UE_LOGF(LogRigVM, Error, "%ls - SetPermutationsOverride_NoLock: Argument '%ls' not found in template.",
					*GetFactoryName().ToString(), TypePair.Key.ToString());
				return false;
			}

			if(!Argument->SupportsTypeIndex_NoLock(InRegistry, TypePair.Value))
			{
				const FRigVMTemplateArgumentType& Type = InRegistry->GetType_NoLock(TypePair.Value);
				UE_LOGF(LogRigVM, Error, "%ls - SetPermutationsOverride_NoLock: Argument '%ls' doesn't support type '%ls'.",
					*GetFactoryName().ToString(), TypePair.Key.ToString(), *Type.CPPType.ToString());
				return false;
			}
		}
	}
}
*/

uint32 GetTypeHash_NoLock(const FRigVMTemplateArgument& InArgument, const FRigVMRegistryHandle& InRegistry)
{
	uint32 Hash = GetTypeHash(InArgument.Name.ToString());
	Hash = HashCombine(Hash, GetTypeHash((int32)InArgument.Direction));
	InArgument.ForEachType([&](const TRigVMTypeIndex TypeIndex)
	{
		Hash = HashCombine(Hash, InRegistry->GetHashForType_NoLock(TypeIndex));
		return true;
	}, InRegistry);
	return Hash;
}

uint32 GetTypeHash(const FRigVMTemplate& InTemplate)
{
	return InTemplate.ComputeTypeHash(FRigVMRegistryReadLock());
}

uint32 GetTypeHash_NoLock(const FRigVMTemplate& InTemplate, const FRigVMRegistryHandle& InRegistry)
{
	if(InTemplate.Hash != UINT32_MAX)
	{
		return InTemplate.Hash;
	}

	uint32 Hash = GetTypeHash(InTemplate.GetNotation().ToString());
	for(const FRigVMTemplateArgument& Argument : InTemplate.Arguments)
	{
		Hash = HashCombine(Hash, GetTypeHash_NoLock(Argument, InRegistry));
	}

	// todo: in Dev-EngineMerge we should add the execute arguments to the hash as well

	if(const FRigVMDispatchFactory* Factory = InTemplate.GetDispatchFactory_NoLock(InRegistry))
	{
		Hash = HashCombine(Hash, GetTypeHash(Factory->GetFactoryName().ToString()));
	}

	InTemplate.Hash = Hash;
	return Hash;
}

uint32 FRigVMTemplate::ComputeTypeHash(const FRigVMRegistryHandle& InRegistry) const
{
	check(InRegistry.GetRegistry() == OwnerRegistry);

	return GetTypeHash_NoLock(*this, InRegistry);
}

FRigVMTypeCacheScope_NoLock::FRigVMTypeCacheScope_NoLock()
	: Registry(nullptr)
	, Argument(nullptr)
	, bShouldCopyTypes(false)
{
}

FRigVMTypeCacheScope_NoLock::FRigVMTypeCacheScope_NoLock(const FRigVMRegistryHandle& InRegistry, const FRigVMTemplateArgument& InArgument)
	: Registry(const_cast<FRigVMRegistry_NoLock*>(InRegistry.GetRegistry()))
	, Argument(&InArgument)
	, bShouldCopyTypes(InArgument.FilterType != nullptr)//  ||bUseCategories)??
{
}

FRigVMTypeCacheScope_NoLock::~FRigVMTypeCacheScope_NoLock()
{
}

const FRigVMTypeCacheScope_NoLock& FRigVMTypeCacheScope_NoLock::UpdateIfRequired(const FRigVMRegistryHandle& InRegistry, const FRigVMTemplateArgument& InArgument)
{
	if(Registry != InRegistry.GetRegistry())
	{
		Registry = const_cast<FRigVMRegistry_NoLock*>(InRegistry.GetRegistry());
		Argument = nullptr;
	}
	if(Argument != &InArgument)
	{
		Argument = &InArgument;
		bShouldCopyTypes = InArgument.FilterType != nullptr;
	}
	return *this;
}

int32 FRigVMTypeCacheScope_NoLock::GetNumTypes_NoLock() const
{
	check(Registry);
	if(!NumTypes.IsSet())
	{
		if(bShouldCopyTypes)
		{
			UpdateTypesIfRequired();
			NumTypes = Types.GetValue().Num();
		}
		else
		{
			check(Argument);
			NumTypes = Argument->GetNumTypes_NoLock(Registry->GetHandle_NoLock());
		}
	}
	return NumTypes.GetValue();
}

TRigVMTypeIndex FRigVMTypeCacheScope_NoLock::GetTypeIndex_NoLock(int32 InIndex) const
{
	check(Registry);
	if(!Types.IsSet())
	{
		if(bShouldCopyTypes)
		{
			UpdateTypesIfRequired();
		}
		else
		{
			return Argument->GetTypeIndex_NoLock(InIndex, Registry->GetHandle_NoLock());
		}
	}

	// singleton arguments may have only one type
	const TArray<TRigVMTypeIndex>& TypeArray = Types.GetValue();
	return TypeArray.IsValidIndex(InIndex) ? TypeArray[InIndex] : TypeArray[0];
}

void FRigVMTypeCacheScope_NoLock::UpdateTypesIfRequired() const
{
	check(Registry);
	check(bShouldCopyTypes);
	if(Types.IsSet())
	{
		return;
	}
	Types = TArray<TRigVMTypeIndex>();
	TArray<TRigVMTypeIndex>& TypesArray = Types.GetValue();
	TypesArray.Reserve(Argument->GetNumTypes_NoLock(Registry->GetHandle_NoLock()));

	Argument->ForEachType([&TypesArray](const TRigVMTypeIndex Type)
	{
		TypesArray.Add(Type);
		return true;
	}, Registry->GetHandle_NoLock());
}
