// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMUserWorkflow.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMUserWorkflow)

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool URigVMUserWorkflowOptions::RequiresDialog() const
{
	for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
	{
		if(const FProperty* Property = CastField<FProperty>(*It))
		{
			if(RequiresDialog(Property))
			{
				return true;
			}
		}
	}
	return false;
}

bool URigVMUserWorkflowOptions::RequiresDialog(const FProperty* InProperty) const
{
	if(!InProperty->HasAnyPropertyFlags(CPF_Edit))
	{
		return false;
	}
	if(InProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
	{
		return false;
	}
	return true;
}

void URigVMUserWorkflowOptions::Report(EMessageSeverity::Type InSeverity,  const FString& InMessage) const
{
	if (ReportDelegate.IsBound())
	{
		ReportDelegate.Execute(InSeverity, GetSubject(), InMessage);
	}
	else
	{
		if (InSeverity == EMessageSeverity::Error)
		{
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *InMessage, *FString());
		}
		else if (InSeverity == EMessageSeverity::Warning)
		{
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *InMessage, *FString());
		}
		else
		{
			UE_LOGF(LogRigVM, Display, "%ls", *InMessage);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

bool FRigVMUserWorkflow::Perform(const URigVMUserWorkflowOptions* InOptions, UObject* InController) const
{
	if(!IsValid() || !ValidateOptions(InOptions))
	{
		return false;
	}

	if(PerformDelegate.IsBound())
	{
		return PerformDelegate.Execute(InOptions, InController);
	}
	else if(PerformDynamicDelegate.IsBound())
	{
		return PerformDynamicDelegate.Execute(InOptions, InController);
	}
	
	return false;
}

bool FRigVMUserWorkflow::ValidateOptions(const URigVMUserWorkflowOptions* InOptions) const
{
	UClass* ExpectedOptionsClass = GetOptionsClass();
	if(!ensure(ExpectedOptionsClass != nullptr))
	{
		return false;
	}

	if(!ensure(ExpectedOptionsClass->IsChildOf(URigVMUserWorkflowOptions::StaticClass())))
	{
		return false;
	}

	if(!ensure(InOptions != nullptr))
	{
		return false;
	}

	if(!ensure(InOptions->IsValid()))
	{
		return false;
	}

	if(!ensure(InOptions->GetClass()->IsChildOf(ExpectedOptionsClass)))
	{
		InOptions->Reportf(
			EMessageSeverity::Error,
			TEXT("Workflow '%s' cannot execute: Options are of type '%s' - expected was type '%s'."),
			*GetTitle(),
			*InOptions->GetClass()->GetName(),
			*ExpectedOptionsClass->GetName());
		return false;
	}

	return true;
}

