// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCore/RigVMVariant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMVariant)

TAutoConsoleVariable<bool> CVarRigVMEnableVariants(TEXT("RigVM.EnableVariants"), true, TEXT("enables the UI bits to create a variant of assets / functions"));
