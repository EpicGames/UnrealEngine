// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Animation/RigVMFunction_AnimEasing.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_AnimEasing)

FRigVMFunction_AnimEasingType_Execute()
{
}

FRigVMFunction_AnimEasing_Execute()
{
	if (FMath::IsNearlyEqual(SourceMinimum, SourceMaximum))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum are the same."));
	}

	Result = FMath::Clamp<float>((Value - SourceMinimum) / (SourceMaximum - SourceMinimum), 0.f, 1.f);
	Result = FRigVMMathLibrary::EaseFloat(Result, Type);
	Result = FMath::Lerp<float>(TargetMinimum, TargetMaximum, Result);
}


