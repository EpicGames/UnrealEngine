// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Animation/RigVMFunction_AnimEvalRichCurve.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_AnimEvalRichCurve)

FRigVMFunction_AnimEvalRichCurve_Execute()
{
	if (FMath::IsNearlyEqual(SourceMinimum, SourceMaximum))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum are the same."));
	}

	Result = FMath::Clamp<float>((Value - SourceMinimum) / (SourceMaximum - SourceMinimum), 0.f, 1.f);

	if (Curve.GetRichCurveConst()->GetNumKeys() > 0)
	{
		Result = Curve.GetRichCurveConst()->Eval(Result, 0.f);
	}

	Result = FMath::Lerp<float>(TargetMinimum, TargetMaximum, Result);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AnimEvalRichCurve)
{
	Unit.Value = 0.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 0.f), TEXT("unexpected curve result"));
	Unit.Value = 0.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 0.5f), TEXT("unexpected curve result"));
	Unit.Value = 1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected curve result"));
	Unit.Value = 1.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected curve result"));
	return true;
}
#endif
