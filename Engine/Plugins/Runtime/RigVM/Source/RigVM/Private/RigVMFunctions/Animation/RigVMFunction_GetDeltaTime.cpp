// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Animation/RigVMFunction_GetDeltaTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_GetDeltaTime)

FRigVMFunction_GetDeltaTime_Execute()
{
	Result = ExecuteContext.GetDeltaTime<float>();
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_GetDeltaTime)
{
	ExecuteContext.SetDeltaTime(0.2);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 0.2f), TEXT("unexpected delta time"));
	return true;
}
#endif

FRigVMFunction_GetAbsoluteTime_Execute()
{
	Result = ExecuteContext.GetAbsoluteTime<float>();
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_GetAbsoluteTime)
{
	ExecuteContext.SetAbsoluteTime(7.0f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 7.0f), TEXT("unexpected absolute time"));
	return true;
}
#endif