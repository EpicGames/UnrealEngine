// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Animation/RigVMFunction_GetWorldTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_GetWorldTime)

FRigVMFunction_GetWorldTime_Execute()
{
	FDateTime Now = FDateTime::Now();
	Year = (float)Now.GetYear();
	Month = (float)Now.GetMonth();
	Day = (float)Now.GetDay();
	WeekDay = (float)Now.GetDayOfWeek();
	Hours = (float)Now.GetHour();
	Minutes = (float)Now.GetMinute();
	Seconds = float(Now.GetSecond()) + float(Now.GetMillisecond()) * 0.001f;
	OverallSeconds = (float)FPlatformTime::Seconds();
}


