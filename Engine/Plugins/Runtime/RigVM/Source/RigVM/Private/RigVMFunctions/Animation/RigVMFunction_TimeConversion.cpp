// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Animation/RigVMFunction_TimeConversion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_TimeConversion)

FRigVMFunction_FramesToSeconds_Execute()
{
    if(ExecuteContext.GetFramesPerSecond() > UE_SMALL_NUMBER)
    {
		Seconds = Frames / ExecuteContext.GetFramesPerSecond<float>();
	}
	else
	{
		Seconds = 0.f;
	}
}

FRigVMFunction_SecondsToFrames_Execute()
{
    if(ExecuteContext.GetFramesPerSecond() > UE_SMALL_NUMBER)
    {
		Frames = Seconds * ExecuteContext.GetFramesPerSecond<float>();
	}
	else
	{
		Frames = 0.f;
	}
}

