// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_DebugLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DebugLine)

FRigVMFunction_DebugLineNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, A, B, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}
