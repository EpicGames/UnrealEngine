// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_DebugLineStrip.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DebugLineStrip)

FRigVMFunction_DebugLineStripNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	ExecuteContext.GetDrawInterface()->DrawLineStrip(WorldOffset, TArrayView<const FVector>(Points.GetData(), Points.Num()), Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}

