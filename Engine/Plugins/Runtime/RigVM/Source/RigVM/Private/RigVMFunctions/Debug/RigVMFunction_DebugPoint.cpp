// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_DebugPoint.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DebugPoint)

FRigVMFunction_DebugPoint_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	static const FVector Center = FVector::ZeroVector;
	const FVector DrawVector = Vector;

	switch (Mode)
	{
		case ERigUnitDebugPointMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawVector, Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugPointMode::Vector:
		{
			ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, Center, DrawVector, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
	}
}

FRigVMStructUpgradeInfo FRigVMFunction_DebugPoint::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugVector NewNode;
	NewNode.Value = Vector;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.BoneSpace = Space;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Vector"), TEXT("Value"));
	Info.AddRemappedPin(TEXT("Space"), TEXT("BoneSpace"));
	return Info;
}

FRigVMFunction_DebugPointMutable_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	static const FVector Center = FVector::ZeroVector;
	const FVector DrawVector = Vector;

	switch (Mode)
	{
		case ERigUnitDebugPointMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawVector, Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugPointMode::Vector:
		{
			ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, Center, DrawVector, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
	}
}

FRigVMStructUpgradeInfo FRigVMFunction_DebugPointMutable::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

