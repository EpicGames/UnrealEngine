// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_DebugPrimitives.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DebugPrimitives)

FRigVMFunction_DebugRectangle_Execute()
{
	FRigVMFunction_DebugRectangleNoSpace::StaticExecute(
		ExecuteContext, 
		Transform,
		Color,
		Scale,
		Thickness,
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigVMFunction_DebugRectangle::GetUpgradeInfo() const
{
	FRigVMFunction_DebugRectangleNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Scale = Scale;
	NewNode.Thickness = Thickness;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMFunction_DebugRectangleNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	ExecuteContext.GetDrawInterface()->DrawRectangle(WorldOffset, Transform, Scale, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}

FRigVMFunction_DebugArc_Execute()
{
	FRigVMFunction_DebugArcNoSpace::StaticExecute(
		ExecuteContext, 
		Transform,
		Color,
		Radius,
		MinimumDegrees,
		MaximumDegrees,
		Thickness,
		Detail,
		WorldOffset, 
		bEnabled,
		DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigVMFunction_DebugArc::GetUpgradeInfo() const
{
	FRigVMFunction_DebugArcNoSpace NewNode;
	NewNode.Transform = Transform;
	NewNode.Color = Color;
	NewNode.Radius = Radius;
	NewNode.MinimumDegrees = MinimumDegrees;
	NewNode.MaximumDegrees = MaximumDegrees;
	NewNode.Thickness = Thickness;
	NewNode.Detail = Detail;
	NewNode.WorldOffset = WorldOffset;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	return Info;
}

FRigVMFunction_DebugArcNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	ExecuteContext.GetDrawInterface()->DrawArc(WorldOffset, Transform, Radius, FMath::DegreesToRadians(MinimumDegrees), FMath::DegreesToRadians(MaximumDegrees), Color, Thickness, Detail, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}

FRigVMFunction_DebugBoxNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled || !Box.IsValid)
	{
		return;
	}

	FTransform Transform = FTransform::Identity;
	Transform.SetTranslation(Box.GetCenter());
	Transform.SetScale3D(Box.GetSize());
	ExecuteContext.GetDrawInterface()->DrawBox(WorldOffset, Transform, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}
