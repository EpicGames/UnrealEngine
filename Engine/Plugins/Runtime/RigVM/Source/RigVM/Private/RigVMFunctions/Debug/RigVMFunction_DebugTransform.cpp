// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_DebugTransform.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DebugTransform)

FRigVMFunction_DebugTransformMutableNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	FTransform DrawTransform = Transform;

	switch (Mode)
	{
		case ERigUnitDebugTransformMode::Axes:
		{
			ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, DrawTransform, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, DrawTransform.GetLocation(), Scale, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitDebugTransformMode::Box:
		{
			DrawTransform.SetScale3D(DrawTransform.GetScale3D() * Scale);
			ExecuteContext.GetDrawInterface()->DrawBox(WorldOffset, DrawTransform, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
	}
}

FRigVMFunction_DebugTransformArrayMutableNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled || Transforms.Num() == 0)
	{
		return;
	}

	for(const FTransform& Transform : Transforms)
	{
		FRigVMFunction_DebugTransformMutableNoSpace::StaticExecute(
			ExecuteContext, 
			Transform,
			Mode,
			Color,
			Thickness,
			Scale,
			WorldOffset, 
			bEnabled,
			DebugDrawSettings);
	}

	if(ParentIndices.Num() == Transforms.Num())
	{
		for(int32 Index = 0; Index < ParentIndices.Num(); Index++)
		{
			if(Transforms.IsValidIndex(ParentIndices[Index]))
			{
				ExecuteContext.GetDrawInterface()->DrawLine(
					WorldOffset,
					Transforms[Index].GetTranslation(),
					Transforms[ParentIndices[Index]].GetTranslation(),
					Color,
					Thickness,
					DebugDrawSettings.DepthPriority,
					DebugDrawSettings.Lifetime
				);
			}
		}
	}
}

