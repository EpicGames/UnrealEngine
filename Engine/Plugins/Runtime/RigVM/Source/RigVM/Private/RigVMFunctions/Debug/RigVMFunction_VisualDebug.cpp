// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_VisualDebug)

FRigVMFunction_VisualDebugVector_Execute()
{
	FRigVMFunction_VisualDebugVectorNoSpace::StaticExecute(ExecuteContext, Value, bEnabled, Mode, Color, Thickness, Scale, DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigVMFunction_VisualDebugVector::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugVectorNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Color = Color;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("BoneSpace"), TEXT("Space.Name"));
	return Info;
}

FRigVMFunction_VisualDebugVectorNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	static const FTransform WorldOffset = FTransform::Identity;

	switch(Mode)
	{
		case ERigUnitVisualDebugPointMode::Point:
		{
			ExecuteContext.GetDrawInterface()->DrawPoint(WorldOffset, Value, Thickness, Color, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		case ERigUnitVisualDebugPointMode::Vector:
		{
			ExecuteContext.GetDrawInterface()->DrawLine(WorldOffset, FVector::ZeroVector, Value * Scale, Color, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}
}

FRigVMFunction_VisualDebugQuat_Execute()
{
	FRigVMFunction_VisualDebugQuatNoSpace::StaticExecute(ExecuteContext, Value, bEnabled, Thickness, Scale, DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigVMFunction_VisualDebugQuat::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugQuatNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("BoneSpace"), TEXT("Space.Name"));
	return Info;
}

FRigVMFunction_VisualDebugQuatNoSpace_Execute()
{
    FTransform Transform = FTransform::Identity;
    Transform.SetRotation(Value);

	FRigVMFunction_VisualDebugTransformNoSpace::StaticExecute(ExecuteContext, Transform, bEnabled, Thickness, Scale, DebugDrawSettings);
}

FRigVMFunction_VisualDebugTransform_Execute()
{
	FRigVMFunction_VisualDebugTransformNoSpace::StaticExecute(ExecuteContext, Value, bEnabled, Thickness, Scale, DebugDrawSettings);
}

FRigVMStructUpgradeInfo FRigVMFunction_VisualDebugTransform::GetUpgradeInfo() const
{
	FRigVMFunction_VisualDebugTransformNoSpace NewNode;
	NewNode.Value = Value;
	NewNode.Thickness = Thickness;
	NewNode.Scale = Scale;
	NewNode.bEnabled = bEnabled;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("BoneSpace"), TEXT("Space.Name"));
	return Info;
}

FRigVMFunction_VisualDebugTransformNoSpace_Execute()
{
	if (ExecuteContext.GetDrawInterface() == nullptr || !bEnabled)
	{
		return;
	}

	static const FTransform WorldOffset = FTransform::Identity;

	ExecuteContext.GetDrawInterface()->DrawAxes(WorldOffset, Value, Scale, Thickness, DebugDrawSettings.DepthPriority, DebugDrawSettings.Lifetime);
}
