// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Debug/RigVMFunction_VisualLog.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_VisualLog)

namespace UE::RigVM::Private
{
	constexpr ELogVerbosity::Type DefaultVerbosity = ELogVerbosity::Display;
}

FRigVMFunction_VisualLogText_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::CategorizedLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogLocation_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::LocationLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Location, static_cast<uint16>(Radius), ObjectColor.ToFColor(true), TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogSphere_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::SphereLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Center, Radius, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogCone_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::ConeLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Origin, Direction, Length, Angle, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogCylinder_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::CylinderLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Start, End, Radius, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogCapsule_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::CapsuleLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Base, HalfHeight, Radius, Rotation, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogBox_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::BoxLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Box, FMatrix::Identity, ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogOrientedBox_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::BoxLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Box, Transform.ToMatrixWithScale(), ObjectColor.ToFColor(true), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogArrow_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::ArrowLineLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, SegmentStart, SegmentEnd, ObjectColor.ToFColor(true), static_cast<uint16>(ArrowHeadSize), TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogCircle_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::DiscLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, Center, UpAxis, Radius, ObjectColor.ToFColor(true), static_cast<uint16>(Thickness), bWireframe, TEXT("%s"), *Text);
#endif
}

FRigVMFunction_VisualLogSegment_Execute()
{
#if ENABLE_VISUAL_LOG
	FVisualLogger::SegmentLogf(ExecuteContext.GetOwningObject(), Category, UE::RigVM::Private::DefaultVerbosity, SegmentStart, SegmentEnd, ObjectColor.ToFColor(true), static_cast<uint16>(Thickness), TEXT("%s"), *Text);
#endif
}

