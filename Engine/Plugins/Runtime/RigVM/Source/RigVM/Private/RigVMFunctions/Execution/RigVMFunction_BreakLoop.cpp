// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_BreakLoop.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_BreakLoop)

/**
 * Sets BlockToRun to ControlFlowCompletedName, signaling the enclosing loop
 * to exit after the current iteration finishes. Because BlockToRun is aliased
 * (via Singleton) to the loop's own BlockToRun register, the loop's next
 * JumpToBranch will route to the Completed branch instead of re-entering the body.
 */
FRigVMFunction_BreakFromLoop_Execute()
{
	// Route the shared BlockToRun register to the Completed branch,
	// causing the enclosing loop to terminate on its next iteration check.
	BlockToRun = ControlFlowCompletedName;
}
