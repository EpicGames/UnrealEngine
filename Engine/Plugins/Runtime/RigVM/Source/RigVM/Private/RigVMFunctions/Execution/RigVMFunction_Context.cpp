// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_Context.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Context)

FRigVMFunction_IsHostBeingDebugged_Execute()
{
#if WITH_EDITOR
	Result = ExecuteContext.IsHostBeingDebugged();
#else
	Result = false;
#endif
}


