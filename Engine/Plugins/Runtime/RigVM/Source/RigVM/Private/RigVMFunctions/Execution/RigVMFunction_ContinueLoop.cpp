// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_ContinueLoop.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_ContinueLoop)

/**
 * Intentionally empty. ContinueLoop's skip behaviour is a compiler artifact:
 * a JumpForward instruction emitted after the CallExtern bypasses the remaining
 * loop body nodes. No runtime state modification is needed.
 */
FRigVMFunction_ContinueLoop_Execute()
{
}
