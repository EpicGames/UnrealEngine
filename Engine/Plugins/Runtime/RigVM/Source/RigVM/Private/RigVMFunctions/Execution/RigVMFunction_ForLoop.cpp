// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_ForLoop.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_ForLoop)

FRigVMFunction_ForLoopCount_Execute()
{
	if(BlockToRun.IsNone())
	{
		Index = 0;
		BlockToRun = ExecutePinName;
	}
	else if(BlockToRun == ExecutePinName)
	{
		Index++;
	}

	if(Index == Count)
	{
		BlockToRun = ControlFlowCompletedName;
	}

	Ratio = GetRatioFromIndex(Index, Count);
}

