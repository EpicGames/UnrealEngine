// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_Sequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Sequence)

FRigVMFunction_Sequence_Execute()
{
	// nothing to do here. the execute context is actually
	// the same shared memory for all pins
}


