// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_UserDefinedEvent)

FString FRigVMFunction_UserDefinedEvent::GetUnitLabel() const
{
	if(EventName.IsNone())
	{
		static const FString InvalidEvent = TEXT("Invalid Event");
		return InvalidEvent;
	}
	return *EventName.ToString();
}

FRigVMFunction_UserDefinedEvent_Execute()
{
	if(EventName.IsNone())
	{
		ExecuteContext.SetEventName(EventName);
	}
	if(EventName.IsNone())
	{
		UE_RIGVMSTRUCT_REPORT_ERROR(TEXT("Unable to run event - EventName is None."));
	}
}

