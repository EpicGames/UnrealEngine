// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathBool.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathBool)

FRigVMFunction_MathBoolMake_Execute()
{
}

FRigVMStructUpgradeInfo FRigVMFunction_MathBoolMake::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_Constant::StaticStruct());
}

FRigVMFunction_MathBoolConstTrue_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathBoolConstFalse_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathBoolNot_Execute()
{
	Result = !Value;
}

FRigVMFunction_MathBoolAnd_Execute()
{
	Result = A && B;
}

FRigVMFunction_MathBoolNand_Execute()
{
	Result = (!A) && (!B);
}

FRigVMFunction_MathBoolNand2_Execute()
{
	Result = !(A && B);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathBoolNand::GetUpgradeInfo() const
{
	FRigVMFunction_MathBoolNand2 NewNode;
	NewNode.A = A;
	NewNode.B = B;

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

FRigVMFunction_MathBoolOr_Execute()
{
	Result = A || B;
}

FRigVMFunction_MathBoolEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathBoolEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathBoolNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathBoolNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathBoolToggled_Execute()
{
	if(!Initialized)
	{
		Initialized = true;
		Toggled = false;
		LastValue = Value;
	}
	else
	{
		Toggled = LastValue != Value;
	}

	LastValue = Value;
}

FRigVMFunction_MathBoolFlipFlop_Execute()
{
	TimeLeft = TimeLeft - ExecuteContext.GetDeltaTime<float>();
	if (TimeLeft <= 0.f)
	{
		LastValue = !LastValue;
		TimeLeft = Duration;
	}

	Result = LastValue;
}

FRigVMFunction_MathBoolOnce_Execute()
{
	if(!Result && LastValue)
	{
		TimeLeft = Duration;
	}

	Result = LastValue;

	TimeLeft -= ExecuteContext.GetDeltaTime<float>();
	if(TimeLeft <= 0.f)
	{
		LastValue = false;
	}
}

FRigVMFunction_MathBoolToFloat_Execute()
{
	Result = Value ? 1.f : 0.f;
}

FRigVMFunction_MathBoolToInteger_Execute()
{
	Result = Value ? 1 : 0;
}

FRigVMFunction_MathBoolAny_Execute()
{
	Result = false;
	for (const bool& Item : Array)
	{
		if (Item)
		{
			Result = true;
			break;
		}
	}
	IsEmpty = Array.IsEmpty();
}

FRigVMFunction_MathBoolAll_Execute()
{
	Result = true;
	for (const bool& Item : Array)
	{
		Result &= Item;
	}
	IsEmpty = Array.IsEmpty();
}