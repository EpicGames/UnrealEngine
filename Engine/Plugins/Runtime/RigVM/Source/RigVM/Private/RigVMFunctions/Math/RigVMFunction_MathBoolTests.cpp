// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "RigVMFunctions/Math/RigVMFunction_MathBool.h"
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolNot)
{
	Unit.Value = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.Value = false;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolAnd)
{
	Unit.A = false;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = false;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolNand)
{
	Unit.A = false;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = false;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolOr)
{
	Unit.A = false;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = false;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;

}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolEquals)
{
	Unit.A = false;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = false;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolNotEquals)
{
	Unit.A = false;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.A = false;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = false;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.A = true;
	Unit.B = true;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolAny)
{
	Unit.Array = { true, false, false };
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == false, TEXT("unexpected result"));
	Unit.Array = { false, false, false };
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == false, TEXT("unexpected result"));
	Unit.Array = {};
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathBoolAll)
{
	Unit.Array = { true, false, true };
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == false, TEXT("unexpected result"));
	Unit.Array = { true, true, true };
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == false, TEXT("unexpected result"));
	Unit.Array = {};
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	AddErrorIfFalse(Unit.IsEmpty == true, TEXT("unexpected result"));
	return true;
}

#endif
