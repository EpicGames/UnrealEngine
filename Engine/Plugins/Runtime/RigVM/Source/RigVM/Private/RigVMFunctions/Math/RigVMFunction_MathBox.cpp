// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathBox.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathBox)

FRigVMFunction_MathBoxFromArray_Execute()
{
	Box = FBox(EForceInit::ForceInit);
	for(const FVector& Position : Array)
	{
		Box += Position;
	}
	if(Array.IsEmpty())
	{
		Box += FVector::ZeroVector;
	}
	Minimum = Box.Min;
	Maximum = Box.Max;
	Center = Box.GetCenter();
	Size = Box.GetSize();
}

FRigVMFunction_MathBoxIsValid_Execute()
{
	Valid = Box.IsValid > 0; 
}

FRigVMFunction_MathBoxGetCenter_Execute()
{
	if(Box.IsValid)
	{
		Center = Box.GetCenter();
	}
	else
	{
		Center = FVector::ZeroVector;
	}
}

FRigVMFunction_MathBoxGetSize_Execute()
{
	if(Box.IsValid)
	{
		Size = Box.GetSize();
		Extent = Box.GetExtent();
	}
	else
	{
		Size = Extent = FVector::ZeroVector;
	}
}

FRigVMFunction_MathBoxShift_Execute()
{
	if(!Box.IsValid)
	{
		Result = FBox(EForceInit::ForceInit);
		return;
	}
	Result = Box.ShiftBy(Amount);
}

FRigVMFunction_MathBoxMoveTo_Execute()
{
	if(!Box.IsValid)
	{
		Result = FBox(EForceInit::ForceInit);
		return;
	}
	Result = Box.MoveTo(Center);
}

FRigVMFunction_MathBoxExpand_Execute()
{
	if(!Box.IsValid)
	{
		Result = FBox(EForceInit::ForceInit);
		return;
	}
	Result = Box.ExpandBy(Amount);
}

FRigVMFunction_MathBoxTransform_Execute()
{
	if(!Box.IsValid)
	{
		Result = FBox(EForceInit::ForceInit);
		return;
	}
	Result = Box.TransformBy(Transform);
}

FRigVMFunction_MathBoxGetDistance_Execute()
{
	if(!Box.IsValid)
	{
		Valid = false;
		Distance = 0.f;
		return;
	}

	Distance = static_cast<float>(Box.ComputeSquaredDistanceToPoint(Position));
	if(!Square && Distance >= 0)
	{
		Distance = FMath::Sqrt(Distance);
	}
	Valid = true;
}

FRigVMFunction_MathBoxIsInside_Execute()
{
	if(!Box.IsValid)
	{
		Result = false;
		return;
	}
	Result = Box.IsInsideOrOn(Position);
}

FRigVMFunction_MathBoxGetVolume_Execute()
{
	if(!Box.IsValid)
	{
		Volume = 0.f;
		return;
	}
	Volume = static_cast<float>(Box.GetVolume());
}
