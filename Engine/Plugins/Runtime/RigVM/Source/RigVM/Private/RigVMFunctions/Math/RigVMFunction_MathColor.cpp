// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathColor.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathColor)

FRigVMFunction_MathColorMake_Execute()
{
	Result = FLinearColor(R, G, B, A);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathColorMake::GetUpgradeInfo() const
{
	FRigVMStructUpgradeInfo Info = FRigVMStructUpgradeInfo::MakeFromStructToFactory(
		StaticStruct(),
		FRigVMDispatch_Constant::StaticStruct(),
		{{TEXT("Result"), TEXT("Value")}}
	);
	Info.AddRemappedPin(TEXT("R"), TEXT("Value.R"));
	Info.AddRemappedPin(TEXT("G"), TEXT("Value.G"));
	Info.AddRemappedPin(TEXT("B"), TEXT("Value.B"));
	Info.AddRemappedPin(TEXT("A"), TEXT("Value.A"));
	return Info;
}

FRigVMFunction_MathColorFromFloat_Execute()
{
	Result = FLinearColor(Value, Value, Value);
}

FRigVMFunction_MathColorFromDouble_Execute()
{
	Result = FLinearColor((float)Value, (float)Value, (float)Value);
}

FRigVMFunction_MathColorAdd_Execute()
{
	Result = A + B;
}

FRigVMFunction_MathColorSub_Execute()
{
	Result = A - B;
}

FRigVMFunction_MathColorMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathColorLerp_Execute()
{
	Result = FMath::Lerp<FLinearColor>(A, B, T);
}

