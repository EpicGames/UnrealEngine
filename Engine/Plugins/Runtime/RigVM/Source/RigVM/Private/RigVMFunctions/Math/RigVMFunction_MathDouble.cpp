// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathDouble.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathDouble)

FRigVMFunction_MathDoubleMake_Execute()
{
}

FRigVMStructUpgradeInfo FRigVMFunction_MathDoubleMake::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_Constant::StaticStruct());
}

FRigVMFunction_MathDoubleConstPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathDoubleConstHalfPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathDoubleConstTwoPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathDoubleConstE_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathDoubleAdd_Execute()
{
	Result = A + B;
}

FRigVMFunction_MathDoubleSub_Execute()
{
	Result = A - B;
}

FRigVMFunction_MathDoubleMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathDoubleDiv_Execute()
{
	if(FMath::IsNearlyZero(B))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is nearly 0.0"));
		Result = 0.0;
		return;
	}
	Result = A / B;
}

FRigVMFunction_MathDoubleMod_Execute()
{
	if(FMath::IsNearlyZero(B) || B < 0.0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B needs to be greater than 0"));
		Result = 0.0;
		return;
	}
	Result = FMath::Fmod(A, B);
}

FRigVMFunction_MathDoubleMin_Execute()
{
	Result = FMath::Min<double>(A, B);
}

FRigVMFunction_MathDoubleMax_Execute()
{
	Result = FMath::Max<double>(A, B);
}

FRigVMFunction_MathDoublePow_Execute()
{
	Result = FMath::Pow(A, B);
}

FRigVMFunction_MathDoubleSqrt_Execute()
{
	if(Value < 0.0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is below zero"));
		Result = 0.0;
		return;
	}
	Result = FMath::Sqrt(Value);
}

FRigVMFunction_MathDoubleNegate_Execute()
{
	Result = -Value;
}

FRigVMFunction_MathDoubleAbs_Execute()
{
	Result = FMath::Abs(Value);
}

FRigVMFunction_MathDoubleOneMinus_Execute()
{
	Result = 1.0 - Value;
}

FRigVMFunction_MathDoubleFloor_Execute()
{
	Result = FMath::FloorToDouble(Value);
	Int = static_cast<int32>(Result);
}

FRigVMFunction_MathDoubleCeil_Execute()
{
	Result = FMath::CeilToDouble(Value);
	Int = static_cast<int32>(Result);
}

FRigVMFunction_MathDoubleRound_Execute()
{
	Result = FMath::RoundToDouble(Value);
	Int = static_cast<int32>(Result);
}

FRigVMFunction_MathDoubleToInt_Execute()
{
	Result = static_cast<int32>(FMath::FloorToInt(Value));
}

FRigVMFunction_MathDoubleToFloat_Execute()
{
	Result = static_cast<float>(Value);
}

FRigVMFunction_MathDoubleSign_Execute()
{
	Result = Value < 0.0 ? -1.0 : 1.0;
}

FRigVMFunction_MathDoubleClamp_Execute()
{
	Result = FMath::Clamp<double>(Value, Minimum, Maximum);
}

FRigVMFunction_MathDoubleWrap_Execute()
{
	Result = FMath::Wrap<double>(Value, Minimum, Maximum);
}

FRigVMFunction_MathDoubleLerp_Execute()
{
	Result = FMath::Lerp<double>(A, B, T);
}

FRigVMFunction_MathDoubleRemap_Execute()
{
	double Ratio = 0.0;
	if (FMath::IsNearlyEqual(SourceMinimum, SourceMaximum))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum are the same."));
	}
	else
	{
		Ratio = (Value - SourceMinimum) / (SourceMaximum - SourceMinimum);
	}
	if (bClamp)
	{
		Ratio = FMath::Clamp<double>(Ratio, 0.0, 1.0);
	}
	Result = FMath::Lerp<double>(TargetMinimum, TargetMaximum, Ratio);
}

FRigVMFunction_MathDoubleEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathDoubleEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathDoubleNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathDoubleNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathDoubleGreater_Execute()
{
	Result = A > B;
}

FRigVMFunction_MathDoubleLess_Execute()
{
	Result = A < B;
}

FRigVMFunction_MathDoubleGreaterEqual_Execute()
{
	Result = A >= B;
}

FRigVMFunction_MathDoubleLessEqual_Execute()
{
	Result = A <= B;
}

FRigVMFunction_MathDoubleIsNearlyZero_Execute()
{
	if(Tolerance < 0.0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = FMath::IsNearlyZero(Value, FMath::Max<double>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathDoubleIsNearlyEqual_Execute()
{
	if(Tolerance < 0.0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = FMath::IsNearlyEqual(A, B, FMath::Max<double>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathDoubleDeg_Execute()
{
	Result = FMath::RadiansToDegrees(Value);
}

FRigVMFunction_MathDoubleRad_Execute()
{
	Result = FMath::DegreesToRadians(Value);
}

FRigVMFunction_MathDoubleSin_Execute()
{
	Result = FMath::Sin(Value);
}

FRigVMFunction_MathDoubleCos_Execute()
{
	Result = FMath::Cos(Value);
}

FRigVMFunction_MathDoubleTan_Execute()
{
	Result = FMath::Tan(Value);
}

FRigVMFunction_MathDoubleAsin_Execute()
{
	if (!FMath::IsWithinInclusive<double>(Value, -1.0, 1.0))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is outside of valid range (-1 to 1)"));
		Result = 0.0;
		return;
	}
	Result = FMath::Asin(Value);
}

FRigVMFunction_MathDoubleAcos_Execute()
{
	if (!FMath::IsWithinInclusive<double>(Value, -1.0, 1.0))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is outside of valid range (-1 to 1)"));
		Result = 0.0;
		return;
	}
	Result = FMath::Acos(Value);
}

FRigVMFunction_MathDoubleAtan_Execute()
{
	Result = FMath::Atan(Value);
}

FRigVMFunction_MathDoubleAtan2_Execute()
{
	Result = FMath::Atan2(A, B);
}

FRigVMFunction_MathDoubleLawOfCosine_Execute()
{
	if ((A <= 0.0) || (B <= 0.0) || (C <= 0.0) || (A + B < C) || (A + C < B) || (B + C < A))
	{
		AlphaAngle = BetaAngle = GammaAngle = 0.0;
		bValid = false;
		return;
	}

	GammaAngle = FMath::Acos((A * A + B * B - C * C) / (2.0 * A * B));
	BetaAngle = FMath::Acos((A * A + C * C - B * B) / (2.0 * A * C));
	AlphaAngle = PI - GammaAngle - BetaAngle;
	bValid = true;
}

FRigVMFunction_MathDoubleExponential_Execute()
{
	Result = FMath::Exp(Value);
}

FRigVMFunction_MathDoubleArraySum_Execute()
{
	Sum = 0.0;
	for (double Value : Array)
	{
		Sum += Value;
	}
}

FRigVMFunction_MathDoubleArrayAverage_Execute()
{
	Average = 0.0;
	if (!Array.IsEmpty())
	{
		for (double Value : Array)
		{
			Average += Value;
		}
		Average = Average / Array.Num();
	}
	else
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Array is empty"));
	}
}





