// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathFloat.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathFloat)

FRigVMFunction_MathFloatMake_Execute()
{
}

FRigVMStructUpgradeInfo FRigVMFunction_MathFloatMake::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_Constant::StaticStruct());
}

FRigVMFunction_MathFloatConstPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathFloatConstHalfPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathFloatConstTwoPi_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathFloatConstE_Execute()
{
	// nothing to do here - value is assigned in the constructor
}

FRigVMFunction_MathFloatAdd_Execute()
{
	Result = A + B;
}

FRigVMFunction_MathFloatSub_Execute()
{
	Result = A - B;
}

FRigVMFunction_MathFloatMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathFloatDiv_Execute()
{
	if(FMath::IsNearlyZero(B))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is nearly 0.f"));
		Result = 0.f;
		return;
	}
	Result = A / B;
}

FRigVMFunction_MathFloatMod_Execute()
{
	if(FMath::IsNearlyZero(B) || B < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B needs to be greater than 0"));
		Result = 0.f;
		return;
	}
	Result = FMath::Fmod(A, B);
}

FRigVMFunction_MathFloatMin_Execute()
{
	Result = FMath::Min<float>(A, B);
}

FRigVMFunction_MathFloatMax_Execute()
{
	Result = FMath::Max<float>(A, B);
}

FRigVMFunction_MathFloatPow_Execute()
{
	Result = FMath::Pow(A, B);
}

FRigVMFunction_MathFloatSqrt_Execute()
{
	if(Value < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is below zero"));
		Result = 0.f;
		return;
	}
	Result = FMath::Sqrt(Value);
}

FRigVMFunction_MathFloatNegate_Execute()
{
	Result = -Value;
}

FRigVMFunction_MathFloatAbs_Execute()
{
	Result = FMath::Abs(Value);
}

FRigVMFunction_MathFloatOneMinus_Execute()
{
	Result = 1.0f - Value;
}

FRigVMFunction_MathFloatFloor_Execute()
{
	Result = FMath::FloorToFloat(Value);
	Int = (int32)Result;
}

FRigVMFunction_MathFloatCeil_Execute()
{
	Result = FMath::CeilToFloat(Value);
	Int = (int32)Result;
}

FRigVMFunction_MathFloatRound_Execute()
{
	Result = FMath::RoundToFloat(Value);
	Int = (int32)Result;
}

FRigVMFunction_MathFloatToInt_Execute()
{
	Result = FMath::FloorToInt(Value);
}

FRigVMFunction_MathFloatToDouble_Execute()
{
	Result = Value;
}

FRigVMFunction_MathFloatSign_Execute()
{
	Result = Value < 0.f ? -1.f : 1.f;
}

FRigVMFunction_MathFloatClamp_Execute()
{
	Result = FMath::Clamp<float>(Value, Minimum, Maximum);
}

FRigVMFunction_MathFloatWrap_Execute()
{
	Result = FMath::Wrap<float>(Value, Minimum, Maximum);
}

FRigVMFunction_MathFloatLerp_Execute()
{
	Result = FMath::Lerp<float>(A, B, T);
}

FRigVMFunction_MathFloatRemap_Execute()
{
	float Ratio = 0.f;
	if (FMath::IsNearlyEqual(SourceMinimum, SourceMaximum))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum are the same."));
	}
	else
	{
		Ratio = (Value - SourceMinimum) / (SourceMaximum - SourceMinimum);
	}
	if (bClamp)
	{
		Ratio = FMath::Clamp<float>(Ratio, 0.f, 1.f);
	}
	Result = FMath::Lerp<float>(TargetMinimum, TargetMaximum, Ratio);
}

FRigVMFunction_MathFloatEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathFloatEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathFloatNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathFloatNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathFloatGreater_Execute()
{
	Result = A > B;
}

FRigVMFunction_MathFloatLess_Execute()
{
	Result = A < B;
}

FRigVMFunction_MathFloatGreaterEqual_Execute()
{
	Result = A >= B;
}

FRigVMFunction_MathFloatLessEqual_Execute()
{
	Result = A <= B;
}

FRigVMFunction_MathFloatIsNearlyZero_Execute()
{
	if(Tolerance < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = FMath::IsNearlyZero(Value, FMath::Max<float>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathFloatIsNearlyEqual_Execute()
{
	if(Tolerance < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = FMath::IsNearlyEqual(A, B, FMath::Max<float>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathFloatSelectBool_Execute()
{
	Result = Condition ? IfTrue : IfFalse;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathFloatSelectBool::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigVMFunction_MathFloatDeg_Execute()
{
	Result = FMath::RadiansToDegrees(Value);
}

FRigVMFunction_MathFloatRad_Execute()
{
	Result = FMath::DegreesToRadians(Value);
}

FRigVMFunction_MathFloatSin_Execute()
{
	Result = FMath::Sin(Value);
}

FRigVMFunction_MathFloatCos_Execute()
{
	Result = FMath::Cos(Value);
}

FRigVMFunction_MathFloatTan_Execute()
{
	Result = FMath::Tan(Value);
}

FRigVMFunction_MathFloatAsin_Execute()
{
	if (!FMath::IsWithinInclusive<float>(Value, -1.f, 1.f))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is outside of valid range (-1 to 1)"));
		Result = 0.f;
		return;
	}
	Result = FMath::Asin(Value);
}

FRigVMFunction_MathFloatAcos_Execute()
{
	if (!FMath::IsWithinInclusive<float>(Value, -1.f, 1.f))
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is outside of valid range (-1 to 1)"));
		Result = 0.f;
		return;
	}
	Result = FMath::Acos(Value);
}

FRigVMFunction_MathFloatAtan_Execute()
{
	Result = FMath::Atan(Value);
}

FRigVMFunction_MathFloatAtan2_Execute()
{
	Result = FMath::Atan2(A, B);
}

FRigVMFunction_MathFloatLawOfCosine_Execute()
{
	if ((A <= 0.f) || (B <= 0.f) || (C <= 0.f) || (A + B < C) || (A + C < B) || (B + C < A))
	{
		AlphaAngle = BetaAngle = GammaAngle = 0.f;
		bValid = false;
		return;
	}

	GammaAngle = FMath::Acos((A * A + B * B - C * C) / (2.f * A * B));
	BetaAngle = FMath::Acos((A * A + C * C - B * B) / (2.f * A * C));
	AlphaAngle = PI - GammaAngle - BetaAngle;
	bValid = true;
}

FRigVMFunction_MathFloatExponential_Execute()
{
	Result = FMath::Exp(Value);
}

FRigVMFunction_MathFloatArraySum_Execute()
{
	Sum = 0.f;
	for (float Value : Array)
	{
		Sum += Value;
	}
}

FRigVMFunction_MathFloatArrayAverage_Execute()
{
	Average = 0.f;
	if (!Array.IsEmpty())
	{
		for (float Value : Array)
		{
			Average += Value;
		}
		Average = Average / static_cast<float>(Array.Num());
	}
	else
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Array is empty"));
	}
}







