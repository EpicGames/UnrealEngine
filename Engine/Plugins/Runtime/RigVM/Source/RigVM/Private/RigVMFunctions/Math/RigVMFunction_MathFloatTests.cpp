// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "RigVMFunctions/Math/RigVMFunction_MathFloat.h"
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatAdd)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 5.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatSub)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, -1.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatMul)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 6.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatDiv)
{
	Unit.A = 8.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatMod)
{
	Unit.A = 7.f;
	Unit.B = 5.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatMin)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatMax)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatPow)
{
	Unit.A = 2.f;
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 8.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatSqrt)
{
	Unit.Value = 9.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatNegate)
{
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, -2.f), TEXT("unexpected result"));
	Unit.Value = -4.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatAbs)
{
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	Unit.Value = -3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatFloor)
{
	Unit.Value = 2.4f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	Unit.Value = 2.7f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatCeil)
{
	Unit.Value = 2.4f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	Unit.Value = 2.7f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatRound)
{
	Unit.Value = 2.4f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	Unit.Value = 2.7f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatSign)
{
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected result"));
	Unit.Value = -3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, -1.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatClamp)
{
	Unit.Minimum = 1.f;
	Unit.Maximum = 3.f;
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	Unit.Value = -3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected result"));
	Unit.Value = 12.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatWrap)
{
	Unit.Minimum = 1.f;
	Unit.Maximum = 3.f;
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));

	Unit.Value = -0.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.5f), TEXT("unexpected result"));

	Unit.Value = 3.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.5f), TEXT("unexpected result"));

	Unit.Value = -1.0f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.0f), TEXT("unexpected result"));

	Unit.Value = 5.0f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.0f), TEXT("unexpected result"));

	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatLerp)
{
	Unit.A = 2.f;
	Unit.B = 8.f;
	Unit.T = 0.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected result"));
	Unit.T = 1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 8.f), TEXT("unexpected result"));
	Unit.T = 0.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 5.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatRemap)
{
	Unit.Value = 5.f;
	Unit.SourceMinimum = 2.f;
	Unit.SourceMaximum = 8.f;
	Unit.TargetMinimum = 12.f;
	Unit.TargetMaximum = 18.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 15.f), TEXT("unexpected result"));
	Unit.Value = 11.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 21.f), TEXT("unexpected result"));
	Unit.bClamp = true;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 18.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatEquals)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatNotEquals)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatGreater)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatLess)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatGreaterEqual)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatLessEqual)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 1.f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatIsNearlyZero)
{
	Unit.Value = 0.f;
	Unit.Tolerance = 0.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.Value = 0.001f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.Tolerance = 0.01f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatIsNearlyEqual)
{
	Unit.A = 2.f;
	Unit.B = 2.f;
	Unit.Tolerance = 0.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = 2.001f;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.Tolerance = 0.01f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatSelectBool)
{
	Unit.Condition = false;
	Unit.IfTrue = 2.f;
	Unit.IfFalse = 3.f;
	Execute();
	AddErrorIfFalse(Unit.Result == 3.f, TEXT("unexpected result"));
	Unit.Condition = true;
	Execute();
	AddErrorIfFalse(Unit.Result == 2.f, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatDeg)
{
	Unit.Value = PI;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 180.f, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatRad)
{
	Unit.Value = 180.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, PI, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatSin)
{
	Unit.Value = HALF_PI;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatCos)
{
	Unit.Value = PI;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, -1.f, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatTan)
{
	Unit.Value = 1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.5574f, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatAsin)
{
	Unit.Value = 1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, HALF_PI, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatAcos)
{
	Unit.Value = -1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, PI, 0.001f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatAtan)
{
	Unit.Value = 1.5574f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f, 0.01f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatLawOfCosine)
{
	Unit.A = 3.f;
	Unit.B = 4.f;
	Unit.C = 5.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.GammaAngle, PI * 0.5f, 0.01f), TEXT("unexpected gamma angle"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.BetaAngle, 0.9272f, 0.01f), TEXT("unexpected beta angle"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.AlphaAngle, 0.6435f, 0.01f), TEXT("unexpected alpha angle"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatArraySum)
{
	Unit.Array = {1.1f, 2.2f, 3.3f};
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Sum, 6.6f, 0.01f), TEXT("unexpected sum"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathFloatArrayAverage)
{
	Unit.Array = { 1.1f, 2.2f, 3.3f };
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Average, 2.2f, 0.01f), TEXT("unexpected average"));
	return true;
}

#endif
