// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathInt.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"
#include "RigVMCore/RigVMStructTest.h"

#include "limits"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathInt)

FRigVMFunction_MathIntMake_Execute()
{
}

FRigVMStructUpgradeInfo FRigVMFunction_MathIntMake::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_Constant::StaticStruct());
}

FRigVMFunction_MathIntAdd_Execute()
{
	Result = A + B;
}

FRigVMFunction_MathIntSub_Execute()
{
	Result = A - B;
}

FRigVMFunction_MathIntMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathIntDiv_Execute()
{
	if(B == 0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is 0"));
		Result = 0;
		return;
	}
	Result = A / B;
}

FRigVMFunction_MathIntMod_Execute()
{
	if(B <= 0)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B needs to be greater than 0"));
		Result = 0;
		return;
	}
	Result = A % B;
}

FRigVMFunction_MathIntMin_Execute()
{
	Result = FMath::Min<int32>(A, B);
}

FRigVMFunction_MathIntMax_Execute()
{
	Result = FMath::Max<int32>(A, B);
}

FRigVMFunction_MathIntPow_Execute()
{
	if (B >= 0)
	{
		// A quick way to compute exponentiate an integer that doesn't require powf()
		Result = 1;
		int32 P = A;
		uint32 N = static_cast<uint32>(B);
		for (;;)
		{
			if (N & 1)
			{
				// If the lowest bit of n is set, multiply the current exponentiated value into the result.
				Result = P * Result;
			}
			N >>= 1;
			if (N == 0)
			{
				break;
			}
			P = P * P;
		}
	}
	else if (FMath::Abs(A) > 1)
	{
		// Negative integer powers of values greater than 1 in magnitude will always be a fraction, round to zero.
		Result = 0;
	}
	else
	{
		// Negative powers of 1 are always 1. Negative powers of 0 are inf.
		Result = A == 0 ? std::numeric_limits<int32>::max() : 1;
	}
}	


FRigVMFunction_MathIntNegate_Execute()
{
	Result = -Value;
}

FRigVMFunction_MathIntAbs_Execute()
{
	Result = FMath::Abs(Value);
}

FRigVMFunction_MathIntToFloat_Execute()
{
	Result = static_cast<float>(Value);
}

FRigVMFunction_MathIntToDouble_Execute()
{
	Result = static_cast<double>(Value);
}

FRigVMFunction_MathIntSign_Execute()
{
	Result = Value < 0 ? -1 : 1;
}

FRigVMFunction_MathIntClamp_Execute()
{
	Result = FMath::Clamp<int32>(Value, Minimum, Maximum);
}

FRigVMFunction_MathIntEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathIntEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathIntNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathIntNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathIntGreater_Execute()
{
	Result = A > B;
}

FRigVMFunction_MathIntLess_Execute()
{
	Result = A < B;
}

FRigVMFunction_MathIntGreaterEqual_Execute()
{
	Result = A >= B;
}

FRigVMFunction_MathIntLessEqual_Execute()
{
	Result = A <= B;
}

FRigVMFunction_MathIntArraySum_Execute()
{
	Sum = 0;
	for (int32 Value : Array)
	{
		Sum += Value;
	}
}

FRigVMFunction_MathIntArrayAverage_Execute()
{
	Average = 0;
	if (!Array.IsEmpty())
	{
		for (int32 Value : Array)
		{
			Average += Value;
		}
		Average = Average / Array.Num();
	}
	else
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Array is empty"));
	}
}

FRigVMFunction_MathIntToString_Execute()
{
	Result = FString::FromInt(Number);
	if(Number >= 0)
	{
		while(Result.Len() < FMath::Min<int32>(8, PaddedSize))
		{
			static const FString Zero = FString::FromInt(0);
			Result = Zero + Result;
		}
	}
}

FRigVMFunction_MathIntToName_Execute()
{
	FString String;
	FRigVMFunction_MathIntToString::StaticExecute(ExecuteContext, Number, PaddedSize, String);
	Result = *String;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathIntToName)
{
	Unit.Number = 13;
	Unit.PaddedSize = 5;
	Execute();
	AddErrorIfFalse(Unit.Result == TEXT("00013"), TEXT("unexpected result"));
	return true;
}