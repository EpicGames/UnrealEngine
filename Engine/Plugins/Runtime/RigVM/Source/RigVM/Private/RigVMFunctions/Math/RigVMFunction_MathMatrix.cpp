// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathMatrix.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathMatrix)

FRigVMFunction_MathMatrixFromTransform_Execute()
{
	Result = Transform.ToMatrixWithScale();
}

FRigVMStructUpgradeInfo FRigVMFunction_MathMatrixFromTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathMatrixFromTransformV2 NewNode;
	NewNode.Value = Transform;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Transform"), TEXT("Value"));
	return Info;
}

FRigVMFunction_MathMatrixFromTransformV2_Execute()
{
	Result = Value.ToMatrixWithScale();
}

FRigVMFunction_MathMatrixToTransform_Execute()
{
	Result.SetFromMatrix(Value);
}

FRigVMFunction_MathMatrixFromVectors_Execute()
{
	Result = FMatrix(X, Y, Z, FVector::ZeroVector);
	Result.SetOrigin(Origin);
}

FRigVMFunction_MathMatrixToVectors_Execute()
{
	X = Value.GetScaledAxis(EAxis::X);
	Y = Value.GetScaledAxis(EAxis::Y);
	Z = Value.GetScaledAxis(EAxis::Z);
	Origin = Value.GetOrigin();
}

FRigVMFunction_MathMatrixMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathMatrixInverse_Execute()
{
	Result = Value.Inverse();
}

