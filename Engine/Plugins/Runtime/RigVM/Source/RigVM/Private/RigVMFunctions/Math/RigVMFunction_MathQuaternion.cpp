// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathQuaternion.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "AnimationCoreLibrary.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathQuaternion)

FRigVMFunction_MathQuaternionMake_Execute()
{
	Result = FQuat(X, Y, Z, W);
}

FRigVMFunction_MathQuaternionFromAxisAndAngle_Execute()
{
	if (Axis.IsNearlyZero())
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Axis is nearly zero"));
		Result = FQuat::Identity;
		return;
	}
	Result = FQuat(Axis.GetUnsafeNormal(), Angle);
}

FRigVMFunction_MathQuaternionFromEuler_Execute()
{
	Result = AnimationCore::QuatFromEuler(Euler, RotationOrder);
}

FRigVMFunction_MathQuaternionFromRotator_Execute()
{
	Result = FQuat(Rotator);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathQuaternionFromRotator::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionFromRotatorV2 NewNode;
	NewNode.Value = Rotator;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Rotator"), TEXT("Value"));
	return Info;
}

FRigVMFunction_MathQuaternionFromRotatorV2_Execute()
{
	Result = FQuat(Value);
}

FRigVMFunction_MathQuaternionFromTwoVectors_Execute()
{
	if (A.IsNearlyZero() || B.IsNearlyZero())
	{
		Result = FQuat::Identity;
		return;
	}
	Result = FRigVMMathLibrary::FindQuatBetweenVectors(A, B);
}

FRigVMFunction_MathQuaternionToAxisAndAngle_Execute()
{
	Value.GetNormalized().ToAxisAndAngle(Axis, Angle);
	if (Axis.IsNearlyZero())
	{
		Axis = FVector(1.f, 0.f, 0.f);
		Angle = 0.f;
	}


	float AngleSign = Angle < 0.f ? -1.f : 1.f;
	Angle = FMath::Abs(Angle);

	if (Angle > TWO_PI)
	{
		Angle = FMath::Fmod(Angle, TWO_PI);
	}
	if (Angle > PI)
	{
		Angle = TWO_PI - Angle;
		AngleSign = -AngleSign;
	}

	Angle = Angle * AngleSign;
}

FRigVMFunction_MathQuaternionToVectors_Execute()
{
	const FQuat Rotation = Value.GetNormalized();
	Forward = Rotation.GetForwardVector();
	Right = Rotation.GetRightVector();
	Up = Rotation.GetUpVector();
}

FRigVMFunction_MathQuaternionScale_Execute()
{
	FVector Axis = FVector::ZeroVector;
	float Angle = 0.f;
	Value.ToAxisAndAngle(Axis, Angle);
	Value = FQuat(Axis, Angle * Scale);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathQuaternionScale::GetUpgradeInfo() const
{
	FRigVMFunction_MathQuaternionScaleV2 NewNode;
	NewNode.Value = Value;
	NewNode.Factor = Scale;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Scale"), TEXT("Factor"));
	Info.AddRemappedPin(TEXT("Value"), TEXT("Result"), false, true);
	return Info;
}

FRigVMFunction_MathQuaternionScaleV2_Execute()
{
	FVector Axis = FVector::ZeroVector;
	float Angle = 0.f;
	Value.ToAxisAndAngle(Axis, Angle);
	Result = FQuat(Axis, Angle * Factor);
}

FRigVMFunction_MathQuaternionToEuler_Execute()
{
	Result = AnimationCore::EulerFromQuat(Value, RotationOrder);
}

FRigVMFunction_MathQuaternionToRotator_Execute()
{
	Result = Value.Rotator();
}

FRigVMFunction_MathQuaternionMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathQuaternionInverse_Execute()
{
	Result = Value.Inverse();
}

FRigVMFunction_MathQuaternionSlerp_Execute()
{
	Result = FQuat::Slerp(A, B, T);
}

FRigVMFunction_MathQuaternionEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathQuaternionEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathQuaternionNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathQuaternionNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathQuaternionSelectBool_Execute()
{
	Result = Condition ? IfTrue : IfFalse;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathQuaternionSelectBool::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigVMFunction_MathQuaternionDot_Execute()
{
	Result = static_cast<float>(A | B);
}

FRigVMFunction_MathQuaternionUnit_Execute()
{
	Result = Value.GetNormalized();
}

FRigVMFunction_MathQuaternionRotateVector_Execute()
{
	Result = Transform.RotateVector(Vector);
}

FRigVMFunction_MathQuaternionGetAxis_Execute()
{
	switch (Axis)
	{
		default:
		case EAxis::X:
		{
			Result = Quaternion.GetAxisX();
			break;
		}
		case EAxis::Y:
		{
			Result = Quaternion.GetAxisY();
			break;
		}
		case EAxis::Z:
		{
			Result = Quaternion.GetAxisZ();
			break;
		}
	}
}


FRigVMFunction_MathQuaternionSwingTwist_Execute()
{
	if (TwistAxis.IsNearlyZero())
	{
		Swing = Twist = FQuat::Identity;
		return;
	}

	FVector NormalizedAxis = TwistAxis.GetSafeNormal();
	Input.ToSwingTwist(NormalizedAxis, Swing, Twist);
}

FRigVMFunction_MathQuaternionRotationOrder_Execute()
{
}

FRigVMFunction_MathQuaternionMakeRelative_Execute()
{
	Local = Parent.Inverse() * Global;
	Local.Normalize();
}

FRigVMFunction_MathQuaternionMakeAbsolute_Execute()
{
	Global = Parent * Local;
	Global.Normalize();
}

FRigVMFunction_MathQuaternionMirrorTransform_Execute()
{
	FTransform Transform = FTransform::Identity;
	Transform.SetRotation(Value);
	FRigVMFunction_MathTransformMirrorTransform::StaticExecute(ExecuteContext, Transform, MirrorAxis, AxisToFlip, CentralTransform, Transform);
	Result = Transform.GetRotation();
}

FRigVMFunction_MathQuaternionFromXVector_Execute()
{
	Result = Value.ToOrientationQuat();
}
