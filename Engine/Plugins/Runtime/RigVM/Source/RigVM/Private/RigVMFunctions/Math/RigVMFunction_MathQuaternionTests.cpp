// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "RigVMFunctions/Math/RigVMFunction_MathQuaternion.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMCore/RigVMStructTest.h"
#include "AnimationCoreLibrary.h"

namespace FRigVMFunction_MathQuatTest_Utils
{
	bool IsNearlyEqual(float A, float B, float Tolerance = 0.001f)
	{
		return FMath::IsNearlyEqual(A, B, Tolerance);
	}

	bool IsNearlyEqual(const FVector& A, const FVector& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.X, B.X, Tolerance) &&
			   FMath::IsNearlyEqual(A.Y, B.Y, Tolerance) &&
			   FMath::IsNearlyEqual(A.Z, B.Z, Tolerance);
	}

	bool IsNearlyEqual(const FQuat& A, const FQuat& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.X, B.X, Tolerance) &&
			   FMath::IsNearlyEqual(A.Y, B.Y, Tolerance) &&
			   FMath::IsNearlyEqual(A.Z, B.Z, Tolerance) &&
			   FMath::IsNearlyEqual(A.W, B.W, Tolerance);
	}

	bool IsNearlyEqual(const FRotator& A, const FRotator& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.Yaw, B.Yaw, Tolerance) &&
			   FMath::IsNearlyEqual(A.Roll, B.Roll, Tolerance) &&
			   FMath::IsNearlyEqual(A.Pitch, B.Pitch, Tolerance);
	}

	bool IsNearlyEqual(const FQuat& A, const FVector& B, double Tolerance = 0.001)
	{
		const FVector Euler = A.Euler();
		return IsNearlyEqual(Euler, B, Tolerance);
	}
};

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionFromAxisAndAngle)
{
	Unit.Axis = FVector(3.f, 0.f, 0.f);
	Unit.Angle = HALF_PI;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(-90.f, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionFromEuler)
{
	Unit.Euler = FVector(30.f, 20.f, 10.f);
	Execute();
	FVector Euler = AnimationCore::EulerFromQuat(Unit.Result, Unit.RotationOrder);
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Euler, FVector(30.f, 20.f, 10.f)), TEXT("unexpected result"));
	Unit.RotationOrder = EEulerRotationOrder::YZX;
	Execute();
	Euler = AnimationCore::EulerFromQuat(Unit.Result, Unit.RotationOrder);
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Euler, FVector(30.f, 20.f, 10.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionFromRotator)
{
	Unit.Rotator = FRotator(30.f, 20.f, 10.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(10.f, 30.f, 20.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionFromTwoVectors)
{
	Unit.A = FVector(0.f, 0.f, 2.f);
	Unit.B = FVector(0.f, 4.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(90, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionToAxisAndAngle)
{
	Unit.Value = FQuat::MakeFromEuler(FVector(-90.f, 0.f, 0.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Axis, FVector(1.f, 0.f, 0.f)), TEXT("unexpected result"));
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Angle, HALF_PI), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionToEuler)
{
	Unit.Value = AnimationCore::QuatFromEuler(FVector(10.f, 20.f, 30.f), Unit.RotationOrder);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(10.f, 20.f, 30.f)), TEXT("unexpected result"));
	Unit.RotationOrder = EEulerRotationOrder::YZX;
	Unit.Value = AnimationCore::QuatFromEuler(FVector(10.f, 20.f, 30.f), Unit.RotationOrder);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(10.f, 20.f, 30.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionToRotator)
{
	Unit.Value = FQuat::MakeFromEuler(FVector(10.f, 30.f, 20.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FRotator(30.f, 20.f, 10.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionMul)
{
	Unit.A = FQuat::MakeFromEuler(FVector(90.f, 0.f, 0.f));
	Unit.B = FQuat::MakeFromEuler(FVector(0.f, 90.f, 0.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(90.f, 0.f, 90.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionInverse)
{
	Unit.Value = FQuat::MakeFromEuler(FVector(90.f, 0.f, 0.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(-90.f, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionSlerp)
{
	Unit.A = FQuat::MakeFromEuler(FVector(60.f, 0.f, 0.f));
	Unit.B = FQuat::MakeFromEuler(FVector(20.f, 0.f, 0.f));
	Unit.T = 0.5f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(40.f, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionEquals)
{
	Unit.A = FQuat(1.f, 0.f, 0.f, 0.f);
	Unit.B = FQuat(1.f, 0.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = FQuat(1.f, 1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionNotEquals)
{
	Unit.A = FQuat(1.f, 0.f, 0.f, 0.f);
	Unit.B = FQuat(1.f, 0.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = FQuat(1.f, 1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionSelectBool)
{
	Unit.Condition = false;
	Unit.IfTrue = FQuat(1.f, 0.f, 0.f, 0.f);
	Unit.IfFalse = FQuat(1.f, 1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FQuat(1.f, 1.f, 0.f, 0.f)), TEXT("unexpected result"));
	Unit.Condition = true;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FQuat(1.f, 0.f, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionDot)
{
	Unit.A = FQuat(1.f, 2.f, 3.f, 4.f);
	Unit.B = FQuat(5.f, 6.f, 7.f, 8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, static_cast<float>(Unit.A | Unit.B)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionUnit)
{
	Unit.Value = FQuat(FVector(3.f, 0.f, 0.f), HALF_PI);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(-143.130096, 0.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionRotateVector)
{
	Unit.Transform = FQuat(FVector(1.f, 0.f, 0.f), -HALF_PI);
	Unit.Vector = FVector(0.f, 0.f, 1.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 1.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionGetAxis)
{
	Unit.Quaternion = FQuat::Identity;
	Unit.Axis = EAxis::X;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, 0.f, 0.f)), TEXT("unexpected result"));
	Unit.Axis = EAxis::Y;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 1.f, 0.f)), TEXT("unexpected result"));
	Unit.Axis = EAxis::Z;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 0.f, 1.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionMakeRelative)
{
	FRigVMFunction_MathTransformMakeRelative TransformUnit;
	Unit.Global = FQuat(FVector(1.f, 0.f, 0.f), -HALF_PI);
	Unit.Parent = FQuat(FVector(0.2f, 0.8f, 0.f), -HALF_PI).GetNormalized();
	
	TransformUnit.Global.SetRotation(Unit.Global);
	TransformUnit.Parent.SetRotation(Unit.Parent);

	Unit.Execute();
	TransformUnit.Execute();

	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Local, TransformUnit.Local.GetRotation()), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathQuaternionMakeAbsolute)
{
	FRigVMFunction_MathTransformMakeAbsolute TransformUnit;
	Unit.Local = FQuat(FVector(1.f, 0.f, 0.f), -HALF_PI);
	Unit.Parent = FQuat(FVector(0.2f, 0.8f, 0.f), -HALF_PI).GetNormalized();
	
	TransformUnit.Local.SetRotation(Unit.Local);
	TransformUnit.Parent.SetRotation(Unit.Parent);

	Unit.Execute();
	TransformUnit.Execute();

	AddErrorIfFalse(FRigVMFunction_MathQuatTest_Utils::IsNearlyEqual(Unit.Global, TransformUnit.Global.GetRotation()), TEXT("unexpected result"));
	return true;
}

#endif
