// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathRBFInterpolate.h"


#include "Containers/HashTable.h"
#include "Misc/MemStack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathRBFInterpolate)

// Helper function for additive blending of quaternions.
static FQuat AddQuatWithWeight(const FQuat& Q, const FQuat& V, float Weight)
{
	FQuat BlendQuat = V * Weight;

	if ((Q | BlendQuat) >= 0.0f)
		return Q + BlendQuat;
	else
		return Q - BlendQuat;
}


struct SmoothingKernelFunctor
{
	SmoothingKernelFunctor(
		ERBFKernelType InKernelType,
		float InSigma
	)
		: KernelType(InKernelType)
		, Sigma(InSigma)
	{ }

	float operator()(float Distance) const
	{
		float Weight = 0.0f;
		switch (KernelType)
		{
		case ERBFKernelType::Linear:
			Weight = RBFKernel::Linear(Distance, Sigma);
			break;
		case ERBFKernelType::Gaussian:
			Weight = RBFKernel::Gaussian(Distance, Sigma);
			break;
		case ERBFKernelType::Exponential:
			Weight = RBFKernel::Exponential(Distance, Sigma);
			break;
		case ERBFKernelType::Cubic:
			Weight = RBFKernel::Cubic(Distance, Sigma);
			break;
		case ERBFKernelType::Quintic:
			Weight = RBFKernel::Quintic(Distance, Sigma);
			break;
		}

		return Weight;
	}

	ERBFKernelType KernelType;
	float Sigma;
};

struct FRigVMFunction_MathRBFQuatWeightFunctor
{
	FRigVMFunction_MathRBFQuatWeightFunctor(
		ERBFQuatDistanceType InDistanceType,
		ERBFKernelType InKernelType,
		float InSigma,
		const FVector &InTwistAxis
	)
		: DistanceType(InDistanceType)
		, TwistAxis(InTwistAxis)
		, SmoothingKernel(InKernelType, InSigma)
	{ }

	float operator()(const FQuat &A, const FQuat &B) const
	{
		double Distance = 0.0;
		switch (DistanceType)
		{
		case ERBFQuatDistanceType::Euclidean:
			Distance = RBFDistanceMetric::Euclidean(A, B);
			break;
		case ERBFQuatDistanceType::ArcLength:
			Distance = RBFDistanceMetric::ArcLength(A, B);
			break;
		case ERBFQuatDistanceType::SwingAngle:
			Distance = RBFDistanceMetric::SwingAngle(A, B, TwistAxis);
			break;
		case ERBFQuatDistanceType::TwistAngle:
			Distance = RBFDistanceMetric::TwistAngle(A, B, TwistAxis);
			break;
		}

		return SmoothingKernel(static_cast<float>(Distance));
	}

	ERBFQuatDistanceType DistanceType;
	const FVector &TwistAxis;
	SmoothingKernelFunctor SmoothingKernel;
};

struct FRigVMFunction_MathRBFVectorWeightFunctor
{
	FRigVMFunction_MathRBFVectorWeightFunctor(
		ERBFVectorDistanceType InDistanceType,
		ERBFKernelType InKernelType,
		float InSigma
	)
		: DistanceType(InDistanceType)
		, SmoothingKernel(InKernelType, InSigma)
	{ }

	float operator()(const FVector &A, const FVector &B) const
	{
		double Distance = 0.0;
		switch (DistanceType)
		{
		case ERBFVectorDistanceType::Euclidean:
			Distance = RBFDistanceMetric::Euclidean(A, B);
			break;
		case ERBFVectorDistanceType::Manhattan:
			Distance = RBFDistanceMetric::Manhattan(A, B);
			break;
		case ERBFVectorDistanceType::ArcLength:
			Distance = RBFDistanceMetric::ArcLength(A, B);
			break;
		}

		return SmoothingKernel(static_cast<float>(Distance));
	}

	ERBFVectorDistanceType DistanceType;
	SmoothingKernelFunctor SmoothingKernel;
};

static uint64 HashValue(float v)
{
	// To ensure both -0 and +0 hash to the same value. Yes, that's a thing.
	if (v == 0.0f)
		return 0ULL;
	const uint64 x = *reinterpret_cast<uint32*>(&v);
	return (x << 32ull) | x;
}

static uint64 HashValue(double v)
{
	// To ensure both -0 and +0 hash to the same value. Yes, that's a thing.
	if (v == 0.0f)
		return 0ULL;
	return *reinterpret_cast<uint64*>(&v);
}

static uint64 HashValues(std::initializer_list<uint64> Values, uint64 Seed)
{
	// Murmur64.
	const uint64 Magic = 0xc6a4a7935bd1e995ULL;
	const int Shift = 47;

	uint64_t Hash = Seed ^ (Values.size() * Magic);

	for (auto V: Values)
	{
		V *= Magic;
		V ^= V >> Shift;
		V *= Magic;

		Hash ^= V;
		Hash *= Magic;
	}

	Hash ^= Hash >> Shift;
	Hash *= Magic;
	Hash ^= Hash >> Shift;

	return Hash;
}


template<typename T>
void FRigVMFunction_MathRBFInterpolateQuatBase::GetInterpolatedWeights(
	FRigVMFunction_MathRBFInterpolateQuatWorkData& WorkData,
	const TArrayView<const T>& Targets,
	const FQuat &Input,
	ERBFQuatDistanceType DistanceFunction,
	ERBFKernelType SmoothingFunction,
	float SmoothingAngle,
	bool bNormalizeOutput,
	FVector TwistAxis,
	TArray<float> &Weights
)
{
	TArray<FQuat > Quats;
	Quats.Reserve(Targets.Num());

	for (const auto& Target : Targets)
	{
		Quats.Add(Target.Target);
	}

	FRigVMFunction_MathRBFQuatWeightFunctor WeightFunc(
		DistanceFunction, SmoothingFunction, FMath::DegreesToRadians(SmoothingAngle), TwistAxis);

	if (WorkData.Targets.Num() != Quats.Num())
	{
		WorkData.Targets = Quats;
		WorkData.Hash = HashTargets(TArrayView<const FQuat>(Quats.GetData(), Quats.Num()));
		WorkData.Interpolator = TRBFInterpolator<FQuat>(WorkData.Targets, WeightFunc);
		WorkData.bAreTargetsConstant = true;
	}
	else if (WorkData.bAreTargetsConstant)
	{
		// Are the targets still constant? If the hash changed, then, no, and they shall stay non-constant 
		// for the lifetime of this unit's instance.
		uint64 Hash = HashTargets(TArrayView<const FQuat>(Quats.GetData(), Quats.Num()));
		if (Hash != WorkData.Hash)
		{
			WorkData.bAreTargetsConstant = false;
		}
	}

	if (!WorkData.bAreTargetsConstant)
	{
		// Re-initialize the interpolator with the new target values.
		WorkData.Interpolator = TRBFInterpolator<FQuat>(Quats, WeightFunc);
	}

	WorkData.Interpolator.Interpolate(Weights, Input, /*bClip=*/true, bNormalizeOutput);
}

uint64 FRigVMFunction_MathRBFInterpolateQuatBase::HashTargets(const TArrayView<const FQuat>& Targets)
{
	uint64 Hash = 0;
	for (const FQuat& T : Targets)
	{
		Hash = HashValues({ HashValue(T.X), HashValue(T.Y), HashValue(T.Z), HashValue(T.W) }, Hash);
	}
	return Hash;
}



template<typename T>
void FRigVMFunction_MathRBFInterpolateVectorBase::GetInterpolatedWeights(
	FRigVMFunction_MathRBFInterpolateVectorWorkData& WorkData,
	const TArrayView<const T>& Targets,
	const FVector& Input,
	ERBFVectorDistanceType DistanceFunction,
	ERBFKernelType SmoothingFunction,
	float SmoothingRadius,
	bool bNormalizeOutput,
	TArray<float>& Weights
)
{
	TArray<FVector > Vectors;
	Vectors.Reserve(Targets.Num());

	for (const auto& Target : Targets)
	{
		Vectors.Add(Target.Target);
	}

	FRigVMFunction_MathRBFVectorWeightFunctor WeightFunc(
		DistanceFunction, SmoothingFunction, SmoothingRadius);

	if (WorkData.Targets.Num() != Vectors.Num())
	{
		WorkData.Targets = Vectors;
		WorkData.Hash = HashTargets(TArrayView<const FVector>(Vectors.GetData(), Vectors.Num()));
		WorkData.Interpolator = TRBFInterpolator<FVector>(WorkData.Targets, WeightFunc);
	}
	else if (WorkData.bAreTargetsConstant)
	{
		// Are the targets still constant? If the hash changed, then, no, and they shall stay non-constant 
		// for the lifetime of this unit's instance.
		uint64 Hash = HashTargets(TArrayView<const FVector>(Vectors.GetData(), Vectors.Num()));
		if (Hash != WorkData.Hash)
		{
			WorkData.bAreTargetsConstant = false;
		}
	}

	if (!WorkData.bAreTargetsConstant)
	{
		// Re-initialize the interpolator with the new target values.
		WorkData.Interpolator = TRBFInterpolator<FVector>(Vectors, WeightFunc);
	}

	WorkData.Interpolator.Interpolate(Weights, Input, /*bClip=*/true, bNormalizeOutput);
}

uint64 FRigVMFunction_MathRBFInterpolateVectorBase::HashTargets(const TArrayView<const FVector>& Targets)
{
	uint64 Hash = 0;
	for (const FVector& T : Targets)
	{
		Hash = HashValues({ HashValue(T.X), HashValue(T.Y), HashValue(T.Z)}, Hash);
	}
	return Hash;
}



FRigVMFunction_MathRBFInterpolateQuatFloat_Execute()
{
	const TArrayView<const FMathRBFInterpolateQuatFloat_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateQuatFloat_Target>(
		WorkData, TargetsView, Input,  
		DistanceFunction, SmoothingFunction, SmoothingAngle, 
		bNormalizeOutput, TwistAxis, Weights);

	Output = 0.0f;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}


FRigVMFunction_MathRBFInterpolateQuatVector_Execute()
{
	const TArrayView<const FMathRBFInterpolateQuatVector_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateQuatVector_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingAngle, 
		bNormalizeOutput, TwistAxis, Weights);

	Output = FVector::ZeroVector;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}


FRigVMFunction_MathRBFInterpolateQuatColor_Execute()
{
	const TArrayView<const FMathRBFInterpolateQuatColor_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateQuatColor_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingAngle, 
		bNormalizeOutput, TwistAxis, Weights);

	Output = FLinearColor::Transparent;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}

FRigVMFunction_MathRBFInterpolateQuatQuat_Execute()
{
	const TArrayView<const FMathRBFInterpolateQuatQuat_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateQuatQuat_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingAngle,
		bNormalizeOutput, TwistAxis, Weights);
	
	Output = FQuat::Identity;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output = AddQuatWithWeight(Output, Targets[i].Value, Weights[i]);
	}
	Output.Normalize();
}


FRigVMFunction_MathRBFInterpolateQuatXform_Execute()
{
	const TArrayView<const FMathRBFInterpolateQuatXform_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateQuatXform_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingAngle,
		bNormalizeOutput, TwistAxis, Weights);

	FVector Scale = FVector::ZeroVector;
	FVector Translation = FVector::ZeroVector;
	FQuat Rotation = FQuat::Identity;

	for (int32 i = 0; i < Targets.Num(); i++)
	{
		const FTransform& Xform = Targets[i].Value;
		float W = Weights[i];

		Scale += Xform.GetScale3D() * W;
		Translation += Xform.GetTranslation() * W;
		Rotation = AddQuatWithWeight(Rotation, Xform.GetRotation(), W);
	}
	Rotation.Normalize();

	Output = FTransform(Rotation, Translation, Scale);
}


FRigVMFunction_MathRBFInterpolateVectorFloat_Execute()
{
	const TArrayView<const FMathRBFInterpolateVectorFloat_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateVectorFloat_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingRadius, 
		bNormalizeOutput, Weights);

	Output = 0.0f;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}


FRigVMFunction_MathRBFInterpolateVectorVector_Execute()
{
	const TArrayView<const FMathRBFInterpolateVectorVector_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateVectorVector_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingRadius, 
		bNormalizeOutput, Weights);

	Output = FVector::ZeroVector;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}


FRigVMFunction_MathRBFInterpolateVectorColor_Execute()
{
	const TArrayView<const FMathRBFInterpolateVectorColor_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateVectorColor_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingRadius, 
		bNormalizeOutput, Weights);

	Output = FLinearColor::Transparent;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output += Targets[i].Value * Weights[i];
	}
}


FRigVMFunction_MathRBFInterpolateVectorQuat_Execute()
{
	const TArrayView<const FMathRBFInterpolateVectorQuat_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateVectorQuat_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingRadius, 
		bNormalizeOutput, Weights);

	Output = FQuat::Identity;
	for (int32 i = 0; i < Targets.Num(); i++)
	{
		Output = AddQuatWithWeight(Output, Targets[i].Value, Weights[i]);
	}
	Output.Normalize();
}


FRigVMFunction_MathRBFInterpolateVectorXform_Execute()
{
	const TArrayView<const FMathRBFInterpolateVectorXform_Target> TargetsView(Targets.GetData(), Targets.Num());
	TArray<float> Weights;
	GetInterpolatedWeights<FMathRBFInterpolateVectorXform_Target>(
		WorkData, TargetsView, Input,
		DistanceFunction, SmoothingFunction, SmoothingRadius, 
		bNormalizeOutput, Weights);

	FVector Scale = FVector::ZeroVector;
	FVector Translation = FVector::ZeroVector;
	FQuat Rotation = FQuat::Identity;

	for (int32 i = 0; i < Targets.Num(); i++)
	{
		const FTransform& Xform = Targets[i].Value;
		float W = Weights[i];

		Scale += Xform.GetScale3D() * W;
		Translation += Xform.GetTranslation() * W;
		Rotation = AddQuatWithWeight(Rotation, Xform.GetRotation(), W);
	}
	Rotation.Normalize();

	Output = FTransform(Rotation, Translation, Scale);
}

