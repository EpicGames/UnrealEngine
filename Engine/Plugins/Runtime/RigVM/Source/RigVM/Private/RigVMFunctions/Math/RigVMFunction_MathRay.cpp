// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathRay.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathRay)

FRigVMFunction_MathRayIntersectRay_Execute()
{
	const FVector DirectionA = A.Direction * 10000.0;
	const FVector DirectionB = B.Direction * 10000.0;
	
	FVector ClosestA, ClosestB;
	FMath::SegmentDistToSegmentSafe(
		A.Origin - DirectionA,
		A.Origin + DirectionA,
		B.Origin - DirectionB,
		B.Origin + DirectionB,
		ClosestA, ClosestB);

	Result = (ClosestA + ClosestB) * 0.5;
	Distance = 0;
	if(!ClosestA.Equals(ClosestB))
	{
		Distance = static_cast<float>((ClosestA - ClosestB).Size());
	}
	RatioA = static_cast<float>(A.GetParameter(ClosestA));	
	RatioB = static_cast<float>(B.GetParameter(ClosestB));	
}

FRigVMFunction_MathRayIntersectPlane_Execute()
{
	const FVector NormalizedPlaneNormal = PlaneNormal.GetSafeNormal(UE_SMALL_NUMBER, FVector::UpVector);
	if (FMath::IsNearlyZero(FVector::DotProduct(NormalizedPlaneNormal, Ray.Direction)))
	{
		Result = FVector::ZeroVector;
		Distance = Ratio = 0.f;
		return;
	}
	
	Ratio = static_cast<float>(FMath::RayPlaneIntersectionParam(Ray.Origin, Ray.Direction, FPlane(PlanePoint, NormalizedPlaneNormal)));
	Result = Ray.PointAt(Ratio);
	Distance = 0;
	if(!Ray.Origin.Equals(Result))
	{
		Distance = static_cast<float>((Ray.Origin - Result).Size());
	}
}

FRigVMFunction_MathRayGetAt_Execute()
{
	Result = Ray.PointAt(Ratio);
}

FRigVMFunction_MathRayTransform_Execute()
{
	Result.Origin = Transform.TransformPosition(Ray.Origin);
	Result.Direction = Transform.TransformVector(Ray.Direction);
}
