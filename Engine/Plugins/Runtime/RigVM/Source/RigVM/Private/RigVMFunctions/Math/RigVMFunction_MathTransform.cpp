// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "AnimationCoreLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathTransform)

FRigVMFunction_MathTransformMake_Execute()
{
	Result = FTransform(Rotation, Translation, Scale);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathTransformMake::GetUpgradeInfo() const
{
	FRigVMStructUpgradeInfo Info = FRigVMStructUpgradeInfo::MakeFromStructToFactory(
		StaticStruct(),
		FRigVMDispatch_Constant::StaticStruct(),
		{{TEXT("Result"), TEXT("Value")}}
	);
	Info.AddRemappedPin(TEXT("Translation"), TEXT("Value.Translation"));
	Info.AddRemappedPin(TEXT("Rotation"), TEXT("Value.Rotation"));
	Info.AddRemappedPin(TEXT("Scale"), TEXT("Value.Scale"));
	return Info;
}

FRigVMFunction_MathTransformFromEulerTransform_Execute()
{
	Result = EulerTransform.ToFTransform();
}

FRigVMStructUpgradeInfo FRigVMFunction_MathTransformFromEulerTransform::GetUpgradeInfo() const
{
	FRigVMFunction_MathTransformFromEulerTransformV2 NewNode;
	NewNode.Value = EulerTransform;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("EulerTransform"), TEXT("Value"));
	return Info;
}

FRigVMFunction_MathTransformFromEulerTransformV2_Execute()
{
	Result = Value.ToFTransform();
}

FRigVMFunction_MathTransformToEulerTransform_Execute()
{
	Result.FromFTransform(Value);
}

FRigVMFunction_MathTransformToVectors_Execute()
{
	const FQuat Rotation = Value.GetRotation().GetNormalized();
	Forward = Rotation.GetForwardVector();
	Right = Rotation.GetRightVector();
	Up = Rotation.GetUpVector();
}

FRigVMFunction_MathTransformMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathTransformMakeRelative_Execute()
{
	Local = Global.GetRelativeTransform(Parent);
	Local.NormalizeRotation();
}

FRigVMFunction_MathTransformMakeAbsolute_Execute()
{
	Global = Local * Parent;
	Global.NormalizeRotation();
}

FString FRigVMFunction_MathTransformAccumulateArray::GetUnitLabel() const
{
	static const FString RelativeLabel = TEXT("Make Transform Array Relative");
	static const FString AbsoluteLabel = TEXT("Make Transform Array Absolute");
	return (TargetSpace == ERigVMTransformSpace::GlobalSpace) ? AbsoluteLabel : RelativeLabel;
}

FRigVMFunction_MathTransformAccumulateArray_Execute()
{
	if(Transforms.Num() == 0)
	{
		return;
	}

	if(ParentIndices.Num() > 0 && ParentIndices.Num() != Transforms.Num())
	{
		UE_RIGVMSTRUCT_REPORT_ERROR(TEXT("If the indices are specified their num (%d) has to match the transforms (%d)."), ParentIndices.Num(), Transforms.Num());
		return;
	}

	if(TargetSpace == ERigVMTransformSpace::LocalSpace)
	{
		if(ParentIndices.IsEmpty())
		{
			for(int32 Index=Transforms.Num()-1; Index>=0;Index--)
			{
				const FTransform& ParentTransform = (Index == 0) ? Root : Transforms[Index - 1];
				Transforms[Index] = Transforms[Index].GetRelativeTransform(ParentTransform);
			}
		}
		else
		{
			for(int32 Index=Transforms.Num()-1; Index>=0;Index--)
			{
				const int32 ParentIndex = ParentIndices[Index];
				const FTransform& ParentTransform = (ParentIndex == INDEX_NONE || ParentIndex >= Index) ? Root : Transforms[ParentIndex];
				Transforms[Index] = Transforms[Index].GetRelativeTransform(ParentTransform);
			}
		}
	}
	else
	{
		if(ParentIndices.IsEmpty())
		{
			for(int32 Index=0; Index<Transforms.Num(); Index++)
			{
				const FTransform& ParentTransform = (Index == 0) ? Root : Transforms[Index - 1];
				Transforms[Index] = Transforms[Index] * ParentTransform;
			}
		}
		else
		{
			for(int32 Index=0; Index<Transforms.Num(); Index++)
			{
				const int32 ParentIndex = ParentIndices[Index];
				const FTransform& ParentTransform = (ParentIndex == INDEX_NONE || ParentIndex >= Index) ? Root : Transforms[ParentIndex];
				Transforms[Index] = Transforms[Index] * ParentTransform;
			}
		}
	}
}

FRigVMFunction_MathTransformInverse_Execute()
{
	Result = Value.Inverse();
}

FRigVMFunction_MathTransformLerp_Execute()
{
	Result = FRigVMMathLibrary::LerpTransform(A, B, T);
}

FRigVMFunction_MathTransformSelectBool_Execute()
{
	Result = Condition ? IfTrue : IfFalse;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathTransformSelectBool::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigVMFunction_MathTransformRotateVector_Execute()
{
	Result = Transform.TransformVector(Vector);
}

FRigVMFunction_MathTransformTransformVector_Execute()
{
	Result = Transform.TransformPosition(Location);
}

FRigVMFunction_MathTransformFromSRT_Execute()
{
	Transform.SetLocation(Location);
	Transform.SetRotation(AnimationCore::QuatFromEuler(Rotation, RotationOrder));
	Transform.SetScale3D(Scale);
	EulerTransform.FromFTransform(Transform);
}


FRigVMFunction_MathTransformArrayToSRT_Execute()
{
	if (Transforms.Num() == 0)
	{
		return;
	}

	Translations.SetNumUninitialized(Transforms.Num());
	Rotations.SetNumUninitialized(Transforms.Num());
	Scales.SetNumUninitialized(Transforms.Num());

	for (int32 Index = 0; Index < Transforms.Num(); Index++)
	{
		Translations[Index] = Transforms[Index].GetTranslation();
		Rotations[Index] = Transforms[Index].GetRotation();
		Scales[Index] = Transforms[Index].GetScale3D();
	}

}

FRigVMFunction_MathTransformClampSpatially_Execute()
{
	FVector Position;
	FRigVMFunction_MathVectorClampSpatially::StaticExecute(ExecuteContext, Value.GetTranslation(), Axis, Type, Minimum, Maximum, Space, bDrawDebug, DebugColor, DebugThickness, Position);
	Result = Value;
	Result.SetTranslation(Position);
}

FRigVMFunction_MathTransformMirrorTransform_Execute()
{
	FRigVMMirrorSettings MirrorSettings;
	MirrorSettings.MirrorAxis = MirrorAxis;
	MirrorSettings.AxisToFlip = AxisToFlip;

	FTransform Local = FTransform::Identity;
	FRigVMFunction_MathTransformMakeRelative::StaticExecute(ExecuteContext, Value, CentralTransform, Local);
	Local = MirrorSettings.MirrorTransform(Local);
	FRigVMFunction_MathTransformMakeAbsolute::StaticExecute(ExecuteContext, Local, CentralTransform, Result);
}

