// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMCore/RigVMStructTest.h"

namespace FRigVMFunction_MathTransformTest_Utils
{
	bool IsNearlyEqual(float A, float B, float Tolerance = 0.001f)
	{
		return FMath::IsNearlyEqual(A, B, Tolerance);
	}

	bool IsNearlyEqual(const FVector& A, const FVector& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.X, B.X, Tolerance) &&
			FMath::IsNearlyEqual(A.Y, B.Y, Tolerance) &&
			FMath::IsNearlyEqual(A.Z, B.Z, Tolerance);
	}

	bool IsNearlyEqual(const FQuat& A, const FQuat& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.X, B.X, Tolerance) &&
			FMath::IsNearlyEqual(A.Y, B.Y, Tolerance) &&
			FMath::IsNearlyEqual(A.Z, B.Z, Tolerance) &&
			FMath::IsNearlyEqual(A.W, B.W, Tolerance);
	}

	bool IsNearlyEqual(const FRotator& A, const FRotator& B, double Tolerance = 0.001)
	{
		return FMath::IsNearlyEqual(A.Yaw, B.Yaw, Tolerance) &&
			FMath::IsNearlyEqual(A.Roll, B.Roll, Tolerance) &&
			FMath::IsNearlyEqual(A.Pitch, B.Pitch, Tolerance);
	}

	bool IsNearlyEqual(const FQuat& A, const FVector& B, double Tolerance = 0.001)
	{
		FVector Euler = A.Euler();
		return IsNearlyEqual(Euler, B, Tolerance);
	}
};

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformFromEulerTransform)
{
	Unit.EulerTransform = FEulerTransform(FVector(1.f, 2.f, 3.f), FRotator(10.f, 20.f, 30.f), FVector(4.f, 5.f, 6.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetRotation(), FVector(30.f, 10.f, 20.f)), TEXT("unexpected result"));
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetScale3D(), FVector(4.f, 5.f, 6.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformToEulerTransform)
{
	Unit.Value = FTransform(FQuat::MakeFromEuler(FVector(30.f, 10.f, 20.f)), FVector(1.f, 2.f, 3.f), FVector(4.f, 5.f, 6.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.Location, FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.Rotation, FRotator(10.f, 20.f, 30.f)), TEXT("unexpected result"));
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.Scale, FVector(4.f, 5.f, 6.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformMul)
{
	Unit.A = FTransform(FQuat::Identity, FVector(1.f, 2.f, 3.f));
	Unit.B = FTransform(FQuat::Identity, FVector(4.f, 5.f, 6.f), FVector(3.f, 3.f, 3.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(7.f, 11.f, 15.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformMakeRelative)
{
	Unit.Global = FTransform(FQuat::Identity, FVector(7.f, 11.f, 15.f));
	Unit.Parent = FTransform(FQuat::Identity, FVector(4.f, 5.f, 6.f), FVector(3.f, 3.f, 3.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Local.GetLocation(), FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformInverse)
{
	Unit.Value = FTransform(FQuat::Identity, FVector(1.f, 2.f, 3.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(-1.f, -2.f, -3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformLerp)
{
	Unit.A = FTransform(FQuat::Identity, FVector(1.f, 2.f, 3.f));
	Unit.B = FTransform(FQuat::Identity, FVector(4.f, 5.f, 6.f));
	Unit.T = 0.5f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(2.5f, 3.5f, 4.5f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformSelectBool)
{
	Unit.Condition = false;
	Unit.IfTrue = FTransform(FQuat::Identity, FVector(1.f, 2.f, 3.f));
	Unit.IfFalse = FTransform(FQuat::Identity, FVector(4.f, 5.f, 6.f));
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(4.f, 5.f, 6.f)), TEXT("unexpected result"));
	Unit.Condition = true;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result.GetLocation(), FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformRotateVector)
{
	Unit.Transform = FTransform(FQuat(FVector(1.f, 0.f, 0.f), -HALF_PI), FVector(1.f, 2.f, 3.f));
	Unit.Vector = FVector(0.f, 0.f, 4.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 4.f, 0.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformTransformVector)
{
	Unit.Transform = FTransform(FQuat(FVector(1.f, 0.f, 0.f), -HALF_PI), FVector(1.f, 2.f, 3.f));
	Unit.Location = FVector(0.f, 0.f, 4.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, 6.f, 3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathTransformArrayToSRT)
{
	Unit.Transforms.SetNumUninitialized(5);

	for (int32 Index = 0; Index < Unit.Transforms.Num(); Index++)
	{
		Unit.Transforms[Index] = FTransform(FQuat(FVector(1.f, 0.f, 0.f), HALF_PI * float(Index)), FVector(float(Index), 0.f, 0.f));
	}

	Execute();

	for (int32 Index = 0; Index < Unit.Transforms.Num(); Index++)
	{
		AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Translations[Index], FVector(float(Index), 0.f, 0.f)), TEXT("unexpected result"));
		AddErrorIfFalse(FRigVMFunction_MathTransformTest_Utils::IsNearlyEqual(Unit.Rotations[Index], FQuat(FVector(1.f, 0.f, 0.f), HALF_PI * float(Index))), TEXT("unexpected result"));
	}
	return true;
}

#endif
