// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"
#include "RigVMFunctions/Math/RigVMFunction_MathTransform.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_Core.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_MathVector)

FRigVMFunction_MathVectorMake_Execute()
{
	Result = FVector(X, Y, Z);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorMake::GetUpgradeInfo() const
{
	FRigVMStructUpgradeInfo Info = FRigVMStructUpgradeInfo::MakeFromStructToFactory(
		StaticStruct(),
		FRigVMDispatch_Constant::StaticStruct(),
		{{TEXT("Result"), TEXT("Value")}}
	);
	Info.AddRemappedPin(TEXT("X"), TEXT("Value.X"));
	Info.AddRemappedPin(TEXT("Y"), TEXT("Value.Y"));
	Info.AddRemappedPin(TEXT("Z"), TEXT("Value.Z"));
	return Info;
}

FRigVMFunction_MathVectorFromFloat_Execute()
{
	Result = FVector(Value, Value, Value);
}

FRigVMFunction_MathVectorFromDouble_Execute()
{
	Result = FVector(Value, Value, Value);
}

FRigVMFunction_MathVectorAdd_Execute()
{
	Result = A + B;
}

FRigVMFunction_MathVectorSub_Execute()
{
	Result = A - B;
}

FRigVMFunction_MathVectorMul_Execute()
{
	Result = A * B;
}

FRigVMFunction_MathVectorScale_Execute()
{
	Result = Value * Factor;
}

FRigVMFunction_MathVectorDiv_Execute()
{
	if(FMath::IsNearlyZero(B.X) || FMath::IsNearlyZero(B.Y) || FMath::IsNearlyZero(B.Z))
	{
		if (FMath::IsNearlyZero(B.X))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.X is nearly 0.f"));
		}
		if (FMath::IsNearlyZero(B.Y))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.Y is nearly 0.f"));
		}
		if (FMath::IsNearlyZero(B.Z))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.Z is nearly 0.f"));
		}
		Result = FVector::ZeroVector;
		return;
	}
	Result = A / B;
}

FRigVMFunction_MathVectorMod_Execute()
{
	if(FMath::IsNearlyZero(B.X) || FMath::IsNearlyZero(B.Y) || FMath::IsNearlyZero(B.Z) || B.X < 0.f || B.Y < 0.f || B.Z < 0.f)
	{
		if (FMath::IsNearlyZero(B.X) || B.X < 0.f)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.X needs to be greater than 0"));
		}
		if (FMath::IsNearlyZero(B.Y) || B.Y < 0.f)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.Y needs to be greater than 0"));
		}
		if (FMath::IsNearlyZero(B.Z) || B.Z < 0.f)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B.Z needs to be greater than 0"));
		}
		Result = FVector::ZeroVector;
		return;
	}

	Result.X = FMath::Fmod(A.X, B.X);
	Result.Y = FMath::Fmod(A.Y, B.Y);
	Result.Z = FMath::Fmod(A.Z, B.Z);
}

FRigVMFunction_MathVectorMin_Execute()
{
	Result.X = FMath::Min<double>(A.X, B.X);
	Result.Y = FMath::Min<double>(A.Y, B.Y);
	Result.Z = FMath::Min<double>(A.Z, B.Z);
}

FRigVMFunction_MathVectorMax_Execute()
{
	Result.X = FMath::Max<double>(A.X, B.X);
	Result.Y = FMath::Max<double>(A.Y, B.Y);
	Result.Z = FMath::Max<double>(A.Z, B.Z);
}

FRigVMFunction_MathVectorNegate_Execute()
{
	Result = -Value;
}

FRigVMFunction_MathVectorAbs_Execute()
{
	Result.X = FMath::Abs(Value.X);
	Result.Y = FMath::Abs(Value.Y);
	Result.Z = FMath::Abs(Value.Z);
}

FRigVMFunction_MathVectorFloor_Execute()
{
	Result.X = FMath::FloorToFloat(Value.X);
	Result.Y = FMath::FloorToFloat(Value.Y);
	Result.Z = FMath::FloorToFloat(Value.Z);
}

FRigVMFunction_MathVectorCeil_Execute()
{
	Result.X = FMath::CeilToFloat(Value.X);
	Result.Y = FMath::CeilToFloat(Value.Y);
	Result.Z = FMath::CeilToFloat(Value.Z);
}

FRigVMFunction_MathVectorRound_Execute()
{
	Result.X = FMath::RoundToFloat(Value.X);
	Result.Y = FMath::RoundToFloat(Value.Y);
	Result.Z = FMath::RoundToFloat(Value.Z);
}

FRigVMFunction_MathVectorSign_Execute()
{
	Result = Value.GetSignVector();
}

FRigVMFunction_MathVectorClamp_Execute()
{
	Result.X = FMath::Clamp<double>(Value.X, Minimum.X, Maximum.X);
	Result.Y = FMath::Clamp<double>(Value.Y, Minimum.Y, Maximum.Y);
	Result.Z = FMath::Clamp<double>(Value.Z, Minimum.Z, Maximum.Z);
}

FRigVMFunction_MathVectorLerp_Execute()
{
	Result = FMath::Lerp<FVector>(A, B, T);
}

FRigVMFunction_MathVectorRemap_Execute()
{
	FVector Ratio = FVector::ZeroVector;
	if (FMath::IsNearlyEqual(SourceMinimum.X, SourceMaximum.X) || FMath::IsNearlyEqual(SourceMinimum.Y, SourceMaximum.Y) || FMath::IsNearlyEqual(SourceMinimum.Z, SourceMaximum.Z))
	{
		if (FMath::IsNearlyEqual(SourceMinimum.X, SourceMaximum.X))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum X are the same."));
		}
		if (FMath::IsNearlyEqual(SourceMinimum.Y, SourceMaximum.Y))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum Y are the same."));
		}
		if (FMath::IsNearlyEqual(SourceMinimum.Z, SourceMaximum.Z))
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("The source minimum and maximum Z are the same."));
		}
	}
	else
	{
		Ratio = (Value - SourceMinimum) / (SourceMaximum - SourceMinimum);
	}
	if (bClamp)
	{
		Ratio.X = FMath::Clamp<double>(Ratio.X, 0, 1);
		Ratio.Y = FMath::Clamp<double>(Ratio.Y, 0, 1);
		Ratio.Z = FMath::Clamp<double>(Ratio.Z, 0, 1);
	}
	Result = FMath::Lerp<FVector>(TargetMinimum, TargetMaximum, Ratio);
}

FRigVMFunction_MathVectorEquals_Execute()
{
	Result = A == B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreEquals::StaticStruct());
}

FRigVMFunction_MathVectorNotEquals_Execute()
{
	Result = A != B;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorNotEquals::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo::MakeFromStructToFactory(StaticStruct(), FRigVMDispatch_CoreNotEquals::StaticStruct());
}

FRigVMFunction_MathVectorIsNearlyZero_Execute()
{
	if(Tolerance < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = Value.IsNearlyZero(FMath::Max<float>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathVectorIsNearlyEqual_Execute()
{
	if(Tolerance < 0.f)
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Tolerance is below zero"));
	}
	Result = (A - B).IsNearlyZero(FMath::Max<float>(Tolerance, SMALL_NUMBER));
}

FRigVMFunction_MathVectorSelectBool_Execute()
{
	Result = Condition ? IfTrue : IfFalse;
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorSelectBool::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigVMFunction_MathVectorDeg_Execute()
{
	Result = FMath::RadiansToDegrees(Value);
}

FRigVMFunction_MathVectorRad_Execute()
{
	Result = FMath::DegreesToRadians(Value);
}

FRigVMFunction_MathVectorLengthSquared_Execute()
{
	Result = static_cast<float>(Value.SizeSquared());
}

FRigVMFunction_MathVectorLength_Execute()
{
	Result = static_cast<float>(Value.Size());
}

FRigVMFunction_MathVectorDistance_Execute()
{
	Result = static_cast<float>(FVector::Distance(A, B));
}

FRigVMFunction_MathVectorCross_Execute()
{
	Result = A ^ B;
}

FRigVMFunction_MathVectorDot_Execute()
{
	Result = static_cast<float>(A | B);
}

FRigVMFunction_MathVectorUnit_Execute()
{
	if (Value.IsNearlyZero())
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is nearly zero"));
		Result = FVector::ZeroVector;
		return;
	}
	Result = Value.GetUnsafeNormal();
}

FRigVMFunction_MathVectorSetLength_Execute()
{
	if (Value.IsNearlyZero())
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is nearly zero"));
		Result = FVector::ZeroVector;
		return;
	}
	Result = Value.GetUnsafeNormal() * Length;
}

FRigVMFunction_MathVectorClampLength_Execute()
{
	if (Value.IsNearlyZero())
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Value is nearly zero"));
		Result = FVector::ZeroVector;
		return;
	}
	float Length = static_cast<float>(Value.Size());
	Result = Value * FMath::Clamp<float>(Length, MinimumLength, MaximumLength) / Length;
}

FRigVMFunction_MathVectorMirror_Execute()
{
	if (Normal.IsNearlyZero())
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Normal is nearly zero"));
		Result = FVector::ZeroVector;
		return;
	}
	Result = Value.MirrorByVector(Normal.GetSafeNormal());
}

FRigVMFunction_MathVectorAngle_Execute()
{
	if (A.IsNearlyZero() || B.IsNearlyZero())
	{
		if (A.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("A is nearly zero"));
		}
		if (B.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is nearly zero"));
		}
		Result = 0.f;
		return;
	}
	Result = static_cast<float>(FRigVMMathLibrary::FindQuatBetweenVectors(A, B).GetAngle());
}

FRigVMFunction_MathVectorParallel_Execute()
{
	if (A.IsNearlyZero() || B.IsNearlyZero())
	{
		if (A.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("A is nearly zero"));
		}
		if (B.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is nearly zero"));
		}
		Result = false;
		return;
	}
	Result = FVector::Parallel(A, B);
}

FRigVMFunction_MathVectorOrthogonal_Execute()
{
	if (A.IsNearlyZero() || B.IsNearlyZero())
	{
		if (A.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("A is nearly zero"));
		}
		if (B.IsNearlyZero())
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("B is nearly zero"));
		}
		Result = false;
		return;
	}
	Result = FVector::Orthogonal(A, B);
}

FRigVMFunction_MathVectorBezierFourPoint_Execute()
{
	FRigVMMathLibrary::FourPointBezier(Bezier, T, Result, Tangent);
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorBezierFourPoint::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

FRigVMFunction_MathVectorMakeBezierFourPoint_Execute()
{
}

FRigVMStructUpgradeInfo FRigVMFunction_MathVectorMakeBezierFourPoint::GetUpgradeInfo() const
{
	// this node is no longer supported
	return FRigVMStructUpgradeInfo();
}

void FRigVMFunction_DrawArrow(FRigVMDrawInterface* DrawInterface, const FTransform& ArrowTransform, const FVector& A, const FVector& B, const FVector& C)
{
	DrawInterface->DrawArrow(FTransform(A) * ArrowTransform, B * 0.25f, C * 0.08f, FLinearColor::Yellow, 0.f);
}

void FRigVMFunction_DrawPlane(FRigVMDrawInterface* DrawInterface, EAxis::Type Axis, float Minimum, float& CurrentMaximum, FTransform& CurrentSpace, FVector Result)
{
	FVector A, B, Normal;

	switch (Axis)
	{
	case EAxis::X:
	{
		A = FVector::YAxisVector;
		B = FVector::ZAxisVector;
		Normal = FVector::XAxisVector;
		break;
	}
	case EAxis::Y:
	{
		A = FVector::XAxisVector;
		B = FVector::ZAxisVector;
		Normal = FVector::YAxisVector;
		break;
	}
	default:
	{
		A = FVector::XAxisVector;
		B = FVector::YAxisVector;
		Normal = FVector::ZAxisVector;
		break;
	}
	}

	const FVector PointOnPlane = FVector::PointPlaneProject(Result, FPlane(CurrentSpace.GetTranslation(), Normal));
	const double PlaneScale = FMath::Max<double>(10., FVector::Distance(CurrentSpace.GetTranslation(), PointOnPlane)) * 0.525;

	FTransform PlaneRotation = FTransform::Identity;
	PlaneRotation.SetRotation(FMatrix(A, Normal.Cross(A), Normal, FVector::ZeroVector).ToQuat());
	const FTransform PlaneTranslation = FTransform((PointOnPlane - CurrentSpace.GetTranslation()) * 0.5);

	const FTransform MinimumWorldTransform = FTransform(Normal * Minimum) * CurrentSpace * PlaneTranslation;
	DrawInterface->DrawPlane(PlaneRotation * MinimumWorldTransform, FVector2D(PlaneScale, PlaneScale), FLinearColor::Yellow, true, FLinearColor::Yellow, GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy());
	DrawInterface->DrawArrow(MinimumWorldTransform, Normal * PlaneScale * 0.25f, A * PlaneScale * 0.08f, FLinearColor::Yellow, 0.f);

	if (CurrentMaximum > SMALL_NUMBER)
	{
		const FTransform MaximumWorldTransform = FTransform(Normal * CurrentMaximum) * CurrentSpace * PlaneTranslation;
		DrawInterface->DrawPlane(PlaneRotation * MaximumWorldTransform, FVector2D(PlaneScale, PlaneScale), FLinearColor::Yellow, true, FLinearColor::Yellow, GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy());
		DrawInterface->DrawArrow(MaximumWorldTransform, -Normal * PlaneScale * 0.25f, A * PlaneScale * 0.08f, FLinearColor::Yellow, 0.f);
	}
};

void FRigVMFunction_DrawCylinder(FRigVMDrawInterface* DrawInterface, EAxis::Type Axis, float Minimum, float& CurrentMaximum, FTransform& CurrentSpace, FVector Result)
{
	FTransform CircleTransform = FTransform::Identity;
	switch (Axis)
	{
	case EAxis::X:
	{
		CircleTransform.SetRotation(FQuat(FVector(0.f, 1.f, 0.f), PI * 0.5f));
		break;
	}
	case EAxis::Y:
	{
		CircleTransform.SetRotation(FQuat(FVector(1.f, 0.f, 0.f), PI * 0.5f));
		break;
	}
	default:
	{
		break;
	}
	}

	const FTransform CombinedTransform = CircleTransform * CurrentSpace;
	const FVector PointOnAxis = CombinedTransform.InverseTransformPosition(Result) * FVector(0, 0, 1);
	const float Extent = FMath::Abs<float>((float)PointOnAxis.Z);
	const FTransform CenterTransform = FTransform(PointOnAxis * 0.5) * CombinedTransform;
	const TArray<FTransform> Transforms = {
		FTransform(FVector(0, 0, Extent * 0.5)) * CenterTransform,
		FTransform(FVector(0, 0, -Extent * 0.5)) * CenterTransform
	};

	if (Minimum > SMALL_NUMBER)
	{
		for (const FTransform& Transform : Transforms)
		{
			DrawInterface->DrawCircle(Transform, FTransform::Identity, Minimum, FLinearColor::Yellow, 0.f, 32);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, FVector::XAxisVector * Minimum, FVector::XAxisVector * Minimum, FVector::YAxisVector * Minimum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, -FVector::XAxisVector * Minimum, -FVector::XAxisVector * Minimum, FVector::YAxisVector * Minimum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, FVector::YAxisVector * Minimum, FVector::YAxisVector * Minimum, FVector::XAxisVector * Minimum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, -FVector::YAxisVector * Minimum, -FVector::YAxisVector * Minimum, FVector::XAxisVector * Minimum);
		}
		DrawInterface->DrawLines(FTransform::Identity,
			{
				Transforms[0].TransformPosition(FVector::XAxisVector * Minimum),
				Transforms[1].TransformPosition(FVector::XAxisVector * Minimum),
				Transforms[0].TransformPosition(-FVector::XAxisVector * Minimum),
				Transforms[1].TransformPosition(-FVector::XAxisVector * Minimum),
				Transforms[0].TransformPosition(FVector::YAxisVector * Minimum),
				Transforms[1].TransformPosition(FVector::YAxisVector * Minimum),
				Transforms[0].TransformPosition(-FVector::YAxisVector * Minimum),
				Transforms[1].TransformPosition(-FVector::YAxisVector * Minimum),
			},
			FLinearColor::Yellow,
			0
			);
	}
	if (CurrentMaximum > SMALL_NUMBER)
	{
		for (const FTransform& Transform : Transforms)
		{
			TArray<FVector> LinesToDraw;
			LinesToDraw.Reserve(8);
			DrawInterface->DrawCircle(Transform, FTransform::Identity, CurrentMaximum, FLinearColor::Yellow, 0.f, 32);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, -FVector::XAxisVector * CurrentMaximum, FVector::XAxisVector * CurrentMaximum, FVector::YAxisVector * CurrentMaximum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, FVector::XAxisVector * CurrentMaximum, -FVector::XAxisVector * CurrentMaximum, FVector::YAxisVector * CurrentMaximum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, -FVector::YAxisVector * CurrentMaximum, FVector::YAxisVector * CurrentMaximum, FVector::XAxisVector * CurrentMaximum);
			FRigVMFunction_DrawArrow(DrawInterface, Transform, FVector::YAxisVector * CurrentMaximum, -FVector::YAxisVector * CurrentMaximum, FVector::XAxisVector * CurrentMaximum);
		}
		DrawInterface->DrawLines(FTransform::Identity,
			{
				Transforms[0].TransformPosition(FVector::XAxisVector * CurrentMaximum),
				Transforms[1].TransformPosition(FVector::XAxisVector * CurrentMaximum),
				Transforms[0].TransformPosition(-FVector::XAxisVector * CurrentMaximum),
				Transforms[1].TransformPosition(-FVector::XAxisVector * CurrentMaximum),
				Transforms[0].TransformPosition(FVector::YAxisVector * CurrentMaximum),
				Transforms[1].TransformPosition(FVector::YAxisVector * CurrentMaximum),
				Transforms[0].TransformPosition(-FVector::YAxisVector * CurrentMaximum),
				Transforms[1].TransformPosition(-FVector::YAxisVector * CurrentMaximum),
			},
			FLinearColor::Yellow,
			0
			);
	}
};

void FRigVMFunction_DrawSphere(FRigVMDrawInterface* DrawInterface, EAxis::Type Axis, float Minimum, float& CurrentMaximum, FTransform& CurrentSpace, FVector Result)
{
	FVector XAxisMin = FVector::XAxisVector * Minimum;
	FVector XAxisMax = FVector::XAxisVector * CurrentMaximum;
	FVector YAxisMin = FVector::YAxisVector * Minimum;
	FVector YAxisMax = FVector::YAxisVector * CurrentMaximum;
	FVector ZAxisMin = FVector::ZAxisVector * Minimum;
	FVector ZAxisMax = FVector::ZAxisVector * CurrentMaximum;

	FTransform CircleTransform = FTransform::Identity;
	if (Minimum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, Minimum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMin, XAxisMin, YAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMin, -XAxisMin, YAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMin, YAxisMin, YAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMin, -YAxisMin, YAxisMin);
	}
	if (CurrentMaximum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, CurrentMaximum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMax, XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMax, -XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMax, YAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMax, -YAxisMax, ZAxisMax);
	}
	CircleTransform.SetRotation(FQuat(FVector(0.f, 1.f, 0.f), PI * 0.5f));
	if (Minimum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, Minimum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMin, XAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMin, -XAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMin, YAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMin, -YAxisMin, ZAxisMin);
	}
	if (CurrentMaximum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, CurrentMaximum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMax, XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMax, -XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMax, YAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMax, -YAxisMax, ZAxisMax);
	}
	CircleTransform.SetRotation(FQuat(FVector(1.f, 0.f, 0.f), PI * 0.5f));
	if (Minimum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, Minimum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMin, XAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMin, -XAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMin, YAxisMin, ZAxisMin);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMin, -YAxisMin, ZAxisMin);
	}
	if (CurrentMaximum > SMALL_NUMBER)
	{
		DrawInterface->DrawCircle(CurrentSpace, CircleTransform, CurrentMaximum, FLinearColor::Yellow, 0.f, 32);
		const FTransform ArrowTransform = CircleTransform * CurrentSpace;
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -XAxisMax, XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, XAxisMax, -XAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, -YAxisMax, YAxisMax, ZAxisMax);
		FRigVMFunction_DrawArrow(DrawInterface, ArrowTransform, YAxisMax, -YAxisMax, ZAxisMax);
	}
};

FRigVMFunction_MathVectorClampSpatially_Execute()
{
	Result = FRigVMMathLibrary::ClampSpatially(Value, Axis, Type, Minimum, Maximum, Space);

	FRigVMDrawInterface* DrawInterface = ExecuteContext.GetDrawInterface();
	if (DrawInterface != nullptr && bDrawDebug)
	{
		ERigVMClampSpatialMode::Type Mode = Type;
		float CurrentMaximum = Maximum;
		FTransform CurrentSpace = Space;
		
		switch (Mode)
		{
			case ERigVMClampSpatialMode::Plane:
			{
				FRigVMFunction_DrawPlane(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
				break;
			}
			case ERigVMClampSpatialMode::Capsule:
			{
				const float Radius = Minimum;
				if(CurrentMaximum < Radius * 2.f)
				{
					CurrentMaximum = 0.f;
					FRigVMFunction_DrawSphere(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
					break;
				}
			
				const float Length = FMath::Max(CurrentMaximum, Radius * 2);
				const float HalfLength = Length * 0.5f;
				const float HalfCylinderLength = HalfLength - Radius;
					
				FTransform CircleTransform = FTransform::Identity;
				switch (Axis)
				{
					case EAxis::X:
					{
						CircleTransform.SetRotation(FQuat(FVector(0.f, 1.f, 0.f), PI * 0.5f));
						break;
					}
					case EAxis::Y:
					{
						CircleTransform.SetRotation(FQuat(FVector(1.f, 0.f, 0.f), PI * 0.5f));
						break;
					}
					default:
					{
						break;
					}
				}

				CircleTransform  = CircleTransform * Space;
					
				const FVector CylinderDirection = FVector::ZAxisVector * HalfCylinderLength;
				DrawInterface->DrawLines(CircleTransform,
					{
						FVector::XAxisVector * Minimum - CylinderDirection,
						FVector::XAxisVector * Minimum + CylinderDirection,
						-FVector::XAxisVector * Minimum - CylinderDirection,
						-FVector::XAxisVector * Minimum + CylinderDirection,
						FVector::YAxisVector * Minimum - CylinderDirection,
						FVector::YAxisVector * Minimum + CylinderDirection,
						-FVector::YAxisVector * Minimum - CylinderDirection,
						-FVector::YAxisVector * Minimum + CylinderDirection
					},
					FLinearColor::Yellow,
					0
				);

				CurrentMaximum = 0.f;
				{
					TGuardValue<FTransform> SpaceGuard(CurrentSpace, FTransform(CylinderDirection) * CircleTransform);
					FRigVMFunction_DrawSphere(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
				}
				{
					TGuardValue<FTransform> SpaceGuard(CurrentSpace, FTransform(-CylinderDirection) * CircleTransform);
					FRigVMFunction_DrawSphere(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
				}
				break;
			}
			case ERigVMClampSpatialMode::Cylinder:
			{
				FRigVMFunction_DrawCylinder(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
				break;
			}
			default:
			case ERigVMClampSpatialMode::Sphere:
			{
				FRigVMFunction_DrawSphere(DrawInterface, Axis, Minimum, CurrentMaximum, CurrentSpace, Result);
				break;
			}
		}
		
		DrawInterface->DrawPoint(FTransform::Identity, Result, DebugThickness * 8.f, DebugColor);
	}
}

FRigVMFunction_MathIntersectPlane_Execute()
{
	const FVector NormalizedPlaneNormal = PlaneNormal.GetSafeNormal(UE_SMALL_NUMBER, FVector(0.f, 0.f, 1.f));
	if (FMath::IsNearlyZero(FVector::DotProduct(NormalizedPlaneNormal, Direction)))
	{
		Result = FVector::ZeroVector;
		Distance = 0.f;
		return;
	}
	
	FPlane Plane(PlanePoint, NormalizedPlaneNormal);

	Result = FMath::RayPlaneIntersection(Start, Direction, Plane);
	Distance = static_cast<float>((Start - Result).Size());
}

FRigVMFunction_MathDistanceToPlane_Execute()
{
	FVector UnitVectorNormal = PlaneNormal.GetSafeNormal();

	if (!UnitVectorNormal.IsZero())
	{ 
		FPlane Plane(PlanePoint, UnitVectorNormal);

		ClosestPointOnPlane = FVector::PointPlaneProject(Point, Plane);
		SignedDistance = static_cast<float>(FVector::PointPlaneDist(Point, PlanePoint, UnitVectorNormal));
	}
	else
	{
		ClosestPointOnPlane = FVector::ZeroVector;
		SignedDistance = 0;
	}
}

FRigVMFunction_MathVectorMakeRelative_Execute()
{
	Local = Global - Parent;
}

FRigVMFunction_MathVectorMakeAbsolute_Execute()
{
	Global = Parent + Local;
}

FRigVMFunction_MathVectorMirrorTransform_Execute()
{
	FTransform Transform = FTransform::Identity;
	Transform.SetTranslation(Value);
	FRigVMFunction_MathTransformMirrorTransform::StaticExecute(ExecuteContext, Transform, MirrorAxis, AxisToFlip, CentralTransform, Transform);
	Result = Transform.GetTranslation();
}

FRigVMFunction_MathVectorArraySum_Execute()
{
	Sum = FVector::ZeroVector;
	for (const FVector& Value : Array)
	{
		Sum += Value;
	}
}

FRigVMFunction_MathVectorArrayAverage_Execute()
{
	Average = FVector::ZeroVector;
	if (!Array.IsEmpty())
	{
		for (const FVector& Value : Array)
		{
			Average += Value;
		}
		Average = Average / Array.Num();
	}
	else
	{
		UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("Array is empty"));
	}
}

