// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#include "RigVMFunctions/Math/RigVMFunction_MathVector.h"
#include "RigVMCore/RigVMStructTest.h"

namespace FRigVMFunction_MathVectorTest_Utils
{
	bool IsNearlyEqual(const FVector& A, const FVector& B, double Tolerance = SMALL_NUMBER)
	{
		return FMath::IsNearlyEqual(A.X, B.X, (FVector::FReal)Tolerance) &&
			FMath::IsNearlyEqual(A.Y, B.Y, (FVector::FReal)Tolerance) &&
			FMath::IsNearlyEqual(A.Z, B.Z, (FVector::FReal)Tolerance);
	}
};

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorAdd)
{
	Unit.A = FVector(4.f, 6.f, 8.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(5.f, 8.f, 11.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorFromFloat)
{
	Unit.Value = 7.f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(7.f, 7.f, 7.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorSub)
{
	Unit.A = FVector(4.f, 6.f, 8.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(3.f, 4.f, 5.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorMul)
{
	Unit.A = FVector(4.f, 6.f, 8.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 12.f, 24.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorScale)
{
	Unit.Value = FVector(4.f, 6.f, 8.f);
	Unit.Factor = 2.f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(8.f, 12.f, 16.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorDiv)
{
	Unit.A = FVector(4.f, 6.f, 8.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 3.f, 8.f / 3.f), KINDA_SMALL_NUMBER), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorMod)
{
	Unit.A = FVector(4.f, 6.f, 9.f);
	Unit.B = FVector(5.f, 4.f, 6.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 2.f, 3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorMin)
{
	Unit.A = FVector(4.f, 2.f, 8.f);
	Unit.B = FVector(1.f, 6.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorMax)
{
	Unit.A = FVector(4.f, 2.f, 8.f);
	Unit.B = FVector(1.f, 6.f, 3.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 6.f, 8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorNegate)
{
	Unit.Value = FVector(4.f, -2.f, 8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(-4.f, 2.f, -8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorAbs)
{
	Unit.Value = FVector(-4.f, 2.f, -8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 2.f, 8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorFloor)
{
	Unit.Value = FVector(4.1f, 2.6f, 8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 2.f, 8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorCeil)
{
	Unit.Value = FVector(4.1f, 2.6f, 8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(5.f, 3.f, 8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorRound)
{
	Unit.Value = FVector(4.1f, 2.6f, 8.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 3.f, 8.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorSign)
{
	Unit.Value = FVector(4.1f, -2.6f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, -1.f, 1.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorClamp)
{
	Unit.Value = FVector(-3.f, 2.f, 8.f);
	Unit.Minimum = FVector(0.f, 1.f, 2.f);
	Unit.Maximum = FVector(3.f, 4.f, 5.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 2.f, 5.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorLerp)
{
	Unit.A = FVector(2.f, 4.f, 8.f);
	Unit.B = FVector(12.f, 24.f, 38.f);
	Unit.T = 0.5f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(7.f, 14.f, 23.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorRemap)
{
	Unit.Value = FVector(3.f, 13.f, 23.f);
	Unit.SourceMinimum = FVector(2.f, 12.f, 22.f);
	Unit.SourceMaximum = FVector(4.f, 14.f, 24.f);
	Unit.TargetMinimum = FVector(36.f, 46.f, 56.f);
	Unit.TargetMaximum = FVector(38.f, 48.f, 58.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(37.f, 47.f, 57.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorEquals)
{
	Unit.A = FVector(1.f, 2.f, 3.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = FVector(1.f, 2.f, 4.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = FVector(2.f, 3.f, 4.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorNotEquals)
{
	Unit.A = FVector(1.f, 2.f, 3.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = FVector(1.f, 2.f, 4.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = FVector(2.f, 3.f, 4.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorIsNearlyZero)
{
	Unit.Value = FVector(0.f, 0.f, 0.f);
	Unit.Tolerance = 0.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.Value = FVector(0.001f, 0.003f, 0.005f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.Tolerance = 0.01f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorIsNearlyEqual)
{
	Unit.A = FVector(1.f, 2.f, 3.f);
	Unit.B = FVector(1.f, 2.f, 3.f);
	Unit.Tolerance = 0.f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B= FVector(1.001f, 2.003f, 3.005f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.Tolerance = 0.01f;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorSelectBool)
{
	Unit.Condition = false;
	Unit.IfTrue = FVector(1.f, 2.f, 3.f);
	Unit.IfFalse = FVector(4.f, 5.f, 7.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 5.f, 7.f)), TEXT("unexpected result"));
	Unit.Condition = true;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, 2.f, 3.f)), TEXT("unexpected result"));
	return true;
}

#if 0  // LWC_TODO: Fix this for LWC

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorDeg)
{
	Unit.Value = FVector(-PI, PI * 2.f, PI * 0.5f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(-180.f, 360.f, 90.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorRad)
{
	Unit.Value = FVector(-180.f, 360.f, 90.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(-PI, PI * 2.f, PI * 0.5f)), TEXT("unexpected result"));
	return true;
}

#endif

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorLengthSquared)
{
	Unit.Value = FVector(2.f, 3.f, 4.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 29.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorLength)
{
	Unit.Value = FVector(2.f, 3.f, 4.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, FMath::Sqrt(29.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorDistance)
{
	Unit.A = FVector(3.f, 4.f, 5.f);
	Unit.B = FVector(5.f, 7.f, 9.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, FMath::Sqrt(29.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorCross)
{
	Unit.A = FVector(1.f, 0.f, 0.f);
	Unit.B = FVector(0.f, 2.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 0.f, 2.f)), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorDot)
{
	Unit.A = FVector(1.f, 2.f, 3.f);
	Unit.B = FVector(4.f, 5.f, 6.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 32.f), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorUnit)
{
	Unit.Value = FVector(4.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, 0.f, 0.f), 0.001f), TEXT("unexpected result (0)"));
	Unit.Value = FVector(0.f, 5.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 1.f, 0.f), 0.001f), TEXT("unexpected result (1)"));
	Unit.Value = FVector(0.f, 0.f, 6.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 0.f, 1.f), 0.001f), TEXT("unexpected result (2)"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorSetLength)
{
	Unit.Value = FVector(4.f, 0.f, 0.f);
	Unit.Length = 2.f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(2.f, 0.f, 0.f), 0.001f), TEXT("unexpected result (0)"));
	Unit.Value = FVector(0.f, 5.f, 0.f);
	Unit.Length = 3.f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(0.f, 3.f, 0.f), 0.001f), TEXT("unexpected result (1)"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorClampLength)
{
	Unit.Value = FVector(4.f, 0.f, 0.f);
	Unit.MinimumLength = 2.f;
	Unit.MaximumLength = 6.f;
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(4.f, 0.f, 0.f), 0.001f), TEXT("unexpected result (0)"));
	Unit.Value = FVector(1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(2.f, 0.f, 0.f), 0.001f), TEXT("unexpected result (1)"));
	Unit.Value = FVector(8.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(6.f, 0.f, 0.f), 0.001f), TEXT("unexpected result (1)"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorMirror)
{
	Unit.Value = FVector(1.f, 2.f, 3.f);
	Unit.Normal = FVector(1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(-1.f, 2.f, 3.f)), TEXT("unexpected result"));
	Unit.Normal = FVector(0.f, 3.f, 0.f);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Result, FVector(1.f, -2.f, 3.f), 1e-6), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorAngle)
{
	Unit.A = FVector(-4.f, 0.f, 0.f);
	Unit.B = FVector(0.f, 3.f, 0.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, HALF_PI), TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorParallel)
{
	Unit.A = FVector(-4.f, 0.f, 0.f);
	Unit.B = FVector(1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	Unit.B = FVector(0.f, 1.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorOrthogonal)
{
	Unit.A = FVector(-4.f, 0.f, 0.f);
	Unit.B = FVector(1.f, 0.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected result"));
	Unit.B = FVector(0.f, 1.f, 0.f);
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathDistanceToPlane)
{
	Unit.Point = FVector(5, -6, 3);
	Unit.PlanePoint = FVector(4, 5, 0);
	Unit.PlaneNormal = FVector(3, -2, 1);
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.ClosestPointOnPlane, FVector(-1,-2,1), KINDA_SMALL_NUMBER), TEXT("unexpected result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.SignedDistance, 7.48331477f, KINDA_SMALL_NUMBER), TEXT("unexpected result"));
	
	Unit.Point = FVector(5, -6, 3);
	Unit.PlanePoint = FVector(2, 2, 2);
	Unit.PlaneNormal = FVector(0,0,0);
	Execute();
	AddErrorIfFalse(Unit.ClosestPointOnPlane.IsZero(), TEXT("unexpected result"));
	AddErrorIfFalse(Unit.SignedDistance == 0, TEXT("unexpected result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorArraySum)
{
	Unit.Array = { FVector(1.f, 2.f, 3.f), FVector(3.f, 2.f, 1.f), FVector(-1.f, 2.f, 3.5f) };
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Sum, FVector(3.f, 6.f, 7.5f), KINDA_SMALL_NUMBER), TEXT("unexpected sum"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_MathVectorArrayAverage)
{
	Unit.Array = { FVector(1.f, 2.f, 3.f), FVector(3.f, 2.f, 1.f), FVector(-1.f, 2.f, 3.5f) };
	Execute();
	AddErrorIfFalse(FRigVMFunction_MathVectorTest_Utils::IsNearlyEqual(Unit.Average, FVector(1.f, 2.f, 2.5f), KINDA_SMALL_NUMBER), TEXT("unexpected average"));
	return true;
}

#endif
