// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_Noise.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Noise)

FRigVMFunction_NoiseFloat_Execute()
{

	const float Noise = FMath::PerlinNoise1D(Value * Frequency + Time) + 0.5f;
	Result = FMath::Lerp<float>(Minimum, Maximum, Noise);
	Time = Time + Speed * ExecuteContext.GetDeltaTime<float>();
}

FRigVMFunction_NoiseDouble_Execute()
{
	const double Noise = FMath::PerlinNoise1D(static_cast<float>(Value * Frequency + Time)) + 0.5;
	Result = FMath::Lerp<double>(Minimum, Maximum, Noise);
	Time = Time + Speed * ExecuteContext.GetDeltaTime<double>();
}

FRigVMFunction_NoiseVector_Execute()
{
	FRigVMFunction_NoiseVector2::StaticExecute(ExecuteContext, Position, Speed, Frequency, Minimum, Maximum, Result, Time);
}

FRigVMStructUpgradeInfo FRigVMFunction_NoiseVector::GetUpgradeInfo() const
{
	FRigVMFunction_NoiseVector2 NewNode;
	NewNode.Value = Position;
	NewNode.Speed = Speed;
	NewNode.Frequency = Frequency;
	NewNode.Minimum = Minimum;
	NewNode.Maximum = Maximum;

	FRigVMStructUpgradeInfo Info(*this, NewNode);
	Info.AddRemappedPin(TEXT("Position"), TEXT("Value"));
	return Info;
}

FRigVMFunction_NoiseVector2_Execute()
{
	const double NoiseX = FMath::PerlinNoise1D(static_cast<float>(Value.X * Frequency.X + Time.X)) + 0.5;
	const double NoiseY = FMath::PerlinNoise1D(static_cast<float>(Value.Y * Frequency.Y + Time.Y)) + 0.5;
	const double NoiseZ = FMath::PerlinNoise1D(static_cast<float>(Value.Z * Frequency.Z + Time.Z)) + 0.5;
	Result.X = FMath::Lerp<double>(Minimum, Maximum, NoiseX);
	Result.Y = FMath::Lerp<double>(Minimum, Maximum, NoiseY);
	Result.Z = FMath::Lerp<double>(Minimum, Maximum, NoiseZ);
	Time = Time + Speed * ExecuteContext.GetDeltaTime<float>();
}

