// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_Random.h"
#include "GenericPlatform/GenericPlatformMath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Random)

float FRigVMFunction_Random_Helper(int32& Seed)
{
	Seed = (Seed * 196314165) + 907633515;
	union { float f; int32 i; } Result;
	union { float f; int32 i; } Temp;
	const float SRandTemp = 1.0f;
	Temp.f = SRandTemp;
	Result.i = (Temp.i & 0xff800000) | (Seed & 0x007fffff);
	return FPlatformMath::Fractional(Result.f);
}

FRigVMFunction_RandomInt_Execute()
{

	if (Seed != BaseSeed)
	{
		LastSeed = BaseSeed = Seed;
	}

	TimeLeft = TimeLeft - ExecuteContext.GetDeltaTime<float>();
	if (TimeLeft > 0.f)
	{
		Result = LastResult;
		return;
	}

	LastSeed = (LastSeed * 196314165) + 907633515;

	Result = Minimum + ((uint32)LastSeed) % (FMath::Max(Maximum - Minimum, 0) + 1);
	TimeLeft = Duration < -SMALL_NUMBER ? FLT_MAX : Duration;
	LastResult = Result;
}

FRigVMFunction_RandomFloat_Execute()
{

	if(Seed != BaseSeed)
	{
		LastSeed = BaseSeed = Seed;
	}

	TimeLeft = TimeLeft - ExecuteContext.GetDeltaTime<float>();
	if (TimeLeft > 0.f)
	{
		Result = LastResult;
		return;
	}

	Result = FRigVMFunction_Random_Helper(LastSeed);
	Result = FMath::Lerp<float>(Minimum, Maximum, Result);
	TimeLeft = Duration < -SMALL_NUMBER ? FLT_MAX : Duration;
	LastResult = Result;
}

FRigVMFunction_RandomVector_Execute()
{

	if(Seed != BaseSeed)
	{
		LastSeed = BaseSeed = Seed;
	}

	TimeLeft = TimeLeft - ExecuteContext.GetDeltaTime<float>();
	if (TimeLeft > 0.f)
	{
		Result = LastResult;
		return;
	}

	Result.X = FMath::Lerp<float>(Minimum, Maximum, FRigVMFunction_Random_Helper(LastSeed));
	Result.Y = FMath::Lerp<float>(Minimum, Maximum, FRigVMFunction_Random_Helper(LastSeed));
	Result.Z = FMath::Lerp<float>(Minimum, Maximum, FRigVMFunction_Random_Helper(LastSeed));
	TimeLeft = Duration < -SMALL_NUMBER ? FLT_MAX : Duration;
	LastResult = Result;
}
