// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMFunction_SpringMath.h"

#include "Math/SpringMath.h"
#include "Math/UnrealMathUtility.h"

FRigVMFunction_DampFloat_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	Result = Value;
	FMath::ExponentialSmoothingApprox(Result, Target, DeltaTime, SmoothingTime);
}

FRigVMFunction_DampVector_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	Result = Value;
	FMath::ExponentialSmoothingApprox(Result, Target, DeltaTime, SmoothingTime);
}

FRigVMFunction_DampQuaternion_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	Result = Value;
	SpringMath::ExponentialSmoothingApproxQuat(Result, Target, DeltaTime, SmoothingTime);
}

FRigVMFunction_SpringDampFloat_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	SpringMath::CriticalSpringDamper(Value, ValueVelocity, Target, SmoothingTime, DeltaTime);
}

FRigVMFunction_SpringDampVector_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	SpringMath::CriticalSpringDamper(Value, ValueVelocity, Target, SmoothingTime, DeltaTime);
}

FRigVMFunction_SpringDampQuat_Execute()
{
	float DeltaTime = ExecuteContext.GetDeltaTime<float>();
	SpringMath::CriticalSpringDamperQuat(Value, ValueVelocity, Target, SmoothingTime, DeltaTime);
}
