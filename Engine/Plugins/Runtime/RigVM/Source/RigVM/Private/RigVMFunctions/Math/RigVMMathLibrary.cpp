// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "AHEasing/easing.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMMathLibrary)

////////////////////////////////////////////////////////////////////////////////
// FRigVMMirrorSettings
////////////////////////////////////////////////////////////////////////////////

FTransform FRigVMMirrorSettings::MirrorTransform(const FTransform& InTransform) const
{
	FTransform Transform = InTransform;
	FQuat Quat = Transform.GetRotation();

	Transform.SetLocation(MirrorVector(Transform.GetLocation()));

	switch (AxisToFlip)
	{
	case EAxis::X:
		{
			FVector Y = MirrorVector(Quat.GetAxisY());
			FVector Z = MirrorVector(Quat.GetAxisZ());
			FMatrix Rotation = FRotationMatrix::MakeFromYZ(Y, Z);
			Transform.SetRotation(FQuat(Rotation));
			break;
		}
	case EAxis::Y:
		{
			FVector X = MirrorVector(Quat.GetAxisX());
			FVector Z = MirrorVector(Quat.GetAxisZ());
			FMatrix Rotation = FRotationMatrix::MakeFromXZ(X, Z);
			Transform.SetRotation(FQuat(Rotation));
			break;
		}
	default:
		{
			FVector X = MirrorVector(Quat.GetAxisX());
			FVector Y = MirrorVector(Quat.GetAxisY());
			FMatrix Rotation = FRotationMatrix::MakeFromXY(X, Y);
			Transform.SetRotation(FQuat(Rotation));
			break;
		}
	}

	return Transform;
}

FVector FRigVMMirrorSettings::MirrorVector(const FVector& InVector) const
{
	FVector Axis = FVector::ZeroVector;
	Axis.SetComponentForAxis(MirrorAxis, 1.f);
	return InVector.MirrorByVector(Axis);
}

FRigVMSimPoint FRigVMSimPoint::IntegrateVerlet(const FVector& InForce, float InBlend, float InDeltaTime) const
{
	FRigVMSimPoint Point = *this;
	if(Point.Mass > SMALL_NUMBER)
	{
		Point.LinearVelocity = FMath::Lerp<FVector>(Point.LinearVelocity, InForce / Point.Mass, FMath::Clamp<float>(InBlend * InDeltaTime, 0.f, 1.f)) * FMath::Clamp<float>(1.f - LinearDamping, 0.f, 1.f);
		Point.Position = Point.Position + Point.LinearVelocity * InDeltaTime;
	}
	return Point;
}

FRigVMSimPoint FRigVMSimPoint::IntegrateSemiExplicitEuler(const FVector& InForce, float InDeltaTime) const
{
	FRigVMSimPoint Point = *this;
	if(Point.Mass > SMALL_NUMBER)
	{
		Point.LinearVelocity += InForce * InDeltaTime / Point.Mass;
		Point.LinearVelocity -= LinearVelocity * LinearDamping;
		Point.Position += Point.LinearVelocity * InDeltaTime;
	}
	return Point;
}

float FRigVMMathLibrary::AngleBetween(const FVector& A, const FVector& B)
{
	if (A.IsNearlyZero() || B.IsNearlyZero())
	{
		return 0.f;
	}

	return static_cast<float>(FMath::Acos(FVector::DotProduct(A, B) / (A.Size() * B.Size())));
}

float FRigVMMathLibrary::EaseFloat(float Value, ERigVMAnimEasingType Type)
{
	switch(Type)
	{
		case ERigVMAnimEasingType::Linear:
		{
			break;
		}
		case ERigVMAnimEasingType::QuadraticEaseIn:
		{
			Value = QuadraticEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::QuadraticEaseOut:
		{
			Value = QuadraticEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::QuadraticEaseInOut:
		{
			Value = QuadraticEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::CubicEaseIn:
		{
			Value = CubicEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::CubicEaseOut:
		{
			Value = CubicEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::CubicEaseInOut:
		{
			Value = CubicEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::QuarticEaseIn:
		{
			Value = QuarticEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::QuarticEaseOut:
		{
			Value = QuarticEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::QuarticEaseInOut:
		{
			Value = QuarticEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::QuinticEaseIn:
		{
			Value = QuinticEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::QuinticEaseOut:
		{
			Value = QuinticEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::QuinticEaseInOut:
		{
			Value = QuinticEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::SineEaseIn:
		{
			Value = SineEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::SineEaseOut:
		{
			Value = SineEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::SineEaseInOut:
		{
			Value = SineEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::CircularEaseIn:
		{
			Value = CircularEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::CircularEaseOut:
		{
			Value = CircularEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::CircularEaseInOut:
		{
			Value = CircularEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::ExponentialEaseIn:
		{
			Value = ExponentialEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::ExponentialEaseOut:
		{
			Value = ExponentialEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::ExponentialEaseInOut:
		{
			Value = ExponentialEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::ElasticEaseIn:
		{
			Value = ElasticEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::ElasticEaseOut:
		{
			Value = ElasticEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::ElasticEaseInOut:
		{
			Value = ElasticEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::BackEaseIn:
		{
			Value = BackEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::BackEaseOut:
		{
			Value = BackEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::BackEaseInOut:
		{
			Value = BackEaseInOut(Value);
			break;
		}
		case ERigVMAnimEasingType::BounceEaseIn:
		{
			Value = BounceEaseIn(Value);
			break;
		}
		case ERigVMAnimEasingType::BounceEaseOut:
		{
			Value = BounceEaseOut(Value);
			break;
		}
		case ERigVMAnimEasingType::BounceEaseInOut:
		{
			Value = BounceEaseInOut(Value);
			break;
		}
	}

	return Value;
}

FTransform FRigVMMathLibrary::LerpTransform(const FTransform& A, const FTransform& B, float T)
{
	FTransform Result = FTransform::Identity;
	Result.SetLocation(FMath::Lerp<FVector>(A.GetLocation(), B.GetLocation(), T));
	Result.SetRotation(FQuat::Slerp(A.GetRotation(), B.GetRotation(), T));
	Result.SetScale3D(FMath::Lerp<FVector>(A.GetScale3D(), B.GetScale3D(), T));
	return Result;
}

FVector FRigVMMathLibrary::ClampSpatially(const FVector& Value, EAxis::Type Axis, ERigVMClampSpatialMode::Type Type, float Minimum, float Maximum, FTransform Space)
{
	FVector Local = Space.InverseTransformPosition(Value);

	auto Clamp = [](float InValue, float InMinimum, float InMaximum)
	{
		if(InMaximum <= InMinimum || InMaximum < SMALL_NUMBER)
		{
			return FMath::Max<float>(InValue, InMinimum);
		}
		return FMath::Clamp<float>(InValue, InMinimum, InMaximum);
	};

	auto CollidePlane = [Clamp, &Local, Axis, Minimum, &Maximum]
	{
		switch (Axis)
		{
		case EAxis::X:
			{
				Local.X = Clamp((float)Local.X, Minimum, Maximum);
				break;
			}
		case EAxis::Y:
			{
				Local.Y = Clamp((float)Local.Y, Minimum, Maximum);
				break;
			}
		default:
			{
				Local.Z = Clamp((float)Local.Z, Minimum, Maximum);
				break;
			}
		}
	};

	auto CollideCylinder = [Clamp, &Local, Axis, Minimum, &Maximum]
	{
		switch (Axis)
		{
			case EAxis::X:
			{
				FVector OnPlane = Local * FVector(0.f, 1.f, 1.f);
				if (!OnPlane.IsNearlyZero())
				{
					const float Length = (float)OnPlane.Size();
					OnPlane = OnPlane * Clamp(Length, Minimum, Maximum) / Length;
					Local.Y = OnPlane.Y;
					Local.Z = OnPlane.Z;
				}
				break;
			}
			case EAxis::Y:
			{
				FVector OnPlane = Local * FVector(1.f, 0.f, 1.f);
				if (!OnPlane.IsNearlyZero())
				{
					const float Length = (float)OnPlane.Size();
					OnPlane = OnPlane * Clamp(Length, Minimum, Maximum) / Length;
					Local.X = OnPlane.X;
					Local.Z = OnPlane.Z;
				}
				break;
			}
			default:
			{
				FVector OnPlane = Local * FVector(1.f, 1.f, 0.f);
				if (!OnPlane.IsNearlyZero())
				{
					const float Length = (float)OnPlane.Size();
					OnPlane = OnPlane * Clamp(Length, Minimum, Maximum) / Length;
					Local.X = OnPlane.X;
					Local.Y = OnPlane.Y;
				}
				break;
			}
		}
	};

	auto CollideSphere = [Clamp, &Local, Minimum, &Maximum]
	{
		if (!Local.IsNearlyZero())
		{
			const float Length = (float)Local.Size();
			Local = Local * Clamp(Length, Minimum, Maximum) / Length;
		}
	};
	
	switch (Type)
	{
		case ERigVMClampSpatialMode::Plane:
		{
			CollidePlane();
			break;
		}
		case ERigVMClampSpatialMode::Capsule:
		{
			const float Radius = Minimum;
			if(Maximum < Radius * 2.f)
			{
				Maximum = 0.f;
				CollideSphere();
				break;
			}
				
			const float Length = FMath::Max(Maximum, Radius * 2);
			const float HalfLength = Length * 0.5f;
			const float HalfCylinderLength = HalfLength - Radius;
				
			FVector Normal = FVector::XAxisVector;
			switch (Axis)
			{
				case EAxis::X:
				{
					break;
				}
				case EAxis::Y:
				{
					Normal = FVector::YAxisVector;
					break;
				}
				default:
				{
					Normal = FVector::ZAxisVector;
					break;
				}
			}

			const double DotOnNormal = Normal.Dot(Local);

			// if we are on the cylinder part of the collision, fall through 
			if(FMath::Abs(DotOnNormal) < HalfCylinderLength)
			{
				Maximum = 0.f;
				CollideCylinder();
				break;
			}

			// since now we are going to be colliding with either the upper of lower half sphere
			const FVector LocalSphereCenter = Normal * FMath::Sign(DotOnNormal) * HalfCylinderLength;
			Space = FTransform(LocalSphereCenter) * Space;
			Local = Space.InverseTransformPosition(Value);
			CollideSphere();
			break;
		}
		case ERigVMClampSpatialMode::Cylinder:
		{
			CollideCylinder();
			break;
		}
		default:
		case ERigVMClampSpatialMode::Sphere:
		{
			CollideSphere();
			break;
		}

	}

	return Space.TransformPosition(Local);
}

FQuat FRigVMMathLibrary::FindQuatBetweenVectors(const FVector& A, const FVector& B)
{
	return FindQuatBetweenNormals(A.GetSafeNormal(), B.GetSafeNormal());
}

FQuat FRigVMMathLibrary::FindQuatBetweenNormals(const FVector& A, const FVector& B)
{
	const FQuat::FReal Dot = FVector::DotProduct(A, B);
	FQuat::FReal W = 1 + Dot;
	FQuat Result;

	if (W < SMALL_NUMBER)
	{
		// A and B point in opposite directions
		W = 2 - W;
		Result = FQuat(
			-A.Y * B.Z + A.Z * B.Y,
			-A.Z * B.X + A.X * B.Z,
			-A.X * B.Y + A.Y * B.X,
			W).GetNormalized();

		const FVector Normal = FMath::Abs(A.X) > FMath::Abs(A.Y) ?
			FVector(0.f, 1.f, 0.f) : FVector(1.f, 0.f, 0.f);
		const FVector BiNormal = FVector::CrossProduct(A, Normal);
		const FVector TauNormal = FVector::CrossProduct(A, BiNormal);
		Result = Result * FQuat(TauNormal, PI);
	}
	else
	{
		//Axis = FVector::CrossProduct(A, B);
		Result = FQuat(
			A.Y * B.Z - A.Z * B.Y,
			A.Z * B.X - A.X * B.Z,
			A.X * B.Y - A.Y * B.X,
			W);
	}

	Result.Normalize();
	return Result;
}

FVector FRigVMMathLibrary::GetEquivalentEulerAngle(const FVector& InEulerAngle, const EEulerRotationOrder& InOrder)
{
	FVector TheOtherAngle;

	switch (InOrder)
	{
	case EEulerRotationOrder::XYZ:
		TheOtherAngle.X = InEulerAngle.X + 180;
		TheOtherAngle.Y = 180 - InEulerAngle.Y;
		TheOtherAngle.Z = InEulerAngle.Z + 180;
		break;
	case EEulerRotationOrder::XZY:
		TheOtherAngle.X = InEulerAngle.X + 180;
		TheOtherAngle.Z = 180 - InEulerAngle.Z;
		TheOtherAngle.Y = InEulerAngle.Y + 180;
		break;
	case EEulerRotationOrder::YXZ:
		TheOtherAngle.Y = InEulerAngle.Y + 180;
		TheOtherAngle.X = 180 - InEulerAngle.X;
		TheOtherAngle.Z = InEulerAngle.Z + 180;
		break;
	case EEulerRotationOrder::YZX:
		TheOtherAngle.Y = InEulerAngle.Y + 180;
		TheOtherAngle.Z = 180 - InEulerAngle.Z;
		TheOtherAngle.X = InEulerAngle.X + 180;
		break;
	case EEulerRotationOrder::ZXY:
		TheOtherAngle.Z = InEulerAngle.Z + 180;
		TheOtherAngle.X = 180 - InEulerAngle.X;
		TheOtherAngle.Y = InEulerAngle.Y + 180;
		break;
	case EEulerRotationOrder::ZYX:
		TheOtherAngle.Z = InEulerAngle.Z + 180;
		TheOtherAngle.Y = 180 - InEulerAngle.Y;
		TheOtherAngle.X = InEulerAngle.X + 180;
		break;
	};

	TheOtherAngle.UnwindEuler();
	
	return TheOtherAngle;
}

FVector& FRigVMMathLibrary::ChooseBetterEulerAngleForAxisFilter(const FVector& Base, FVector& A, FVector& B)
{
	// simply compare the sum of difference should be enough
	const double Diff1 = (A - Base).GetAbs().Dot(FVector::OneVector);
	const double Diff2 = (B - Base).GetAbs().Dot(FVector::OneVector);

	return Diff1 < Diff2 ? A : B;
}

void FRigVMMathLibrary::FourPointBezier(const FRigVMFourPointBezier& Bezier, float T, FVector& OutPosition, FVector& OutTangent)
{
	FourPointBezier(Bezier.A, Bezier.B, Bezier.C, Bezier.D, T, OutPosition, OutTangent);
}

void FRigVMMathLibrary::FourPointBezier(const FVector& A, const FVector& B, const FVector& C, const FVector& D, float T, FVector& OutPosition, FVector& OutTangent)
{
	const FVector AB = FMath::Lerp<FVector>(A, B, T);
	const FVector BC = FMath::Lerp<FVector>(B, C, T);
	const FVector CD = FMath::Lerp<FVector>(C, D, T);
	const FVector ABBC = FMath::Lerp<FVector>(AB, BC, T);
	const FVector BCCD = FMath::Lerp<FVector>(BC, CD, T);
	OutPosition = FMath::Lerp<FVector>(ABBC, BCCD, T);
	OutTangent = (BCCD - ABBC).GetSafeNormal();
}
