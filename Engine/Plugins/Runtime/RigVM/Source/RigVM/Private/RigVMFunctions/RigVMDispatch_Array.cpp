// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMStringUtils.h"
#include "Math/GuardedInt.h"
#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Array)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Array"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
///
#if WITH_EDITOR

UScriptStruct* FRigVMDispatch_ArrayBase::GetFactoryDispatchForOpCode(ERigVMOpCode InOpCode)
{
	switch(InOpCode)
	{
		case ERigVMOpCode::ArrayReset:
		{
			return FRigVMDispatch_ArrayReset::StaticStruct();
		}
		case ERigVMOpCode::ArrayGetNum:
		{
			return FRigVMDispatch_ArrayGetNum::StaticStruct();
		} 
		case ERigVMOpCode::ArraySetNum:
		{
			return FRigVMDispatch_ArraySetNum::StaticStruct();
		}
		case ERigVMOpCode::ArrayGetAtIndex:
		{
			return FRigVMDispatch_ArrayGetAtIndex::StaticStruct();
		}  
		case ERigVMOpCode::ArraySetAtIndex:
		{
			return FRigVMDispatch_ArraySetAtIndex::StaticStruct();
		}
		case ERigVMOpCode::ArrayAdd:
		{
			return FRigVMDispatch_ArrayAdd::StaticStruct();
		}
		case ERigVMOpCode::ArrayInsert:
		{
			return FRigVMDispatch_ArrayInsert::StaticStruct();
		}
		case ERigVMOpCode::ArrayRemove:
		{
			return FRigVMDispatch_ArrayRemove::StaticStruct();
		}
		case ERigVMOpCode::ArrayFind:
		{
			return FRigVMDispatch_ArrayFind::StaticStruct();
		}
		case ERigVMOpCode::ArrayAppend:
		{
			return FRigVMDispatch_ArrayAppend::StaticStruct();
		}
		case ERigVMOpCode::ArrayClone:
		{
			return FRigVMDispatch_ArrayClone::StaticStruct();
		}
		case ERigVMOpCode::ArrayIterator:
		{
			return FRigVMDispatch_ArrayIterator::StaticStruct();
		}
		case ERigVMOpCode::ArrayUnion:
		{
			return FRigVMDispatch_ArrayUnion::StaticStruct();
		}
		case ERigVMOpCode::ArrayDifference:
		{
			return FRigVMDispatch_ArrayDifference::StaticStruct();
		}
		case ERigVMOpCode::ArrayIntersection:
		{
			return FRigVMDispatch_ArrayIntersection::StaticStruct();
		}
		case ERigVMOpCode::ArrayReverse:
		{
			return FRigVMDispatch_ArrayReverse::StaticStruct();
		}
		default:
		{
			break;
		}
	}
	return nullptr;
}

FName FRigVMDispatch_ArrayBase::GetFactoryNameForOpCode(ERigVMOpCode InOpCode)
{
	if(const UScriptStruct* FactoryStruct = GetFactoryDispatchForOpCode(InOpCode))
	{
		return *(DispatchPrefix + FactoryStruct->GetName());
	}
	return NAME_None;
}

FString FRigVMDispatch_ArrayBase::GetArgumentDefaultValue(const FName& InArgumentName,
                                                          TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == NumName || 
		InArgumentName == IndexName || 
		InArgumentName == CountName)
	{
		static const FString ZeroValue = TEXT("0");
		return ZeroValue;
	}

	return FRigVMDispatch_CoreBase::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

#endif

FRigVMTemplateArgumentInfo FRigVMDispatch_ArrayBase::CreateArgumentInfo(const FName& InName, ERigVMPinDirection InDirection)
{
	static const TArray<FRigVMTemplateArgument::ETypeCategory> ArrayCategories = {
		FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
	};

	static const TArray<FRigVMTemplateArgument::ETypeCategory> ElementCategories = {
		FRigVMTemplateArgument::ETypeCategory_SingleAnyValue
	};

	if(InName == ArrayName || InName == ValuesName || InName == OtherName || InName == CloneName || InName == ResultName)
	{
		return FRigVMTemplateArgumentInfo(InName, InDirection, ArrayCategories);
	}
	if(InName == ElementName)
	{
		return FRigVMTemplateArgumentInfo(InName, InDirection, ElementCategories);
	}
	if(InName == NumName || InName == IndexName || InName == CountName)
	{
		return FRigVMTemplateArgumentInfo(InName, InDirection, RigVMTypeUtils::TypeIndex::Int32);
	}
	if(InName == SuccessName || InName == IsEmptyName)
	{
		return FRigVMTemplateArgumentInfo(InName, InDirection, RigVMTypeUtils::TypeIndex::Bool);
	}
	if(InName == RatioName)
	{
		return FRigVMTemplateArgumentInfo(InName, InDirection, RigVMTypeUtils::TypeIndex::Float);
	}

	checkNoEntry();
	return FRigVMTemplateArgumentInfo(NAME_None, ERigVMPinDirection::Invalid, INDEX_NONE);
}

TMap<uint32, int32> FRigVMDispatch_ArrayBase::GetArrayHash(FScriptArrayHelper& InArrayHelper, const FArrayProperty* InArrayProperty)
{
	TMap<uint32, int32> Hash;
	Hash.Reserve(InArrayHelper.Num());

	const FProperty* ElementProperty = InArrayProperty->Inner;
	
	for(int32 Index = 0; Index < InArrayHelper.Num(); Index++)
	{
		uint32 HashValue;
		if (ElementProperty->PropertyFlags & CPF_HasGetValueTypeHash)
		{
			HashValue = ElementProperty->GetValueTypeHash(InArrayHelper.GetRawPtr(Index));
		}
		else
		{
			FString Value;
			ElementProperty->ExportTextItem_Direct(Value, InArrayHelper.GetRawPtr(Index), nullptr, nullptr, PPF_None);
			HashValue = TextKeyUtil::HashString(Value);
		}
					
		if(!Hash.Contains(HashValue))
		{
			Hash.Add(HashValue, Index);
		}
	}

	return Hash;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMExecuteArgument>& FRigVMDispatch_ArrayBaseMutable::GetExecuteArguments_Impl(
	const FRigVMDispatchContext& InContext) const
{
	static const TArray<FRigVMExecuteArgument> Arguments = {
			{FRigVMStruct::ExecuteContextName, ERigVMPinDirection::IO}
	};
	return Arguments;
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayMake::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ValuesName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::Output));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayMake::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ValuesName || InArgumentName == ArrayName);
	return {
		{ValuesName, InTypeIndex},
		{ArrayName, InTypeIndex}
	};
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayMake::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayMakeToolTip", "Creates a new array from its elements.");
}

FText FRigVMDispatch_ArrayMake::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ValuesName)
	{
		return LOCTEXT("ArrayMake_ValuesArgumentToolTip", "The elements of the array to create.");
	}
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayMake_ArrayArgumentToolTip", "The resulting array.");
	}
	return FText();
}

FString FRigVMDispatch_ArrayMake::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == ValuesName && InMetaDataKey == FRigVMStruct::FixedSizeArrayMetaName)
	{
		return TrueString;
	}
	return FRigVMDispatch_ArrayBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FString FRigVMDispatch_ArrayMake::GetArgumentDefaultValue(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ValuesName)
	{
		const TRigVMTypeIndex ElementTypeIndex = FRigVMRegistry::Get().GetBaseTypeFromArrayTypeIndex(InTypeIndex);
		FString ElementDefaultValue = FRigVMDispatch_ArrayBase::GetArgumentDefaultValue(InArgumentName, ElementTypeIndex);
		if(ElementDefaultValue.IsEmpty())
		{
			static const FString EmptyBracesString = TEXT("()");
			ElementDefaultValue = EmptyBracesString;
		}

		// default to one element
		return RigVMStringUtils::JoinDefaultValue({ElementDefaultValue});
	}

	return FRigVMDispatch_ArrayBase::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

#endif

FName FRigVMDispatch_ArrayMake::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	if(InOperandIndex < InTotalOperands - 1)
	{
		return FRigVMBranchInfo::GetFixedArrayLabel(ValuesName, *FString::FromInt(InOperandIndex - 1));
	}
	check(InOperandIndex == (InTotalOperands - 1));
	return ArrayName;
}

void FRigVMDispatch_ArrayMake::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles.Last().GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles.Last().GetOutputData());

	const int32 Num = Handles.Num() - 1;
	if(InContext.IsValidArraySize(Num))
	{
		ArrayHelper.Resize(Num);

		for(int32 Index = 0; Index < Num; Index++)
		{
			const uint8* SourceMemory = Handles[Index].GetInputData();
			uint8* TargetMemory = ArrayHelper.GetRawPtr(Index);
			URigVMMemoryStorage::CopyProperty(ArrayProperty->Inner, TargetMemory, Handles[Index].GetResolvedProperty(), SourceMemory);
		}
	}
	else
	{
		ArrayHelper.Resize(0);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayReset::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	check(InTotalOperands == 1);
	return ArrayName;
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayReset::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayReset::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);
	return {
		{ArrayName, InTypeIndex}
	};
}

void FRigVMDispatch_ArrayReset::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	ArrayHelper.Resize(0);
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayReset::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayResetToolTip", "Removes all elements from an array.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayReset::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayReset_ArrayArgumentToolTip", "The array to be cleared.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayGetNum::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		NumName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayGetNum::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(NumName, ERigVMPinDirection::Output) );
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayGetNum::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);
	return {
		{ArrayName, InTypeIndex},
		{NumName, RigVMTypeUtils::TypeIndex::Int32}
	};
}

void FRigVMDispatch_ArrayGetNum::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	const FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	int32* Num = (int32*)Handles[1].GetOutputData();
	*Num = ArrayHelper.Num();
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayGetNum::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
    return LOCTEXT("ArrayGetNumToolTip", "Returns the number of elements of an array");
}

FText FRigVMDispatch_ArrayGetNum::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayGetNum_ArrayArgumentToolTip", "The array to retrieve the size for.");
	}
	if(InArgumentName == NumName)
	{
		return LOCTEXT("ArrayGetNum_NumArgumentToolTip", "The size of the input array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArraySetNum::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		NumName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArraySetNum::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(NumName, ERigVMPinDirection::Input));
	};
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArraySetNum::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);
	return {
		{ArrayName, InTypeIndex},
		{NumName, RigVMTypeUtils::TypeIndex::Int32}
	};
}

void FRigVMDispatch_ArraySetNum::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	const int32* Num = (const int32*)Handles[1].GetInputData();
	if(InContext.IsValidArraySize(*Num))
	{
		ArrayHelper.Resize(*Num);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArraySetNum::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArraySetNumToolTip", "Sets the numbers of elements of an array.\nModifies the input array.");
}

FText FRigVMDispatch_ArraySetNum::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArraySetNum_ArrayArgumentToolTip", "The array to set the size for.");
	}
	if(InArgumentName == NumName)
	{
		return LOCTEXT("ArraySetNum_NumArgumentToolTip", "The new size of the array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayIsEmpty::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		IsEmptyName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayIsEmpty::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(IsEmptyName, ERigVMPinDirection::Output) );
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayIsEmpty::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);
	return {
		{ArrayName, InTypeIndex},
		{IsEmptyName, RigVMTypeUtils::TypeIndex::Bool}
	};
}

void FRigVMDispatch_ArrayIsEmpty::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	const FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	bool* IsEmpty = (bool*)Handles[1].GetOutputData();
	*IsEmpty = ArrayHelper.Num() == 0;
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayIsEmpty::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
    return LOCTEXT("ArrayIsEmptyToolTip", "Returns true if the array is empty");
}

FText FRigVMDispatch_ArrayIsEmpty::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayIsEmpty_ArrayArgumentToolTip", "The array to check for emptiness.");
	}
	if(InArgumentName == IsEmptyName)
	{
		return LOCTEXT("ArrayIsEmpty_NumArgumentToolTip", "True if the array is empty.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayInit::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		NumName,
		ElementName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayInit::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(NumName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ElementName, ERigVMPinDirection::Input));
	};
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayInit::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);
	return {
		{ArrayName, InTypeIndex},
		{NumName, RigVMTypeUtils::TypeIndex::Int32},
		{ElementName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex)}
	};
}

void FRigVMDispatch_ArrayInit::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	const int32* Num = (const int32*)Handles[1].GetInputData();
	const FProperty* ValueProperty = Handles[2].GetResolvedProperty();
	const uint8* ValueData = Handles[2].GetInputData();

	if(InContext.IsValidArraySize(*Num))
	{
		ArrayHelper.EmptyAndAddValues(*Num);
		for (int32 Index=0; Index<ArrayHelper.Num(); ++Index)
		{
			URigVMMemoryStorage::CopyProperty(ArrayProperty->Inner, ArrayHelper.GetElementPtr(Index), ValueProperty, ValueData);
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayInit::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayInitToolTip", "Sets the size of the array, initializing all elements to the given value.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayInit::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayInit_ArrayArgumentToolTip", "The array to initialize.");
	}
	if(InArgumentName == NumName)
	{
		return LOCTEXT("ArrayInit_NumArgumentToolTip", "The new size of the array.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayInit_ElementArgumentToolTip", "The value that is set to all elements.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayGetAtIndex::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		IndexName,
		ElementName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayGetAtIndex::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(IndexName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ElementName, ERigVMPinDirection::Output) );
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayGetAtIndex::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == ElementName);

	if(InArgumentName == ArrayName)
	{
		return {
			{ArrayName, InTypeIndex},
			{IndexName, RigVMTypeUtils::TypeIndex::Int32},
			{ElementName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex)}
		};
	}

	return {
		{ArrayName, InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex)},
		{IndexName, RigVMTypeUtils::TypeIndex::Int32},
		{ElementName, InTypeIndex}
	};
}

void FRigVMDispatch_ArrayGetAtIndex::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	const int32 Index = *(const int32*)Handles[1].GetInputData();
	int32 OutIndex = Index;
	uint8* TargetMemory = Handles[2].GetOutputData();

	if(InContext.IsValidArrayIndex(OutIndex, ArrayHelper))
	{
		const uint8* SourceMemory = ArrayHelper.GetRawPtr(OutIndex);
		URigVMMemoryStorage::CopyProperty(Handles[2].GetProperty(), TargetMemory, ArrayProperty->Inner, SourceMemory);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayGetAtIndex::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayGetAtIndexToolTip", "Returns an element of an array by index.");
}

FText FRigVMDispatch_ArrayGetAtIndex::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayGetAtIndex_ArrayArgumentToolTip", "The array to retrieve an element from.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayGetAtIndex_IndexArgumentToolTip", "The index of the element to retrieve.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayGetAtIndex_ElementArgumentToolTip", "The element at the given index.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArraySetAtIndex::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		IndexName,
		ElementName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArraySetAtIndex::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(IndexName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ElementName, ERigVMPinDirection::Input) );
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArraySetAtIndex::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == ElementName);

	if(InArgumentName == ArrayName)
	{
		return {
			{ArrayName, InTypeIndex},
			{IndexName, RigVMTypeUtils::TypeIndex::Int32},
			{ElementName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex)}
		};
	}

	return {
		{ArrayName, InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex)},
		{IndexName, RigVMTypeUtils::TypeIndex::Int32},
		{ElementName, InTypeIndex}
	};
}

void FRigVMDispatch_ArraySetAtIndex::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	const int32 Index = *(const int32*)Handles[1].GetInputData();
	int32 OutIndex = Index;
	const uint8* SourceMemory = Handles[2].GetInputData();

	if(InContext.IsValidArrayIndex(OutIndex, ArrayHelper))
	{
		uint8* TargetMemory = ArrayHelper.GetRawPtr(OutIndex);
		URigVMMemoryStorage::CopyProperty(ArrayProperty->Inner, TargetMemory, Handles[2].GetResolvedProperty(), SourceMemory);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArraySetAtIndex::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArraySetAtIndexToolTip", "Sets an element of an array by index.\nModifies the input array.");
}

FText FRigVMDispatch_ArraySetAtIndex::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArraySetAtIndex_ArrayArgumentToolTip", "The array to set an element for.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArraySetAtIndex_IndexArgumentToolTip", "The index of the element to set.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArraySetAtIndex_ElementArgumentToolTip", "The new value for element to set.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayAdd::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		ElementName,
		IndexName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayAdd::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ElementName, ERigVMPinDirection::Input) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(IndexName, ERigVMPinDirection::Output) );
	}
	return CachedArgumentInfos;
}

void FRigVMDispatch_ArrayAdd::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	const uint8* SourceMemory = Handles[1].GetInputData();
	int32& Index = *(int32*)Handles[2].GetOutputData();

	if(InContext.IsValidArraySize(ArrayHelper.Num() + 1))
	{
		Index = ArrayHelper.AddValue();
		uint8* TargetMemory = ArrayHelper.GetRawPtr(Index);
		URigVMMemoryStorage::CopyProperty(ArrayProperty->Inner, TargetMemory, Handles[1].GetResolvedProperty(), SourceMemory);
	}
	else
	{
		Index = INDEX_NONE;
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayAdd::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayAddToolTip", "Adds an element to an array and returns the new element's index.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayAdd::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayAdd_ArrayArgumentToolTip", "The array to add an element to.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayAdd_ElementArgumentToolTip", "The element to add to the array.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayAdd_IndexArgumentToolTip", "The index of the newly added element.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMDispatch_ArrayInsert::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	int32 Index = *(int32*)Handles[1].GetOutputData();
	const uint8* SourceMemory = Handles[2].GetInputData();

	if(InContext.IsValidArraySize(ArrayHelper.Num() + 1))
	{
		// we support wrapping the index around similar to python
		if(Index < 0)
		{
			Index = ArrayHelper.Num() + Index;
		}
		Index = FMath::Clamp<int32>(Index, 0, ArrayHelper.Num());
		
		ArrayHelper.InsertValues(Index, 1);

		uint8* TargetMemory = ArrayHelper.GetRawPtr(Index);
		URigVMMemoryStorage::CopyProperty(ArrayProperty->Inner, TargetMemory, Handles[2].GetResolvedProperty(), SourceMemory);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayInsert::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayInsertToolTip", "Inserts an element into an array at a given index.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayInsert::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayInsert_ArrayArgumentToolTip", "The array to insert an element into.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayInsert_IndexArgumentToolTip", "The index at which to insert the element.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayInsert_ElementArgumentToolTip", "The element to insert into the array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayRemove::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		IndexName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayRemove::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace( CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO) );
		CachedArgumentInfos.Emplace( CreateArgumentInfo(IndexName, ERigVMPinDirection::Input) );
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayRemove::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName);

	return {
		{ArrayName, InTypeIndex},
		{IndexName, RigVMTypeUtils::TypeIndex::Int32}
	};
}

void FRigVMDispatch_ArrayRemove::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	const int32 Index = *(const int32*)Handles[1].GetInputData();
	int32 OutIndex = Index;

	if(InContext.IsValidArrayIndex(OutIndex, ArrayHelper))
	{
		ArrayHelper.RemoveValues(OutIndex, 1);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayRemove::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayRemoveToolTip", "Removes an element from an array by index.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayRemove::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayRemove_ArrayArgumentToolTip", "The array to remove an element from.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayRemove_IndexArgumentToolTip", "The index at which to remove the element.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMDispatch_ArrayReverse::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());
	for(int32 A=0, B=ArrayHelper.Num()-1; A<B; A++, B--)
	{
		ArrayHelper.SwapValues(A, B);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayReverse::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayReverseToolTip", "Reverses the order of the elements of an array.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayReverse::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayReverse_ArrayArgumentToolTip", "The array to reverse.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayFind::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		ElementName,
		IndexName,
		SuccessName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayFind::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ElementName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(IndexName, ERigVMPinDirection::Output));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(SuccessName, ERigVMPinDirection::Output));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayFind::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == ElementName);

	if(InArgumentName == ArrayName)
	{
		return {
			{ArrayName, InTypeIndex},
			{ElementName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex)},
			{IndexName, RigVMTypeUtils::TypeIndex::Int32},
			{SuccessName, RigVMTypeUtils::TypeIndex::Bool}
		};
	}

	return {
		{ArrayName, InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex)},
		{ElementName, InTypeIndex},
		{IndexName, RigVMTypeUtils::TypeIndex::Int32},
		{SuccessName, RigVMTypeUtils::TypeIndex::Bool}
	};
}

void FRigVMDispatch_ArrayFind::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	int32& Index = *(int32*)Handles[2].GetOutputData();
	bool& bSuccess = *(bool*)Handles[3].GetOutputData();

	Index = INDEX_NONE;
	bSuccess = false;

	const FProperty* PropertyA = Handles[1].GetResolvedProperty();
	const FProperty* PropertyB = ArrayProperty->Inner;

	if(PropertyA->SameType(PropertyB))
	{
		const uint8* MemoryA = Handles[1].GetInputData();

		for(int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
		{
			const uint8* MemoryB = ArrayHelper.GetRawPtr(ElementIndex);
			if(PropertyA->Identical(MemoryA, MemoryB))
			{
				Index = ElementIndex;
				bSuccess = true;
				break;
			}
		}
	}
	else
	{
		static constexpr TCHAR IncompatibleTypes[] = TEXT("Array('%s') doesn't support searching for element('%s').");
		InContext.GetPublicData<>().Logf(EMessageSeverity::Error, IncompatibleTypes, *PropertyB->GetCPPType(), *PropertyA->GetCPPType());
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayFind::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayFindToolTip", "Searches a potential element in an array and returns its index.");
}

FText FRigVMDispatch_ArrayFind::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayFind_ArrayArgumentToolTip", "The array to search within.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayFind_ElementArgumentToolTip", "The element to look for.");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayFind_IndexArgumentToolTip", "The index of the found element (or -1).");
	}
	if(InArgumentName == SuccessName)
	{
		return LOCTEXT("ArrayFind_SuccessArgumentToolTip", "True if the element has been found.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayAppend::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		OtherName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayAppend::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::IO));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(OtherName, ERigVMPinDirection::Input));
	}
	return CachedArgumentInfos;
}


FRigVMTemplateTypeMap FRigVMDispatch_ArrayAppend::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == OtherName);

	return {
		{ArrayName, InTypeIndex},
		{OtherName, InTypeIndex}
	};
}

void FRigVMDispatch_ArrayAppend::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* SourceArrayProperty = CastFieldChecked<FArrayProperty>(Handles[1].GetResolvedProperty());
	FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, Handles[1].GetInputData());

	if(SourceArrayHelper.Num() > 0)
	{
		const FArrayProperty* TargetArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
		FScriptArrayHelper TargetArrayHelper(TargetArrayProperty, Handles[0].GetOutputData());

		if(InContext.IsValidArraySize(TargetArrayHelper.Num() + SourceArrayHelper.Num()))
		{
			const FProperty* TargetProperty = TargetArrayProperty->Inner;
			const FProperty* SourceProperty = SourceArrayProperty->Inner;

			const int32 SourceItemCount = SourceArrayHelper.Num();
			int32 TargetIndex = TargetArrayHelper.AddValues(SourceItemCount);
			for(int32 SourceIndex = 0; SourceIndex < SourceItemCount; SourceIndex++, TargetIndex++)
			{
				uint8* TargetMemory = TargetArrayHelper.GetRawPtr(TargetIndex);
				const uint8* SourceMemory = SourceArrayHelper.GetRawPtr(SourceIndex);
				URigVMMemoryStorage::CopyProperty(TargetProperty, TargetMemory, SourceProperty, SourceMemory);
			}
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayAppend::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayAppendToolTip", "Appends another array to the main one.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayAppend::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayAppend_ArrayArgumentToolTip", "The array to append the other array to.");
	}
	if(InArgumentName == OtherName)
	{
		return LOCTEXT("ArrayAppend_OtherArgumentToolTip", "The second array to append to the first one.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayClone::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		CloneName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayClone::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(CloneName, ERigVMPinDirection::Output));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayClone::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == CloneName);

	return {
		{ArrayName, InTypeIndex},
		{CloneName, InTypeIndex}
	};
}

void FRigVMDispatch_ArrayClone::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	const FArrayProperty* CloneProperty = CastFieldChecked<FArrayProperty>(Handles[1].GetProperty());
	FScriptArrayHelper CloneHelper(CloneProperty, Handles[1].GetOutputData());

	CloneHelper.Resize(ArrayHelper.Num());
	if(ArrayHelper.Num() > 0)
	{
		const FProperty* TargetProperty = CloneProperty->Inner;
		const FProperty* SourceProperty = ArrayProperty->Inner;
		for(int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
		{
			uint8* TargetMemory = CloneHelper.GetRawPtr(ElementIndex);
			const uint8* SourceMemory = ArrayHelper.GetRawPtr(ElementIndex);
			URigVMMemoryStorage::CopyProperty(TargetProperty, TargetMemory, SourceProperty, SourceMemory);
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayClone::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayCloneToolTip", "Clones an array and returns a duplicate.");
}

FText FRigVMDispatch_ArrayClone::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayClone_ArrayArgumentToolTip", "The array to clone");
	}
	if(InArgumentName == CloneName)
	{
		return LOCTEXT("ArrayClone_CloneArgumentToolTip", "The duplicate of the input array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMDispatch_ArrayUnion::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* OtherProperty = CastFieldChecked<FArrayProperty>(Handles[1].GetResolvedProperty());
	FScriptArrayHelper OtherHelper(OtherProperty, Handles[1].GetInputData());

	if(OtherHelper.Num() > 0)
	{
		const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetProperty());
		FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetOutputData());

		const FProperty* ArrayElementProperty = ArrayProperty->Inner;
		const FProperty* OtherElementProperty = OtherProperty->Inner;

		const TMap<uint32, int32> HashA = GetArrayHash(ArrayHelper, ArrayProperty);
		const TMap<uint32, int32> HashB = GetArrayHash(OtherHelper, OtherProperty);

		int32 FinalCount = HashA.Num();
		TArray<int32> OtherIndicesToAdd;
		OtherIndicesToAdd.Reserve(HashB.Num());
		for(const TPair<uint32, int32>& Pair : HashB)
		{
			if(!HashA.Contains(Pair.Key))
			{
				FinalCount++;
				if (!InContext.IsValidArraySize(FinalCount))
				{
					break;
				}
				OtherIndicesToAdd.Add(Pair.Value);
			}
		}

		// Check if we get a 32-bit overflow due to malformed GetSize() result and bail early if so.
		const FGuardedInt32 TempStorageSize = FGuardedInt32(ArrayHelper.Num()) * ArrayElementProperty->GetSize();
		
		if (InContext.IsValidArraySize(FinalCount) && TempStorageSize.IsValid())
		{
			// copy the complete array to a temp storage
			TArray<uint8, TAlignedHeapAllocator<16>> TempStorage;
			const int32 NumElementsA = ArrayHelper.Num();
			TempStorage.AddZeroed(TempStorageSize.GetChecked());
			uint8* TempMemory = TempStorage.GetData();
			for(int32 Index = 0; Index < NumElementsA; Index++)
			{
				ArrayElementProperty->InitializeValue(TempMemory);
				ArrayElementProperty->CopyCompleteValue(TempMemory, ArrayHelper.GetRawPtr(Index));
				TempMemory += ArrayElementProperty->GetSize();
			}

			ArrayHelper.Resize(0);

			for(const TPair<uint32, int32>& Pair : HashA)
			{
				const int32 AddedIndex = ArrayHelper.AddValue();
				TempMemory = TempStorage.GetData() + Pair.Value * ArrayElementProperty->GetSize();
					
				URigVMMemoryStorage::CopyProperty(
					ArrayElementProperty,
					ArrayHelper.GetRawPtr(AddedIndex),
					ArrayElementProperty,
					TempMemory
				);
			}

			TempMemory = TempStorage.GetData();
			for(int32 Index = 0; Index < NumElementsA; Index++)
			{
				ArrayElementProperty->DestroyValue(TempMemory);
				TempMemory += ArrayElementProperty->GetSize();
			}

			for (const int32& OtherIndex : OtherIndicesToAdd)
			{
				const int32 AddedIndex = ArrayHelper.AddValue();
				URigVMMemoryStorage::CopyProperty(
					ArrayElementProperty,
					ArrayHelper.GetRawPtr(AddedIndex),
					OtherElementProperty,
					OtherHelper.GetRawPtr(OtherIndex)
				);
			}
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayUnion::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayUnionToolTip", "Merges two arrays while ensuring unique elements.\nModifies the input array.");
}

FText FRigVMDispatch_ArrayUnion::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayUnion_ArrayArgumentToolTip", "The array to merge the other array with.");
	}
	if(InArgumentName == OtherName)
	{
		return LOCTEXT("ArrayUnion_OtherArgumentToolTip", "The second array to merge to the first one.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayDifference::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		OtherName,
		ResultName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayDifference::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(OtherName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ResultName, ERigVMPinDirection::Output));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayDifference::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == OtherName || InArgumentName == ResultName);

	return {
		{ArrayName, InTypeIndex},
		{OtherName, InTypeIndex},
		{ResultName, InTypeIndex}
	};
}

void FRigVMDispatch_ArrayDifference::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	const FArrayProperty* OtherProperty = CastFieldChecked<FArrayProperty>(Handles[1].GetResolvedProperty());
	FScriptArrayHelper OtherHelper(OtherProperty, Handles[1].GetInputData());
	const FArrayProperty* ResultProperty = CastFieldChecked<FArrayProperty>(Handles[2].GetProperty());
	FScriptArrayHelper ResultHelper(ResultProperty, Handles[2].GetOutputData());

	const FProperty* ArrayElementProperty = ArrayProperty->Inner;
	const FProperty* OtherElementProperty = OtherProperty->Inner;
	const FProperty* ResultElementProperty = ResultProperty->Inner;

	ResultHelper.Resize(0);

	const TMap<uint32, int32> HashA = GetArrayHash(ArrayHelper, ArrayProperty);
	const TMap<uint32, int32> HashB = GetArrayHash(OtherHelper, OtherProperty);

	for(const TPair<uint32, int32>& Pair : HashA)
	{
		if(!HashB.Contains(Pair.Key))
		{
			const int32 AddedIndex = ResultHelper.AddValue();
			URigVMMemoryStorage::CopyProperty(
				ResultElementProperty,
				ResultHelper.GetRawPtr(AddedIndex),
				ArrayElementProperty,
				ArrayHelper.GetRawPtr(Pair.Value)
			);
		}
	}
	for(const TPair<uint32, int32>& Pair : HashB)
	{
		if(!HashA.Contains(Pair.Key))
		{
			const int32 AddedIndex = ResultHelper.AddValue();
			URigVMMemoryStorage::CopyProperty(
				ResultElementProperty,
				ResultHelper.GetRawPtr(AddedIndex),
				OtherElementProperty,
				OtherHelper.GetRawPtr(Pair.Value)
			);
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayDifference::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayDifferenceToolTip", "Creates a new array containing the difference between the two input arrays.\nDifference here means elements that are only contained in either A or B.");
}

FText FRigVMDispatch_ArrayDifference::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayDifference_ArrayArgumentToolTip", "The first array to compare the other array with.");
	}
	if(InArgumentName == OtherName)
	{
		return LOCTEXT("ArrayDifference_OtherArgumentToolTip", "The second array to compare the other array with.");
	}
	if(InArgumentName == ResultName)
	{
		return LOCTEXT("ArrayDifference_ResultArgumentToolTip", "The resulting difference array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMDispatch_ArrayIntersection::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	const FArrayProperty* OtherProperty = CastFieldChecked<FArrayProperty>(Handles[1].GetResolvedProperty());
	FScriptArrayHelper OtherHelper(OtherProperty, Handles[1].GetInputData());
	const FArrayProperty* ResultProperty = CastFieldChecked<FArrayProperty>(Handles[2].GetProperty());
	FScriptArrayHelper ResultHelper(ResultProperty, Handles[2].GetOutputData());

	const FProperty* ArrayElementProperty = ArrayProperty->Inner;
	const FProperty* ResultElementProperty = ResultProperty->Inner;

	ResultHelper.Resize(0);

	const TMap<uint32, int32> HashA = GetArrayHash(ArrayHelper, ArrayProperty);
	const TMap<uint32, int32> HashB = GetArrayHash(OtherHelper, OtherProperty);

	for(const TPair<uint32, int32>& Pair : HashA)
	{
		if(HashB.Contains(Pair.Key))
		{
			const int32 AddedIndex = ResultHelper.AddValue();
			URigVMMemoryStorage::CopyProperty(
				ResultElementProperty,
				ResultHelper.GetRawPtr(AddedIndex),
				ArrayElementProperty,
				ArrayHelper.GetRawPtr(Pair.Value)
			);
		}
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayIntersection::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayIntersectionToolTip", "Creates a new array containing the intersection between the two input arrays.\nIntersection here means elements that are contained in both A and B.");
}

FText FRigVMDispatch_ArrayIntersection::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayIntersection_ArrayArgumentToolTip", "The first array to compare the other array with.");
	}
	if(InArgumentName == OtherName)
	{
		return LOCTEXT("ArrayIntersection_OtherArgumentToolTip", "The second array to compare the other array with.");
	}
	if(InArgumentName == ResultName)
	{
		return LOCTEXT("ArrayIntersection_ResultArgumentToolTip", "The resulting intersecting array.");
	}
	return FText();
}

#endif

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FName FRigVMDispatch_ArrayIterator::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ArrayName,
		ElementName,
		IndexName,
		CountName,
		RatioName,
		FRigVMStruct::ControlFlowBlockToRunName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_ArrayIterator::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ArrayName, ERigVMPinDirection::Input));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(ElementName, ERigVMPinDirection::Output));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(IndexName, ERigVMPinDirection::Output));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(CountName, ERigVMPinDirection::Output));
		CachedArgumentInfos.Emplace(CreateArgumentInfo(RatioName, ERigVMPinDirection::Output));
		CachedArgumentInfos.Emplace(FRigVMStruct::ControlFlowBlockToRunName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::FName);
	}
	return CachedArgumentInfos;
}

const TArray<FRigVMExecuteArgument>& FRigVMDispatch_ArrayIterator::GetExecuteArguments_Impl(
	const FRigVMDispatchContext& InContext) const
{
	UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(TArray<FRigVMExecuteArgument>, ExecuteArguments);

	if(ExecuteArguments.IsEmpty())
	{
		ExecuteArguments = FRigVMDispatch_ArrayBaseMutable::GetExecuteArguments_Impl(InContext);
		ExecuteArguments.Emplace(FRigVMStruct::ControlFlowCompletedName, ERigVMPinDirection::Output);
	}
	return ExecuteArguments;
}

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMDispatch_ArrayIterator::GetControlFlowBlocks_Impl(
	const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const
{
	static const TArray<FName> Blocks = {FRigVMStruct::ExecuteContextName, FRigVMStruct::ControlFlowCompletedName};
	return Blocks;
}
#endif

const bool FRigVMDispatch_ArrayIterator::IsControlFlowBlockSliced(const FName& InBlockName) const
{
	return InBlockName == FRigVMStruct::ExecuteContextName;
}

FRigVMTemplateTypeMap FRigVMDispatch_ArrayIterator::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	check(InArgumentName == ArrayName || InArgumentName == ElementName);

	if(InArgumentName == ArrayName)
	{
		return {
			{ArrayName, InTypeIndex},
			{ElementName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex)},
			{IndexName, RigVMTypeUtils::TypeIndex::Int32},
			{CountName, RigVMTypeUtils::TypeIndex::Int32},
			{RatioName, RigVMTypeUtils::TypeIndex::Float},
			{FRigVMStruct::ControlFlowBlockToRunName, RigVMTypeUtils::TypeIndex::FName}
		};
	}

	return {
		{ArrayName, InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex)},
		{ElementName, InTypeIndex},
		{IndexName, RigVMTypeUtils::TypeIndex::Int32},
		{CountName, RigVMTypeUtils::TypeIndex::Int32},
		{RatioName, RigVMTypeUtils::TypeIndex::Float},
		{FRigVMStruct::ControlFlowBlockToRunName, RigVMTypeUtils::TypeIndex::FName}
	};}

void FRigVMDispatch_ArrayIterator::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Handles[0].GetResolvedProperty());
	FScriptArrayHelper ArrayHelper(ArrayProperty, Handles[0].GetInputData());
	int32& Index = *(int32*)Handles[2].GetOutputData();
	int32& Count = *(int32*)Handles[3].GetOutputData();
	float& Ratio = *(float*)Handles[4].GetOutputData();
	FName& Block = *(FName*)Handles[5].GetOutputData();

	if(Block.IsNone())
	{
		Block = FRigVMStruct::ExecuteContextName;
		Count = ArrayHelper.Num();
		Index = -1;;
		Ratio = 0.f;
	}
	
	// Do not rely on Count, the array might have changed inside the loop
	const int32 Num = ArrayHelper.Num();
	if (Num != Count)
	{
		static constexpr TCHAR InvalidSizeMessage[] = TEXT("ArrayIterator: The size of the array has changed inside the loop (initially %d, current %d).");
		InContext.GetPublicData<>().Logf(EMessageSeverity::Warning, InvalidSizeMessage, Count, Num);
	}

	Index++;
	if(Num > 1)
	{
		Ratio = float(Index) / float(Num - 1);
	}
	else
	{
		Ratio = 0.f;
	}

	if(Index >= Num)
	{
		Block = FRigVMStruct::ControlFlowCompletedName;
		return;
	}
	
	if (ArrayHelper.IsValidIndex(Index))
	{
		uint8* TargetMemory = Handles[1].GetOutputData();
		const uint8* SourceMemory = ArrayHelper.GetRawPtr(Index);
		URigVMMemoryStorage::CopyProperty(Handles[1].GetProperty(), TargetMemory, ArrayProperty->Inner, SourceMemory);
	}
}

#if WITH_EDITOR

FText FRigVMDispatch_ArrayIterator::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ArrayIteratorToolTip", "Loops over the elements in an array.");
}

FText FRigVMDispatch_ArrayIterator::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == ArrayName)
	{
		return LOCTEXT("ArrayIterator_ArrayArgumentToolTip", "The array to iterate over");
	}
	if(InArgumentName == IndexName)
	{
		return LOCTEXT("ArrayIterator_IndexArgumentToolTip", "The index of the current element.");
	}
	if(InArgumentName == CountName)
	{
		return LOCTEXT("ArrayIterator_CountArgumentToolTip", "The count of all elements in the array.");
	}
	if(InArgumentName == ElementName)
	{
		return LOCTEXT("ArrayIterator_ElementArgumentToolTip", "The current element.");
	}
	if(InArgumentName == RatioName)
	{
		return LOCTEXT("ArrayIterator_RatioArgumentToolTip", "A float ratio from 0.0 (first element) to 1.0 (last element).");
	}
	if(InArgumentName == FRigVMStruct::ControlFlowCompletedName)
	{
		return LOCTEXT("ArrayIterator_CompletedArgumentToolTip", "The execute block to run once the loop has completed.");
	}
	return FText();
}

#endif

#undef LOCTEXT_NAMESPACE
