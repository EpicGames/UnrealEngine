// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_CastEnum.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_CastEnum)

#define LOCTEXT_NAMESPACE "RigVMDispatch_CastEnum"

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CastEnumToInt::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ElementCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleEnumValue
		};
		
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Input, ElementCategories);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Int32);
	}

	return CachedArgumentInfos;
}

bool FRigVMDispatch_CastEnumToInt::GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const
{
	if (InArgumentName == ValueName)
	{
		OutPermutations.Add(
	{
			{ ValueName, InTypeIndex },
			{ ResultName, RigVMTypeUtils::TypeIndex::Int32 }
		});
	}
	return !OutPermutations.IsEmpty();
}

#if WITH_EDITOR

FString FRigVMDispatch_CastEnumToInt::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	return TEXT("Cast to int");
}

FText FRigVMDispatch_CastEnumToInt::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("CastEnumToIntToolTip", "Casts from enum to int");
}

FText FRigVMDispatch_CastEnumToInt::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ValueName)
	{
		return LOCTEXT("CastEnumToIntValueToolTip", "The enum value to cast to an integer");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("CastEnumToIntResultToolTip", "The resulting integer value");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

void FRigVMDispatch_CastEnumToInt::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	const FProperty* ValueProperty = CastFieldChecked<FProperty>(Handles[0].GetResolvedProperty());
	const FProperty* ResultProperty = CastFieldChecked<FProperty>(Handles[1].GetProperty());
	if (!ResultProperty || !ValueProperty)
	{
		return;
	}

	const uint8* ValuePtr = Handles[0].GetInputData();
	int32* ResultPtr = (int32*)Handles[1].GetOutputData();
	if (ValuePtr == nullptr || ResultPtr == nullptr)
	{
		return;
	}
	
	const FNumericProperty* NumericProperty = nullptr;
	if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ValueProperty))
	{
		NumericProperty = EnumProperty->GetUnderlyingProperty();
	}
	else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(ValueProperty))
	{
		NumericProperty = ByteProperty;
	}

	if(NumericProperty == nullptr)
	{
		*ResultPtr = 0;
		checkNoEntry();
		return;
	}
	check(NumericProperty->IsInteger())

	const int64 Value = NumericProperty->GetSignedIntPropertyValue(ValuePtr);
	*ResultPtr = (int32)Value;

#if WITH_EDITOR
	if (*ResultPtr == INDEX_NONE)
	{
		const FRigVMExecuteContext& ExecuteContext = InContext.GetPublicData<>();
		if(ExecuteContext.GetLog() != nullptr)
		{
			ExecuteContext.Report(EMessageSeverity::Error, InContext.GetPublicData<>().GetFunctionName(), InContext.GetPublicData<>().GetInstructionIndex(), TEXT("Enum value invalid"));
		}
	}
#endif
}



const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CastIntToEnum::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ElementCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleEnumValue
		};
		
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Int32);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, ElementCategories);
	}

	return CachedArgumentInfos;
}

bool FRigVMDispatch_CastIntToEnum::GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const
{
	if (InArgumentName == ResultName)
	{
		OutPermutations.Add(
	{
			{ ValueName, RigVMTypeUtils::TypeIndex::Int32 },
			{ ResultName, InTypeIndex }
		});
	}
	return !OutPermutations.IsEmpty();
}

#if WITH_EDITOR

FString FRigVMDispatch_CastIntToEnum::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	return TEXT("Cast to enum");
}

FText FRigVMDispatch_CastIntToEnum::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("CastIntToEnumToolTip", "Casts from int to enum");
}

FText FRigVMDispatch_CastIntToEnum::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ValueName)
	{
		return LOCTEXT("CastIntToEnumValueToolTip", "The integer value to cast to an enum value");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("CastIntToEnumResultToolTip", "The resulting enum value");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

void FRigVMDispatch_CastIntToEnum::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	const FProperty* ValueProperty = CastFieldChecked<FProperty>(Handles[0].GetResolvedProperty());
	const FProperty* ResultProperty = CastFieldChecked<FProperty>(Handles[1].GetProperty());
	if (!ResultProperty || !ValueProperty)
	{
		return;
	}

	const int32* ValuePtr = (int32*)Handles[0].GetInputData();
	uint8* ResultPtr = Handles[1].GetOutputData();
	if (ValuePtr == nullptr || ResultPtr == nullptr)
	{
		return;
	}

	const FNumericProperty* NumericProperty = nullptr;
	if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ResultProperty))
	{
		NumericProperty = EnumProperty->GetUnderlyingProperty();
	}
	else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(ResultProperty))
	{
		NumericProperty = ByteProperty;
	}

	if(NumericProperty == nullptr)
	{
		*ResultPtr = 0;
		checkNoEntry();
		return;
	}
	check(NumericProperty->IsInteger());

	const int64 Value = (int32)*ValuePtr;
	NumericProperty->SetIntPropertyValue(ResultPtr, Value);
	
#if WITH_EDITOR
	if (*ResultPtr == INDEX_NONE)
	{
		const FRigVMExecuteContext& ExecuteContext = InContext.GetPublicData<>();
		if(ExecuteContext.GetLog() != nullptr)
		{
			ExecuteContext.Report(EMessageSeverity::Error, InContext.GetPublicData<>().GetFunctionName(), InContext.GetPublicData<>().GetInstructionIndex(), TEXT("Enum value invalid"));
		}
	}
#endif
}

#undef LOCTEXT_NAMESPACE
