// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_CastObject.h"
#include "RigVMCore/RigVMRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_CastObject)

#define LOCTEXT_NAMESPACE "RigVMDispatch_CastObject"

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CastObject::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ElementCategories =
		{
			FRigVMTemplateArgument::ETypeCategory_SingleObjectValue
		};
		
		TArray<FRigVMTemplateArgumentInfo> Infos;
		Infos.Emplace(ValueName, ERigVMPinDirection::Input, ElementCategories);
		Infos.Emplace(ResultName, ERigVMPinDirection::Output, INDEX_NONE);
		
		CachedArgumentInfos = BuildArgumentListFromPrimaryArgument(Infos, ValueName, InRegistry);
	}

	return CachedArgumentInfos;
}

bool FRigVMDispatch_CastObject::GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const
{
	if (InArgumentName == ValueName)
	{
		const TArray<TRigVMTypeIndex>& ObjectTypes = InRegistry->GetTypesForCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue);
		for (const TRigVMTypeIndex& Type : ObjectTypes)
		{
			OutPermutations.Add(
	{
				{ ValueName, InTypeIndex },
				{ ResultName, Type }
			});
		}
	}
	else if (InArgumentName == ResultName)
	{
		const TArray<TRigVMTypeIndex>& ObjectTypes = InRegistry->GetTypesForCategory_NoLock(FRigVMTemplateArgument::ETypeCategory_SingleObjectValue);
		for (const TRigVMTypeIndex& Type : ObjectTypes)
		{
			OutPermutations.Add(
	{
				{ ValueName, Type },
				{ ResultName, InTypeIndex }
			});
		}
	}
	return !OutPermutations.IsEmpty();
}

#if WITH_EDITOR

FString FRigVMDispatch_CastObject::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	const TRigVMTypeIndex* ValueTypeIndexPtr = InTypes.Find(ValueName);
	const TRigVMTypeIndex* ResultTypeIndexPtr = InTypes.Find(ResultName);
	
	if(ValueTypeIndexPtr && ResultTypeIndexPtr)
	{
		const TRigVMTypeIndex& ValueTypeIndex = *ValueTypeIndexPtr;
		const TRigVMTypeIndex& ResultTypeIndex = *ResultTypeIndexPtr;
		if(ValueTypeIndex != INDEX_NONE && ResultTypeIndex != INDEX_NONE)
		{
			const FRigVMRegistry& Registry = FRigVMRegistry::Get();
			if(!Registry.IsWildCardType(ValueTypeIndex) && !Registry.IsWildCardType(ResultTypeIndex))
			{
				const FRigVMTemplateArgumentType& ValueType = Registry.GetType(ValueTypeIndex);
				const FRigVMTemplateArgumentType& ResultType = Registry.GetType(ResultTypeIndex);
				static const FString Prefix = TEXT("Cast to ");
				if(ValueType.CPPTypeObject && ResultType.CPPTypeObject)
				{
					return Prefix + ResultType.CPPTypeObject->GetName();
				}
			}
		}
	}
	return Super::GetNodeTitle(InTypes);
}

FText FRigVMDispatch_CastObject::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	const TRigVMTypeIndex* ValueTypeIndexPtr = InTypes.Find(ValueName);
	const TRigVMTypeIndex* ResultTypeIndexPtr = InTypes.Find(ResultName);
	if(ValueTypeIndexPtr && ResultTypeIndexPtr)
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		const FRigVMTemplateArgumentType& ValueType = Registry.GetType(*ValueTypeIndexPtr);
		const FRigVMTemplateArgumentType& ResultType = Registry.GetType(*ResultTypeIndexPtr);
		if(ValueType.CPPTypeObject && ValueType.CPPTypeObject->IsA<UClass>() && ResultType.CPPTypeObject && ResultType.CPPTypeObject->IsA<UClass>())
		{
			return FText::Format(LOCTEXT("CastToolTipFormat", "Cast from {0} to {1}"), CastChecked<UClass>(ValueType.CPPTypeObject)->GetDisplayNameText(), CastChecked<UClass>(ResultType.CPPTypeObject)->GetDisplayNameText());
		}
	}

	return LOCTEXT("CastToolTip", "Casts between object types");
}

FText FRigVMDispatch_CastObject::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ValueName)
	{
		return LOCTEXT("ValueToolTip", "The object to cast to a new type");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("ResultToolTip", "The resulting cast object. This may be potentially nullptr as well if the cast was not successful.");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

void FRigVMDispatch_CastObject::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	UObject** ValueObjectPtr = (UObject**)Handles[0].GetInputData();
	UObject** ResultObjectPtr = (UObject**)Handles[1].GetOutputData();
	const FObjectPropertyBase* ValueObjectProperty = CastFieldChecked<FObjectPropertyBase>(Handles[0].GetResolvedProperty());
	const FObjectPropertyBase* ResultObjectProperty = CastFieldChecked<FObjectPropertyBase>(Handles[1].GetProperty());
	if(ValueObjectPtr && ResultObjectPtr && ValueObjectProperty && ResultObjectProperty)
	{
		if(*ValueObjectPtr && (*ValueObjectPtr)->GetClass()->IsChildOf(ResultObjectProperty->PropertyClass))
		{
			*ResultObjectPtr = *ValueObjectPtr;
		}
		else
		{
			*ResultObjectPtr = nullptr;
		}
	}
}

#undef LOCTEXT_NAMESPACE
