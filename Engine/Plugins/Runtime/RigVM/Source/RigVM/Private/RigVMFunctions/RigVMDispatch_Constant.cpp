// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Constant)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Constant"

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_Constant::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		CachedArgumentInfos.Emplace(FRigVMTemplateArgumentInfo(ValueName, ERigVMPinDirection::IO, Categories));
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_Constant::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	return {
		{ValueName, InTypeIndex},
	};
}

#if WITH_EDITOR

FString FRigVMDispatch_Constant::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	const FString SuperNodeTitle = FRigVMDispatch_CoreBase::GetNodeTitle(InTypes);
	if(const TRigVMTypeIndex* TypeIndexPtr = InTypes.Find(ValueName))
	{
		const TRigVMTypeIndex& TypeIndex = *TypeIndexPtr;
		if(TypeIndex != INDEX_NONE)
		{
			const FRigVMRegistry& Registry = FRigVMRegistry::Get();
			if(!Registry.IsWildCardType(TypeIndex))
			{
				const FRigVMTemplateArgumentType& Type = Registry.GetType(TypeIndex);
				static constexpr TCHAR Format[] = TEXT("%s %s");
				if(Type.CPPTypeObject)
				{
					return FString::Printf(Format, *SuperNodeTitle, *Type.CPPTypeObject->GetName());
				}
				return FString::Printf(Format, *SuperNodeTitle, *Type.CPPType.ToString());
			}
		}
	}
	return SuperNodeTitle;
}

FText FRigVMDispatch_Constant::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("ConstantToolTip", "Provides a constant value to the graph");
}

FText FRigVMDispatch_Constant::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ValueName)
	{
		return LOCTEXT("ConstantValueToolTip", "The constant value defined in this node");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

FString FRigVMDispatch_Constant::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if (InArgumentName == ValueName && InMetaDataKey == FRigVMStruct::ConstantMetaName)
	{
		return TEXT("True");
	}

	return FRigVMDispatch_CoreBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

#endif

void FRigVMDispatch_Constant::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	// nothing to do here - it's a constant
}

#undef LOCTEXT_NAMESPACE
