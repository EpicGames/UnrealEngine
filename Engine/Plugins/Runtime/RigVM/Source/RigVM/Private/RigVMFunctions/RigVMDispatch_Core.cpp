// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Core.h"
#include "RigVMCore/RigVMRegistry.h"
#include "EulerTransform.h"
#include "RigVMTypeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Core)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Core"

FName FRigVMDispatch_CoreEquals::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		AName,
		BName,
		ResultName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CoreEquals::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		CachedArgumentInfos.Emplace(AName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(BName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_CoreEquals::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(AName, InTypeIndex);
	Types.Add(BName, InTypeIndex);
	Types.Add(ResultName, RigVMTypeUtils::TypeIndex::Bool);
	return Types;
}

#if WITH_EDITOR

FText FRigVMDispatch_CoreEquals::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == AName)
	{
		return LOCTEXT("CoreEqualsAToolTip", "The first value to compare");
	}
	if (InArgumentName == BName)
	{
		return LOCTEXT("CoreEqualsBToolTip", "The second value to compare");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("CoreEqualsResultToolTip", "The boolean result of the comparison");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

FRigVMFunctionPtr FRigVMDispatch_CoreEquals::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	const TRigVMTypeIndex TypeIndex = InTypes.FindChecked(AName);
	check(TypeIndex == InTypes.FindChecked(BName));
	check(InTypes.FindChecked(ResultName) == RigVMTypeUtils::TypeIndex::Bool);

	if(TypeIndex == RigVMTypeUtils::TypeIndex::Float)
	{
		return &FRigVMDispatch_CoreEquals::Equals<float>;
	}
	if(TypeIndex == RigVMTypeUtils::TypeIndex::Double)
	{
		return &FRigVMDispatch_CoreEquals::Equals<double>;
	}
	if(TypeIndex == RigVMTypeUtils::TypeIndex::Int32)
	{
		return &FRigVMDispatch_CoreEquals::Equals<int32>;
	}
	if(TypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return &FRigVMDispatch_CoreEquals::NameEquals;
	}
	if(TypeIndex == RigVMTypeUtils::TypeIndex::FString)
	{
		return &FRigVMDispatch_CoreEquals::StringEquals;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FVector>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FVector>;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FVector2D>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FVector2D>;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FRotator>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FRotator>;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FQuat>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FQuat>;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FTransform>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FTransform>;
	}
	if(TypeIndex == InRegistry->GetTypeIndex_NoLock<FLinearColor>())
	{
		return &FRigVMDispatch_CoreEquals::MathTypeEquals<FLinearColor>;
	}
	return &FRigVMDispatch_CoreEquals::Execute;
}

bool FRigVMDispatch_CoreEquals::AdaptResult(bool bResult, const FRigVMExtendedExecuteContext& InContext)
{
	// if the factory is the not equals factory - let's invert the result
	if(InContext.Factory->GetScriptStruct() == FRigVMDispatch_CoreNotEquals::StaticStruct())
	{
		return !bResult;
	}
	return bResult;
}

void FRigVMDispatch_CoreEquals::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FProperty* PropertyA = Handles[0].GetResolvedProperty(); 
	const FProperty* PropertyB = Handles[1].GetResolvedProperty(); 
	check(PropertyA);
	check(PropertyB);
	check(RigVMTypeUtils::AreCompatible(PropertyA, PropertyB));
	check(Handles[2].IsBool());

	const uint8* A = Handles[0].GetInputData();
	const uint8* B = Handles[1].GetInputData();
	bool& Result = *reinterpret_cast<bool*>(Handles[2].GetOutputData());
	Result = PropertyA->Identical(A, B);
	Result = AdaptResult(Result, InContext);
}

// duplicate the code here so that the FRigVMDispatch_CoreNotEquals has it's own static variables
// to store the types are registration time.
const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_CoreNotEquals::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		CachedArgumentInfos.Emplace(AName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(BName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::Bool);
	}
	return CachedArgumentInfos;
}

#undef LOCTEXT_NAMESPACE
