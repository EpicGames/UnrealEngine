// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_If.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_If)
#define LOCTEXT_NAMESPACE "RigVMDispatch_If"

FName FRigVMDispatch_If::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		ConditionName,
		TrueName,
		FalseName,
		ResultName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_If::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
	
		CachedArgumentInfos.Emplace(ConditionName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Bool);
		CachedArgumentInfos.Emplace(TrueName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(FalseName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, ValueCategories);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_If::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(ConditionName, RigVMTypeUtils::TypeIndex::Bool);
	Types.Add(TrueName, InTypeIndex);
	Types.Add(FalseName, InTypeIndex);
	Types.Add(ResultName, InTypeIndex);
	return Types;
}

#if WITH_EDITOR

FString FRigVMDispatch_If::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == TrueName || InArgumentName == FalseName)
	{
		if(InMetaDataKey == FRigVMStruct::ComputeLazilyMetaName)
		{
			return TrueString;
		}
	}
	return FRigVMDispatch_CoreBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FText FRigVMDispatch_If::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ConditionName)
	{
		return LOCTEXT("ConditionToolTip", "The condition on which we'll decide between the True and False values");
	}
	if (InArgumentName == TrueName)
	{
		return LOCTEXT("TrueToolTip", "The value to use if the condition is True");
	}
	if (InArgumentName == FalseName)
	{
		return LOCTEXT("FalseToolTip", "The value to use if the condition is False");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("ResultToolTip", "The resulting value - either True or False based on the condition");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

FRigVMFunctionPtr FRigVMDispatch_If::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	return &FRigVMDispatch_If::Execute;
}

void FRigVMDispatch_If::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FProperty* Property = Handles[1].GetResolvedProperty(); 

#if WITH_EDITOR
	check(Handles[0].IsBool());
#endif

	const bool& Condition = *(const bool*)Handles[0].GetInputData();

	FRigVMMemoryHandle& InputHandle = Condition ? Handles[1] : Handles[2];
	if(InputHandle.IsLazy())
	{
		InputHandle.ComputeLazyValueIfNecessary(InContext, InContext.GetSliceHash());
	}

	const uint8* Input = InputHandle.GetInputData();
	uint8* Result = Handles[3].GetOutputData();

	(void)CopyProperty(
		Handles[3].GetProperty(), Result,
		InputHandle.GetResolvedProperty(), Input);
}

#undef LOCTEXT_NAMESPACE
