// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_MakeStruct.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_MakeStruct)
#define LOCTEXT_NAMESPACE "RigVMDispatch_MakeStruct"

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_MakeStruct::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {
			FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue
		};
		
		CachedArgumentInfos.Emplace(ElementsName, ERigVMPinDirection::Input, Categories);
		CachedArgumentInfos.Emplace(StructName, ERigVMPinDirection::Output, Categories);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_MakeStruct::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	return {
		{ElementsName, InTypeIndex},
		{StructName, InTypeIndex}
	};
}

#if WITH_EDITOR

FString FRigVMDispatch_MakeStruct::GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const
{
	const FString SuperNodeTitle = FRigVMDispatch_CoreBase::GetNodeTitle(InTypes);
	if(const TRigVMTypeIndex* StructTypePtr = InTypes.Find(StructName))
	{
		const TRigVMTypeIndex& StructType = *StructTypePtr;
		if(StructType != INDEX_NONE)
		{
			const FRigVMRegistry& Registry = FRigVMRegistry::Get();
			if(!Registry.IsWildCardType(StructType))
			{
				if(UScriptStruct* Struct = Cast<UScriptStruct>(Registry.GetType(StructType).CPPTypeObject))
				{
					static constexpr TCHAR Format[] = TEXT("%s %s");
					return FString::Printf(Format, *SuperNodeTitle, *Struct->GetName());
				}
			}
		}
	}
	return SuperNodeTitle;
}

FText FRigVMDispatch_MakeStruct::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("MakeStructToolTip", "Composes a struct out of its elements");
}

FString FRigVMDispatch_MakeStruct::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == ElementsName && InMetaDataKey == FRigVMStruct::ShowOnlySubPinsMetaName)
	{
		return TrueString;
	}
	if(InArgumentName == StructName && InMetaDataKey == FRigVMStruct::HideSubPinsMetaName)
	{
		return TrueString;
	}
	return FRigVMDispatch_CoreBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FString FRigVMDispatch_MakeStruct::GetKeywords() const
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	FString Keywords = FRigVMDispatch_CoreBase::GetKeywords();
	const TArray<TRigVMTypeIndex>& StructTypeIndices = Registry.GetTypesForCategory(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue);
	for(const TRigVMTypeIndex& StructTypeIndex : StructTypeIndices)
	{
		const FRigVMTemplateArgumentType& Type = Registry.GetType(StructTypeIndex);
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
		{
			static constexpr TCHAR Format[] = TEXT(",Make%s");
			Keywords += FString::Printf(Format, *ScriptStruct->GetName());
		}
	}
	return Keywords;
}

FText FRigVMDispatch_MakeStruct::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ElementsName)
	{
		return LOCTEXT("MakeStructElementsToolTip", "The elements of the structure to make");
	}
	if (InArgumentName == StructName)
	{
		return LOCTEXT("MakeStructStructToolTip", "The created structure");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

void FRigVMDispatch_MakeStruct::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches)
{
	const FProperty* SourceProperty = Handles[0].GetResolvedProperty();
	const FProperty* TargetProperty = Handles[1].GetProperty();
	check(SourceProperty->SameType(TargetProperty));
	const uint8* SourceMemory = Handles[0].GetInputData();
	uint8* TargetMemory = Handles[1].GetOutputData();
	URigVMMemoryStorage::CopyProperty(TargetProperty, TargetMemory, SourceProperty, SourceMemory);
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_BreakStruct::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {
			FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue
		};

		CachedArgumentInfos.Emplace(StructName, ERigVMPinDirection::Input, Categories);
		CachedArgumentInfos.Emplace(ElementsName, ERigVMPinDirection::Output, Categories);
	}
	return CachedArgumentInfos;
}

#if WITH_EDITOR

FText FRigVMDispatch_BreakStruct::GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const
{
	return LOCTEXT("BreakStructToolTip", "Decomposes a struct into its elements");
}

FString FRigVMDispatch_BreakStruct::GetKeywords() const
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	FString Keywords = FRigVMDispatch_CoreBase::GetKeywords();
	const TArray<TRigVMTypeIndex>& StructTypeIndices = Registry.GetTypesForCategory(FRigVMTemplateArgument::ETypeCategory_SingleScriptStructValue);
	for(const TRigVMTypeIndex& StructTypeIndex : StructTypeIndices)
	{
		const FRigVMTemplateArgumentType& Type = Registry.GetType(StructTypeIndex);
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
		{
			static constexpr TCHAR Format[] = TEXT(",Break%s");
			Keywords += FString::Printf(Format, *ScriptStruct->GetName());
		}
	}
	return Keywords;
}

FText FRigVMDispatch_BreakStruct::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == ElementsName)
	{
		return LOCTEXT("BreakStructElementsToolTip", "The elements of the structure");
	}
	if (InArgumentName == StructName)
	{
		return LOCTEXT("BreakStructStructToolTip", "The structure to break up into its elements");
	}
	return FRigVMDispatch_MakeStruct::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

#undef LOCTEXT_NAMESPACE
