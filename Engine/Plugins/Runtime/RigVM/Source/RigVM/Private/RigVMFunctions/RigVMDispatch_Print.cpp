// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Print.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVM.h"
#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Print)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Print"

FName FRigVMDispatch_Print::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		PrefixName,
		ValueName,
		EnabledName,
		ScreenDurationName,
		ScreenColorName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_Print::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};

		CachedArgumentInfos.Emplace(PrefixName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FString);
		CachedArgumentInfos.Emplace(ValueName, ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(EnabledName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Bool);
		CachedArgumentInfos.Emplace(ScreenDurationName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Float);
		CachedArgumentInfos.Emplace(ScreenColorName, ERigVMPinDirection::Input, InRegistry->GetTypeIndex_NoLock<FLinearColor>());
	}
	return CachedArgumentInfos;
}

const TArray<FRigVMExecuteArgument>& FRigVMDispatch_Print::GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const
{
	static const TArray<FRigVMExecuteArgument> Arguments = {
		{TEXT("ExecuteContext"), ERigVMPinDirection::IO}
	};
	return Arguments;
}

FRigVMTemplateTypeMap FRigVMDispatch_Print::OnNewArgumentType(const FName& InArgumentName,
                                                              TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(PrefixName, RigVMTypeUtils::TypeIndex::FString);
	Types.Add(ValueName, InTypeIndex);
	Types.Add(EnabledName, RigVMTypeUtils::TypeIndex::Bool);
	Types.Add(ScreenDurationName, RigVMTypeUtils::TypeIndex::Float);
	Types.Add(ScreenColorName, InRegistry->GetTypeIndex_NoLock<FLinearColor>());
	return Types;
}

#if WITH_EDITOR

FString FRigVMDispatch_Print::GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == EnabledName)
	{
		return TEXT("True");
	}
	if(InArgumentName == ScreenDurationName)
	{
		return TEXT("0.0");
	}
	return FRigVMDispatchFactory::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

FString FRigVMDispatch_Print::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == ScreenColorName)
	{
		if(InMetaDataKey == FRigVMStruct::DetailsOnlyMetaName)
		{
			return TEXT("True");
		}
	}
	return FRigVMDispatchFactory::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FText FRigVMDispatch_Print::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == PrefixName)
	{
		return LOCTEXT("PrefixToolTip", "The prefix to place at the start of the printed out message");
	}
	if (InArgumentName == ValueName)
	{
		return LOCTEXT("ValueToolTip", "The value to print");
	}
	if (InArgumentName == EnabledName)
	{
		return LOCTEXT("EnabledToolTip", "If False the printing of the message will be skipped");
	}
	if (InArgumentName == ScreenDurationName)
	{
		return LOCTEXT("ScreenDurationToolTip", "The duration of the printed message to appear on the screen. Zero will show it for a single frame. Negative will not display the message.");
	}
	if (InArgumentName == ScreenColorName)
	{
		return LOCTEXT("ScreenColorToolTip", "The color of the message on the screen");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

void FRigVMDispatch_Print::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
#if WITH_EDITOR
	const FProperty* ValueProperty = Handles[1].GetResolvedProperty(); 
	check(ValueProperty);
	check(Handles[0].IsString());
	check(Handles[2].IsBool());
	check(Handles[3].IsFloat());
	check(Handles[4].IsType<FLinearColor>());

	const FString& Prefix = *(const FString*)Handles[0].GetInputData();
	const bool bEnabled = *(const bool*)Handles[2].GetInputData();
	const float& ScreenDuration = *(const float*)Handles[3].GetInputData();
	const FLinearColor& ScreenColor = *(const FLinearColor*)Handles[4].GetInputData();
	const uint8* Value = Handles[1].GetInputData();
	
	if(!bEnabled)
	{
		return;
	}

	FString String;
	ValueProperty->ExportText_Direct(String, Value, Value, nullptr, PPF_None, nullptr);

	FString ObjectPath;
	if(InContext.GetVM())
	{
		ObjectPath = InContext.GetVM()->GetName();
	}

	static constexpr TCHAR LogFormat[] = TEXT("%s[%04d] %s%s");
	InContext.GetPublicData<>().Logf({EMessageSeverity::Info, false}, LogFormat, *ObjectPath, InContext.GetPublicData<>().GetInstructionIndex(), *Prefix, *String);
	const UObject* WorldObject = (const UObject*)InContext.GetVM();

	if(ScreenDuration >= 0.0f && WorldObject)
	{
		static constexpr TCHAR PrintStringFormat[] = TEXT("[%04d] %s%s");
		UKismetSystemLibrary::PrintString(WorldObject, FString::Printf(PrintStringFormat, InContext.GetPublicData<>().GetInstructionIndex(), *Prefix, *String), true, false, ScreenColor, ScreenDuration);
	}
#endif
}

#undef LOCTEXT_NAMESPACE
