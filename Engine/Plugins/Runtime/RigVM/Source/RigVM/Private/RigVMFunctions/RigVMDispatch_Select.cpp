// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Select.h"
#include "RigVMStringUtils.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Select)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Select"

FName FRigVMDispatch_SelectInt32::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	if(InOperandIndex > 0 && InOperandIndex < InTotalOperands - 1)
	{
		return FRigVMBranchInfo::GetFixedArrayLabel(ValuesName, *FString::FromInt(InOperandIndex - 1));
	}
	
	return InOperandIndex == 0 ? IndexName : ResultName;
}


const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_SelectInt32::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if (CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ArrayCategories = {
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue
		};
		static const TArray<FRigVMTemplateArgument::ETypeCategory> SingleCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		CachedArgumentInfos.Emplace(IndexName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Int32);
		CachedArgumentInfos.Emplace(ValuesName, ERigVMPinDirection::Input, ArrayCategories);
		CachedArgumentInfos.Emplace(ResultName, ERigVMPinDirection::Output, SingleCategories);
	}

	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_SelectInt32::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(IndexName, RigVMTypeUtils::TypeIndex::Int32);
	if(InArgumentName == ValuesName)
	{
		Types.Add(ValuesName, InTypeIndex);
		Types.Add(ResultName, InRegistry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex));
	}
	else
	{
		Types.Add(ValuesName, InRegistry->GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex));
		Types.Add(ResultName, InTypeIndex);
	}
	return Types;
}

#if WITH_EDITOR

FString FRigVMDispatch_SelectInt32::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == ValuesName)
	{
		if(InMetaDataKey == FRigVMStruct::ComputeLazilyMetaName)
		{
			return TrueString;
		}
		if(InMetaDataKey == FRigVMStruct::FixedSizeArrayMetaName)
		{
			return TrueString;
		}
	}
	return FRigVMDispatch_CoreBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FString FRigVMDispatch_SelectInt32::GetArgumentDefaultValue(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == IndexName)
	{
		static const FString ZeroValue = TEXT("0");
		return ZeroValue;
	}
	if(InArgumentName == ValuesName)
	{
		const TRigVMTypeIndex ElementTypeIndex = FRigVMRegistry::Get().GetBaseTypeFromArrayTypeIndex(InTypeIndex);
		FString ElementDefaultValue = FRigVMDispatch_CoreBase::GetArgumentDefaultValue(InArgumentName, ElementTypeIndex);
		if(ElementDefaultValue.IsEmpty())
		{
			static const FString EmptyBracesString = TEXT("()");
			ElementDefaultValue = EmptyBracesString;
		}

		// default to two entries
		return RigVMStringUtils::JoinDefaultValue({ElementDefaultValue, ElementDefaultValue});
	}
	return FRigVMDispatch_CoreBase::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

FText FRigVMDispatch_SelectInt32::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == IndexName)
	{
		return LOCTEXT("IndexToolTip", "The index of the value to select");
	}
	if (InArgumentName == ValuesName)
	{
		return LOCTEXT("ValuesToolTip", "The fixed array of values to select from");
	}
	if (InArgumentName == ResultName)
	{
		return LOCTEXT("ResultToolTip", "The resulting selected value from the values array based on the index");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif


FRigVMFunctionPtr FRigVMDispatch_SelectInt32::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	return &FRigVMDispatch_SelectInt32::Execute;
}

void FRigVMDispatch_SelectInt32::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FProperty* Property = Handles.Last().GetResolvedProperty();
	const int32 NumCases = Handles.Num() - 2;

	if(NumCases < 1)
	{
		static constexpr TCHAR InvalidIndexMessage[] = TEXT("SelectInt32: No cases to select from.");
		InContext.GetPublicData<>().Log(EMessageSeverity::Error, InvalidIndexMessage);
		return;
	}

#if WITH_EDITOR
	check(Handles[0].IsInt32());
#endif

	int32 Index = *(const int32*)Handles[0].GetInputData();
	if(!FMath::IsWithin(Index, 0, NumCases))
	{
		static constexpr TCHAR InvalidIndexMessage[] = TEXT("SelectInt32: Invalid index %d - num cases is %d. Falling back to index 0.");
		InContext.GetPublicData<>().Logf(EMessageSeverity::Warning, InvalidIndexMessage, Index, NumCases);
		Index = 0;
	}

	FRigVMMemoryHandle& InputHandle = Handles[1 + Index];
	if(InputHandle.IsLazy())
	{
		InputHandle.ComputeLazyValueIfNecessary(InContext, InContext.GetSliceHash());
	}

	const uint8* Input = InputHandle.GetInputData();
	uint8* Result = Handles.Last().GetOutputData();

	// copy property performs compatibility check and will report accordingly
	(void)CopyProperty(
		Handles.Last().GetProperty(), Result,
		InputHandle.GetResolvedProperty(), Input);
}

#undef LOCTEXT_NAMESPACE
