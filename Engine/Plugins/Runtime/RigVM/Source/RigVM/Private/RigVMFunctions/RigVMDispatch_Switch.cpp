// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMDispatch_Switch.h"
#include "RigVMStringUtils.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStruct.h"
#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatch_Switch)
#define LOCTEXT_NAMESPACE "RigVMDispatch_Switch"

FName FRigVMDispatch_SwitchBase::GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const
{
	static const FLazyName ArgumentNames[] = {
		IndexName,
		FRigVMStruct::ControlFlowBlockToRunName
	};
	check(InTotalOperands == UE_ARRAY_COUNT(ArgumentNames));
	check(InOperandIndex < InTotalOperands);
	return ArgumentNames[InOperandIndex];
}

const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_SwitchInt32::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		CachedArgumentInfos.Emplace(IndexName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Int32);
		CachedArgumentInfos.Emplace(FRigVMStruct::ControlFlowBlockToRunName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::FName);
	}
	return CachedArgumentInfos;
}

const TArray<FRigVMExecuteArgument>& FRigVMDispatch_SwitchBase::GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const
{
	static const TArray<FRigVMExecuteArgument> Arguments =
	{
		{FRigVMStruct::ExecuteContextName, ERigVMPinDirection::Input},
		{CasesName, ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::ExecuteArray},
		{FRigVMStruct::ControlFlowCompletedName, ERigVMPinDirection::Output}
	};
	return Arguments;
}

#if WITH_EDITOR

FString FRigVMDispatch_SwitchBase::GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
{
	if(InArgumentName == CasesName)
	{
		if(InMetaDataKey == FRigVMStruct::FixedSizeArrayMetaName)
		{
			return TrueString;
		}
	}
	return FRigVMDispatch_CoreBase::GetArgumentMetaData(InArgumentName, InMetaDataKey);
}

FString FRigVMDispatch_SwitchBase::GetArgumentDefaultValue(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex) const
{
	if(InArgumentName == CasesName)
	{
		static const FString TwoCases = TEXT("((),())");
		return TwoCases;
	}
	return FRigVMDispatch_CoreBase::GetArgumentDefaultValue(InArgumentName, InTypeIndex);
}

FName FRigVMDispatch_SwitchInt32::GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const
{
	static const FString CasesPrefix = CasesName.ToString() + TEXT(".");
	const FString ArgumentNameString = InArgumentName.ToString();
	if(ArgumentNameString.StartsWith(CasesPrefix))
	{
		FString Left, Right;
		verify(RigVMStringUtils::SplitPinPathAtEnd(ArgumentNameString, Left, Right));
		if(Right.IsNumeric())
		{
			const int32 CaseIndex = FCString::Atoi(*Right);
			return GetCaseDisplayName(CaseIndex);
		}
	}
	return FRigVMDispatch_CoreBase::GetDisplayNameForArgument(InArgumentName, InTypes);
}

FText FRigVMDispatch_SwitchBase::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == IndexName)
	{
		return LOCTEXT("IndexToolTip", "The index of the case to switch to");
	}
	if (InArgumentName == CasesName)
	{
		return LOCTEXT("CasesToolTip", "The fixed set of cases to switch to");
	}
	return FRigVMDispatch_CoreBase::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMDispatch_SwitchInt32::GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const
{
	static const TArray<FName> DefaultBlocks = {FRigVMStruct::ControlFlowCompletedName};
	
	if(!InContext.StringRepresentation.IsEmpty())
	{
		static const FString CasesPrefix = CasesName.ToString() + TEXT("=");
		TArray<FString> NameValuePairs = RigVMStringUtils::SplitDefaultValue(InContext.StringRepresentation);
		for(const FString& NameValuePair : NameValuePairs)
		{
			if(NameValuePair.StartsWith(CasesPrefix))
			{
				const FString Values = NameValuePair.Mid(CasesPrefix.Len());

				static TMap<FString, TArray<FName>> BlockCache;
				if(const TArray<FName>* ExistingBlocks = BlockCache.Find(Values))
				{
					return *ExistingBlocks;
				}
				
				TArray<FName>& Blocks = BlockCache.Add(Values);
				
				const TArray<FString> CaseNames = RigVMStringUtils::SplitDefaultValue(Values);
				for(int32 CaseIndex = 0; CaseIndex < CaseNames.Num(); CaseIndex++)
				{
					Blocks.Add(GetCaseName(CaseIndex));
				}
				
				Blocks.Append(DefaultBlocks);
				return Blocks;
			}
		}
	}

	return DefaultBlocks;
}
#endif

FRigVMFunctionPtr FRigVMDispatch_SwitchInt32::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	return &FRigVMDispatch_SwitchInt32::Execute;
}

void FRigVMDispatch_SwitchInt32::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
#if WITH_EDITOR
	check(Handles[0].IsInt32());
	check(Handles[1].IsName());
#endif

	const int32 Index = *reinterpret_cast<const int32*>(Handles[0].GetInputData());
	FName& BlockToRun = *reinterpret_cast<FName*>(Handles[1].GetOutputData());

	if(BlockToRun.IsNone())
	{
		BlockToRun = GetCaseName(Index);
	}
	else
	{
		BlockToRun = FRigVMStruct::ControlFlowCompletedName;
	}
}

FName FRigVMDispatch_SwitchBase::GetCaseName(int32 InIndex)
{
	using MapType = TMap<int32, FName>;
	UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(MapType, NamesFromInt);

	if (FName* Found = NamesFromInt.Find(InIndex))
	{
		return *Found;
	}
	
	const FName Result = FRigVMBranchInfo::GetFixedArrayLabel(CasesName, *FString::FromInt(InIndex));
	NamesFromInt.Add(InIndex, Result);
	return Result;
}

FName FRigVMDispatch_SwitchInt32::GetCaseDisplayName(int32 InIndex)
{
	using MapType = TMap<int32, FName>;
	UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(MapType, NamesFromInt);

    if (FName* Found = NamesFromInt.Find(InIndex))
    {
    	return *Found;
    }
    
	static constexpr TCHAR Format[] = TEXT("Case %d");
	const FName Result = *FString::Printf(Format, InIndex);
    NamesFromInt.Add(InIndex, Result);
    return Result;
}


const TArray<FRigVMTemplateArgumentInfo>& FRigVMDispatch_SwitchEnum::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> EnumCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleEnumValue
		};
		
		CachedArgumentInfos.Emplace(IndexName, ERigVMPinDirection::Input, EnumCategories);
		CachedArgumentInfos.Emplace(FRigVMStruct::ControlFlowBlockToRunName, ERigVMPinDirection::Hidden, RigVMTypeUtils::TypeIndex::FName);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigVMDispatch_SwitchEnum::OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	return {
			{IndexName, InTypeIndex},
			{FRigVMStruct::ControlFlowBlockToRunName, RigVMTypeUtils::TypeIndex::FName}
	};
}

#if WITH_EDITOR

FName FRigVMDispatch_SwitchEnum::GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const
{
	static const FString CasesPrefix = CasesName.ToString() + TEXT(".");
	const FString ArgumentNameString = InArgumentName.ToString();
	if(ArgumentNameString.StartsWith(CasesPrefix))
	{
		FString Left, Right;
		verify(RigVMStringUtils::SplitPinPathAtEnd(ArgumentNameString, Left, Right));
		if(Right.IsNumeric())
		{
			UEnum* Enum = nullptr;
			if (const TRigVMTypeIndex* IndexType = InTypes.Find(IndexName))
			{
				FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(*IndexType);
				if (UEnum* TypeEnum = Cast<UEnum>(Type.CPPTypeObject))
				{
					Enum = TypeEnum;
				}
			}
			const int32 CaseIndex = FCString::Atoi(*Right);
			return GetCaseDisplayName(CaseIndex, Enum);
		}
	}
	return FRigVMDispatch_CoreBase::GetDisplayNameForArgument(InArgumentName, InTypes);
}

#endif

#if WITH_EDITORONLY_DATA
const TArray<FName>& FRigVMDispatch_SwitchEnum::GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const
{
	static const TArray<FName> DefaultBlocks = {FRigVMStruct::ControlFlowCompletedName};

	UEnum* Enum = nullptr;
	if (const TRigVMTypeIndex* IndexTypeIndex = InTypeMap.Find(IndexName))
	{
		const FRigVMTemplateArgumentType& IndexType = FRigVMRegistry::Get().GetType(*IndexTypeIndex);
		if (UEnum* IndexEnum = Cast<UEnum>(IndexType.CPPTypeObject))
		{
			Enum = IndexEnum;
		}
	}
	
	if (Enum)
	{
		static TMap<FString, TArray<FName>> BlockCache;
		if(const TArray<FName>* ExistingBlocks = BlockCache.Find(Enum->GetName()))
		{
			return *ExistingBlocks;
		}
		TArray<FName>& Blocks = BlockCache.Add(Enum->GetName());
		
		const int32 NumCases = Enum->NumEnums() - 1; // Exclude 'MAX' enum value.
		for (int32 i=0; i<NumCases; ++i)
		{
			if (Enum->HasMetaData(TEXT("Hidden"), i))
			{
				continue;
			}
			
			Blocks.Add(GetCaseName(i));
		}
		Blocks.Append(DefaultBlocks);
		return Blocks;
	}

	return DefaultBlocks;
}
#endif

FRigVMFunctionPtr FRigVMDispatch_SwitchEnum::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	return &FRigVMDispatch_SwitchEnum::Execute;
}

void FRigVMDispatch_SwitchEnum::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
#if WITH_EDITOR
	check(CastField<FEnumProperty>(Handles[0].GetProperty()) || CastField<FByteProperty>(Handles[0].GetProperty()));
	check(Handles[1].IsName());
#endif
	
	int32 Index = INDEX_NONE;
	{
		const FNumericProperty* NumericProperty = nullptr;
		const FProperty* IndexProperty = Handles[0].GetProperty();
		UEnum* Enum = nullptr;
		if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(IndexProperty))
		{
			Enum = EnumProperty->GetEnum();
			NumericProperty = EnumProperty->GetUnderlyingProperty();
		}
		else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(IndexProperty))
		{
			Enum = ByteProperty->GetIntPropertyEnum();
			NumericProperty = ByteProperty;
		}
		check(NumericProperty && NumericProperty->IsInteger());
		check(Enum);
		
		const uint8* ValuePtr = Handles[0].GetInputData();
		const int64 Value = NumericProperty->GetSignedIntPropertyValue(ValuePtr);
		
		Index = Enum->GetIndexByValue(Value);
	}

	FName& BlockToRun = *reinterpret_cast<FName*>(Handles[1].GetOutputData());

	if(BlockToRun.IsNone() && Index != INDEX_NONE)
	{
		BlockToRun = GetCaseName(Index);
	}
	else
	{
		BlockToRun = FRigVMStruct::ControlFlowCompletedName;
	}
}

#if WITH_EDITORONLY_DATA
FName FRigVMDispatch_SwitchEnum::GetCaseDisplayName(int32 InIndex, const UEnum* InEnum)
{
	using MapType = TMap<int32, FName>;
	UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(MapType, NamesFromInt);

	FName Result = NAME_None;
	if (InEnum)
	{
		// The input index represents the index of visible values,
		// so we need to find the actual real index
		Result = *InEnum->GetDisplayNameTextByIndex(InIndex).ToString();
	}
    return Result;
}

#endif


#undef LOCTEXT_NAMESPACE
