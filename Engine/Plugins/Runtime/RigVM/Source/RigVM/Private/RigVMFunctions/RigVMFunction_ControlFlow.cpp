// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMFunction_ControlFlow.h"
#include "RigVMFunctions/Simulation/RigVMFunction_Timeline.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_ControlFlow)

FRigVMFunction_ControlFlowBranch_Execute()
{
	if(BlockToRun.IsNone())
	{
		if(Condition)
		{
			static const FLazyName TrueName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, True)); 
			BlockToRun = TrueName;
		}
		else
		{
			static const FLazyName FalseName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, False)); 
			BlockToRun = FalseName;
		}
	}
	else
	{
		BlockToRun = FRigVMStruct::ControlFlowCompletedName;
	}
}

FRigVMFunction_ControlFlowRunOnce_Execute()
{
	if(BlockToRun.IsNone())
	{
		if (!HasRun)
		{
			static const FLazyName OnceName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowRunOnce, Once)); 
			BlockToRun = OnceName.Resolve();
			HasRun = true;
			return;
		}
	}
	BlockToRun = FRigVMStruct::ControlFlowCompletedName;
}

FRigVMFunction_ControlFlowTimer_Execute()
{
	static const FLazyName JustStartedName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, JustStarted)); 
	static const FLazyName IsRunningName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, IsRunning)); 
	static const FLazyName JustFinishedName(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, JustFinished)); 
	
	if(BlockToRun.IsNone())
	{
		FRigVMFunction_TimerValue::StaticExecute(ExecuteContext, Reset, Speed, Duration, Repeats, bJustStarted, bIsRunning, bJustFinished, Time, Ratio, AccumulatedValue, bIsInitialized);

		if (bJustStarted)
		{
			BlockToRun = JustStartedName;
			return;
		}
		if (bIsRunning)
		{
			BlockToRun = IsRunningName;
			return;
		}
		if (bJustFinished)
		{
			BlockToRun = JustFinishedName;
			return;
		}
	}

	if (BlockToRun == JustStartedName && bIsRunning)
	{
		BlockToRun = IsRunningName;
		return;
	}
	if (BlockToRun == IsRunningName && bJustFinished)
	{
		BlockToRun = JustFinishedName;
		return;
	}

	BlockToRun = FRigVMStruct::ControlFlowCompletedName;
}
