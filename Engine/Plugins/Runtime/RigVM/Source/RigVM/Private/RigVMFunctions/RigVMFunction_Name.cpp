// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMFunction_Name.h"
#include "RigVMFunctions/RigVMFunction_String.h"
#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Name)

FRigVMFunction_NameConcat_Execute()
{
	check(ExecuteContext.GetNameCache());
	
	Result = ExecuteContext.GetNameCache()->Concat(A, B);
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_NameConcat)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName ControlRig(TEXT("ControlRig"));
	
	Unit.A = Control;
	Unit.B = Rig;
	Execute();
	AddErrorIfFalse(Unit.Result == ControlRig, TEXT("unexpected name concat result"));

	Unit.A = Control;
	Unit.B = NAME_None;
	Execute();
	AddErrorIfFalse(Unit.Result == Control, TEXT("unexpected name concat result"));

	Unit.A = NAME_None;
	Unit.B = Rig;
	Execute();
	AddErrorIfFalse(Unit.Result == Rig, TEXT("unexpected name concat result"));
	return true;
}

#endif

FRigVMFunction_NameTruncate_Execute()
{
	check(ExecuteContext.GetNameCache());

	Remainder = Name;
	Chopped = NAME_None;

	if(Name.IsNone() || Count <= 0)
	{
		return;
	}

	if (FromEnd)
	{
		Remainder = ExecuteContext.GetNameCache()->LeftChop(Name, Count);
		Chopped = ExecuteContext.GetNameCache()->Right(Name, Count);
	}
	else
	{
		Remainder = ExecuteContext.GetNameCache()->RightChop(Name, Count);
		Chopped = ExecuteContext.GetNameCache()->Left(Name, Count);
	}
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_NameTruncate)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName ControlRig(TEXT("ControlRig"));
	const FName Con(TEXT("Con"));
	const FName trolRig(TEXT("trolRig"));
	
	Unit.Name = ControlRig;
	Unit.Count = 3;
	Unit.FromEnd = true;
	Execute();
	AddErrorIfFalse(Unit.Chopped == Rig, *FString::Printf(TEXT("unexpected name truncate result 1 '%s'"), *Unit.Chopped.ToString()));
	AddErrorIfFalse(Unit.Remainder == Control, *FString::Printf(TEXT("unexpected name truncate result 2 '%s'"), *Unit.Remainder.ToString()));

	Unit.Name = ControlRig;
	Unit.Count = 7;
	Unit.FromEnd = false;
	Execute();
	AddErrorIfFalse(Unit.Chopped == Control, *FString::Printf(TEXT("unexpected name truncate result 3 '%s'"), *Unit.Chopped.ToString()));
	AddErrorIfFalse(Unit.Remainder == Rig, *FString::Printf(TEXT("unexpected name truncate result 4 '%s'"), *Unit.Remainder.ToString()));

	Unit.Name = ControlRig;
	Unit.Count = 3;
	Unit.FromEnd = false;
	Execute();
	AddErrorIfFalse(Unit.Chopped == Con, *FString::Printf(TEXT("unexpected name truncate result 5 '%s'"), *Unit.Chopped.ToString()));
	AddErrorIfFalse(Unit.Remainder == trolRig, *FString::Printf(TEXT("unexpected name truncate result 6 '%s'"), *Unit.Remainder.ToString()));

	Unit.Name = ControlRig;
	Unit.Count = 0;
	Unit.FromEnd = false;
	Execute();
	AddErrorIfFalse(Unit.Chopped == NAME_None, *FString::Printf(TEXT("unexpected name truncate result 7 '%s'"), *Unit.Chopped.ToString()));
	AddErrorIfFalse(Unit.Remainder == ControlRig, *FString::Printf(TEXT("unexpected name truncate result 8 '%s'"), *Unit.Remainder.ToString()));

	Unit.Name = NAME_None;
	Unit.Count = 6;
	Unit.FromEnd = false;
	Execute();
	AddErrorIfFalse(Unit.Chopped == NAME_None, *FString::Printf(TEXT("unexpected name truncate result 9 '%s'"), *Unit.Chopped.ToString()));
	AddErrorIfFalse(Unit.Remainder == NAME_None, *FString::Printf(TEXT("unexpected name truncate result 10 '%s'"), *Unit.Remainder.ToString()));
	return true;
}
#endif

FRigVMFunction_NameReplace_Execute()
{
	check(ExecuteContext.GetNameCache());

	Result = ExecuteContext.GetNameCache()->Replace(Name, Old, New, ESearchCase::IgnoreCase);
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_NameReplace)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName ControlRig(TEXT("ControlRig"));
	const FName Foo(TEXT("Foo"));
	const FName FooRig(TEXT("FooRig"));
	const FName ControlFoo(TEXT("ControlFoo"));
	
	Unit.Name = ControlRig;
	Unit.Old = Control;
	Unit.New = Foo;
	Execute();
	AddErrorIfFalse(Unit.Result == FooRig, TEXT("unexpected name replace result"));

	Unit.Name = ControlRig;
	Unit.Old = Rig;
	Unit.New = Foo;
	Execute();
	AddErrorIfFalse(Unit.Result == ControlFoo, TEXT("unexpected name replace result"));

	Unit.Name = ControlRig;
	Unit.Old = NAME_None;
	Unit.New = Foo;
	Execute();
	AddErrorIfFalse(Unit.Result == ControlRig, TEXT("unexpected name replace result"));

	Unit.Name = ControlRig;
	Unit.Old = Control;
	Unit.New = NAME_None;
	Execute();
	AddErrorIfFalse(Unit.Result == Rig, TEXT("unexpected name replace result"));

	return true;
}
#endif

FRigVMFunction_EndsWith_Execute()
{
	check(ExecuteContext.GetNameCache());

	Result = ExecuteContext.GetNameCache()->EndsWith(Name, Ending, ESearchCase::IgnoreCase);
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_EndsWith)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName ControlRig(TEXT("ControlRig"));
	
	Unit.Name = ControlRig;
	Unit.Ending = Control;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected name endswith result"));

	Unit.Name = ControlRig;
	Unit.Ending = Rig;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected name endswith result"));

	return true;
}

#endif

FRigVMFunction_StartsWith_Execute()
{
	check(ExecuteContext.GetNameCache());

	Result = ExecuteContext.GetNameCache()->StartsWith(Name, Start, ESearchCase::IgnoreCase);
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_StartsWith)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName ControlRig(TEXT("ControlRig"));
	
	Unit.Name = ControlRig;
	Unit.Start = Control;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected name startswith result"));

	Unit.Name = ControlRig;
	Unit.Start = Rig;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected name startswith result"));

	return true;
}
#endif

FRigVMFunction_Contains_Execute()
{
	check(ExecuteContext.GetNameCache());

	Result = ExecuteContext.GetNameCache()->Contains(Name, Search, ESearchCase::IgnoreCase);
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_Contains)
{
	const FName Control(TEXT("Control"));
	const FName Rig(TEXT("Rig"));
	const FName Foo(TEXT("Foo"));
	const FName ControlRig(TEXT("ControlRig"));
	
	Unit.Name = ControlRig;
	Unit.Search = Control;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected name contains result"));

	Unit.Name = ControlRig;
	Unit.Search = Rig;
	Execute();
	AddErrorIfFalse(Unit.Result == true, TEXT("unexpected name contains result"));

	Unit.Name = ControlRig;
	Unit.Search = Foo;
	Execute();
	AddErrorIfFalse(Unit.Result == false, TEXT("unexpected name contains result"));

	return true;
}
#endif

FRigVMFunction_IsNone_Execute()
{
	Result = Name.IsNone();
}

FRigVMFunction_IsNameValid_Execute()
{
	Result = !Value.IsNone();
}

FRigVMFunction_GetNameNumericSuffix_Execute()
{
	Suffix = INDEX_NONE;
	Success = false;

	if (Name.IsNone())
	{
		return;
	}

	if (Name.GetNumber() != NAME_NO_NUMBER_INTERNAL)
	{
		Suffix = NAME_INTERNAL_TO_EXTERNAL(Name.GetNumber());
		Success = true;
		return;
	}

	FRigVMFunction_StringToInteger::StaticExecute(
		ExecuteContext, 
		Name.ToString(),
		true, // chop left
		false, // chop right
		Suffix,
		Success);
}
