// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/RigVMFunction_String.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "RigVMCore/RigVM.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_String)
#define LOCTEXT_NAMESPACE "RigVMFunction_String"

FRigVMFunction_StringConcat_Execute()
{
	Result = A + B;
}

FRigVMFunction_StringTruncate_Execute()
{
	Remainder = Name;
	Chopped = FString();

	if(Name.IsEmpty() || Count <= 0)
	{
		return;
	}

	if (FromEnd)
	{
		Remainder = Name.LeftChop(Count);
		Chopped = Name.Right(Count);
	}
	else
	{
		Remainder = Name.RightChop(Count);
		Chopped = Name.Left(Count);
	}
}

FRigVMFunction_StringReplace_Execute()
{
	Result = Name.Replace(*Old, *New, ESearchCase::CaseSensitive);
}

FRigVMFunction_StringEndsWith_Execute()
{
	Result = Name.EndsWith(Ending, ESearchCase::CaseSensitive);
}

FRigVMFunction_StringStartsWith_Execute()
{
	Result = Name.StartsWith(Start, ESearchCase::CaseSensitive);
}

FRigVMFunction_StringContains_Execute()
{
	Result = Name.Contains(Search, ESearchCase::CaseSensitive);
}

FRigVMFunction_StringLength_Execute()
{
	Length = Value.Len();
}

FRigVMFunction_StringTrimWhitespace_Execute()
{
	Result = Value;
	Result.TrimStartAndEndInline();
}

FRigVMFunction_StringToUppercase_Execute()
{
	Result = Value.ToUpper();
}

FRigVMFunction_StringToLowercase_Execute()
{
	Result = Value.ToLower();
}

FRigVMFunction_StringReverse_Execute()
{
	Reverse = Value.Reverse();
}

FRigVMFunction_StringLeft_Execute()
{
	Result = Value.Left(Count);
}

FRigVMFunction_StringRight_Execute()
{
	Result = Value.Right(Count);
}

FRigVMFunction_StringMiddle_Execute()
{
	if(Count < 0)
	{
		Result = Value.Mid(Start);
	}
	else
	{
		Result = Value.Mid(Start, Count);
	}
}

FRigVMFunction_StringFind_Execute()
{
	Index = INDEX_NONE;

	if(!Value.IsEmpty() && !Search.IsEmpty())
	{
		Index = Value.Find(Search, ESearchCase::CaseSensitive);
	}

	Found = Index != INDEX_NONE;
}

FRigVMFunction_StringSplit_Execute()
{
	Result.Reset();
	if(!Value.IsEmpty())
	{
		if(Separator.IsEmpty())
		{
			UE_RIGVMSTRUCT_REPORT_ERROR(TEXT("Separator is empty."));
			return;
		}

		FString ValueRemaining = Value;
		FString Left, Right;
		while(ValueRemaining.Split(Separator, &Left, &Right, ESearchCase::CaseSensitive, ESearchDir::FromStart))
		{
			Result.Add(Left);
			Left.Empty();
			ValueRemaining = Right;
		}

		if (!Right.IsEmpty())
		{
			Result.Add(Right);
		}
	}
}

FRigVMFunction_StringJoin_Execute()
{
	Result.Reset();
	if(!Values.IsEmpty())
	{
		Result = RigVMStringUtils::JoinStringsConst(Values, *Separator);
	}
}

FRigVMFunction_StringPadInteger_Execute()
{
	if (Digits >= 2 && Digits <= 16)
	{
		Result = FString::Printf(TEXT("%0*d"), Digits, Value); 
	}
	else
	{
		Result = FString::FormatAsNumber(Value);
	}
}

FRigVMFunction_StringToInteger_Execute()
{
	Success = false;
	Result = INDEX_NONE;

	FStringView View(Value);

	auto IsValidChar = [](int32 InIndex, const TCHAR& InChar)
	{
		return FChar::IsDigit(InChar) || (InIndex == 0 && InChar == '-');
	};
	
	if (ChopLeft)
	{
		for (int32 Index = View.Len() - 1; Index >= 0; --Index)
		{
			const TCHAR& C = View[Index];
			if (!IsValidChar(Index, C))
			{
				View.MidInline(Index+1);
				break;
			}
		}
	}
	
	if (ChopRight)
	{
		for (int32 Index = 0; Index < View.Len(); ++Index)
		{
			const TCHAR& C = View[Index];
			if (!IsValidChar(Index, C))
			{
				View.LeftInline(Index);
				break;
			}
		}

		while (!View.IsEmpty())
		{
			const int32 Index = View.Len()-1;
			const TCHAR& C = View[Index];
			if (IsValidChar(Index, C))
			{
				break;
			}				
			View.LeftChopInline(1);
		}
	}

	if (View.IsEmpty())
	{
		return;
	}

	for (int32 i = 0; i < View.Len(); ++i)
	{
		const TCHAR& C = View[i];
		if (!IsValidChar(i, C))
		{
#if WITH_EDITOR
			if(ExecuteContext.GetLog() != nullptr)
			{
				ExecuteContext.Report(EMessageSeverity::Error, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), TEXT("String contains non-digit characters."));
			}
#endif
			return;
		}
	}

	if (View.Len() == Value.Len())
	{
		Result = FCString::Atoi(*Value);
	}
	else
	{
		Result = FCString::Atoi(*FString(View));
	}
	Success = true;
}

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_ToString::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		CachedArgumentInfos.Emplace(TEXT("Value"), ERigVMPinDirection::Input, ValueCategories);
		CachedArgumentInfos.Emplace(TEXT("Result"), ERigVMPinDirection::Output, RigVMTypeUtils::TypeIndex::FString);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigDispatch_ToString::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(TEXT("Value"), InTypeIndex);
	Types.Add(TEXT("Result"), RigVMTypeUtils::TypeIndex::FString);
	return Types;
}

#if WITH_EDITOR

FText FRigDispatch_ToString::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == TEXT("Value"))
	{
		return LOCTEXT("ToStringValueToolTip", "The value to convert to a string");
	}
	if (InArgumentName == TEXT("Result"))
	{
		return LOCTEXT("ToStringResultToolTip", "The resulting string representation of the value");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

FRigVMFunctionPtr FRigDispatch_ToString::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	// treat name and string special
	const TRigVMTypeIndex& ValueTypeIndex = InTypes.FindChecked(TEXT("Value"));
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return [](FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
		{
			check(Handles[0].IsName());
			check(Handles[1].IsString());

			const FName& Value = *(const FName*)Handles[0].GetInputData();
			FString& Result = *(FString*)Handles[1].GetOutputData();
			Result = Value.ToString();
		};
	}
	if(ValueTypeIndex == RigVMTypeUtils::TypeIndex::FString)
	{
		return [](FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
		{
			check(Handles[0].IsString());
			check(Handles[1].IsString());

			const FString& Value = *(const FString*)Handles[0].GetInputData();
			FString& Result = *(FString*)Handles[1].GetOutputData();
			Result = Value;
		};
	}
	
	return &FRigDispatch_ToString::Execute;
}

void FRigDispatch_ToString::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FProperty* ValueProperty = Handles[0].GetResolvedProperty(); 
	check(ValueProperty);
	check(Handles[1].IsString());

	const uint8* Value = Handles[0].GetInputData();
	FString& Result = *(FString*)Handles[1].GetOutputData();
	Result.Reset();
	ValueProperty->ExportText_Direct(Result, Value, Value, nullptr, PPF_None, nullptr);
}

const TArray<FRigVMTemplateArgumentInfo>& FRigDispatch_FromString::GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const
{
	if(CachedArgumentInfos.IsEmpty())
	{
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		CachedArgumentInfos.Emplace(TEXT("String"), ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::FString);
		CachedArgumentInfos.Emplace(TEXT("Result"), ERigVMPinDirection::Output, ValueCategories);
	}
	return CachedArgumentInfos;
}

FRigVMTemplateTypeMap FRigDispatch_FromString::OnNewArgumentType(const FName& InArgumentName,
	TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	FRigVMTemplateTypeMap Types;
	Types.Add(TEXT("String"), RigVMTypeUtils::TypeIndex::FString);
	Types.Add(TEXT("Result"), InTypeIndex);
	return Types;
}

#if WITH_EDITOR

FText FRigDispatch_FromString::GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const
{
	if (InArgumentName == TEXT("String"))
	{
		return LOCTEXT("FromStringStringToolTip", "The string to convert the value from");
	}
	if (InArgumentName == TEXT("Result"))
	{
		return LOCTEXT("FromStringResultToolTip", "The resulting value based on the string");
	}
	return FRigVMDispatchFactory::GetArgumentTooltip(InArgumentName, InTypeIndex);
}

#endif

FRigVMFunctionPtr FRigDispatch_FromString::GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const
{
	// treat name and string special
	const TRigVMTypeIndex& ResultTypeIndex = InTypes.FindChecked(TEXT("Result"));
	if(ResultTypeIndex == RigVMTypeUtils::TypeIndex::FName)
	{
		return [](FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
		{
			check(Handles[0].IsString());
			check(Handles[1].IsName());

			const FString& String = *(FString*)Handles[0].GetInputData();
			FName& Result = *(FName*)Handles[1].GetOutputData();
			Result = *String;
		};
	}
	if(ResultTypeIndex == RigVMTypeUtils::TypeIndex::FString)
	{
		return [](FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
		{
			check(Handles[0].IsString());
			check(Handles[1].IsString());

			const FString& String = *(FString*)Handles[0].GetInputData();
			FString& Result = *(FString*)Handles[1].GetOutputData();
			Result = String;
		};
	}

	return &FRigDispatch_FromString::Execute;
}

void FRigDispatch_FromString::Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
{
	const FProperty* ValueProperty = Handles[1].GetResolvedProperty(); 
	check(ValueProperty);
	check(Handles[0].IsString());

	const FString& String = *(const FString*)Handles[0].GetInputData();
	uint8* Value = Handles[1].GetOutputData();

	class FRigDispatch_FromString_ErrorPipe : public FOutputDevice
	{
	public:

		TArray<FString> Errors;

		FRigDispatch_FromString_ErrorPipe()
			: FOutputDevice()
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			Errors.Add(FString::Printf(TEXT("Error convert to string: %s"), V));
		}
	};

	FRigDispatch_FromString_ErrorPipe ErrorPipe;
	ValueProperty->ImportText_Direct(*String, Value, nullptr, PPF_None, &ErrorPipe);

	if(!ErrorPipe.Errors.IsEmpty())
	{
#if WITH_EDITOR
		const FRigVMExecuteContext& ExecuteContext = InContext.GetPublicData<>();
#endif
		ValueProperty->InitializeValue(Value);
		for(const FString& Error : ErrorPipe.Errors)
		{
#if WITH_EDITOR
			if(ExecuteContext.GetLog() != nullptr)
			{
				ExecuteContext.Report(EMessageSeverity::Error, InContext.GetPublicData<>().GetFunctionName(), InContext.GetPublicData<>().GetInstructionIndex(), Error);
			}
#endif
			FString ObjectPath;
			if(InContext.GetVM())
			{
				ObjectPath = InContext.GetVM()->GetName();
			}
			
			static constexpr TCHAR ErrorLogFormat[] = TEXT("%s: [%04d] %s");
			UE_LOG(LogRigVM, Error, ErrorLogFormat, *ObjectPath, InContext.GetPublicData<>().GetInstructionIndex(), *Error);
		}
	}
}

#undef LOCTEXT_NAMESPACE
