// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_Accumulate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Accumulate)

FRigVMFunction_AccumulateFloatAdd_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}
	
	if(bIntegrateDeltaTime)
	{
		Result = AccumulatedValue = AccumulatedValue + Increment * ExecuteContext.GetDeltaTime<float>();
	}
	else
	{
		Result = AccumulatedValue = AccumulatedValue + Increment;
	}
}

FRigVMFunction_AccumulateVectorAdd_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}
	
	if (bIntegrateDeltaTime)
	{
		Result = AccumulatedValue = AccumulatedValue + Increment * ExecuteContext.GetDeltaTime();
	}
	else
	{
		Result = AccumulatedValue = AccumulatedValue + Increment;
	}
}

FRigVMFunction_AccumulateFloatMul_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	float Factor = Multiplier;
	if (bIntegrateDeltaTime)
	{
		Factor = FMath::Lerp<float>(1.f, Factor, ExecuteContext.GetDeltaTime());
	}
	Result = AccumulatedValue = AccumulatedValue * Factor;
}

FRigVMFunction_AccumulateVectorMul_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	FVector Factor = Multiplier;
	if (bIntegrateDeltaTime)
	{
		Factor = FMath::Lerp<FVector>(FVector::OneVector, Factor, ExecuteContext.GetDeltaTime());
	}
	Result = AccumulatedValue = AccumulatedValue * Factor;
}

FRigVMFunction_AccumulateQuatMul_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	FQuat Factor = Multiplier;
	if (bIntegrateDeltaTime)
	{
		Factor = FQuat::Slerp(FQuat::Identity, Factor, ExecuteContext.GetDeltaTime());
	}
	if (bFlipOrder)
	{
		Result = AccumulatedValue = Factor * AccumulatedValue;
	}
	else
	{
		Result = AccumulatedValue = AccumulatedValue * Factor;
	}
}

FRigVMFunction_AccumulateTransformMul_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	FTransform Factor = Multiplier;
	if (bIntegrateDeltaTime)
	{
		Factor.SetTranslation(FMath::Lerp<FVector>(FVector::OneVector, Factor.GetTranslation(), ExecuteContext.GetDeltaTime()));
		Factor.SetRotation(FQuat::Slerp(FQuat::Identity, Factor.GetRotation(), ExecuteContext.GetDeltaTime()));
		Factor.SetScale3D(FMath::Lerp<FVector>(FVector::OneVector, Factor.GetScale3D(), ExecuteContext.GetDeltaTime()));
	}
	if (bFlipOrder)
	{
		Result = AccumulatedValue = Factor * AccumulatedValue;
	}
	else
	{
		Result = AccumulatedValue = AccumulatedValue * Factor;
	}
}

FRigVMFunction_AccumulateFloatLerp_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	Result = AccumulatedValue = FMath::Lerp<float>(AccumulatedValue, TargetValue, FMath::Clamp<float>(bIntegrateDeltaTime ? ExecuteContext.GetDeltaTime<float>() * Blend : Blend, 0.f, 1.f));
}

FRigVMFunction_AccumulateVectorLerp_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	Result = AccumulatedValue = FMath::Lerp<FVector>(AccumulatedValue, TargetValue, FMath::Clamp<double>(bIntegrateDeltaTime ? ExecuteContext.GetDeltaTime() * Blend : Blend, 0., 1.));
}

FRigVMFunction_AccumulateQuatLerp_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	Result = AccumulatedValue = FQuat::Slerp(AccumulatedValue, TargetValue, FMath::Clamp<double>(bIntegrateDeltaTime ? ExecuteContext.GetDeltaTime() * Blend : Blend, 0., 1.));
}

FRigVMFunction_AccumulateTransformLerp_Execute()
{
	if (!bIsInitialized)
	{
		Result = AccumulatedValue = InitialValue;
		bIsInitialized = true;
	}

	double B = FMath::Clamp<double>(bIntegrateDeltaTime ? ExecuteContext.GetDeltaTime() * Blend : Blend, 0., 1.);
	AccumulatedValue.SetTranslation(FMath::Lerp<FVector>(AccumulatedValue.GetTranslation(), TargetValue.GetTranslation(), B));
	AccumulatedValue.SetRotation(FQuat::Slerp(AccumulatedValue.GetRotation(), TargetValue.GetRotation(), B));
	AccumulatedValue.SetScale3D(FMath::Lerp<FVector>(AccumulatedValue.GetScale3D(), TargetValue.GetScale3D(), B));
	Result = AccumulatedValue;
}

FRigVMFunction_AccumulateFloatRange_Execute()
{
	if (!bIsInitialized)
	{
		Minimum = AccumulatedMinimum = Maximum = AccumulatedMaximum = Value;
		bIsInitialized = true;
	}

	Minimum = AccumulatedMinimum = FMath::Min(AccumulatedMinimum, Value);
	Maximum = AccumulatedMaximum = FMath::Max(AccumulatedMaximum, Value);
}

FRigVMFunction_AccumulateVectorRange_Execute()
{
	if (!bIsInitialized)
	{
		Minimum = AccumulatedMinimum = Maximum = AccumulatedMaximum = Value;
		bIsInitialized = true;
	}

	Minimum.X = AccumulatedMinimum.X = FMath::Min(AccumulatedMinimum.X, Value.X);
	Minimum.Y = AccumulatedMinimum.Y = FMath::Min(AccumulatedMinimum.Y, Value.Y);
	Minimum.Z = AccumulatedMinimum.Z = FMath::Min(AccumulatedMinimum.Z, Value.Z);
	Maximum.X = AccumulatedMaximum.X = FMath::Max(AccumulatedMaximum.X, Value.X);
	Maximum.Y = AccumulatedMaximum.Y = FMath::Max(AccumulatedMaximum.Y, Value.Y);
	Maximum.Z = AccumulatedMaximum.Z = FMath::Max(AccumulatedMaximum.Z, Value.Z);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateFloatAdd)
{
	ExecuteContext.SetDeltaTime(0.5f);

	Unit.Increment = 1.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected accumulate result"));
	Unit.bIntegrateDeltaTime = true;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.5f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateVectorAdd)
{
	using TFloat = decltype(Unit.Result.X);
	ExecuteContext.SetDeltaTime(0.5f);

	Unit.Increment = FVector(1.0f, 0.f, 0.f);
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)1.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)1.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)2.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)3.f), TEXT("unexpected accumulate result"));
	Unit.bIntegrateDeltaTime = true;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)3.5f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateFloatMul)
{
	Unit.Multiplier = 2.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 8.f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateVectorMul)
{
	using TFloat = decltype(Unit.Result.X);
	Unit.Multiplier = FVector(2.f, 2.f, 2.f);
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)2.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)2.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)4.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)8.f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateFloatLerp)
{
	Unit.InitialValue = 0.f;
	Unit.TargetValue = 8.f;
	Unit.Blend = 0.5f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 6.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 7.f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateVectorLerp)
{
	using TFloat = decltype(Unit.Result.X);
	Unit.InitialValue = FVector::ZeroVector;
	Unit.TargetValue = FVector(8.f, 0.f, 0.f);
	Unit.Blend = 0.5f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)4.f), TEXT("unexpected accumulate result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)4.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)6.f), TEXT("unexpected accumulate result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result.X, (TFloat)7.f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateFloatRange)
{
	Unit.Value = 4.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum, 4.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum, 4.f), TEXT("unexpected accumulate result"));
	Unit.Value = 5.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum, 5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum, 5.f), TEXT("unexpected accumulate result"));
	Unit.Value = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum, 3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum, 5.f), TEXT("unexpected accumulate result"));
	Unit.Value = 7.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum, 3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum, 7.f), TEXT("unexpected accumulate result"));
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum, 2.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum, 7.f), TEXT("unexpected accumulate result"));
	return true;
}

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_AccumulateVectorRange)
{
	using TFloat = decltype(Unit.Minimum.X);
	Unit.Value = FVector(3.f, 4.f, 5.f);
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.X, (TFloat)3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Y, (TFloat)4.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Z, (TFloat)5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.X, (TFloat)3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Y, (TFloat)4.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Z, (TFloat)5.f), TEXT("unexpected accumulate result"));
	Unit.Value = FVector(5.f, 6.f, 7.f);
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.X, (TFloat)5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Y, (TFloat)6.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Z, (TFloat)7.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.X, (TFloat)5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Y, (TFloat)6.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Z, (TFloat)7.f), TEXT("unexpected accumulate result"));
	Unit.Value = FVector(1.f, 2.f, 3.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.X, (TFloat)1.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Y, (TFloat)2.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Z, (TFloat)3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.X, (TFloat)5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Y, (TFloat)6.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Z, (TFloat)7.f), TEXT("unexpected accumulate result"));
	Unit.Value = FVector(1.f, 12.f, 13.f);
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.X, (TFloat)1.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Y, (TFloat)2.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Minimum.Z, (TFloat)3.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.X, (TFloat)5.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Y, (TFloat)12.f), TEXT("unexpected accumulate result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Maximum.Z, (TFloat)13.f), TEXT("unexpected accumulate result"));
	return true;
}

#endif
