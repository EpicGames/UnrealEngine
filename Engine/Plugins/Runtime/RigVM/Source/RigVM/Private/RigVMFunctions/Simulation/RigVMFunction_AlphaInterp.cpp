// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_AlphaInterp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_AlphaInterp)

FRigVMFunction_AlphaInterp_Execute()
{
	ScaleBiasClamp.bMapRange = bMapRange;
	ScaleBiasClamp.bClampResult = bClampResult;
	ScaleBiasClamp.bInterpResult = bInterpResult;

	ScaleBiasClamp.InRange = InRange;
	ScaleBiasClamp.OutRange = OutRange;
	ScaleBiasClamp.ClampMin = ClampMin;
	ScaleBiasClamp.ClampMax = ClampMax;
	ScaleBiasClamp.Scale = Scale;
	ScaleBiasClamp.Bias = Bias;
	ScaleBiasClamp.InterpSpeedIncreasing = InterpSpeedIncreasing;
	ScaleBiasClamp.InterpSpeedDecreasing = InterpSpeedDecreasing;

	Result = ScaleBiasClamp.ApplyTo(Value, ExecuteContext.GetDeltaTime<float>());
}

FString FRigVMFunction_AlphaInterp::ProcessPinLabelForInjection(const FString& InLabel) const
{
	FString Formula;
	if (bMapRange)
	{
		Formula += FString::Printf(TEXT(" Map(%.02f, %.02f, %.02f, %.02f)"), InRange.Min, InRange.Max, OutRange.Min, OutRange.Max);
	}
	if (bInterpResult)
	{
		Formula += FString::Printf(TEXT(" Interp(%.02f, %.02f)"), InterpSpeedIncreasing, InterpSpeedDecreasing);
	}
	if (bClampResult)
	{
		Formula += FString::Printf(TEXT(" Clamp(%.02f, %.02f)"), ClampMin, ClampMax);
	}

	if (Formula.IsEmpty())
	{
		return InLabel;
	}
	return FString::Printf(TEXT("%s: %s"), *InLabel, *Formula);
}

FRigVMFunction_AlphaInterpVector_Execute()
{
	ScaleBiasClamp.bMapRange = bMapRange;
	ScaleBiasClamp.bClampResult = bClampResult;
	ScaleBiasClamp.bInterpResult = bInterpResult;

	ScaleBiasClamp.InRange = InRange;
	ScaleBiasClamp.OutRange = OutRange;
	ScaleBiasClamp.ClampMin = ClampMin;
	ScaleBiasClamp.ClampMax = ClampMax;
	ScaleBiasClamp.Scale = Scale;
	ScaleBiasClamp.Bias = Bias;
	ScaleBiasClamp.InterpSpeedIncreasing = InterpSpeedIncreasing;
	ScaleBiasClamp.InterpSpeedDecreasing = InterpSpeedDecreasing;

	Result.X = ScaleBiasClamp.ApplyTo(static_cast<float>(Value.X), ExecuteContext.GetDeltaTime<float>());
	Result.Y = ScaleBiasClamp.ApplyTo(static_cast<float>(Value.Y), ExecuteContext.GetDeltaTime<float>());
	Result.Z = ScaleBiasClamp.ApplyTo(static_cast<float>(Value.Z), ExecuteContext.GetDeltaTime<float>());
}

FString FRigVMFunction_AlphaInterpVector::ProcessPinLabelForInjection(const FString& InLabel) const
{
	FString Formula;
	if (bMapRange)
	{
		Formula += FString::Printf(TEXT(" Map(%.02f, %.02f, %.02f, %.02f)"), InRange.Min, InRange.Max, OutRange.Min, OutRange.Max);
	}
	if (bInterpResult)
	{
		Formula += FString::Printf(TEXT(" Interp(%.02f, %.02f)"), InterpSpeedIncreasing, InterpSpeedDecreasing);
	}
	if (bClampResult)
	{
		Formula += FString::Printf(TEXT(" Clamp(%.02f, %.02f)"), ClampMin, ClampMax);
	}

	if (Formula.IsEmpty())
	{
		return InLabel;
	}
	return FString::Printf(TEXT("%s: %s"), *InLabel, *Formula);
}

FRigVMFunction_AlphaInterpQuat_Execute()
{
	ScaleBiasClamp.bMapRange = bMapRange;
	ScaleBiasClamp.bClampResult = bClampResult;
	ScaleBiasClamp.bInterpResult = bInterpResult;

	ScaleBiasClamp.InRange = InRange;
	ScaleBiasClamp.OutRange = OutRange;
	ScaleBiasClamp.ClampMin = ClampMin;
	ScaleBiasClamp.ClampMax = ClampMax;
	ScaleBiasClamp.Scale = Scale;
	ScaleBiasClamp.Bias = Bias;
	ScaleBiasClamp.InterpSpeedIncreasing = InterpSpeedIncreasing;
	ScaleBiasClamp.InterpSpeedDecreasing = InterpSpeedDecreasing;

	const double T = ScaleBiasClamp.ApplyTo(1.f, ExecuteContext.GetDeltaTime<float>());
	Result = FQuat::Slerp(FQuat::Identity, Value, T);
}

FString FRigVMFunction_AlphaInterpQuat::ProcessPinLabelForInjection(const FString& InLabel) const
{
	FString Formula;
	if (bMapRange)
	{
		Formula += FString::Printf(TEXT(" Map(%.02f, %.02f, %.02f, %.02f)"), InRange.Min, InRange.Max, OutRange.Min, OutRange.Max);
	}
	if (bInterpResult)
	{
		Formula += FString::Printf(TEXT(" Interp(%.02f, %.02f)"), InterpSpeedIncreasing, InterpSpeedDecreasing);
	}
	if (bClampResult)
	{
		Formula += FString::Printf(TEXT(" Clamp(%.02f, %.02f)"), ClampMin, ClampMax);
	}

	if (Formula.IsEmpty())
	{
		return InLabel;
	}
	return FString::Printf(TEXT("%s: %s"), *InLabel, *Formula);
}
