// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_DeltaFromPrevious.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_DeltaFromPrevious)

FRigVMFunction_DeltaFromPreviousFloat_Execute()
{
	if (!bIsInitialized)
	{
		Delta = 0.f;
		PreviousValue = Cache = Value;
		bIsInitialized = true;
	}

	PreviousValue = Cache;
	Delta = Cache - Value;

	if (FMath::Abs(ExecuteContext.GetDeltaTime()) > SMALL_NUMBER)
	{
		Cache = Value;
	}
}

FRigVMFunction_DeltaFromPreviousVector_Execute()
{
	if (!bIsInitialized)
	{
		Delta = FVector::ZeroVector;
		PreviousValue = Cache = Value;
		bIsInitialized = true;
	}

	PreviousValue = Cache;
	Delta = Cache - Value;

	if (FMath::Abs(ExecuteContext.GetDeltaTime()) > SMALL_NUMBER)
	{
		Cache = Value;
	}
}

FRigVMFunction_DeltaFromPreviousQuat_Execute()
{
	if (!bIsInitialized)
	{
		Delta = FQuat::Identity;
		PreviousValue = Cache = Value;
		bIsInitialized = true;
	}

	PreviousValue = Cache;
	Delta = Cache.Inverse() * Value;

	if (FMath::Abs(ExecuteContext.GetDeltaTime()) > SMALL_NUMBER)
	{
		Cache = Value;
	}
}

FRigVMFunction_DeltaFromPreviousTransform_Execute()
{
	if (!bIsInitialized)
	{
		Delta = FTransform::Identity;
		PreviousValue = Cache = Value;
		bIsInitialized = true;
	}

	PreviousValue = Cache;
	Delta = Value.GetRelativeTransform(Cache);

	if (FMath::Abs(ExecuteContext.GetDeltaTime()) > SMALL_NUMBER)
	{
		Cache = Value;
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_DeltaFromPreviousFloat)
{
	ExecuteContext.SetDeltaTime(0.1f);
	Unit.Value = 1.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Delta, 0.f), TEXT("unexpected average result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.PreviousValue, 1.f), TEXT("unexpected average result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Delta, 0.f), TEXT("unexpected average result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.PreviousValue, 1.f), TEXT("unexpected average result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Delta, 0.f), TEXT("unexpected average result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.PreviousValue, 1.f), TEXT("unexpected average result"));
	Unit.Value = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Delta, -1.f), TEXT("unexpected average result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.PreviousValue, 1.f), TEXT("unexpected average result"));
	Unit.Value = 5.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Delta, -3.f), TEXT("unexpected average result"));
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.PreviousValue, 2.f), TEXT("unexpected average result"));
	return true;
}

#endif
