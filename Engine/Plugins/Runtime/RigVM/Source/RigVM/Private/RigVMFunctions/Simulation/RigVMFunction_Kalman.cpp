// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_Kalman.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Kalman)

FRigVMFunction_KalmanFloat_Execute()
{
	if(BufferSize <= 0)
	{
		if (Buffer.IsEmpty()) // only report this error once
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}

		Result = Value;
		return;
	}

	if(Buffer.Num() != BufferSize)
	{
		Buffer.SetNum(BufferSize);

		for (float& Element : Buffer)
		{
			Element = FLT_MAX;
			LastInsertIndex = -1;
		}
	}

	if(LastInsertIndex == Buffer.Num() - 1)
	{
		LastInsertIndex = -1;
	}
	Buffer[++LastInsertIndex] = Value;

	Result = 0.f;
	int32 NumberValidEntries = 0;
	for(const float F : Buffer)
	{
		if (F == FLT_MAX)
		{
			break;
		}
		Result += F;
		NumberValidEntries++;
	}

	if (NumberValidEntries > 0)
	{
		Result = Result / float(NumberValidEntries);
	}
}

FRigVMFunction_KalmanVector_Execute()
{
	if(BufferSize <= 0)
	{
		if (Buffer.IsEmpty()) // only report this error once
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}

		Result = Value;
		return;
	}

	if(Buffer.Num() != BufferSize)
	{
		Buffer.SetNum(BufferSize);

		for (FVector& Element : Buffer)
		{
			Element.X = FLT_MAX;
			LastInsertIndex = -1;
		}
	}

	if(LastInsertIndex == Buffer.Num() - 1)
	{
		LastInsertIndex = -1;
	}
	Buffer[++LastInsertIndex] = Value;

	Result = FVector::ZeroVector;
	int32 NumberValidEntries = 0;
	for (const FVector& F : Buffer)
	{
		if (F.X == FLT_MAX)
		{
			break;
		}
		Result += F;
		NumberValidEntries++;
	}

	if (NumberValidEntries > 0)
	{
		Result = Result / float(NumberValidEntries);
	}
}

FRigVMFunction_KalmanTransform_Execute()
{
	if(BufferSize <= 0)
	{
		if (Buffer.IsEmpty()) // only report this error once
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}

		Result = Value;
		return;
	}

	if(Buffer.Num() != BufferSize)
	{
		Buffer.SetNum(BufferSize);

		for (FTransform& Element : Buffer)
		{
			Element.SetTranslation(FVector(FLT_MAX));
			LastInsertIndex = -1;
		}
	}

	if(LastInsertIndex == Buffer.Num() - 1)
	{
		LastInsertIndex = -1;
	}
	Buffer[++LastInsertIndex] = Value;

	FVector Location = FVector::ZeroVector;
	FVector AxisX = FVector::ZeroVector;
	FVector AxisY = FVector::ZeroVector;
	FVector Scale = FVector::ZeroVector;
	
	int32 NumberValidEntries = 0;

	for(const FTransform& F : Buffer)
	{
		if (F.GetLocation().X == FLT_MAX)
		{
			break;
		}
		Location += F.GetLocation();
		AxisX += F.TransformVectorNoScale(FVector(1.f, 0.f, 0.f));
		AxisY += F.TransformVectorNoScale(FVector(0.f, 1.f, 0.f));
		Scale += F.GetScale3D();
		NumberValidEntries++;
	}

	if (NumberValidEntries > 0)
	{
		Location = Location / float(NumberValidEntries);
		AxisX = (AxisX / float(NumberValidEntries)).GetSafeNormal();
		AxisY = (AxisY / float(NumberValidEntries)).GetSafeNormal();
		Scale = Scale / float(NumberValidEntries);
	}

	Result.SetLocation(Location);
	Result.SetRotation(FRotationMatrix::MakeFromXY(AxisX, AxisY).ToQuat());
	Result.SetScale3D(Scale);
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_KalmanFloat)
{
	Unit.Value = 1.f;
	Unit.BufferSize = 4;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected average result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected average result"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 1.f), TEXT("unexpected average result"));
	Unit.Value = 4.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected average result"));
	Unit.Value = 6.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected average result"));
	Unit.Value = 5.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected average result"));
	return true;
}

#endif
