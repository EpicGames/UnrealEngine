// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_TimeOffset.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_TimeOffset)

FRigVMFunction_TimeOffsetFloat_Execute()
{
	Result = Value;

	const bool bIsInitialized = !Buffer.IsEmpty();

	if(BufferSize <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}
		Result = Value;
		return;
	}

	if (TimeRange <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("TimeRange is too small."));
		}
		Result = Value;
		return;
	}

	if(!bIsInitialized)
	{
		UpperBound = 0;
	}

	int32 MaxSize = FMath::Clamp<int32>(BufferSize, 2, 512);
	Buffer.SetNum(MaxSize);
	DeltaTimes.SetNum(MaxSize);
	MaxSize = FMath::Min<int32>(MaxSize, Buffer.Num());

	if (UpperBound == 0)
	{
		Result = Value;
	}
	else
	{
		float A = Value;
		float B = Value;
		float T = 0.f;

		int32 Index = LastInsertIndex;
		if (SecondsAgo < DeltaTimes[Index] || UpperBound == 1)
		{
			B = Buffer[Index];
			T = SecondsAgo / DeltaTimes[Index];
		}
		else
		{
			float AccumulatedTime = 0.f;
			for (int32 TimeIndex = 0; TimeIndex < UpperBound; TimeIndex++)
			{
				B = Buffer[Index];

				if (AccumulatedTime >= SecondsAgo)
				{
					T = 1.f - (AccumulatedTime - SecondsAgo) / DeltaTimes[Index];
					break;
				}
				AccumulatedTime = AccumulatedTime + DeltaTimes[Index];;
				A = B;

				Index = (Index - 1 + UpperBound) % UpperBound;
			}
		}

		Result = FMath::Lerp<float>(A, B, T);
	}

	if (ExecuteContext.GetDeltaTime() > UE_SMALL_NUMBER)
	{
		if (UpperBound == 0)
		{
			LastInsertIndex = UpperBound;
			Buffer[UpperBound] = Value;
			DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
		}
		else
		{
			const float SecondsPerEntry = TimeRange / static_cast<float>(MaxSize - 1);
			if (DeltaTimes[LastInsertIndex] > (SecondsPerEntry - ExecuteContext.GetDeltaTime<float>() * 0.5f))
			{
				if (UpperBound < MaxSize)
				{
					LastInsertIndex = UpperBound;
					Buffer[UpperBound] = Value;
					DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
				}
				else
				{
					LastInsertIndex = (LastInsertIndex + 1) % UpperBound;
					Buffer[LastInsertIndex] = Value;
					DeltaTimes[LastInsertIndex] = ExecuteContext.GetDeltaTime<float>();
				}
			}
			else
			{
				DeltaTimes[LastInsertIndex] += ExecuteContext.GetDeltaTime<float>();
			}
		}
	}
}

FRigVMFunction_TimeOffsetVector_Execute()
{
	Result = Value;

	const bool bIsInitialized = !Buffer.IsEmpty();

	if(BufferSize <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}
		Result = Value;
		return;
	}

	if (TimeRange <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("TimeRange is too small."));
		}
		Result = Value;
		return;
	}

	if(!bIsInitialized)
	{
		UpperBound = 0;
	}

	int32 MaxSize = FMath::Clamp<int32>(BufferSize, 2, 512);
	Buffer.SetNum(MaxSize);
	DeltaTimes.SetNum(MaxSize);
	MaxSize = FMath::Min<int32>(MaxSize, Buffer.Num());

	if (UpperBound == 0)
	{
		Result = Value;
	}
	else
	{
		FVector A = Value;
		FVector B = Value;
		float T = 0.f;

		int32 Index = LastInsertIndex;
		if (SecondsAgo < DeltaTimes[Index] || UpperBound == 1)
		{
			B = Buffer[Index];
			T = SecondsAgo / DeltaTimes[Index];
		}
		else
		{
			float AccumulatedTime = 0.f;
			for (int32 TimeIndex = 0; TimeIndex < UpperBound; TimeIndex++)
			{
				B = Buffer[Index];

				if (AccumulatedTime >= SecondsAgo)
				{
					T = 1.f - (AccumulatedTime - SecondsAgo) / DeltaTimes[Index];
					break;
				}
				AccumulatedTime = AccumulatedTime + DeltaTimes[Index];;
				A = B;

				Index = (Index - 1 + UpperBound) % UpperBound;
			}
		}

		Result = FMath::Lerp<FVector>(A, B, T);
	}

	if (ExecuteContext.GetDeltaTime() > UE_SMALL_NUMBER)
	{
		if (UpperBound == 0)
		{
			LastInsertIndex = UpperBound;
			Buffer[UpperBound] = Value;
			DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
		}
		else
		{
			const float SecondsPerEntry = TimeRange / static_cast<float>(MaxSize - 1);
			if (DeltaTimes[LastInsertIndex] > (SecondsPerEntry - ExecuteContext.GetDeltaTime<float>() * 0.5f))
			{
				if (UpperBound < MaxSize)
				{
					LastInsertIndex = UpperBound;
					Buffer[UpperBound] = Value;
					DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
				}
				else
				{
					LastInsertIndex = (LastInsertIndex + 1) % UpperBound;
					Buffer[LastInsertIndex] = Value;
					DeltaTimes[LastInsertIndex] = ExecuteContext.GetDeltaTime<float>();
				}
			}
			else
			{
				DeltaTimes[LastInsertIndex] += ExecuteContext.GetDeltaTime<float>();
			}
		}
	}
}

FRigVMFunction_TimeOffsetTransform_Execute()
{
	Result = Value;

	const bool bIsInitialized = !Buffer.IsEmpty();

	if(BufferSize <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("BufferSize is too small."));
		}
		Result = Value;
		return;
	}

	if (TimeRange <= 0)
	{
		if (!bIsInitialized)
		{
			UE_RIGVMSTRUCT_REPORT_WARNING(TEXT("TimeRange is too small."));
		}
		Result = Value;
		return;
	}

	if(!bIsInitialized)
	{
		UpperBound = 0;
	}

	int32 MaxSize = FMath::Clamp<int32>(BufferSize, 2, 512);
	Buffer.SetNum(MaxSize);
	DeltaTimes.SetNum(MaxSize);
	MaxSize = FMath::Min<int32>(MaxSize, Buffer.Num());

	if (UpperBound == 0)
	{
		Result = Value;
	}
	else
	{
		FTransform A = Value;
		FTransform B = Value;
		float T = 0.f;

		int32 Index = LastInsertIndex;
		if (SecondsAgo < DeltaTimes[Index] || UpperBound == 1)
		{
			B = Buffer[Index];
			T = SecondsAgo / DeltaTimes[Index];
		}
		else
		{
			float AccumulatedTime = 0.f;
			for (int32 TimeIndex = 0; TimeIndex < UpperBound; TimeIndex++)
			{
				B = Buffer[Index];

				if (AccumulatedTime >= SecondsAgo)
				{
					T = 1.f - (AccumulatedTime - SecondsAgo) / DeltaTimes[Index];
					break;
				}
				AccumulatedTime = AccumulatedTime + DeltaTimes[Index];;
				A = B;

				Index = (Index - 1 + UpperBound) % UpperBound;
			}
		}

		Result.SetTranslation(FMath::Lerp<FVector>(A.GetTranslation(), B.GetTranslation(), T));
		Result.SetRotation(FQuat::Slerp(A.GetRotation(), B.GetRotation(), T));
		Result.SetScale3D(FMath::Lerp<FVector>(A.GetScale3D(), B.GetScale3D(), T));
	}

	if (ExecuteContext.GetDeltaTime() > UE_SMALL_NUMBER)
	{
		if (UpperBound == 0)
		{
			LastInsertIndex = UpperBound;
			Buffer[UpperBound] = Value;
			DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
		}
		else
		{
			const float SecondsPerEntry = TimeRange / static_cast<float>(MaxSize - 1);
			if (DeltaTimes[LastInsertIndex] > SecondsPerEntry - ExecuteContext.GetDeltaTime<float>() * 0.5f)
			{
				if (UpperBound < MaxSize)
				{
					LastInsertIndex = UpperBound;
					Buffer[UpperBound] = Value;
					DeltaTimes[UpperBound++] = ExecuteContext.GetDeltaTime<float>();
				}
				else
				{
					LastInsertIndex = (LastInsertIndex + 1) % UpperBound;
					Buffer[LastInsertIndex] = Value;
					DeltaTimes[LastInsertIndex] = ExecuteContext.GetDeltaTime<float>();
				}
			}
			else
			{
				DeltaTimes[LastInsertIndex] += ExecuteContext.GetDeltaTime<float>();
			}
		}
	}
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_TimeOffsetFloat)
{
	ExecuteContext.SetDeltaTime(1.f);
	Unit.SecondsAgo = 0.5;
	Unit.TimeRange = 5.f;
	Unit.BufferSize = 16;
	
	Unit.Value = 2.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected previous result"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected previous result"));
	Unit.Value = 4.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected previous result"));
	Unit.Value = 6.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 5.f), TEXT("unexpected previous result"));
	Unit.Value = 8.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 7.f), TEXT("unexpected previous result"));
	Unit.Value = 10.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 9.f), TEXT("unexpected previous result"));
	Unit.Value = 12.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 11.f), TEXT("unexpected previous result"));

	ExecuteContext.SetDeltaTime(0.f);
	Unit.SecondsAgo = 1.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 9.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 2.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 7.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 3.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 5.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 4.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected previous result"));

	ExecuteContext.SetDeltaTime(1.f);
	Unit.SecondsAgo = 0.5;
	Unit.Value = 14.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 13.f), TEXT("unexpected previous result"));
	Unit.Value = 16.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 15.f), TEXT("unexpected previous result"));
	Unit.Value = 18.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 17.f), TEXT("unexpected previous result"));
	Unit.Value = 20.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 19.f), TEXT("unexpected previous result"));
	Unit.Value = 22.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 21.f), TEXT("unexpected previous result"));
	Unit.Value = 24.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 23.f), TEXT("unexpected previous result"));
	Unit.Value = 26.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 25.f), TEXT("unexpected previous result"));
	Unit.Value = 28.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 27.f), TEXT("unexpected previous result"));
	Unit.Value = 30.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 29.f), TEXT("unexpected previous result"));
	Unit.Value = 32.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 31.f), TEXT("unexpected previous result"));
	Unit.Value = 34.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 33.f), TEXT("unexpected previous result"));
	Unit.Value = 36.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 35.f), TEXT("unexpected previous result"));

	ExecuteContext.SetDeltaTime(0.f);
	Unit.SecondsAgo = 1.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 33.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 2.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 31.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 3.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 29.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 4.5;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 27.f), TEXT("unexpected previous result"));


	ExecuteContext.SetDeltaTime(1.f);
	Unit.TimeRange = 8.f;
	Unit.BufferSize = 3;
	Unit.SecondsAgo = 0.5f;

	Unit.Value = 2.f;
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected previous result"));
	Unit.Value = 4.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.f), TEXT("unexpected previous result"));
	Unit.Value = 6.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 5.f), TEXT("unexpected previous result"));

	ExecuteContext.SetDeltaTime(0.f);
	Unit.SecondsAgo = 1.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 4.f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 2.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 3.33333f, 0.001f), TEXT("unexpected previous result"));
	Unit.SecondsAgo = 3.f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Result, 2.f), TEXT("unexpected previous result"));

	return true;
}

#endif
