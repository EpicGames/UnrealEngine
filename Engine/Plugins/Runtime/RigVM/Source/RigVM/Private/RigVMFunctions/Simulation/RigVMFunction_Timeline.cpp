// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_Timeline.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Timeline)

FRigVMFunction_Timeline_Execute()
{
	if (!bIsInitialized)
	{
		Time = AccumulatedValue = 0.f;
		bIsInitialized = true;
	}

	Time = AccumulatedValue = AccumulatedValue + ExecuteContext.GetDeltaTime<float>() * Speed;
}

#if WITH_DEV_AUTOMATION_TESTS
#include "RigVMCore/RigVMStructTest.h"

IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(FRigVMFunction_Timeline)
{
	ExecuteContext.SetDeltaTime(1.f);
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Time, 1.f), TEXT("unexpected time"));
	InitAndExecute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Time, 1.f), TEXT("unexpected time"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Time, 2.f), TEXT("unexpected time"));
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Time, 3.f), TEXT("unexpected time"));
	Unit.Speed = 0.5f;
	Execute();
	AddErrorIfFalse(FMath::IsNearlyEqual(Unit.Time, 3.5f), TEXT("unexpected time"));
	return true;
}

#endif

FRigVMFunction_TimeLoop_Execute()
{
	if (!bIsInitialized)
	{
		Absolute = Relative = FlipFlop = AccumulatedAbsolute = AccumulatedRelative = 0.f;
		NumIterations = 0;
		Even = false;
		bIsInitialized = true;
	}

	const float DurationClamped = FMath::Max(Duration, 0.0001f);
	const float Increment = ExecuteContext.GetDeltaTime<float>() * Speed;
	Absolute = AccumulatedAbsolute = AccumulatedAbsolute + Increment;

	AccumulatedRelative = AccumulatedRelative + Increment;
	while(AccumulatedRelative > DurationClamped)
	{
		AccumulatedRelative -= DurationClamped;
		NumIterations++;
	}

	Relative = AccumulatedRelative;

	Even = (NumIterations & 1) == 0;
	if(Even) // check is this is even or odd
	{
		FlipFlop = Relative;
	}
	else
	{
		FlipFlop = Duration - Relative;
	}

	if(Normalize)
	{
		Relative = Relative / DurationClamped;
		FlipFlop = FlipFlop / DurationClamped;
	}
}

FRigVMFunction_TimerValue_Execute()
{
	if (!bIsInitialized || Reset)
	{
		AccumulatedValue = 0.f;
		bIsInitialized = true;
		IsRunning = true;
	}

	const float AbsoluteDuration = FMath::Abs(Duration);
	if(!IsRunning || AbsoluteDuration < SMALL_NUMBER)
	{
		JustStarted = JustFinished = false;
	}
	else if(IsRunning)
	{
		JustStarted = AccumulatedValue < SMALL_NUMBER;
		IsRunning = AccumulatedValue < AbsoluteDuration + SMALL_NUMBER;

		AccumulatedValue = AccumulatedValue + ExecuteContext.GetDeltaTime<float>() * Speed;

		JustFinished = AccumulatedValue > AbsoluteDuration - SMALL_NUMBER;
		if(JustFinished)
		{
			IsRunning = false;
		}
	}

	Time = FMath::Clamp(AccumulatedValue, 0.f, AbsoluteDuration);
	if (AbsoluteDuration < SMALL_NUMBER)
	{
		Ratio = 0.f;
	}
	else
	{
		Ratio = FMath::Clamp(Time / AbsoluteDuration, 0.f, 1.f);
	}

	if(JustFinished)
	{
		if(Repeats)
		{
			AccumulatedValue = 0.f;
			IsRunning = true;
		}
		else
		{
			IsRunning = false;
		}
	}
}