// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMFunctions/Simulation/RigVMFunction_Verlet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunction_Verlet)

FRigVMFunction_VerletIntegrateVector_Execute()
{
	if (!bInitialized)
	{
		Point.Mass = 1.f;
		Position = Point.Position = Target;
		Velocity = Acceleration = Point.LinearVelocity = FVector::ZeroVector;
		bInitialized = true;
	}

	Point.LinearDamping = Damp;
	if (ExecuteContext.GetDeltaTime() > UE_SMALL_NUMBER)
	{
		float U = FMath::Clamp<float>(Blend * ExecuteContext.GetDeltaTime<float>(), 0.f, 1.f);
		const FVector CombinedForce = (Target - Point.Position) * FMath::Max(Strength, 0.0001f) + Force * ExecuteContext.GetDeltaTime() * 60.f;
		const FVector PreviousVelocity = Point.LinearVelocity;
		Point = Point.IntegrateVerlet(CombinedForce, Blend, ExecuteContext.GetDeltaTime<float>());
		Acceleration = (Point.LinearVelocity - PreviousVelocity) / ExecuteContext.GetDeltaTime<float>();
		Position = Point.Position;
		Velocity = Point.LinearVelocity;
	}
}

