// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMHost.h"
#include "Engine/UserDefinedEnum.h"
#include "ObjectTrace.h"
#include "RigVMCore/RigVMNativized.h"
#include "RigVMObjectVersion.h"
#include "RigVMTypeUtils.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ObjectSaveContext.h"
#include "SceneView.h"
#include "CanvasItem.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Misc/ScopeLock.h"

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#endif// WITH_EDITOR

#include "RigVMRuntimeAsset.h"
#include "Engine/BlueprintGeneratedClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMHost)

#define LOCTEXT_NAMESPACE "RigVMHost"

// CVar to disable all rigvm execution 
static TAutoConsoleVariable<int32> CVarRigVMDisableExecutionAll(TEXT("RigVM.DisableExecutionAll"), 0, TEXT("if nonzero we disable all execution of rigvms."));

// CVar to enable swapping to nativized vms 
static TAutoConsoleVariable<int32> CVarRigVMEnableNativizedVMs(TEXT("RigVM.EnableNativizedVMs"), 0, TEXT("if nonzero we enable swapping to nativized VMs."));

URigVMHost::URigVMHost(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DeltaTime(0.0f)
	, AbsoluteTime(0.0f)
	, FramesPerSecond(0.0f)
	, bAccumulateTime(true)
#if WITH_EDITOR
	, bIsBeingDebugged(false)
	, bEnableNativizedVMs(true)
#endif
#if RIGVM_TRACE_ENABLED
	, bIsPlayingRewindDebugTrace(false)
#endif
#if WITH_EDITOR
	, RigVMLog(nullptr)
	, bEnableLogging(true)
#endif
	, EventQueue()
	, EventQueueToRun()
	, EventsToRunOnce()
	, EvaluationsLeft(INDEX_NONE)
	, bRequiresInitExecution(false)
	, InitBracket(0)
	, ExecuteBracket(0)
#if WITH_EDITOR
	, DebugInfo(MakeShared<FRigVMDebugInfo>())
#endif
#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
	, ProfilingRunsLeft(0)
	, AccumulatedCycles(0)
#endif
{
}
 
TArray<URigVMHost*> URigVMHost::FindRigVMHosts(UObject* Outer, TSubclassOf<URigVMHost> OptionalClass)
{
	TArray<URigVMHost*> Result;
	
	if(Outer == nullptr)
	{
		return Result; 
	}
	
	const AActor* OuterActor = Cast<AActor>(Outer);
	if(OuterActor == nullptr)
	{
		OuterActor = Outer->GetTypedOuter<AActor>();
	}
	
	for (TObjectIterator<URigVMHost> Itr; Itr; ++Itr)
	{
		URigVMHost* RigInstance = *Itr;
		if (!RigInstance)
		{
			continue;
		}
		
		const UClass* RigInstanceClass = RigInstance->GetClass();
		if (OptionalClass == nullptr || (RigInstanceClass && RigInstanceClass->IsChildOf(OptionalClass)))
		{
			if(RigInstance->IsInOuter(Outer))
			{
				Result.Add(RigInstance);
				continue;
			}

			if(OuterActor)
			{
				if(RigInstance->IsInOuter(OuterActor))
				{
					Result.Add(RigInstance);
					continue;
				}
			}
		}
	}

	return Result;
}

bool URigVMHost::IsGarbageOrDestroyed(const UObject* InObject)
{
	if(!IsValid(InObject))
	{
		return true;
	}
	return InObject->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed) ||
		InObject->HasAnyInternalFlags(EInternalObjectFlags::Garbage);
}

UWorld* URigVMHost::GetWorld() const
{
	if (const UObject* Outer = GetOuter())
	{
		return Outer->GetWorld();
	}
	return nullptr;
}

void URigVMHost::Serialize(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("URigVMHost(%s)"), *GetName()));

	Super::Serialize(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

	// advertise dependencies on user defined structs and user defined enums
	// to make sure they are loaded prior to the VM.
	if (Ar.IsObjectReferenceCollector() && VM != nullptr)
	{
		const TArray<const UObject*> UserDefinedDependencies = VM->GetUserDefinedDependencies({ GetDefaultMemoryByType(ERigVMMemoryType::Literal), GetDefaultMemoryByType(ERigVMMemoryType::Work) });
		for (const UObject* UserDefinedDependency : UserDefinedDependencies)
		{
			if (Cast<UUserDefinedStruct>(UserDefinedDependency) ||
				Cast<UUserDefinedEnum>(UserDefinedDependency))
			{
				FSoftObjectPath PathToTypeObject(UserDefinedDependency);
				PathToTypeObject.Serialize(Ar);
			}
		}
	}

	if (Ar.IsLoading())
	{
		RecreateCachedMemory();
	}
}

void URigVMHost::PostLoad()
{
	Super::PostLoad();
	
	FRigVMRegistry_RWLock::Get().RefreshEngineTypesIfRequired();
	
	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	ExtendedExecuteContext.InvalidateCachedMemory();

	// In packaged builds, initialize the CDO VM
	// In editor, the VM will be recompiled and initialized at URigVMBlueprint::HandlePackageDone::RecompileVM
#if WITH_EDITOR
	if(GetPackage()->bIsCookedForEditor)
#endif
	{
		if (VM != nullptr)
		{
			// Ensure the VM has been fully loaded as we depend on it
			VM->ConditionalPostLoad();

			if (HasAnyFlags(RF_ClassDefaultObject))
			{
				InitializeCDOVM();
			}

			if (!ensure(VM->ValidateBytecode()))
			{
				UE_LOGF(LogRigVM, Warning, "%ls: Invalid bytecode detected. VM will be reset.", *GetPathName());
				VM->Reset(ExtendedExecuteContext);
			}
		}
	}
}

void URigVMHost::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	GenerateUserDefinedDependenciesData(GetRigVMExtendedExecuteContext());
	GenerateRequiredPluginsData(GetRigVMExtendedExecuteContext());
}

void URigVMHost::BeginDestroy()
{
	if (GeneratedBy)
	{
		GeneratedBy->RemoveInstance(this);
	}
	
	Super::BeginDestroy();

	InitializedEvent.Clear();
	ExecutedEvent.Clear();

	TRACE_OBJECT_LIFETIME_END(this);
}

// The duplicate/import paths copy GeneratedBy via UPROPERTY but skip the asset's
// InstantiateObject entry, so VM/Variables stay un-rebound until we re-run InitializeInstance.
void URigVMHost::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (GeneratedBy)
	{
		GeneratedBy->InitializeInstance(this);
	}
}

#if WITH_EDITOR
void URigVMHost::PostEditImport()
{
	Super::PostEditImport();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (GeneratedBy)
	{
		GeneratedBy->InitializeInstance(this);
	}
}
#endif

void URigVMHost::SetDeltaTime(float InDeltaTime)
{
	DeltaTime = InDeltaTime;
}

void URigVMHost::SetAbsoluteTime(float InAbsoluteTime, bool InSetDeltaTimeZero)
{
	if(InSetDeltaTimeZero)
	{
		DeltaTime = 0.f;
	}
	AbsoluteTime = InAbsoluteTime;
	bAccumulateTime = false;
}

void URigVMHost::SetAbsoluteAndDeltaTime(float InAbsoluteTime, float InDeltaTime)
{
	AbsoluteTime = InAbsoluteTime;
	DeltaTime = InDeltaTime;
}

void URigVMHost::SetFramesPerSecond(float InFramesPerSecond)
{
	FramesPerSecond = InFramesPerSecond;	
}

float URigVMHost::GetCurrentFramesPerSecond() const
{
	if(FramesPerSecond > SMALL_NUMBER)
	{
		return FramesPerSecond;
	}
	if(DeltaTime > SMALL_NUMBER)
	{
		return 1.f / DeltaTime;
	}
	return 60.f;
}

bool URigVMHost::CanExecute() const
{
	return DisableExecution() == false;
}

void URigVMHost::Initialize(bool bRequestInit)
{
	TRACE_OBJECT_LIFETIME_BEGIN_WITH_OUTER(this, GetBoundOuterForTrace());

	if(IsInitializing())
	{
		UE_LOGF(LogRigVM, Warning, "%ls: Initialize is being called recursively.", *GetPathName());
		return;
	}

	if (IsTemplate())
	{
		// don't initialize template class 
		return;
	}

	InitializeFromCDO();
	InstantiateVMFromCDO();

	if (bRequestInit)
	{
		RequestInit();
	}
}

bool URigVMHost::InitializeVM(const FName& InEventName)
{
	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	const TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariablesImpl(false);
	if (!VM->IsNativized())
	{
		if (VM->GetExternalVariableDefs().Num() != ExternalVariables.Num())
		{
			return false;	// The rig did compile with errors
		}
	}

	// update the VM's external variables
	VM->SetExternalVariablesInstanceData(ExtendedExecuteContext, ExternalVariables);

	const bool bResult = VM->InitializeInstance(ExtendedExecuteContext);
	if(bResult)
	{
		bRequiresInitExecution = false;
	}

	// reset the time and caches during init
	AbsoluteTime = DeltaTime = 0.f;
		
	ExtendedExecuteContext.GetPublicDataSafe<>().GetNameCache()->Reset();

	if (InitializedEvent.IsBound())
	{
		FRigVMBracketScope BracketScope(InitBracket);
		InitializedEvent.Broadcast(this, InEventName);
	}

	return bResult;
}

void URigVMHost::Evaluate_AnyThread()
{
	// we can have other systems trying to poke into running instances of Control Rigs
	// on the anim thread and query data, such as
	// URigVMHostSkeletalMeshComponent::RebuildDebugDrawSkeleton,
	// using a lock here to prevent them from having an inconsistent view of the rig at some
	// intermediate stage of evaluation, for example, during evaluate, we can have a call
	// to copy hierarchy, which empties the hierarchy for a short period of time
	// and we don't want other systems to see that.
	UE::TScopeLock EvaluateLock(GetEvaluateMutex());
	
	// The EventQueueToRun should only be modified in this function
	ensureMsgf(EventQueueToRun.IsEmpty(), TEXT("Detected a recursive call to the control rig evaluation function %s"), *GetPackage()->GetPathName());
	
	// make sure to only evaluate if we have valid evaluations left	
	if (EvaluationsLeft >= 0 && !IsReadOnly())
	{
		if (EvaluationsLeft == 0)
		{
			EarlyExitEvent.Broadcast(this, EEarlyExitReason_NoEvaluationsLeft);
			return;
		}
	}
	
	// create a copy since we need to change it here temporarily,
	// and UI / the rig may change the event queue while it is running
	TGuardValue<TArray<FName>> EventQueueToRunGuard(EventQueueToRun, EventQueue);

	AdaptEventQueueForEvaluate(EventQueueToRun);
	
	// insert the events queued to run once at the beginning of the queue (in the order specified)
	{
		UE::TScopeLock EventQueueToRunOnceLock(GetEventQueueToRunOnceMutex());
		EventQueueToRun.Insert(EventsToRunOnce, 0);
	}

#if WITH_EDITOR
	FName FirstEvent = NAME_None;
	if (!EventQueueToRun.IsEmpty())
	{
		FirstEvent = EventQueueToRun[0];
	}
	FFirstEntryEventGuard FirstEntryEventGuard(&InstructionVisitInfo, FirstEvent);
#endif
	
	for (const FName& EventName : EventQueueToRun)
	{
		Execute(EventName);
	}

	if (EvaluationsLeft > 0)
	{
		EvaluationsLeft--;
	}
}

bool URigVMHost::IsReadOnly() const
{
#if RIGVM_TRACE_ENABLED
	return IsPlayingRewindDebugTrace();
#else
	return false;
#endif
}

#if RIGVM_TRACE_ENABLED

bool URigVMHost::IsPlayingRewindDebugTrace() const
{
	return bIsPlayingRewindDebugTrace;
}

void URigVMHost::StartPlayingRewindDebugTrace()
{
	bIsPlayingRewindDebugTrace = true;

	if (RigVMExtendedExecuteContext)
	{
#if WITH_EDITOR
		RigVMExtendedExecuteContext->InstructionVisitInfo = &InstructionVisitInfo;
		RigVMExtendedExecuteContext->ProfilingInfo = &ProfilingInfo;
		RigVMExtendedExecuteContext->DebugInfoWeak = DebugInfo.ToWeakPtr();
#endif
	}
}

void URigVMHost::StopPlayingRewindDebugTrace()
{
	bIsPlayingRewindDebugTrace = false;

	if (RigVMExtendedExecuteContext)
	{
#if WITH_EDITOR
		RigVMExtendedExecuteContext->InstructionVisitInfo = nullptr;
		RigVMExtendedExecuteContext->ProfilingInfo = nullptr;
		RigVMExtendedExecuteContext->DebugInfoWeak.Reset();
#endif
	}
}

#endif

UObject* URigVMHost::GetBoundOuterForTrace() const
{
	return GetOuter();
}

TArray<FRigVMExternalVariable> URigVMHost::GetExternalVariables() const
{
	return GetExternalVariablesImpl(true);
}

TArray<FRigVMExternalVariable> URigVMHost::GetPublicVariables() const
{
	return GetExternalVariables().FilterByPredicate([] (const FRigVMExternalVariable& Variable) -> bool
	{
		return Variable.IsPublic();
	});
}

UStruct* URigVMHost::GetVariablesStruct() const
{
	if (GeneratedBy)
	{
		return const_cast<UPropertyBag*>(Variables.GetPropertyBagStruct());
	}
	return GetClass();
}

uint8* URigVMHost::GetVariablesMemory() 
{
	if (GeneratedBy)
	{
		return Variables.GetMutableValue().GetMemory();
	}
	return (uint8*) this;
}

FRigVMExternalVariable URigVMHost::GetVariableByName(const FName& InVariableName) const
{
	if (const FProperty* Property = GetVariableProperty(InVariableName))
	{
		return FRigVMExternalVariable::Make(GetVariableGuid(InVariableName), Property, const_cast<URigVMHost*>(this)->GetVariablesMemory());
	}
	return FRigVMExternalVariable();
}

FRigVMExternalVariable URigVMHost::GetPublicVariableByName(const FName& InVariableName) const
{
	if (const FProperty* Property = GetPublicVariableProperty(InVariableName))
	{
		return FRigVMExternalVariable::Make(GetVariableGuid(InVariableName), Property, const_cast<URigVMHost*>(this)->GetVariablesMemory());
	}
	return FRigVMExternalVariable();
}

TArray<FName> URigVMHost::GetScriptAccessibleVariables() const
{
	TArray<FRigVMExternalVariable> PublicVariables = GetPublicVariables();
	TArray<FName> Names;
	for (const FRigVMExternalVariable& PublicVariable : PublicVariables)
	{
		Names.Add(PublicVariable.GetName());
	}
	return Names;
}

FName URigVMHost::GetVariableType(const FName& InVariableName) const
{
	const FRigVMExternalVariable PublicVariable = GetPublicVariableByName(InVariableName);
	if (PublicVariable.IsValid(true /* allow nullptr */))
	{
		return PublicVariable.GetExtendedCPPType();
	}
	return NAME_None;
}

FString URigVMHost::GetVariableAsString(const FName& InVariableName) const
{
#if WITH_EDITOR
	if (const FProperty* Property = GetVariableProperty(InVariableName))
	{
		FString Result;
		const uint8* Container = const_cast<URigVMHost*>(this)->GetVariablesMemory();
		if (FBlueprintEditorUtils::PropertyValueToString(Property, Container, Result, nullptr))
		{
			return Result;
		}
	}
#endif
	return FString();
}

bool URigVMHost::SetVariableFromString(const FName& InVariableName, const FString& InValue)
{
#if WITH_EDITOR
	if (const FProperty* Property = GetVariableProperty(InVariableName))
	{
		uint8* Container = GetVariablesMemory();
		return FBlueprintEditorUtils::PropertyValueFromString(Property, InValue, Container, nullptr);
	}
#endif
	return false;
}

void URigVMHost::InvalidateCachedMemory()
{
	if (VM)
	{
		VM->InvalidateCachedMemory(GetRigVMExtendedExecuteContext());
	}
}

void URigVMHost::RecreateCachedMemory()
{
	if (VM)
	{
		RequestInit();
	}
}

bool URigVMHost::Execute(const FName& InEventName)
{
	if(!CanExecute())
	{
		return false;
	}

	bool bJustRanInit = false;
	if(bRequiresInitExecution)
	{
		const TGuardValue<float> AbsoluteTimeGuard(AbsoluteTime, AbsoluteTime);
		const TGuardValue<float> DeltaTimeGuard(DeltaTime, DeltaTime);
		if(!InitializeVM(InEventName))
		{
			return false;
		}
		bJustRanInit = true;
	}

	if(EventQueueToRun.IsEmpty())
	{
		EventQueueToRun = EventQueue;
	}
	
	const bool bIsEventInQueue = EventQueueToRun.Contains(InEventName);
	const bool bIsEventFirstInQueue = !EventQueueToRun.IsEmpty() && EventQueueToRun[0] == InEventName; 
	const bool bIsEventLastInQueue = !EventQueueToRun.IsEmpty() && EventQueueToRun.Last() == InEventName;

	ensure(!HasAnyFlags(RF_ClassDefaultObject));
	
	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	TGuardValue<TFunction<void(const FName& InEventName)>> OnExecutionReachedExitCallback(ExtendedExecuteContext.OnExecutionReachedExitCallback, TFunction<void(const FName & InEventName)>([this](const FName& InEventName)
		{
			HandleExecutionReachedExit(InEventName);
		}));

	FRigVMExecuteContext& PublicContext = ExtendedExecuteContext.GetPublicData<>();
	PublicContext.SetDeltaTime(DeltaTime);
	PublicContext.SetAbsoluteTime(AbsoluteTime);
	PublicContext.SetFramesPerSecond(GetCurrentFramesPerSecond());
#if WITH_EDITOR
	PublicContext.SetHostBeingDebugged(bIsBeingDebugged);
#endif
#if RIGVM_TRACE_ENABLED
	PublicContext.SetHostPlayingRewindDebugTrace(bIsPlayingRewindDebugTrace);
#endif
	PublicContext.SetOwningComponent(GetOwningSceneComponent());
#if UE_RIGVM_DEBUG_EXECUTION
	PublicContext.bDebugExecution = bDebugExecutionEnabled;
#endif

	TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAssetInterface = GeneratedBy;
#if WITH_EDITOR
	ExtendedExecuteContext.SetInstructionVisitInfo(&InstructionVisitInfo);
	
	if (!RuntimeAssetInterface)
	{
		RuntimeAssetInterface = GetClass();
	}

	ExtendedExecuteContext.SetDebugInfo(DebugInfo, false);
	
	if (IsProfilingEnabled())
	{
		ExtendedExecuteContext.SetProfilingInfo(&ProfilingInfo);
	}
	else
	{
		ExtendedExecuteContext.SetProfilingInfo(nullptr);
	}
#endif

	// setup the draw interface for debug drawing
	if(!bIsEventInQueue || bIsEventFirstInQueue)
	{
		if (!IsReadOnly())
		{
			DrawInterface.Reset();
		}
	}
	PublicContext.SetDrawInterface(&DrawInterface);

	// draw container contains persistent draw instructions, 
	// so we cannot call Reset(), which will clear them,
	// instead, we re-initialize them from the CDO
	if (RuntimeAssetInterface && !HasAnyFlags(RF_ClassDefaultObject))
	{
		DrawContainer = RuntimeAssetInterface->GetDrawContainer();
	}
	PublicContext.SetDrawContainer(&DrawContainer);

	// guard against recursion
	if(IsExecuting())
	{
		UE_LOGF(LogRigVM, Warning, "%ls: Execute is being called recursively.", *GetPathName());
		return false;
	}

	if (ExecutedEvent.IsBound())
	{
		FRigVMBracketScope BracketScope(ExecuteBracket);
		PreExecutedEvent.Broadcast(this, InEventName);
	}

	const bool bSuccess = Execute_Internal(InEventName);

#if WITH_EDITOR

	// for the last event in the queue - clear the log message queue
	if (RigVMLog != nullptr && bEnableLogging)
	{
		if (bJustRanInit)
		{
			RigVMLog->KnownMessages.Reset();
			LoggedMessages.Reset();
		}
		else if(bIsEventLastInQueue)
		{
			for (const FRigVMLog::FLogEntry& Entry : RigVMLog->Entries)
			{
				if (Entry.FunctionName == NAME_None || Entry.InstructionIndex == INDEX_NONE || Entry.Message.IsEmpty())
				{
					continue;
				}

				FString PerInstructionMessage = 
					FString::Printf(
						TEXT("Instruction[%d] '%s': '%s'"),
						Entry.InstructionIndex,
						*Entry.FunctionName.ToString(),
						*Entry.Message
					);

				LogOnce(Entry.Severity, Entry.InstructionIndex, PerInstructionMessage);
			}
		}
	}
#endif

	if(!bIsEventInQueue || bIsEventLastInQueue) 
	{
		DeltaTime = 0.f;
	}

	if (ExecutedEvent.IsBound())
	{
		FRigVMBracketScope BracketScope(ExecuteBracket);
		ExecutedEvent.Broadcast(this, InEventName);
	}

	if (PublicContext.GetDrawInterface() && PublicContext.GetDrawContainer() && bIsEventLastInQueue) 
	{
		PublicContext.GetDrawInterface()->Instructions.Append(PublicContext.GetDrawContainer()->Instructions);
	}

	return bSuccess;
}

bool URigVMHost::DisableExecution()
{
	return CVarRigVMDisableExecutionAll->GetInt() == 1;
}

#if RIGVM_TRACE_ENABLED

void URigVMHost::TraceConstantData(FRigVMTraceArchiveWriter& InArchive) const
{
	InArchive.UsingCustomVersion(FRigVMObjectVersion::GUID);

	uint32 VMHash = VM ? VM->GetVMHash() : 0;
	InArchive << VMHash;
	uint32 ByteCodeHash = VM ? VM->GetByteCode().GetByteCodeHash() : 0;
	InArchive << ByteCodeHash;
}

bool URigVMHost::MatchesTracedConstantData(FRigVMTraceArchiveReader& InArchive) const
{
	const uint32 ExpectedVMHash = VM ? VM->GetVMHash() : 0;
	uint32 VMHash = 0;
	InArchive << VMHash;
	const uint32 ExpectedByteCodeHash = VM ? VM->GetByteCode().GetByteCodeHash() : 0;
	uint32 ByteCodeHash = 0;
	InArchive << ByteCodeHash;

	return VMHash == ExpectedVMHash && ByteCodeHash == ExpectedByteCodeHash;
}

void URigVMHost::LoadTracedConstantData(FRigVMTraceArchiveReader& InArchive)
{
	// nothing to do for the base implementation
}

void URigVMHost::TraceExecuteData(FRigVMTraceArchiveWriter& InArchive) const
{
	InArchive.UsingCustomVersion(FRigVMObjectVersion::GUID);
	
	const FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	TArray<FName> CurrentEventQueue = EventQueue;
	InArchive << CurrentEventQueue;
}

void URigVMHost::LoadTracedExecuteData(FRigVMTraceArchiveReader& InArchive)
{
	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	TArray<FName> CurrentEventQueue;
	InArchive << CurrentEventQueue;
	EventQueue = CurrentEventQueue;
}

#endif

bool URigVMHost::InitializeCDOVM()
{
	check(VM != nullptr);
	check(VM->HasAnyFlags(RF_ClassDefaultObject | RF_DefaultSubObject));

	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	// update the VM's external variables
	VM->ClearExternalVariables(ExtendedExecuteContext);
	VM->SetExternalVariableDefs(GetExternalVariablesImpl(false));
	return VM->Initialize(ExtendedExecuteContext);
}

bool URigVMHost::Execute_Internal(const FName& InEventName)
{
	if (VM == nullptr)
	{
		return false;
	}

	if (IsReadOnly())
	{
		return true;
	}

	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	static constexpr TCHAR InvalidatedVMFormat[] = TEXT("%s: Invalidated VM - aborting execution.");
	if(VM->IsNativized())
	{
		if(!IsValidLowLevel() ||
			!VM->IsValidLowLevel())
		{
			UE_LOG(LogRigVM, Warning, InvalidatedVMFormat, *GetClass()->GetName());
			return false;
		}
	}
	else
	{
		// sanity check the validity of the VM to ensure stability.
		if(!VM->IsContextValidForExecution(ExtendedExecuteContext)
			|| !IsValidLowLevel()
			|| !VM->IsValidLowLevel()
		)
		{
			UE_LOG(LogRigVM, Warning, InvalidatedVMFormat, *GetClass()->GetName());
			return false;
		}
	}

#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
	const uint64 StartCycles = FPlatformTime::Cycles64();
	if(ProfilingRunsLeft <= 0)
	{
		ProfilingRunsLeft = UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM;
		AccumulatedCycles = 0;
	}
#endif
	
	const bool bSuccess = VM->ExecuteVM(ExtendedExecuteContext, InEventName) != ERigVMExecuteResult::Failed;

#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
	const uint64 EndCycles = FPlatformTime::Cycles64();
	const uint64 Cycles = EndCycles - StartCycles;
	AccumulatedCycles += Cycles;
	ProfilingRunsLeft--;
	if(ProfilingRunsLeft == 0)
	{
		const double Milliseconds = FPlatformTime::ToMilliseconds64(AccumulatedCycles);
		UE_LOGF(LogRigVM, Display, "%ls: %d runs took %.03lfms.", *GetClass()->GetName(), UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM, Milliseconds);
	}
#endif

	return bSuccess;
}

bool URigVMHost::SupportsEvent(const FName& InEventName) const
{
	if (VM)
	{
		return VM->ContainsEntry(InEventName);
	}
	return false;
}

const TArray<FName>& URigVMHost::GetSupportedEvents() const
{
	if (VM)
	{
		return VM->GetEntryNames();
	}

	static const TArray<FName> EmptyEvents; 
	return EmptyEvents;
}

bool URigVMHost::ExecuteEvent(const FName& InEventName)
{
	if(SupportsEvent(InEventName))
	{
		TGuardValue<TArray<FName>> EventQueueGuard(EventQueue, {InEventName});
		Evaluate_AnyThread();
		return true;
	}
	return false;
}

void URigVMHost::RequestInit()
{
	bRequiresInitExecution = true;
}

void URigVMHost::RequestRunOnceEvent(const FName& InEventName, int32 InEventIndex)
{
	UE::TScopeLock EventQueueToRunOnceLock(GetEventQueueToRunOnceMutex());
	EventsToRunOnce.AddUnique(InEventName);
}

bool URigVMHost::RemoveRunOnceEvent(const FName& InEventName)
{
	UE::TScopeLock EventQueueToRunOnceLock(GetEventQueueToRunOnceMutex());
	return EventsToRunOnce.Remove(InEventName) > 0;
}

bool URigVMHost::IsRunOnceEvent(const FName& InEventName) const
{
	UE::TScopeLock EventQueueToRunOnceLock(GetEventQueueToRunOnceMutex());
	return EventsToRunOnce.Contains(InEventName);
}

void URigVMHost::SetEventQueue(const TArray<FName>& InEventNames)
{
	EventQueue = InEventNames;
}

void URigVMHost::UpdateVMSettings()
{
	if (VM)
	{
#if WITH_EDITOR
		// setup array handling and error reporting on the VM
		VMRuntimeSettings.SetLogFunction([this](const FRigVMLogSettings& InLogSettings, const FRigVMExecuteContext* InContext, const FString& Message)
			{
				check(InContext);

				if (RigVMLog)
				{
					RigVMLog->Report(InLogSettings, InContext->GetFunctionName(), InContext->GetInstructionIndex(), Message);
				}
				else
				{
					LogOnce(InLogSettings.Severity, InContext->GetInstructionIndex(), Message);
				}
			});
#endif

		GetRigVMExtendedExecuteContext().SetRuntimeSettings(VMRuntimeSettings);
	}
}

URigVM* URigVMHost::GetVM()
{
	if (VM == nullptr)
	{
		Initialize(true);
		check(VM);
	}
	return VM;
}

URigVM* URigVMHost::GetSourceVM() const
{
	URigVMHost* CDO = GetClass()->GetDefaultObject<URigVMHost>();
	URigVM* SourceVM = nullptr;
	if (GeneratedBy && GeneratedBy->GetVM())
	{
		SourceVM = GeneratedBy->GetVM();
	}
	else if (CDO && CDO->VM)
	{
		SourceVM = CDO->VM;
	}
	else
	{
		ensure(false);
	}
	return SourceVM;
}

const FRigVMMemoryStorageStruct* URigVMHost::GetDefaultMemoryByType(ERigVMMemoryType InMemoryType) const
{
	check(VM);
	return VM->GetDefaultMemoryByType(InMemoryType);
}

FRigVMMemoryStorageStruct* URigVMHost::GetMemoryByType(ERigVMMemoryType InMemoryType)
{
	check(VM);
	return VM->GetMemoryByType(GetRigVMExtendedExecuteContext(), InMemoryType);
}

const FRigVMMemoryStorageStruct* URigVMHost::GetMemoryByType(ERigVMMemoryType InMemoryType) const
{
	check(VM);
	return VM->GetMemoryByType(GetRigVMExtendedExecuteContext(), InMemoryType);
}

FRigVMMemoryStorageStruct* URigVMHost::GetDebugMemory(bool bCreateIfRequired)
{
	if (bCreateIfRequired)
	{
		if (GetMemoryByType(ERigVMMemoryType::Debug)->Num() == 0)
		{
			if (GetMemoryByType(ERigVMMemoryType::Work)->Num() > 0)
			{
				if (VM)
				{
					// create the debug memory for this instance
					return VM->CreateDebugMemory(GetRigVMExtendedExecuteContext());
				}
			}
		}
	}
	return GetMemoryByType(ERigVMMemoryType::Debug);
}

void URigVMHost::DrawIntoPDI(FPrimitiveDrawInterface* PDI, const FTransform& InTransform)
{
	for (const FRigVMDrawInstruction& Instruction : DrawInterface)
	{
		if (!Instruction.IsValid())
		{
			continue;
		}

		FTransform InstructionTransform = Instruction.Transform * InTransform;
		switch (Instruction.PrimitiveType)
		{
			case ERigVMDrawSettings::Points:
			{
				for (const FVector& Point : Instruction.Positions)
				{
					PDI->DrawPoint(InstructionTransform.TransformPosition(Point), Instruction.Color, Instruction.Thickness, SDPG_Foreground);
				}
				break;
			}
			case ERigVMDrawSettings::Lines:
			{
				const TArray<FVector>& Points = Instruction.Positions;
				PDI->AddReserveLines(SDPG_Foreground, Points.Num() / 2, false, Instruction.Thickness > SMALL_NUMBER);
				for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex += 2)
				{
					PDI->DrawLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_Foreground, Instruction.Thickness);
				}
				break;
			}
			case ERigVMDrawSettings::LineStrip:
			{
				const TArray<FVector>& Points = Instruction.Positions;
				PDI->AddReserveLines(SDPG_Foreground, Points.Num() - 1, false, Instruction.Thickness > SMALL_NUMBER);
				for (int32 PointIndex = 0; PointIndex < Points.Num() - 1; PointIndex++)
				{
					PDI->DrawLine(InstructionTransform.TransformPosition(Points[PointIndex]), InstructionTransform.TransformPosition(Points[PointIndex + 1]), Instruction.Color, SDPG_Foreground, Instruction.Thickness);
				}
				break;
			}
			case ERigVMDrawSettings::DynamicMesh:
			{
				FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());
				MeshBuilder.AddVertices(Instruction.MeshVerts);
				MeshBuilder.AddTriangles(Instruction.MeshIndices);
				MeshBuilder.Draw(PDI, InstructionTransform.ToMatrixWithScale(), Instruction.MaterialRenderProxy, SDPG_World/*SDPG_Foreground*/);
				break;
			}
			case ERigVMDrawSettings::Text:
			{
				// Text is rendered via FCanvas in DrawIntoCanvas, not PDI.
				break;
			}
			default:
			{
				break;
			}
		}
	}
}

void URigVMHost::DrawIntoCanvas(FCanvas* Canvas, const FSceneView* View, const FTransform& InTransform)
{
	if (!Canvas || !View)
	{
		return;
	}

	const UFont* Font = GEngine ? GEngine->GetSmallFont() : nullptr;
	if (!Font)
	{
		return;
	}

	for (const FRigVMDrawInstruction& Instruction : DrawInterface)
	{
		if (Instruction.PrimitiveType != ERigVMDrawSettings::Text || !Instruction.IsValid())
		{
			continue;
		}

		const FTransform InstructionTransform = Instruction.Transform * InTransform;
		const FVector WorldPos = InstructionTransform.TransformPosition(Instruction.Positions[0]);

		// Skip points at or behind the camera. FSceneView::ScreenToPixel silently inverts negative W,
		// so WorldToPixel alone would produce mirrored screen coordinates for behind-view anchors.
		const FVector4 ScreenPoint = View->WorldToScreen(WorldPos);
		if (ScreenPoint.W <= 0.0f)
		{
			continue;
		}

		FVector2D ScreenPos;
		if (!View->ScreenToPixel(ScreenPoint, ScreenPos))
		{
			continue;
		}

		FCanvasTextItem TextItem(ScreenPos, FText::FromString(Instruction.Text), Font, Instruction.Color);
		TextItem.Scale = FVector2D(FMath::Max(Instruction.FontScale, 0.0f));
		TextItem.EnableShadow(FLinearColor::Black);
		Canvas->DrawItem(TextItem);
	}
}

USceneComponent* URigVMHost::GetOwningSceneComponent()
{
	return GetTypedOuter<USceneComponent>();
}

void URigVMHost::PostInitInstanceIfRequired()
{
	Variables.Refresh();
}

void URigVMHost::SwapVMToNativizedIfRequired(UClass* InNativizedClass)
{
	if (HasAnyFlags(RF_NeedPostLoad))
	{
		return;
	}
	
	if(VM == nullptr)
	{
		return;
	}

	const bool bNativizedVMEnabled = AreNativizedVMsEnabled();

	// GetNativizedClass can be pretty costly, let's try to skip this if it is not absolutely necessary
	if(InNativizedClass == nullptr && bNativizedVMEnabled)
	{
		const uint32 Hash = VM->GetVMHash(); 
		if (LastNativizedVMHash.Get(0) == Hash && WeakNativizedVMClass.IsSet())
		{
			InNativizedClass = WeakNativizedVMClass.GetValue().Get();
		}
		else
		{
			const TArray<FRigVMExternalVariableDef> ExternalVariableDefs = RigVMTypeUtils::GetExternalVariableDefs(GetExternalVariables());
			if(!HasAnyFlags(RF_ClassDefaultObject))
			{
				if(URigVM* SourceVM = GetSourceVM())
				{
					if(SourceVM->IsNativized())
					{
						InNativizedClass = SourceVM->GetClass();
					}
					else
					{
						InNativizedClass = SourceVM->GetNativizedClass(Hash);
					}
				}
			}
			else
			{
				InNativizedClass = VM->GetNativizedClass(Hash);
			}

			LastNativizedVMHash = Hash;
			WeakNativizedVMClass = InNativizedClass;
		}
	}

	if(VM->IsNativized())
	{
		if((InNativizedClass == nullptr) || !bNativizedVMEnabled)
		{
			VM = GetSourceVM();
			GetRigVMExtendedExecuteContext().ResetNumExecutions();
#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
			ProfilingRunsLeft = 0;
			AccumulatedCycles = 0;
#endif
		}
	}
	else
	{
		if(InNativizedClass && bNativizedVMEnabled)
		{
#if WITH_EDITOR
			URigVM* SourceVM = VM;
#endif
			VM = InNativizedClass->GetDefaultObject<URigVM>();
#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
			ProfilingRunsLeft = 0;
			AccumulatedCycles = 0;
#endif

#if WITH_EDITOR
			// let's set the bytecode for UI purposes.
			// this is only used for traversing node from execute stack to node and back etc. 
			// since the hash between nativized VM and current matches we assume the bytecode is identical as well.
			if(URigVMNativized* NativizedVM = Cast<URigVMNativized>(VM))
			{
				NativizedVM->SetSourceVM(SourceVM);
			}
			GetRigVMExtendedExecuteContext().ResetNumExecutions();
#endif
		}
	}
}

bool URigVMHost::AreNativizedVMsEnabled() const
{
#if WITH_EDITOR
	return bEnableNativizedVMs && (CVarRigVMEnableNativizedVMs->GetInt() != 0);
#else
	return CVarRigVMEnableNativizedVMs->GetInt() != 0;
#endif
}

bool URigVMHost::CanSwapVMToNativized() const
{
	if (VM == nullptr)
	{
		return false;
	}
	
	if (VM->IsNativized())
	{
		return true;
	}

	const uint32 Hash = VM->GetVMHash();
	if (LastNativizedVMHash.Get(0) != Hash || !WeakNativizedVMClass.IsSet())
	{
		LastNativizedVMHash = Hash;
		WeakNativizedVMClass = VM->GetNativizedClass(LastNativizedVMHash.GetValue());
	}

	if (WeakNativizedVMClass.IsSet())
	{
		return WeakNativizedVMClass.GetValue().IsValid();
	}

	return false;
}

#if WITH_EDITORONLY_DATA
void URigVMHost::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URigVM::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMMemoryStorage::StaticClass()));
}

#if UE_RIGVM_DEBUG_EXECUTION
const FString URigVMHost::GetDebugExecutionString()
{
	TGuardValue<bool> DebugExecutionGuard(bDebugExecutionEnabled, true);
	FRigVMExecuteContext& PublicContext = GetRigVMExtendedExecuteContext().GetPublicData<FRigVMExecuteContext>();
	PublicContext.DebugMemoryString.Reset();
	
	Evaluate_AnyThread();

	return PublicContext.DebugMemoryString;
}
#endif
#endif

void URigVMHost::SetRigVMExtendedExecuteContext(FRigVMExtendedExecuteContext* InRigVMExtendedExecuteContext)
{
	if (RigVMExtendedExecuteContext)
	{
		RigVMExtendedExecuteContext->Host = nullptr;
	}
	RigVMExtendedExecuteContext = InRigVMExtendedExecuteContext;
	if (RigVMExtendedExecuteContext)
	{
		RigVMExtendedExecuteContext->Host = this;
	}
}

UObject* URigVMHost::ResolveUserDefinedTypeById(const FString& InTypeName) const
{
	const FSoftObjectPath* ResultPathPtr = UserDefinedStructGuidToPathName.Find(InTypeName);
	if (ResultPathPtr == nullptr)
	{
		ResultPathPtr = UserDefinedEnumToPathName.Find(InTypeName);
	}

	if (ResultPathPtr == nullptr)
	{
		return nullptr;
	}

	if (UObject* TypeObject = ResultPathPtr->TryLoad())
	{
		// Ensure we have a hold on this type so it doesn't get nixed on the next GC.
		const_cast<URigVMHost*>(this)->UserDefinedTypesInUse.Add(TypeObject);
		return TypeObject;
	}

	return nullptr;
}

void URigVMHost::PostInitInstance(URigVMHost* InCDO)
{
	const EObjectFlags SubObjectFlags =
		HasAnyFlags(RF_ClassDefaultObject) ?
		RF_Public | RF_DefaultSubObject :
		RF_Transient | RF_Transactional;

	FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

	ExtendedExecuteContext.SetContextPublicDataStruct(GetPublicContextStruct());

#if WITH_EDITOR
	ExtendedExecuteContext.GetPublicData<>().SetLog(RigVMLog); // may be nullptr
#endif

	UpdateVMSettings();

	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (InCDO)
		{
			ensure(VM == nullptr || VM == InCDO->GetVM());
			if (VM == nullptr)	// some Engine Tests does not have the VM
			{
				VM = InCDO->GetVM();
			}
		}
	}
	else // we are the CDO
	{
		// set up the VM
		if (VM == nullptr)
		{
			VM = NewObject<URigVM>(this, TEXT("RigVM_VM"), SubObjectFlags);
		}

		// for default objects we need to check if the CDO is rooted. specialized Control Rigs
		// such as the FK control rig may not have a root since they are part of a C++ package.

		// since the sub objects are created after the constructor
		// GC won't consider them part of the CDO, even if they have the sub object flags
		// so even if CDO is rooted and references these sub objects, 
		// it is not enough to keep them alive.
		// Hence, we have to add them to root here.
		if(GetClass()->IsNative())
		{
			VM->AddToRoot();
		}
	}

	RequestInit();
}

void URigVMHost::PostInitProperties()
{
	// This call is replicating URigVMBlueprintGeneratedClass::PostInitInstance
	Super::PostInitProperties();
	
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (GeneratedBy)
		{
			PostInitInstance(nullptr);
		}
	}
}

void URigVMHost::GenerateUserDefinedDependenciesData(FRigVMExtendedExecuteContext& InContext)
{
	if (VM)
	{
		const TArray<const UObject*> UserDefinedDependencies = GetUserDefinedDependencies({ GetDefaultMemoryByType(ERigVMMemoryType::Literal), GetDefaultMemoryByType(ERigVMMemoryType::Work) });
		UserDefinedStructGuidToPathName.Reset();
		UserDefinedEnumToPathName.Reset();
		UserDefinedTypesInUse.Reset();

		for (const UObject* UserDefinedDependency : UserDefinedDependencies)
		{
			if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(UserDefinedDependency))
			{
				const FString GuidBasedName = RigVMTypeUtils::GetUniqueStructTypeName(UserDefinedStruct);
				UserDefinedStructGuidToPathName.Add(GuidBasedName, UserDefinedStruct);
			}
			else if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(UserDefinedDependency))
			{
				const FString EnumName = RigVMTypeUtils::CPPTypeFromEnum(UserDefinedEnum);
				UserDefinedEnumToPathName.Add(EnumName, UserDefinedEnum);
			}
		}
	}
}

TArray<const UObject*> URigVMHost::GetUserDefinedDependencies(const TArray<const FRigVMMemoryStorageStruct*> InMemory)
{
	TArray<const UObject*> Dependencies;
	for (const FRigVMMemoryStorageStruct* MemoryStorage : InMemory)
	{
		if (MemoryStorage)
		{
			MemoryStorage->GetUserDefinedDependencies(Dependencies);
		}
	}

	const TArray<const FRigVMFunction*>& Functions = VM->GetFunctions();
	for (const FRigVMFunction* Function : Functions)
	{
		FRigVMRegistry_NoLock* Registry = VM->GetLocalizedRegistry();

		// if the VM doesn't have a localized registry
		// then access the global one and lock it for reading.
		TUniquePtr<FRigVMRegistryReadLock> ReadLock;
		if(Registry == nullptr)
		{
			FRigVMRegistry_RWLock& RegistryRef = FRigVMRegistry_RWLock::Get();
			Registry = &RegistryRef;
			ReadLock = MakeUnique<FRigVMRegistryReadLock>(RegistryRef);
		}
		
		const TArray<TRigVMTypeIndex>& TypeIndices = Function->GetArgumentTypeIndices_NoLock(Registry->GetHandle_NoLock());
		for (const TRigVMTypeIndex& TypeIndex : TypeIndices)
		{
			const FRigVMTemplateArgumentType& Type = Registry->GetType_NoLock(TypeIndex);
			if (Cast<UUserDefinedStruct>(Type.CPPTypeObject) ||
				Cast<UUserDefinedEnum>(Type.CPPTypeObject))
			{
				Dependencies.AddUnique(Type.CPPTypeObject);
			}
		}
	}

	return Dependencies;
}

const TArray<FString>& URigVMHost::GetRequiredPlugins() const
{
	return RequiredPlugins;
}

void URigVMHost::GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& InContext)
{
	RequiredPlugins.Reset();
	if (VM)
	{
		VM->GetRequiredPlugins(RequiredPlugins);
	}

	const TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
	for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		if (!ExternalVariable.GetCPPTypeObject())
		{
			continue;
		}
		const FString PluginName = RigVMTypeUtils::GetPluginName(ExternalVariable.GetCPPTypeObject());
		if (!PluginName.IsEmpty())
		{
			RequiredPlugins.AddUnique(PluginName);
		}
	}
}

#if WITH_EDITOR

void URigVMHost::EnableNativizedVM(bool bEnable)
{
	if (bEnableNativizedVMs == bEnable)
	{
		return;
	}
	bEnableNativizedVMs = bEnable;
	Initialize(true);
}

#endif

void URigVMHost::HandleExecutionReachedExit(const FName& InEventName)
{
	if (bAccumulateTime)
	{
		AbsoluteTime += DeltaTime;
	}
}

TArray<FRigVMExternalVariable> URigVMHost::GetExternalVariablesImpl(bool bFallbackToBlueprint) const
{
	TArray<FRigVMExternalVariable> ExternalVariables;

	if (GeneratedBy)
	{
		FRigVMPropertyBag& MutableVariables = const_cast<URigVMHost*>(this)->Variables;
		ExternalVariables = GeneratedBy->GetExternalVariables();
		for (FRigVMExternalVariable& Variable : ExternalVariables)
		{
			Variable.SetMemory(MutableVariables.GetDataByName<uint8>(Variable.GetName()));
		}
	}
	else
	{
		for (TFieldIterator<FProperty> PropertyIt(GetClass()); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if(Property->IsNative())
			{
				continue;
			}

			const FGuid Guid = Cast<UBlueprintGeneratedClass>(GetClass())->FindBlueprintPropertyGuidFromName(Property->GetFName());
			FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(Guid, Property, (UObject*)this);
			if(!ExternalVariable.IsValid())
			{
				UE_LOGF(LogRigVM, Warning, "%ls: Property '%ls' of type '%ls' is not supported.", *GetClass()->GetName(), *Property->GetName(), *Property->GetCPPType());
				continue;
			}

			ExternalVariables.Add(ExternalVariable);
		}

#if WITH_EDITOR

		if (bFallbackToBlueprint)
		{
			// if we have a difference in the blueprint variables compared to us - let's 
			// use those instead. the assumption here is that the blueprint is dirty and
			// hasn't been compiled yet.
			if (UBlueprint* Blueprint = Cast<UBlueprint>(GetClass()->ClassGeneratedBy))
			{
				TArray<FRigVMExternalVariable> BlueprintVariables;
				for (const FBPVariableDescription& VariableDescription : Blueprint->NewVariables)
				{
					FRigVMExternalVariable ExternalVariable = RigVMTypeUtils::ExternalVariableFromBPVariableDescription(VariableDescription, GetClass(),(UObject*)this);
					if (ExternalVariable.GetBaseCPPType().IsNone())
					{
						continue;
					}

					ExternalVariable.SetMemory(nullptr);
					BlueprintVariables.Add(ExternalVariable);
				}

				if (ExternalVariables.Num() != BlueprintVariables.Num())
				{
					return BlueprintVariables;
				}

				TMap<FName, int32> NameMap;
				for (int32 Index = 0; Index < ExternalVariables.Num(); Index++)
				{
					NameMap.Add(ExternalVariables[Index].GetName(), Index);
				}

				for (FRigVMExternalVariable BlueprintVariable : BlueprintVariables)
				{
					const int32* Index = NameMap.Find(BlueprintVariable.GetName());
					if (Index == nullptr)
					{
						return BlueprintVariables;
					}

					FRigVMExternalVariable ExternalVariable = ExternalVariables[*Index];
					if (!ExternalVariable.IsSameType(BlueprintVariable) ||
						ExternalVariable.IsPublic() != BlueprintVariable.IsPublic())
					{
						return BlueprintVariables;
					}
				}
			}
		}
#endif
	}

	return ExternalVariables;
}

void URigVMHost::InstantiateVMFromCDO()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		SwapVMToNativizedIfRequired();

		FRigVMExtendedExecuteContext& ExtendedExecuteContext = GetRigVMExtendedExecuteContext();

		URigVMHost* CDO = GetClass()->GetDefaultObject<URigVMHost>();
		URigVM* SourceVM = nullptr;
		if (GeneratedBy && GeneratedBy->GetVM())
		{
			SourceVM = GeneratedBy->GetVM();
		}
		else if (CDO && CDO->VM)
		{
			SourceVM = CDO->VM;
		}
		else
		{
			ensure(false);
		}
		if (VM && SourceVM)
		{
			if(!VM->IsNativized())
			{
				ExtendedExecuteContext.WorkMemoryStorage = SourceVM->GetDefaultWorkMemory();
				ExtendedExecuteContext.DebugMemoryStorage = FRigVMMemoryStorageStruct(ERigVMMemoryType::Invalid);
				ExtendedExecuteContext.VMHash = SourceVM->GetVMHash();
			}
		}
		else if (VM)
		{
			VM->Reset(ExtendedExecuteContext);
			ExtendedExecuteContext.Reset();
		}
		else
		{
			ensure(false);
		}
	}

	RequestInit();
}

void URigVMHost::CopyExternalVariableDefaultValuesFromCDO()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GeneratedBy;
		if (!RuntimeAsset)
		{
			RuntimeAsset = GetClass();
		}
		
		if (RuntimeAsset)
		{
			RuntimeAsset->InitializeVariables(this);
		}
	}
}

void URigVMHost::InitializeFromCDO()
{
	// copy CDO property you need to here
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// similar to FControlRigBlueprintCompilerContext::CopyTermDefaultsToDefaultObject,
		// where CDO is initialized from BP there,
		// we initialize all other instances of Control Rig from the CDO here
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GeneratedBy)
		{
			if ((VM == nullptr || VM != RuntimeAsset->GetVM()) && RuntimeAsset->GetVM())
			{
				VM = RuntimeAsset->GetVM();
			}

			PostInitInstanceIfRequired();

			// copy draw container
			DrawContainer = RuntimeAsset->GetDrawContainer();

			// copy vm settings
			VMRuntimeSettings = RuntimeAsset->GetVMRuntimeSettings();
		}
		else
		{
			URigVMHost* CDO = GetClass()->GetDefaultObject<URigVMHost>();

			if ((VM == nullptr || VM != CDO->VM) && CDO->VM)
			{
				VM = CDO->VM;
			}

			PostInitInstanceIfRequired();

			// copy draw container
			DrawContainer = CDO->DrawContainer;

			// copy vm settings
			VMRuntimeSettings = CDO->VMRuntimeSettings;
		}
	}
}

void URigVMHost::CopyVMMemory(FRigVMExtendedExecuteContext& TargetContext, const FRigVMExtendedExecuteContext& SourceContext)
{
	TargetContext.CopyMemoryStorage(SourceContext);
}

void URigVMHost::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != NULL)
	{
		RemoveUserDataOfClass(InUserData->GetClass());
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* URigVMHost::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	const TArray<UAssetUserData*>* ArrayPtr = GetAssetUserDataArray();
	for (int32 DataIdx = 0; DataIdx < ArrayPtr->Num(); DataIdx++)
	{
		UAssetUserData* Datum = (*ArrayPtr)[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return NULL;
}

void URigVMHost::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
#if WITH_EDITOR
	for (int32 DataIdx = 0; DataIdx < AssetUserDataEditorOnly.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserDataEditorOnly[DataIdx];
		if (Datum != NULL && Datum->IsA(InUserDataClass))
		{
			AssetUserDataEditorOnly.RemoveAt(DataIdx);
			return;
		}
	}
#endif
}

const TArray<UAssetUserData*>* URigVMHost::GetAssetUserDataArray() const
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	else
	{
		static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
		CachedAssetUserData.Reset();
		CachedAssetUserData.Append(AssetUserData);
		CachedAssetUserData.Append(AssetUserDataEditorOnly);
		return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
	}
#else
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
}

#if WITH_EDITOR	

void URigVMHost::LogOnce(EMessageSeverity::Type InSeverity, int32 InInstructionIndex, const FString& InMessage)
{
	if(LoggedMessages.Contains(InMessage))
	{
		return;
	}

	switch (InSeverity)
	{
		case EMessageSeverity::Error:
		{
			UE_LOGF(LogRigVM, Error, "%ls", *InMessage);
			break;
		}
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
		{
			UE_LOGF(LogRigVM, Warning, "%ls", *InMessage);
			break;
		}
		case EMessageSeverity::Info:
		{
			UE_LOGF(LogRigVM, Display, "%ls", *InMessage);
			break;
		}
		default:
		{
			break;
		}
	}

	LoggedMessages.Add(InMessage, true);
}

#endif

#undef LOCTEXT_NAMESPACE
