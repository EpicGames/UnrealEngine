// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVM.h: Module implementation.
=============================================================================*/

#include "RigVMModule.h"
#include "Modules/ModuleManager.h"
#include "HAL/IConsoleManager.h"
#include "RigVMObjectVersion.h"
#include "UObject/DevObjectVersion.h"
#if WITH_EDITOR
#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#endif

#if RIGVM_TRACE_ENABLED
#include "Features/IModularFeatures.h"
#endif

IMPLEMENT_MODULE(FRigVMModule, RigVM)

DEFINE_LOG_CATEGORY(LogRigVM);

// Unique Control Rig Object version id
const FGuid FRigVMObjectVersion::GUID(0xDC49959B, 0x53C04DE7, 0x9156EA88, 0x5E7C5D39);
// Register RigVM custom version with Core
static FDevVersionRegistration GRegisterRigVMObjectVersion(FRigVMObjectVersion::GUID, FRigVMObjectVersion::LatestVersion, TEXT("Dev-RigVM"));

#if UE_RIGVM_UOBJECT_PROPERTIES_ENABLED
TAutoConsoleVariable<bool> CVarRigVMEnableUObjects(TEXT("RigVM.UObjectSupport"), true, TEXT("When true the RigVMCompiler will allow UObjects."));
#endif

#if UE_RIGVM_UINTERFACE_PROPERTIES_ENABLED
TAutoConsoleVariable<bool> CVarRigVMEnableUInterfaces(TEXT("RigVM.UInterfaceSupport"), true, TEXT("When true the RigVMCompiler will allow UInterfaces."));
#endif

void FRigVMModule::StartupModule()
{
#if WITH_EDITOR
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("RigVMMemoryStorageStruct", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPropertyBagDetails::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();
#endif

#if RIGVM_TRACE_ENABLED
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerRigVMExtension);
#endif
}

void FRigVMModule::ShutdownModule()
{
#if WITH_EDITOR
	// Unregister the details customization
	if (FPropertyEditorModule* PropertyModule = FModuleManager::Get().GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyModule->UnregisterCustomPropertyTypeLayout("RigVMMemoryStorageStruct");
		PropertyModule->NotifyCustomizationModuleChanged();
	}
#endif

#if RIGVM_TRACE_ENABLED
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerRigVMExtension);
#endif
}

bool RigVMCore::SupportsUObjects()
{
#if UE_RIGVM_UOBJECT_PROPERTIES_ENABLED
	return CVarRigVMEnableUObjects.GetValueOnAnyThread();
#else
	return false;
#endif
}

bool RigVMCore::SupportsUInterfaces()
{
#if UE_RIGVM_UINTERFACE_PROPERTIES_ENABLED
	return CVarRigVMEnableUInterfaces.GetValueOnAnyThread();
#else
	return false;
#endif
}