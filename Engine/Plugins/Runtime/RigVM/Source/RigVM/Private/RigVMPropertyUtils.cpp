// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMPropertyUtils.h"

#include "RigVMModule.h"
#include "RigVMTypeUtils.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Misc/AssertionMacros.h"
#include "UObject/TextProperty.h"

namespace
{
	static const FLazyName TypeName_Bool(TEXT("bool"));
	static const FLazyName TypeName_Int32(TEXT("int32"));
	static const FLazyName TypeName_Int64(TEXT("int64"));
	static const FLazyName TypeName_UInt32(TEXT("uint32"));
	static const FLazyName TypeName_UInt64(TEXT("uint64"));
	static const FLazyName TypeName_Float(TEXT("float"));
	static const FLazyName TypeName_Double(TEXT("double"));
	static const FLazyName TypeName_FString(TEXT("FString"));
	static const FLazyName TypeName_FName(TEXT("FName"));
	static const FLazyName TypeName_FText(TEXT("FText"));
	static const FLazyName TypeName_UInt8(TEXT("uint8"));
}

void RigVMPropertyUtils::GetTypeFromProperty(const FProperty* InProperty, FName& OutTypeName, UObject*& OutTypeObject)
{
	if (CastField<FBoolProperty>(InProperty))
	{
		OutTypeName = TypeName_Bool;
		OutTypeObject = nullptr;
	}
	else if (CastField<FIntProperty>(InProperty))
	{
		OutTypeName = TypeName_Int32;
		OutTypeObject = nullptr;
	}
	else if (CastField<FInt64Property>(InProperty))
	{
		OutTypeName = TypeName_Int64;
		OutTypeObject = nullptr;
	}
	else if (CastField<FUInt32Property>(InProperty))
	{
		OutTypeName = TypeName_UInt32;
		OutTypeObject = nullptr;
	}
	else if (CastField<FUInt64Property>(InProperty))
	{
		OutTypeName = TypeName_UInt64;
		OutTypeObject = nullptr;
	}
	else if (CastField<FFloatProperty>(InProperty))
	{
		OutTypeName = TypeName_Float;
		OutTypeObject = nullptr;
	}
	else if (CastField<FDoubleProperty>(InProperty))
	{
		OutTypeName = TypeName_Double;
		OutTypeObject = nullptr;
	}
	else if (CastField<FStrProperty>(InProperty))
	{
		OutTypeName = TypeName_FString;
		OutTypeObject = nullptr;
	}
	else if (CastField<FNameProperty>(InProperty))
	{
		OutTypeName = TypeName_FName;
		OutTypeObject = nullptr;
	}
	else if (CastField<FTextProperty>(InProperty))
	{
		OutTypeName = TypeName_FText;
		OutTypeObject = nullptr;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		OutTypeName = *RigVMTypeUtils::CPPTypeFromEnum(EnumProperty->GetEnum());
		OutTypeObject = EnumProperty->GetEnum();
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		if (UEnum* BytePropertyEnum = ByteProperty->Enum)
		{
			OutTypeName = *RigVMTypeUtils::CPPTypeFromEnum(BytePropertyEnum);
			OutTypeObject = BytePropertyEnum;
		}
		else
		{
			OutTypeName = TypeName_UInt8;
			OutTypeObject = nullptr;
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		OutTypeName = *RigVMTypeUtils::GetUniqueStructTypeName(StructProperty->Struct);
		OutTypeObject = StructProperty->Struct;
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		FName InnerTypeName = NAME_None;
		GetTypeFromProperty(ArrayProperty->Inner, InnerTypeName, OutTypeObject);
		OutTypeName = *RigVMTypeUtils::ArrayTypeFromBaseType(InnerTypeName.ToString());
	}
	else if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InProperty))
	{
		if (RigVMCore::SupportsUObjects())
		{
			OutTypeName = *FString::Printf(TEXT("TSubclassOf<%s%s>"),
				ClassProperty->MetaClass->GetPrefixCPP(),
				*ClassProperty->MetaClass->GetName());
			OutTypeObject = ClassProperty->MetaClass;
		}
		else
		{
			OutTypeName = NAME_None;
			OutTypeObject = nullptr;
		}
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		if (RigVMCore::SupportsUObjects())
		{
			OutTypeName = *FString::Printf(TEXT("TObjectPtr<%s%s>"),
				ObjectProperty->PropertyClass->GetPrefixCPP(),
				*ObjectProperty->PropertyClass->GetName());
			OutTypeObject = ObjectProperty->PropertyClass;
		}
		else
		{
			OutTypeName = NAME_None;
			OutTypeObject = nullptr;
		}
	}
	else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(InProperty))
	{
		if (RigVMCore::SupportsUInterfaces())
		{
			OutTypeName = *FString::Printf(TEXT("TScriptInterface<%s%s>"),
				InterfaceProperty->InterfaceClass->GetPrefixCPP(),
				*InterfaceProperty->InterfaceClass->GetName());
			OutTypeObject = InterfaceProperty->InterfaceClass;
		}
		else
		{
			OutTypeName = NAME_None;
			OutTypeObject = nullptr;
		}
	}
	else
	{
		// this can happen due to unsupported property types
		// within data assets or user defined structs
		OutTypeName = NAME_None;
		OutTypeObject = nullptr;
	}
}

struct FHashCombineFast
{
	inline uint32 operator()(uint32 A, uint32 B) const { return HashCombineFast(A, B); }
};

struct FHashCombineStable
{
	inline uint32 operator()(uint32 A, uint32 B) const { return HashCombine(A, B); }
};

template<typename HASH_COMBINE_OP>
static uint32 GetPropertyHashPrivate(const FProperty* InProperty, const uint8* InMemory, EPropertyPointerType InContainerType)
{
	if (!ensure(InProperty) || InMemory == nullptr)
	{
		return 0;
	}

	// Offset into the container to the location where the actual property value is stored.
	if (InContainerType == EPropertyPointerType::Container)
	{
		InMemory += InProperty->GetOffset_ForInternal();
	}

	// If the property type provides its own hashing function, use that as a preference. The types below do not
	// have hashing implemented, and so we hand-roll some of them.
	if (InProperty->PropertyFlags & CPF_HasGetValueTypeHash)
	{
		return InProperty->GetValueTypeHash(InMemory);
	}

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return BoolProperty->GetPropertyValue(InMemory);
	}
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, InMemory);
		int32 Hash = ::GetTypeHash(ArrayHelper.Num());
		for (int32 Index = 0; Index < ArrayHelper.Num(); Index++)
		{
			Hash = HASH_COMBINE_OP{}(Hash, GetPropertyHashPrivate<HASH_COMBINE_OP>(ArrayProperty->Inner, ArrayHelper.GetRawPtr(Index), EPropertyPointerType::Direct));
		}
		return Hash;
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
	{
		FScriptMapHelper MapHelper(MapProperty, InMemory);
		int32 Hash = ::GetTypeHash(MapHelper.Num());
		for (int32 Index = 0; Index < MapHelper.Num(); Index++)
		{
			Hash = HASH_COMBINE_OP{}(Hash, GetPropertyHashPrivate<HASH_COMBINE_OP>(MapProperty->KeyProp, MapHelper.GetKeyPtr(Index), EPropertyPointerType::Direct));
			Hash = HASH_COMBINE_OP{}(Hash, GetPropertyHashPrivate<HASH_COMBINE_OP>(MapProperty->ValueProp, MapHelper.GetValuePtr(Index), EPropertyPointerType::Direct));
		}
		return Hash;
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
	{
		FScriptSetHelper SetHelper(SetProperty, InMemory);
		int32 Hash = ::GetTypeHash(SetHelper.Num());
		for (int32 Index = 0; Index < SetHelper.Num(); Index++)
		{
			Hash = HASH_COMBINE_OP{}(Hash, GetPropertyHashPrivate<HASH_COMBINE_OP>(SetProperty->ElementProp, SetHelper.GetElementPtr(Index), EPropertyPointerType::Direct));
		}
		return Hash;
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		const UScriptStruct* StructType = StructProperty->Struct;
		// UserDefinedStruct overrides GetStructTypeHash to work without valid CppStructOps
		if (StructType->IsA<UUserDefinedStruct>() || (StructType->GetCppStructOps() && StructType->GetCppStructOps()->HasGetTypeHash()))
		{
			return StructType->GetStructTypeHash(InMemory);
		}
		else
		{
			uint32 Hash = 0;
			for (TFieldIterator<FProperty> It(StructType); It; ++It)
			{
				const FProperty* SubProperty = *It;
				Hash = HASH_COMBINE_OP{}(Hash, GetPropertyHashPrivate<HASH_COMBINE_OP>(*It, InMemory + SubProperty->GetOffset_ForInternal(), EPropertyPointerType::Direct));
			}
			return Hash;
		}
	}

	// If we get here, we're missing support for a property type that doesn't do its own hashing. 
	checkNoEntry();
	return 0;
}

uint32 RigVMPropertyUtils::GetPropertyHashFast(const FProperty* InProperty, const uint8* InMemory, EPropertyPointerType InContainerType)
{
	return GetPropertyHashPrivate<FHashCombineFast>(InProperty, InMemory, InContainerType);
}

uint32 RigVMPropertyUtils::GetPropertyHashStable(const FProperty* InProperty, const uint8* InMemory, EPropertyPointerType InContainerType)
{
	return GetPropertyHashPrivate<FHashCombineStable>(InProperty, InMemory, InContainerType);
}
