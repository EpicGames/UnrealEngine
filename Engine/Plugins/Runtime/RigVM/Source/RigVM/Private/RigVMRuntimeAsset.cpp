// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMRuntimeAsset.h"

#include "RigVMHost.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

bool URigVMRuntimeAsset::IsUpToDate() const
{
	return RVMA_UpToDate == Status || RVMA_UpToDateWithWarnings == Status;
}

void URigVMRuntimeAsset::MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent)
{
	MarkPackageDirty();
#if WITH_EDITOR
	PostEditChangeProperty(PropertyChangedEvent);
#endif
}

void URigVMRuntimeAsset::MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus)
{
	TSharedPtr<TGuardValue<TEnumAsByte<ERigVMAssetStatus>>> StatusGuard;
	if (bSkipDirtyAssetStatus)
	{
		StatusGuard = MakeShared<TGuardValue<TEnumAsByte<ERigVMAssetStatus>>>(Status, Status);
	}
	MarkAssetAsModified();
}

const TArray<FString>& URigVMRuntimeAsset::GetRequiredPlugins(bool bRefresh) const
{
	if (bRefresh)
	{
		URigVMRuntimeAsset* MutableThis = const_cast<URigVMRuntimeAsset*>(this);
		MutableThis->GenerateRequiredPluginsData(MutableThis->ExecuteContext);
	}

	return RequiredPlugins;
}

void URigVMRuntimeAsset::PostInitProperties()
{
	Super::PostInitProperties();
	
	ExecuteContext.SetContextPublicDataStruct(FRigVMExecuteContext::StaticStruct());
}

void URigVMRuntimeAsset::Initialize(UClass* InEditorOnlyDataClass)
{
	Variables.AddProperties({});
	ensure(Variables.GetPropertyBagStruct());

	const FString VMName = FString::Printf(TEXT("%s_VM"), *GetName());
	VM = NewObject<URigVM>(this, URigVM::StaticClass(), *VMName);
	
#if WITH_EDITORONLY_DATA
	check(EditorAsset.Get() == nullptr);

	const FString EditorAssetName = FString::Printf(TEXT("%s_EditorOnly"), *GetName());
	EditorAsset = NewObject<UObject>(this, InEditorOnlyDataClass, *EditorAssetName, RF_Public | RF_Transactional);
#endif
}

void URigVMRuntimeAsset::RemoveInstance(UObject* InInstance)
{
	ArchetypeInstances.Remove(InInstance);
}

void URigVMRuntimeAsset::GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& InContext)
{
	RequiredPlugins.Reset();
	if (VM)
	{
		VM->GetRequiredPlugins(RequiredPlugins);
	}

	const TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
	for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		if (!ExternalVariable.GetCPPTypeObject())
		{
			continue;
		}
		const FString PluginName = RigVMTypeUtils::GetPluginName(ExternalVariable.GetCPPTypeObject());
		if (!PluginName.IsEmpty())
		{
			RequiredPlugins.AddUnique(PluginName);
		}
	}
}

TScriptInterface<IRigVMRuntimeAssetInterface> IRigVMRuntimeAssetInterface::GetInterfaceOuter(const UObject* InObject)
{
	if (!InObject)
	{
		return nullptr;
	}
	 
	UObject* Outer = InObject->GetOuter();
	while (Outer)
	{
		if (Outer->Implements<URigVMRuntimeAssetInterface>())
		{
			return Outer;
		}
		Outer = Outer->GetOuter();
	}
	return nullptr;
}

void URigVMRuntimeAsset::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITORONLY_DATA
	if (EditorAsset)
	{
		constexpr EObjectFlags RequiredFlags = RF_Public | RF_Transactional;
		if ((EditorAsset->GetFlags() & RequiredFlags) != RequiredFlags)
		{
			EditorAsset->SetFlags(RequiredFlags);
		}
		EditorAsset->PostLoad();
	}
#endif
}

void URigVMRuntimeAsset::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);
#if WITH_EDITORONLY_DATA
	EditorAsset->PreDuplicate(DupParams);
#endif
}

void URigVMRuntimeAsset::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITORONLY_DATA
	const FString NewEditorAssetName = FString::Printf(TEXT("%s_EditorOnly"), *GetName());
	EditorAsset->Rename(*NewEditorAssetName, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
#endif
}

void URigVMRuntimeAsset::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);
#if WITH_EDITORONLY_DATA
	// Whenever the asset is renamed/moved, generated classes parented to the old package
	// are not moved to the new package automatically (see FAssetRenameManager), so we
	// have to manually perform the move/rename, to avoid invalid reference to the old package
	
	// Note: while asset duplication doesn't duplicate the classes either, it is not a problem there
	// because we always recompile in post duplicate.
	TArray<UObject*> Objects;
	GetObjectsWithOuter(OldOuter->GetPackage(), Objects, EGetObjectsFlags::None);
	for (UObject* Object : Objects)
	{
		if (URigVMMemoryStorageGeneratorClass* MemoryClass = Cast<URigVMMemoryStorageGeneratorClass>(Object))
		{
			MemoryClass->Rename(nullptr, GetPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
	}

	// Store the RuntimeAsset's old/new paths for EditorAsset to use in its PostRename.
	// This is needed because by the time EditorAsset::PostRename is called, we've lost
	// access to the old package path (OldOuter for EditorAsset will be this RuntimeAsset,
	// which has already been moved to the new location).
	PreRenameOldAssetPath = FString::Printf(TEXT("%s.%s"), *OldOuter->GetPathName(), *OldName.ToString());
	PreRenameNewAssetPath = GetPathName();

	if (EditorAsset)
	{
		const FString OldEditorAssetName = EditorAsset->GetName();
		const FString NewEditorAssetName = FString::Printf(TEXT("%s_EditorOnly"), *GetName());

		EditorAsset->Rename(*NewEditorAssetName, this, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);

		// When moving (name unchanged), Rename() returns early without calling PostRename.
		// Call it directly to ensure EditorAsset processes the path change.
		if (OldEditorAssetName == NewEditorAssetName)
		{
			EditorAsset->PostRename(this, EditorAsset->GetFName());
		}
	}

	// Clear after EditorAsset has used them
	PreRenameOldAssetPath.Empty();
	PreRenameNewAssetPath.Empty();
#endif
}

void URigVMRuntimeAsset::MarkInstancesAsGarbage()
{
	// Clean up archetype instances before destroying the asset
	TArray<UObject*> InstancesToDestroy = GetArchetypeInstances(false);
	for (UObject* Instance : InstancesToDestroy)
	{
		if (IsValid(Instance))
		{
			Instance->MarkAsGarbage();
		}
	}
	ArchetypeInstances.Empty();
}

TArray<FRigVMExternalVariable> URigVMRuntimeAsset::GetExternalVariables() 
{
	TArray<FRigVMExternalVariable> ExternalVariables;
	TArray<FRigVMGraphVariableDescription> GraphVariables = GetAssetVariables();
	Algo::Transform(GraphVariables, ExternalVariables, [](const FRigVMGraphVariableDescription& Variable)
		{
			return RigVMVariableUtils::ExternalVariableFromRigVMVariableDescription(Variable);
		});

	for (FRigVMExternalVariable& Variable : ExternalVariables)
	{
		if (const FProperty* Property = FindGeneratedPropertyByName(Variable.GetName()))
		{
			Variable.SetProperty(Property);
			Variable.SetMemory(Variables.GetDataByName<uint8>(Property->GetFName()));
		}
	}
	return ExternalVariables;
}

TArray<FRigVMGraphVariableDescription> URigVMRuntimeAsset::GetAssetVariables() const
{
	TArray<FRigVMGraphVariableDescription> VariableDescs;

	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();
	VariableDescs.Reserve(PropertyDescs.Num());

	for (int32 PropertyIndex=0; PropertyIndex<PropertyDescs.Num(); PropertyIndex++)
	{
		const FPropertyBagPropertyDesc& Desc = PropertyDescs[PropertyIndex];
		FRigVMGraphVariableDescription& NewVariable = VariableDescs.Add_GetRef(RigVMVariableUtils::VariableDescriptionFromPropertyDesc(Desc));
		NewVariable.DefaultValue = Variables.GetDataAsString(PropertyIndex);
	}
	
	return VariableDescs;
}

FRigVMGraphVariableDescription URigVMRuntimeAsset::FindAssetVariable(const FName& InName) const
{
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();

	for (int32 PropertyIndex=0; PropertyIndex<PropertyDescs.Num(); PropertyIndex++)
	{
		const FPropertyBagPropertyDesc& Desc = PropertyDescs[PropertyIndex];
#if WITH_EDITOR
		if (Desc.GetMetaData(FRigVMPropertyDescription::MD_DisplayName).Equals(InName.ToString()))
#else
		if (Desc.Name.IsEqual(InName))
#endif
		{
			FRigVMGraphVariableDescription Result = RigVMVariableUtils::VariableDescriptionFromPropertyDesc(Desc);
			Result.DefaultValue = Variables.GetDataAsStringByNameSafe(Desc.Name);
			return Result;
		}
	}
	
	return FRigVMGraphVariableDescription();
}

int32 URigVMRuntimeAsset::GetVariableIndex(const FName& InName) const
{
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();

	for (int32 PropertyIndex = 0; PropertyIndex < PropertyDescs.Num(); PropertyIndex++)
	{
		const FPropertyBagPropertyDesc& Desc = PropertyDescs[PropertyIndex];
#if WITH_EDITOR
		if (Desc.GetMetaData(FRigVMPropertyDescription::MD_DisplayName).Equals(InName.ToString()))
#else
		if (Desc.Name.IsEqual(InName))
#endif
		{
			return PropertyIndex;
		}
	}

	return INDEX_NONE;
}

int32 URigVMRuntimeAsset::GetVariableIndex(const FGuid& InGuid) const
{
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();

	// Find the variable by GUID
	const FPropertyBagPropertyDesc* FoundDesc = PropertyBag->FindPropertyDescByID(InGuid);
	if (!FoundDesc)
	{
		return INDEX_NONE;
	}

	// Return the cached index if valid
	const int32 CachedIndex = FoundDesc->GetCachedIndex();
	if (CachedIndex != INDEX_NONE)
	{
		return CachedIndex;
	}

	// Fallback: find the index by iterating through all descriptors
	TConstArrayView<FPropertyBagPropertyDesc> PropertyDescs = PropertyBag->GetPropertyDescs();
	for (int32 PropertyIndex = 0; PropertyIndex < PropertyDescs.Num(); PropertyIndex++)
	{
		if (&PropertyDescs[PropertyIndex] == FoundDesc)
		{
			return PropertyIndex;
		}
	}

	return INDEX_NONE;
}

const TArray<UAssetUserData*>* URigVMRuntimeAsset::GetAssetUserDataArray() const
{
#if WITH_EDITOR
	if (IsRunningCookCommandlet())
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	else
	{
		static thread_local TArray<TObjectPtr<UAssetUserData>> CachedAssetUserData;
		CachedAssetUserData.Reset();
		CachedAssetUserData.Append(AssetUserData);
		CachedAssetUserData.Append(AssetUserDataEditorOnly);
		return &ToRawPtrTArrayUnsafe(CachedAssetUserData);
	}
#else
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#endif
}

FProperty* URigVMRuntimeAsset::FindGeneratedPropertyByName(const FName& InName)
{
	const FName SanitizedName = Variables.SanitizePropertyName(InName);
	return Variables.FindPropertyByName(SanitizedName);
}

#if WITH_EDITOR

bool URigVMRuntimeAsset::RemoveMemberVariable(const FName& InName)
{
	FRigVMAssetVariablesChangeScope VariablesChangeScope(this, InName);
	
	Modify();
	const FName SanitizedName = Variables.SanitizePropertyName(InName);
	return Variables.RemovePropertyByName(SanitizedName) == EPropertyBagAlterationResult::Success;
}

bool URigVMRuntimeAsset::BulkRemoveMemberVariables(const TArray<FName>& InNames)
{
	FRigVMAssetVariablesChangeScope VariablesChangeScope(this, NAME_None);
	
	Modify();
	TArray<FName> SanitizedNames;
	Algo::Transform(InNames, SanitizedNames, [this](const FName& InName)
		{
			return Variables.SanitizePropertyName(InName);
		});
	EPropertyBagAlterationResult Result = Variables.RemovePropertiesByName(SanitizedNames);
	Variables.Refresh();
	return Result == EPropertyBagAlterationResult::Success;
}

bool URigVMRuntimeAsset::RenameMemberVariable(const FName& InOldName, const FName& InNewName)
{
	if (FindGeneratedPropertyByName(InNewName) != nullptr)
	{
		return false;
	}

	FRigVMGraphVariableDescription Variable = FindAssetVariable(InOldName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InOldName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		Modify();
		
		// Renaming the property is not enough since we need to also modify the metadata to account for the change to DisplayName
		if (Variables.RemovePropertyByName(InOldName) == EPropertyBagAlterationResult::Success)
		{
			Variable.Name = InNewName;
			FRigVMPropertyDescription NewDesc = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
			Variables.AddProperties({NewDesc});

			// Restore original index
			if (OriginalIndex != INDEX_NONE)
			{
				SetVariableIndex(Variable.Name, OriginalIndex);
			}

			return true;
		}
	}
	
	return false;
}

bool URigVMRuntimeAsset::ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		Modify();
		RemoveMemberVariable(InName);

		Variable.CPPType = InCPPType;
		Variable.CPPTypeObject = nullptr;
		Variable.CPPTypeObjectPath = NAME_None;
		RigVMTypeUtils::FixCPPTypeAndObject(Variable.CPPType, Variable.CPPTypeObject);
		Variable.bPublic = bIsPublic;
		Variable.bPrivate = bIsReadOnly;
		Variable.DefaultValue = InDefaultValue;
		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);

		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariableIndex(const FName& InName, int32 NewIndex)
{
	// Find the variable by name to get its GUID
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		return SetVariableIndex(Variable.Guid, NewIndex);
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex)
{
	// Find the source property descriptor by GUID
	const UPropertyBag* PropertyBag = Variables.GetPropertyBagStruct();
	const FPropertyBagPropertyDesc* SourceDesc = PropertyBag->FindPropertyDescByID(InVariableGuid);
	if (!SourceDesc)
	{
		return false;
	}

	// Get current index for comparison
	const int32 OldIndex = SourceDesc->GetCachedIndex();
	if (OldIndex == INDEX_NONE)
	{
		return false; // Invalid cached index
	}

	// No change needed
	if (OldIndex == NewIndex)
	{
		return true;
	}

	// Get target property descriptor at NewIndex (no array copy needed!)
	const FPropertyBagPropertyDesc* TargetDesc = PropertyBag->FindPropertyDescByIndex(NewIndex);
	if (!TargetDesc)
	{
		return false; // Invalid index
	}

	const FName SourceName = SourceDesc->Name;
	const FName TargetName = TargetDesc->Name;

#if WITH_EDITOR
	// Get the display name for the change scope
	const FName VariableDisplayName = *SourceDesc->GetMetaData(FRigVMPropertyDescription::MD_DisplayName);
#else
	const FName VariableDisplayName = SourceName;
#endif

	// Modify for undo/redo
	Modify();

	// Scope for variables change notification
	FRigVMAssetVariablesChangeScope VariablesChangeScope(this, VariableDisplayName);

	// Use the property bag's built-in reorder functionality with property names
	// When moving down (OldIndex < NewIndex), insert after the target
	// When moving up (OldIndex > NewIndex), insert before the target
	const bool bInsertBefore = (OldIndex > NewIndex);
	const EPropertyBagAlterationResult Result = Variables.ReorderProperty(SourceName, TargetName, bInsertBefore);
	Variables.Refresh();

	if (Result != EPropertyBagAlterationResult::Success)
	{
		return false;
	}

	return true;
}

FText URigVMRuntimeAsset::GetVariableTooltip(const FName& InName) const
{
	return FText::FromString(GetVariableMetadataValue(InName, FBlueprintMetadata::MD_Tooltip));
}

bool URigVMRuntimeAsset::SetVariableTooltip(const FName& InName, const FText& InTooltip)
{
	return SetVariableMetadataValue(InName, FBlueprintMetadata::MD_Tooltip, InTooltip.ToString());
}

FString URigVMRuntimeAsset::GetVariableCategory(const FName& InName) const
{
	return GetVariableMetadataValue(InName, TEXT("Category"));
}

bool URigVMRuntimeAsset::SetVariableCategory(const FName& InName, const FString& InCategory)
{
	return SetVariableMetadataValue(InName, TEXT("Category"), InCategory);
}

FString URigVMRuntimeAsset::GetVariableMetadataValue(const FName& InName, const FName& InKey) const
{
	if (const FProperty* Property = Variables.FindPropertyByName(InName))
	{
		return Property->GetMetaData(InKey);
	}
	return FString();
}

bool URigVMRuntimeAsset::SetVariableMetadataValue(const FName& InName, const FName& InKey, const FString& InValue)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Description.MetaData.FindOrAdd(InKey) = InValue;

		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::RemoveVariableMetadataValue(const FName& InName, const FName& InKey)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Description.MetaData.Remove(InKey);

		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);
		Variable.bExposedOnSpawn = bInExposeOnSpawn;

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);
		Variable.bExposeToCinematics = bInExposeToCinematics;

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariablePrivate(const FName& InName, const bool bInPrivate)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);
		Variable.bPrivate = bInPrivate;

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::SetVariablePublic(const FName& InName, const bool bIsPublic)
{
	FRigVMGraphVariableDescription Variable = FindAssetVariable(InName);
	if (!Variable.Name.IsNone())
	{
		// Store original index before removal
		const int32 OriginalIndex = GetVariableIndex(InName);

		FRigVMAssetVariablesChangeScope VariablesChangeScope(this, Variable.Name);
		RemoveMemberVariable(InName);
		Variable.bPublic = bIsPublic;

		FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Variable, true);
		Variables.AddProperties({Description});

		// Restore original index
		if (OriginalIndex != INDEX_NONE)
		{
			SetVariableIndex(Variable.Name, OriginalIndex);
		}

		return true;
	}
	return false;
}

FName URigVMRuntimeAsset::AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue)
{
	FRigVMAssetVariablesChangeScope VariablesChangeScope(this, InVariableToCreate.GetName());
	Modify();
	
	TSet<FString> VariableNames;
	for (int32 i=0; i<Variables.Num(); i++)
	{
		const FProperty* Property = Variables.GetProperty(i);
		if (Property->HasMetaData(FRigVMPropertyDescription::MD_DisplayName))
		{
			VariableNames.Add(Property->GetMetaData(FRigVMPropertyDescription::MD_DisplayName));
		}
	}
	
	FString VariableName = InVariableToCreate.GetName().ToString();
	FRigVMPropertyDescription::SanitizeName(VariableName, true);
	int32 Index=1;
	while (VariableNames.Contains(VariableName))
	{
		VariableName = FString::Printf(TEXT("%s_%d"), *InVariableToCreate.GetName().ToString(), Index++);
		FRigVMPropertyDescription::SanitizeName(VariableName, true);
	}
	
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromExternalVariable(InVariableToCreate, CPPType, &CPPTypeObject);

	FRigVMPropertyDescription Description(*VariableName, CPPType, CPPTypeObject, InDefaultValue, true);
	Description.Guid = InVariableToCreate.GetGuid();
	Description.PropertyFlags = CPF_Edit | CPF_BlueprintVisible | CPF_DisableEditOnInstance | CPF_Interp;
	if (InVariableToCreate.IsPublic())
	{
		Description.PropertyFlags &= ~CPF_DisableEditOnInstance;
	}
	else
	{
		Description.PropertyFlags |= CPF_DisableEditOnInstance;
	}

	// If the caller passed a Guid that is already in use by another property in this
	// bag (e.g. duplicate-paste of a variable carries the source's Guid forward), regenerate it.
	// Two descs sharing a Guid corrupt tagged-property load via the PropertyGuid name-redirect at
	// UStruct::SerializeVersionedTaggedProperties — the second tag's name gets rewritten to the first
	// matching property's name and the second variable's payload silently overwrites the first.
	if (Description.Guid.IsValid())
	{
		if (const UPropertyBag* BagStruct = Variables.GetPropertyBagStruct())
		{
			for (const FPropertyBagPropertyDesc& ExistingDesc : BagStruct->GetPropertyDescs())
			{
				if (ExistingDesc.ID == Description.Guid)
				{
					// Reaching this branch IS the bug we want surfaced: a caller fed us a Guid
					// already used by another variable in this bag. Unconditional ensure breaks
					// in dev builds; the warning surfaces in shipping logs.
					ensureMsgf(false, TEXT("AddHostMemberVariableFromExternal: incoming variable '%s' shared Guid %s with existing variable '%s'; regenerating Guid."),
						*VariableName, *Description.Guid.ToString(), *ExistingDesc.Name.ToString());
					UE_LOGF(LogRigVM, Warning,
						"AddHostMemberVariableFromExternal: incoming variable '%ls' shared Guid %ls with existing variable '%ls'; regenerating Guid.",
						*VariableName, *Description.Guid.ToString(), *ExistingDesc.Name.ToString());
					Description.Guid = FGuid::NewGuid();
					break;
				}
			}
		}
	}
	else
	{
		Description.Guid = FGuid::NewGuid();
	}

	int32 OldNumVariables = Variables.Num();

	Variables.AddProperties({Description});
	if (Variables.Num() == OldNumVariables)
	{
		return NAME_None;
	}

	return Description.Name;
}

#endif

void URigVMRuntimeAsset::UpdateSupportedEventNames()
{
	SupportedEventNames.Empty();
	if (VM)
	{
		SupportedEventNames = VM->GetEntryNames();
	}
}

TArray<UObject*> URigVMRuntimeAsset::GetArchetypeInstances(bool bIncludeDerivedClass) const
{
	ArchetypeInstances = ArchetypeInstances.FilterByPredicate([](TWeakObjectPtr<UObject> Instance)
		{
			return Instance.IsValid() && !Instance->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed | RF_MirroredGarbage);
		});
	TArray<UObject*> RawPtrs;
	Algo::Transform(ArchetypeInstances, RawPtrs, [](TWeakObjectPtr<UObject> Instance){ return Instance.Get(); });
	return RawPtrs;
}

URigVMHost* URigVMRuntimeAsset::InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags) 
{
	URigVMHost* Host = NewObject<URigVMHost>(InOuter, GetRigVMHostClass(), InName, InFlags);
	Host->GeneratedBy = TScriptInterface<IRigVMRuntimeAssetInterface>(this);
	InitializeInstance(Host);
	return Host;
}

bool URigVMRuntimeAsset::InitializeInstance(URigVMHost* InInstance)
{
	if (ensure(InInstance->GeneratedBy == this))
	{
		UE::TScopeLock EvaluateLock(InInstance->GetEvaluateMutex());
		InInstance->PostInitInstanceIfRequired();
		InInstance->Variables = Variables;
		InInstance->VM = VM;

		FRigVMExtendedExecuteContext& Context = InInstance->GetRigVMExtendedExecuteContext();
		{
			// update the VM's external variables
			TArray<FRigVMExternalVariable> ExternalVariables = InInstance->GetExternalVariables();
			VM->SetExternalVariableDefs(ExternalVariables);
			VM->SetExternalVariablesInstanceData(Context, ExternalVariables);
		}
		VM->InitializeInstance(Context);
		VM->Initialize(Context);
		InInstance->DrawContainer = DrawContainer;
		InInstance->VMRuntimeSettings = RuntimeSettings;

		// Register only after the instance is fully initialized so concurrent
		// readers of ArchetypeInstances never observe a partially-bound host.
		ArchetypeInstances.AddUnique(InInstance);
		return true;
	}
	return false;
}

bool URigVMRuntimeAsset::InitializeVariables(URigVMHost* InInstance)
{
	if (ensure(InInstance->GeneratedBy == this))
	{
		UE::TScopeLock EvaluateLock(InInstance->GetEvaluateMutex());
		InInstance->Variables = Variables;
		return true;
	}
	return false;
}

#if WITH_EDITOR
FRigVMAssetVariablesChangeScope::FRigVMAssetVariablesChangeScope(URigVMRuntimeAsset* InAsset, const FName InVariableName)
	: VariableName(InVariableName)
	, Asset(nullptr)
{
	if (InAsset && !InAsset->bIsModifyingVariables)
	{
		Asset = InAsset;
		Asset->bIsModifyingVariables = true;
		Asset->OnPreVariablesChangedDelegate.Broadcast(InVariableName);
	}
}

FRigVMAssetVariablesChangeScope::~FRigVMAssetVariablesChangeScope()
{
	if (Asset)
	{
		Asset->bIsModifyingVariables = false;
		Asset->OnPostVariablesChangedDelegate.Broadcast(VariableName);
	}
}
#endif

