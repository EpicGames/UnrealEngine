// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMSettings)

#define LOCTEXT_NAMESPACE "RigVMSettings"

URigVMEditorSettings::URigVMEditorSettings(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bEnableContextMenuTimeSlicing(true)
	, bEnableFuzzySearchInActionMenu(false)
	, MinimumFuzzyScore(25.f)
#endif
{
#if WITH_EDITORONLY_DATA
	bHighlightSimilarNodes = false;
	bFadeOutUnrelatedNodes = false;
	bUseFlashLight = false;
	bAutoLinkMutableNodes = false;
#endif
}

URigVMProjectSettings::URigVMProjectSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Default tags are now added via Engine/Plugins/Runtime/RigVM/Config/Editor.ini
	// to avoid external plugins overriding default tags when they only want to append new tags
	// See UObject::LoadConfig for details
}

#undef LOCTEXT_NAMESPACE // RigVMSettings
FRigVMTag URigVMProjectSettings::GetTag(FName InTagName) const
{
	if(const FRigVMTag* FoundTag = FindTag(InTagName))
	{
		return *FoundTag;
	}
	return FRigVMTag();
}

const FRigVMTag* URigVMProjectSettings::FindTag(FName InTagName) const
{
	return VariantTags.FindByPredicate([InTagName](const FRigVMTag& Tag)
	{
		return InTagName == Tag.Name;
	});
}
