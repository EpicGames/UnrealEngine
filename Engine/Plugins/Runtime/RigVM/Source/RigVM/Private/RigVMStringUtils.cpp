// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMStringUtils.h"

#include "String/Split.h"

FString RigVMStringUtils::JoinStrings(const FString& InStringA, const FString& InStringB, const TCHAR* InSeparator, const TCHAR* InPrefix, const TCHAR* InSuffix)
{
	const int32 SeparatorLen = FCString::Strlen(InSeparator);
	const int32 PrefixLen = InPrefix ? FCString::Strlen(InPrefix) : 0;
	const int32 SuffixLen = InSuffix ? FCString::Strlen(InSuffix) : 0;

	const int32 RequiredLen = PrefixLen + InStringA.Len() + SeparatorLen + InStringB.Len() + SuffixLen;

	TStringBuilder<256> Builder;
	Builder.Reserve(RequiredLen + 1 /* \0 */);

	if(InPrefix)
	{
		Builder.Append(InPrefix);
	}
	Builder.Append(InStringA);
	Builder.Append(InSeparator);
	Builder.Append(InStringB);
	if(InSuffix)
	{
		Builder.Append(InSuffix);
	}
	return FString(Builder);
}

FString RigVMStringUtils::JoinStrings(const TArray<FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix, const TCHAR* InSuffix)
{
	return JoinStringsConst(TArrayView<const FString>(InStrings.GetData(), InStrings.Num()), InSeparator, InPrefix, InSuffix);
}

FString RigVMStringUtils::JoinStrings(const TArrayView<FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix, const TCHAR* InSuffix)
{
	return JoinStringsConst(TArrayView<const FString>(InStrings.GetData(), InStrings.Num()), InSeparator, InPrefix, InSuffix);
}

FString RigVMStringUtils::JoinStringsConst(const TArrayView<const FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix, const TCHAR* InSuffix)
{
	if(InStrings.IsEmpty())
	{
		if(InPrefix && InSuffix)
		{
			return FString(InPrefix) + FString(InSuffix);
		}
		if(InPrefix)
		{
			return InPrefix;
		}
		if(InSuffix)
		{
			return InSuffix;
		}
		return FString();
	}
	
	const int32 SeparatorLen = FCString::Strlen(InSeparator);
	const int32 PrefixLen = InPrefix ? FCString::Strlen(InPrefix) : 0;
	const int32 SuffixLen = InSuffix ? FCString::Strlen(InSuffix) : 0;
	int32 StringsTotalLen = 0;
	for(const FString& String : InStrings)
	{
		StringsTotalLen += String.Len();
	}

	const int32 RequiredLen = PrefixLen + StringsTotalLen + SeparatorLen * (InStrings.Num() - 1) + SuffixLen;
	TStringBuilder<256> Builder;
	Builder.Reserve(RequiredLen + 1 /* \0 */);

	if(InPrefix)
	{
		Builder.Append(InPrefix);
	}
	for(int32 Index = 0; Index < InStrings.Num(); Index++)
	{
		if(Index > 0)
		{
			Builder.Append(InSeparator);
		}
		Builder.Append(InStrings[Index]);
	}
	if(InSuffix)
	{
		Builder.Append(InSuffix);
	}
	return FString(Builder);
}

// Splits a string up into its parts 
bool RigVMStringUtils::SplitString(const FString& InString, const TCHAR* InSeparator, TArray<FString>& InOutParts)
{
	const int32 OriginalPartsCount = InOutParts.Num();
	FStringView Remaining = InString;

	int32 Index = Remaining.Find(InSeparator, 0, ESearchCase::IgnoreCase);
	while (Index != INDEX_NONE)
	{
		InOutParts.Emplace(Remaining.Left(Index));
		Remaining.RightChopInline(Index + 1);
		Index = Remaining.Find(InSeparator, 0, ESearchCase::IgnoreCase);
	}

	if(!Remaining.IsEmpty())
	{
		InOutParts.Emplace(Remaining);
	}

	return InOutParts.Num() > OriginalPartsCount;
}

// Creates a template notation such as "MyTemplate(float,bool,float)"
FString RigVMStringUtils::MakeTemplateNotation(const FString& InTemplateName, const TArray<FString>& InArgumentNotations)
{
	const FString JoinedArguments = JoinStrings(InArgumentNotations, TEXT(","), TEXT("("), TEXT(")"));
	
	TStringBuilder<256> Builder;
	Builder.Reserve(InTemplateName.Len() + JoinedArguments.Len() + 1 /* \0 */);
	Builder.Append(InTemplateName);
	Builder.Append(JoinedArguments);
	return FString(Builder);
}

bool RigVMStringUtils::SplitNodePathAtStart(const FString& InNodePath, FString& LeftMost, FString& Right)
{
	return InNodePath.Split(TEXT("|"), &LeftMost, &Right, ESearchCase::IgnoreCase, ESearchDir::FromStart);
}

bool RigVMStringUtils::SplitNodePathAtEnd(const FString& InNodePath, FString& Left, FString& RightMost)
{
	return InNodePath.Split(TEXT("|"), &Left, &RightMost, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
}

bool RigVMStringUtils::SplitNodePath(const FString& InNodePath, TArray<FString>& Parts)
{
	return SplitString(InNodePath, TEXT("|"), Parts);
}

FString RigVMStringUtils::JoinNodePath(const FString& Left, const FString& Right)
{
	ensure(!Left.IsEmpty() && !Right.IsEmpty());
	TStringBuilder<256> Builder;
	Builder.Reserve(Left.Len() + 1 + Right.Len() + 1 /* \0 */);
	Builder.Append(Left);
	Builder.Append(TEXT("|"));
	Builder.Append(Right);
	return FString(Builder);
}

FString RigVMStringUtils::JoinNodePath(const TArray<FString>& InParts)
{
	return JoinStrings(InParts, TEXT("|"));
}

bool RigVMStringUtils::SplitPinPathAtStart(const FString& InPinPath, FString& LeftMost, FString& Right)
{
	return InPinPath.Split(TEXT("."), &LeftMost, &Right, ESearchCase::IgnoreCase, ESearchDir::FromStart);
}

bool RigVMStringUtils::SplitPinPathAtStart(FStringView InPinPath, FStringView& LeftMost, FStringView& Right)
{
	return UE::String::SplitFirst(InPinPath, TEXT("."), LeftMost, Right, ESearchCase::IgnoreCase);
}

bool RigVMStringUtils::SplitPinPathAtEnd(const FString& InPinPath, FString& Left, FString& RightMost)
{
	return InPinPath.Split(TEXT("."), &Left, &RightMost, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
}

bool RigVMStringUtils::SplitPinPath(const FString& InPinPath, TArray<FString>& Parts)
{
	return SplitString(InPinPath, TEXT("."), Parts);
}

FString RigVMStringUtils::JoinPinPath(const FString& Left, const FString& Right)
{
	ensure(!Left.IsEmpty() && !Right.IsEmpty());
	TStringBuilder<256> Builder;
	Builder.Reserve(Left.Len() + 1 + Right.Len() + 1 /* \0 */);
	Builder.Append(Left);
	Builder.Append(TEXT("."));
	Builder.Append(Right);
	return FString(Builder);
}

FString RigVMStringUtils::JoinPinPath(const TArray<FString>& InParts)
{
	return JoinStrings(InParts, TEXT("."));
}

FString RigVMStringUtils::JoinDefaultValue(const TArray<FString>& InParts)
{
	return JoinStrings(InParts, TEXT(","), TEXT("("), TEXT(")"));
}

TArray<FString> RigVMStringUtils::SplitDefaultValue(const FString& InDefaultValue)
{
	TArray<FString> Parts;
	if (InDefaultValue.IsEmpty())
	{
		return Parts;
	}

	if(InDefaultValue[0] != TCHAR('('))
	{
		return Parts;
	}
	if(InDefaultValue[InDefaultValue.Len() - 1] != TCHAR(')'))
	{
		return Parts;
	}

	FString Content = InDefaultValue.Mid(1, InDefaultValue.Len() - 2);
	int32 BraceCount = 0;
	int32 QuoteCount = 0;

	const int32 ContentLen = Content.Len();
	FString CurrentPart;
	CurrentPart.Reserve(ContentLen);

	int32 LastPartStartIndex = 0;
	for (int32 CharIndex = 0; CharIndex < ContentLen; CharIndex++)
	{
		const TCHAR Char = Content[CharIndex];

		if (Char == TCHAR(' ') && QuoteCount == 0) // ignore any white space that is not between quotes (i.e. not a pin/variable name)
		{
			continue;
		}

		if (QuoteCount > 0)
		{
			if (Char == TCHAR('"'))
			{
				QuoteCount = 0;
			}
		}
		else if (Char == TCHAR('"'))
		{
			QuoteCount = 1;
		}

		if (Char == TCHAR('('))
		{
			if (QuoteCount == 0)
			{
				BraceCount++;
			}
		}
		else if (Char == TCHAR(')'))
		{
			if (QuoteCount == 0)
			{
				BraceCount--;
				BraceCount = FMath::Max<int32>(BraceCount, 0);
			}
		}
		else if (Char == TCHAR(',') && BraceCount == 0 && QuoteCount == 0)
		{
			Parts.Add(CurrentPart);
			LastPartStartIndex = CharIndex + 1;
			CurrentPart.Reset();
			continue;
		}

		CurrentPart.AppendChar(Char);
	}

	if (!Content.IsEmpty())
	{
		// ignore whitespaces from the start and end of the string
		Parts.Add(Content.Mid(LastPartStartIndex).TrimStartAndEnd());
	}
	return Parts;
}

// Sanitizes a name as per ruleset
void RigVMStringUtils::SanitizeName(FString& InOutName, bool bAllowPeriod, bool bAllowSpace, int32 InMaxNameLength)
{
	// Sanitize the name
	for (int32 i = 0; i < InOutName.Len(); ++i)
	{
		TCHAR& C = InOutName[i];

		const bool bGoodChar =
			FChar::IsAlpha(C) ||											// Any letter (upper and lowercase) anytime
			(C == '_') || (C == '-') || 									// _  and - anytime
			(bAllowPeriod && (C == '.')) ||
			(bAllowSpace && (C == ' ')) ||
			((i > 0) && FChar::IsDigit(C));									// 0-9 after the first character

		if (!bGoodChar)
		{
			C = '_';
		}
	}

	if (InOutName.Len() > InMaxNameLength)
	{
		InOutName.LeftChopInline(InOutName.Len() - InMaxNameLength);
	}
}
