// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTrace.h"
#include "ObjectTrace.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "RigVMHost.h"
#include "RigVMCore/RigVMExecuteContext.h"

#if RIGVM_TRACE_ENABLED

UE_TRACE_CHANNEL_DEFINE(RigVMChannel, "Traces RigVM (Rig Virtual Machine) execution and variable state for rewind debugging including literal \
memory, work memory, debug memory, external memory, draw interface, instruction visit info, and profiling data for \
debugging rig evaluation and control rig behavior.")

UE_TRACE_EVENT_DEFINE(RigVM, Literals);
UE_TRACE_EVENT_DEFINE(RigVM, Execute);

TAutoConsoleVariable<bool> CVarRigVMEnableRewindDebuggerTracing(TEXT("RigVM.EnableRewindDebugger"), 0, TEXT("Enable tracing of RigVM instances in RewindDebugger"));

void FRigVMTrace::RecordingStarted()
{
	if (!CVarRigVMEnableRewindDebuggerTracing.GetValueOnAnyThread())
	{
		return;
	}
	UE::Trace::ToggleChannel(TEXT("RigVM"), true);
}

void FRigVMTrace::RecordingStopped()
{
	UE::Trace::ToggleChannel(TEXT("RigVM"), false);
}

void FRigVMTrace::SetupRigVMEvaluation(FRigVMExtendedExecuteContext& InContext)
{
	URigVMHost* Host = InContext.Host;
	if (Host == nullptr)
	{
		return;
	}

	URigVM* VM = Host->GetVM();
	if (VM == nullptr)
	{
		return;
	}
	
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(RigVMChannel);
	if (!bChannelEnabled || VM == nullptr)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(Host))
	{
		return;
	}

	// determine the world this host is using within.
	// for control rig this funnels through the object
	// binding and will find the level this is instantiated within.
	UWorld* World = Host->GetWorld();
	if (!World || CANNOT_TRACE_OBJECT(World) || CANNOT_TRACE_OBJECT(VM))
	{
		return;
	}

#if WITH_EDITOR
	Host->SetIsInDebugMode(true);
#endif
	
	VM->CreateDebugMemory(InContext);
	InContext.MarkAllOperandsForDebugging();
	
	TRACE_OBJECT_WITH_OUTER(Host, Host->GetBoundOuterForTrace());
}

void FRigVMTrace::TraceRigVMEvaluation(const FRigVMExtendedExecuteContext& InContext)
{
	URigVMHost* Host = InContext.Host;
	if (Host == nullptr)
	{
		return;
	}

	URigVM* VM = Host->GetVM();
	if (VM == nullptr)
	{
		return;
	}
	
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(RigVMChannel);
	if (!bChannelEnabled || VM == nullptr)
	{
		return;
	}

	if (CANNOT_TRACE_OBJECT(Host))
	{
		return;
	}
	
	// determine the world this host is using within.
	// for control rig this funnels through the object
	// binding and will find the level this is instantiated within.
	UWorld* World = Host->GetWorld();
	if (!World || CANNOT_TRACE_OBJECT(World) || CANNOT_TRACE_OBJECT(VM))
	{
		return;
	}

	TRACE_OBJECT_WITH_OUTER(Host, Host->GetBoundOuterForTrace());

	const bool bTraceLiterals = TRACE_OBJECT(VM);
	constexpr bool bTraceExecute = true; 

	FRigVMTraceArchive LiteralMemoryArchive;
	if (bTraceLiterals)
	{
		FRigVMTraceArchiveWriter LiteralMemoryWriter(LiteralMemoryArchive);
		LiteralMemoryWriter << VM->GetDefaultLiteralMemory();
	}

	FRigVMTraceArchive WorkMemoryArchive;
	if(bTraceExecute)
	{
		FRigVMTraceArchiveWriter WorkMemoryWriter(WorkMemoryArchive);
		WorkMemoryWriter << const_cast<FRigVMMemoryStorageStruct&>(InContext.WorkMemoryStorage);
	}

	FRigVMTraceArchive DebugMemoryArchive;
	if(bTraceExecute)
	{
		FRigVMTraceArchiveWriter DebugMemoryWriter(DebugMemoryArchive);
		DebugMemoryWriter << const_cast<FRigVMMemoryStorageStruct&>(InContext.DebugMemoryStorage);
	}

	FRigVMTraceArchive ExternalMemoryArchive;
	if (bTraceExecute)
	{
		FRigVMTraceArchiveWriter ExternalMemoryWriter(ExternalMemoryArchive);
		{
			FBinaryArchiveFormatter BinaryArchiveFormatter(ExternalMemoryWriter);
			{
				FStructuredArchive StructuredArchive(BinaryArchiveFormatter);
				{
					FStructuredArchive::FRecord RootRecord = StructuredArchive.Open().EnterRecord();
					{
						TArray<FName> FieldNames;
						const TArray<FRigVMExternalVariable> ExternalVariables = Host->GetExternalVariablesImpl(false);
						TArray<FRigVMExternalVariable> FilteredExternalVariables;
						for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
						{
							if (ExternalVariable.GetProperty() && ExternalVariable.GetMemory())
							{
								FilteredExternalVariables.Add(ExternalVariable);
								const FName FieldName = FilteredExternalVariables.Last().GetProperty()->GetFName();
								FieldNames.Add(FieldName);
							}
						}

						ExternalMemoryWriter << FieldNames;
						
						for (FRigVMExternalVariable& FilteredExternalVariable : FilteredExternalVariables)
						{
							check (FilteredExternalVariable.GetProperty() && FilteredExternalVariable.GetMemory())

							const FName FieldName = FilteredExternalVariable.GetProperty()->GetFName();
							FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*FieldName.ToString());
							FilteredExternalVariable.GetProperty()->SerializeItem(PropertyField, FilteredExternalVariable.GetMemory());
						}
					}
				}
			}
		}
	}

	FRigVMTraceArchive DrawInterfaceArchive;
	if (bTraceExecute)
	{
		FRigVMTraceArchiveWriter DrawInterfaceWriter(DrawInterfaceArchive);
		{
			FBinaryArchiveFormatter BinaryArchiveFormatter(DrawInterfaceWriter);
			{
				FStructuredArchive StructuredArchive(BinaryArchiveFormatter);
				{
					FStructuredArchive::FRecord RootRecord = StructuredArchive.Open().EnterRecord();
					{
						if (const FProperty* DrawInterfaceProperty = URigVMHost::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URigVMHost, DrawInterface)))
						{
							FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*DrawInterfaceProperty->GetName());
							DrawInterfaceProperty->SerializeItem(PropertyField, &Host->GetDrawInterface());
						}
						if (const FProperty* DrawContainerProperty = URigVMHost::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URigVMHost, DrawContainer)))
						{
							FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*DrawContainerProperty->GetName());
							DrawContainerProperty->SerializeItem(PropertyField, &Host->GetDrawContainer());
						}
					}
				}
			}
		}
	}

	FRigVMTraceArchive HostConstantDataArchive;
	if (bTraceLiterals)
	{
		FRigVMTraceArchiveWriter HostConstantDataWriter(HostConstantDataArchive);
		Host->TraceConstantData(HostConstantDataWriter);
	}

	FRigVMTraceArchive HostExecuteDataArchive;
	if (bTraceExecute)
	{
		FRigVMTraceArchiveWriter HostExecuteDataWriter(HostExecuteDataArchive);
		Host->TraceExecuteData(HostExecuteDataWriter);
	}

	FString JoinedEntries;
	if (bTraceExecute)
	{
		TArray<FString> Entries;
		for (const int32& EntryIndex : InContext.EntriesBeingExecuted)
		{
			if (EntryIndex >= 0 && EntryIndex < VM->GetByteCode().NumEntries())
			{
				Entries.AddUnique(VM->GetByteCode().GetEntry(EntryIndex).Name.ToString());
			}
		}
		if (Entries.Num() > 0)
		{
			JoinedEntries = RigVMStringUtils::JoinStrings(Entries, TEXT(","));
		}
	}

	static const TArray<int32> EmptyInt32Array;
	static const TArray<uint64> EmptyUInt64Array;

#if WITH_EDITOR
	FRigVMInstructionVisitInfo* InstructionVisitInfo = InContext.GetRigVMInstructionVisitInfo();
	const TArray<int32>& VisitedInstructions = InstructionVisitInfo ? InstructionVisitInfo->GetInstructionVisitOrder() : EmptyInt32Array;

	FRigVMProfilingInfo* ProfilingInfo = InContext.GetRigVMProfilingInfo();
	uint64 OverallCycles = ProfilingInfo ? ProfilingInfo->GetOverallCycles() : 0;
	const TArray<uint64>& InstructionCycles = ProfilingInfo ? ProfilingInfo->InstructionCyclesDuringLastRun : EmptyUInt64Array;
#else
	const TArray<int32>& VisitedInstructions = EmptyInt32Array;
	uint64 OverallCycles = 0;
	const TArray<uint64>& InstructionCycles = EmptyUInt64Array;
#endif

	if (bTraceLiterals)
	{
		LiteralMemoryArchive.Compress();
		HostConstantDataArchive.Compress();

		FString HostGeneratedBy;
#if WITH_EDITOR
		if (const UClass* Class = Host->GetClass())
		{
			if (const UObject* GeneratedBy = Class->ClassGeneratedBy.Get())
			{
				if (const UPackage* Package = GeneratedBy->GetOutermost())
				{
					HostGeneratedBy = Package->GetPathName();
				}
			}
		}
#endif

		UE_TRACE_LOG(RigVM, Literals, RigVMChannel)
		<< Literals.Cycles(FPlatformTime::Cycles64())
		<< Literals.RecordingTime(FObjectTrace::GetObjectWorldElapsedTime(World))
		<< Literals.RigVMObjectVersion(FRigVMObjectVersion::LatestVersion)
		<< Literals.HostId(FObjectTrace::GetObjectId(Host))
		<< Literals.BoundOuterId(FObjectTrace::GetObjectId(Host->GetBoundOuterForTrace()))
		<< Literals.VMHash(InContext.VMHash)
		<< Literals.ByteCodeHash(VM->GetByteCode().GetByteCodeHash())
		<< Literals.LiteralMemory(LiteralMemoryArchive.GetOverallData(), LiteralMemoryArchive.OverallNum())
		<< Literals.HostGeneratedBy(*HostGeneratedBy, HostGeneratedBy.Len())
		<< Literals.HostConstantData(HostConstantDataArchive.GetOverallData(), HostConstantDataArchive.OverallNum());
	}

	if (bTraceExecute)
	{
		WorkMemoryArchive.Compress();
		DebugMemoryArchive.Compress();
		ExternalMemoryArchive.Compress();
		DrawInterfaceArchive.Compress();
		HostExecuteDataArchive.Compress();

		UE_TRACE_LOG(RigVM, Execute, RigVMChannel)
		<< Execute.Cycles(FPlatformTime::Cycles64())
		<< Execute.RecordingTime(FObjectTrace::GetObjectWorldElapsedTime(World))
		<< Execute.AbsoluteTime(Host->GetAbsoluteTime())
		<< Execute.DeltaTime(Host->GetDeltaTime())
		<< Execute.HostId(FObjectTrace::GetObjectId(Host))
		<< Execute.WorldId(FObjectTrace::GetObjectId(World))
		<< Execute.RigVMObjectVersion(FRigVMObjectVersion::LatestVersion)
		<< Execute.WorkMemory(WorkMemoryArchive.GetOverallData(), WorkMemoryArchive.OverallNum())
		<< Execute.DebugMemory(DebugMemoryArchive.GetOverallData(), DebugMemoryArchive.OverallNum())
		<< Execute.ExternalMemory(ExternalMemoryArchive.GetOverallData(), ExternalMemoryArchive.OverallNum())
		<< Execute.Entries(*JoinedEntries, JoinedEntries.Len())
		<< Execute.VisitedInstructions(VisitedInstructions.GetData(), VisitedInstructions.Num())
		<< Execute.OverallCycles(OverallCycles)
		<< Execute.InstructionCycles(InstructionCycles.GetData(), InstructionCycles.Num())
		<< Execute.DrawInterface(DrawInterfaceArchive.GetOverallData(), DrawInterfaceArchive.OverallNum())
		<< Execute.HostExecuteData(HostExecuteDataArchive.GetOverallData(), HostExecuteDataArchive.OverallNum());
	}
}

bool FRigVMTrace::RestoreRigVMConstantData(FRigVMExtendedExecuteContext& InOutContext, const FRigVMTraceConstantData& InConstantData)
{
	URigVMHost* Host = InOutContext.Host;
	if (Host == nullptr)
	{
		return false;
	}

	URigVM* VM = InOutContext.GetVM();
	if (VM == nullptr)
	{
		return false;
	}

	if (VM->GetVMHash() != InConstantData.VMHash)
	{
		return false;
	}

	Host->StartPlayingRewindDebugTrace();

	// restore / load the literal memory
	{
		FRigVMTraceArchiveReader LiteralMemoryReader(const_cast<FRigVMTraceArchive&>(InConstantData.LiteralMemory));
		VM->GetDefaultLiteralMemory().Serialize(LiteralMemoryReader);
	}

	// restore host relevant constant data
	{
		FRigVMTraceArchiveReader HostConstantDataReader(const_cast<FRigVMTraceArchive&>(InConstantData.HostConstantData));
		Host->LoadTracedConstantData(HostConstantDataReader);
	}

	return true;
}

bool FRigVMTrace::RestoreRigVMExecuteData(FRigVMExtendedExecuteContext& InOutContext, const FRigVMTraceExecuteData& InExecuteData)
{
#if WITH_EDITOR
	
	URigVMHost* Host = InOutContext.Host;
	if (Host == nullptr)
	{
		return false;
	}

	URigVM* VM = InOutContext.GetVM();
	if (VM == nullptr)
	{
		return false;
	}

	Host->StartPlayingRewindDebugTrace();

	// restore the absolute and delta times
	Host->SetAbsoluteAndDeltaTime(static_cast<float>(InExecuteData.AbsoluteTime), static_cast<float>(InExecuteData.DeltaTime));

	// restore the entries currently running
	TArray<FString> Entries;
	if (!InExecuteData.Entries.IsEmpty())
	{
		RigVMStringUtils::SplitString(InExecuteData.Entries, TEXT(","), Entries);
	}

	TArray<FString> EntriesInByteCode;
	for(int32 EntryIndex = 0; EntryIndex < VM->GetByteCode().NumEntries(); EntryIndex++)
	{
		const FRigVMByteCodeEntry& Entry = VM->GetByteCode().GetEntry(EntryIndex);
		EntriesInByteCode.Add(Entry.Name.ToString());
	}

	InOutContext.EntriesBeingExecuted.Reset();
	for (const FString& Entry : Entries)
	{
		const int32 EntryIndex = EntriesInByteCode.Find(Entry);
		if (EntryIndex != INDEX_NONE)
		{
			InOutContext.EntriesBeingExecuted.Add(EntryIndex);
		}
	}

	// retore the instruction visited tracking
	if (InOutContext.InstructionVisitInfo)
	{
		InOutContext.InstructionVisitInfo->SetupInstructionTracking(VM->GetByteCode().GetNumInstructions(), VM->GetByteCode().NumCallables());

		if (Entries.Num() > 0)
		{
			InOutContext.InstructionVisitInfo->FirstEntryEventInQueue = *Entries[0];
		}

		InOutContext.InstructionVisitInfo->InstructionVisitOrder = InExecuteData.VisitedInstructions;
		for (const int32& VisitedInstruction : InExecuteData.VisitedInstructions)
		{
			if (InOutContext.InstructionVisitInfo->InstructionVisitedDuringLastRun.IsValidIndex(VisitedInstruction))
			{
				InOutContext.InstructionVisitInfo->InstructionVisitedDuringLastRun[VisitedInstruction]++;
			}
		}
	}

	// restore the profiling info
	if (InOutContext.ProfilingInfo)
	{
		InOutContext.ProfilingInfo->SetupInstructionTracking(VM->GetByteCode().GetNumInstructions(), VM->GetByteCode().NumCallables(), true);
		InOutContext.ProfilingInfo->OverallCycles = InExecuteData.OverallCycles;
		InOutContext.ProfilingInfo->LastExecutionMicroSeconds = static_cast<double>(InOutContext.ProfilingInfo->OverallCycles) * FPlatformTime::GetSecondsPerCycle() * 1000.0 * 1000.0;
		InOutContext.ProfilingInfo->InstructionCyclesDuringLastRun = InExecuteData.InstructionCycles;
	}
	
	// restore / load the work memory
	if (!InExecuteData.WorkMemory.IsPayloadEmpty())
	{
		if (FRigVMMemoryStorageStruct* WorkMemory = VM->GetWorkMemory(InOutContext))
		{
			FRigVMTraceArchiveReader WorkMemoryReader(const_cast<FRigVMTraceArchive&>(InExecuteData.WorkMemory));
			WorkMemory->Serialize(WorkMemoryReader);
		}
		else
		{
			return false;
		}
	}

	// restore / load the debug memory
	if (!InExecuteData.DebugMemory.IsPayloadEmpty())
	{
		FRigVMMemoryStorageStruct* DebugMemory = VM->GetDebugMemory(InOutContext);
		if (!DebugMemory || DebugMemory->Num() == 0)
		{
			DebugMemory = VM->CreateDebugMemory(InOutContext);
			InOutContext.MarkAllOperandsForDebugging();
		}

		FRigVMTraceArchiveReader DebugMemoryReader(const_cast<FRigVMTraceArchive&>(InExecuteData.DebugMemory));
		DebugMemory->Serialize(DebugMemoryReader);
	}

	// restore / load the external memory
	if (!InExecuteData.ExternalMemory.IsPayloadEmpty())
	{
		FRigVMTraceArchiveReader ExternalMemoryReader(const_cast<FRigVMTraceArchive&>(InExecuteData.ExternalMemory));
		{
			FBinaryArchiveFormatter BinaryArchiveFormatter(ExternalMemoryReader);
			{
				FStructuredArchive StructuredArchive(BinaryArchiveFormatter);
				{
					FStructuredArchive::FRecord RootRecord = StructuredArchive.Open().EnterRecord();
					{
						TArray<FName> FieldNames;
						ExternalMemoryReader << FieldNames;

						TArray<FRigVMExternalVariable> ExternalVariables = Host->GetExternalVariablesImpl(false);
						for (FRigVMExternalVariable& ExternalVariable : ExternalVariables)
						{
							if (ExternalVariable.GetProperty() && ExternalVariable.GetMemory())
							{
								const FName FieldName = ExternalVariable.GetProperty()->GetFName();
								if (!FieldNames.Contains(FieldName))
								{
									continue;
								}
								FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*FieldName.ToString());
								ExternalVariable.GetProperty()->SerializeItem(PropertyField, ExternalVariable.GetMemory());
							}
						}
					}
				}
			}
		}
	}

	// restore / load the debug draw interface
	if (!InExecuteData.DrawInterface.IsPayloadEmpty())
	{
		FRigVMTraceArchiveReader DrawInterfaceReader(const_cast<FRigVMTraceArchive&>(InExecuteData.DrawInterface));
		{
			FBinaryArchiveFormatter BinaryArchiveFormatter(DrawInterfaceReader);
			{
				FStructuredArchive StructuredArchive(BinaryArchiveFormatter);
				{
					FStructuredArchive::FRecord RootRecord = StructuredArchive.Open().EnterRecord();
					{
						const FProperty* DrawInterfaceProperty = URigVMHost::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URigVMHost, DrawInterface));
						check(DrawInterfaceProperty);
						{
							FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*DrawInterfaceProperty->GetName());
							DrawInterfaceProperty->SerializeItem(PropertyField, &Host->GetDrawInterface());
						}
						const FProperty* DrawContainerProperty = URigVMHost::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URigVMHost, DrawContainer));
						check(DrawContainerProperty);
						{
							FStructuredArchiveSlot PropertyField = RootRecord.EnterField(*DrawContainerProperty->GetName());
							DrawContainerProperty->SerializeItem(PropertyField, &Host->GetDrawContainer());
						}
					}
				}
			}
		}
	}

	// restore host relevant execute data
	if (!InExecuteData.HostExecuteData.IsPayloadEmpty())
	{
		FRigVMTraceArchiveReader HostExecuteDataReader(const_cast<FRigVMTraceArchive&>(InExecuteData.HostExecuteData));
		Host->LoadTracedExecuteData(HostExecuteDataReader);
	}

#endif

	return true;
}

#endif

