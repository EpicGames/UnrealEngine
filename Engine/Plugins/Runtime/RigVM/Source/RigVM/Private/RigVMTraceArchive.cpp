// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTraceArchive.h"
#include "ObjectTrace.h"
#include "RigVMTrace.h"
#include "Misc/Compression.h"
#include "RigVMObjectVersion.h"

FRigVMTraceArchive::FRigVMTraceArchive()
{
	Reset();
}

void FRigVMTraceArchive::Reset()
{
	Buffer.Reset();
	Buffer.AddZeroed(sizeof(FHeader));
	Header() = FHeader();
}

void FRigVMTraceArchive::Empty()
{
	Buffer.Empty();
	Buffer.AddZeroed(sizeof(FHeader));
	Header() = FHeader();
}

bool FRigVMTraceArchive::IsPayloadEmpty() const
{
	return PayloadNum() == 0;
}

bool FRigVMTraceArchive::Compress()
{
	if(IsCompressed())
	{
		return false;
	}

	if(IsPayloadEmpty())
	{
		return false;
	}

	const FHeader OriginalHeader = Header();
	verify(OriginalHeader.UncompressedSize != INDEX_NONE);
	verify(OriginalHeader.UncompressedSize == PayloadNum());
	
	// It is possible for compression to actually increase the size of the data, so we over allocate here to handle that.
	const int32 UpperCompressedSizePayloadOnly = FCompression::CompressMemoryBound(NAME_Oodle, OriginalHeader.UncompressedSize);
	const int32 UpperCompressedSizeIncludingHeader = UpperCompressedSizePayloadOnly + sizeof(FHeader);

	TArray<uint8> CompressedBuffer;
	CompressedBuffer.SetNumUninitialized(UpperCompressedSizeIncludingHeader);

	int32 CompressedSizePayloadOnly = UpperCompressedSizePayloadOnly;

	// compress the payload only
	if (FCompression::CompressMemory(NAME_Oodle, CompressedBuffer.GetData() + sizeof(FHeader), CompressedSizePayloadOnly, GetPayloadData(), PayloadNum(), COMPRESS_BiasMemory))
	{
		// In the case that compressing it actually increases the size, we leave it uncompressed
		if (CompressedSizePayloadOnly < OriginalHeader.UncompressedSize)
		{
			CompressedBuffer.SetNum(CompressedSizePayloadOnly + sizeof(FHeader));
			Buffer = MoveTemp(CompressedBuffer);
			Buffer.Shrink();

			FHeader& CurrentHeader = Header();
			CurrentHeader = OriginalHeader;
			CurrentHeader.CompressedSize = CompressedSizePayloadOnly;
			CurrentHeader.bIsCompressed = true;
			
			verify(CurrentHeader.CompressedSize != INDEX_NONE);
			verify(CurrentHeader.CompressedSize == PayloadNum());
			verify(CurrentHeader.UncompressedSize != INDEX_NONE);
			verify(CurrentHeader.UncompressedSize > CurrentHeader.CompressedSize);
			return true;
		}
	}

	return false;
}

bool FRigVMTraceArchive::Decompress()
{
	if(!IsCompressed())
	{
		return true;
	}

	const FHeader OriginalHeader = Header();
	verify(OriginalHeader.CompressedSize != INDEX_NONE);
	verify(OriginalHeader.UncompressedSize != INDEX_NONE);
	verify(OriginalHeader.UncompressedSize > OriginalHeader.CompressedSize);
	verify(OriginalHeader.CompressedSize == PayloadNum());
	
	TArray<uint8> UncompressedBuffer;
	UncompressedBuffer.SetNumUninitialized(OriginalHeader.UncompressedSize + sizeof(FHeader));

	if (FCompression::UncompressMemory(NAME_Oodle, UncompressedBuffer.GetData() + sizeof(FHeader), OriginalHeader.UncompressedSize, GetPayloadData(), PayloadNum()))
	{
		Buffer = MoveTemp(UncompressedBuffer);
		
		FHeader& CurrentHeader = Header();
		CurrentHeader = OriginalHeader;
		CurrentHeader.CompressedSize = INDEX_NONE;
		CurrentHeader.bIsCompressed = false;
		
		verify(CurrentHeader.UncompressedSize != INDEX_NONE);
		verify(CurrentHeader.UncompressedSize == PayloadNum());
		return true;
	}

	return false;
}

const FRigVMTraceArchive::FHeader& FRigVMTraceArchive::GetHeader() const
{
	return const_cast<FRigVMTraceArchive*>(this)->Header();
}

FRigVMTraceArchive::FHeader& FRigVMTraceArchive::Header()
{
	check(Buffer.Num() >= sizeof(FHeader));
	return *reinterpret_cast<FHeader*>(GetOverallData());
}

bool FRigVMTraceArchive::IsCompressed() const
{
	return GetHeader().bIsCompressed;
}

int32 FRigVMTraceArchive::GetUncompressedSize() const
{
	return GetHeader().UncompressedSize;
}

int32 FRigVMTraceArchive::GetCompressedSize() const
{
	return GetHeader().CompressedSize;
}

uint8* FRigVMTraceArchive::GetPayloadData()
{
	if (IsPayloadEmpty())
	{
		return nullptr;
	}
	return Buffer.GetData() + sizeof(FHeader);
}

const uint8* FRigVMTraceArchive::GetPayloadData() const
{
	if (IsPayloadEmpty())
	{
		return nullptr;
	}
	return Buffer.GetData() + sizeof(FHeader);
}

int32 FRigVMTraceArchive::PayloadNum() const
{
	return FMath::Max<int32>(0, Buffer.Num() - sizeof(FHeader));
}

uint8* FRigVMTraceArchive::GetOverallData()
{
	return Buffer.GetData();
}

const uint8* FRigVMTraceArchive::GetOverallData() const
{
	return Buffer.GetData();
}

int32 FRigVMTraceArchive::OverallNum() const
{
	return Buffer.Num();
}

void FRigVMTraceArchive::SetOverallBuffer(const TArrayView<const uint8>& InBuffer)
{
	SetOverallBuffer(TArrayView<uint8>(const_cast<uint8*>(InBuffer.GetData()), InBuffer.Num()));
}

void FRigVMTraceArchive::SetOverallBuffer(const TArrayView<uint8>& InBuffer)
{
	if (InBuffer.IsEmpty())
	{
		Reset();
		return;
	}
	
	check(InBuffer.Num() >= sizeof(FHeader));
	Buffer.Reset();
	Buffer.Append(InBuffer);

	FHeader& H = Header();
		
	if (H.bIsCompressed)
	{
		verify(H.CompressedSize != INDEX_NONE);
		verify(H.UncompressedSize != INDEX_NONE);
		verify(H.CompressedSize < H.UncompressedSize)
	}
	else
	{
		verify(H.CompressedSize == INDEX_NONE);
	}

	verify(H.VersionsOffset < H.UncompressedSize)
}

FRigVMTraceArchiveWriter::FRigVMTraceArchiveWriter(FRigVMTraceArchive& InArchive)
: Archive( InArchive )
, Offset(0)
{
	SetIsSaving(true);
	SetIsLoading(false);
	UsingCustomVersion(FRigVMObjectVersion::GUID);
	ArIgnoreOuterRef = 1;
}

FRigVMTraceArchiveWriter::~FRigVMTraceArchiveWriter()
{
	if (IsSaving())
	{
		WriteVersions();
		
		FRigVMTraceArchive::FHeader& H = Archive.Header();
		verify(!H.bIsCompressed);
		verify(H.CompressedSize == INDEX_NONE);
		H.UncompressedSize = Archive.PayloadNum();
	}
}

void FRigVMTraceArchiveWriter::Serialize(void* V, int64 Length)
{
	if(Length == 0)
	{
		check(V == nullptr);
	}
	else
	{
		check(V);
		int32 Start = IntCastChecked<int32, int64>(Offset) + sizeof(FRigVMTraceArchive::FHeader);

		if(Archive.Buffer.IsValidIndex(Start))
		{
			const int32 MissingBytes = IntCastChecked<int32, int64>(Length) - (Archive.Buffer.Num() - Start);
			if(MissingBytes > 0)
			{
				(void)Archive.Buffer.AddUninitialized(MissingBytes);
			}
		}
		else if(Start == Archive.Buffer.Num())
		{
			Start = Archive.Buffer.AddUninitialized(IntCastChecked<int32, int64>(Length));
		}
		else
		{
			checkNoEntry();
		}

		FMemory::Memcpy( Archive.GetOverallData() + Start, V, Length );
		Offset += Length;
	}
}

int64 FRigVMTraceArchiveWriter::Tell()
{
	return Offset;
}

int64 FRigVMTraceArchiveWriter::TotalSize()
{
	return Archive.PayloadNum();
}

void FRigVMTraceArchiveWriter::Seek(int64 InPos)
{
	check((InPos >= 0) && (InPos <= Archive.PayloadNum()));
	Offset = InPos;
}

void FRigVMTraceArchiveWriter::WriteUObject(const UObject* Obj)
{
	uint64 Id = 0;
	
#if RIGVM_TRACE_ENABLED
	if (Obj)
	{
		if (const uint64* ExistingId = ObjectIdMap.Find(Obj))
		{
			Id = *ExistingId;
		}
		else
		{
			// first ensure the object has been traced 
			if (const UClass* Class = Cast<UClass>(Obj))
			{
				TRACE_TYPE(Class);
			}
			TRACE_OBJECT(Obj);
		
			// then serialize the Object as a Trace Object id;
			Id = FObjectTrace::GetObjectId(Obj);

			// store this in the map for faster lookup
			ObjectIdMap.Add(Obj, Id);
		}
	}
#endif

	*this << Id;
}

FArchive& FRigVMTraceArchiveWriter::operator<<(UObject*& Obj)
{
	WriteUObject(Obj);
	return *this;
}

FArchive& FRigVMTraceArchiveWriter::operator<<(FName& Value)
{
	if(const int64* StringOffsetPtr = NameToOffset.Find(Value))
	{
		uint8 State = StoringStringAsOffset;
		*this << State;
		
		int64 StringOffset = *StringOffsetPtr;
		*this << StringOffset;
	}
	else
	{
		uint8 State = StoringStringAsString;
		*this << State;
		NameToOffset.Add(Value, Tell());
		FString NameAsString = Value.IsNone() ? FString() : Value.ToString();
		*this << NameAsString;
	}
	return *this;
}

FArchive& FRigVMTraceArchiveWriter::operator<<(FText& Value)
{
	FString ValueString = Value.ToString();
	*this << ValueString;
	return *this;
}

void FRigVMTraceArchiveWriter::WriteVersions()
{
	verify(!Archive.Header().bIsCompressed);
	
	const FCustomVersionContainer& Versions = GetCustomVersions();
	FCustomVersionArray VersionArray = Versions.GetAllVersions();
	if (VersionArray.IsEmpty())
	{
		return;
	}
	Archive.Header().NumVersions = VersionArray.Num();
	Archive.Header().VersionsOffset = Offset;
	*this << VersionArray;
}

FRigVMTraceArchiveReader::FRigVMTraceArchiveReader(FRigVMTraceArchive& InArchive)
: FRigVMTraceArchiveWriter(InArchive)
{
	SetIsSaving(false);
	SetIsLoading(true);
	if (!Archive.Decompress())
	{
		checkNoEntry();
		return;
	}
	ReadVersions();
}

void FRigVMTraceArchiveReader::Serialize(void* V, int64 Length)
{
	if(Length == 0)
	{
		check(V == nullptr);
	}
	else
	{
		check(V);
		check(Archive.PayloadNum() >= Offset + Length);
		FMemory::Memcpy( V, Archive.GetPayloadData() + IntCastChecked<int32, int64>(Offset), Length );
		Offset += Length;
	}
}

void FRigVMTraceArchiveReader::ReadUObject(UObject*& Obj)
{
	uint64 Id = 0;
	const int64 OffsetOfId = Tell();
	*this << Id;

#if RIGVM_TRACE_ENABLED
	if (Id == 0)
	{
		Obj = nullptr;
	}
	else
	{
		Obj = FObjectTrace::GetObjectFromId(Id);
	}
#else
	Obj = nullptr;
#endif
}

FArchive& FRigVMTraceArchiveReader::operator<<(UObject*& Obj)
{
	ReadUObject(Obj);
	return *this;
}

FArchive& FRigVMTraceArchiveReader::operator<<(FName& Value)
{
	uint8 State = UINT8_MAX;
	*this << State;

	auto ReadNameString = [this, &Value]()
	{
		const int64 OffsetOfString = Tell();
		FString NameAsString;
		*this << NameAsString;
		if(NameAsString.IsEmpty())
		{
			Value = FName(NAME_None);
		}
		else
		{
			Value = *NameAsString;
		}
		OffsetToName.FindOrAdd(OffsetOfString) = Value;
	};

	if(State == StoringStringAsString)
	{
		// read the string at the current position
		ReadNameString();
	}
	else if(State == StoringStringAsOffset)
	{
		int64 OffsetOfString = -1;
		*this << OffsetOfString;

		const int32 OffsetOfString32 = IntCastChecked<int32, int64>(OffsetOfString);
		check(OffsetOfString32 >= 0 && OffsetOfString32 < Archive.PayloadNum());
		
		if(const FName* NamePtr = OffsetToName.Find(OffsetOfString))
		{
			Value = *NamePtr;
		}
		else
		{
			TGuardValue<int64> OffsetGuard(Offset, Offset);
			Seek(OffsetOfString);
			ReadNameString();
		}
	}
	else
	{
		checkNoEntry();
	}

	return *this;
}

FArchive& FRigVMTraceArchiveReader::operator<<(FText& Value)
{
	FString ValueString;
	*this << ValueString;
	Value = FText::FromString(ValueString);
	return *this;
}

void FRigVMTraceArchiveReader::ReadVersions()
{
	const FRigVMTraceArchive::FHeader Header = Archive.Header();
	verify(!Header.bIsCompressed);
	verify(Header.CompressedSize == INDEX_NONE);
	
	const int32 NumVersions = Header.NumVersions; 
	if (NumVersions > 0)
	{
		const int64 VersionOffset = Header.VersionsOffset;
		check(VersionOffset != INDEX_NONE);

		TGuardValue<int64> OffsetGuard(Offset, Offset);
		Seek(VersionOffset);

		FCustomVersionArray VersionArray;
		*this << VersionArray;

		FCustomVersionContainer VersionContainer;
		for (const FCustomVersion& Version : VersionArray)
		{
			const FName FriendlyName = Version.GetFriendlyName();
			VersionContainer.SetVersion(Version.Key, Version.Version, FriendlyName);
		}
		SetCustomVersions(VersionContainer);
	}
}

FArchive& operator<<(FArchive& Ar, FRigVMTraceArchive& Data)
{
	Ar << Data.Buffer;
	return Ar;
}
