// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMTypeUtils.h: Module implementation.
=============================================================================*/

#include "RigVMTypeUtils.h"
#include "RigVMModule.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/UserDefinedEnum.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/LinkerLoad.h"
#include "Interfaces/IPluginManager.h"


static FDelayedAutoRegisterHelper GRigVMPluginPrefixCacheInit(EDelayedRegisterRunPhase::EndOfEngineInit,
	[]() { FRigVMPluginsPrefixCache::Get()->Refresh(); });

namespace RigVMTypeUtils
{
	const TCHAR BoolType[] = TEXT("bool");
	const TCHAR FloatType[] = TEXT("float");
	const TCHAR DoubleType[] = TEXT("double");
	const TCHAR IntType[] = TEXT("int");
	const TCHAR Int32Type[] = TEXT("int32");
	const TCHAR Int64Type[] = TEXT("int64");
	const TCHAR UInt8Type[] = TEXT("uint8");
	const TCHAR UInt32Type[] = TEXT("uint32");
	const TCHAR UInt64Type[] = TEXT("uint64");
	const TCHAR FNameType[] = TEXT("FName");
	const TCHAR FStringType[] = TEXT("FString");
	const TCHAR FTextType[] = TEXT("FText");
	const TCHAR BoolArrayType[] = TEXT("TArray<bool>");
	const TCHAR FloatArrayType[] = TEXT("TArray<float>");
	const TCHAR DoubleArrayType[] = TEXT("TArray<double>");
	const TCHAR Int32ArrayType[] = TEXT("TArray<int32>");
	const TCHAR UInt32ArrayType[] = TEXT("TArray<uint32>");
	const TCHAR UInt8ArrayType[] = TEXT("TArray<uint8>");
	const TCHAR FNameArrayType[] = TEXT("TArray<FName>");
	const TCHAR FStringArrayType[] = TEXT("TArray<FString>");
	const TCHAR FTextArrayType[] = TEXT("TArray<FText>");

	const FLazyName BoolTypeName(BoolType);
	const FLazyName FloatTypeName(FloatType);
	const FLazyName DoubleTypeName(DoubleType);
	const FLazyName IntTypeName(IntType);
	const FLazyName Int32TypeName(Int32Type);
	const FLazyName Int64TypeName(Int64Type);
	const FLazyName UInt8TypeName(UInt8Type);
	const FLazyName UInt32TypeName(UInt32Type);
	const FLazyName UInt64TypeName(UInt64Type);
	const FLazyName FNameTypeName(FNameType);
	const FLazyName FStringTypeName(FStringType);
	const FLazyName FTextTypeName(FTextType);
	const FLazyName BoolArrayTypeName(BoolArrayType);
	const FLazyName FloatArrayTypeName(FloatArrayType);
	const FLazyName DoubleArrayTypeName(DoubleArrayType);
	const FLazyName Int32ArrayTypeName(Int32ArrayType);
	const FLazyName UInt32ArrayTypeName(UInt32ArrayType);
	const FLazyName UInt8ArrayTypeName(UInt8ArrayType);
	const FLazyName FNameArrayTypeName(FNameArrayType);
	const FLazyName FStringArrayTypeName(FStringArrayType);

	TRigVMTypeIndex TypeIndex::Execute = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::ExecuteArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::Bool = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::Float = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::Double = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::Int32 = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::UInt32 = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::UInt8 = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::FName = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::FString = INDEX_NONE;
	TRigVMTypeIndex TypeIndex::WildCard = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::BoolArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::FloatArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::DoubleArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::Int32Array = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::UInt32Array = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::UInt8Array = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::FNameArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::FStringArray = INDEX_NONE;	
	TRigVMTypeIndex TypeIndex::WildCardArray = INDEX_NONE;	
}

bool RigVMTypeUtils::RequiresCPPTypeObject(const FString& InCPPType)
{
	static const TCHAR* PrefixesRequiringCPPTypeObject[] = {
		TEXT("F"), 
		TEXT("E"), 
		TEXT("U"), 
		TEXT("TObjectPtr<"), 
		TEXT("TScriptInterface<"),
		TEXT("TSubclassOf<")
	};
	static const TCHAR* CPPTypesNotRequiringCPPTypeObject[] = {
		TEXT("FString"), 
		TEXT("FName"), 
		TEXT("float"),
		TEXT("uint"),
		TEXT("uint8"),
		TEXT("uint16"),
		TEXT("uint32"),
		TEXT("uint64")
	};

	FStringView InCPPTypeView = InCPPType;
	while (InCPPTypeView.StartsWith(TEXT("TArray<")) && InCPPTypeView.EndsWith(TEXT(">")))
	{
		InCPPTypeView.RemovePrefix(UE_ARRAY_COUNT("TArray<") - 1);
		InCPPTypeView.RemoveSuffix(1); // >
	}

	for(const TCHAR* Type : CPPTypesNotRequiringCPPTypeObject)
	{
		if(InCPPTypeView == Type)
		{
			return false;
		}
	}

	for(const TCHAR* Prefix : PrefixesRequiringCPPTypeObject)
	{
		if(InCPPTypeView.StartsWith(Prefix, ESearchCase::CaseSensitive))
		{
			return true;
		}
	}

	return false;
}

UObject* RigVMTypeUtils::FindObjectGlobally(const TCHAR* InObjectName, bool bUseRedirector)
{
	// Do a global search for the CPP type. Note that searching with ANY_PACKAGE _does not_
	// apply redirectors. So only if this fails do we apply them manually below.
	UObject* Object = FindFirstObject<UField>(InObjectName, EFindFirstObjectOptions::NativeFirst);
	if(Object != nullptr)
	{
		return Object;
	}

	// If its an enum, it might be defined as a namespace with the actual enum inside (see ERigVMClampSpatialMode). 
	FString ObjectNameStr(InObjectName);
	if (!ObjectNameStr.IsEmpty() && ObjectNameStr[0] == 'E')
	{
		FString Left, Right;
		if (ObjectNameStr.Split(TEXT("::"), &Left, &Right))
		{
			ObjectNameStr = Left;
			Object = FindFirstObject<UField>(*ObjectNameStr, EFindFirstObjectOptions::NativeFirst);
			if(Object != nullptr)
			{
				return Object;
			}
		}
	}

	if (!bUseRedirector)
	{
		return nullptr;
	}

	FCoreRedirectObjectName OldObjectName (ObjectNameStr);
	FCoreRedirectObjectName NewObjectName;
	const bool bFoundRedirect = FCoreRedirects::RedirectNameAndValues(
		ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Struct | ECoreRedirectFlags::Type_Enum,
		OldObjectName,
		NewObjectName,
		nullptr,
		ECoreRedirectMatchFlags::AllowPartialMatch); // AllowPartialMatch to allow redirects from one package to another

	if (!bFoundRedirect)
	{
		return nullptr;
	}

	const FString RedirectedObjectName = NewObjectName.ObjectName.ToString();
	UPackage *Package = nullptr;
	if (!NewObjectName.PackageName.IsNone())
	{
		Package = FindPackage(nullptr, *NewObjectName.PackageName.ToString());
	}
	if (Package != nullptr)
	{
		Object = FindObject<UField>(Package, *RedirectedObjectName);
	}
	if (Package == nullptr || Object == nullptr)
	{
		// Hail Mary pass.
		Object = FindFirstObject<UField>(*RedirectedObjectName, EFindFirstObjectOptions::NativeFirst);
	}
	return Object;
}

FString RigVMTypeUtils::CPPTypeFromObject(const UObject* InCPPTypeObject, EClassArgType InClassArgType)
{
	if (const UClass* Class = Cast<UClass>(InCPPTypeObject))
	{
		if (InClassArgType == EClassArgType::AsClass)
		{
			return FString::Printf(RigVMTypeUtils::TSubclassOfTemplate, Class->GetPrefixCPP(), *Class->GetName());
		}
		else if (Class->IsChildOf(UInterface::StaticClass()))
		{
			return FString::Printf(RigVMTypeUtils::TScriptInterfaceTemplate, TEXT("I"), *Class->GetName());
		}
		else
		{
			return FString::Printf(RigVMTypeUtils::TObjectPtrTemplate, Class->GetPrefixCPP(), *Class->GetName());
		}
	}
	else if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InCPPTypeObject))
	{
		return GetUniqueStructTypeName(ScriptStruct);
	}
	else if (const UEnum* Enum = Cast<UEnum>(InCPPTypeObject))
	{
		return RigVMTypeUtils::CPPTypeFromEnum(Enum);
	}

	return FString();
}

UObject* RigVMTypeUtils::UserDefinedTypeFromCPPType(FString& InOutCPPType, const FRigVMUserDefinedTypeResolver* InTypeResolver)
{
	const FString OriginalTypeName = InOutCPPType;
	UObject* CPPTypeObject = nullptr;
	InOutCPPType.Reset();

	// try to resolve the type name using a path name potentially
	if(InOutCPPType.IsEmpty() && InTypeResolver != nullptr && InTypeResolver->IsValid())
	{
		FString TypeNameToLookUp = OriginalTypeName;
		while(IsArrayType(TypeNameToLookUp))
		{
			TypeNameToLookUp = BaseTypeFromArrayType(TypeNameToLookUp);
		}

		// Ask the resolver to the name of the user-defined struct/enum to an object.
		// For example FUserDefinedStruct_23E408214EE9E6DA5BFADDA0F9F4F577 -> /Game/Animation/MyUserDefinedStruct.MyUserDefinedStruct
		CPPTypeObject = InTypeResolver->GetTypeObjectByName(TypeNameToLookUp); 
		if(CPPTypeObject)
		{
			InOutCPPType = PostProcessCPPType(OriginalTypeName, CPPTypeObject);
			return CPPTypeObject;
		}
	}

#if WITH_EDITOR
	
	// potentially this type hasn't been loaded yet. Let's try again by visiting relevant assets
	if(InOutCPPType.IsEmpty())
	{
		const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

		TArray<FAssetData> AssetDataList;
		FARFilter AssetFilter;
		AssetFilter.bRecursiveClasses = true;
		AssetFilter.bIncludeOnlyOnDiskAssets = IsRunningCookCommandlet();	// When cooking, only enumerate on-disk assets to ensure deterministic results

		InOutCPPType = OriginalTypeName;
		while(IsArrayType(InOutCPPType))
		{
			InOutCPPType = BaseTypeFromArrayType(InOutCPPType);
		}

		if(OriginalTypeName.Contains(TEXT("FUserDefinedStruct_")))
		{
			static const FLazyName GuidTag(GET_MEMBER_NAME_CHECKED(UUserDefinedStruct, Guid));

			AssetFilter.ClassPaths = { UUserDefinedStruct::StaticClass()->GetClassPathName() };
			AssetRegistry.GetAssets(AssetFilter, AssetDataList);

			// Temporary workaround:
			// User Defined Struct has been moved from Engine to CoreUObject at CL34495787 for UE-216472
			// But currently the AssetRegistry is not applying CoreRedirects to the ClassPath, so searching using the latest class path
			// will not give the complete list of assets.
			// To get the complete list, we need to search again using old names of the class. 
			// This can be removed once the AssetRegistry issue (UE-168245) is fixed
			TArray<FString> OldPathNames = FLinkerLoad::FindPreviousPathNamesForClass(UUserDefinedStruct::StaticClass()->GetClassPathName().ToString(), false);
			for (const FString& OldPathName : OldPathNames)
			{
				AssetFilter.ClassPaths = { FTopLevelAssetPath(OldPathName) };
				AssetRegistry.GetAssets(AssetFilter, AssetDataList);
			}

			// first pass - try to find it using the tag
			if(CPPTypeObject == nullptr)
			{
				for(const FAssetData& AssetData : AssetDataList)
				{
					if(AssetData.FindTag(GuidTag))
					{
						const FString GuidBasedName = GetUniqueStructTypeName(AssetData.GetTagValueRef<FGuid>(GuidTag));
						if(GuidBasedName == InOutCPPType)
						{
							// Don't force a load during async loading
							UObject* Obj = AssetData.FastGetAsset(!IsAsyncLoading());
							if (Obj == nullptr)
							{
								Obj = AssetData.ToSoftObjectPath().ResolveObject();
							}
							UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Obj);
							if (UserDefinedStruct == nullptr)
							{
								UserDefinedStruct = Cast<UUserDefinedStruct>(AssetData.GetAsset());
							}
							if (UserDefinedStruct)
							{
								CPPTypeObject = UserDefinedStruct;
								InOutCPPType = PostProcessCPPType(OriginalTypeName, CPPTypeObject);
								return CPPTypeObject;
							}
						}
					}
				}
			}

			// second pass - deal with the ones that don't have a tag and resolve them
			if(CPPTypeObject == nullptr)
			{
				for(const FAssetData& AssetData : AssetDataList)
				{
					if(AssetData.FindTag(GuidTag))
					{
						continue;
					}

					// Don't force a load during async loading
					UObject* Obj = AssetData.FastGetAsset(!IsAsyncLoading());
					if (Obj == nullptr)
					{
						Obj = AssetData.ToSoftObjectPath().ResolveObject();
					}
					UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Obj);
					if (UserDefinedStruct == nullptr)
					{
						UserDefinedStruct = Cast<UUserDefinedStruct>(AssetData.GetAsset());
					}
					if (UserDefinedStruct)
					{
						const FString GuidBasedName = GetUniqueStructTypeName(UserDefinedStruct);
						if(GuidBasedName == InOutCPPType)
						{
							CPPTypeObject = UserDefinedStruct;
							InOutCPPType = PostProcessCPPType(OriginalTypeName, CPPTypeObject);
							return CPPTypeObject;
						}
					}
				}
			}
		}
		else
		{
			AssetFilter.ClassPaths = { UUserDefinedEnum::StaticClass()->GetClassPathName() };
			AssetRegistry.GetAssets(AssetFilter, AssetDataList);

			for(const FAssetData& AssetData : AssetDataList)
			{
				// Don't force a load during async loading
				UObject* Obj = AssetData.FastGetAsset(!IsAsyncLoading());
				if (Obj == nullptr)
				{
					Obj = AssetData.ToSoftObjectPath().ResolveObject();
				}
				UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(Obj);
				if (UserDefinedEnum == nullptr)
				{
					UserDefinedEnum = Cast<UUserDefinedEnum>(AssetData.GetAsset());
				}
				if (UserDefinedEnum)
				{
					const FString EnumCPPName = CPPTypeFromEnum(UserDefinedEnum);
					if(EnumCPPName == InOutCPPType)
					{
						CPPTypeObject = UserDefinedEnum;
						InOutCPPType = PostProcessCPPType(OriginalTypeName, CPPTypeObject);
						return CPPTypeObject;
					}
				}
			}
		}
	}
#endif

	return CPPTypeObject;
}

UObject* RigVMTypeUtils::ObjectFromCPPType(FString& InOutCPPType, bool bUseRedirector, const FRigVMUserDefinedTypeResolver* InTypeResolver)
{
	if (!RequiresCPPTypeObject(InOutCPPType))
	{
		return nullptr;
	}

	// try to find the CPPTypeObject by name
	FString BaseCPPType = InOutCPPType;
	while (IsArrayType(BaseCPPType))
	{
		BaseCPPType = BaseTypeFromArrayType(BaseCPPType);
	}
	FString CPPType = BaseCPPType;
	const bool bIsClass = CPPType.StartsWith(TSubclassOfPrefix);

	static const FString PrefixObjectPtr = TObjectPtrPrefix;
	static const FString PrefixSubclassOf = TSubclassOfPrefix;
	static const FString PrefixScriptInterface = TScriptInterfacePrefix;

	if (CPPType.StartsWith(PrefixObjectPtr))
	{
		// Chop the prefix + the U indicating object class
		CPPType = CPPType.Mid(PrefixObjectPtr.Len() + 1, CPPType.Len() - (PrefixObjectPtr.Len() + 2));
	}
	else if (CPPType.StartsWith(PrefixSubclassOf))
	{
		// Chop the prefix + the U indicating object class
		CPPType = CPPType.Mid(PrefixSubclassOf.Len() + 1, CPPType.Len() - (PrefixSubclassOf.Len() + 2));
	}
	else if (CPPType.StartsWith(TScriptInterfacePrefix))
	{
		// Chop the prefix + the I indicating interface class
		CPPType = CPPType.Mid(PrefixScriptInterface.Len() + 1, CPPType.Len() - (PrefixScriptInterface.Len() + 2));
	}

	UObject* CPPTypeObject = FindObjectGlobally(*CPPType, bUseRedirector);
	if (CPPTypeObject == nullptr)
	{
		// If we've mistakenly stored the struct type with the 'F', 'U', or 'A' prefixes, we need to strip them
		// off first. Enums are always named with their prefix intact.
		if (!CPPType.IsEmpty() && (CPPType[0] == TEXT('F') || CPPType[0] == TEXT('U') || CPPType[0] == TEXT('A')))
		{
			CPPType = CPPType.Mid(1);
		}
		CPPTypeObject = RigVMTypeUtils::FindObjectGlobally(*CPPType, bUseRedirector);
	}

	if(CPPTypeObject == nullptr)
	{
		CPPType = BaseCPPType;
		CPPTypeObject = UserDefinedTypeFromCPPType(CPPType, InTypeResolver);
	}

	if(CPPTypeObject == nullptr)
	{
		InOutCPPType.Reset();
		return nullptr;
	}

	CPPType = CPPTypeFromObject(CPPTypeObject, bIsClass ? EClassArgType::AsClass : EClassArgType::AsObject);
	InOutCPPType.ReplaceInline(*BaseCPPType, *CPPType);
	return CPPTypeObject;
}

UObject* RigVMTypeUtils::GetCPPTypeObjectFromProperty(const FProperty* InProperty)
{
	UObject* CPPTypeObject = nullptr;
	
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		InProperty = ArrayProperty->Inner;
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		CPPTypeObject = StructProperty->Struct;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		CPPTypeObject = EnumProperty->GetEnum();
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		CPPTypeObject = ByteProperty->Enum;
	}
	else if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InProperty))
	{
		if(RigVMCore::SupportsUObjects())
		{
			CPPTypeObject = ClassProperty->MetaClass;
		}
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		if(RigVMCore::SupportsUObjects())
		{
			CPPTypeObject = ObjectProperty->PropertyClass;
		}
	}

	return CPPTypeObject;
}

FString RigVMTypeUtils::PostProcessCPPType(const FString& InCPPType, UObject* InCPPTypeObject, const FRigVMUserDefinedTypeResolver* InResolvalInfo, bool bIsSerializing)
{
	FString CPPType = InCPPType;
	CleanupCPPType(CPPType);
	
	if (InCPPTypeObject)
	{
		const bool bIsClass = CPPType.StartsWith(TSubclassOfPrefix);
		CPPType = CPPTypeFromObject(InCPPTypeObject, bIsClass ? EClassArgType::AsClass : EClassArgType::AsObject);
		if(CPPType != InCPPType)
		{
			FString TemplateType = InCPPType;
			while (RigVMTypeUtils::IsArrayType(TemplateType))
			{
				CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(CPPType);
				TemplateType = RigVMTypeUtils::BaseTypeFromArrayType(TemplateType);
			}		
		}
	}
	else if (RequiresCPPTypeObject(CPPType))
	{
		if (!bIsSerializing)
		{
			// Uses redirectors and updates the CPPType if necessary
			ObjectFromCPPType(CPPType, true, InResolvalInfo);
		}
	}

	return CPPType;
}

bool RigVMTypeUtils::FixCPPTypeAndObject(FString& InOutCPPType, TObjectPtr<UObject>& InOutCPPTypeObject, const FRigVMUserDefinedTypeResolver* InResolvalInfo)
{
	const FString OldCPPType = InOutCPPType;
	UObject* OldCPPTypeObject = InOutCPPTypeObject;
	InOutCPPType = PostProcessCPPType(InOutCPPType, InOutCPPTypeObject, InResolvalInfo);

	// Type object might have changed through a redirect
	bool bFindObjectAgain = true;

	// If its a user defined enum, do not attempt to find the object from the cpp type
	if (const UEnum* Enum = Cast<UEnum>(InOutCPPTypeObject))
	{
		if (Enum->CppType.IsEmpty())
		{
			bFindObjectAgain = false;
		}
	}

	if (bFindObjectAgain)
	{
		InOutCPPTypeObject = ObjectFromCPPType(InOutCPPType);
	}
	
	return !InOutCPPType.Equals(OldCPPType) || InOutCPPTypeObject != OldCPPTypeObject;
}

UObject* RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(const FString& InObjectPath)
{
	UObject* Result = nullptr;
	if (InObjectPath.IsEmpty())
	{
		return Result;
	}

	if (InObjectPath == FName(NAME_None).ToString())
	{
		return Result;
	}

	// we do this to avoid ambiguous searches for 
	// common names such as "transform" or "vector"
	UPackage* Package = nullptr;
	FString PackageName;
	FString CPPTypeObjectName = InObjectPath;
	if (InObjectPath.Split(TEXT("."), &PackageName, &CPPTypeObjectName))
	{
		Package = FindPackage(nullptr, *PackageName);
	}

	if (UObject* ObjectWithinPackage = FindObject<UObject>(Package, *CPPTypeObjectName))
	{
		Result = ObjectWithinPackage;
	}

	if (!Result)
	{
		Result = FindFirstObject<UObject>(*InObjectPath, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
	}
	if (!Result)
	{
		const FCoreRedirectObjectName OldObjectName(InObjectPath);
		const FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Struct, OldObjectName);
		if (OldObjectName != NewObjectName)
		{
			Result = FindObjectFromCPPTypeObjectPath(NewObjectName.ToString());
		}
	}
	return Result;
}

bool RigVMTypeUtils::AreCompatible(const FProperty* InSourceProperty, const FProperty* InTargetProperty)
{
	bool bCompatible = InSourceProperty->SameType(InTargetProperty);
	if (!bCompatible)
	{
		if(const FFloatProperty* TargetFloatProperty = CastField<FFloatProperty>(InTargetProperty))
		{
			bCompatible = InSourceProperty->IsA<FDoubleProperty>();
		}
		else if(const FDoubleProperty* TargetDoubleProperty = CastField<FDoubleProperty>(InTargetProperty))
		{
			bCompatible = InSourceProperty->IsA<FFloatProperty>();
		}
		else if (const FByteProperty* TargetByteProperty = CastField<FByteProperty>(InTargetProperty))
		{
			bCompatible = InSourceProperty->IsA<FEnumProperty>();
		}
		else if (const FEnumProperty* TargetEnumProperty = CastField<FEnumProperty>(InTargetProperty))
		{
			bCompatible = InSourceProperty->IsA<FByteProperty>();
		}
		else if(const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(InTargetProperty))
		{
			if(const FArrayProperty* SourceArrayProperty = CastField<FArrayProperty>(InSourceProperty))
			{
				if(TargetArrayProperty->Inner->IsA<FFloatProperty>())
				{
					bCompatible = SourceArrayProperty->Inner->IsA<FDoubleProperty>();
				}
				else if(TargetArrayProperty->Inner->IsA<FDoubleProperty>())
				{
					bCompatible = SourceArrayProperty->Inner->IsA<FFloatProperty>();
				}
				else if(FByteProperty* TargetArrayInnerByteProperty = CastField<FByteProperty>(TargetArrayProperty->Inner))
				{
					bCompatible = SourceArrayProperty->Inner->IsA<FEnumProperty>();
				}
				else if(FEnumProperty* TargetArrayInnerEnumProperty = CastField<FEnumProperty>(TargetArrayProperty->Inner))
				{
					bCompatible = SourceArrayProperty->Inner->IsA<FByteProperty>();
				}
			}
		}
	}
	return bCompatible;
}

FString RigVMTypeUtils::GetPluginName(const UObject* InObject)
{
	if(!InObject)
	{
		return FString();
	}
	
	return FRigVMPluginsPrefixCache::Get()->GetPluginName(InObject);
}

TArray<FString> RigVMTypeUtils::GetPluginModulePrefixes(const IPlugin& InPlugin)
{
	const FPluginDescriptor Descriptor = InPlugin.GetDescriptor();
	
	TArray<FString> ModulePrefixes;
	ModulePrefixes.Reserve(Descriptor.Modules.Num());

	for (const FModuleDescriptor& Module : Descriptor.Modules)
	{
		static constexpr TCHAR Format[] = TEXT("/Script/%s.");
		ModulePrefixes.Add(FString::Printf(Format, *Module.Name.ToString()));
	}

	static constexpr TCHAR PluginFormat[] = TEXT("/%s/");
	ModulePrefixes.AddUnique(FString::Printf(PluginFormat, *InPlugin.GetName()));

	return ModulePrefixes;
}

bool RigVMTypeUtils::CPPTypeFromExternalVariable(const FRigVMExternalVariable& InExternalVariable, FString& OutCPPType, UObject** OutCPPTypeObject)
{
	FString Prefix = "";
	FString Suffix = "";
	if (InExternalVariable.IsArray())
	{
		Prefix = TEXT("TArray<");
		Suffix = TEXT(">");
	}

	*OutCPPTypeObject = nullptr;
	if (InExternalVariable.GetBaseCPPType() == BoolTypeName)
	{
		OutCPPType = Prefix + BoolType + Suffix;
	}
	else if (InExternalVariable.GetBaseCPPType() == Int32TypeName)
	{
		OutCPPType = Prefix + Int32Type + Suffix;
	}
	else if (InExternalVariable.GetBaseCPPType() == FloatTypeName)
	{
		OutCPPType = Prefix + FloatType + Suffix;
	}
	else if (InExternalVariable.GetBaseCPPType() == DoubleTypeName)
	{
		OutCPPType = Prefix + DoubleType + Suffix;
	}
	else if (InExternalVariable.GetBaseCPPType() == FNameTypeName)
	{
		OutCPPType = Prefix + FNameType + Suffix;
	}
	else if (InExternalVariable.GetBaseCPPType() == FStringTypeName)
	{
		OutCPPType = Prefix + FStringType + Suffix;
	}
	else if (const UScriptStruct* Struct = Cast<UScriptStruct>(InExternalVariable.GetCPPTypeObject()))
	{
		OutCPPType = Prefix + *GetUniqueStructTypeName(Struct) + Suffix;
		*OutCPPTypeObject = const_cast<UScriptStruct*>(Struct);	
	}
	else if (const UEnum* Enum = Cast<UEnum>(InExternalVariable.GetCPPTypeObject()))
	{
		OutCPPType = Prefix + Enum->GetFName().ToString() + Suffix;
		*OutCPPTypeObject = const_cast<UEnum*>(Enum);
	}
	else if (const UClass* Class = Cast<UClass>(InExternalVariable.GetCPPTypeObject()))
	{
		Prefix += TEXT("TObjectPtr<U");
		Suffix += TEXT(">");
		OutCPPType = Prefix + Class->GetFName().ToString() + Suffix;
		*OutCPPTypeObject = const_cast<UClass*>(Class);
	}
	else
	{
		return false;
	}
		
	return true;
}

TSharedRef<FRigVMPluginsPrefixCache> FRigVMPluginsPrefixCache::Get()
{
	static TSharedRef<FRigVMPluginsPrefixCache> sPluginsCache = MakeShared<FRigVMPluginsPrefixCache>();
	static bool bDelegatesRegistered = false;
	if (!bDelegatesRegistered)
	{
		IPluginManager& PluginManager = IPluginManager::Get();
		
		PluginManager.OnNewPluginMounted().AddSP(sPluginsCache, &FRigVMPluginsPrefixCache::OnPluginMounted);
		PluginManager.OnPluginUnmounted().AddSP(sPluginsCache, &FRigVMPluginsPrefixCache::OnPluginUnmounted);
		
		bDelegatesRegistered = true;
	}
	return sPluginsCache;
}

FString FRigVMPluginsPrefixCache::GetPluginName(const UObject* InObject) const
{
	if (!bInitialized)
	{
		const_cast<FRigVMPluginsPrefixCache*>(this)->Refresh();
	}
	
	const FString Path = FSoftObjectPath(InObject).ToString();
	if (Path.IsEmpty())
	{
		return FString();
	}
	
	FReadScopeLock ReadLock(CacheLock);
	
	static const FString ScriptPrefix = TEXT("/Script/");
	if (Path.StartsWith(ScriptPrefix))
	{
		int32 PeriodPos = INDEX_NONE;
		if (Path.FindChar(TEXT('.'), PeriodPos))
		{
			if (const FRigVMPluginReferences* References = CachedPluginPrefixes.Find(Path.Left(PeriodPos + 1)))
			{
				return *References->PluginName;
			}
		}
	}
	else
	{
		const int32 SecondSlashPos = Path.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 1);
		if (SecondSlashPos != INDEX_NONE)
		{
			if (const FRigVMPluginReferences* References = CachedPluginPrefixes.Find(Path.Left(SecondSlashPos + 1)))
			{
				return *References->PluginName;
			}
		}
	}
	
	return FString();
}

void FRigVMPluginsPrefixCache::Refresh()
{
	FWriteScopeLock WriteLock(CacheLock);
	
	CachedPluginPrefixes.Reset();
	
	TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetDiscoveredPlugins();
	for (const TSharedRef<IPlugin>& Plugin : Plugins)
	{
		const TArray<FString> Prefixes = RigVMTypeUtils::GetPluginModulePrefixes(Plugin.Get());
		for (const FString& Prefix : Prefixes)
		{
			FRigVMPluginReferences& NewPrefix = CachedPluginPrefixes.FindOrAdd(Prefix);
			NewPrefix.PluginName = Plugin->GetName();
			NewPrefix.Count++;
		}
	}
	
	bInitialized = true;
}

FRigVMPluginsPrefixCache::FRigVMPluginsPrefixCache()
	: bInitialized(false)
{
	
}

FRigVMPluginsPrefixCache::~FRigVMPluginsPrefixCache()
{
	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.OnNewPluginMounted().RemoveAll(this);
	PluginManager.OnPluginUnmounted().RemoveAll(this);
}

void FRigVMPluginsPrefixCache::OnPluginMounted(IPlugin& InPlugin)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FRigVMPluginsPrefixCache::OnPluginLoaded");
	const TArray<FString> Prefixes = RigVMTypeUtils::GetPluginModulePrefixes(InPlugin);
	for (const FString& Prefix : Prefixes)
	{
		FRigVMPluginReferences& NewPrefix = CachedPluginPrefixes.FindOrAdd(Prefix);
		NewPrefix.PluginName = InPlugin.GetName();
		NewPrefix.Count++;
	}
}

void FRigVMPluginsPrefixCache::OnPluginUnmounted(IPlugin& InPlugin)
{
	const TArray<FString> Prefixes = RigVMTypeUtils::GetPluginModulePrefixes(InPlugin);
	for (const FString& Prefix : Prefixes)
	{
		if (FRigVMPluginReferences* Plugin = CachedPluginPrefixes.Find(Prefix))
		{
			Plugin->Count--;
			if (Plugin->Count == 0)
			{
				CachedPluginPrefixes.Remove(Prefix);
			}
		}
	}
}