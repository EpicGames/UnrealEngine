// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMRuntimeAsset.h"
#include "UObject/ObjectMacros.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMBlueprintGeneratedClass.generated.h"

#define UE_API RIGVM_API


UCLASS(MinimalAPI)
class URigVMBlueprintGeneratedClass : public UBlueprintGeneratedClass, public IRigVMGraphFunctionHost, public IRigVMRuntimeAssetInterface
{
	GENERATED_UCLASS_BODY()

public:

	// UClass interface
	UE_API virtual uint8* GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const override;
	UE_API virtual void PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph) override;

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;

	// IRigVMGraphFunctionHost interface
	virtual const FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() const override { return &GraphFunctionStore; }
	virtual FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() override { return &GraphFunctionStore; }
	
	// IRigVMRuntimeAssetInterface implementation
#if WITH_EDITORONLY_DATA
	virtual UObject* GetEditorOnlyData() override { return ClassGeneratedBy; }
	virtual const UObject* GetEditorOnlyData() const override { return ClassGeneratedBy; }
#endif
	UE_API virtual UClass* GetRigVMHostClass() const override;
	virtual URigVM* GetVM() override { return GetDefaultObject<URigVMHost>()->GetVM(); }
	virtual TArray<FRigVMExternalVariable> GetExternalVariables() override { return GetDefaultObject<URigVMHost>()->GetExternalVariables(); }
	virtual const UStruct* GetVariablesStruct() override { return this; }
	virtual uint8* GetVariablesMemory() override { return reinterpret_cast<uint8*>(GetDefaultObject()); }
	virtual const uint8* GetVariablesMemory() const override { return reinterpret_cast<const uint8*>(GetDefaultObject()); }
	virtual FProperty* FindGeneratedPropertyByName(const FName& InName) override { return FindPropertyByName(InName); }
	virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() override { return GetDefaultObject<URigVMHost>()->VMRuntimeSettings;}
	virtual FRigVMVariant& GetAssetVariant() override { return AssetVariant; }
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName() override { return GetDefaultObject<URigVMHost>()->UserDefinedStructGuidToPathName; }
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName() override { return GetDefaultObject<URigVMHost>()->UserDefinedEnumToPathName; }
	virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse() override { return GetDefaultObject<URigVMHost>()->UserDefinedTypesInUse; }
	virtual TArray<FName>& GetSupportedEventNames() override { return SupportedEventNames; }
	UE_API virtual void UpdateSupportedEventNames() override;
	UE_API virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeDerivedClass) const override;
	virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() override { return &GetDefaultObject<URigVMHost>()->GetRigVMExtendedExecuteContext(); }
	virtual void RemoveInstance(UObject* InInstance) override {}
	virtual URigVMHost* InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags) override { return NewObject<URigVMHost>(InOuter, this, InName, InFlags); }
	virtual bool InitializeInstance(URigVMHost* InInstance) override { return true; }
	UE_API virtual bool InitializeVariables(URigVMHost* InInstance) override;
	virtual bool IsObjectInstanceOfAsset(URigVMHost* InInstance) override { return InInstance->IsA(this); }
	virtual FRigVMDrawContainer& GetDrawContainer() override { return GetDefaultObject<URigVMHost>()->DrawContainer; }
	virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override { return GetDefaultObject<URigVMHost>()->GetAssetUserDataArray(); }
#if WITH_EDITOR
	virtual FRigVMDebugInfo& GetDebugInfo() override { return GetDefaultObject<URigVMHost>()->GetDebugInfo(); }
#endif

	UPROPERTY()
	FRigVMGraphFunctionStore GraphFunctionStore;

	UPROPERTY(AssetRegistrySearchable)
	TArray<FName> SupportedEventNames;

	/** Variant information about this asset */
	UPROPERTY(AssetRegistrySearchable)
	FRigVMVariant AssetVariant;
};

#undef UE_API
