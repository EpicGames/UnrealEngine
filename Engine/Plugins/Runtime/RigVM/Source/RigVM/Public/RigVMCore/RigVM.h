// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RigVMByteCode.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMMemoryCommon.h"
#include "RigVMCore/RigVMPropertyPath.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMExecuteContext.h"
#include "RigVMMemory.h"
#include "RigVMMemoryDeprecated.h"
#include "RigVMRegistry.h"
#include "RigVMStatistics.h"
#include "Templates/Function.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#if WITH_EDITOR
#include "HAL/PlatformTime.h"
#include "RigVMDebugInfo.h"
#endif
#include "RigVM.generated.h"

class FArchive;
class URigVMHost;
struct FFrame;
struct FRigVMDispatchFactory;

// The type of parameter for a VM
UENUM(BlueprintType)
enum class ERigVMParameterType : uint8
{
	Input,
	Output,
	Invalid
};

/**
 * The RigVMParameter define an input or output of the RigVM.
 * Parameters are mapped to work state memory registers and can be
 * used to set input parameters as well as retrieve output parameters.
 */
USTRUCT(BlueprintType)
struct FRigVMParameter
{
	GENERATED_BODY()

public:

	FRigVMParameter()
		: Type(ERigVMParameterType::Invalid)
		, Name(NAME_None)
		, RegisterIndex(INDEX_NONE)
		, CPPType()
		, ScriptStruct(nullptr)
		, ScriptStructPath()
	{
	}

	RIGVM_API void Serialize(FArchive& Ar);
	RIGVM_API void Save(FArchive& Ar);
	RIGVM_API void Load(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FRigVMParameter& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	// returns true if the parameter is valid
	bool IsValid() const { return Type != ERigVMParameterType::Invalid; }

	// returns the type of this parameter
	ERigVMParameterType GetType() const { return Type; }

	// returns the name of this parameters
	const FName& GetName() const { return Name; }

	// returns the register index of this parameter in the work memory
	int32 GetRegisterIndex() const { return RegisterIndex; }

	// returns the cpp type of the parameter
	FString GetCPPType() const { return CPPType; }
	
	// Returns the script struct used by this parameter (in case it is a struct)
	RIGVM_API UScriptStruct* GetScriptStruct() const;

private:

	FRigVMParameter(ERigVMParameterType InType, const FName& InName, int32 InRegisterIndex, const FString& InCPPType, UScriptStruct* InScriptStruct)
		: Type(InType)
		, Name(InName)
		, RegisterIndex(InRegisterIndex)
		, CPPType(InCPPType)
		, ScriptStruct(InScriptStruct)
		, ScriptStructPath(NAME_None)
	{
		if (ScriptStruct)
		{
			ScriptStructPath = *ScriptStruct->GetPathName();
		}
	}

	UPROPERTY()
	ERigVMParameterType Type;

	UPROPERTY()
	FName Name;

	UPROPERTY()
	int32 RegisterIndex;

	UPROPERTY()
	FString CPPType;

	UPROPERTY(transient)
	TObjectPtr<UScriptStruct> ScriptStruct;

	UPROPERTY()
	FName ScriptStructPath;

	friend class URigVM;
	friend class URigVMCompiler;
};

/**
 * The RigVM is the main object for evaluating FRigVMByteCode instructions.
 * It combines the byte code, a list of required function pointers for 
 * execute instructions and required memory in one class.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVM : public UObject
{
	GENERATED_BODY()

public:
	RIGVM_API URigVM();
	RIGVM_API virtual ~URigVM();

	// UObject interface
	RIGVM_API virtual void Serialize(FArchive& Ar);
	RIGVM_API virtual void Save(FArchive& Ar);
	RIGVM_API virtual void Load(FArchive& Ar);
	RIGVM_API void CopyDataForSerialization(URigVM* InVM);
	RIGVM_API void CopyExternalVariableDefs(URigVM* InVM);

	RIGVM_API static bool IsSerializingToDisk(const FArchive& Ar);

	RIGVM_API virtual void PostLoad() override;

#if WITH_EDITORONLY_DATA
	static RIGVM_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	RIGVM_API bool IsContextValidForExecution(FRigVMExtendedExecuteContext& Context) const;

	// returns true if this is a nativized VM
	virtual bool IsNativized() const { return false; }

	// Stores the current VM hash
	RIGVM_API virtual void SetVMHash(uint32 InVMHash);

	// returns the cached VM hash
	RIGVM_API virtual uint32 GetVMHash() const;

	// Generates a unique hash to compare VMs
	RIGVM_API virtual uint32 ComputeVMHash() const;
	RIGVM_API virtual uint32 ComputeVMHash(const TArray<FRigVMExternalVariableDef>& InExternalVariables) const;
	RIGVM_API virtual uint32 ComputeVMHash(const TArray<FRigVMExternalVariable>& InExternalVariables) const;

	// returns the VM's matching nativized class if it exists
	RIGVM_API UClass* GetNativizedClass(uint32 InHash) const;

	// resets the container and maintains all memory
	RIGVM_API virtual void Reset(FRigVMExtendedExecuteContext& Context);

	// resets the container and removes all memory
	RIGVM_API virtual void Empty(FRigVMExtendedExecuteContext& Context);

	// Prepares caches and memory for execution
	RIGVM_API virtual bool Initialize(FRigVMExtendedExecuteContext& Context);

	// Initializes cached memory handles and optionally copies work memory from the CDO to the Context
	RIGVM_API virtual bool InitializeInstance(FRigVMExtendedExecuteContext& Context, bool bCopyMemory = true);

	// Executes the VM.
	// You can optionally provide optional additional operands.
	RIGVM_API virtual ERigVMExecuteResult ExecuteVM(FRigVMExtendedExecuteContext& Context, const FName& InEntryName = NAME_None);

	UE_DEPRECATED(5.4, "Please, use ExecuteVM with Context param")
	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported."))
	virtual bool Execute(FRigVMExtendedExecuteContext& Context, const FName& InEntryName = NAME_None) { return false; }

	// Executes a single branch on the VM. We assume that the memory is already set correctly at this point.
	RIGVM_API ERigVMExecuteResult ExecuteBranch(FRigVMExtendedExecuteContext& Context, const FRigVMBranchInfo& InBranchToRun);

private:
	RIGVM_API ERigVMExecuteResult ExecuteInstructions(FRigVMExtendedExecuteContext& Context, int32 InFirstInstruction, int32 InLastInstruction);

public:

	// Add a function for execute instructions to this VM.
	// Execute instructions can then refer to the function by index.
	UFUNCTION()
	RIGVM_API virtual int32 AddRigVMFunction(UScriptStruct* InRigVMStruct, const FName& InMethodName);
	RIGVM_API virtual int32 AddRigVMFunction(const FString& InFunctionName);
	RIGVM_API virtual int32 AddRigVMFunction(const FName& InFunctionFName);

private:
	RIGVM_API virtual int32 AddRigVMFunctionImpl(const FName& InFunctionFName, const FString& InFunctionName);

public:
	// Returns the name of a function given its index
	UFUNCTION()
	RIGVM_API virtual FString GetRigVMFunctionName(int32 InFunctionIndex) const;

	// Returns a memory storage by type
	RIGVM_API virtual FRigVMMemoryStorageStruct* GetMemoryByType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType);
	RIGVM_API virtual const FRigVMMemoryStorageStruct* GetMemoryByType(const FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType) const;

	// The default const literal memory
	FRigVMMemoryStorageStruct* GetLiteralMemory()
	{
		return &LiteralMemoryStorage;
	}
	const FRigVMMemoryStorageStruct* GetLiteralMemory() const
	{
		return &LiteralMemoryStorage;
	}

	// The instance mutable work memory the VM will use to execute
	FRigVMMemoryStorageStruct* GetWorkMemory(FRigVMExtendedExecuteContext& Context)
	{
		return GetMemoryByType(Context, ERigVMMemoryType::Work);
	}
	const FRigVMMemoryStorageStruct* GetWorkMemory(const FRigVMExtendedExecuteContext& Context) const
	{
		return GetMemoryByType(Context, ERigVMMemoryType::Work);
	}

	// The instance debug watch memory
	FRigVMMemoryStorageStruct* GetDebugMemory(FRigVMExtendedExecuteContext& Context)
	{
		return GetMemoryByType(Context, ERigVMMemoryType::Debug);
	}
	const FRigVMMemoryStorageStruct* GetDebugMemory(const FRigVMExtendedExecuteContext& Context) const
	{
		return GetMemoryByType(Context, ERigVMMemoryType::Debug);
	}

	// creates the debug memory for a single instance of this VM
	RIGVM_API FRigVMMemoryStorageStruct* CreateDebugMemory(FRigVMExtendedExecuteContext& Context);

	// The default non mutable reference literal memory (generated by the compiler and copied to the instances as initial state)
	const FRigVMMemoryStorageStruct& GetDefaultLiteralMemory() const
	{
		return LiteralMemoryStorage;
	}
	FRigVMMemoryStorageStruct& GetDefaultLiteralMemory()
	{
		return LiteralMemoryStorage;
	}

	// The default non mutable reference work memory (generated by the compiler and copied to the instances as initial state)
	const FRigVMMemoryStorageStruct& GetDefaultWorkMemory() const
	{
		return DefaultWorkMemoryStorage;
	}
	FRigVMMemoryStorageStruct& GetDefaultWorkMemory()
	{
		return DefaultWorkMemoryStorage;
	}

	// Generates the Default memory and copy it to the Context if required by the type of memory
	RIGVM_API virtual void GenerateMemoryType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>* InProperties);

	// Used by the compiler and EngineTests to generate the default VM memory storages
	RIGVM_API virtual void GenerateDefaultMemoryType(ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>* InProperties);

	// Returns a default memory storage by type
	RIGVM_API virtual const FRigVMMemoryStorageStruct* GetDefaultMemoryByType(ERigVMMemoryType InMemoryType) const;
	RIGVM_API virtual FRigVMMemoryStorageStruct* GetDefaultMemoryByType(ERigVMMemoryType InMemoryType);


public:

	RIGVM_API virtual void ClearMemory(FRigVMExtendedExecuteContext& Context);

	UPROPERTY()
	FRigVMMemoryStorageStruct LiteralMemoryStorage;

	UPROPERTY()
	FRigVMMemoryStorageStruct DefaultWorkMemoryStorage;

	UPROPERTY()
	FRigVMMemoryStorageStruct DefaultDebugMemoryStorage_DEPRECATED;

#if WITH_EDITORONLY_DATA
	// Deprecated 5.4
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please, use DefaultWorkMemoryStorage for compiling and WorkMemoryStorage in the ExtendedExecuteContext for intance execution"))
	TObjectPtr<URigVMMemoryStorage> WorkMemoryStorageObject_DEPRECATED;
#endif

#if WITH_EDITORONLY_DATA
	// Deprecated 5.4
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please, use LiteralMemoryStorage"))
	TObjectPtr<URigVMMemoryStorage> LiteralMemoryStorageObject_DEPRECATED;
#endif

#if WITH_EDITORONLY_DATA
	// Deprecated 5.4, 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please use DefaultDebugMemoryStorage for compiling and DebugMemoryStorage in the ExtendedExecuteContext for intance execution"))
	TObjectPtr<URigVMMemoryStorage> DebugMemoryStorageObject_DEPRECATED;
#endif

	TArray<FRigVMPropertyPathDescription> ExternalPropertyPathDescriptions;
	TArray<FRigVMPropertyPath> ExternalPropertyPaths;

	// The byte code of the VM
	UPROPERTY()
	FRigVMByteCode ByteCodeStorage;
	FRigVMByteCode* ByteCodePtr;
	FRigVMByteCode& GetByteCode() { return const_cast<FRigVMByteCode&>(const_cast<const URigVM*>(this)->GetByteCode()); }
	virtual const FRigVMByteCode& GetByteCode() const { return *ByteCodePtr; }

	// Returns the instructions of the VM
	RIGVM_API virtual const FRigVMInstructionArray& GetInstructions();

	// Returns true if this VM's bytecode contains a given entry
	RIGVM_API virtual bool ContainsEntry(const FName& InEntryName) const;

	// Returns the index of an entry
	RIGVM_API virtual int32 FindEntry(const FName& InEntryName) const;

	// Returns a list of all valid entry names for this VM's bytecode
	RIGVM_API virtual const TArray<FName>& GetEntryNames() const;

	// returns false if an entry can not be executed
	RIGVM_API bool CanExecuteEntry(const FRigVMExtendedExecuteContext& Context, const FName& InEntryName, bool bLogErrorForMissingEntry = true) const;

	// returns the traits for this VM's bytecode and Context
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMExtendedExecuteContext& InContext, const UScriptStruct* InScriptStruct = nullptr)
	{
		return GetByteCode().GetTraits(*GetLiteralMemory(), InContext.WorkMemoryStorage, InScriptStruct);
	}

	// returns the traits of a given type for this VM's bytecode and Context
	template<typename T>
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMExtendedExecuteContext& InContext)
	{
		return GetByteCode().GetTraits<T>(*GetLiteralMemory(), InContext.WorkMemoryStorage);
	}

	// returns the traits for this VM's bytecode and Context, as well as any additional memory handles
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMExtendedExecuteContext& InContext, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles, const UScriptStruct* InScriptStruct = nullptr)
	{
		return GetByteCode().GetTraits(*GetLiteralMemory(), InContext.WorkMemoryStorage, OutAdditionalMemoryHandles, InScriptStruct);
	}

	// returns the traits of a given type for this VM's bytecode and Context, as well as any additional memory handles
	template<typename T>
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMExtendedExecuteContext& InContext, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles)
	{
		return GetByteCode().GetTraits<T>(*GetLiteralMemory(), InContext.WorkMemoryStorage, OutAdditionalMemoryHandles);
	}
	
	// returns the traits for the provided memory for a single instruction
	TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMExtendedExecuteContext& InContext, const UScriptStruct* InScriptStruct = nullptr)
	{
		return GetByteCode().GetTraitsForInstruction(InInstruction, *GetLiteralMemory(), InContext.WorkMemoryStorage, InScriptStruct);
	}

	// returns the traits of a given type for the provided memory for a single instruction
	template<typename T>
	TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMExtendedExecuteContext& InContext)
	{
		return GetByteCode().GetTraitsForInstruction<T>(InInstruction, *GetLiteralMemory(), InContext.WorkMemoryStorage);
	}

#if WITH_EDITOR
	
	// Returns true if the given instruction has been visited during the last run
	bool WasInstructionVisitedDuringLastRun(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		return GetInstructionVisitedCount(Context, InIndex) > 0;
	}

	// Returns the number of times an instruction has been hit
	int32 GetInstructionVisitedCount(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		if (const FRigVMInstructionVisitInfo* InstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
		{
			return InstructionVisitInfo->GetInstructionVisitedCountDuringLastRun(InIndex);
		}
		return 0;
	}

	// Returns the number of times a callable has been hit
	int32 GetCallableVisitedCount(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		if (const FRigVMInstructionVisitInfo* InstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
		{
			return InstructionVisitInfo->GetCallableVisitedCountDuringLastRun(InIndex);
		}
		return 0;
	}

	// Returns accumulated cycles spent in an instruction during the last run
	// This requires bEnabledProfiling to be turned on in the runtime settings.
	// If there is no information available this function returns UINT64_MAX.
	uint64 GetInstructionCycles(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		if (const FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo())
		{
			return RigVMProfilingInfo->GetInstructionCyclesDuringLastRun(InIndex);
		}
		return UINT64_MAX;
	}

	// Returns accumulated cycles spent in a callable  during the last run
	// This requires bEnabledProfiling to be turned on in the runtime settings.
	// If there is no information available this function returns UINT64_MAX.
	uint64 GetCallableCycles(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		if (const FRigVMProfilingInfo* RigVMProfilingInfo = Context.GetRigVMProfilingInfo())
		{
			return RigVMProfilingInfo->GetCallableCyclesDuringLastRun(InIndex);
		}
		return UINT64_MAX;
	}

	// Returns accumulated duration of the instruction in microseconds during the last run
	// Note: this requires bEnabledProfiling to be turned on in the runtime settings.
	// If there is no information available this function returns -1.0.
	double GetInstructionMicroSeconds(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		const uint64 Cycles = GetInstructionCycles(Context, InIndex);
		if(Cycles == UINT64_MAX)
		{
			return -1.0;
		}
		return double(Cycles) * FPlatformTime::GetSecondsPerCycle() * 1000.0 * 1000.0;
	}

	// Returns accumulated duration of the callable in microseconds during the last run
	// Note: this requires bEnabledProfiling to be turned on in the runtime settings.
	// If there is no information available this function returns -1.0.
	double GetCallableMicroSeconds(const FRigVMExtendedExecuteContext& Context, int32 InIndex) const
	{
		const uint64 Cycles = GetCallableCycles(Context, InIndex);
		if(Cycles == UINT64_MAX)
		{
			return -1.0;
		}
		return double(Cycles) * FPlatformTime::GetSecondsPerCycle() * 1000.0 * 1000.0;
	}

	// Returns the order of all instructions during the last run
	const TArray<int32> GetInstructionVisitOrder(const FRigVMExtendedExecuteContext& Context) const
	{
		if (const FRigVMInstructionVisitInfo* InstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
		{
			return InstructionVisitInfo->GetInstructionVisitOrder();
		}

		return TArray<int32>();
	}

	UE_DEPRECATED(5.7, "ResumeExecution is no longer supported.")
	RIGVM_API bool ResumeExecution(FRigVMExtendedExecuteContext& Context, const FName& InEntryName = NAME_None);

#endif // WITH_EDITOR

	// Returns the parameters of the VM
	RIGVM_API const TArray<FRigVMParameter>& GetParameters() const;

	// Returns a parameter given it's name
	RIGVM_API FRigVMParameter GetParameterByName(const FName& InParameterName);

	FRigVMParameter AddParameter(FRigVMExtendedExecuteContext& Context, ERigVMParameterType InType, const FName& InParameterName, const FName& InWorkMemoryPropertyName)
	{
		check(GetWorkMemory(Context));

		if(ParametersNameMap.Contains(InParameterName))
		{
			return FRigVMParameter();
		}

		const FProperty* Property = GetWorkMemory(Context)->FindPropertyByName(InWorkMemoryPropertyName);
		if (Property == nullptr)
		{
			return FRigVMParameter();
		}
		const int32 PropertyIndex = GetWorkMemory(Context)->GetPropertyIndex(Property);

		UScriptStruct* Struct = nullptr;
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			Struct = StructProperty->Struct;
		}
		
		FRigVMParameter Parameter(InType, InParameterName, PropertyIndex, Property->GetCPPType(), Struct);
		ParametersNameMap.Add(Parameter.Name, Parameters.Add(Parameter));
		return Parameter;
	}

	// Retrieve the array size of the parameter
	int32 GetParameterArraySize(FRigVMExtendedExecuteContext& Context, const FRigVMParameter& InParameter)
	{
		const int32 PropertyIndex = InParameter.GetRegisterIndex();
		if (PropertyIndex != INDEX_NONE)
		{
			const FProperty* Property = GetWorkMemory(Context)->GetProperties()[PropertyIndex];
			const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
			if (ArrayProperty)
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, GetWorkMemory(Context)->GetData<uint8>(PropertyIndex));
				return ArrayHelper.Num();
			}
		}
		return 1;
	}

	// Retrieve the array size of the parameter
	int32 GetParameterArraySize(FRigVMExtendedExecuteContext& Context, int32 InParameterIndex)
	{
		return GetParameterArraySize(Context, Parameters[InParameterIndex]);
	}

	// Retrieve the array size of the parameter
	int32 GetParameterArraySize(FRigVMExtendedExecuteContext& Context, const FName& InParameterName)
	{
		const int32 ParameterIndex = ParametersNameMap.FindChecked(InParameterName);
		return GetParameterArraySize(Context, ParameterIndex);
	}

	//UE_DEPRECATED(5.4, "Please, use GetParameterValue with WorkMemory param")
	//template<class T> T GetParameterValue(const FRigVMParameter& InParameter, int32 InArrayIndex = 0, T DefaultValue = T{}) { return DefaultValue; }

	// Retrieve the value of a parameter
	template<class T>
	T GetParameterValue(FRigVMExtendedExecuteContext& Context, const FRigVMParameter& InParameter, int32 InArrayIndex = 0, T DefaultValue = T{})
	{
		if (InParameter.GetRegisterIndex() != INDEX_NONE)
		{
			if(GetWorkMemory(Context)->IsArray(InParameter.GetRegisterIndex()))
			{
				TArray<T>& Storage = *GetWorkMemory(Context)->GetData<TArray<T>>(InParameter.GetRegisterIndex());
				if(Storage.IsValidIndex(InArrayIndex))
				{
					return Storage[InArrayIndex];
				}
			}
			else
			{
				return *GetWorkMemory(Context)->GetData<T>(InParameter.GetRegisterIndex());
			}
		}
		return DefaultValue;
	}

	//UE_DEPRECATED(5.4, "Please, use GetParameterValue with WorkMemory param")
	//template<class T> T GetParameterValue(int32 InParameterIndex, int32 InArrayIndex = 0, T DefaultValue = T{}) { return DefaultValue; }

	// Retrieve the value of a parameter given its index
	template<class T>
	T GetParameterValue(FRigVMExtendedExecuteContext& Context, int32 InParameterIndex, int32 InArrayIndex = 0, T DefaultValue = T{})
	{
		return GetParameterValue<T>(Context, Parameters[InParameterIndex], InArrayIndex, DefaultValue);
	}

	//UE_DEPRECATED(5.4, "Please, use GetParameterValue with WorkMemory param")
	//template<class T> T GetParameterValue(const FName& InParameterName, int32 InArrayIndex = 0, T DefaultValue = T{}) { return DefaultValue; }

	// Retrieve the value of a parameter given its name
	template<class T>
	T GetParameterValue(FRigVMExtendedExecuteContext& Context, const FName& InParameterName, int32 InArrayIndex = 0, T DefaultValue = T{})
	{
		int32 ParameterIndex = ParametersNameMap.FindChecked(InParameterName);
		return GetParameterValue<T>(Context, ParameterIndex, InArrayIndex, DefaultValue);
	}

	//UE_DEPRECATED(5.4, "Please, use SetParameterValue with WorkMemory param")
	//template<class T> void SetParameterValue(const FRigVMParameter& InParameter, const T& InNewValue, int32 InArrayIndex = 0) {}

	// Set the value of a parameter
	template<class T>
	void SetParameterValue(FRigVMExtendedExecuteContext& Context, const FRigVMParameter& InParameter, const T& InNewValue, int32 InArrayIndex = 0)
	{
		if (InParameter.GetRegisterIndex() != INDEX_NONE)
		{
			if(GetWorkMemory(Context)->IsArray(InParameter.GetRegisterIndex()))
			{
				TArray<T>& Storage = *GetWorkMemory(Context)->GetData<TArray<T>>(InParameter.GetRegisterIndex());
				if(Storage.IsValidIndex(InArrayIndex))
				{
					Storage[InArrayIndex] = InNewValue;
				}
			}
			else
			{
				T& Storage = *GetWorkMemory(Context)->GetData<T>(InParameter.GetRegisterIndex());
				Storage = InNewValue;
			}
		}
	}

	//UE_DEPRECATED(5.4, "Please, use SetParameterValue with WorkMemory param")
	//template<class T> void SetParameterValue(int32 ParameterIndex, const T& InNewValue, int32 InArrayIndex = 0) {}

	// Set the value of a parameter given its index
	template<class T>
	void SetParameterValue(FRigVMExtendedExecuteContext& Context, int32 ParameterIndex, const T& InNewValue, int32 InArrayIndex = 0)
	{
		return SetParameterValue<T>(Context, Parameters[ParameterIndex], InNewValue, InArrayIndex);
	}

	//UE_DEPRECATED(5.4, "Please, use SetParameterValue with WorkMemory param")
	//template<class T> void SetParameterValue(const FName& InParameterName, const T& InNewValue, int32 InArrayIndex = 0) {}

	// Set the value of a parameter given its name
	template<class T>
	void SetParameterValue(FRigVMExtendedExecuteContext& Context, const FName& InParameterName, const T& InNewValue, int32 InArrayIndex = 0)
	{
		int32 ParameterIndex = ParametersNameMap.FindChecked(InParameterName);
		return SetParameterValue<T>(Context, ParameterIndex, InNewValue, InArrayIndex);
	}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	bool GetParameterValueBool(const FName& InParameterName, int32 InArrayIndex = 0) { return false; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	float GetParameterValueFloat(const FName& InParameterName, int32 InArrayIndex = 0) { return 0.f; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	double GetParameterValueDouble(const FName& InParameterName, int32 InArrayIndex = 0) { return 0.0; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	int32 GetParameterValueInt(const FName& InParameterName, int32 InArrayIndex = 0) { return 0; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FName GetParameterValueName(const FName& InParameterName, int32 InArrayIndex = 0) { return NAME_None; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FString GetParameterValueString(const FName& InParameterName, int32 InArrayIndex = 0) { return FString(); }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FVector2D GetParameterValueVector2D(const FName& InParameterName, int32 InArrayIndex = 0) { return FVector2D::ZeroVector; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FVector GetParameterValueVector(const FName& InParameterName, int32 InArrayIndex = 0) { return FVector::ZeroVector; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FQuat GetParameterValueQuat(const FName& InParameterName, int32 InArrayIndex = 0) { return FQuat::Identity; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	FTransform GetParameterValueTransform(const FName& InParameterName, int32 InArrayIndex = 0) { return FTransform::Identity; }

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueBool(const FName& InParameterName, bool InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueFloat(const FName& InParameterName, float InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueDouble(const FName& InParameterName, double InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueInt(const FName& InParameterName, int32 InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueName(const FName& InParameterName, const FName& InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueString(const FName& InParameterName, const FString& InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueVector2D(const FName& InParameterName, const FVector2D& InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueVector(const FName& InParameterName, const FVector& InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueQuat(const FName& InParameterName, const FQuat& InValue, int32 InArrayIndex = 0) {}

	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported, please, update your code."))
	void SetParameterValueTransform(const FName& InParameterName, const FTransform& InValue, int32 InArrayIndex = 0) {}

	// Clears the external variables of the VM
	RIGVM_API void ClearExternalVariables(FRigVMExtendedExecuteContext& Context);

	// Returns the external variables of the VM (without variable memory, as it is stored in Context)
	const TArray<FRigVMExternalVariableDef>& GetExternalVariableDefs() const
	{
		return ExternalVariables;
	}

	// Returns the external variables of the VM with variable memory
	const TArray<FRigVMExternalVariable> GetExternalVariables(const FRigVMExtendedExecuteContext& Context) const
	{
		TArray<FRigVMExternalVariable> ExternalVariablesWithInstanceData;

		const int32 Num = ExternalVariables.Num();
		for (int i=0; i<Num; ++i)
		{
			const FRigVMExternalVariableDef& ExternalVariable = ExternalVariables[i];
			const FRigVMExternalVariableRuntimeData& ExternalVariableData = Context.ExternalVariableRuntimeData[i];

			ExternalVariablesWithInstanceData.Add(FRigVMExternalVariable(ExternalVariable, ExternalVariableData.Memory));
		}

		return ExternalVariablesWithInstanceData;
	}

	// Returns an external variable Def given it's name
	RIGVM_API FRigVMExternalVariableDef GetExternalVariableDefByName(const FName& InExternalVariableName);

	// Returns an external variable given it's name
	RIGVM_API FRigVMExternalVariable GetExternalVariableByName(const FRigVMExtendedExecuteContext& Context, const FName& InExternalVariableName);

	// Sets the external variables without the instance data
	RIGVM_API void SetExternalVariableDefs(const TArray<FRigVMExternalVariable>& InExternalVariables);

	// Sets the external variables instance data required for execution
	virtual void SetExternalVariablesInstanceData(FRigVMExtendedExecuteContext& Context, const TArray<FRigVMExternalVariable>& InExternalVariables, bool bAllowNullMemory = false)
	{
		const int32 NumExternalVariables = InExternalVariables.Num();
		check(ExternalVariables.Num() == NumExternalVariables);
		
		Context.ExternalVariableRuntimeData.Reset(NumExternalVariables);
		
		for (int32 i = 0; i < NumExternalVariables; i++)
		{
			const FRigVMExternalVariable& InExternalVariable = InExternalVariables[i];
			FRigVMExternalVariableDef& ExternalVariableDef = ExternalVariables[i];

			// Only check name and property, to allow the case where an UUserStruct is deleted while used inside a Rig
			check(ExternalVariableDef.GetName() == InExternalVariable.GetName());
			check(ExternalVariableDef.GetProperty() == InExternalVariable.GetProperty());
			check(bAllowNullMemory || InExternalVariable.GetMemory() != nullptr);
			
			Context.ExternalVariableRuntimeData.Add(FRigVMExternalVariableRuntimeData(const_cast<uint8*>(InExternalVariable.GetMemory())));
		}
	}

	// Adds a new external / unowned variable to the VM
	FRigVMOperand AddExternalVariable(FRigVMExtendedExecuteContext& Context, const FRigVMExternalVariable& InExternalVariable, bool bAllowNullMemory = false)
	{
		check(bAllowNullMemory || InExternalVariable.GetMemory() != nullptr);

		const int32 VariableIndex = ExternalVariables.Add(InExternalVariable);
		Context.ExternalVariableRuntimeData.Add(FRigVMExternalVariableRuntimeData(const_cast<uint8*>(InExternalVariable.GetMemory())));

		return FRigVMOperand(ERigVMMemoryType::External, VariableIndex);
	}

	RIGVM_API void SetPropertyValueFromString(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InOperand, const FString& InDefaultValue);

	// returns the statistics information
	UFUNCTION(BlueprintPure, Category = "RigVM", meta=(DeprecatedFunction))
	FRigVMStatistics GetStatistics() const
	{
		FRigVMStatistics Statistics;
		if(GetLiteralMemory())
		{
			//Statistics.LiteralMemory = GetLiteralMemory()->GetStatistics();
		}
		//if(GetWorkMemory(Context))
		//{
		//	Statistics.WorkMemory = GetWorkMemory(Context)->GetStatistics();
		//}

		Statistics.ByteCode = ByteCodePtr->GetStatistics();
		Statistics.BytesForCaching = static_cast<int32>(FirstHandleForInstruction.GetAllocatedSize()); // +Context.CachedMemoryHandles.GetAllocatedSize(); // Requires context, but fn deprecated already
		Statistics.BytesForCDO =
			Statistics.LiteralMemory.TotalBytes +
			Statistics.WorkMemory.TotalBytes +
			Statistics.ByteCode.DataBytes +
			Statistics.BytesForCaching;
		
		Statistics.BytesPerInstance =
			Statistics.WorkMemory.DataBytes +
			Statistics.BytesForCaching;

		return Statistics;
	}

	// Returns the localized registry this VM uses or nullptr
	RIGVM_API FRigVMRegistry_NoLock* GetLocalizedRegistry() const;

	// Creates a localized registry for this VM
	RIGVM_API bool CreateLocalizedRegistryIfRequired(bool bForce = false);
	RIGVM_API bool CreateLocalizedRegistryIfRequired_NoLock(bool bForce = false);

protected:
	//  The local registry to use by the VM
	TSharedPtr<FRigVMRegistry_NoLock> LocalizedRegistry;

	RIGVM_API FString GetLocalizedRegistryAsString() const;
	RIGVM_API void SetLocalizedRegistryFromString(const FString& InData);

public:
	
#if WITH_EDITOR
	// returns the instructions as text, OperandFormatFunction is an optional argument that allows you to override how operands are displayed, for example, see SRigVMExecutionStackView::PopulateStackView
	RIGVM_API TArray<FString> DumpByteCodeAsTextArray(FRigVMExtendedExecuteContext& Context, const TArray<int32> & InInstructionOrder = TArray<int32>(), bool bIncludeLineNumbers = true, TFunction<FString(const FString& RegisterName, const FString& RegisterOffsetName)> OperandFormatFunction = nullptr);

	RIGVM_API FString DumpByteCodeAsText(FRigVMExtendedExecuteContext& Context, const TArray<int32>& InInstructionOrder = TArray<int32>(), bool bIncludeLineNumbers = true);
#endif

#if WITH_EDITOR
	// FormatFunction is an optional argument that allows you to override how operands are displayed, for example, see SRigVMExecutionStackView::PopulateStackView
	RIGVM_API FString GetOperandLabel(FRigVMExtendedExecuteContext& Context, const FRigVMOperand & InOperand, TFunction<FString(const FString& RegisterName, const FString& RegisterOffsetName)> FormatFunction = nullptr);
#endif

private:
	RIGVM_API bool ResolveFunctionsIfRequired();
	RIGVM_API void RefreshInstructionsIfRequired();

public:
	RIGVM_API void InvalidateCachedMemory(FRigVMExtendedExecuteContext& Context);

private:
	RIGVM_API void InstructionOpEval(
		FRigVMExtendedExecuteContext& Context,
		FRigVMRegistryHandle& RegistryHandle,
		int32 InstructionIndex,
		int32 InHandleBaseIndex,
		const TFunctionRef<void(
			FRigVMExtendedExecuteContext& /*Context*/,
			FRigVMRegistryHandle& /*RegistryHandle*/,
			int32 /*InstructionIndex*/,
			int32 /*InHandleIndex*/,
			const FRigVMBranchInfoKey& /*InBranchInfoKey*/,
			const FRigVMOperand& /*InArg*/
		)>& InOpFunc);
	RIGVM_API void PrepareMemoryForExecution(FRigVMExtendedExecuteContext& Context);
	RIGVM_API void CacheMemoryHandles(FRigVMExtendedExecuteContext& Context);
	RIGVM_API void RebuildByteCodeOnLoad();

	UPROPERTY(transient)
	FRigVMInstructionArray Instructions;

protected:

	[[nodiscard]] virtual bool SetInstructionIndex(FRigVMExtendedExecuteContext& Context, FRigVMExecuteContext& PublicData, int32 InInstructionIndex) const
	{
#if WITH_EDITOR
		if (Context.DebugInfoStrong != nullptr && Context.DebugInfoStrong->IsActive())
		{
			if (Context.DebugInfoStrong->ShouldStop(Context, PublicData.InstructionIndex))
			{
				return false;
			}
		}
#endif
		PublicData.InstructionIndex = InInstructionIndex;
		return true;
	}

	[[nodiscard]] bool IncrementInstructionIndex(FRigVMExtendedExecuteContext& Context, FRigVMExecuteContext& PublicData) const
	{
		return SetInstructionIndex(Context, PublicData, PublicData.InstructionIndex + 1);
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	uint32 NumExecutions_DEPRECATED = 0;
#endif

	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(AccessDetector);

public:
	RIGVM_API bool ValidateBytecode();
	RIGVM_API void RefreshArgumentNameCaches();
	RIGVM_API TArray<const UObject*> GetUserDefinedDependencies(const TArray<const FRigVMMemoryStorageStruct*> InMemory);
	RIGVM_API void GetRequiredPlugins(TArray<FString>& OutPlugins);

private:

	UPROPERTY()
	TArray<FName> FunctionNamesStorage;
	TArray<FName>* FunctionNamesPtr;
public:
	TArray<FName>& GetFunctionNames() { return const_cast<TArray<FName>&>(const_cast<const URigVM*>(this)->GetFunctionNames()); }
	virtual const TArray<FName>& GetFunctionNames() const { return *FunctionNamesPtr; }
private:
	TArray<const FRigVMFunction*> FunctionsStorage;
	TArray<const FRigVMFunction*>* FunctionsPtr;
public:
	TArray<const FRigVMFunction*>& GetFunctions() { return const_cast<TArray<const FRigVMFunction*>&>(const_cast<const URigVM*>(this)->GetFunctions()); }
	virtual const TArray<const FRigVMFunction*>& GetFunctions() const { return *FunctionsPtr; }
private:
	TArray<const FRigVMDispatchFactory*> FactoriesStorage;
	TArray<const FRigVMDispatchFactory*>* FactoriesPtr;
public:
	TArray<const FRigVMDispatchFactory*>& GetFactories() { return const_cast<TArray<const FRigVMDispatchFactory*>&>(const_cast<const URigVM*>(this)->GetFactories()); }
	virtual const TArray<const FRigVMDispatchFactory*>& GetFactories() const { return *FactoriesPtr; }
private:

	UPROPERTY()
	TArray<FRigVMParameter> Parameters;

	TMap<FName, int32> ParametersNameMap;

	TArray<uint32> FirstHandleForInstruction;

	int32 MemoryHandleCount = 0;

	TArray<FRigVMMemoryHandle> PreCachedMemoryHandles;
	TArray<ERigVMMemoryType> PreCachedMemoryHandlesMemoryType;

	TArray<FRigVMExternalVariableDef> ExternalVariables;
	TArray<FRigVMLazyBranch> LazyBranches;
	
	// this function should be kept in sync with FRigVMOperand::GetContainerIndex()
	static int32 GetContainerIndex(ERigVMMemoryType InType)
	{
		if(InType == ERigVMMemoryType::External)
		{
			return (int32)ERigVMMemoryType::Work;
		}
		
		if(InType == ERigVMMemoryType::Debug)
		{
			return 2;
		}
		return (int32)InType;
	}
	
	// debug watch register memory needs to be cleared for each execution
	RIGVM_API void ClearDebugMemory(FRigVMExtendedExecuteContext& Context);

	RIGVM_API void PreCacheSingleMemoryHandle(const FRigVMRegistryHandle& InRegistry, int32 InInstructionIndex, int32 InHandleIndex, const FRigVMBranchInfoKey& InBranchInfoKey, const FRigVMOperand& InArg, bool bForExecute = false);

	RIGVM_API static void GeneratePatchedHandles(
		const TArrayView<FRigVMMemoryHandle>& OutHandles,
		const TArray<FRigVMExternalVariableRuntimeData>& ExternalVariablesRuntimeData,
		const TStaticArray<uint8*, (int32)ERigVMMemoryType::Num>& VMMemoryPtrs,
		const TArray<FRigVMMemoryHandle>& PreCachedMemoryHandles,
		const TArray<ERigVMMemoryType>& PreCachedMemoryHandlesMemoryType,
		int32 FirstHandleIndex,
		int32 OperandCount);

	RIGVM_API static FRigVMMemoryHandle GeneratePatchedHandle(
		const TArray<FRigVMExternalVariableRuntimeData>& ExternalVariablesRuntimeData,
		const TStaticArray<uint8*, (int32)ERigVMMemoryType::Num>& VMMemoryPtrs,
		ERigVMMemoryType HandleMemoryType,
		const FRigVMMemoryHandle& PreCachedHandle);

	void CopyOperandForDebuggingIfNeeded(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InArg, const FRigVMMemoryHandle& InHandle)
	{
#if WITH_EDITOR
		if (const FRigVMOperand* DebugOperandPtr = Context.GetDebugOperandForOperand(InArg))
		{
			CopyOperandForDebuggingImpl(Context, InArg, InHandle, *DebugOperandPtr);
		}
#endif
	}

	RIGVM_API void CopyOperandForDebuggingImpl(FRigVMExtendedExecuteContext& Context, const FRigVMOperand& InArg, const FRigVMMemoryHandle& InHandle, const FRigVMOperand& InDebugOperand);

	RIGVM_API FRigVMCopyOp GetCopyOpForOperands(const FRigVMOperand& InSource, const FRigVMOperand& InTarget);
	RIGVM_API void RefreshExternalPropertyPaths();
	
protected:

	struct FEntryExecuteGuard
	{
	public:
		FEntryExecuteGuard(TArray<int32>& InOutStack, int32 InEntryIndex)
		: Stack(InOutStack)
		{
			Stack.Push(InEntryIndex);
		}

		~FEntryExecuteGuard()
		{
			// Don't release memory to avoid churn
			Stack.Pop(EAllowShrinking::No);
		}
	private:

		TArray<int32>& Stack;
	};

#if WITH_EDITOR
	struct FInstructionBracketGuard
	{
	public:
		FInstructionBracketGuard(FRigVMExtendedExecuteContext& InOutContext, int32 InFirstInstruction, int32 InLastInstruction)
		: Context(InOutContext)
		{
			const TTuple<int32, int32> Tuple(InFirstInstruction, InLastInstruction);
			if(Context.InstructionBrackets.Contains(Tuple))
			{
				static constexpr TCHAR Format[] = TEXT("Re-Entry of Instructions %d - %d.");
				ErrorMessage = FString::Printf(Format, Tuple.Get<0>(), Tuple.Get<1>());
			}
			Context.InstructionBrackets.Add(Tuple);
		}

		~FInstructionBracketGuard()
		{
			Context.InstructionBrackets.Pop();
		}
		
	private:

		FRigVMExtendedExecuteContext& Context;
		FString ErrorMessage;

		friend class URigVM;
	};
#endif

private:

	UPROPERTY()
	uint32 CachedVMHash = 0;

	mutable TOptional<uint32> CachedNativizedVMHash;

	mutable TArray<FName> EntryNames;

	bool bIsSerializing = false;

protected:
	RIGVM_API const URigVMHost* GetHostCDO() const;

	RIGVM_API void Reset_Internal();
	RIGVM_API void ClearMemory_Internal();
	RIGVM_API void InvalidateCachedMemory_Internal();

	RIGVM_API void SetupInstructionTracking(FRigVMExtendedExecuteContext& Context, int32 InInstructionCount, int32 InCallableCount, const FName& InEntryName);
	RIGVM_API void StartProfiling(FRigVMExtendedExecuteContext& Context);
	RIGVM_API void StopProfiling(FRigVMExtendedExecuteContext& Context);

	FTransactionallySafeCriticalSection ResolveFunctionsMutex;
	mutable FTransactionallySafeCriticalSection CreateLocalizedRegistryMutex;

	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
	friend class FRigVMCodeConverter;
};
