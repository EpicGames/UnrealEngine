// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMTypeUtils.h"
#include "Engine/AssetUserData.h"
#include "Engine/DataAsset.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RigVMAssetUserData.generated.h"

/**
* User data that can be attached to assets to provide namespaced data access.
*/
UCLASS(Abstract, MinimalAPI)
class UNameSpacedUserData : public UAssetUserData
{
	GENERATED_BODY()

public:

	/** A helper struct to represent a single user data */
	struct FUserData
	{
	public:

		FUserData()
		: Path()
		, CPPType(NAME_None)
		, Property(nullptr)
		, Memory(nullptr)
		{
		}

		FUserData(const FString& InPath, const FProperty* InProperty, const uint8* InMemory)
		: Path(InPath)
		, CPPType(NAME_None)
		, Property(InProperty)
		, Memory(InMemory)
		{
		}

		FUserData(const FString& InPath, const FName& InCPPType, const FProperty* InProperty, const uint8* InMemory)
		: Path(InPath)
		, CPPType(InCPPType)
		, Property(InProperty)
		, Memory(InMemory)
		{
		}

		// Returns true if this user data is mapped correctly
		bool IsValid() const
		{
			return (!Path.IsEmpty()) && (Memory != nullptr);
		}

		operator bool() const { return IsValid(); }

		operator FString() const { return Path; }

		// returns the string path inside of the asset user data
		const FString& GetPath() const { return Path; }

		// returns the name of the last segment of the path
		RIGVM_API FString GetName() const;

#if WITH_EDITOR

		// returns the display name of the last segment of the path
		RIGVM_API FString GetDisplayName() const;

#endif

		// returns the CPP type of this user data (computes it from the property as needed)
		const FName& GetCPPType() const
		{
			if(CPPType.IsNone() && Property != nullptr)
			{
				CPPType = *RigVMTypeUtils::GetCPPTypeFromProperty(Property);
			}
			return CPPType;
		}

		RIGVM_API bool IsArray() const;

		RIGVM_API bool IsArrayElement() const;

		RIGVM_API bool IsUObject() const;

		// returns the property of this user data or nullptr
		const FProperty* GetProperty() const { return Property; }

		// returns the mapped memory of this user data
		const uint8* GetMemory() const { return Memory; }

		// joints two user data path strings 
		static FString Join(const FString& InLeft, const FString& InRight)
		{
			check(!InLeft.IsEmpty());
			check(!InRight.IsEmpty());
			return InLeft + TEXT(".") + InRight;
		}

		// splits a user data string path at the first segment
		static bool SplitAtStart(const FString& InPath, FString& OutLeft, FString& OutRight)
		{
			return InPath.Split(TEXT("."), &OutLeft, &OutRight, ESearchCase::IgnoreCase, ESearchDir::FromStart);
		}

		// splits a user data string path at the last segment
		static bool SplitAtEnd(const FString& InPath, FString& OutLeft, FString& OutRight)
		{
			return InPath.Split(TEXT("."), &OutLeft, &OutRight, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		}

		static TArray<UStruct*> GetSuperStructs(UStruct* InStruct);

	private:

		FString Path;
		mutable FName CPPType;
		const FProperty* Property;
		const uint8* Memory;
	};

	virtual ~UNameSpacedUserData() override
	{
		InvalidateCache();
	}

	RIGVM_API virtual void Serialize(FArchive& Ar) override;

	/** The namespace to use when looking up values inside of the user data. */
	UPROPERTY(EditAnywhere, Category = General)
	FString NameSpace;

	/** Returns true if userdata exists for a given path */
	bool ContainsUserData(const FString& InPath) const { return GetUserData(InPath) != nullptr; }

	/** Returns a user data path given its string path */
	RIGVM_API virtual const FUserData* GetUserData(const FString& InPath, FString* OutErrorMessage = nullptr) const;

	/** Retrieves the user data paths given a (optional) parent path */
	RIGVM_API virtual const TArray<const FUserData*>& GetUserDataArray(const FString& InParentPath = FString(), FString* OutErrorMessage = nullptr) const;

	/** Retrieves the user data path given the path string */
	const FUserData* GetParentUserData(const FString& InPath, FString* OutErrorMessage = nullptr) const
	{
		FString Left, Right;
		if(FUserData::SplitAtEnd(InPath, Left, Right))
		{
			return GetUserData(Left, OutErrorMessage);
		}
		return nullptr;
	}

protected:

	RIGVM_API const FUserData* GetUserDataWithinStruct(const UStruct* InStruct, const uint8* InMemory, const FString& InPath, const FString& InPropertyName, FString* OutErrorMessage = nullptr) const;
	RIGVM_API const TArray<const FUserData*>& GetUserDataArrayWithinStruct(UStruct* InStruct, const uint8* InMemory, const FString& InPath, FString* OutErrorMessage = nullptr) const;
	RIGVM_API const TArray<const FUserData*>& GetUserDataArrayWithinArray(const FArrayProperty* InArrayProperty, const uint8* InMemory, const FString& InPath, FString* OutErrorMessage = nullptr) const;
	RIGVM_API static const FProperty* FindPropertyByName(const UStruct* InStruct, const FName& InName);

	RIGVM_API void InvalidateCache() const;
	RIGVM_API const FUserData* StoreCacheForUserData(const FUserData& InUserData) const; 
	RIGVM_API const TArray<const FUserData*>& StoreCacheForUserDataArray(const FString& InPath, const TArray<const FUserData*>& InUserDataArray) const;
	RIGVM_API virtual bool IsPropertySupported(const FProperty* InProperty, const FString& InPath, bool bCheckPropertyFlags, FString* OutErrorMessage = nullptr) const;
	
	RIGVM_API static const TArray<const FUserData*> EmptyUserDatas;

	mutable TMap<FString, FUserData*> CachedUserData;
	mutable TMap<FString, TArray<const FUserData*>> CachedUserDataArray;
	mutable FTransactionallySafeCriticalSection CacheLock;

	static inline constexpr TCHAR PathNotFoundFormat[] = TEXT("UserData path '%s' could not be found.");
	static inline constexpr TCHAR InvalidMemoryFormat[] = TEXT("UserData at path '%s' does not have any data backing it up.");
	static inline constexpr TCHAR InvalidArrayIndexFormat[] = TEXT("Provided array index '%s' for path '%s' is invalid.");
	static inline constexpr TCHAR OutOfBoundArrayIndexFormat[] = TEXT("Provided array index for path '%s' is out of bounds (num == %d).");
	static inline constexpr TCHAR UnSupportedSubPathsFormat[] = TEXT("Cannot access sub paths of user data type '%s' for path '%s'.");
};

/**
* Namespaced user data which provides access to a linked data asset
*/
UCLASS(BlueprintType, MinimalAPI)
class UDataAssetLink : public UNameSpacedUserData
{
	GENERATED_BODY()

public:

	/** If assigned, the data asset link will provide access to the data asset's content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = General, BlueprintGetter = GetDataAsset, BlueprintSetter = SetDataAsset, Meta = (DisplayAfter="NameSpace"))
	TSoftObjectPtr<UDataAsset> DataAsset;

	UFUNCTION(BlueprintGetter)
	TSoftObjectPtr<UDataAsset> GetDataAsset() const { return DataAsset; }

	UFUNCTION(BlueprintSetter)
	RIGVM_API void SetDataAsset(TSoftObjectPtr<UDataAsset> InDataAsset);

	RIGVM_API virtual const FUserData* GetUserData(const FString& InPath, FString* OutErrorMessage = nullptr) const override;
	RIGVM_API virtual const TArray<const FUserData*>& GetUserDataArray(const FString& InParentPath = FString(), FString* OutErrorMessage = nullptr) const override;

	RIGVM_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual bool IsPostLoadThreadSafe() const override { return false; }
	RIGVM_API virtual void PostLoad() override;
	RIGVM_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	UPROPERTY()
	TObjectPtr<UDataAsset> DataAssetCached;
	
	static inline constexpr TCHAR DataAssetNullFormat[] = TEXT("User data path '%s' could not be found (DataAsset not provided)");
};
