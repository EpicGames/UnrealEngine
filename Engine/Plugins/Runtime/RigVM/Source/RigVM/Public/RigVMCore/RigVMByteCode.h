// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "RigVMCore/RigVMMemoryCommon.h"
#include "RigVMDefines.h"
#include "RigVMMemoryDeprecated.h"
#include "RigVMRegistry.h"
#include "RigVMStatistics.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "RigVMGraphFunctionIdentifier.h"

#include "RigVMByteCode.generated.h"

#define UE_API RIGVM_API

class FArchive;
class UObject;
struct FRigVMByteCode;

struct FRigVMBranchInfoKey
{
	FRigVMBranchInfoKey()
		: InstructionIndex(INDEX_NONE)
		, ArgumentIndex(INDEX_NONE)
		, Label(NAME_None)
	{}

	FRigVMBranchInfoKey(int32 InInstructionIndex, int32 InArgumentIndex)
		: InstructionIndex(InInstructionIndex)
		, ArgumentIndex(InArgumentIndex)
		, Label(NAME_None)
	{}

	FRigVMBranchInfoKey(int32 InInstructionIndex, const FName& InLabel)
		: InstructionIndex(InInstructionIndex)
		, ArgumentIndex(INDEX_NONE)
		, Label(InLabel)
	{}

	FRigVMBranchInfoKey(int32 InInstructionIndex, int32 InArgumentIndex, const FName& InLabel)
		: InstructionIndex(InInstructionIndex)
		, ArgumentIndex(InArgumentIndex)
		, Label(InLabel)
	{}

	bool IsValid() const
	{
		return InstructionIndex != INDEX_NONE && ArgumentIndex != INDEX_NONE && !Label.IsNone();
	}

	friend uint32 GetTypeHash(const FRigVMBranchInfoKey& InKey)
	{
		return HashCombine(
			HashCombine(
				GetTypeHash(InKey.InstructionIndex),
				GetTypeHash(InKey.ArgumentIndex)
			),
			GetTypeHash(InKey.Label.ToString())
		);
	}

	bool operator ==(const FRigVMBranchInfoKey& InOther) const
	{
		if(InstructionIndex != InOther.InstructionIndex)
		{
			return false;
		}

		if(ArgumentIndex != INDEX_NONE && InOther.ArgumentIndex != INDEX_NONE)
		{
			return ArgumentIndex == InOther.ArgumentIndex;
		}

		if(!Label.IsNone() && !InOther.Label.IsNone())
		{
			return Label == InOther.Label;
		}

		return true;
	}

	int32 InstructionIndex;
	int32 ArgumentIndex;
	FName Label;
};

// A description of a predicate branch in the VM's bytecode
USTRUCT()
struct FRigVMPredicateBranch
{
	GENERATED_BODY()

	FRigVMPredicateBranch()
		: VM(nullptr)
	{
		
	}
	
	UE_API ERigVMExecuteResult Execute(FRigVMExtendedExecuteContext& Context);
	bool IsValid() { return BranchInfo.IsValid(); }
	
	FRigVMBranchInfo BranchInfo;
	FRigVMMemoryHandleArray MemoryHandles;
	URigVM* VM;
};

// An argument for a callable in the bytecode.
USTRUCT()
struct FRigVMCallableArgument
{
	GENERATED_BODY()

	FRigVMCallableArgument()
		: Name(NAME_None)
		, Direction(ERigVMPinDirection::Invalid)
	{}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FString TypeString;

	UPROPERTY()
	FRigVMOperand InterfaceOperand;

	UPROPERTY()
	FRigVMOperand ForwardedOperand;

	UPROPERTY()
	ERigVMPinDirection Direction;

	RIGVM_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMCallableArgument& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	bool IsInput() const
	{
		return Direction == ERigVMPinDirection::Input || Direction == ERigVMPinDirection::IO; 
	}

	bool IsOutput() const
	{
		return Direction == ERigVMPinDirection::Output || Direction == ERigVMPinDirection::IO; 
	}

	friend uint32 GetTypeHash(const FRigVMCallableArgument& InArgument)
	{
		return HashCombine(
			GetTypeHash(InArgument.Name.ToString()),
			GetTypeHash(InArgument.TypeString),
			GetTypeHash(InArgument.InterfaceOperand),
			GetTypeHash(InArgument.ForwardedOperand),
			GetTypeHash(InArgument.Direction));
	}

};

// A callable section of the rigvm bytecode. This is used to represent functions
// and events with arguments.
USTRUCT()
struct FRigVMCallableInfo
{
	GENERATED_BODY()

	FRigVMCallableInfo()
		: Index(INDEX_NONE)
		, Name(NAME_None)
		, FunctionHash(0)
		, FirstInstruction(INDEX_NONE)
		, LastInstruction(INDEX_NONE)
	{
	}

	FRigVMCallableInfo(int32 InIndex, const FName& InName, uint32 InHash, int32 InFirstInstruction, int32 InLastInstruction)
	: Index(InIndex)
	, Name(InName)
	, FunctionHash(InHash)
	, FirstInstruction(InFirstInstruction)
	, LastInstruction(InLastInstruction)
	{
	}

	UPROPERTY()
	int32 Index;
	
	UPROPERTY()
	FName Name;
	
	UPROPERTY()
	uint32 FunctionHash;

	UPROPERTY()
	TArray<FRigVMCallableArgument> Arguments;

	UPROPERTY()
	int32 FirstInstruction;

	UPROPERTY()
	int32 LastInstruction;

	RIGVM_API int32 GetArgumentIndex(const FName& InName) const;
	RIGVM_API const FRigVMCallableArgument* FindArgument(const FName& InName) const;
	int32 GetNumInstructions() const
	{
		return FMath::Max(0, LastInstruction - FirstInstruction + 1);
	}

	RIGVM_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMCallableInfo& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	friend uint32 GetTypeHash(const FRigVMCallableInfo& InCallableInfo)
	{
		uint32 Hash = HashCombine(
			GetTypeHash(InCallableInfo.Index),
			GetTypeHash(InCallableInfo.Name.ToString()),
			GetTypeHash(InCallableInfo.FirstInstruction),
			GetTypeHash(InCallableInfo.LastInstruction)
		);
		for (const FRigVMCallableArgument& Argument : InCallableInfo.Arguments)
		{
			Hash = HashCombine(Hash, GetTypeHash(Argument));
		}
		return Hash;
	}
};

// The code for a single operation within the RigVM
UENUM()
enum class ERigVMOpCode : uint8
{
	Execute_0_Operands, // (DEPRECATED) execute a rig function with 0 operands
	Execute_1_Operands, // (DEPRECATED) execute a rig function with 1 operands
	Execute_2_Operands, // (DEPRECATED) execute a rig function with 2 operands
	Execute_3_Operands, // (DEPRECATED) execute a rig function with 3 operands
	Execute_4_Operands, // (DEPRECATED) execute a rig function with 4 operands
	Execute_5_Operands, // (DEPRECATED) execute a rig function with 5 operands
	Execute_6_Operands, // (DEPRECATED) execute a rig function with 6 operands
	Execute_7_Operands, // (DEPRECATED) execute a rig function with 7 operands
	Execute_8_Operands, // (DEPRECATED) execute a rig function with 8 operands
	Execute_9_Operands, // (DEPRECATED) execute a rig function with 9 operands
	Execute_10_Operands, // (DEPRECATED) execute a rig function with 10 operands
	Execute_11_Operands, // (DEPRECATED) execute a rig function with 11 operands
	Execute_12_Operands, // (DEPRECATED) execute a rig function with 12 operands
	Execute_13_Operands, // (DEPRECATED) execute a rig function with 13 operands
	Execute_14_Operands, // (DEPRECATED) execute a rig function with 14 operands
	Execute_15_Operands, // (DEPRECATED) execute a rig function with 15 operands
	Execute_16_Operands, // (DEPRECATED) execute a rig function with 16 operands
	Execute_17_Operands, // (DEPRECATED) execute a rig function with 17 operands
	Execute_18_Operands, // (DEPRECATED) execute a rig function with 18 operands
	Execute_19_Operands, // (DEPRECATED) execute a rig function with 19 operands
	Execute_20_Operands, // (DEPRECATED) execute a rig function with 20 operands
	Execute_21_Operands, // (DEPRECATED) execute a rig function with 21 operands
	Execute_22_Operands, // (DEPRECATED) execute a rig function with 22 operands
	Execute_23_Operands, // (DEPRECATED) execute a rig function with 23 operands
	Execute_24_Operands, // (DEPRECATED) execute a rig function with 24 operands
	Execute_25_Operands, // (DEPRECATED) execute a rig function with 25 operands
	Execute_26_Operands, // (DEPRECATED) execute a rig function with 26 operands
	Execute_27_Operands, // (DEPRECATED) execute a rig function with 27 operands
	Execute_28_Operands, // (DEPRECATED) execute a rig function with 28 operands
	Execute_29_Operands, // (DEPRECATED) execute a rig function with 29 operands
	Execute_30_Operands, // (DEPRECATED) execute a rig function with 30 operands
	Execute_31_Operands, // (DEPRECATED) execute a rig function with 31 operands
	Execute_32_Operands, // (DEPRECATED) execute a rig function with 32 operands
	Execute_33_Operands, // (DEPRECATED) execute a rig function with 33 operands
	Execute_34_Operands, // (DEPRECATED) execute a rig function with 34 operands
	Execute_35_Operands, // (DEPRECATED) execute a rig function with 35 operands
	Execute_36_Operands, // (DEPRECATED) execute a rig function with 36 operands
	Execute_37_Operands, // (DEPRECATED) execute a rig function with 37 operands
	Execute_38_Operands, // (DEPRECATED) execute a rig function with 38 operands
	Execute_39_Operands, // (DEPRECATED) execute a rig function with 39 operands
	Execute_40_Operands, // (DEPRECATED) execute a rig function with 40 operands
	Execute_41_Operands, // (DEPRECATED) execute a rig function with 41 operands
	Execute_42_Operands, // (DEPRECATED) execute a rig function with 42 operands
	Execute_43_Operands, // (DEPRECATED) execute a rig function with 43 operands
	Execute_44_Operands, // (DEPRECATED) execute a rig function with 44 operands
	Execute_45_Operands, // (DEPRECATED) execute a rig function with 45 operands
	Execute_46_Operands, // (DEPRECATED) execute a rig function with 46 operands
	Execute_47_Operands, // (DEPRECATED) execute a rig function with 47 operands
	Execute_48_Operands, // (DEPRECATED) execute a rig function with 48 operands
	Execute_49_Operands, // (DEPRECATED) execute a rig function with 49 operands
	Execute_50_Operands, // (DEPRECATED) execute a rig function with 50 operands
	Execute_51_Operands, // (DEPRECATED) execute a rig function with 51 operands
	Execute_52_Operands, // (DEPRECATED) execute a rig function with 52 operands
	Execute_53_Operands, // (DEPRECATED) execute a rig function with 53 operands
	Execute_54_Operands, // (DEPRECATED) execute a rig function with 54 operands
	Execute_55_Operands, // (DEPRECATED) execute a rig function with 55 operands
	Execute_56_Operands, // (DEPRECATED) execute a rig function with 56 operands
	Execute_57_Operands, // (DEPRECATED) execute a rig function with 57 operands
	Execute_58_Operands, // (DEPRECATED) execute a rig function with 58 operands
	Execute_59_Operands, // (DEPRECATED) execute a rig function with 59 operands
	Execute_60_Operands, // (DEPRECATED) execute a rig function with 60 operands
	Execute_61_Operands, // (DEPRECATED) execute a rig function with 61 operands
	Execute_62_Operands, // (DEPRECATED) execute a rig function with 62 operands
	Execute_63_Operands, // (DEPRECATED) execute a rig function with 63 operands
	Execute_64_Operands, // (DEPRECATED) execute a rig function with 64 operands
	Zero, // zero the memory of a given register
	BoolFalse, // set a given register to false
	BoolTrue, // set a given register to true
	Copy, // copy the content of one register to another
	Increment, // increment a int32 register
	Decrement, // decrement a int32 register
	Equals, // fill a bool register with the result of (A == B)
	NotEquals, // fill a bool register with the result of (A != B)
	JumpAbsolute, // jump to an absolute instruction index
	JumpForward, // jump forwards given a relative instruction index offset
	JumpBackward, // jump backwards given a relative instruction index offset
	JumpAbsoluteIf, // jump to an absolute instruction index based on a condition register
	JumpForwardIf, // jump forwards given a relative instruction index offset based on a condition register
	JumpBackwardIf, // jump backwards given a relative instruction index offset based on a condition register
	ChangeType, // change the type of a register (deprecated)
	Exit, // exit the execution loop
	BeginBlock, // begins a new memory slice / block
	EndBlock, // ends the last memory slice / block
	ArrayReset, // (DEPRECATED) clears an array and resets its content
	ArrayGetNum, // (DEPRECATED) reads and returns the size of an array (binary op, in array, out int32) 
	ArraySetNum, // (DEPRECATED) resizes an array (binary op, in out array, in int32)
	ArrayGetAtIndex, // (DEPRECATED) returns an array element by index (ternary op, in array, in int32, out element)  
	ArraySetAtIndex, // (DEPRECATED) sets an array element by index (ternary op, in out array, in int32, in element)
	ArrayAdd, // (DEPRECATED) adds an element to an array (ternary op, in out array, in element, out int32 index)
	ArrayInsert, // (DEPRECATED) inserts an element to an array (ternary op, in out array, in int32, in element)
	ArrayRemove, // (DEPRECATED) removes an element from an array (binary op, in out array, in inindex)
	ArrayFind, // (DEPRECATED) finds and returns the index of an element (quaternery op, in array, in element, out int32 index, out bool success)
	ArrayAppend, // (DEPRECATED) appends an array to another (binary op, in out array, in array)
	ArrayClone, // (DEPRECATED) clones an array (binary op, in array, out array)
	ArrayIterator, // (DEPRECATED) iterates over an array (senary op, in array, out element, out index, out count, out ratio, out continue)
	ArrayUnion, // (DEPRECATED) merges two arrays while avoiding duplicates (binary op, in out array, in other array)
	ArrayDifference, // (DEPRECATED) returns a new array containing elements only found in one array (ternary op, in array, in array, out result)
	ArrayIntersection, // (DEPRECATED) returns a new array containing elements found in both of the input arrays (ternary op, in array, in array, out result)
	ArrayReverse, // (DEPRECATED) returns the reverse of the input array (unary op, in out array)
	InvokeEntry, // invokes an entry from the entry list
	JumpToBranch, // jumps to a branch based on a name operand
	Execute, // single execute op (formerly Execute_0_Operands to Execute_64_Operands)
	RunInstructions, // runs a set of instructions lazily
	SetupTraits, // sets up a list of traits on executecontext
	InvokeCallable, // invoke a section of the bytecode as a callable
	Invalid,
	FirstArrayOpCode = ArrayReset,
	LastArrayOpCode = ArrayReverse,
};

// Base class for all VM operations
USTRUCT()
struct FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMBaseOp(ERigVMOpCode InOpCode = ERigVMOpCode::Invalid)
	: OpCode(InOpCode)
	{
	}

	ERigVMOpCode OpCode;

	friend uint32 GetTypeHash(const FRigVMBaseOp& Op)
	{
		return GetTypeHash(Op.OpCode);
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		RigVM::ZeroPaddedMemory(&InMemory->OpCode, reinterpret_cast<uint8*>(InMemory) + sizeof(FRigVMBaseOp));
	}
};

// invoke a callable
USTRUCT()
struct FRigVMInvokeCallableOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMInvokeCallableOp()
	: FRigVMBaseOp()
	, CallableIndex(0)
	, ArgumentCount(0)
	{
	}

	FRigVMInvokeCallableOp(uint16 InCallableIndex, uint16 InArgumentCount)
	: FRigVMBaseOp(ERigVMOpCode::InvokeCallable)
	, CallableIndex(InCallableIndex)
	, ArgumentCount(InArgumentCount)
	{
	}

	FRigVMInvokeCallableOp(ERigVMOpCode InOpCode, uint16 InCallableIndex, uint16 InArgumentCount)
	: FRigVMBaseOp(InOpCode)
	, CallableIndex(InCallableIndex)
	, ArgumentCount(InArgumentCount)
	{
	}

	uint16 CallableIndex;
	uint16 ArgumentCount;

	friend uint32 GetTypeHash(const FRigVMInvokeCallableOp& Op)
	{
		return HashCombine(
			GetTypeHash((const FRigVMBaseOp&)Op),
			HashCombine(
				GetTypeHash(Op.CallableIndex),
				GetTypeHash(Op.ArgumentCount)
			)
		);
	}

	uint16 GetOperandCount() const { return ArgumentCount; }

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMInvokeCallableOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMInvokeCallableOp* This = reinterpret_cast<FRigVMInvokeCallableOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->CallableIndex);
	}
};

// execute a function
USTRUCT()
struct FRigVMExecuteOp : public FRigVMInvokeCallableOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMExecuteOp()
	: FRigVMInvokeCallableOp()
	, FirstPredicateIndex(INDEX_NONE)
	, PredicateCount(0)
	{
	}

	FRigVMExecuteOp(uint16 InFunctionIndex, uint16 InArgumentCount)
	: FRigVMInvokeCallableOp(ERigVMOpCode::Execute, InFunctionIndex, InArgumentCount)
	, FirstPredicateIndex(INDEX_NONE)
	, PredicateCount(0)
	{
	}

	uint16 FirstPredicateIndex;
	uint16 PredicateCount;

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMExecuteOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

// operator used for zero, false, true, increment, decrement
USTRUCT()
struct FRigVMUnaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMUnaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, Arg()
	{
	}

	FRigVMUnaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArg)
		: FRigVMBaseOp(InOpCode)
		, Arg(InArg)
	{
		check(
			InOpCode == ERigVMOpCode::Zero ||
			InOpCode == ERigVMOpCode::BoolFalse ||
			InOpCode == ERigVMOpCode::BoolTrue ||
			InOpCode == ERigVMOpCode::Increment ||
			InOpCode == ERigVMOpCode::Decrement ||
			InOpCode == ERigVMOpCode::JumpAbsoluteIf ||
			InOpCode == ERigVMOpCode::JumpForwardIf ||
			InOpCode == ERigVMOpCode::JumpBackwardIf ||
			InOpCode == ERigVMOpCode::ChangeType ||
			InOpCode == ERigVMOpCode::JumpToBranch ||
			InOpCode == ERigVMOpCode::RunInstructions ||
			InOpCode == ERigVMOpCode::SetupTraits
		);
	}

	FRigVMOperand Arg;

	friend uint32 GetTypeHash(const FRigVMUnaryOp& Op)
	{
		return HashCombine(GetTypeHash(static_cast<const FRigVMBaseOp&>(Op)), GetTypeHash(Op.Arg));
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMUnaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMUnaryOp* This = reinterpret_cast<FRigVMUnaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->Arg);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->Arg);
	}
};

// operator used for beginblock and array reset
USTRUCT()
struct FRigVMBinaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMBinaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, ArgA()
		, ArgB()
	{
	}

	FRigVMBinaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArgA, FRigVMOperand InArgB)
		: FRigVMBaseOp(InOpCode)
		, ArgA(InArgA)
		, ArgB(InArgB)
	{
	}

	FRigVMOperand ArgA;
	FRigVMOperand ArgB;

	friend uint32 GetTypeHash(const FRigVMBinaryOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgA));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgB));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMBinaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMBinaryOp* This = reinterpret_cast<FRigVMBinaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgB);
	}
};

// operator used for some array operations
USTRUCT()
struct FRigVMTernaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMTernaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, ArgA()
		, ArgB()
		, ArgC()
	{
	}

	FRigVMTernaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArgA, FRigVMOperand InArgB, FRigVMOperand InArgC)
		: FRigVMBaseOp(InOpCode)
		, ArgA(InArgA)
		, ArgB(InArgB)
		, ArgC(InArgC)
	{
	}

	FRigVMOperand ArgA;
	FRigVMOperand ArgB;
	FRigVMOperand ArgC;

	friend uint32 GetTypeHash(const FRigVMTernaryOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgA));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgB));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgC));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMTernaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMTernaryOp* This = reinterpret_cast<FRigVMTernaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgB);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgC);
	}
};

// operator used for some array operations
USTRUCT()
struct FRigVMQuaternaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMQuaternaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, ArgA()
		, ArgB()
		, ArgC()
		, ArgD()
	{
	}

	FRigVMQuaternaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArgA, FRigVMOperand InArgB, FRigVMOperand InArgC, FRigVMOperand InArgD)
		: FRigVMBaseOp(InOpCode)
		, ArgA(InArgA)
		, ArgB(InArgB)
		, ArgC(InArgC)
		, ArgD(InArgD)
	{
		check(InOpCode == ERigVMOpCode::ArrayFind);
	}

	FRigVMOperand ArgA;
	FRigVMOperand ArgB;
	FRigVMOperand ArgC;
	FRigVMOperand ArgD;

	friend uint32 GetTypeHash(const FRigVMQuaternaryOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgA));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgB));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgC));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgD));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMQuaternaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMQuaternaryOp* This = reinterpret_cast<FRigVMQuaternaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgB);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgC);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgD);
	}
};

// operator used for some array operations
USTRUCT()
struct FRigVMQuinaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMQuinaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, ArgA()
		, ArgB()
		, ArgC()
		, ArgD()
		, ArgE()
	{
	}

	FRigVMQuinaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArgA, FRigVMOperand InArgB, FRigVMOperand InArgC, FRigVMOperand InArgD, FRigVMOperand InArgE)
		: FRigVMBaseOp(InOpCode)
		, ArgA(InArgA)
		, ArgB(InArgB)
		, ArgC(InArgC)
		, ArgD(InArgD)
		, ArgE(InArgE)
	{
	}

	FRigVMOperand ArgA;
	FRigVMOperand ArgB;
	FRigVMOperand ArgC;
	FRigVMOperand ArgD;
	FRigVMOperand ArgE;

	friend uint32 GetTypeHash(const FRigVMQuinaryOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgA));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgB));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgC));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgD));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgE));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMQuinaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMQuinaryOp* This = reinterpret_cast<FRigVMQuinaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgB);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgC);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgD);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgE);
	}
};

// operator used for some array operations
USTRUCT()
struct FRigVMSenaryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMSenaryOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, ArgA()
		, ArgB()
		, ArgC()
		, ArgD()
		, ArgE()
		, ArgF()
	{
	}

	FRigVMSenaryOp(ERigVMOpCode InOpCode, FRigVMOperand InArgA, FRigVMOperand InArgB, FRigVMOperand InArgC, FRigVMOperand InArgD, FRigVMOperand InArgE, FRigVMOperand InArgF)
		: FRigVMBaseOp(InOpCode)
		, ArgA(InArgA)
		, ArgB(InArgB)
		, ArgC(InArgC)
		, ArgD(InArgD)
		, ArgE(InArgE)
		, ArgF(InArgF)
	{
	}

	FRigVMOperand ArgA;
	FRigVMOperand ArgB;
	FRigVMOperand ArgC;
	FRigVMOperand ArgD;
	FRigVMOperand ArgE;
	FRigVMOperand ArgF;

	friend uint32 GetTypeHash(const FRigVMSenaryOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgA));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgB));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgC));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgD));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgE));
		Hash = HashCombine(Hash, GetTypeHash(Op.ArgF));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMSenaryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMSenaryOp* This = reinterpret_cast<FRigVMSenaryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgA);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgB);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgC);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgD);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgE);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->ArgF);
	}
};

// The kind of copy operation to perform
UENUM()
enum class ERigVMCopyType : uint8
{
	Default,
	FloatToDouble,
	DoubleToFloat
};

// copy the content of one register to another
USTRUCT()
struct FRigVMCopyOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

public:

	FRigVMCopyOp()
	: FRigVMBaseOp(ERigVMOpCode::Copy)
	, Source()
	, Target()
	, RegisterType(ERigVMRegisterType::Invalid)
	, CopyType(ERigVMCopyType::Default)
	{
	}

	FRigVMCopyOp(
		FRigVMOperand InSource,
		FRigVMOperand InTarget
	)
		: FRigVMBaseOp(ERigVMOpCode::Copy)
		, Source(InSource)
		, Target(InTarget)
		, RegisterType(ERigVMRegisterType::Invalid)
		, CopyType(ERigVMCopyType::Default)
	{
	}

	bool IsValid() const
	{
		return
			Source.IsValid() &&
			Target.IsValid() &&
			(Source != Target) &&
			(Target.GetMemoryType() != ERigVMMemoryType::Literal);
	}

	FRigVMOperand Source;
	FRigVMOperand Target;

	friend uint32 GetTypeHash(const FRigVMCopyOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.Source));
		Hash = HashCombine(Hash, GetTypeHash(Op.Target));
		return Hash;
	}

private:
	ERigVMRegisterType RegisterType;
	ERigVMCopyType CopyType;

public:
	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMCopyOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMCopyOp* This = reinterpret_cast<FRigVMCopyOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->Source);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->Source);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->Target);
		RigVM::ZeroPaddedMemory(&This->RegisterType, &This->CopyType);
		RigVM::ZeroPaddedMemory(&This->CopyType, reinterpret_cast<uint8*>(This) + sizeof(FRigVMCopyOp));
	}
};

// used for equals and not equals comparisons
USTRUCT()
struct FRigVMComparisonOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMComparisonOp()
		: FRigVMBaseOp(ERigVMOpCode::Invalid)
		, A()
		, B()
		, Result()
	{
	}

	FRigVMComparisonOp(
		ERigVMOpCode InOpCode,
		FRigVMOperand InA,
		FRigVMOperand InB,
		FRigVMOperand InResult
	)
		: FRigVMBaseOp(InOpCode)
		, A(InA)
		, B(InB)
		, Result(InResult)
	{
		check(
			InOpCode == ERigVMOpCode::Equals ||
			InOpCode == ERigVMOpCode::NotEquals
			);
	}

	FRigVMOperand A;
	FRigVMOperand B;
	FRigVMOperand Result;

	friend uint32 GetTypeHash(const FRigVMComparisonOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.A));
		Hash = HashCombine(Hash, GetTypeHash(Op.B));
		Hash = HashCombine(Hash, GetTypeHash(Op.Result));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMComparisonOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMComparisonOp* This = reinterpret_cast<FRigVMComparisonOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->A);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->A);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->B);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->Result);
	}
};

// jump to a new instruction index.
// the instruction can be absolute, relative forward or relative backward
// based on the opcode 
USTRUCT()
struct FRigVMJumpOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMJumpOp()
	: FRigVMBaseOp(ERigVMOpCode::Invalid)
	, InstructionIndex(INDEX_NONE)
	{
	}

	FRigVMJumpOp(ERigVMOpCode InOpCode, int32 InInstructionIndex)
	: FRigVMBaseOp(InOpCode)
	, InstructionIndex(InInstructionIndex)
	{
		check(InOpCode >= ERigVMOpCode::JumpAbsolute);
		check(InOpCode <= ERigVMOpCode::JumpBackward);
	}

	int32 InstructionIndex;

	friend uint32 GetTypeHash(const FRigVMJumpOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.InstructionIndex));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMJumpOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMJumpOp* This = reinterpret_cast<FRigVMJumpOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->InstructionIndex);
	}

	static int32 GetTargetInstruction(ERigVMOpCode InOpCode, const int32 InCurrentInstruction, const int32 InOpJumpIndex)
	{
		check(InOpCode == ERigVMOpCode::JumpAbsolute ||
			InOpCode == ERigVMOpCode::JumpForward ||
			InOpCode == ERigVMOpCode::JumpBackward ||
			InOpCode == ERigVMOpCode::JumpAbsoluteIf ||
			InOpCode == ERigVMOpCode::JumpForwardIf ||
			InOpCode == ERigVMOpCode::JumpBackwardIf);

		if (InOpCode == ERigVMOpCode::JumpAbsolute ||
			InOpCode == ERigVMOpCode::JumpAbsoluteIf)
		{
			return InOpJumpIndex;
		}

		if (InOpCode == ERigVMOpCode::JumpForward ||
			InOpCode == ERigVMOpCode::JumpForwardIf)
		{
			return InCurrentInstruction + InOpJumpIndex;
		}

		return InCurrentInstruction - InOpJumpIndex;
	}
};

// jump to a new instruction index based on a condition.
// the instruction can be absolute, relative forward or relative backward
// based on the opcode 
USTRUCT()
struct FRigVMJumpIfOp : public FRigVMUnaryOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMJumpIfOp()
		: InstructionIndex(INDEX_NONE)
		, Condition(true)
	{
	}

	FRigVMJumpIfOp(ERigVMOpCode InOpCode, FRigVMOperand InConditionArg, int32 InInstructionIndex, bool InCondition = false)
		: FRigVMUnaryOp(InOpCode, InConditionArg)
		, InstructionIndex(InInstructionIndex)
		, Condition(InCondition)
	{
		ensure(InOpCode >= ERigVMOpCode::JumpAbsoluteIf);
		ensure(InOpCode <= ERigVMOpCode::JumpBackwardIf);
	}

	int32 InstructionIndex;
	bool Condition;

	friend uint32 GetTypeHash(const FRigVMJumpIfOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMUnaryOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.InstructionIndex));
		Hash = HashCombine(Hash, GetTypeHash(Op.Condition));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMJumpIfOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
	
	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMJumpIfOp* This = reinterpret_cast<FRigVMJumpIfOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->Arg);
		FRigVMOperand::ZeroPaddedMemoryIfNeeded(&This->Arg);
		RigVM::ZeroPaddedMemory<bool>(&This->Condition, reinterpret_cast<uint8*>(This) + sizeof(FRigVMJumpIfOp));
	}
};

// change the type of a register
USTRUCT()
struct FRigVMChangeTypeOp : public FRigVMUnaryOp
{
	GENERATED_USTRUCT_BODY()

	static UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMChangeTypeOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

// invoke another entry
USTRUCT()
struct FRigVMInvokeEntryOp : public FRigVMBaseOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMInvokeEntryOp()
		: FRigVMBaseOp(ERigVMOpCode::InvokeEntry)
		, EntryName(NAME_None)
	{}

	FRigVMInvokeEntryOp(FName InEntryName)
		: FRigVMBaseOp(ERigVMOpCode::InvokeEntry)
		, EntryName(InEntryName)
	{}

	FName EntryName;

	friend uint32 GetTypeHash(const FRigVMInvokeEntryOp& Op)
	{
		return HashCombine(GetTypeHash(static_cast<const FRigVMBaseOp&>(Op)), GetTypeHash(Op.EntryName.ToString()));
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMInvokeEntryOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMBaseOp* InMemory)
	{
		FRigVMInvokeEntryOp* This = reinterpret_cast<FRigVMInvokeEntryOp*>(InMemory);
		RigVM::ZeroPaddedMemory(&This->OpCode, &This->EntryName);
	}
};

// jump into a branch based on a name argument
USTRUCT()
struct FRigVMJumpToBranchOp : public FRigVMUnaryOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMJumpToBranchOp()
		: FRigVMUnaryOp()
		, FirstBranchInfoIndex(INDEX_NONE)
	{
	}

	FRigVMJumpToBranchOp(FRigVMOperand InBranchNameArg, int32 InFirstBranchInfoIndex)
		: FRigVMUnaryOp(ERigVMOpCode::JumpToBranch, InBranchNameArg)
		, FirstBranchInfoIndex(InFirstBranchInfoIndex)
	{
	}

	int32 FirstBranchInfoIndex;

	friend uint32 GetTypeHash(const FRigVMJumpToBranchOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.FirstBranchInfoIndex));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMJumpToBranchOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

// runs a set of instructions lazily
USTRUCT()
struct FRigVMRunInstructionsOp : public FRigVMUnaryOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMRunInstructionsOp()
		: FRigVMUnaryOp()
		, StartInstruction(INDEX_NONE)
		, EndInstruction(INDEX_NONE)
	{
	}

	FRigVMRunInstructionsOp(FRigVMOperand InExecutionStateArg, int32 InStartInstruction, int32 InEndInstruction)
		: FRigVMUnaryOp(ERigVMOpCode::RunInstructions, InExecutionStateArg)
		, StartInstruction(InStartInstruction)
		, EndInstruction(InEndInstruction)
	{
	}

	int32 StartInstruction;
	int32 EndInstruction;

	friend uint32 GetTypeHash(const FRigVMRunInstructionsOp& Op)
	{
		uint32 Hash = GetTypeHash(static_cast<const FRigVMBaseOp&>(Op));
		Hash = HashCombine(Hash, GetTypeHash(Op.StartInstruction));
		Hash = HashCombine(Hash, GetTypeHash(Op.EndInstruction));
		return Hash;
	}

	UE_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMRunInstructionsOp& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

// sets up a list of traits in the execute context
USTRUCT()
struct FRigVMSetupTraitsOp : public FRigVMUnaryOp
{
	GENERATED_USTRUCT_BODY()

	FRigVMSetupTraitsOp()
		: FRigVMUnaryOp()
	{
	}

	FRigVMSetupTraitsOp(FRigVMOperand InTraitListArg)
		: FRigVMUnaryOp(ERigVMOpCode::SetupTraits, InTraitListArg)
	{
	}
};

/**
 * The FRigVMInstruction represents
 * a single instruction within the VM.
 */
USTRUCT()
struct FRigVMInstruction
{
	GENERATED_USTRUCT_BODY()

	FRigVMInstruction(ERigVMOpCode InOpCode = ERigVMOpCode::Invalid, int32 InIndex = INDEX_NONE, int32 InByteCodeIndex = INDEX_NONE, uint8 InOperandAlignment = 0)
		: Index(InIndex)
		, ByteCodeIndex(InByteCodeIndex)
		, OpCode(InOpCode)
		, OperandAlignment(InOperandAlignment)
	{
	}

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	int32 ByteCodeIndex;

	UPROPERTY()
	ERigVMOpCode OpCode;

	UPROPERTY()
	uint8 OperandAlignment;
};

/**
 * The FRigVMInstructionArray represents all current instructions
 * within a RigVM and can be used to iterate over all operators and retrieve
 * each instruction's data.
 */
USTRUCT()
struct FRigVMInstructionArray
{
	GENERATED_USTRUCT_BODY()

public:

	UE_API FRigVMInstructionArray();

	// Resets the data structure and maintains all storage.
	UE_API void Reset();

	// Resets the data structure and removes all storage.
	UE_API void Empty();

	// Returns true if a given instruction index is valid.
	bool IsValidIndex(int32 InIndex) const { return Instructions.IsValidIndex(InIndex); }

	// Returns the number of instructions.
	int32 Num() const { return Instructions.Num(); }

	// const accessor for an instruction given its index
	const FRigVMInstruction& operator[](int32 InIndex) const { return Instructions[InIndex]; }

	TArray<FRigVMInstruction>::RangedForConstIteratorType begin() const { return Instructions.begin(); }
	TArray<FRigVMInstruction>::RangedForConstIteratorType end() const { return Instructions.end(); }

	const FRigVMInstruction* GetData() const { return Instructions.GetData(); }

private:

	// hide utility constructor
	UE_API FRigVMInstructionArray(const FRigVMByteCode& InByteCode, bool bByteCodeIsAligned = true);

	UPROPERTY()
	TArray<FRigVMInstruction> Instructions;

	friend struct FRigVMByteCode;
};

USTRUCT()
struct FRigVMByteCodeEntry
{
	GENERATED_USTRUCT_BODY()

	FRigVMByteCodeEntry()
		: Name(NAME_None)
		, InstructionIndex(0)
	{}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	int32 InstructionIndex;

	UE_API FString GetSanitizedName() const;
};

/**
 * The FRigVMByteCode is a container to store a list of instructions with
 * their corresponding data. The byte code is then used within a VM to 
 * execute. To iterate over the instructions within the byte code you can 
 * use GetInstructions() to retrieve a FRigVMInstructionArray.
 */
USTRUCT()
struct FRigVMByteCode
{
	GENERATED_USTRUCT_BODY()

public:

	UE_API FRigVMByteCode();

	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMByteCode& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// resets the container and maintains all memory
	UE_API void Reset();

	// resets the container and removes all memory
	UE_API void Empty();

	// shrinks the container and reduces the memory
	UE_API void Shrink();

	// returns a unique hash identifying this bytecode
	UE_API uint32 GetByteCodeHash() const;

	// returns a unique hash for an operator at a given instruction index
	UE_API uint32 GetOperatorHash(const FRigVMInstruction& InInstruction) const;

	// overload for GetTypeHash
	friend uint32 GetTypeHash(const FRigVMByteCode& InByteCode)
	{
		return InByteCode.GetByteCodeHash();
	}
	
	// returns the number of instructions in this container
	UE_API int32 Num() const;

	// const accessor for a byte given its index
	const uint8& operator[](int32 InIndex) const { return ByteCode[InIndex]; }

	// returns the number of entries
	UE_API int32 NumEntries() const;

	// returns the entry with a given index
	UE_API const FRigVMByteCodeEntry& GetEntry(int32 InEntryIndex) const;

	// returns the index of an entry given a name or INDEX_NONE
	UE_API int32 FindEntryIndex(const FName& InEntryName) const;

	// returns the number of callables
	UE_API int32 NumCallables() const;

	// returns true if the provided callable index is valid
	UE_API bool IsValidCallableIndex(int32 InCallableIndex) const;
	UE_API bool IsValidCallableIndex(uint16 InCallableIndex) const;

	// returns the callable info with a given index
	UE_API const FRigVMCallableInfo* GetCallable(int32 InCallableIndex) const;
	UE_API FRigVMCallableInfo* GetCallable(int32 InCallableIndex);
	UE_API const FRigVMCallableInfo* GetCallable(uint16 InCallableIndex) const;
	UE_API FRigVMCallableInfo* GetCallable(uint16 InCallableIndex);

	// returns the callable's index' with a given its hash
	UE_API int32 GetCallableIndex(const uint32& InHash) const;

	// returns a callable given its hash
	UE_API const FRigVMCallableInfo* FindCallable(const uint32& InHash) const;

	// returns a callable given its name
	UE_API const FRigVMCallableInfo* FindCallable(const FName& InName) const;

	// returns a callable given an instruction within the callable
	UE_API const FRigVMCallableInfo* FindCallableForInstruction(int32 InInstructionIndex) const;

	// adds an execute operator given its function index operands
	UE_API int32 AddExecuteOp(int32 InFunctionIndex, const FRigVMOperandArray& InOperands, const int32 InStartPredicateIndex, const int32 InPredicateCount);
	
	// adds an execute operator given its function index operands
	UE_API int32 InlineFunction(const FRigVMByteCode* FunctionByteCode, int32 InFirstIndex, int32 InLastIndex);

	// adds a zero operator to zero the memory of a given argument
	UE_API int32 AddZeroOp(const FRigVMOperand& InArg);

	// adds a false operator to set a given argument to false
	UE_API int32 AddFalseOp(const FRigVMOperand& InArg);

	// adds a true operator to set a given argument to true
	UE_API int32 AddTrueOp(const FRigVMOperand& InArg);

	// adds a copy operator to copy the content of a source argument to a target argument
	UE_API int32 AddCopyOp(const FRigVMOperand& InSource, const FRigVMOperand& InTarget);

	// adds a copy operator to copy the content of a source argument to a target argument
	UE_API int32 AddCopyOp(const FRigVMCopyOp& InCopyOp);

	// adds an increment operator to increment a int32 argument
	UE_API int32 AddIncrementOp(const FRigVMOperand& InArg);

	// adds an decrement operator to decrement a int32 argument
	UE_API int32 AddDecrementOp(const FRigVMOperand& InArg);

	// adds an equals operator to store the comparison result of A and B into a Result argument
	UE_API int32 AddEqualsOp(const FRigVMOperand& InA, const FRigVMOperand& InB, const FRigVMOperand& InResult);

	// adds an not-equals operator to store the comparison result of A and B into a Result argument
	UE_API int32 AddNotEqualsOp(const FRigVMOperand& InA, const FRigVMOperand& InB, const FRigVMOperand& InResult);

	// adds an absolute, forward or backward jump operator
	UE_API int32 AddJumpOp(ERigVMOpCode InOpCode, int32 InInstructionIndex);

	// adds an absolute, forward or backward jump operator based on a condition argument
	UE_API int32 AddJumpIfOp(ERigVMOpCode InOpCode, int32 InInstructionIndex, const FRigVMOperand& InConditionArg, bool bJumpWhenConditionIs = false);

	// adds an exit operator to exit the execution loop
	UE_API int32 AddExitOp();

	// adds an operator to end the last memory slice
	UE_API int32 AddBeginBlockOp(FRigVMOperand InCountArg, FRigVMOperand InIndexArg);

	// adds an operator to end the last memory slice
	UE_API int32 AddEndBlockOp();

	// adds an invoke entry operator
	UE_API int32 AddInvokeEntryOp(const FName& InEntryName);

	// adds an invoke callable operator
	UE_API int32 AddInvokeCallableOp(const uint32& InCallableHash, const FRigVMOperandArray& InOperands);
	UE_API int32 AddInvokeCallableOpByIndex(const uint16& InCallableIndex, const FRigVMOperandArray& InOperands);

	// adds a jump to branch operator
	UE_API int32 AddJumpToBranchOp(FRigVMOperand InBranchNameArg, int32 InFirstBranchInfoIndex);

	// adds a run instructions op
	UE_API int32 AddRunInstructionsOp(FRigVMOperand InExecuteStateArg, int32 InStartInstruction, int32 InEndInstruction);

	// adds a setup traits op
	UE_API int32 AddSetupTraitsOp(FRigVMOperand InTraitListArg);

	// adds information about a branch for an instruction's argument
	UE_API int32 AddBranchInfo(const FRigVMBranchInfo& InBranchInfo);
	UE_API int32 AddBranchInfo(const FName& InBranchLabel, int32 InInstructionIndex, int32 InArgumentIndex, int32 InFirstBranchInstruction, int32 InLastBranchInstruction);

	// adds information about a predicate branch for an instruction
	UE_API int32 AddPredicateBranch(const FRigVMPredicateBranch& InPredicateBranch);

	// returns the number of branches within this bytecode
	int32 NumBranches() const { return BranchInfos.Num(); }

	// returns the branch given the branch index
	const FRigVMBranchInfo& GetBranch(int32 InIndex) const
	{
		return BranchInfos[InIndex];
	}

	// returns the branch for a given argument index
	UE_API const FRigVMBranchInfo* FindBranch(int32 InInstructionIndex, int32 InArgumentIndex) const; 

	// returns the branch for a given label
	UE_API const FRigVMBranchInfo* FindBranch(int32 InInstructionIndex, const FName& InLabel) const; 

	// returns the number of predicate branches within this bytecode
	int32 NumPredicateBranches() const { return PredicateBranches.Num(); }

	// returns the predicate branch given the branch index
	const FRigVMPredicateBranch& GetPredicateBranch(int32 InIndex) const
	{
		return PredicateBranches[InIndex];
	}

	// returns an instruction array for iterating over all operators
	FRigVMInstructionArray GetInstructions() const
	{
		return FRigVMInstructionArray(*this, bByteCodeIsAligned);
	}

	// returns true if the provided opcode refers to a callable
	static bool IsCallableOpCode(const ERigVMOpCode InOpCode)
	{
		return InOpCode == ERigVMOpCode::InvokeCallable || InOpCode == ERigVMOpCode::Execute;
	}

	// returns the size of a callable op
	static int32 GetCallableOpSize(const ERigVMOpCode InOpCode)
	{
		return InOpCode == ERigVMOpCode::Execute ? sizeof(FRigVMExecuteOp) : sizeof(FRigVMInvokeCallableOp);
	}

	// returns the opcode at a given byte index
	ERigVMOpCode GetOpCodeAt(int32 InByteCodeIndex) const
	{
		checkSlow(InByteCodeIndex >= 0 && InByteCodeIndex < ByteCode.Num());
		return static_cast<ERigVMOpCode>(ByteCode[InByteCodeIndex]);
	}

	// returns the size of the operator in bytes at a given byte index
	UE_API int32 GetOpNumBytesAt(int32 InByteCodeIndex, bool bIncludeOperands) const;

	// returns an operator at a given byte code index
	template<class OpType>
	const OpType& GetOpAt(int32 InByteCodeIndex) const
	{
		checkSlow(InByteCodeIndex >= 0 && InByteCodeIndex <= ByteCode.Num() - sizeof(OpType));
		return *reinterpret_cast<const OpType*>(ByteCode.GetData() + InByteCodeIndex);
	}

	// returns an operator for a given instruction
	template<class OpType>
	const OpType& GetOpAt(const FRigVMInstruction& InInstruction) const
	{
		return GetOpAt<OpType>(InInstruction.ByteCodeIndex);
	}

	// returns an operator at a given byte code index
	template<class OpType>
	OpType& GetOpAt(int32 InByteCodeIndex)
	{
		checkSlow(InByteCodeIndex >= 0 && InByteCodeIndex <= ByteCode.Num() - sizeof(OpType));
		return *reinterpret_cast<OpType*>(ByteCode.GetData() + InByteCodeIndex);
	}

	// returns an operator for a given instruction
	template<class OpType>
	OpType& GetOpAt(const FRigVMInstruction& InInstruction)
	{
		return GetOpAt<OpType>(InInstruction.ByteCodeIndex);
	}

	// returns a list of operands at a given byte code index
	FRigVMOperandArray GetOperandsAt(int32 InByteCodeIndex, int32 InArgumentCount) const
	{
		if (InArgumentCount == 0)
		{
			return {};
		}
		
		checkSlow(InByteCodeIndex >= 0 && InByteCodeIndex <= ByteCode.Num() - sizeof(FRigVMOperand) * InArgumentCount);
		return FRigVMOperandArray(&GetOpAt<FRigVMOperand>(InByteCodeIndex), InArgumentCount);
	}

	// returns the operands for a given execute instruction
	UE_DEPRECATED(5.8, "Please use GetOperandsForCallableOp instead.")
	FRigVMOperandArray GetOperandsForExecuteOp(const FRigVMInstruction& InInstruction) const
	{
		return GetOperandsForCallableOp(InInstruction);
	}

	FRigVMOperandArray GetOperandsForCallableOp(const FRigVMInstruction& InInstruction) const
	{
		int32 ByteCodeIndex = InInstruction.ByteCodeIndex;
		const FRigVMInvokeCallableOp& CallableOp = GetOpAt<FRigVMInvokeCallableOp>(ByteCodeIndex);

		// if the bytecode is not aligned the OperandAlignment needs to be 0
		check(bByteCodeIsAligned || InInstruction.OperandAlignment == 0);
		
		if (IsCallableOpCode(CallableOp.OpCode))
		{
			ByteCodeIndex += GetCallableOpSize(CallableOp.OpCode) + InInstruction.OperandAlignment;
		}
		return GetOperandsAt(ByteCodeIndex, CallableOp.GetOperandCount());
	}
	
	// returns all of the operands for a given instruction
	UE_API FRigVMOperandArray GetOperandsForOp(const FRigVMInstruction& InInstruction) const;

	// returns the byte index of the first operand for this instructions
	UE_API int32 GetFirstOperandByteIndex(const FRigVMInstruction& InInstruction) const;

	// returns all instructions for a given operand
	UE_API TArray<int32> GetInstructionsForOperand(const FRigVMOperand& InOperand) const;

#if WITH_EDITOR
	// returns all instructions for a given input operand
	UE_API TArray<int32> GetInstructionsForInputOperand(const FRigVMOperand& InOperand) const;

	// returns all instructions for a given output operand 
	UE_API TArray<int32> GetInstructionsForOutputOperand(const FRigVMOperand& InOperand) const;
#endif

	// returns true if the operator in question is used by multiple instructions
	bool IsOperandShared(const FRigVMOperand& InOperand) const { return GetInstructionsForOperand(InOperand).Num() > 1; }

	// returns the raw data of the byte code
	TConstArrayView<uint8> GetByteCode() const
	{
		return {ByteCode.GetData(), ByteCode.Num()};
	}

	// returns the statistics information
	FRigVMByteCodeStatistics GetStatistics() const
	{
		FRigVMByteCodeStatistics Statistics;
		Statistics.InstructionCount = GetInstructions().Num();
		Statistics.DataBytes = static_cast<int32>(ByteCode.GetAllocatedSize());
		return Statistics;
	}

	// returns the number of instructions within this byte code
	int32 GetNumInstructions() const { return NumInstructions; }

	// returns the alignment for an operator given its opcode
	static UE_API int32 GetOpAlignment(ERigVMOpCode InOpCode);

	// returns the alignment for an operand
	static UE_API int32 GetOperandAlignment();

	UE_API FString DumpToText() const;

	bool HasPublicContextAssetPath() const
	{
		return bHasPublicContextAssetPath;
	}

	const FTopLevelAssetPath& GetPublicContextAssetPath() const
	{
		return PublicContextAssetPath;
	}

	void SetPublicContextAssetPath(const FTopLevelAssetPath& InPublicContextAssetPath)
	{
		PublicContextAssetPath = InPublicContextAssetPath;
		bHasPublicContextAssetPath = true;
	}

#if WITH_EDITOR

	// returns the subject which was used to inject a given instruction
	UE_API UObject* GetSubjectForInstruction(int32 InInstructionIndex) const;

	// returns the first hit instruction index for a given subject (or INDEX_NONE)
	UE_API int32 GetFirstInstructionIndexForSubject(UObject* InSubject) const;

	// returns all found instruction indices for a given subject
	UE_API const TArray<int32>& GetAllInstructionIndicesForSubject(UObject* InSubject) const;

	// returns the callpath which was used to inject a given instruction
	UE_API FString GetCallPathForInstruction(int32 InInstructionIndex) const;

	// returns the first hit instruction index for a given callpath (or INDEX_NONE)
	UE_API int32 GetFirstInstructionIndexForCallPath(const FString& InCallPath, bool bStartsWith = false, bool bEndsWith = false) const;

	// returns all found instruction indices for a given callpath
	UE_API TArray<int32> GetAllInstructionIndicesForCallPath(const FString& InCallPath, bool bStartsWith = false, bool bEndsWith = false) const;

	// returns the first hit instruction index for a given callpath (or INDEX_NONE)
	UE_API int32 GetFirstInstructionIndexForCallstack(const TArray<TWeakObjectPtr<UObject>>& InCallstack) const;

	// returns all found instruction indices for a given callpath
	UE_API const TArray<int32>& GetAllInstructionIndicesForCallstack(const TArray<TWeakObjectPtr<UObject>>& InCallstack) const;

	// returns the callstack which was used to inject a given instruction
	UE_API const TArray<TWeakObjectPtr<UObject>>* GetCallstackForInstruction(int32 InInstructionIndex) const;

	// returns the callstack hash which was used to inject a given instruction
	UE_API uint32 GetCallstackHashForInstruction(int32 InInstructionIndex) const;

	// computes a hash for a given callstack
	static UE_API uint32 GetCallstackHash(const TArray<TWeakObjectPtr<UObject>>& InCallstack);
	static UE_API uint32 GetCallstackHash(const TArrayView<TWeakObjectPtr<UObject> const>& InCallstack);

	// returns the input operands of a given instruction
	FRigVMOperandArray GetInputOperands(int32 InInstructionIndex) const
	{
		if(InputOperandsPerInstruction.IsValidIndex(InInstructionIndex))
		{
			if(InputOperandsPerInstruction[InInstructionIndex].Num() > 0)
			{
				return FRigVMOperandArray(const_cast<FRigVMOperand*>(InputOperandsPerInstruction[InInstructionIndex].GetData()), InputOperandsPerInstruction[InInstructionIndex].Num());
			}
		}
		return FRigVMOperandArray();
	}

	// returns the output operands of a given instruction
	FRigVMOperandArray GetOutputOperands(int32 InInstructionIndex) const
	{
		if(OutputOperandsPerInstruction.IsValidIndex(InInstructionIndex))
		{
			if(OutputOperandsPerInstruction[InInstructionIndex].Num() > 0)
			{
				return FRigVMOperandArray(const_cast<FRigVMOperand*>(OutputOperandsPerInstruction[InInstructionIndex].GetData()), OutputOperandsPerInstruction[InInstructionIndex].Num());
			}
		}
		return FRigVMOperandArray();
	}

	UE_API void SetOperandsForInstruction(const FRigVMInstruction& InInstruction, const FRigVMOperandArray& InputOperands, const FRigVMOperandArray& OutputOperands);

#endif

	// returns the traits for the provided memory
	UE_API TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, const UScriptStruct* InScriptStruct = nullptr) const;

	// returns the traits for the provided memory and any additional memory handles
	UE_API TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles, const UScriptStruct* InScriptStruct = nullptr) const;
	
	// returns the traits of a given type for the provided memory
	template<typename T>
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory) const
	{
		return GetTraits(InLiteralMemory, InWorkMemory, T::StaticStruct());
	}

	// returns the traits of a given type for the provided memory and any additional memorty handles
	template<typename T>
	TMap<int32, TArray<FRigVMTraitScope>> GetTraits(FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles) const
	{
		return GetTraits(InLiteralMemory, InWorkMemory, OutAdditionalMemoryHandles, T::StaticStruct());
	}

	// returns the traits for the provided memory for a single instruction
	UE_API TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, const UScriptStruct* InScriptStruct = nullptr) const;

	// returns the traits for the provided memory for a single instruction and any additional memory handles
	UE_API TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles, const UScriptStruct* InScriptStruct = nullptr) const;

	// returns the traits of a given type for the provided memory for a single instruction
	template<typename T>
	TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory) const
	{
		return GetTraitsForInstruction(InInstruction, InLiteralMemory, InWorkMemory, T::StaticStruct());
	}

	// returns the traits of a given type for the provided memory for a single instruction and any additional memory handles
	template<typename T>
	TArray<FRigVMTraitScope> GetTraitsForInstruction(const FRigVMInstruction& InInstruction, FRigVMMemoryStorageStruct& InLiteralMemory, FRigVMMemoryStorageStruct& InWorkMemory, TArray<FRigVMMemoryHandle>& OutAdditionalMemoryHandles) const
	{
		return GetTraitsForInstruction(InInstruction, InLiteralMemory, InWorkMemory, T::StaticStruct(), OutAdditionalMemoryHandles);
	}

private:

	template<class OpType>
	int32 AddOp(const OpType& InOp)
	{
		check(
			(InOp.OpCode > ERigVMOpCode::Execute_64_Operands) && 
			//!(InOp.OpCode >= ERigVMOpCode::ArrayReset && InOp.OpCode <= ERigVMOpCode::ArrayReverse) && 
			(InOp.OpCode < ERigVMOpCode::Invalid) 
		);
		
		const int32 ByteIndex = ByteCode.AddZeroed(sizeof(OpType));
		uint8* Pointer = &ByteCode[ByteIndex];
		FMemory::Memcpy(Pointer, &InOp, sizeof(OpType));
		InOp.ZeroPaddedMemoryIfNeeded(reinterpret_cast<OpType*>(Pointer));
		NumInstructions++;
		return ByteIndex;
	}

	UE_API void AlignByteCode();

	// memory for all instructions
	UPROPERTY()
	TArray<uint8> ByteCode;

	// number of instructions stored here
	UPROPERTY()
	int32 NumInstructions;
	
#if WITH_EDITORONLY_DATA

	TArray<TWeakObjectPtr<UObject>> SubjectPerInstruction;
	TMap<TWeakObjectPtr<UObject>, TArray<int32>> SubjectToInstructions;
	TArray<FString> CallPathPerInstruction;
	TMap<FString, TArray<int32>> CallPathToInstructions;
	TArray<TArray<TWeakObjectPtr<UObject>>> CallstackPerInstruction;
	TMap<uint32, TArray<int32>> CallstackHashToInstructions;
	TArray<uint32> CallstackHashPerInstruction;
	TArray<TArray<FRigVMOperand>> InputOperandsPerInstruction;
	TArray<TArray<FRigVMOperand>> OutputOperandsPerInstruction;

#endif

#if WITH_EDITOR

	UE_API void SetSubject(int32 InInstructionIndex, const FString& InCallPath, const TArray<TWeakObjectPtr<UObject>>& InCallstack);
	UE_API void AddInstructionForSubject(UObject* InSubject, int32 InInstructionIndex);

#endif

	// a look up table from entry name to instruction index
	UPROPERTY()
	TArray<FRigVMByteCodeEntry> Entries;

	// a list of all lazily evaluation branches
	UPROPERTY()
	TArray<FRigVMBranchInfo> BranchInfos;

	// a list of all predicate branches
	UPROPERTY()
	TArray<FRigVMPredicateBranch> PredicateBranches;

	// a list of stored callables
	UPROPERTY()
	TArray<FRigVMCallableInfo> CallableInfos;

	UPROPERTY()
	FTopLevelAssetPath PublicContextAssetPath;

	UE_API const FRigVMBranchInfo* GetBranchInfo(const FRigVMBranchInfoKey& InBranchInfoKey) const;
	mutable TMap<FRigVMBranchInfoKey, const FRigVMBranchInfo*> BranchInfoLookup;

	// if this is set to true the stored bytecode is aligned / padded
	bool bByteCodeIsAligned;
	// If the serialization has loaded a PublicContextPathName, so we check on new versions and skip check on older
	bool bHasPublicContextAssetPath = false;

	static UE_API TArray<int32> EmptyInstructionIndices;

	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
	friend class FRigVMByteCodeTest;
	friend class FRigVMInvokeEntryTest;
	friend class URigVM;
	friend struct FRigVMCodeGenerator;
	friend struct FRigVMGraphFunctionData;
};

#undef UE_API
