// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "RigVMDebugDrawSettings.generated.h"

USTRUCT()
struct FRigVMDebugDrawSettings
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input, DisplayPriority = MAX_int32), Category = "DebugSettings")
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	UPROPERTY(meta = (Input, DisplayPriority = MAX_int32), Category = "DebugSettings")
	float Lifetime = -1.f;
};
