// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/ObjectMacros.h"

#include "RigVMDebugInfo.generated.h"

#define UE_API RIGVM_API

class UObject;
class URigVMNode;
struct FRigVMExtendedExecuteContext;

UENUM()
enum class ERigVMStepMode : uint8
{
	None,
	Stopped,	// Parked at a specific instruction
	Forward,    // Next instruction at same callable depth
	Into,       // Enter next callable
	Out,        // Exit current callable
	RunTo       // Run until target instruction (first hit)
};

/*
FRigVMStepCondition tracks:
- Mode - None, Stopped, Forward, Into, Out, RunTo
- OriginInstruction - instruction index where we're stopped
- OriginCallstackPath - array of call site instruction indices (full callstack context)
- OriginIterationPath - array of loop iteration indices
- bHasPassedOrigin - flag to ensure we've reached the origin before detecting forward progress

Step behaviors:
- Into - stops when entering a callable (deeper), advancing at same level, or exiting a callable (shallower)
- Forward - stops when advancing at same level or exiting a callable (shallower)
- Out - stops only when exiting a callable (shallower)
- RunTo - stops at target instruction regardless of context
- Stopped - re-halts at exact same location (callstack + instruction + iteration)
*/
USTRUCT()
struct FRigVMStepCondition
{
	GENERATED_BODY()

	FRigVMStepCondition()
		: Mode(ERigVMStepMode::None)
		, OriginInstruction(INDEX_NONE)
		, TargetInstruction(INDEX_NONE)
		, bHasPassedOrigin(false)
	{
	}

	ERigVMStepMode Mode;
	int32 OriginInstruction;
	int32 TargetInstruction;  // For RunTo mode
	bool bHasPassedOrigin;    // True once we've passed the origin point during re-execution

	// Callstack path - instruction index at each call site
	TArray<int32, TInlineAllocator<16>> OriginCallstackPath;

	// Iteration path when stopped - for re-halting at same location
	TArray<int32, TInlineAllocator<16>> OriginIterationPath;

	FString ToString() const
	{
		static const TCHAR* ModeNames[] = {
			TEXT("None"),
			TEXT("Stopped"),
			TEXT("Forward"),
			TEXT("Into"),
			TEXT("Out"),
			TEXT("RunTo")
		};

		FString CallstackPathStr = TEXT("[");
		for (int32 i = 0; i < OriginCallstackPath.Num(); i++)
		{
			if (i > 0) CallstackPathStr += TEXT(", ");
			CallstackPathStr += FString::Printf(TEXT("%d"), OriginCallstackPath[i]);
		}
		CallstackPathStr += TEXT("]");

		FString IterationPathStr = TEXT("[");
		for (int32 i = 0; i < OriginIterationPath.Num(); i++)
		{
			if (i > 0) IterationPathStr += TEXT(", ");
			IterationPathStr += FString::Printf(TEXT("%d"), OriginIterationPath[i]);
		}
		IterationPathStr += TEXT("]");

		return FString::Printf(TEXT("Mode=%s, OriginInstr=%d, TargetInstr=%d, PassedOrigin=%d, Callstack=%s, IterPath=%s"),
			ModeNames[static_cast<uint8>(Mode)],
			OriginInstruction,
			TargetInstruction,
			bHasPassedOrigin ? 1 : 0,
			*CallstackPathStr,
			*IterationPathStr);
	}
};

USTRUCT()
struct FRigVMDebugInfo
{
	GENERATED_BODY()

	FRigVMDebugInfo()
	{
	}

	void Reset()
	{
		StepCondition = FRigVMStepCondition();
	}

	bool IsActive() const
	{
		return StepCondition.Mode != ERigVMStepMode::None;
	}

	UE_API bool ShouldStop(const FRigVMExtendedExecuteContext& Context, int32 CurrentInstruction);

	void StepForward()
	{
		StepCondition.Mode = ERigVMStepMode::Forward;
	}

	void StepInto()
	{
		StepCondition.Mode = ERigVMStepMode::Into;
	}

	void StepOut()
	{
		StepCondition.Mode = ERigVMStepMode::Out;
	}

	void RunToInstruction(int32 Instruction)
	{
		StepCondition.Mode = ERigVMStepMode::RunTo;
		StepCondition.TargetInstruction = Instruction;
	}

	void BeginExecution()
	{
		StepCondition.bHasPassedOrigin = false;
	}

	const FRigVMStepCondition& GetStepCondition() const
	{
		return StepCondition;
	}

	void SetStepCondition(const FRigVMStepCondition& InStepCondition)
	{
		StepCondition = InStepCondition;
	}

	DECLARE_EVENT_ThreeParams(URigVM, FExecutionHaltedEvent, int32, UObject*, const FName&);
	FExecutionHaltedEvent& ExecutionHalted()
	{
		return OnExecutionHalted;
	}

private:
	UE_API void TransitionToStopped(const FRigVMExtendedExecuteContext& Context, int32 CurrentInstruction);
	UE_API static void CaptureCallstackPath(const FRigVMExtendedExecuteContext& Context, TArray<int32, TInlineAllocator<16>>& OutPath);
	UE_API static int32 CompareCallstackPath(const FRigVMExtendedExecuteContext& Context, const TArray<int32, TInlineAllocator<16>>& InPath);
	UE_API static void CaptureIterationPath(const FRigVMExtendedExecuteContext& Context, TArray<int32, TInlineAllocator<16>>& OutPath);
	UE_API static int32 CompareIterationPath(const FRigVMExtendedExecuteContext& Context, const TArray<int32, TInlineAllocator<16>>& InPath);

	FRigVMStepCondition StepCondition;
	FExecutionHaltedEvent OnExecutionHalted;
};

#undef UE_API
