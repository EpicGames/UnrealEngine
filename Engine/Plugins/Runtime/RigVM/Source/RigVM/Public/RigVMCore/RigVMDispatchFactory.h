// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "RigVMCore/RigVMRegistryHandle.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMStructUpgradeInfo.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMCore/RigVMTypeIndex.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"

#include "RigVMDispatchFactory.generated.h"

struct FRigVMRegistry_RWLock;
struct FRigVMDispatchFactory;
class URigVMPin;

/**
 * A context used for inquiring from dispatch factories
 */
struct FRigVMDispatchContext
{
	FRigVMDispatchContext()
		: Instance()
		, Subject(nullptr)
		, StringRepresentation()
	{
	}

	TSharedPtr<FStructOnScope> Instance;
	const UObject* Subject;
	FString StringRepresentation;
};

/**
 * A factory to generate a template and its dispatch functions
 */
USTRUCT()
struct FRigVMDispatchFactory
{
	GENERATED_BODY()

public:

	FRigVMDispatchFactory()
		: FactoryIndex(INDEX_NONE)
		, FactoryName(NAME_None)
		, FactoryScriptStruct(nullptr)
		, CachedTemplate(nullptr)
		, OwnerRegistry(nullptr)
	{
		ArgumentNamesMutex = new FCriticalSection();
	}
	
	virtual ~FRigVMDispatchFactory()
	{
		delete ArgumentNamesMutex;
	}

	// returns the index of this factory in its registry
	int32 GetFactoryIndex() const { return FactoryIndex; }

	// returns the name of this factory
	RIGVM_API FName GetFactoryName() const;

	// returns the name of the factory given the factory's struct 
	RIGVM_API static FName GetFactoryName(const UScriptStruct* InFactoryStruct);

	// returns the struct for this dispatch factory
	UScriptStruct* GetScriptStruct() const { return FactoryScriptStruct; };

	// returns the registry this factory belongs top
	const FRigVMRegistry_NoLock* GetOwnerRegistry() const { return OwnerRegistry; }

	// returns true if nodes provided by this factory can be renamed
	virtual bool SupportsRenaming() const { return false; }

#if WITH_EDITOR
	
	// returns the title of the node for a given type set.
	virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const { return GetScriptStruct()->GetDisplayNameText().ToString(); }

	// returns the color of the node for a given type set.
	RIGVM_API virtual FLinearColor GetNodeColor() const;

	// returns the tooltip for the node
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const;

	// returns the default value for an argument
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const;

	// returns the tooltip for an argument
	virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const { return FText(); }

	// returns the names of the input aggregate arguments
	virtual TArray<FName> GetAggregateInputArguments() const { return TArray<FName>(); }

	// returns the names of the output aggregate arguments
	virtual TArray<FName> GetAggregateOutputArguments() const { return TArray<FName>(); }

	// returns the next name to be used for an aggregate pin
	RIGVM_API virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const;

	// Returns the display name text for an argument 
	RIGVM_API virtual FName GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const;

	// Returns meta data on the property of the permutations 
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const;

	// Returns true if the factory provides metadata for a given argument
	bool HasArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const
	{
		return !GetArgumentMetaData(InArgumentName, InMetaDataKey).IsEmpty();
	}

	// Returns the category this factory is under
	RIGVM_API virtual FString GetCategory() const;

	// Returns the keywords used for looking up this factory
	RIGVM_API virtual FString GetKeywords() const;

	// Returns true if the argument is lazy
	RIGVM_API bool IsLazyInputArgument(const FName& InArgumentName) const;

#endif

	// Returns the name to use for the branch info / argument based on the operand index.
	// operand index count may be different than the number of arguments, since fixed size array
	// arguments get unrolled.
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const;

#if WITH_EDITORONLY_DATA
	// returns true if the dispatch is a control flow dispatch
	bool IsControlFlowDispatch(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const { return !GetControlFlowBlocks(InContext, InTypeMap).IsEmpty(); }

	// returns the control flow blocks of this dispatch
	RIGVM_API const TArray<FName>& GetControlFlowBlocks(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const;
#endif

	// returns true if a given control flow block needs to be sliced
	virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const { return false; }

	// Returns the execute context support for this dispatch factory
	virtual UScriptStruct* GetExecuteContextStruct() const { return FRigVMExecuteContext::StaticStruct(); }

	// Returns truf if this factory supports a given executecontext struct
	RIGVM_API bool SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const;

	// registered needed types during registration of the factory.
	// this is called within a NoLock code path - so make sure to access
	// the FRigVMRegistry_NoLock within as needed.
	virtual void RegisterDependencyTypes_NoLock(FRigVMRegistryHandle& InRegistry) const {}

	// returns the arguments of the template
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const;

	// returns the execute arguments of the template
	RIGVM_API TArray<FRigVMExecuteArgument> GetExecuteArguments_NoLock(const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const;

	// this function is deprecated, please use GetPermutationsFromArgumentType
	// returns the new permutation argument types after a new type is defined for one argument
	// this happens if types are being loaded later after this factory has already been deployed (like UUserDefinedStruct)
	virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const { return FRigVMTemplateTypeMap(); }

	// returns the new permutations argument types after a new type is defined for one argument
	virtual bool GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const
	{
		FRigVMTemplateTypeMap Permutation = OnNewArgumentType(InArgumentName, InTypeIndex, InRegistry);
		if (!Permutation.IsEmpty())
		{
			OutPermutations.Add(Permutation);
			return true;
		}
		return false;
	}

	// returns the upgrade info to use for this factory
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo(const FRigVMTemplateTypeMap& InTypes, const FRigVMDispatchContext& InContext) const { return FRigVMStructUpgradeInfo(); }

	// returns the dispatch function for a given type set
	RIGVM_API FRigVMFunctionPtr GetOrCreateDispatchFunction(const FRigVMTemplateTypeMap& InTypes) const;

	// builds and returns the template
	RIGVM_API const FRigVMTemplate* GetTemplate() const;

	// builds and returns the template
	RIGVM_API const FRigVMTemplate* GetTemplate_NoLock(FRigVMRegistryHandle& InRegistry) const;

	// returns the name of the factory template
	RIGVM_API FName GetTemplateNotation() const;
	RIGVM_API FName GetTemplateNotation_NoLock(FRigVMRegistryHandle& InRegistry) const;

	// returns the name of the permutation for a given set of types
	RIGVM_API FString GetPermutationName(const FRigVMTemplateTypeMap& InTypes) const;
	RIGVM_API FString GetPermutationName_NoLock(const FRigVMTemplateTypeMap& InTypes, FRigVMRegistryHandle& InRegistry) const;

	// returns true if the dispatch uses the same function ptr for all permutations
	virtual bool IsSingleton() const { return false; }

protected:

	// creates the template based on existing argument infos
	RIGVM_API const FRigVMTemplate* CreateTemplateForArgumentInfos_NoLock(const TArray<FRigVMTemplateArgumentInfo>& InArguments, FRigVMRegistryHandle& InRegistry);

	// returns the dispatch function for a given type set
	RIGVM_API FRigVMFunctionPtr GetOrCreateDispatchFunction_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	// for each type defined in the primary argument, this function will call GetPermutationsFromArgumentType to construct an array of arguments with the appropiate permutations
	RIGVM_API TArray<FRigVMTemplateArgumentInfo> BuildArgumentListFromPrimaryArgument(const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FName& InPrimaryArgumentName, FRigVMRegistryHandle& InRegistry) const;

	// returns the name of the permutation for a given set of types
	RIGVM_API FString GetPermutationNameImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	FRigVMFunctionPtr CreateDispatchFunction_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const { return nullptr; }
	
	TArray<FRigVMFunction> CreateDispatchPredicates_NoLock(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;
	
	virtual TArray<FRigVMFunction> GetDispatchPredicatesImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const { return TArray<FRigVMFunction>(); }

	RIGVM_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const;

	RIGVM_API static bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr);

	template <
	typename T,
	typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static FString GetDefaultValueForStruct(const T& InValue)
	{
		static FString ValueString;
		if(ValueString.IsEmpty())
		{
			TBaseStructure<T>::Get()->ExportText(ValueString, &InValue, &InValue, nullptr, PPF_None, nullptr);
		}
		return ValueString;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	static FString GetDefaultValueForStruct(const T& InValue)
	{
		static FString ValueString;
		if(ValueString.IsEmpty())
		{
			T::StaticStruct()->ExportText(ValueString, &InValue, &InValue, nullptr, PPF_None, nullptr);
		}
		return ValueString;
	}

#if WITH_EDITOR
	bool CheckArgumentType(bool bCondition, const FName& InArgumentName) const
	{
		if(!bCondition)
		{
			UE_LOGF(LogRigVM, Error, "Fatal: '%ls' Argument '%ls' has incorrect type.", *GetFactoryName().ToString(), *InArgumentName.ToString())
		}
		return bCondition;
	}
#endif
	
#if WITH_EDITORONLY_DATA
	RIGVM_API virtual const TArray<FName>& GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const;
#endif

	static const inline TCHAR* DispatchPrefix = TEXT("DISPATCH_");
	static const inline TCHAR* TrueString = TEXT("True");

	const TArray<FName>* UpdateArgumentNameCache(int32 InNumberOperands, FRigVMRegistryHandle& InRegistry) const;
	const TArray<FName>* UpdateArgumentNameCache_NoLock(int32 InNumberOperands, FRigVMRegistryHandle& InRegistry) const;

	int32 FactoryIndex;
	mutable FName FactoryName;
	FString FactoryNameString;
	UScriptStruct* FactoryScriptStruct;
	mutable const FRigVMTemplate* CachedTemplate;
	const FRigVMRegistry_NoLock* OwnerRegistry;
	static FCriticalSection GetTemplateMutex;
	mutable TMap<int32, TSharedPtr<TArray<FName>>> ArgumentNamesMap;
	FCriticalSection* ArgumentNamesMutex;
	mutable TArray<FRigVMTemplateArgumentInfo> CachedArgumentInfos;
	friend struct FRigVMTemplate;
	friend struct FRigVMRegistry_NoLock;
	friend struct FRigVMFunction;
	friend class URigVM;
	friend class FRigVMCodeConverter;
};
