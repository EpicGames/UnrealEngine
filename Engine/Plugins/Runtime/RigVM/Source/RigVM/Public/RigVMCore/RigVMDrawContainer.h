// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMDrawInstruction.h"
#include "RigVMDrawContainer.generated.h"

USTRUCT()
struct FRigVMDrawContainer
{
	GENERATED_BODY()
	virtual ~FRigVMDrawContainer() = default;

	int32 Num() const { return Instructions.Num(); }
	RIGVM_API int32 GetIndex(const FName& InName) const;
	const FRigVMDrawInstruction& operator[](int32 InIndex) const { return Instructions[InIndex]; }
	FRigVMDrawInstruction& operator[](int32 InIndex) { return Instructions[InIndex]; }
	const FRigVMDrawInstruction& operator[](const FName& InName) const { return Instructions[GetIndex(InName)]; }
	FRigVMDrawInstruction& operator[](const FName& InName) { return Instructions[GetIndex(InName)]; }

	TArray<FRigVMDrawInstruction>::RangedForIteratorType      begin() { return Instructions.begin(); }
	TArray<FRigVMDrawInstruction>::RangedForConstIteratorType begin() const { return Instructions.begin(); }
	TArray<FRigVMDrawInstruction>::RangedForIteratorType      end() { return Instructions.end(); }
	TArray<FRigVMDrawInstruction>::RangedForConstIteratorType end() const { return Instructions.end(); }

	uint32 GetAllocatedSize() const { return static_cast<uint32>(Instructions.GetAllocatedSize()); }

	RIGVM_API void Reset();

	UPROPERTY(EditAnywhere, Category = "DrawContainer")
	TArray<FRigVMDrawInstruction> Instructions;
};
