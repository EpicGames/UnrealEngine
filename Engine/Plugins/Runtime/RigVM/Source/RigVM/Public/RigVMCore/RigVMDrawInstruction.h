// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMeshBuilder.h"
#include "Engine/EngineTypes.h"
#include "RigVMDrawInstruction.generated.h"

class FMaterialRenderProxy;

UENUM()
namespace ERigVMDrawSettings
{
	enum Primitive : int
	{
		Points,
		Lines,
		LineStrip,
		DynamicMesh,
		Text
	};
}

USTRUCT()
struct FRigVMDrawInstruction
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	TEnumAsByte<ERigVMDrawSettings::Primitive> PrimitiveType = ERigVMDrawSettings::Points;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	TArray<FVector> Positions;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	FLinearColor Color = FLinearColor::Red;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	float Thickness = 0.f;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	FTransform Transform = FTransform::Identity;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	float Lifetime = -1.f;

	// Only used when PrimitiveType is Text. The string is rendered on the 2D canvas by the edit
	// mode via FCanvasTextItem, anchored at Positions[0] (additional entries in Positions are
	// ignored).
	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	FString Text;

	UPROPERTY(EditAnywhere, Category = "DrawInstruction")
	float FontScale = 1.f;

	// This is to draw cone, they're not UPROPERTY
	TArray<FDynamicMeshVertex> MeshVerts;
	TArray<uint32> MeshIndices;
	FMaterialRenderProxy* MaterialRenderProxy = nullptr;

	FRigVMDrawInstruction() = default;

	FRigVMDrawInstruction(ERigVMDrawSettings::Primitive InPrimitiveType, const FLinearColor& InColor, float InThickness = 0.f, FTransform InTransform = FTransform::Identity, ESceneDepthPriorityGroup InDepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float InLifetime = -1.f)
		: PrimitiveType(InPrimitiveType)
		, Color(InColor)
		, Thickness(InThickness)
		, Transform(InTransform)
		, DepthPriority(InDepthPriority)
		, Lifetime(InLifetime)
	{}

	bool IsValid() const
	{
		// if primitive type is dynamicmesh, we expect these data to be there.
		// otherwise, we can't draw
		if (PrimitiveType == ERigVMDrawSettings::DynamicMesh)
		{
			return MeshVerts.Num() != 0 && MeshIndices.Num() != 0 && MaterialRenderProxy != nullptr;
		}

		if (PrimitiveType == ERigVMDrawSettings::Text)
		{
			return Positions.Num() != 0 && !Text.IsEmpty();
		}

		return Positions.Num() != 0;
	}
};
