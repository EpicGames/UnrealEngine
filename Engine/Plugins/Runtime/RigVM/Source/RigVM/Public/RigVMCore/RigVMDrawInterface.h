// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMDrawContainer.h"
#include "RigVMDrawInterface.generated.h"

#define UE_API RIGVM_API

USTRUCT()
struct FRigVMDrawInterface : public FRigVMDrawContainer
{
public:

	GENERATED_BODY()

	UE_API void DrawInstruction(const FRigVMDrawInstruction& InInstruction);
	UE_API void DrawPoint(const FTransform& WorldOffset, const FVector& Position, float Size, const FLinearColor& Color, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawPoints(const FTransform& WorldOffset, const TArrayView<const FVector>& Points, float Size, const FLinearColor& Color, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawLine(const FTransform& WorldOffset, const FVector& LineStart, const FVector& LineEnd, const FLinearColor& Color, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawLines(const FTransform& WorldOffset, const TArrayView<const FVector>& Positions, const FLinearColor& Color, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawLineStrip(const FTransform& WorldOffset, const TArrayView<const FVector>& Positions, const FLinearColor& Color, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	// This draws a scaled unit cube, with the transforms applied. When drawing a box, put the position/orientation into WorldOffset and the extents into Transform.
	UE_API void DrawBox(const FTransform& WorldOffset, const FTransform& Transform, const FLinearColor& Color, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawSphere(const FTransform& WorldOffset, const FTransform& Transform, float Radius, const FLinearColor& Color, float Thickness = 0.f, int32 Detail = 12, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	// Hemispheres are drawn so that they extend up along +ve Z
	UE_API void DrawHemisphere(const FTransform& WorldOffset, const FTransform& Transform, float Radius, const FLinearColor& Color, float Thickness = 0.f, int32 Detail = 12, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	// Capsules are drawn so that they extend along Z
	UE_API void DrawCapsule(const FTransform& WorldOffset, const FTransform& Transform, float Radius, float Length, const FLinearColor& Color, float Thickness = 0.f, int32 Detail = 12, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawAxes(const FTransform& WorldOffset, const FTransform& Transform, float Size, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawAxes(const FTransform& WorldOffset, const TArrayView<const FTransform>& Transforms, float Size, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawAxes(const FTransform& WorldOffset, const FTransform& Transform, const FLinearColor& InColor, float Size, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawAxes(const FTransform& WorldOffset, const TArrayView<const FTransform>& Transforms, const FLinearColor& InColor, float Size, float Thickness = 0.f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawRectangle(const FTransform& WorldOffset, const FTransform& Transform, float Size, const FLinearColor& Color, float Thickness, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawArc(const FTransform& WorldOffset, const FTransform& Transform, float Radius, float MinimumAngle, float MaximumAngle, const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	// Circles are drawn in the XY plane
	UE_API void DrawCircle(const FTransform& WorldOffset, const FTransform& Transform, float Radius, const FLinearColor& Color, float Thickness, int32 Detail, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawCone(const FTransform& WorldOffset, const FTransform& ConeOffset, float Angle1, float Angle2, uint32 NumSides, bool bDrawSideLines, const FLinearColor& SideLineColor, FMaterialRenderProxy* const MaterialRenderProxy, float SideLineThickness = 1.0f, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawArrow(const FTransform& WorldOffset, const FVector& Direction, const FVector& Side, const FLinearColor& InColor, float Thickness, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawPlane(const FTransform& WorldOffset, const FVector2D& Scale, const FLinearColor& MeshColor, bool bDrawLines, const FLinearColor& LineColor, FMaterialRenderProxy* const MaterialRenderProxy, ESceneDepthPriorityGroup DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground, float Lifetime = -1.f);
	UE_API void DrawText(const FTransform& WorldOffset, const FVector& Position, const FString& Text, const FLinearColor& Color, float FontScale = 1.f, float Lifetime = -1.f);

	UE_API bool IsEnabled() const;
};

#undef UE_API
