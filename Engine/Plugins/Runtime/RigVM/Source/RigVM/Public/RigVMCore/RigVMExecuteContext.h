// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "RigVMDefines.h"
#include "RigVMModule.h"
#include "RigVMCore/RigVMDebugInfo.h"
#include "RigVMCore/RigVMProfilingInfo.h"
#include "RigVMCore/RigVMNameCache.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMCore/RigVMTraitScope.h"
#include "RigVMLog.h"
#include "RigVMDrawInterface.h"
#include "RigVMDrawContainer.h"
#include "RigVMMemoryStorage.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include "RigVMExecuteContext.generated.h"

#define UE_API RIGVM_API

class URigVM;
class URigVMHost;
struct FRigVMDispatchFactory;
struct FRigVMExecuteContext;
struct FRigVMExtendedExecuteContext;
struct FRigVMLogSettings;
struct FRigVMCallableInfo;
struct FRigVMFunction;

extern UE_API TAutoConsoleVariable<bool> CVarRigVMReportAllMessages;

enum class ERigVMSliceType : uint8
{
	Invalid,
	Loop,
	Callable
};

struct FRigVMSlice
{
public:

	FRigVMSlice()
    : SliceType(ERigVMSliceType::Invalid)
	, LowerBound(0)
	, UpperBound(0)
	, Index(INDEX_NONE)
	, Offset(0)
	, InstructionOrCallableIndex(INDEX_NONE)
	{
		Reset();
	}

	static FRigVMSlice MakeForLoop(int32 InCount)
	{
		FRigVMSlice Slice;
		Slice.SliceType = ERigVMSliceType::Loop;
		Slice.UpperBound = InCount - 1;
		Slice.Reset();
		return Slice;
	}

	static FRigVMSlice MakeForCallable()
	{
		FRigVMSlice Slice;
		Slice.SliceType = ERigVMSliceType::Callable;
		// for callables the upperbound of 0 represents a single slice
		Slice.Reset();
		return Slice;
	}

	bool IsValid() const
	{ 
		return Index != INDEX_NONE && SliceType != ERigVMSliceType::Invalid;
	}

	ERigVMSliceType GetType() const
	{
		return SliceType;
	}

	bool IsLoop() const
	{
		return SliceType == ERigVMSliceType::Loop;
	}

	bool IsCallable() const
	{
		return SliceType == ERigVMSliceType::Callable;
	}

	bool IsComplete() const
	{
		return Index > UpperBound;
	}

	int32 GetIndex() const
	{
		return Index + Offset;
	}

	int32 GetRelativeIndex() const
	{
		return Index - LowerBound;
	}

	void SetRelativeIndex(int32 InIndex)
	{
		Index = InIndex + LowerBound;
	}

	float GetRelativeRatio() const
	{
		return float(GetRelativeIndex()) / float(FMath::Max<int32>(1, Num() - 1));
	}

	int32 GetOffset() const
	{
		return Offset;
	}

	void SetOffset(int32 InSliceOffset)
	{
		Offset = InSliceOffset;
	}

	int32 GetInstructionOrCallableIndex() const
	{
		return InstructionOrCallableIndex;
	}

	void SetInstructionOrCallableIndex(int32 InInstructionOrCallableIndex)
	{
		InstructionOrCallableIndex = InInstructionOrCallableIndex;
	}

	int32 Num() const
	{
		return 1 + UpperBound - LowerBound;
	}

	int32 TotalNum() const
	{
		return UpperBound + 1 + Offset;
	}

	operator bool() const
	{
		return IsValid();
	}

	bool operator !() const
	{
		return !IsValid();
	}

	operator int32() const
	{
		return Index;
	}

	FRigVMSlice& operator++()
	{
		Index++;
		return *this;
	}

	FRigVMSlice operator++(int32)
	{
		FRigVMSlice TemporaryCopy = *this;
		++*this;
		return TemporaryCopy;
	}

	bool Next()
	{
		if (!IsValid())
		{
			return false;
		}

		if (IsComplete())
		{
			return false;
		}

		Index++;
		return true;
	}

	void Reset()
	{
		if (UpperBound >= LowerBound)
		{
			Index = LowerBound;
		}
		else
		{
			Index = INDEX_NONE;
		}
	}

	uint32 GetHash() const
	{
		return HashCombine(GetTypeHash(SliceType), GetTypeHash(LowerBound), GetTypeHash(Index));
	}

private:

	ERigVMSliceType SliceType;
	int32 LowerBound;
	int32 UpperBound;
	int32 Index;
	int32 Offset;
	int32 InstructionOrCallableIndex;
};

USTRUCT()
struct FRigVMRuntimeSettings
{
	GENERATED_BODY()

	/**
	 * The largest allowed size for arrays within the RigVM.
	 * Accessing or creating larger arrays will cause runtime errors in the rig.
	 */
	UPROPERTY(EditAnywhere, Category = "VM")
	int32 MaximumArraySize = 2048;

#if WITH_EDITORONLY_DATA
	// When enabled records the timing of each instruction / node
	// on each node and within the execution stack window.
	// Keep in mind when looking at nodes in a function the duration
	// represents the accumulated duration of all invocations
	// of the function currently running.
	// 
	// Note: This can only be used when in Debug Mode. Click the "Release" button
	// in the top toolbar to switch to Debug mode.
	UPROPERTY(EditAnywhere, Category = "VM")
	bool bEnableProfiling = false;
#endif

	/*
	 * The function to use for logging anything from the VM to the host
	 */
	using LogFunctionType = TFunction<void(const FRigVMLogSettings&,const FRigVMExecuteContext*,const FString&)>;
	TSharedPtr<LogFunctionType> LogFunction = nullptr;

	void SetLogFunction(LogFunctionType InLogFunction)
	{
		LogFunction = MakeShared<LogFunctionType>(InLogFunction);
	}

	/*
	 * Validate the settings
	 */
	void Validate()
	{
		MaximumArraySize = FMath::Clamp(MaximumArraySize, 1, INT32_MAX);
	}
};

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FRigVMExecuteContextFixLayout
{
	virtual ~FRigVMExecuteContextFixLayout() = default;
};

// This structure is used for semantically describing a pin
// on a RigVMStruct - but without any content. So rather than
// adding an actual FRigVMExecuteContext this can be added as
// the type for a pin and it won't use any memory.
// To customize which FRigVMExecuteContext structure to use
// on your node you can add the ExecuteContext=FMyExecuteContext
// meta data tag on the USTRUCT macro.
USTRUCT(BlueprintType)
struct FRigVMExecutePin
#if CPP
	: public FRigVMExecuteContextFixLayout
#endif
{
	GENERATED_BODY()
};

/**
 * The execute context is used for mutable nodes to
 * indicate execution order.
 */
USTRUCT(BlueprintType, meta=(DisplayName="Execute Context"))
struct FRigVMExecuteContext : public FRigVMExecutePin
{
	GENERATED_BODY()

	FRigVMExecuteContext()
		: InstructionIndex(0)
		, EventName(NAME_None)
		, FunctionName(NAME_None)
		, NumExecutions(0)
		, DeltaTime(0.0)
		, AbsoluteTime(0.0)
		, FramesPerSecond(1.0 / 60.0)
#if WITH_EDITOR
		, bHostBeingDebugged(false)
#endif
#if RIGVM_TRACE_ENABLED
		, bHostPlayingRewindDebugTrace(false)
#endif
		, RuntimeSettings()
		, NameCache(nullptr)
#if WITH_EDITOR
		, LogPtr(nullptr)
#endif
		, DrawInterfacePtr(nullptr)
		, DrawContainerPtr(nullptr)
		, ToWorldSpaceTransform(FTransform::Identity)
		, OwningObject(nullptr)
		, OwningActor(nullptr)
		, World(nullptr)
		, Traits()
	{
	}

	virtual ~FRigVMExecuteContext() = default;

	virtual void Log(const FRigVMLogSettings& InLogSettings, const FString& InMessage) const
	{
		if(RuntimeSettings.LogFunction.IsValid())
		{
			(*RuntimeSettings.LogFunction)(InLogSettings, this, InMessage);
		}
		else
		{
			if(InLogSettings.Severity == EMessageSeverity::Error)
			{
				UE_LOGF(LogRigVM, Error, "Instruction %d: %ls", InstructionIndex, *InMessage);
			}
			else if(InLogSettings.Severity == EMessageSeverity::Warning)
			{
				UE_LOGF(LogRigVM, Warning, "Instruction %d: %ls", InstructionIndex, *InMessage);
			}
			else
			{
				UE_LOGF(LogRigVM, Display, "Instruction %d: %ls", InstructionIndex, *InMessage);
			}
		}
	}

	template <typename... Types>
	void Logf(const FRigVMLogSettings& InLogSettings, ::UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		Log(InLogSettings, FString::Printf(Fmt, Args...));
	}

	int32 GetInstructionIndex() const { return InstructionIndex; }

	uint32 GetNumExecutions() const { return NumExecutions; }

	FName GetFunctionName() const { return FunctionName; }
	
	FName GetEventName() const { return EventName; }
	void SetEventName(const FName& InName) { EventName = InName; }

	template<typename T = double> requires std::is_floating_point_v<T>
	T GetDeltaTime() const { return static_cast<T>(DeltaTime); }
	void SetDeltaTime(double InDeltaTime) { DeltaTime = InDeltaTime; }

	template<typename T = double> requires std::is_floating_point_v<T>
	T GetAbsoluteTime() const { return static_cast<T>(AbsoluteTime); } 
	void SetAbsoluteTime(double InAbsoluteTime) { AbsoluteTime = InAbsoluteTime; }

	template<typename T = double> requires std::is_floating_point_v<T>
	T GetFramesPerSecond() const { return static_cast<T>(FramesPerSecond); } 
	void SetFramesPerSecond(double InFramesPerSecond) { FramesPerSecond = InFramesPerSecond; }

#if WITH_EDITOR
	bool IsHostBeingDebugged() const { return bHostBeingDebugged; }
	void SetHostBeingDebugged(bool InIsHostBeingDebugged) { bHostBeingDebugged = InIsHostBeingDebugged; }
#endif

#if RIGVM_TRACE_ENABLED
	bool IsHostPlayingRewindDebugTrace() const { return bHostPlayingRewindDebugTrace; }
	void SetHostPlayingRewindDebugTrace(bool InIsHostPlayingRewindDebugTrace) { bHostPlayingRewindDebugTrace = InIsHostPlayingRewindDebugTrace; }
#endif

	/** The current transform going from rig (global) space to world space */
	const FTransform& GetToWorldSpaceTransform() const { return ToWorldSpaceTransform; };

	/** The current object this VM is owned by */
	const UObject* GetOwningObject() const { return OwningObject; }

	/** The current component this VM is owned by */
	const USceneComponent* GetOwningComponent() const { return Cast<USceneComponent>(OwningObject); }

	/** The current component this VM is owned by */
	USceneComponent* GetMutableOwningComponent() const { return const_cast<USceneComponent*>(Cast<USceneComponent>(OwningObject)); }

	/** The current actor this VM is owned by */
	const AActor* GetOwningActor() const { return OwningActor; }

	/** The world this VM is running in */
	const UWorld* GetWorld() const { return World; }

	UE_API void SetOwningObject(const UObject* InOwningObject);

	UE_API void SetOwningComponent(const USceneComponent* InOwningComponent);

	UE_API void SetOwningActor(const AActor* InActor);

	UE_API void SetWorld(const UWorld* InWorld);

	void SetToWorldSpaceTransform(const FTransform& InToWorldSpaceTransform) { ToWorldSpaceTransform = InToWorldSpaceTransform; }

	/**
	 * Converts a transform from VM (global) space to world space
	 */
	FTransform ToWorldSpace(const FTransform& InTransform) const
	{
		return InTransform * ToWorldSpaceTransform;
	}

	/**
	 * Converts a transform from world space to VM (global) space
	 */
	FTransform ToVMSpace(const FTransform& InTransform) const
	{
		return InTransform.GetRelativeTransform(ToWorldSpaceTransform);
	}

	/**
	 * Converts a location from VM (global) space to world space
	 */
	FVector ToWorldSpace(const FVector& InLocation) const
	{
		return ToWorldSpaceTransform.TransformPosition(InLocation);
	}

	/**
	 * Converts a location from world space to VM (global) space
	 */
	FVector ToVMSpace(const FVector& InLocation) const
	{
		return ToWorldSpaceTransform.InverseTransformPosition(InLocation);
	}

	/**
	 * Converts a rotation from VM (global) space to world space
	 */
	FQuat ToWorldSpace(const FQuat& InRotation) const
	{
		return ToWorldSpaceTransform.TransformRotation(InRotation);
	}

	/**
	 * Converts a rotation from world space to VM (global) space
	 */
	FQuat ToVMSpace(const FQuat& InRotation) const
	{
		return ToWorldSpaceTransform.InverseTransformRotation(InRotation);
	}
	
	FRigVMNameCache* GetNameCache() const { return NameCache; }

#if WITH_EDITOR
	FRigVMLog* GetLog() const { return LogPtr; }
	void SetLog(FRigVMLog* InLog) { LogPtr = InLog; }
	virtual void Report(const FRigVMLogSettings& InLogSettings, const FName& InFunctionName, int32 InInstructionIndex, const FString& InMessage) const
	{
		if (FRigVMLog* Log = GetLog())
		{
			Log->Report(InLogSettings, InFunctionName, InInstructionIndex, InMessage);
		}
		else if(CVarRigVMReportAllMessages.GetValueOnAnyThread())
		{
			Logf(InLogSettings, TEXT("Instruction[%d] '%s': '%s'"), InstructionIndex, *InFunctionName.ToString(), *InMessage);
		}
	}
#endif

	FRigVMDrawInterface* GetDrawInterface() const { return DrawInterfacePtr; }
	void SetDrawInterface(FRigVMDrawInterface* InDrawInterface) { DrawInterfacePtr = InDrawInterface; }

	const FRigVMDrawContainer* GetDrawContainer() const { return DrawContainerPtr; }
	FRigVMDrawContainer* GetDrawContainer() { return DrawContainerPtr; }
	void SetDrawContainer(FRigVMDrawContainer* InDrawContainer) { DrawContainerPtr = InDrawContainer; }

	TArrayView<const FRigVMTraitScope> GetTraits() const
	{
		if(Traits.IsEmpty())
		{
			return TArrayView<const FRigVMTraitScope>();
		}
		return TArrayView<const FRigVMTraitScope>(Traits.GetData(), Traits.Num());
	}

	TArrayView<FRigVMTraitScope> GetTraits()
	{
		return TArrayView<FRigVMTraitScope>(Traits.GetData(), Traits.Num());
	}

	virtual void Initialize()
	{
		if(NameCache == nullptr)
		{
			static FRigVMNameCache StaticNameCache;
			NameCache = &StaticNameCache;
		}
	}

	virtual void Copy(const FRigVMExecuteContext* InOtherContext)
	{
		EventName = InOtherContext->EventName;
		FunctionName = InOtherContext->FunctionName;
		InstructionIndex = InOtherContext->InstructionIndex;
		NumExecutions = InOtherContext->NumExecutions;
		RuntimeSettings = InOtherContext->RuntimeSettings;
		NameCache = InOtherContext->NameCache;
	}

	/**
	 * Serialize this type from another
	 */
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/**
	 * Completes the execution of this run.  
	 */
	void MarkExecutionCompleted()
	{
		NumExecutions++;
	}

	/**
	 * Resets the number of executions
	 */
	void ResetNumExecutions()
	{
		NumExecutions = 0;
	}

	int32 InstructionIndex;
	
protected:

	virtual void Reset()
	{
		InstructionIndex = 0;
	}
	
	FName EventName;
	
	FName FunctionName;

	uint32 NumExecutions;

	double DeltaTime;

	double AbsoluteTime;

	double FramesPerSecond;

#if WITH_EDITOR
	bool bHostBeingDebugged;
#endif

#if RIGVM_TRACE_ENABLED
	bool bHostPlayingRewindDebugTrace;
#endif

	FRigVMRuntimeSettings RuntimeSettings;

	mutable FRigVMNameCache* NameCache;

#if WITH_EDITOR
	FRigVMLog* LogPtr;
#endif
	FRigVMDrawInterface* DrawInterfacePtr;
	FRigVMDrawContainer* DrawContainerPtr;

	/** The current transform going from rig (global) space to world space */
	FTransform ToWorldSpaceTransform;

	/** The current object (usually a component) this VM is owned by */
	const UObject* OwningObject;

	/** The current actor this VM is owned by */
	const AActor* OwningActor;

	/** The world this VM is running in */
	const UWorld* World;

	/** The traits accessible to the current instruction */
	TArray<FRigVMTraitScope> Traits;

	/** Additional memory handles for each trait */
	TArray<FRigVMMemoryHandle> AdditionalTraitMemoryHandles;

#if UE_RIGVM_DEBUG_EXECUTION
public:
	bool bDebugExecution = false;
	FString DebugMemoryString;
protected:
	TArray<FString> PreviousWorkMemory;

	UEnum* InstanceOpCodeEnum;
#endif

public:
	const FRigVMExtendedExecuteContext* GetExtendedExecuteContext() const { return ExtendedExecuteContext; }
	FRigVMExtendedExecuteContext* GetExtendedExecuteContext() { return ExtendedExecuteContext; }
	UE_API const FRigVMFunction* FindDispatchFunction(const FName& InDispatchFunctionName) const;

private:
	// Pointer back to owner execute context. Required for lazy branches Execute
	FRigVMExtendedExecuteContext* ExtendedExecuteContext = nullptr; 

	friend struct TRigVMLazyValueBase;
	friend struct FRigVMPredicateBranch;
	friend struct FRigVMExtendedExecuteContext;
	friend class URigVM;
	friend class URigVMNativized;
	friend struct FRigVMCallstackGuard;
};

template<>
struct TStructOpsTypeTraits<FRigVMExecuteContext> : public TStructOpsTypeTraitsBase2<FRigVMExecuteContext>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

struct FRigVMExternalVariableRuntimeData
{
	explicit FRigVMExternalVariableRuntimeData(uint8* InMemory)
		: Memory(InMemory)
	{
	}

	uint8* Memory = nullptr;
};

class FRigVMNativizedVMVariables
{
};

struct FRigVMExecuteCallstackEntry
{
	FRigVMExecuteCallstackEntry() = default;

	FRigVMExecuteCallstackEntry(int32 InInstructionIndex, const FRigVMCallableInfo* InCallable)
		: InstructionIndex(InInstructionIndex)
		, Callable(InCallable)
	{
	}
	
	int32 InstructionIndex = INDEX_NONE;
	const FRigVMCallableInfo* Callable = nullptr;
};

struct FRigVMExecuteCallstack
{
	FRigVMExecuteCallstack() = default;

	void Reset()
	{
		Entries.Reset();
	}

	bool IsValidIndex(int32 InIndex) const
	{
		return Entries.IsValidIndex(InIndex);
	}

	int32 Num() const
	{
		return Entries.Num();
	}

	bool IsEmpty() const
	{
		return Entries.IsEmpty();
	}

	const FRigVMExecuteCallstackEntry& Last() const
	{
		return Entries.Last();
	}

	const FRigVMExecuteCallstackEntry& operator[](int32 InIndex) const
	{
		return Entries[InIndex];
	}

	int32 Push(const FRigVMExecuteCallstackEntry& InEntry)
	{
		return Entries.Add(InEntry);
	}

	FRigVMExecuteCallstackEntry Pop()
	{
		return Entries.Pop();
	}

	TArray<FRigVMExecuteCallstackEntry> Entries;
};

/**
  * Slice addressing and lazy branch execution state for one callable invocation.
  * Loops push/pop slices here; SliceOffsetsPerInstruction and SliceOffsetsPerCallable
  * map each node and callsite to the correct offset into Work memory arrays.
  * Entering a nativized RigVMStruct bends FRigVMExtendedExecuteContext::Frame to point
  * at the Work-owned FRigVMExecuteCallFrame for the duration of StaticExecute.
  */
struct FRigVMExecuteCallFrame
{
	void Reset()
	{
		Slices.Reset();
		Slices.Add(FRigVMSlice::MakeForCallable());
		SliceOffsetsPerInstruction.Reset();
		SliceOffsetsPerCallable.Reset();
	}
	
	// The list of active slices. Slices can introduced by loops or callables
	TArray<FRigVMSlice> Slices;

	// The slice offset per instruction. Typically only loop / control flow nodes will
	// introduce a new loop slice - thus their offset will increase.
	TArray<uint16> SliceOffsetsPerInstruction;

	// The slice offset per callable. When multiple call sites invoke the same callable,
	// the offset for that callable index will be increased to make sure each call site
	// gets its own memory.
	TArray<uint16> SliceOffsetsPerCallable;

	TArray<FRigVMInstructionSetExecuteState> LazyBranchExecuteState;
};

/**
 * The execute context is used for mutable nodes to
 * indicate execution order.
 */
USTRUCT(Blueprintable)
struct FRigVMExtendedExecuteContext
{
public:

	GENERATED_BODY()

	FRigVMExtendedExecuteContext()
	: PublicDataScope(FRigVMExecuteContext::StaticStruct())
	{
		Reset();
		SetDefaultNameCache();
	}

	FRigVMExtendedExecuteContext(const UScriptStruct* InExecuteContextStruct)
		: PublicDataScope(InExecuteContextStruct)
	{
		if(InExecuteContextStruct)
		{
			check(InExecuteContextStruct->IsChildOf(FRigVMExecuteContext::StaticStruct()));
			Reset();
			SetDefaultNameCache();
		}
	}

	FRigVMExtendedExecuteContext(const FRigVMExtendedExecuteContext& InOther)
	{
		*this = InOther;
	}

	UE_API virtual ~FRigVMExtendedExecuteContext();

	// /** Full context reset */
	UE_API void Reset();

	/** Resets VM execution state */
	UE_API void ResetExecutionState();

	UE_API void CopyMemoryStorage(const FRigVMExtendedExecuteContext& Other);

	UE_API FRigVMExtendedExecuteContext& operator =(const FRigVMExtendedExecuteContext& Other);

	UE_API virtual void Initialize(const UScriptStruct* InScriptStruct);

	const UScriptStruct* GetContextPublicDataStruct() const
	{
		return Cast<UScriptStruct>(PublicDataScope.GetStruct());
	}

	void SetContextPublicDataStruct(const UScriptStruct* InScriptStruct)
	{
		if(GetContextPublicDataStruct() == InScriptStruct)
		{
			return;
		}
		Initialize(InScriptStruct);
	}

	template<typename ExecuteContextType = FRigVMExecuteContext>
	const ExecuteContextType& GetPublicData() const
	{
		check(PublicDataScope.GetStruct()->IsChildOf(ExecuteContextType::StaticStruct()));
		return *(const ExecuteContextType*)PublicDataScope.GetStructMemory();
	}

	template<typename ExecuteContextType = FRigVMExecuteContext>
	ExecuteContextType& GetPublicData()
	{
		const UStruct* PublicDataStruct = PublicDataScope.GetStruct();
		check(PublicDataStruct == ExecuteContextType::StaticStruct() || PublicDataStruct->IsChildOf(ExecuteContextType::StaticStruct()));
		return *(ExecuteContextType*)PublicDataScope.GetStructMemory();
	}

	template<typename ExecuteContextType = FRigVMExecuteContext>
	ExecuteContextType& GetPublicDataSafe()
	{
		if(!PublicDataScope.GetStruct()->IsChildOf(ExecuteContextType::StaticStruct()))
		{
			Initialize(ExecuteContextType::StaticStruct());
		}
		return *(ExecuteContextType*)PublicDataScope.GetStructMemory();
	}

	const FRigVMSlice& GetSlice() const
	{
		return Frame->Slices.Last();
	}

	FRigVMSlice& GetSlice()
	{
		return Frame->Slices.Last();
	}

	void BeginLoopSlice(int32 InCount, int32 InRelativeIndex = 0)
	{
		ensure(!IsSliceComplete());
		Frame->Slices.Add(FRigVMSlice::MakeForLoop(InCount));
		Frame->Slices.Last().SetRelativeIndex(InRelativeIndex);
		Frame->Slices.Last().SetInstructionOrCallableIndex(GetPublicData<>().InstructionIndex);
		Frame->Slices.Last().SetOffset(Frame->SliceOffsetsPerInstruction[GetPublicData<>().InstructionIndex]);
	}

	void EndLoopSlice()
	{
		ensure(Frame->Slices.Num() > 1);
		ensure(Frame->Slices.Last().IsLoop());

		// if this slice has reached its upperbound
		// we want to make sure to increment the offset
		// for the instruction the slice originated from,
		// so that next time around the slice indices are not
		// reused.
		const FRigVMSlice PoppedSlice = Frame->Slices.Pop();
		if(PoppedSlice.GetRelativeIndex() == PoppedSlice.Num() - 1)
		{
			if(Frame->SliceOffsetsPerInstruction.IsValidIndex(PoppedSlice.GetInstructionOrCallableIndex()))
			{
				Frame->SliceOffsetsPerInstruction[PoppedSlice.GetInstructionOrCallableIndex()] += static_cast<uint16>(PoppedSlice.Num());
			}
		}
	}

	void BeginCallableSlice(int32 InCallableIndex)
	{
		ensure(!IsSliceComplete());
		Frame->Slices.Add(FRigVMSlice::MakeForCallable());
		Frame->Slices.Last().SetInstructionOrCallableIndex(InCallableIndex);
		Frame->Slices.Last().SetOffset(Frame->SliceOffsetsPerCallable[InCallableIndex]);
	}

	void EndCallableSlice()
	{
		ensure(Frame->Slices.Num() > 0);
		ensure(Frame->Slices.Last().IsCallable());

		const FRigVMSlice PoppedSlice = Frame->Slices.Pop();
		if(Frame->SliceOffsetsPerCallable.IsValidIndex(PoppedSlice.GetInstructionOrCallableIndex()))
		{
			Frame->SliceOffsetsPerCallable[PoppedSlice.GetInstructionOrCallableIndex()]++;
		}
	}

public:
	bool IsSliceComplete() const
	{
		return GetSlice().IsComplete();
	}

	uint32 GetSliceHash() const
	{
		uint32 Hash = 0;
		for(const FRigVMSlice& Slice : Frame->Slices)
		{
			Hash = HashCombine(Hash, Slice.GetHash());
		}
		return Hash;
	}

	const FRigVMExecuteCallstack& GetCallstack() const
	{
		return Callstack;
	}

	uint32 GetCallstackHash() const
	{
		return CallstackHash;
	}

	uint32 GetHashForLazyBranch() const
	{
		return HashCombine(GetTypeHash(GetNumExecutions()), GetCallstackHash());
	}

	UE_API bool StepForward();
	UE_API bool StepInto();
	UE_API bool StepOut();

	bool IsValidArrayIndex(int32& InOutIndex, const FScriptArrayHelper& InArrayHelper) const
	{
		return IsValidArrayIndex(InOutIndex, InArrayHelper.Num());
	}

	bool IsValidArrayIndex(int32& InOutIndex, int32 InArraySize) const
	{
		const int32 InOriginalIndex = InOutIndex;

		// we support wrapping the index around similar to python
		if(InOutIndex < 0)
		{
			InOutIndex = InArraySize + InOutIndex;
		}

		if(InOutIndex < 0 || InOutIndex >= InArraySize)
		{
			const int32 InstructionIndex = GetInstructionIndex();
			const FString ObjectPath = GetVMPathName();
			GetPublicData<>().Logf(EMessageSeverity::Error, TEXT("Array Index (%d) out of bounds (count %d).\n('%s', instruction %d)"), InOriginalIndex, InArraySize, *ObjectPath, InstructionIndex);
			return false;
		}
		return true;
	}

	bool IsValidArraySize(int32 InSize) const
	{
		if(InSize < 0 || InSize > GetPublicData<>().RuntimeSettings.MaximumArraySize)
		{
			const int32 InstructionIndex = GetInstructionIndex();
			const FString ObjectPath = GetVMPathName();
			GetPublicData<>().Logf(EMessageSeverity::Error,
														 TEXT("Array Size (%d) larger than allowed maximum (%d).\nCheck VMRuntimeSettings in class settings.\n('%s', Instruction %d)."),
														 InSize,
														 GetPublicData<>().RuntimeSettings.MaximumArraySize,
														 *ObjectPath,
														 InstructionIndex);
			return false;
		}
		return true;
	}

	void SetRuntimeSettings(FRigVMRuntimeSettings InRuntimeSettings)
	{
		GetPublicData<>().RuntimeSettings = InRuntimeSettings;
		check(GetPublicData<>().RuntimeSettings.MaximumArraySize > 0);
	}

	void SetDefaultNameCache()
	{
		SetNameCache(&NameCache);
	}

	void SetNameCache(FRigVMNameCache* InNameCache)
	{
		GetPublicData<>().NameCache = InNameCache;
	}

	void InvalidateCachedMemory()
	{
		Frame->LazyBranchExecuteState.Reset();
	}
	
	void UpdateInstanceMemory(FRigVMMemoryStorageStruct* InLiteralMemory)
	{
		if(CurrentVMMemory.IsEmpty())
		{
			CurrentVMMemory.SetNumUninitialized(3);
		}
		CurrentVMMemory[0] = &WorkMemoryStorage;
		CurrentVMMemory[1] = InLiteralMemory;
		CurrentVMMemory[2] = &DebugMemoryStorage;
	}

	uint32 GetNumExecutions() const
	{
		if(PublicDataScope.IsValid())
		{
			return GetPublicData<>().GetNumExecutions();
		}
		return 0;
	}

	void ResetNumExecutions()
	{
		if(PublicDataScope.IsValid())
		{
			GetPublicData<>().ResetNumExecutions();
		}
	}

	UE_API int32 GetInstructionIndex() const;
	UE_API const URigVM* GetVM() const;
	UE_API URigVM* GetVM();
	UE_API FString GetVMPathName() const;
	UE_API void SetVM(URigVM* InVM);

	UE_API const FRigVMFunction* FindDispatchFunction(const FName& InDispatchFunctionName) const;

	uint32 VMHash = 0;

	FRigVMMemoryStorageStruct WorkMemoryStorage;

	FRigVMMemoryStorageStruct DebugMemoryStorage;

#if WITH_EDITORONLY_DATA
	// Deprecated 5.4
	UPROPERTY(transient, meta = (DeprecatedProperty, DeprecationMessage = "Please, use WorkMemoryStorage"))
	TObjectPtr<URigVMMemoryStorage> WorkMemoryStorageObject_DEPRECATED;
#endif

#if WITH_EDITORONLY_DATA
	// Deprecated 5.4
	UPROPERTY(transient, meta = (DeprecatedProperty, DeprecationMessage = "Please, use DebugMemoryStorage"))
	TObjectPtr<URigVMMemoryStorage> DebugMemoryStorageObject_DEPRECATED;
#endif

	FStructOnScope PublicDataScope;
	URigVMHost* Host = nullptr;
private:
	URigVM* VM = nullptr;

public:

	FRigVMExecuteCallFrame FrameStorage;
	FRigVMExecuteCallFrame* Frame = nullptr;

	FRigVMExecuteCallstack Callstack;
	uint32 CallstackHash = 0;

	const FRigVMDispatchFactory* Factory = nullptr;
	mutable TMap<FName, const FRigVMFunction*> DispatchFunctionMap;
	FRigVMNameCache NameCache;
	
	int32 ExecutingThreadId = INDEX_NONE;

	TArray<int32> EntriesBeingExecuted;

	ERigVMExecuteResult CurrentExecuteResult = ERigVMExecuteResult::Failed;
	FName CurrentEntryName = NAME_None;
	bool bCurrentlyRunningRootEntry = false;

	TArray<FRigVMMemoryStorageStruct*, TInlineAllocator<3>> CurrentVMMemory;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "CachedMemoryHandles has been deprecated.")
	TArray<FRigVMMemoryHandle> CachedMemoryHandles;
#endif

	/** Callback for external objects to be notified when the VM reaches an Exit Operation */
	TFunction<void(const FName& EventName)> OnExecutionReachedExitCallback;

	TArray<FRigVMExternalVariableRuntimeData> ExternalVariableRuntimeData;
	TSharedPtr<FRigVMNativizedVMVariables> NativizedVMVariables;

	UE_API const FRigVMOperand* GetDebugOperandForOperand(const FRigVMOperand& InOperand) const;
	UE_API void MarkOperandForDebugging(const FRigVMOperand& InOperand, bool bEnableDebugging = true);
	UE_API void UnmarkOperandForDebugging(const FRigVMOperand& InOperand);
	UE_API void MarkAllOperandsForDebugging(bool bEnableDebugging = true);
	UE_API void UnmarkAllOperandsForDebugging();

	/** An optional operand to debug register map only for this context's memory */
	TMap<FRigVMOperand, FRigVMOperand> OperandToDebugRegister;

	/** A set indicating which operands should be debugged */
	TSet<FRigVMOperand> DebuggedOperands;

#if WITH_EDITOR

	UE_DEPRECATED(5.8, "DebugInfo pointer has been deprecated, please use DebugInfoWeak.")
	FRigVMDebugInfo* DebugInfo = nullptr;

	TWeakPtr<FRigVMDebugInfo> DebugInfoWeak;
	FRigVMDebugInfo* DebugInfoStrong = nullptr;

	FRigVMInstructionVisitInfo* InstructionVisitInfo = nullptr;

	void SetInstructionVisitInfo(FRigVMInstructionVisitInfo* InInstructionVisitInfo)
	{
		InstructionVisitInfo = InInstructionVisitInfo;
	}

	FRigVMInstructionVisitInfo* GetRigVMInstructionVisitInfo() const
	{
		return InstructionVisitInfo;
	}

	void SetDebugInfo(const TSharedPtr<FRigVMDebugInfo>& InSharedDebugInfo, bool bForce)
	{
		if (!bForce)
		{
			if (DebugInfoWeak.IsValid())
			{
				if (const TSharedPtr<FRigVMDebugInfo>& DebugInfoShared = DebugInfoWeak.Pin())
				{
					// don't change actively debugged instances.
					if (DebugInfoShared->IsActive())
					{
						return;
					}
				}
			}
		}
		DebugInfoWeak = InSharedDebugInfo.ToWeakPtr();
	}

	UE_DEPRECATED(5.8, "SetDebugInfo with DebugInfo pointer has been deprecated, please use SharedDebugInfo version.")
	void SetDebugInfo(FRigVMDebugInfo* InDebugInfo) {}

	UE_DEPRECATED(5.8, "GetRigVMDebugInfo has been deprecated, please use GetRigVMDebugInfoWeak instead.")
	FRigVMDebugInfo* GetRigVMDebugInfo() const { return nullptr; }
	
	const TWeakPtr<FRigVMDebugInfo>& GetRigVMDebugInfoWeak() const
	{
		return DebugInfoWeak;
	}

	FRigVMProfilingInfo* ProfilingInfo = nullptr;

	void SetProfilingInfo(FRigVMProfilingInfo* InProfilingInfo)
	{
		ProfilingInfo = InProfilingInfo;
	}

	FRigVMProfilingInfo* GetRigVMProfilingInfo() const
	{
		return ProfilingInfo;
	}

	TArray<TTuple<int32, int32>> InstructionBrackets;

#endif // WITH_EDITOR

	friend class URigVMNativized;
};

#undef UE_API
