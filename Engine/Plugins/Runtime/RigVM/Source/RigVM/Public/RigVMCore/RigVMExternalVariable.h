// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMTraits.h"
#include "RigVMModule.h"
#include "RigVMPropertyUtils.h"
#include "RigVMTypeIndex.h"
#include "RigVMTypeUtils.h"
#include "UObject/UnrealType.h"
#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#endif
#include "RigVMExternalVariable.generated.h"

/**
 * The external variable can be used to map external / unowned
 * memory into the VM and back out
 */
USTRUCT(BlueprintType)
struct FRigVMExternalVariableDef
{
	GENERATED_BODY()

public:
	
	FRigVMExternalVariableDef() = default;

	bool operator == (const FRigVMExternalVariableDef& Other) const
	{
		return Guid == Other.Guid
			&& Name == Other.Name
			&& Property == Other.Property
			&& BaseCPPType == Other.BaseCPPType
			&& CPPTypeObject == Other.CPPTypeObject
			&& bIsArray == Other.bIsArray
			&& bIsPublic == Other.bIsPublic
			&& bIsReadOnly == Other.bIsReadOnly;
	}

	bool IsValid() const
	{
		return Name.IsValid() &&
			!Name.IsNone() &&
			BaseCPPType.IsValid() &&
			!BaseCPPType.IsNone();
	}
	
	const FGuid& GetGuid() const
	{
		return Guid;
	}
	
	// void SetGuid(const FGuid& InGuid)
	// {
	// 	Guid = InGuid;
	// }

	const FName& GetName() const
	{
		return Name;
	}

	void SetName(const FName& InName)
	{
		Name = InName;
	}
	
	const FProperty* GetProperty() const
	{
		return Property;
	};

	void SetProperty(const FProperty* InProperty)
	{
		Property = InProperty;
	}
	
	const FName& GetBaseCPPType() const
	{
		return BaseCPPType;
	}

	const FName& GetExtendedCPPType() const
	{
		if(bIsArray)
		{
			if (!ExtendedCPPType.IsSet())
			{
				ExtendedCPPType = *RigVMTypeUtils::ArrayTypeFromBaseType(BaseCPPType.ToString());
			}
			return ExtendedCPPType.GetValue();
		}
		return BaseCPPType;
	}

	bool IsSameType(const FRigVMExternalVariableDef& InOther) const
	{
		return GetBaseCPPType() == InOther.GetBaseCPPType() &&
			IsArray() == InOther.IsArray() &&
			GetCPPTypeObject() == InOther.GetCPPTypeObject();
	}

	const UObject* GetCPPTypeObject() const
	{
		return CPPTypeObject;
	}
	
	UObject* GetCPPTypeObject()
	{
		return CPPTypeObject;
	}
	
	bool IsArray() const
	{
		return bIsArray;
	}
	
	bool IsPublic() const
	{
		return bIsPublic;
	}
	
	void SetIsPublic(bool InIsPublic)
	{
		bIsPublic = InIsPublic;
	}
	
	bool IsReadOnly() const
	{
		return bIsReadOnly;
	}
	
	void SetIsReadOnly(bool InIsReadOnly)
	{
		bIsReadOnly = InIsReadOnly;
	}
	
	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMExternalVariableDef& Variable);

protected:

	RIGVM_API void CheckCPPTypeIntegrity() const;
	
	FGuid Guid = FGuid();
	FName Name = NAME_None;
	const FProperty* Property =  nullptr;
	FName BaseCPPType = NAME_None;
	mutable TOptional<FName> ExtendedCPPType;
	UObject* CPPTypeObject = nullptr;
	bool bIsArray = false;
	bool bIsPublic = false;
	bool bIsReadOnly = false;

	friend struct FRigVMExternalVariableModifier; 
};

USTRUCT()
struct FRigVMExternalVariable : public FRigVMExternalVariableDef
{
	GENERATED_BODY()
public:
	
	FRigVMExternalVariable()
		: FRigVMExternalVariableDef()
	{
	}

	FRigVMExternalVariable(const FRigVMExternalVariableDef& Other)
		: FRigVMExternalVariableDef(Other)
		, Memory(nullptr)
	{
		CheckCPPTypeIntegrity();
	}

	FRigVMExternalVariable(const FRigVMExternalVariableDef& Other, uint8* InMemory)
		: FRigVMExternalVariableDef(Other)
		, Memory(InMemory)
	{
		CheckCPPTypeIntegrity();
	}

	FRigVMExternalVariable(const FRigVMExternalVariable& Other)
	: FRigVMExternalVariableDef(Other)
	, Memory(Other.Memory)
	{
	}

	uint32 GetTypeHash() const
	{
		return RigVMPropertyUtils::GetPropertyHashStable(Property, Memory);
	}

	const uint8* GetMemory() const
	{
		return Memory;
	}

	uint8* GetMemory()
	{
		return Memory;
	}

	void SetMemory(uint8* InMemory)
	{
		Memory = InMemory;
	}

	friend FArchive& operator<<(FArchive& Ar, FRigVMExternalVariable& Variable)
	{
		FRigVMExternalVariableDef& RigVMExternalVariableDef = static_cast<FRigVMExternalVariableDef&>(Variable);
		Ar << RigVMExternalVariableDef;

		return Ar;
	}

	bool operator == (const FRigVMExternalVariableDef& Other) const
	{
		return ((FRigVMExternalVariableDef)*this) == ((FRigVMExternalVariableDef)Other);  
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FProperty* InProperty, void* InContainer, const FName& InOptionalName = NAME_None)
	{
		check(InProperty);

		const FProperty* Property = InProperty;

		FRigVMExternalVariable ExternalVariable;
		ExternalVariable.Guid = InGuid;
		ExternalVariable.Name = InOptionalName.IsNone() ? InProperty->GetFName() : InOptionalName;
		ExternalVariable.Property = Property;
		ExternalVariable.bIsPublic = !InProperty->HasAllPropertyFlags(CPF_DisableEditOnInstance);
		ExternalVariable.bIsReadOnly = InProperty->HasAllPropertyFlags(CPF_BlueprintReadOnly);

		if (InContainer)
		{
			ExternalVariable.Memory = (uint8*)Property->ContainerPtrToValuePtr<uint8>(InContainer);
		}

		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			ExternalVariable.bIsArray = true;
			Property = ArrayProperty->Inner;
		}

		RigVMPropertyUtils::GetTypeFromProperty(Property, ExternalVariable.BaseCPPType, ExternalVariable.CPPTypeObject);

		ExternalVariable.CheckCPPTypeIntegrity();
		return ExternalVariable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, bool& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("bool");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<bool>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("bool");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, int32& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("int32");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<int32>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("int32");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, uint8& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("uint8");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<uint8>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("uint8");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, float& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("float");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<float>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("float");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, double& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("double");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<double>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("double");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, FString& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("FString");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<FString>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("FString");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, FName& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("FName");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<FName>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = TEXT("FName");
		Variable.CPPTypeObject = nullptr;
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, T& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = StaticEnum<T>()->GetFName();
		Variable.CPPTypeObject = StaticEnum<T>();
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<T>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = StaticEnum<T>()->GetFName();
		Variable.CPPTypeObject = StaticEnum<T>();
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, T& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *RigVMTypeUtils::GetUniqueStructTypeName(TBaseStructure<T>::Get());
		Variable.CPPTypeObject = TBaseStructure<T>::Get();
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<T>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *RigVMTypeUtils::GetUniqueStructTypeName(TBaseStructure<T>::Get());
		Variable.CPPTypeObject = TBaseStructure<T>::Get();
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, T& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *RigVMTypeUtils::GetUniqueStructTypeName(T::StaticStruct());
		Variable.CPPTypeObject = T::StaticStruct();
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<T>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *RigVMTypeUtils::GetUniqueStructTypeName(T::StaticStruct());
		Variable.CPPTypeObject = T::StaticStruct();
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, T& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *T::StaticClass()->GetStructCPPName();
		Variable.CPPTypeObject = T::StaticClass();
		Variable.bIsArray = false;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, TArray<T>& InValue)
	{
		FRigVMExternalVariable Variable;
		Variable.Guid = InGuid;
		Variable.Name = InName;
		Variable.BaseCPPType = *T::StaticClass()->GetStructCPPName();
		Variable.CPPTypeObject = T::StaticClass();
		Variable.bIsArray = true;
		Variable.Memory = (uint8*)&InValue;
		Variable.CheckCPPTypeIntegrity();
		return Variable;
	}

	template<typename T>
	T GetValue() const
	{
		ensure(IsValid() && !bIsArray);
		return *(T*)Memory;
	}

	template<typename T>
	T& GetRef()
	{
		ensure(IsValid() && !bIsArray);
		return *(T*)Memory;
	}

	template<typename T>
	const T& GetRef() const
	{
		ensure(IsValid() && !bIsArray);
		return *(T*)Memory;
	}

	template<typename T>
	void SetValue(const T& InValue)
	{
		ensure(IsValid() && !bIsArray);
		(*(T*)Memory) = InValue;
	}

	template<typename T>
	TArray<T> GetArray()
	{
		ensure(IsValid() && bIsArray);
		return *(TArray<T>*)Memory;
	}

	template<typename T>
	void SetArray(const TArray<T>& InValue)
	{
		ensure(IsValid() && bIsArray);
		(*(TArray<T>*)Memory) = InValue;
	}

	bool IsValid(bool bAllowNullPtr = false) const
	{
		return Super::IsValid()
			&& (bAllowNullPtr || Memory != nullptr);
	}

	RIGVM_API TRigVMTypeIndex GetTypeIndex() const;

	FString GetCPPTypeObjectPathName() const
	{
		if (CPPTypeObject)
		{
			return CPPTypeObject->GetPathName();
		}
		return FString();
	}

	template<typename VarType>
	static void MergeExternalVariable(TArray<VarType>& OutVariables, const VarType& InVariable)
	{
		if(!InVariable.IsValid(true))
		{
			return;
		}

		for(const VarType& ExistingVariable : OutVariables)
		{
			if(ExistingVariable.Name == InVariable.Name)
			{
				return;
			}
		}

		OutVariables.Add(InVariable);
	}

#if WITH_EDITOR

	RIGVM_API static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, const FEdGraphPinType& InPinType, bool bInPublic = false, bool bInReadonly = false);

#endif
	
	RIGVM_API static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, const FString& InCPPTypePath, bool bInPublic = false, bool bInReadonly = false);

	RIGVM_API static FRigVMExternalVariable Make(const FGuid InGuid, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, bool bInPublic = false, bool bInReadonly = false);

private:
	
	uint8* Memory = nullptr;

	friend struct FRigVMExternalVariableModifier; 
};

namespace RigVMTypeUtils
{
#if WITH_EDITOR
 	RIGVM_API FRigVMExternalVariable ExternalVariableFromBPVariableDescription(const FBPVariableDescription& InVariableDescription);

	RIGVM_API FRigVMExternalVariable ExternalVariableFromBPVariableDescription(const FBPVariableDescription& InVariableDescription, const UStruct* VariablesStruct, void* Container);

	RIGVM_API FRigVMExternalVariable ExternalVariableFromPinType(const FGuid InGuid, const FName& InName, const FEdGraphPinType& InPinType, bool bInPublic = false, bool bInReadonly = false);

	RIGVM_API FRigVMExternalVariable ExternalVariableFromCPPTypePath(const FGuid InGuid, const FName& InName, const FString& InCPPTypePath, bool bInPublic = false, bool bInReadonly = false);

	RIGVM_API FRigVMExternalVariable ExternalVariableFromCPPType(const FGuid InGuid, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, bool bInPublic = false, bool bInReadonly = false);
#endif // WITH_EDITOR

	RIGVM_API TArray<FRigVMExternalVariableDef> GetExternalVariableDefs(const TArray<FRigVMExternalVariable>& ExternalVariables);
}
