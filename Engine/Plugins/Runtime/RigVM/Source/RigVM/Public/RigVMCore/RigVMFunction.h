// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Blueprint/BlueprintSupport.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "RigVMRegistryHandle.h"
#include "RigVMExecuteContext.h"
#include "RigVMMemory.h"
#include "RigVMTypeIndex.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "RigVMFunction.generated.h"

class UScriptStruct;
struct FRigVMExtendedExecuteContext;
struct FRigVMMemoryHandle;
struct FRigVMPredicateBranch;
struct FRigVMRegistry_NoLock;

typedef TArrayView<FRigVMMemoryHandle> FRigVMMemoryHandleArray;
typedef TArrayView<FRigVMPredicateBranch> FRigVMPredicateBranchArray;

typedef void (*FRigVMFunctionPtr)(FRigVMExtendedExecuteContext& RigVMExecuteContext, FRigVMMemoryHandleArray RigVMMemoryHandles, FRigVMPredicateBranchArray RigVMBranches);

struct FRigVMDispatchFactory;
struct FRigVMTemplate;

/**
 * The Pin Direction is used to differentiate different kinds of 
 * pins in the data flow graph - inputs, outputs etc.
 */
UENUM(BlueprintType)
enum class ERigVMPinDirection : uint8
{
	Input, // A const input value
	Output, // A mutable output value
	IO, // A mutable input and output value
	Visible, // A const value that cannot be connected to
	Hidden, // A mutable hidden value (used for interal state)
	Invalid // The max value for this enum - used for guarding.
};

/**
 * The Function Argument is used to differentiate different kinds of 
 * pins in the data flow graph - inputs or outputs 
 */
UENUM(BlueprintType)
enum class ERigVMFunctionArgumentDirection : uint8
{
	Input, // A const input value
	Output, // A mutable output value
	Invalid // The max value for this enum - used for guarding.
};

// Version of FRigVMFunctionArgument for UHT code generation
struct FRigVMCompiledInFunctionArgument
{
	const TCHAR* Name = nullptr;
	const TCHAR* Type = nullptr;
	ERigVMFunctionArgumentDirection Direction = ERigVMFunctionArgumentDirection::Input;
};

struct FRigVMCompiledInFunction
{
	const TCHAR* MethodName = nullptr;
	FRigVMFunctionPtr Function = nullptr;
	TConstArrayView<FRigVMCompiledInFunctionArgument> Parameters;
};

/**
 * The FRigVMFunctionArgument describes an argument necessary for the C++ invocation of the RIGVM_METHOD backed function
 */
struct FRigVMFunctionArgument : public FRigVMCompiledInFunctionArgument
{
	TSharedPtr<FString> NameString;
	TSharedPtr<FString> TypeString;

	FRigVMFunctionArgument()
		: FRigVMCompiledInFunctionArgument{ .Direction = ERigVMFunctionArgumentDirection::Invalid }
	{
	}

	FRigVMFunctionArgument(FRigVMCompiledInFunctionArgument InBase)
		: FRigVMCompiledInFunctionArgument(InBase)
	{
	}

	FRigVMFunctionArgument(const TCHAR* InName, const TCHAR* InType, const ERigVMFunctionArgumentDirection InDirection = ERigVMFunctionArgumentDirection::Input)
		: FRigVMCompiledInFunctionArgument{ .Name = InName, .Type = InType, .Direction = InDirection }
	{
	}

	FRigVMFunctionArgument(const FString& InName, const FString& InType, const ERigVMFunctionArgumentDirection InDirection = ERigVMFunctionArgumentDirection::Input)
		: FRigVMCompiledInFunctionArgument{ .Direction = InDirection }
	{
		NameString = MakeShareable(new FString(InName));
		TypeString = MakeShareable(new FString(InType));
		Name = NameString->operator*();
		Type = TypeString->operator*();
	}

	void Serialize_NoLock(FArchive& Ar);

	RIGVM_API bool operator ==(const FRigVMFunctionArgument& InOther) const;
};

typedef TArray<FRigVMFunctionArgument> FRigVMFunctionArgumentArray;

/**
 * The FRigVMExecuteArgument describes an execute argument.
 */
struct FRigVMExecuteArgument
{
	FName Name;
	ERigVMPinDirection Direction;
	TRigVMTypeIndex TypeIndex;

	FRigVMExecuteArgument()
		: Name(NAME_None)
		, Direction(ERigVMPinDirection::Invalid)
		, TypeIndex(INDEX_NONE)
	{
	}

	FRigVMExecuteArgument(const FName& InName, const ERigVMPinDirection& InDirection, const TRigVMTypeIndex& InTypeIndex = INDEX_NONE)
		: Name(InName)
		, Direction(InDirection)
		, TypeIndex(InTypeIndex)
	{
	}
};

/**
 * The FRigVMFunction is used to represent a function pointer generated by UHT
 * for a given name. The name might be something like "FMyStruct::MyVirtualMethod"
 */
struct FRigVMFunction
{
	FString Name;
	UScriptStruct* Struct;
	FRigVMDispatchFactory* Factory;
	FRigVMRegistry_NoLock* OwnerRegistry;
	FRigVMFunctionPtr FunctionPtr;
	int32 Index;
	int32 TemplateIndex;
	int32 PermutationIndex;
	TArray<FRigVMFunctionArgument> Arguments;
	mutable TArray<TRigVMTypeIndex> ArgumentTypeIndices;
	mutable TArray<FName> ArgumentNames;

	FRigVMFunction()
		: Name()
		, Struct(nullptr)
		, Factory(nullptr)
		, OwnerRegistry(nullptr)
		, FunctionPtr(nullptr)
		, Index(INDEX_NONE)
		, TemplateIndex(INDEX_NONE)
		, PermutationIndex(INDEX_NONE)
		, Arguments()
		, ArgumentTypeIndices()
		, ArgumentNames()
	{
	}

	RIGVM_API FRigVMFunction(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct = nullptr, int32 InIndex = INDEX_NONE, const TArray<FRigVMFunctionArgument>& InArguments = TArray<FRigVMFunctionArgument>());

	FRigVMFunction(const FString& InName, FRigVMFunctionPtr InFunctionPtr, FRigVMDispatchFactory* InFactory, int32 InIndex = INDEX_NONE, const TArray<FRigVMFunctionArgument>& InArguments = TArray<FRigVMFunctionArgument>())
	: Name(InName)
	, Struct(nullptr)
	, Factory(InFactory)
	, OwnerRegistry(nullptr)
	, FunctionPtr(InFunctionPtr)
	, Index(InIndex)
	, TemplateIndex(INDEX_NONE)
	, PermutationIndex(INDEX_NONE)
	, Arguments(InArguments)
	, ArgumentNames()
	{
	}

	bool IsValid() const { return !Name.IsEmpty() && FunctionPtr != nullptr; }
	RIGVM_API FString GetName() const;
	RIGVM_API FName GetMethodName() const;
	RIGVM_API FString GetModuleName() const;
	RIGVM_API FString GetModuleRelativeHeaderPath() const;
	const TArray<FRigVMFunctionArgument>& GetArguments() const { return Arguments; }
	RIGVM_API int32 GetArgumentIndex(const FName& InArgumentName) const;
	RIGVM_API const TArray<TRigVMTypeIndex>& GetArgumentTypeIndices() const;
	RIGVM_API const TArray<TRigVMTypeIndex>& GetArgumentTypeIndices_NoLock(FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API void GetArgumentTypeIndices_NoLock(FRigVMRegistryHandle& InRegistry, TArray<TRigVMTypeIndex>& OutTypeIndices) const;
	RIGVM_API const FRigVMTemplate* GetTemplate() const;
	RIGVM_API const FRigVMTemplate* GetTemplate_NoLock(const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API const UScriptStruct* GetExecuteContextStruct() const;
	RIGVM_API const UScriptStruct* GetExecuteContextStruct_NoLock(const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API bool SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const;
	RIGVM_API bool SupportsExecuteContextStruct_NoLock(const UScriptStruct* InExecuteContextStruct, const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API const FName& GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API void Serialize_NoLock(FArchive& Ar, FRigVMRegistryHandle& InRegistry, const UObject* Owner = nullptr);

#if WITH_EDITOR
	RIGVM_API bool IsFixedSizeArray(const FName& InArgumentName) const;
	RIGVM_API bool IsLazyValue(const FName& InArgumentName) const;
#endif
};
