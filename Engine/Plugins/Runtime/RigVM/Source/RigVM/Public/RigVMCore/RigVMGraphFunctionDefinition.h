// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMObjectVersion.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMByteCode.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "RigVMVariant.h"
#include "RigVMNodeLayout.h"
#include "RigVMStringUtils.h"
#include "RigVMGraphFunctionIdentifier.h"
#include "RigVMGraphFunctionDefinition.generated.h"

#define UE_API RIGVM_API

class IRigVMGraphFunctionHost;
struct FRigVMGraphFunctionData;

extern RIGVM_API TAutoConsoleVariable<bool> CVarRigVMCompileFunctionsToCallables;
extern RIGVM_API TAutoConsoleVariable<bool> CVarRigVMAllowCallableFallbackToInlining;

USTRUCT()
struct FRigVMFunctionCompilationPropertyDescription
{
	GENERATED_BODY()

	// The name of the property to create
	UPROPERTY()
	FName Name;

	// The complete CPP type to base a new property off of (for ex: 'TArray<TArray<FVector>>')
	UPROPERTY()
	FString CPPType;

	// The tail CPP Type object, for example the UScriptStruct for a struct
	UPROPERTY()
	TSoftObjectPtr<UObject> CPPTypeObject;

	// The default value to use for this property (for example: '(((X=1.000000, Y=2.000000, Z=3.000000)))')
	UPROPERTY()
	FString DefaultValue;

	friend uint32 GetTypeHash(const FRigVMFunctionCompilationPropertyDescription& Description) 
	{
		uint32 Hash = GetTypeHash(Description.Name.ToString());
		Hash = HashCombine(Hash, GetTypeHash(Description.CPPType));
		// we can't hash based on the pointer since that's not deterministic across sessions
		// Hash = HashCombine(Hash, GetTypeHash(Description.CPPTypeObject));
		Hash = HashCombine(Hash, GetTypeHash(Description.DefaultValue));
		return Hash;
	}

	friend FArchive& operator<<(FArchive& Ar, FRigVMFunctionCompilationPropertyDescription& Data)
	{
		Ar << Data.Name;
		Ar << Data.CPPType;
		Ar << Data.CPPTypeObject;
		Ar << Data.DefaultValue;
		return Ar;
	}

	UE_API FRigVMPropertyDescription ToPropertyDescription() const;
	static UE_API TArray<FRigVMPropertyDescription> ToPropertyDescription(const TArray<FRigVMFunctionCompilationPropertyDescription>& InDescriptions);
};

USTRUCT()
struct FRigVMFunctionCompilationPropertyPath
{
	GENERATED_BODY()

	UPROPERTY()
	int32 PropertyIndex = INDEX_NONE;

	UPROPERTY()
	FString HeadCPPType;

	UPROPERTY()
	FString SegmentPath;

	friend uint32 GetTypeHash(const FRigVMFunctionCompilationPropertyPath& Path)
	{
		uint32 Hash = GetTypeHash(Path.PropertyIndex);
		Hash = HashCombine(Hash, GetTypeHash(Path.HeadCPPType));
		Hash = HashCombine(Hash, GetTypeHash(Path.SegmentPath));
		return Hash;
	}

	friend FArchive& operator<<(FArchive& Ar, FRigVMFunctionCompilationPropertyPath& Data)
	{
		Ar << Data.PropertyIndex;
		Ar << Data.HeadCPPType;
		Ar << Data.SegmentPath;
		return Ar;
	}
};

USTRUCT(BlueprintType)
struct FRigVMFunctionCompilationData
{
	GENERATED_BODY()

	FRigVMFunctionCompilationData()
	: Hash(0)
	, bEncounteredSurpressedErrors(false)
	{}

	FRigVMFunctionCompilationData(const FRigVMFunctionCompilationData& InOther)
	{
		*this = InOther;
	}

	UPROPERTY()
	FRigVMByteCode ByteCode;

	UPROPERTY()
	TArray<FName> FunctionNames;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyDescription> WorkPropertyDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyPath> WorkPropertyPathDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyDescription> LiteralPropertyDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyPath> LiteralPropertyPathDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyDescription> DebugPropertyDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyPath> DebugPropertyPathDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyDescription> ExternalPropertyDescriptions;

	UPROPERTY()
	TArray<FRigVMFunctionCompilationPropertyPath> ExternalPropertyPathDescriptions;

	UPROPERTY()
	TMap<int32, FName> ExternalRegisterIndexToVariable;

	UPROPERTY()
	TMap<FString, FRigVMOperand> Operands;

	UPROPERTY()
	TMap<FName, FRigVMOperand> InterfaceOperands;

	UPROPERTY()
	uint32 Hash;

	UPROPERTY(Transient)
	bool bEncounteredSurpressedErrors;

	bool IsValid() const
	{
		return Hash != 0;
	}

	bool RequiresRecompilation() const
	{
		return bEncounteredSurpressedErrors;
	}

	RIGVM_API void Reset();

	RIGVM_API FRigVMFunctionCompilationData& operator =(const FRigVMFunctionCompilationData& InOther);

	/**
	* If this function can be turned into a callable.
	* Exceptions include:
	*    Array parameters passed in.
	*    External variables passed in.
	*    Any use of traits.
	*/
	RIGVM_API bool SupportsCallable() const;

	RIGVM_API const TArray<FRigVMFunctionCompilationPropertyDescription>& GetPropertyDescriptions(ERigVMMemoryType InMemoryType) const;
	RIGVM_API const TArray<FRigVMFunctionCompilationPropertyPath>& GetPropertyPathDescriptions(ERigVMMemoryType InMemoryType) const;
	
	friend uint32 GetTypeHash(const FRigVMFunctionCompilationData& Data) 
	{
		uint32 DataHash = Data.ByteCode.GetByteCodeHash();
		for (const FName& Name : Data.FunctionNames)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Name.ToString()));
		}

		for (const FRigVMFunctionCompilationPropertyDescription& Description : Data.WorkPropertyDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Description));
		}
		for (const FRigVMFunctionCompilationPropertyPath& Path : Data.WorkPropertyPathDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Path));
		}

		for (const FRigVMFunctionCompilationPropertyDescription& Description : Data.LiteralPropertyDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Description));
		}
		for (const FRigVMFunctionCompilationPropertyPath& Path : Data.LiteralPropertyPathDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Path));
		}

		for (const FRigVMFunctionCompilationPropertyDescription& Description : Data.DebugPropertyDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Description));
		}
		for (const FRigVMFunctionCompilationPropertyPath& Path : Data.DebugPropertyPathDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Path));
		}

		for (const FRigVMFunctionCompilationPropertyDescription& Description : Data.ExternalPropertyDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Description));
		}
		for (const FRigVMFunctionCompilationPropertyPath& Path : Data.ExternalPropertyPathDescriptions)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Path));
		}
		
		for (const TPair<int32,FName>& Pair : Data.ExternalRegisterIndexToVariable)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Pair.Key));
			DataHash = HashCombine(DataHash, GetTypeHash(Pair.Value.ToString()));
		}

		for (const TPair<FString, FRigVMOperand>& Pair : Data.Operands)
		{
			DataHash = HashCombine(DataHash, GetTypeHash(Pair.Key));
			DataHash = HashCombine(DataHash, GetTypeHash(Pair.Value));
		}

		{
			TArray<TPair<FString, FRigVMOperand>> SortedInterfaceOperands;
			SortedInterfaceOperands.Reserve(Data.InterfaceOperands.Num());
			for (const TPair<FName, FRigVMOperand>& Pair : Data.InterfaceOperands)
			{
				SortedInterfaceOperands.Emplace(Pair.Key.ToString(), Pair.Value);
			}
			SortedInterfaceOperands.Sort([](const TPair<FString, FRigVMOperand>& A, const TPair<FString, FRigVMOperand>& B)
			{
				return A.Key < B.Key;
			});
			for (const TPair<FString, FRigVMOperand>& Pair : SortedInterfaceOperands)
			{
				DataHash = HashCombine(DataHash, GetTypeHash(Pair.Key));
				DataHash = HashCombine(DataHash, GetTypeHash(Pair.Value));
			}
		}

		return DataHash;
	}

	mutable TOptional<bool> bSupportsCallable;
	mutable FCriticalSection TaskMutex;
	
	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMFunctionCompilationData& Data);
};

USTRUCT(BlueprintType)
struct FRigVMGraphFunctionArgument
{
	GENERATED_BODY()
	
	FRigVMGraphFunctionArgument()
	: Name(NAME_None)
	, DisplayName(NAME_None)
	, CPPType(NAME_None)
	, CPPTypeObject(nullptr)
	, bIsArray(false)
	, Direction(ERigVMPinDirection::Input)
	, bIsConst(false)
	, bIsInputVariable(false)
	{}

	UE_API FRigVMGraphFunctionArgument(const FRigVMExternalVariable& InExternalVariable);

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	FName Name;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	FName DisplayName;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	FName CPPType;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	TSoftObjectPtr<UObject> CPPTypeObject;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	bool bIsArray;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	ERigVMPinDirection Direction;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	FString DefaultValue;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	bool bIsConst;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	bool bIsInputVariable;

	UPROPERTY()
	TMap<FString, FText> PathToTooltip;

	// True if CPP Type or obj type is valid. May potentially load the CPP Type Object
	UE_API bool IsValid() const;

	// validates and potentially loads the CPP Type Object
	UE_API bool IsCPPTypeObjectValid() const;

	// returns true if this argument is an execute context
	UE_API bool IsExecuteContext() const;

	friend uint32 GetTypeHash(const FRigVMGraphFunctionArgument& Argument)
	{
		uint32 Hash = HashCombine(GetTypeHash(Argument.Name.ToString()), GetTypeHash(Argument.DisplayName.ToString()));
		Hash = HashCombine(Hash, GetTypeHash(Argument.CPPType.ToString()));
		Hash = HashCombine(Hash, GetTypeHash(Argument.CPPTypeObject));
		Hash = HashCombine(Hash, GetTypeHash(Argument.bIsArray));
		Hash = HashCombine(Hash, GetTypeHash(Argument.Direction));
		Hash = HashCombine(Hash, GetTypeHash(Argument.DefaultValue));
		Hash = HashCombine(Hash, GetTypeHash(Argument.bIsConst));
		Hash = HashCombine(Hash, GetTypeHash(Argument.bIsInputVariable));
		for (const TPair<FString, FText>& Pair : Argument.PathToTooltip)
		{
			Hash = HashCombine(Hash, GetTypeHash(Pair.Key));
			Hash = HashCombine(Hash, GetTypeHash(Pair.Value.ToString()));
		}
		return Hash;
	}

	bool operator==(const FRigVMGraphFunctionArgument& Other) const
	{
		return true;
	}

	UE_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FRigVMGraphFunctionArgument& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}

	UE_API FRigVMExternalVariable ToExternalVariable() const;
};

USTRUCT(BlueprintType)
struct FRigVMGraphFunctionHeader
{
	GENERATED_BODY()

	FRigVMGraphFunctionHeader()
		: LibraryPointer(nullptr, FString())
		, Name(NAME_None)
	{}

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FRigVMGraphFunctionIdentifier LibraryPointer;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionIdentifier)
	FRigVMVariant Variant;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FName Name;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FString NodeTitle;
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FLinearColor NodeColor = FLinearColor::White;

	UPROPERTY(meta=(DeprecatedProperty))
	FText Tooltip_DEPRECATED;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FString Description;
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FString Category;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	FString Keywords;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionHeader)
	TArray<FRigVMGraphFunctionArgument> Arguments;

	UPROPERTY()
	TMap<FRigVMGraphFunctionIdentifier, uint32> Dependencies;

	UPROPERTY()
	TArray<FRigVMExternalVariable> ExternalVariables;

	UPROPERTY()
	FRigVMNodeLayout Layout;

	UE_API bool IsMutable() const;

	bool IsValid() const { return LibraryPointer.IsValid(); }

	FString GetHash() const
	{
		return RigVMStringUtils::JoinStrings(LibraryPointer.HostObject.ToString(), Name.ToString(), TEXT(":"));
	}

	friend uint32 GetTypeHash(const FRigVMGraphFunctionHeader& Header)
	{
		return GetTypeHash(Header.LibraryPointer);
	}

	bool operator==(const FRigVMGraphFunctionHeader& Other) const
	{
		return LibraryPointer == Other.LibraryPointer;
	}

	UE_DEPRECATED(5.8, "Use GetFunctionHostObject instead")
	UE_API IRigVMGraphFunctionHost* GetFunctionHost(bool bLoadIfNecessary = true) const;
	UE_API TScriptInterface<IRigVMGraphFunctionHost> GetFunctionHostObject(bool bLoadIfNecessary = true) const;

	UE_API FRigVMGraphFunctionData* GetFunctionData(bool bLoadIfNecessary = true) const;

	FText GetTooltip() const
	{
		FString TooltipStr = FString::Printf(TEXT("%s (%s)\n%s"),
		*Name.ToString(),
		*LibraryPointer.GetNodeSoftPath().GetAssetPathString(),
		*Description);
		return FText::FromString(TooltipStr);
	}

	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMGraphFunctionHeader& Data);

	static UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeader(const FSoftObjectPath& InFunctionObjectPath, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);

	static UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeader(const FSoftObjectPath& InHostObjectPath, const FName& InFunctionName, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);

	static UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeader(const FRigVMGraphFunctionIdentifier& InIdentifier, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);

	static UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeaderFromHash(const FString& InFunctionHash, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);

	RIGVM_API TArray<FRigVMExternalVariable> GetExternalVariables() const;

protected:

	static UE_API FName GetFunctionNameFromObjectPath(const FString& InObjectPath, const FName& InOptionalFunctionName = NAME_None);
	
	static UE_API TFunction<FRigVMGraphFunctionHeader(const FSoftObjectPath&, const FName&, bool*)> FindFunctionHeaderFromPathFunc;

	friend class URigVMBuildData;
	friend struct FRigVMGraphFunctionData;
};

USTRUCT()
struct FRigVMGraphFunctionHeaderArray
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FRigVMGraphFunctionHeader> Headers;
};

USTRUCT(BlueprintType)
struct FRigVMGraphFunctionData
{
	GENERATED_BODY()

	FRigVMGraphFunctionData(){}

	FRigVMGraphFunctionData(const FRigVMGraphFunctionHeader& InHeader)
		: Header(InHeader) {	}

	UPROPERTY()
	FRigVMGraphFunctionHeader Header;

	UPROPERTY()
	FRigVMFunctionCompilationData CompilationData;

	UPROPERTY()
	FString SerializedCollapsedNode_DEPRECATED;

	UPROPERTY()
	FRigVMObjectArchive CollapseNodeArchive;

	RIGVM_API bool IsMutable() const;

	RIGVM_API bool IsPublic() const;

	bool operator==(const FRigVMGraphFunctionData& Other) const
	{
		return Header == Other.Header;
	}

	void ClearCompilationData() { CompilationData.Reset(); }

	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMGraphFunctionData& Data);

	static RIGVM_API FRigVMGraphFunctionData* FindFunctionData(const FSoftObjectPath& InHostObjectPath, const FName& InFunctionName, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);	

	static RIGVM_API FRigVMGraphFunctionData* FindFunctionData(const FRigVMGraphFunctionIdentifier& InIdentifier, bool* bOutIsPublic = nullptr, FString* OutErrorMessage = nullptr);	

	static RIGVM_API FString GetArgumentNameFromPinHash(const FString& InPinHash);
	
	RIGVM_API FRigVMOperand GetOperandForArgument(const FName& InArgumentName) const;

	RIGVM_API bool IsAnyOperandSharedAcrossArguments() const;

	RIGVM_API bool PatchSharedArgumentOperandsIfRequired();
	RIGVM_API bool PatchExternalVariablesToArguments();

	static const inline TCHAR* EntryString = TEXT("Entry");
	static const inline TCHAR* ReturnString = TEXT("Return");
	static RIGVM_API TFunction<IRigVMGraphFunctionHost*(UObject*)> GetFunctionHostFromObjectFunc;

	friend class URigVMBuildData;
};

USTRUCT()
struct FRigVMReferenceNodeData
{
	GENERATED_BODY();

public:
	
	FRigVMReferenceNodeData()
		:ReferenceNodePath(), ReferencedFunctionIdentifier()
	{}

	UPROPERTY()
	FString ReferenceNodePath;

	UPROPERTY(meta=(DeprecatedProperty))
	FString ReferencedFunctionPath_DEPRECATED;
	
	UPROPERTY(meta=(DeprecatedProperty))
	FRigVMGraphFunctionHeader ReferencedHeader_DEPRECATED;

	UPROPERTY()
	FRigVMGraphFunctionIdentifier ReferencedFunctionIdentifier;

};

#undef UE_API
