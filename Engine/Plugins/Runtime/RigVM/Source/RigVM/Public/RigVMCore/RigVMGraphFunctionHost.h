// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMGraphFunctionHost.generated.h"

#define UE_API RIGVM_API

struct FRigVMGraphFunctionStore;

UINTERFACE(MinimalAPI)
class URigVMGraphFunctionHost : public UInterface
{
	GENERATED_BODY()
};

// Interface that allows an object to host a rig VM graph function Host.
class IRigVMGraphFunctionHost
{
	GENERATED_BODY()

public:
	
	// IRigVMClientHost interface
	virtual FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() = 0;
	virtual const FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() const = 0;
};

// A management struct containing graph functions
USTRUCT()
struct FRigVMGraphFunctionStore
{
public:

	GENERATED_BODY()

	/** Exposed public functions on this rig */
	UPROPERTY()
	TArray<FRigVMGraphFunctionData> PublicFunctions;

	UPROPERTY(Transient)
	TArray<FRigVMGraphFunctionData> PrivateFunctions;

	UE_API const FRigVMGraphFunctionData* FindFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic = nullptr) const;

	UE_API FRigVMGraphFunctionData* FindFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic = nullptr);

	UE_API FRigVMGraphFunctionData* FindFunctionByName(const FName& Name, bool *bOutIsPublic = nullptr);

	UE_API bool ContainsFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer) const;

	UE_API bool IsFunctionPublic(const FRigVMGraphFunctionIdentifier& InLibraryPointer) const;

	UE_API FRigVMGraphFunctionData* AddFunction(const FRigVMGraphFunctionHeader& FunctionHeader, bool bIsPublic);

	UE_API bool RemoveFunction(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bIsPublic = nullptr);

	UE_API bool MarkFunctionAsPublic(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool bIsPublic);

	UE_API FRigVMGraphFunctionData* UpdateFunctionInterface(const FRigVMGraphFunctionHeader& Header);

	UE_API bool UpdateDependencies(const FRigVMGraphFunctionIdentifier& InLibraryPointer, TMap<FRigVMGraphFunctionIdentifier, uint32>& Dependencies);

	UE_API bool UpdateExternalVariables(const FRigVMGraphFunctionIdentifier& InLibraryPointer, TArray<FRigVMExternalVariable> InExternalVariables);

	UE_API bool UpdateFunctionCompilationData(const FRigVMGraphFunctionIdentifier& InLibraryPointer, const FRigVMFunctionCompilationData& CompilationData);

	UE_API bool RemoveFunctionCompilationData(const FRigVMGraphFunctionIdentifier& InLibraryPointer);

	UE_API bool RemoveAllCompilationData();

	friend FArchive& operator<<(FArchive& Ar, FRigVMGraphFunctionStore& Host)
	{
		UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, TEXT("FRigVMGraphFunctionStore"));

		Ar << Host.PublicFunctions;
		UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("PublicFunctions"));

		// This is only added to make sure SoftObjectPaths can be gathered and fixed up in the case of asset rename
		// It should not affect data on disk
		if (Ar.IsObjectReferenceCollector())
		{
			Ar << Host.PrivateFunctions;
			UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("PrivateFunctions"));
		}

		return Ar;
	}

	void PostLoad()
	{
		for(FRigVMGraphFunctionData& Function: PublicFunctions)
		{
			Function.PatchSharedArgumentOperandsIfRequired();
			Function.PatchExternalVariablesToArguments();
		}
	}

private:

	UE_API const FRigVMGraphFunctionData* FindFunctionImpl(const FRigVMGraphFunctionIdentifier& InLibraryPointer, bool* bOutIsPublic = nullptr) const;
};

#undef UE_API
