// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMObjectVersion.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "RigVMVariant.h"
#include "RigVMGraphFunctionIdentifier.generated.h"

#define UE_API RIGVM_API

USTRUCT(BlueprintType)
struct FRigVMGraphFunctionIdentifier
{
	GENERATED_BODY()

	UPROPERTY(meta=(DeprecatedProperty))
	FSoftObjectPath LibraryNode_DEPRECATED;

private:
	UPROPERTY(VisibleAnywhere, Category=FunctionIdentifier)
	mutable FString LibraryNodePath;

public:

	// A path to the IRigVMGraphFunctionHost that stores the function information, and compilation data (e.g. RigVMBlueprintGeneratedClass)
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionIdentifier)
	FSoftObjectPath HostObject;

	FRigVMGraphFunctionIdentifier()
		: LibraryNodePath(FString()), HostObject(nullptr) {}
	
	FRigVMGraphFunctionIdentifier(FSoftObjectPath InHostObject, FString InLibraryNodePath)
		: LibraryNodePath(InLibraryNodePath), HostObject(InHostObject) {}

	friend uint32 GetTypeHash(const FRigVMGraphFunctionIdentifier& Pointer)
	{
		return HashCombine(GetTypeHash(Pointer.GetLibraryNodePath()), GetTypeHash(Pointer.HostObject.ToString()));
	}

	bool operator==(const FRigVMGraphFunctionIdentifier& Other) const
	{
		return HostObject == Other.HostObject && GetNodeSoftPath().GetSubPathString() == Other.GetNodeSoftPath().GetSubPathString();
	}

	bool IsValid() const
	{
		return !HostObject.IsNull() && (!GetLibraryNodePath().IsEmpty());
	}

	FString GetFunctionName() const
	{
		if(IsValid())
		{
			const FString Path = GetLibraryNodePath();
			FString NodeName;
			if(Path.Split(TEXT("."), nullptr, &NodeName, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
			{
				return NodeName;
			}
		}
		return FString();
	}
	
	FName GetFunctionFName() const
	{
		if(!IsValid())
		{
			return NAME_None;
		}
		return *GetFunctionName();
	}

	FString& GetLibraryNodePath() const
	{
		if (LibraryNodePath.IsEmpty() && LibraryNode_DEPRECATED.IsValid())
		{
			LibraryNodePath = LibraryNode_DEPRECATED.ToString();
		}
		return LibraryNodePath;
	}

	void SetLibraryNodePath(const FString& InPath)
	{
		LibraryNodePath = InPath;
	}

	FSoftObjectPath GetNodeSoftPath() const
	{
		return GetLibraryNodePath();
	}
	
#if WITH_EDITOR
	UE_API FAssetData GetAssetData() const;
#endif

	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMGraphFunctionIdentifier& Data);

	UE_API bool IsVariant() const;
	UE_API TArray<FRigVMVariantRef> GetVariants(bool bIncludeSelf = false) const;
	UE_API TArray<FRigVMGraphFunctionIdentifier> GetVariantIdentifiers(bool bIncludeSelf = false) const;
	UE_API bool IsVariantOf(const FRigVMGraphFunctionIdentifier& InOther) const;

protected:
	
	static UE_API TFunction<TArray<FRigVMVariantRef>(const FGuid& InGuid)> GetVariantRefsByGuidFunc;

	friend class URigVMBuildData;
};

#undef UE_API
