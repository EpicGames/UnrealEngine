// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"

#define UE_API RIGVM_API

struct FRigVMLogSettings
{
	FRigVMLogSettings(EMessageSeverity::Type InSeverity, bool InLogOnce = true)
		: Severity(InSeverity)
		, bLogOnce(InLogOnce)
	{}

	EMessageSeverity::Type Severity;
	bool bLogOnce;
};

struct FRigVMLog
{
public:

	FRigVMLog() {}
	virtual ~FRigVMLog() = default;

#if WITH_EDITOR
	struct FLogEntry
	{
		FLogEntry(EMessageSeverity::Type InSeverity, const FName& InFunctionName, int32 InInstructionIndex, const FString& InMessage)
		: Severity(InSeverity)
		, FunctionName(InFunctionName)
		, InstructionIndex(InInstructionIndex)
		, Message(InMessage)
		{}
		
		EMessageSeverity::Type Severity;
		FName FunctionName;
		int32 InstructionIndex;
		FString Message;
	};
	TArray<FLogEntry> Entries;
	TMap<FString, bool> KnownMessages;

	UE_API TArray<FLogEntry> GetEntries(EMessageSeverity::Type InSeverity = EMessageSeverity::Info, bool bIncludeHigherSeverity = true);
#endif

	UE_API virtual void Reset();
	UE_API virtual void Report(const FRigVMLogSettings& InLogSettings, const FName& InFunctionName, int32 InInstructionIndex, const FString& InMessage);

#if WITH_EDITOR
	UE_API void RemoveRedundantEntries();
#endif
};

#undef UE_API
