// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "RigVMDefines.h"
#include "RigVMTraits.h"
#include "Templates/TypeHash.h"
#include "UObject/ObjectMacros.h"
#include "Misc/OutputDevice.h"
#include "GenericPlatform/GenericPlatformMemory.h"

#include "RigVMMemoryCommon.generated.h"

class FArchive;

#ifdef UE_BUILD_DEBUG
	#define DEBUG_RIGVMMEMORY 0
	//#define DEBUG_RIGVMMEMORY WITH_EDITOR
#else
	#define DEBUG_RIGVMMEMORY 0
#endif

#if DEBUG_RIGVMMEMORY
	RIGVM_API DECLARE_LOG_CATEGORY_EXTERN(LogRigVMMemory, Log, All);
	#define UE_LOG_RIGVMMEMORY(Format, ...) UE_LOG(LogRigVMMemory, Display, (Format), ##__VA_ARGS__)
#else
	#define UE_LOG_RIGVMMEMORY(Format, ...)
#endif

// The ERigVMMemoryType maps to memory container index in RigVM through
// FRigVMOperand::GetContainerIndex() or URigVM::GetContainerIndex(...) 
/**
 * The type of memory used. Typically we differentiate between
 * Work (Mutable) and Literal (Constant) memory.
 */
UENUM()
enum class ERigVMMemoryType: uint8
{
	Work = 0, // Mutable state
	Literal = 1, // Const / fixed state
	External = 2, // Unowned external memory
	Debug = 3, // Owned memory used for debug watches
	Invalid,
	Num = Invalid
};

namespace RigVM
{
	template<typename T = uint8>
	void ZeroPaddedMemory(void* InFirstMember, const void* InSecondMember)
	{
		char* FirstMember = static_cast<char*>(InFirstMember);
		const char* SecondMember = static_cast<const char*>(InSecondMember);
		FirstMember += sizeof(T);
		if(FirstMember < InSecondMember)
		{
			FPlatformMemory::Memzero(FirstMember, SecondMember - FirstMember);
		}
	}
}

// A runtime cache for determining if a set of instruction has to
// run for this execution of the VM
USTRUCT(BlueprintType)
struct FRigVMInstructionSetExecuteState
{
	GENERATED_BODY()

	FRigVMInstructionSetExecuteState()
	{
	}

	RIGVM_API bool RequiresExecute(const uint32& InSliceHash, const uint32& InHashForLazyBranch);

	UPROPERTY(transient)
	TMap<uint32, uint32> SliceHashToCallStackHash;
};

/**
 * The FRigVMOperand represents an argument used for an operator
 * within the virtual machine. Operands provide information about
 * which memory needs to be referred to, which register within the
 * memory all the way to the actual byte address in memory.
 * The FRigVMOperand is a light weight address for a register in
 * a FRigVMMemoryContainer.
 * For external variables the register index represents the
 * index of the external variable within the running VM.
 */
USTRUCT()
struct FRigVMOperand
{
	GENERATED_BODY()

public:

	FRigVMOperand()
		: MemoryType(ERigVMMemoryType::Work)
		, RegisterIndex(UINT16_MAX)
		, RegisterOffset(UINT16_MAX)
	{
	}

	FRigVMOperand(ERigVMMemoryType InMemoryType, int32 InRegisterIndex, int32 InRegisterOffset = INDEX_NONE)
		: MemoryType(InMemoryType)
		, RegisterIndex(InRegisterIndex < 0 ? UINT16_MAX : (uint16)InRegisterIndex)
		, RegisterOffset(InRegisterOffset < 0 ? UINT16_MAX : (uint16)InRegisterOffset)
	{
	}

	bool operator == (const FRigVMOperand& InOther) const
	{
		return MemoryType == InOther.MemoryType &&
			RegisterIndex == InOther.RegisterIndex &&
			RegisterOffset == InOther.RegisterOffset;
	}

	bool operator != (const FRigVMOperand& InOther) const
	{
		return !(*this == InOther);
	}

	// returns the memory type of this argument
	bool IsValid() const { return RegisterIndex != UINT16_MAX; }

	// returns the memory type of this argument
	ERigVMMemoryType GetMemoryType() const { return MemoryType; }

	// returns the index of the container of this argument
	// this function should be kept in sync with URigVM::GetContainerIndex()
	int32 GetContainerIndex() const
	{
		if(MemoryType == ERigVMMemoryType::External)
		{
			return (int32)ERigVMMemoryType::Work;
		}
		
		if(MemoryType == ERigVMMemoryType::Debug)
		{
			return 2;
		}
		return (int32)MemoryType;
	}

	// returns the index of the register of this argument
	int32 GetRegisterIndex() const { return RegisterIndex == UINT16_MAX ? INDEX_NONE : (int32)RegisterIndex; }

	// returns the register offset of this argument
	int32 GetRegisterOffset() const { return RegisterOffset == UINT16_MAX ? INDEX_NONE : (int32)RegisterOffset; }

	friend uint32 GetTypeHash(const FRigVMOperand& Operand)
	{
  		return HashCombine(HashCombine((uint32)Operand.MemoryType, (uint32)Operand.RegisterIndex), (uint32)Operand.RegisterOffset);
  	}
	
	RIGVM_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMOperand& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static void ZeroPaddedMemoryIfNeeded(FRigVMOperand* InOperand)
	{
		RigVM::ZeroPaddedMemory(&InOperand->MemoryType, &InOperand->RegisterIndex);
	}

	RIGVM_API FString ToString() const;

private:

	UPROPERTY()
	ERigVMMemoryType MemoryType;

	/**
	 * The index of the register inside of the specific type of memory (work, literal etc).
	 * For external variables the register index represents the index of the external variable within the running VM.
	 */
	UPROPERTY()
	uint16 RegisterIndex;
	
	UPROPERTY()
	uint16 RegisterOffset;

	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
};

typedef TArrayView<const FRigVMOperand> FRigVMOperandArray;


/**
 * Helper class to catch default value import errors on properties
 */
class FRigVMMemoryStorageImportErrorContext : public FOutputDevice
{
public:

	bool bLogErrors;
	int32 NumErrors;

	explicit FRigVMMemoryStorageImportErrorContext(bool InLogErrors = true)
		: FOutputDevice()
		, bLogErrors(InLogErrors)
		, NumErrors(0)
	{
	}

	RIGVM_API void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;
};
