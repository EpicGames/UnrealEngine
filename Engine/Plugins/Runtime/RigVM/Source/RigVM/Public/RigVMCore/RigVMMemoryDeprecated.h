// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "RigVMMemoryCommon.h"
#include "RigVMStatistics.h"
#include "RigVMTraits.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"

#include "RigVMMemoryDeprecated.generated.h"

#define UE_API RIGVM_API

class FArchive;
class UObject;
class UScriptStruct;

typedef TArray<uint8> FRigVMByteArray;
typedef TArray<FRigVMByteArray> FRigVMNestedByteArray;

/**
 * The type of register within the memory.
 */
UENUM()
enum class ERigVMRegisterType : uint8
{
	Plain, // bool, int32, float, FVector etc. (also structs that do NOT require a constructor / destructor to be valid)
	String, // FString
	Name, // FName
	Struct, // Any USTRUCT (structs which require a constructor / destructor to be valid. structs which have indirection (arrays or strings in them, for example).
	Invalid
};

// The register represents an address with the VM's memory. Within a register
// we can store arbitrary data, so it provides a series of properties to
// describe the memory location.
// Registers also support the notion of slices. A slice is a complete copy of
// the memory - so for example if your register stores 4 Vectors, then a slice 
// would contain 48 bytes (4 * 3 * 4). The register can however store multiple
// slices / copies of that if needed. Slices can be used to provide 
// per-invocation memory to functions within the same register.
// An integrator for example that needs to store a simulated position
// might want access to a separate memory per loop iteration.
USTRUCT()
struct FRigVMRegister
{
	GENERATED_BODY()

	FRigVMRegister()
		: Type(ERigVMRegisterType::Invalid)
		, ByteIndex(INDEX_NONE)
		, ElementSize(0)
		, ElementCount(0)
		, SliceCount(1)
		, AlignmentBytes(0)
		, TrailingBytes(0)
		, Name(NAME_None)
		, ScriptStructIndex(INDEX_NONE)
		, bIsArray(false)
		, bIsDynamic(false)
#if WITH_EDITORONLY_DATA
		, BaseCPPType(NAME_None)
		, BaseCPPTypeObject(nullptr)
#endif
	{
	}

	// The type of register (plain, name, string, etc.)
	UPROPERTY()
	ERigVMRegisterType Type;

	// The index of the first work byte
	UPROPERTY()
	uint32 ByteIndex;

	// The size of each store element (for example 4 for a float)
	UPROPERTY()
	uint16 ElementSize;

	// The number of elements in this register (for example the number of elements in an array)
	UPROPERTY()
	uint16 ElementCount;

	// The number of slices (complete copies) (for example the number of iterations on a fixed loop)
	// Potentially redundant state - can be removed.
	UPROPERTY()
	uint16 SliceCount;

	// The number of leading bytes for alignment.
	// Bytes that had to be introduced before the register
	// to align the registers memory as per platform specification.
	UPROPERTY()
	uint8 AlignmentBytes;

	// The number of trailing bytes.
	// These originate after shrinking a register.
	// Potentially unused state - can be removed.
	UPROPERTY()
	uint16 TrailingBytes;

	// The name of the register (can be None)
	UPROPERTY()
	FName Name;

	// For struct registers this is the index of the
	// struct used - otherwise INDEX_NONE
	UPROPERTY()
	int32 ScriptStructIndex;

	// If true defines this register as an array
	UPROPERTY()
	bool bIsArray;

	// If true defines this register to use dynamic storage.
	// Is this an array or singleton value with multiple slices
	// which potentially requires changing count at runtime. 
	UPROPERTY()
	bool bIsDynamic;

#if WITH_EDITORONLY_DATA

	// Defines the CPP type used for the register
	// This is only used for debugging purposes in editor.
	UPROPERTY(transient)
	FName BaseCPPType;

	// The resolved UScriptStruct / UObject (in the future)
	// used for debugging.
	UPROPERTY(transient)
	TObjectPtr<UObject> BaseCPPTypeObject;

#endif

	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMRegister& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// returns true if this register is using a dynamic array for storage
	bool IsDynamic() const { return bIsDynamic; }

	// returns true if this register is using a dynamic array for storage
	bool IsNestedDynamic() const { return bIsDynamic && bIsArray; }

	// returns the current address of the register within the data byte array.
	// this can change over time - as the register is moving through slices.
	// use GetFirstAllocatedByte to get the fixed first byte.
	uint64 GetWorkByteIndex(int32 InSliceIndex = 0) const
	{
		ensure(InSliceIndex >= 0);
		return ByteIndex + ((uint64)InSliceIndex * GetNumBytesPerSlice());
	}

	// returns the first allocated byte in the data byte array
	uint64 GetFirstAllocatedByte() const
	{ 
		return ByteIndex - (uint64)AlignmentBytes;
	}

	// Returns the leading alignment bytes
	uint8 GetAlignmentBytes() const { return AlignmentBytes; }

	// Returns true if the register stores more than one element
	bool IsArray() const { return bIsArray || (ElementCount > 1); }

	// Returns the number of allocated bytes (including alignment + trailing bytes)
	uint16 GetAllocatedBytes() const { return ElementCount * ElementSize * SliceCount + (uint16)AlignmentBytes + TrailingBytes; }

	// Returns the number of bytes for a complete slice
	uint16 GetNumBytesPerSlice() const { return ElementCount * ElementSize; }

	// Returns the number of bytes for all slices
	uint16 GetNumBytesAllSlices() const { return ElementCount * ElementSize * SliceCount; }

	// Returns the total number of elements (elementcount * slicecount) in the register
	uint32 GetTotalElementCount() const { return (uint32)ElementCount * (uint32)SliceCount; }

};

// The register offset represents a memory offset within a register's memory.
// This can be used to represent memory addresses of array elements within
// a struct, for example.
// A register offset's path can look like MyTransformStruct.Transforms[3].Translation.X
USTRUCT()
struct FRigVMRegisterOffset
{
	GENERATED_BODY()

public:

	// default constructor
	FRigVMRegisterOffset()
		: Segments()
		, Type(ERigVMRegisterType::Invalid)
		, CPPType(NAME_None)
		, ScriptStruct(nullptr)
		, ParentScriptStruct(nullptr)
		, ArrayIndex(0)
		, ElementSize(0)
		, CachedSegmentPath() 
	{
	}

	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMRegisterOffset& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// constructs a path given a struct and a segment path
	UE_API FRigVMRegisterOffset(UScriptStruct* InScriptStruct, const FString& InSegmentPath, int32 InInitialOffset = 0, uint16 InElementSize = 0, const FName& InCPPType = NAME_None);

	// returns the data pointer within a container
	UE_API uint8* GetData(uint8* InContainer) const;
	
	// returns the segments of this path
	const TArray<int32>& GetSegments() const { return Segments; }

	// returns true if this register offset contains an array segment.
	// this means the register offset has to traverse a pointer and
	// cannot be cached / collapsed easily
	UE_API bool ContainsArraySegment() const;

	UE_API bool operator == (const FRigVMRegisterOffset& InOther) const;

	bool IsValid() const { return Type != ERigVMRegisterType::Invalid; }
	ERigVMRegisterType GetType() const { return Type; }
	FName GetCPPType() const { return CPPType; }
	FString GetCachedSegmentPath() const { return CachedSegmentPath; }
	int32 GetArrayIndex() const { return ArrayIndex; }
	UE_API uint16 GetElementSize() const;
	void SetElementSize(uint16 InElementSize) { ElementSize = InElementSize; };
	UE_API UScriptStruct* GetScriptStruct() const;

private:

	// Segments represent the memory offset(s) to use when accessing the target memory.
	// In case of indirection in the source memory each memory offset / jump is stored.
	// So for example: If you are accessing the third Rotation.X of an array of transforms,
	// the segments would read as: [-2, 12] (second array element, and the fourth float)
	// Segment indices less than zero represent array element offsets, while positive numbers
	// represents jumps within a struct.
	UPROPERTY()
	TArray<int32> Segments;

	// Type of resulting register (for example Plain for Transform.Translation.X)
	UPROPERTY()
	ERigVMRegisterType Type;

	// The c++ type of the resulting memory (for example float for Transform.Translation.X)
	UPROPERTY()
	FName CPPType;

	// The c++ script struct of the resulting memory (for example FVector for Transform.Translation)
	UPROPERTY()
	TObjectPtr<UScriptStruct> ScriptStruct;

	// The c++ script struct of the source memory (for example FTransform for Transform.Translation)
	UPROPERTY()
	TObjectPtr<UScriptStruct> ParentScriptStruct;

	// The index of the element within an array (for example 3 for Transform[3])
	UPROPERTY()
	int32 ArrayIndex;

	// The number of bytes of the resulting memory (for example 4 (float) for Transform.Translation.X)
	UPROPERTY()
	uint16 ElementSize;

	// The cached path of the segments within this register, for example FTransform.Translation.X
	UPROPERTY()
	FString CachedSegmentPath;

	friend struct FRigVMRegisterOffsetBuilder;
	friend class URigVM;
};

/**
 * The FRigVMMemoryContainer provides a heterogeneous memory container to store arbitrary
 * data. Each element stored can be referred to using a FRigVMRegister.
 * Elements can be accessed by index (index of the register), FRigVMOperand or by name.
 * Name access is optional and is specified upon construction of the container.
 * The memory container provides a series of templated functions to add and get data.
 *
 * For example:
 * 		int32 Index = Container.Add<float>(4.f);
 *      float& ValueRef = Container.GetRef<float>(Index);
 *
 * This can also be done with arrays:
 *      TArray<float> MyArray = {3.f, 4.f, 5.f};
 * 		int32 Index = Container.AddFixedArray<float>(MyArray);
 *      FRigVMFixedArray<float> ArrayView = Container.GetFixedArray<float>(Index);
 *
 *
 * Registers can store dynamically resizable memory as well by reyling on indirection.
 * 
 * Arrays with a single slice are going to be stored as a FRigVMByteArray, [unit8], so for example for a float array in C++ it would be (TArray<float> Param).
 * Single values with multiple slices are also going to be stored as a FRigVMByteArray, [unit8], so for example for a float array in C++ it would be (float Param). 
 * Arrays with multiple slices are going to be stored as a FRigVMNestedByteArray, [[unit8]], so for example for a float array in C++ it would be (TArray<float> Param).
 */
USTRUCT()
struct FRigVMMemoryContainer
{
	GENERATED_BODY()
	
public:

	UE_API FRigVMMemoryContainer(bool bInUseNames = true);
	
	UE_API FRigVMMemoryContainer(const FRigVMMemoryContainer& Other);
	UE_API ~FRigVMMemoryContainer();

	UE_API bool CopyRegisters(const FRigVMMemoryContainer &InOther);
	UE_API FRigVMMemoryContainer& operator= (const FRigVMMemoryContainer &InOther);

	// returns the memory type of this container
	ERigVMMemoryType GetMemoryType() const { return MemoryType;  }

	// sets the memory type. should only be used when the container is empty
	void SetMemoryType(ERigVMMemoryType InMemoryType) { MemoryType = InMemoryType; }

	// returns true if this container supports name based lookup
	bool SupportsNames() const { return bUseNameMap;  }

	// returns the number of registers in this container
	int32 Num() const { return Registers.Num(); }

	// resets the container but maintains storage.
	UE_API void Reset();

	// resets the container and removes all storage.
	UE_API void Empty();

	// const accessor for a register based on index
	const FRigVMRegister& operator[](int32 InIndex) const { return GetRegister(InIndex); }

	// accessor for a register based on index
	FRigVMRegister& operator[](int32 InIndex) { return GetRegister(InIndex); }

	// const accessor for a register based on an argument
	const FRigVMRegister& operator[](const FRigVMOperand& InArg) const { return GetRegister(InArg); }

	// accessor for a register based on an argument
	FRigVMRegister& operator[](const FRigVMOperand& InArg) { return GetRegister(InArg); }

	// const accessor for a register based on a a name. note: only works if SupportsNames() == true
	const FRigVMRegister& operator[](const FName& InName) const { return GetRegister(InName); }

	// accessor for a register based on a a name. note: only works if SupportsNames() == true
	FRigVMRegister& operator[](const FName& InName) { return GetRegister(InName); }

	TArray<FRigVMRegister>::RangedForIteratorType      begin() { return Registers.begin(); }
	TArray<FRigVMRegister>::RangedForConstIteratorType begin() const { return Registers.begin(); }
	TArray<FRigVMRegister>::RangedForIteratorType      end() { return Registers.end(); }
	TArray<FRigVMRegister>::RangedForConstIteratorType end() const { return Registers.end(); }

	// const accessor for a register based on index
	const FRigVMRegister& GetRegister(int32 InIndex) const { return Registers[InIndex]; }

	// accessor for a register based on index
	FRigVMRegister& GetRegister(int32 InIndex) { return Registers[InIndex]; }

	// const accessor for a register based on an argument
	const FRigVMRegister& GetRegister(const FRigVMOperand& InArg) const { return Registers[InArg.GetRegisterIndex()]; }

	// accessor for a register based on an argument
	FRigVMRegister& GetRegister(const FRigVMOperand& InArg) { return Registers[InArg.GetRegisterIndex()]; }

	// const accessor for a register based on a a name. note: only works if SupportsNames() == true
	const FRigVMRegister& GetRegister(const FName& InName) const { return Registers[GetIndex(InName)]; }

	// accessor for a register based on a a name. note: only works if SupportsNames() == true
	FRigVMRegister& GetRegister(const FName& InName) { return Registers[GetIndex(InName)]; }
	
	UE_API void Serialize(FArchive& Ar);
	UE_API void Save(FArchive& Ar);
	UE_API void Load(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMMemoryContainer& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	// Returns an argument for a given register.
	// This is typically used to store a light weight address for use within a VM.
	FRigVMOperand GetOperand(int32 InRegisterIndex, int32 InRegisterOffset)
	{
		ensure(Registers.IsValidIndex(InRegisterIndex));
		return FRigVMOperand(MemoryType, InRegisterIndex, InRegisterOffset);
	}

	// Returns an argument for a given register.
	// This is typically used to store a light weight address for use within a VM.
	FRigVMOperand GetOperand(int32 InRegisterIndex, const FString& InSegmentPath = FString(), int32 InArrayElement = INDEX_NONE)
	{
		ensure(Registers.IsValidIndex(InRegisterIndex));
		// Register offset must hold on to the ScriptStruct such that it can recalculate the struct size after cook
		UScriptStruct* ScriptStruct = GetScriptStruct(InRegisterIndex);

		int32 InitialOffset = 0;
		int32 ElementSize = 0;
		if (InArrayElement != INDEX_NONE)
		{
			InitialOffset = InArrayElement * Registers[InRegisterIndex].ElementSize;
			ElementSize = Registers[InRegisterIndex].ElementSize;
		}

		return GetOperand(InRegisterIndex, GetOrAddRegisterOffset(InRegisterIndex, ScriptStruct, InSegmentPath, InitialOffset, ElementSize));
	}

private:

	uint8* GetDataPtr(const FRigVMRegister& Register, int32 InRegisterOffset = INDEX_NONE, int32 InSliceIndex = 0, bool bArrayContent = false) const
	{
		if (Register.ElementCount == 0 && !Register.IsNestedDynamic())
		{
			return nullptr;
		}

		uint8* Ptr = nullptr;
		if (Register.IsDynamic())
		{
			Ptr = (uint8*)&Data[static_cast<int32>(Register.GetWorkByteIndex())];

			if (Register.IsNestedDynamic())
			{
				FRigVMNestedByteArray* ArrayStorage = (FRigVMNestedByteArray*)Ptr;
				Ptr = (uint8*)ArrayStorage->GetData();

				if (Ptr)
				{
					Ptr = Ptr + InSliceIndex * sizeof(FRigVMByteArray);
					if (Ptr && bArrayContent)
					{
						Ptr = ((FRigVMByteArray*)Ptr)->GetData();
					}
				}
			}
			else if(bArrayContent)
			{
				FRigVMByteArray* ArrayStorage = (FRigVMByteArray*)Ptr;
				Ptr = (uint8*)ArrayStorage->GetData();

				if (Ptr)
				{
					Ptr = Ptr + InSliceIndex * Register.GetNumBytesPerSlice();
				}
			}
		}
		else
		{
			Ptr = (uint8*)&Data[static_cast<int32>(Register.GetWorkByteIndex(InSliceIndex))];
		}

		if (InRegisterOffset != INDEX_NONE && Ptr != nullptr)
		{
			Ptr = RegisterOffsets[InRegisterOffset].GetData(Ptr);
		}
		return Ptr;
	}

public:
	
	// Returns the script struct used for a given register (can be nullptr for non-struct-registers).
	UScriptStruct* GetScriptStruct(const FRigVMRegister& Register) const
	{
		if (Register.ScriptStructIndex != INDEX_NONE)
		{
			ensure(ScriptStructs.IsValidIndex(Register.ScriptStructIndex));
			return ScriptStructs[Register.ScriptStructIndex];
		}
		return nullptr;
	}

	// Returns the script struct used for a given register index (can be nullptr for non-struct-registers).
	UScriptStruct* GetScriptStruct(int32 InRegisterIndex, int32 InRegisterOffset = INDEX_NONE) const
	{
		if (InRegisterOffset == INDEX_NONE)
		{
			ensure(Registers.IsValidIndex(InRegisterIndex));
			const FRigVMRegister& Register = Registers[InRegisterIndex];
			return GetScriptStruct(Register);
		}
		ensure(RegisterOffsets.IsValidIndex(InRegisterOffset));
		const FRigVMRegisterOffset& Path = RegisterOffsets[InRegisterOffset];
		return Path.GetScriptStruct();
	}
	
	// Returns the index of a register based on the register name.
	// Note: This only works if SupportsNames() == true
	int32 GetIndex(const FName& InName) const
	{
		if (!bUseNameMap)
		{
			return INDEX_NONE;
		}

		if (NameMap.Num() != Registers.Num())
		{
			for (int32 Index = 0; Index < Registers.Num(); Index++)
			{
				if (Registers[Index].Name == InName)
				{
					return Index;
				}
			}
		}
		else
		{
			const int32* Index = NameMap.Find(InName);
			if (Index != nullptr)
			{
				return *Index;
			}
		}

		return INDEX_NONE;
	}

	// Returns true if a given name is available for a new register.
	bool IsNameAvailable(const FName& InPotentialNewName) const
	{
		if (!bUseNameMap)
		{
			return false;
		}
		return GetIndex(InPotentialNewName) == INDEX_NONE;
	}
	
	// Adds a register path and returns its index
	UE_API int32 GetOrAddRegisterOffset(int32 InRegisterIndex, int32 InArrayElement = 0);

	// Adds a register path and returns its index
	UE_API int32 GetOrAddRegisterOffset(int32 InRegisterIndex, const FString& InSegmentPath, int32 InArrayElement = 0);

	// Adds a register path and returns its index
	UE_API int32 GetOrAddRegisterOffset(int32 InRegisterIndex, UScriptStruct* InScriptStruct, const FString& InSegmentPath, int32 InInitialOffset = 0, int32 InElementSize = 0);

	// Updates internal data for topological changes
	UE_API void UpdateRegisters();

	// Allocates a new named register
	UE_API int32 Allocate(const FName& InNewName, int32 InElementSize, int32 InElementCount, int32 InSliceCount, const uint8* InDataPtr = nullptr, bool bUpdateRegisters = true);

	// Allocates a new unnamed register
	UE_API int32 Allocate(int32 InElementSize, int32 InElementCount, int32 InSliceCount, const uint8* InDataPtr = nullptr, bool bUpdateRegisters = true);

	// Performs optional construction of data within a struct register
	UE_API bool Construct(int32 InRegisterIndex, int32 InElementIndex = INDEX_NONE, int32 InSliceIndex = 0);

	// Performs optional destruction of data within a struct register
	UE_API bool Destroy(int32 InRegisterIndex, int32 InElementIndex = INDEX_NONE, int32 InSliceIndex = 0);

	// Ensures to add a script struct to the internal map if needed
	UE_API int32 FindOrAddScriptStruct(UScriptStruct* InScriptStruct);
	
	UPROPERTY()
	bool bUseNameMap;

	UPROPERTY()
	ERigVMMemoryType MemoryType;

	UPROPERTY()
	TArray<FRigVMRegister> Registers;

	UPROPERTY()
	TArray<FRigVMRegisterOffset> RegisterOffsets;

	UPROPERTY(transient)
	TArray<uint8> Data;

	UPROPERTY(transient)
	TArray<TObjectPtr<UScriptStruct>> ScriptStructs;

	UPROPERTY(transient)
	TMap<FName, int32> NameMap;

	UPROPERTY(transient)
	bool bEncounteredErrorDuringLoad;

};

#undef UE_API
