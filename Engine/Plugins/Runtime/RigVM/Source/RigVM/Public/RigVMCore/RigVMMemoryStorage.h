// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "Misc/AssertionMacros.h"
#include "RigVMMemoryCommon.h"
#include "RigVMDefines.h"
#include "RigVMPropertyBag.h"
#if UE_RIGVM_DEBUG_EXECUTION
#include "RigVMModule.h"
#endif
#include "RigVMPropertyPath.h"
#include "RigVMStatistics.h"
#include "RigVMTraits.h"
#include "Templates/Casts.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnum.h"
#include "Templates/Models.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "RigVMStringUtils.h"

#include "RigVMMemoryStorage.generated.h"

class FArchive;
class URigVM;
struct FRigVMMemoryHandle;
struct FRigVMExecuteContext;
struct FRigVMExtendedExecuteContext;

enum class ERigVMExecuteResult : uint8
{
	Failed
	,Succeeded
#if WITH_EDITOR
	,Halted
#endif
};

//////////////////////////////////////////////////////////////////////////////
/// Lazy execution
//////////////////////////////////////////////////////////////////////////////

// A description of a branch in the VM's bytecode
USTRUCT()
struct FRigVMBranchInfo
{
	GENERATED_USTRUCT_BODY()

	FRigVMBranchInfo()
	: Index(INDEX_NONE)
	, Label(NAME_None)
	, InstructionIndex(INDEX_NONE)
	, ArgumentIndex(INDEX_NONE)
	, FirstInstruction(INDEX_NONE)
	, LastInstruction(INDEX_NONE)
	{}

	bool IsValid() const
	{
		return InstructionIndex != INDEX_NONE;
	}

	RIGVM_API void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRigVMBranchInfo& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	friend uint32 GetTypeHash(const FRigVMBranchInfo& InBranchInfo)
	{
		uint32 Hash = GetTypeHash(InBranchInfo.Label.ToString());
		Hash = HashCombine(Hash, GetTypeHash(InBranchInfo.InstructionIndex));
		Hash = HashCombine(Hash, GetTypeHash(InBranchInfo.ArgumentIndex));
		Hash = HashCombine(Hash, GetTypeHash(InBranchInfo.FirstInstruction));
		Hash = HashCombine(Hash, GetTypeHash(InBranchInfo.LastInstruction));
		return Hash;
	}

	static FString GetFixedArrayLabel(const FString& InArrayName, const FString& InElementName)
	{
		return RigVMStringUtils::JoinStrings(InArrayName, InElementName.Replace(TEXT("_"), TEXT("")), TEXT("_"));
	}

	static FName GetFixedArrayLabel(const FName& InArrayName, const FName& InElementName)
	{
		return *GetFixedArrayLabel(InArrayName.ToString(), InElementName.ToString());
	}

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	FName Label;

	UPROPERTY()
	int32 InstructionIndex;

	UPROPERTY()
	int32 ArgumentIndex;

	UPROPERTY()
	int32 FirstInstruction;

	UPROPERTY()
	int32 LastInstruction;
};


/**
 * A branch which can be lazily executed
 */
struct FRigVMLazyBranch
{
public:
	
	FRigVMLazyBranch()
		: VM(nullptr)
		, BranchInfo()
		, FunctionPtr(nullptr)
	{}

	FRigVMLazyBranch(int32 InBranchIndex, const TFunction<ERigVMExecuteResult()>& InFunctionPtr)
		: VM(nullptr)
		, BranchInfo()
		, FunctionPtr(InFunctionPtr)
	{
		BranchInfo.Index = InBranchIndex;
	}

	RIGVM_API ERigVMExecuteResult Execute(FRigVMExtendedExecuteContext& Context);
	RIGVM_API ERigVMExecuteResult ExecuteIfRequired(FRigVMExtendedExecuteContext& Context, uint32 InSliceHash);

private:

	URigVM* VM;
	FRigVMBranchInfo BranchInfo;
	TFunction<ERigVMExecuteResult()> FunctionPtr;

	friend class URigVM;
	friend class URigVMNativized;
};

/**
 * A template container for a lazily computed value
 */
struct TRigVMLazyValueBase
{
public:

	TRigVMLazyValueBase()
		: bFollowPropertyPath(false)
		, SliceHash(UINT32_MAX)
		, MemoryHandle(nullptr)
	{
	}

	RIGVM_API const uint8* GetData(const FRigVMExecuteContext& Context) const;
	const FRigVMMemoryHandle& GetMemoryHandle()const { return *MemoryHandle; }

protected:
	
	bool bFollowPropertyPath;
	uint32 SliceHash;
	FRigVMMemoryHandle* MemoryHandle;
	
	friend class URigVM;
	friend struct FRigVMMemoryHandle;
};

/**
 * A template container for a lazily computed value
 */
template<typename ComputedType>
struct TRigVMLazyValue : public TRigVMLazyValueBase
{
public:

	TRigVMLazyValue(ComputedType InConstValue)
		: TRigVMLazyValueBase()
		, ConstValue(InConstValue)
	{}

	virtual ~TRigVMLazyValue()
	{
	}

	const ComputedType& Get(const FRigVMExecuteContext& ExecuteContext) const
	{
		if(MemoryHandle)
		{
			return *(const ComputedType*)GetData(ExecuteContext);
		}
		return ConstValue;
	}

protected:

	TRigVMLazyValue(const TRigVMLazyValue<ComputedType>& InOther) = delete;

	TRigVMLazyValue(const TRigVMLazyValueBase& InOther)
		: TRigVMLazyValueBase(InOther)
	{}

	ComputedType ConstValue;

	friend struct FRigVMMemoryHandle;
};

//////////////////////////////////////////////////////////////////////////////
/// Memory handle
//////////////////////////////////////////////////////////////////////////////

/**
 * The FRigVMForwardedMemoryHandle is used to forward memory from a call site
 * to a callable. The compiler compiled callables arguments to this structure.
 * FRigVMMemoryHandles pointing to this structure will use the pointer to
 * perform indirection on the memory.
 */
USTRUCT()
struct FRigVMForwardedMemoryHandle
{
	GENERATED_USTRUCT_BODY()

	const FProperty* Property = nullptr;
	uint8* Memory = nullptr;

	void Reset()
	{
		Property = nullptr;
		Memory = nullptr;
	}
};

//////////////////////////////////////////////////////////////////////////////
/// Memory handle
//////////////////////////////////////////////////////////////////////////////

/**
 * The FRigVMMemoryHandle is used to access the memory used within a URigMemoryStorage.
 * The Memory Handle caches the pointer of the head property, and can rely on a
 * RigVMPropertyPath to traverse towards a tail property.
 * For example it can cache the pointer of a TArray<FTransform> property's memory
 * and use a property path of '[2].Translation.X' to retrieve the element 2's translation
 * X float / double.
 * Additionally the FRigVMMemoryHandle can be used to access 'sliced' memory. Sliced
 * memory in this case simply means another level of a nested TArray. So a 'sliced'
 * FTransform is stored as a TArray<FTransform>, which the handle can traverse into and
 * return the memory of a specific FTransform within that array.
 */
struct FRigVMMemoryHandle
{
public:

	// Default constructor
	FRigVMMemoryHandle()
		: Ptr(nullptr)
		, Property(nullptr) 
		, PropertyPath(nullptr)
		, LazyBranch(nullptr)
		, bIsForwarded(false)
	{}

	// Constructor from complete data
	FRigVMMemoryHandle(uint8* InPtr, const FProperty* InProperty,  const FRigVMPropertyPath* InPropertyPath, FRigVMLazyBranch* InLazyBranch = nullptr)
		: Ptr(InPtr)
		, Property(InProperty)
		, PropertyPath(InPropertyPath)
		, LazyBranch(InLazyBranch)
		, bIsForwarded(false)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			bIsForwarded = StructProperty->Struct == FRigVMForwardedMemoryHandle::StaticStruct();
		}
	}

	/**
	 * Returns the cached pointer stored within the handle.
	 * @param bFollowPropertyPath If set to true the memory handle will traverse the memory using the property path
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle.
	 */
	UE_DEPRECATED(5.7, "Please use GetInputData, GetOutputData or GetPrivateData instead.")
	uint8* GetData(bool bFollowPropertyPath, int32 InSliceIndex)
	{
		return GetData_Internal(bFollowPropertyPath, InSliceIndex);
	}

	/**
	 * Computes the data if necessary and returns the cached pointer stored within the handle.
	 * @param bFollowPropertyPath If set to true the memory handle will traverse the memory using the property path
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle (const)
	 */
	UE_DEPRECATED(5.7, "Please use GetInputData, GetOutputData or GetPrivateData instead.")
	const uint8* GetData(bool bFollowPropertyPath, int32 InSliceIndex) const
	{
		return GetData_Internal(bFollowPropertyPath, InSliceIndex);
	}

	/**
	 * Returns the cached pointer stored within the handle.
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle.
	 */
	const uint8* GetInputData(int32 InSliceIndex = INDEX_NONE) const
	{
		return GetData_Internal(true, InSliceIndex);
	}

	/**
	 * Returns the cached pointer stored within the handle.
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle.
	 */
	uint8* GetOutputData(int32 InSliceIndex = INDEX_NONE)
	{
		return GetData_Internal(false, InSliceIndex);
	}

	/**
	 * Returns the cached pointer stored within the handle.
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle.
	 */
	uint8* GetPrivateData(int32 InSliceIndex)
	{
		return GetData_Internal(false, InSliceIndex);
	}

	/**
	 * Computes the data if necessary and returns the cached pointer stored within the handle.
	 * @param InSliceIndex If this is != INDEX_NONE the memory handle will return the slice of the memory requested
	 * @return The traversed memory cached by this handle (const)
	 */
	template<typename ComputedType>
	TRigVMLazyValue<ComputedType> GetDataLazily(int32 InSliceIndex = INDEX_NONE) const
	{
		return GetDataLazily_Internal(false, InSliceIndex);
	}

	/**
	 * Returns true if this memory handle depends on a lazily executed branch
	 * @return True if this memory handle depends on a lazily executed branch
	 */
	bool IsLazy() const
	{
		return LazyBranch != nullptr;
	}
	
	/**
	 * Computes the data if necessary and returns true if the value is valid
	 * @return True if the value of the handle is valid after the compute
	 */
	RIGVM_API bool ComputeLazyValueIfNecessary(FRigVMExtendedExecuteContext& Context, uint32 InSliceHash);

	// Returns the head property of this handle
	const FProperty* GetProperty() const
	{
		if (bIsForwarded)
		{
			const FRigVMForwardedMemoryHandle* ForwardedHandle = (const FRigVMForwardedMemoryHandle*)Ptr;
			checkf(ForwardedHandle->Property, TEXT("If this is null the handle is being interacted with outside of a Invoke Callable instruction."));
			return ForwardedHandle->Property;
		}
		return Property;
	}

	// Returns the [optional] property path used within this handle
	const FRigVMPropertyPath* GetPropertyPath() const
	{
		return PropertyPath;
	}

	// Returns the [optional] property path used within this handle (ref)
	const FRigVMPropertyPath& GetPropertyPathRef() const
	{
		if(PropertyPath)
		{
			return *PropertyPath;
		}
		return FRigVMPropertyPath::Empty;
	}

	// Returns the resolved property the data is pointing to
	const FProperty* GetResolvedProperty(bool bIsHiddenArgument = false) const
	{
		if(bIsHiddenArgument)
		{
			check(PropertyPath == nullptr);
			return GetArrayElementPropertyChecked(GetProperty());
		}
		else if(PropertyPath)
		{
			return PropertyPath->GetTailProperty();
		}
		return GetProperty();
	}

	// Returns true if this memory handle maps to a given type of property
	template<typename PropertyType>
	bool IsPropertyType(bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty(bIsHiddenArgument);
		return ResolvedProperty->IsA<PropertyType>();
	}

	// Returns true if this memory handle maps to a given array type of property
	template<typename PropertyType>
	bool IsPropertyArrayType(bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty(bIsHiddenArgument);
		if(const FProperty* ElementProperty = GetArrayElementProperty(ResolvedProperty))
		{
			return ElementProperty->IsA<PropertyType>();
		}
		return false;
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TIsTArray_V<T>>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		typedef typename T::ElementType ElementType;
		return IsTypeArray<ElementType>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsBool<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsBool(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsBool<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsBoolArray(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsFloat<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsFloat(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsFloat<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsFloatArray(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsDouble<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsDouble(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsDouble<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsDoubleArray(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsInt32<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsInt32(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsInt32<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsInt32Array(bIsHiddenArgument);
	}
	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsName<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsName(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsName<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsNameArray(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsString<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsString(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsString<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsStringArray(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsStruct<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsStructArray<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsStruct<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsStructArray<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsEnum<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsEnumArray<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	bool IsType(bool bIsHiddenArgument = false) const
	{
		return IsObject<T>(bIsHiddenArgument);
	}

	// returns true if the handle is of the given type
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	bool IsTypeArray(bool bIsHiddenArgument = false) const
	{
		return IsObjectArray<T>(bIsHiddenArgument);
	}
	
	// Returns true if this memory handle maps to a bool property
	bool IsBool(bool bIsHiddenArgument = false) const
	{
		return IsPropertyType<FBoolProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a bool array property
	bool IsBoolArray(bool bIsHiddenArgument = false) const
	{
		return IsPropertyArrayType<FBoolProperty>(bIsHiddenArgument);
	}
	
	// Returns true if this memory handle maps to a float property
	bool IsFloat(bool bIsHiddenArgument = false) const
	{
		return IsPropertyType<FFloatProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a float array property
	bool IsFloatArray(bool bIsHiddenArgument = false) const
	{
		return IsPropertyArrayType<FFloatProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a double property
	bool IsDouble(bool bIsHiddenArgument = false) const
	{
		return IsPropertyType<FDoubleProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a double array property
	bool IsDoubleArray(bool bIsHiddenArgument = false) const
	{
		return IsPropertyArrayType<FDoubleProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a int32 property
	bool IsInt32(bool bIsHiddenArgument = false) const
	{
		return IsPropertyType<FIntProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a int32 array property
	bool IsInt32Array(bool bIsHiddenArgument = false) const
	{
		return IsPropertyArrayType<FIntProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an FName property
    bool IsName(bool bIsHiddenArgument = false) const
    {
    	return IsPropertyType<FNameProperty>(bIsHiddenArgument);
    }

	// Returns true if this memory handle maps to an FName array property
    bool IsNameArray(bool bIsHiddenArgument = false) const
    {
    	return IsPropertyArrayType<FNameProperty>(bIsHiddenArgument);
    }

	// Returns true if this memory handle maps to an FString property
	bool IsString(bool bIsHiddenArgument = false) const
	{
		return IsPropertyType<FStrProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an FString array property
	bool IsStringArray(bool bIsHiddenArgument = false) const
	{
		return IsPropertyArrayType<FStrProperty>(bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an enum property
	bool IsEnum(const UEnum* InEnum, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		return IsEnum(ResolvedProperty, InEnum, bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an enum array property
	bool IsEnumArray(const UEnum* InEnum, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		if(bIsHiddenArgument)
		{
			ResolvedProperty = GetArrayElementPropertyChecked(ResolvedProperty);
		}
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty))
		{
			return IsEnum(ArrayProperty->Inner, InEnum, false);
		}
		return false;
	}

	// Returns true if this memory handle maps to an enum property
	template<typename T>
	bool IsEnum(bool bIsHiddenArgument = false) const
	{
		return IsEnum(StaticEnum<T>(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an enum array property
	template<typename T>
	bool IsEnumArray(bool bIsHiddenArgument = false) const
	{
		return IsEnumArray(StaticEnum<T>(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a struct property
	bool IsStruct(const UScriptStruct* InStruct, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		return IsStruct(ResolvedProperty, InStruct, bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a struct array property
	bool IsStructArray(const UScriptStruct* InStruct, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		if(bIsHiddenArgument)
		{
			ResolvedProperty = GetArrayElementPropertyChecked(ResolvedProperty);
		}
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty))
		{
			return IsStruct(ArrayProperty->Inner, InStruct, false);
		}
		return false;
	}

	// Returns true if this memory handle maps to a struct property
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	bool IsStruct(bool bIsHiddenArgument = false) const
	{
		return IsStruct(TBaseStructure<T>::Get(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a struct property
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	bool IsStruct(bool bIsHiddenArgument = false) const
	{
		return IsStruct(T::StaticStruct(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a struct array property
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	bool IsStructArray(bool bIsHiddenArgument = false) const
	{
		return IsStructArray(TBaseStructure<T>::Get(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to a struct array property
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	bool IsStructArray(bool bIsHiddenArgument = false) const
	{
		return IsStructArray(T::StaticStruct(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an object property
	bool IsObject(const UClass* InClass, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		return IsObject(ResolvedProperty, InClass, bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an objct array property
	bool IsObjectArray(const UClass* InClass, bool bIsHiddenArgument = false) const
	{
		const FProperty* ResolvedProperty = GetResolvedProperty();
		if(bIsHiddenArgument)
		{
			ResolvedProperty = GetArrayElementPropertyChecked(ResolvedProperty);
		}
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty))
		{
			return IsObject(ArrayProperty->Inner, InClass, false);
		}
		return false;
	}

	// Returns true if this memory handle maps to an object property
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	bool IsObject(bool bIsHiddenArgument = false) const
	{
		return IsObject(T::StaticClass(), bIsHiddenArgument);
	}

	// Returns true if this memory handle maps to an objct array property
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	bool IsObjectArray(bool bIsHiddenArgument = false) const
	{
		return IsObjectArray(T::StaticClass(), bIsHiddenArgument);
	}

private:

	uint8* GetData_Internal(bool bFollowPropertyPath, int32 InSliceIndex) const
	{
		uint8* Memory = Ptr;
		if (bIsForwarded)
		{
			FRigVMForwardedMemoryHandle* ForwardedHandle = (FRigVMForwardedMemoryHandle*)Memory;
			checkf(ForwardedHandle->Memory, TEXT("If this is null the handle is being interacted with outside of a Invoke Callable instruction."));
			Memory = ForwardedHandle->Memory;
		}
		
		if(InSliceIndex != INDEX_NONE)
		{
			// Sliced memory cannot be accessed using a property path.
			// It refers to opaque memory only
			check(PropertyPath == nullptr);
			check(!bFollowPropertyPath);

			const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(GetProperty());
			FScriptArrayHelper ArrayHelper(ArrayProperty, Memory);

			if (InSliceIndex >= ArrayHelper.Num() - 1)
			{
				// For each slice we copy the default value to the next slice index
				// The Index 0 has the initial default value, we carry it over the next slice on each call
				const int32 NumValuesToAdd = FMath::Max(InSliceIndex + 1 - FMath::Max(ArrayHelper.Num() - 1, 0), 0);
				if (NumValuesToAdd > 0)
				{
					const int32 FirstAddedIndex = ArrayHelper.AddValues(NumValuesToAdd);
					if (FirstAddedIndex > 0)
					{
						if (const uint8* SourceElementMemory = ArrayHelper.GetRawPtr(FirstAddedIndex - 1))
						{
							const FProperty* ElementProperty = ArrayProperty->Inner;
							for (int32 i = FirstAddedIndex; i < ArrayHelper.Num(); ++i)
							{
#if UE_RIGVM_DEBUG_EXECUTION
								FString DefaultValue;
								ElementProperty->ExportText_Direct(
									DefaultValue,
									SourceElementMemory,
									SourceElementMemory,
									nullptr,
									PPF_None,
									nullptr);

								UE_LOGF(LogRigVM, Display, "Adding slice %d for Property '%ls', defaulting to '%ls'.",
									InSliceIndex,
									*ArrayProperty->GetName(),
									*DefaultValue
								);
#endif // UE_RIGVM_DEBUG_EXECUTION
								uint8* DestMemory = ArrayHelper.GetRawPtr(i);
								ElementProperty->CopyCompleteValue(DestMemory, SourceElementMemory);
							}
						}
					}
				}
			}

#if UE_RIGVM_DEBUG_EXECUTION
			// const FProperty* ElementProperty = ArrayProperty->Inner;
			// FString DefaultValue;
			// const uint8* DefaultElementMemory = ArrayHelper.GetRawPtr(InSliceIndex);
			// ElementProperty->ExportText_Direct(
			// 	DefaultValue,
			// 	DefaultElementMemory,
			// 	DefaultElementMemory,
			// 	nullptr,
			// 	PPF_None,
			// 	nullptr);
			//
			// UE_LOGF(LogRigVM, Display, "Getting slice %d for Property '%ls', currently '%ls'.",
			// 	InSliceIndex,
			// 	*ArrayProperty->GetName(),
			// 	*DefaultValue
			// );
#endif

			return ArrayHelper.GetRawPtr(InSliceIndex);
		}

		// traverse the property path to the tail property
		// and return its memory instead.
		if(bFollowPropertyPath && PropertyPath != nullptr)
		{
			return PropertyPath->GetData<uint8>(Memory, GetProperty());
		}
		return Memory;
	}

	TRigVMLazyValueBase GetDataLazily_Internal(bool bFollowPropertyPath, uint32 InSliceHash) const
	{
		// note: this works also for memory handles which don't provide a lazy branch
		TRigVMLazyValueBase LazyValue;
		LazyValue.MemoryHandle = (FRigVMMemoryHandle*)this;
		LazyValue.bFollowPropertyPath = bFollowPropertyPath;
		LazyValue.SliceHash = InSliceHash;
		return LazyValue;
	}

	static const FProperty* GetArrayElementProperty(const FProperty* InProperty)
	{
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			return ArrayProperty->Inner;
		}
		return nullptr;
	}

	static const FProperty* GetArrayElementPropertyChecked(const FProperty* InProperty)
	{
		const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(InProperty);
		return ArrayProperty->Inner;
	}

	static bool IsEnum(const FProperty* InProperty, const UEnum* InEnum, bool bUseArrayElement)
	{
		const FProperty* Property = InProperty;
		if(bUseArrayElement)
		{
			Property = GetArrayElementPropertyChecked(Property);
		}
		if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			if(EnumProperty->GetEnum() == InEnum)
			{
				return true;
			}
		}
		if(const FByteProperty* EnumProperty = CastField<FByteProperty>(Property))
		{
			if(EnumProperty->Enum == InEnum)
			{
				return true;
			}
		}
		return false;
	}

	static bool IsStruct(const FProperty* InProperty, const UScriptStruct* InScriptStruct, bool bUseArrayElement)
	{
		const FProperty* Property = InProperty;
		if(bUseArrayElement)
		{
			Property = GetArrayElementPropertyChecked(Property);
		}
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if(StructProperty->Struct == InScriptStruct)
			{
				return true;
			}
		}
		return false;
	}

	static bool IsObject(const FProperty* InProperty, const UClass* InClass, bool bUseArrayElement)
	{
		const FProperty* Property = InProperty;
		if(bUseArrayElement)
		{
			Property = GetArrayElementPropertyChecked(Property);
		}
		if(const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if(ObjectProperty->PropertyClass == InClass)
			{
				return true;
			}
		}
		return false;
	}

	// The pointer of the memory of the head property
	uint8* Ptr;

	// The head property used by this handle
	const FProperty* Property;

	// The [optional] property path used by this handle
	const FRigVMPropertyPath* PropertyPath;

	// The [optional] lazy branch used by this handle
	FRigVMLazyBranch* LazyBranch;

	// If this is true the handle points to a FRigVMForwardedMemoryHandle
	bool bIsForwarded;

	friend class URigVM;
	friend struct TRigVMLazyValueBase;
	friend class URigVMMemoryStorage;
	friend struct FRigVMMemoryStorageStruct;
};

/**
 * The URigVMMemoryStorageGeneratorClass is used to create / represent heterogeneous
 * memory storages. The generator can produce a UClass which contains a series of
 * properties. This UClass is then used to instantiate URigVMMemoryStorage objects
 * to be consumed by the RigVM. The memory storage objects can contain the literals
 * / constant values used by the virtual machine or work state.
 */
UCLASS(MinimalAPI)
class URigVMMemoryStorageGeneratorClass :
	public UClass
{
	GENERATED_BODY()

	// DECLARE_WITHIN(UObject) is only kept for back-compat, please don't parent the class
	// to the asset object.
	// This class should be parented to the package, instead of the asset object
	// because the engine no longer supports asset object as UClass outer
	// Given in the past we have parented the class to the asset object,
	// this flag has to be kept such that we can load the old asset in the first place and
	// re-parent it back to the package in post load
	DECLARE_WITHIN(UObject)

public:

	// Default constructor
	URigVMMemoryStorageGeneratorClass()
		: Super()
		, MemoryType(ERigVMMemoryType::Literal)
		, CachedMemoryHash(0)
	{}
	
	// UClass overrides
	RIGVM_API void PurgeClass(bool bRecompilingOnLoad) override;
	RIGVM_API void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	RIGVM_API void Serialize(FArchive& Ar) override;
	RIGVM_API virtual void PostLoad() override;

	// Returns the name of a storage class for a given memory type. The name is unique within the package.
	static RIGVM_API const FString& GetClassName(ERigVMMemoryType InMemoryType);

	// Returns an existing class for a memory type within the package (or nullptr) 
	static RIGVM_API URigVMMemoryStorageGeneratorClass* GetStorageClass(UObject* InOuter, ERigVMMemoryType InMemoryType);

	/**
	 * Creates a new class provided a list of properties and property paths
	 * @param InOuter The package or the outer object of this class
	 * @param InMemoryType The memory type to use for this class (affects class name)
	 * @param InProperties The descriptions for the properties to create
	 * @param InPropertyPaths The descriptions for the property paths to create
	 */
	static RIGVM_API URigVMMemoryStorageGeneratorClass* CreateStorageClass(
		UObject* InOuter,
		ERigVMMemoryType InMemoryType,
		const TArray<FRigVMPropertyDescription>& InProperties,
		const TArray<FRigVMPropertyPathDescription>& InPropertyPaths = TArray<FRigVMPropertyPathDescription>());

	/**
	 * Removes an existing storage class
	 * @param InOuter The package or the outer object of this class
	 * @param InMemoryType The memory type to use for this class (affects class name)
	 */
	static RIGVM_API bool RemoveStorageClass(
		UObject* InOuter,
		ERigVMMemoryType InMemoryType);

	// Returns the type of memory of this class (literal, work, etc)
	ERigVMMemoryType GetMemoryType() const { return MemoryType; }

	// Returns a hash of unique to the configuration of the memory
	RIGVM_API uint32 GetMemoryHash() const;

	// The properties stored within this class
	const TArray<const FProperty*>& GetProperties() const { return LinkedProperties; }

	// The property paths stored within this class
	const TArray<FRigVMPropertyPath>& GetPropertyPaths() const { return PropertyPaths; }

	// returns the statistics information
	RIGVM_API FRigVMMemoryStatistics GetStatistics() const;

protected:

	// Adds a single property to a class.
	static RIGVM_API FProperty* AddProperty(URigVMMemoryStorageGeneratorClass* InClass, const FRigVMPropertyDescription& InProperty, FField::FLinkedListBuilder& PropertyListBuilder);

public:

	RIGVM_API void RefreshLinkedProperties();
	RIGVM_API void RefreshPropertyPaths();

	bool IsValidPropertyPathDescriptionIndex(int32 Index) const
	{
		return PropertyPathDescriptions.IsValidIndex(Index);
	}

	const FRigVMPropertyPathDescription* GetPropertyPathDescriptionByIndex(int32 Index) const
	{
		return &PropertyPathDescriptions[Index];
	}

private:

	// The type of memory of this class
	ERigVMMemoryType MemoryType;

	// A cached list of all linked properties (created by RefreshLinkedProperties)
	TArray<const FProperty*> LinkedProperties;

	// A cached list of all property paths (created by RefreshPropertyPaths)
	TArray<FRigVMPropertyPath> PropertyPaths;

	// A list of decriptions for the property paths - used for serialization
	TArray<FRigVMPropertyPathDescription> PropertyPathDescriptions;

	mutable uint32 CachedMemoryHash;

	friend class URigVMMemoryStorage;
	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
	friend class URigVM;
	friend class URigVMHost;
	friend struct FRigVMCodeGenerator;
};

/**
 * The URigVMMemoryStorage represents an instance of heterogeneous memory.
 * The memory layout is defined by the URigVMMemoryStorageGeneratorClass.
 */
UCLASS(MinimalAPI)
class URigVMMemoryStorage : public UObject
{
	GENERATED_BODY()

public:

	//////////////////////////////////////////////////////////////////////////////
	/// Memory Access
	//////////////////////////////////////////////////////////////////////////////

	// Returns the memory type of this memory
	ERigVMMemoryType GetMemoryType() const
	{
		if(URigVMMemoryStorageGeneratorClass* Class = Cast<URigVMMemoryStorageGeneratorClass>(GetClass()))
		{
			return Class->GetMemoryType();
		}
		// empty debug containers don't have a generator class
		return ERigVMMemoryType::Debug;
	}

	// Returns a hash of unique to the configuration of the memory
	uint32 GetMemoryHash() const
	{
		if(URigVMMemoryStorageGeneratorClass* Class = Cast<URigVMMemoryStorageGeneratorClass>(GetClass()))
		{
			return Class->GetMemoryHash();
		}
		// empty debug containers don't have a generator class
		return 0;
	}

	// Returns the number of properties stored in this instance
	int32 Num() const
	{
		return GetProperties().Num();
	}

	// Returns true if a provided property index is valid
	bool IsValidIndex(int32 InIndex) const
	{
		return GetProperties().IsValidIndex(InIndex);
	}

	// Returns the properties provided by this instance
	RIGVM_API const TArray<const FProperty*>& GetProperties() const;

	// Returns the property paths provided by this instance
	RIGVM_API const TArray<FRigVMPropertyPath>& GetPropertyPaths() const;

	// Returns the index of a property given the property itself
	RIGVM_API int32 GetPropertyIndex(const FProperty* InProperty) const;

	// Returns the index of a property given its name
	RIGVM_API int32 GetPropertyIndexByName(const FName& InName) const;

	// Returns a property given its index
	const FProperty* GetProperty(int32 InPropertyIndex) const
	{
		return GetProperties()[InPropertyIndex];
	}

	// Returns a property given its name (or nullptr if the name wasn't found)
	RIGVM_API FProperty* FindPropertyByName(const FName& InName) const;

	// Creates and returns a new operand for a property (and optionally a property path)
	RIGVM_API FRigVMOperand GetOperand(int32 InPropertyIndex, int32 InPropertyPathIndex = INDEX_NONE) const;

	// Creates and returns a new operand for a property (and optionally a property path)
	RIGVM_API FRigVMOperand GetOperandByName(const FName& InName, int32 InPropertyPathIndex = INDEX_NONE) const;

	// returns the statistics information
	FRigVMMemoryStatistics GetStatistics() const
	{
		if(URigVMMemoryStorageGeneratorClass* Class = Cast<URigVMMemoryStorageGeneratorClass>(GetClass()))
		{
			return Class->GetStatistics();
		}

		FRigVMMemoryStatistics Statistics;
		Statistics.RegisterCount = 0;
		Statistics.DataBytes = URigVMMemoryStorage::StaticClass()->GetStructureSize();
		Statistics.TotalBytes = Statistics.DataBytes;
		return Statistics;
	}

	// Returns true if the property at a given index is a TArray
	bool IsArray(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FArrayProperty>();
	}

	// Returns true if the property at a given index is a TMap
	bool IsMap(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FMapProperty>();
	}

	// Returns the memory for a property given its index
	template<typename T>
	T* GetData(int32 InPropertyIndex)
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		check(Properties.IsValidIndex(InPropertyIndex));
		return Properties[InPropertyIndex]->ContainerPtrToValuePtr<T>(this);
	}

	// Returns the memory for a property given its name (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if(PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex);
	}

	// Returns the memory for a property given its index and a matching property path
	template<typename T>
	T* GetData(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		const FProperty* Property = GetProperty(InPropertyIndex);
		return InPropertyPath.GetData<T>(GetData<uint8>(InPropertyIndex), Property);
	}

	// Returns the memory for a property given its name and a matching property path (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if(PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex, InPropertyPath);
	}

	// Returns the memory for a property (and optionally a property path) given an operand
	template<typename T>
	T* GetData(const FRigVMOperand& InOperand)
	{
		const int32 PropertyIndex = InOperand.GetRegisterIndex();
		const int32 PropertyPathIndex = InOperand.GetRegisterOffset();

		check(GetProperties().IsValidIndex(PropertyIndex));
		
		if(PropertyPathIndex == INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		check(GetPropertyPaths().IsValidIndex(PropertyPathIndex));
		return GetData<T>(PropertyIndex, GetPropertyPaths()[PropertyPathIndex]); 
	}

	// Returns the ref of an element stored at a given property index
	template<typename T>
	T& GetRef(int32 InPropertyIndex)
	{
		return *GetData<T>(InPropertyIndex);
	}

	// Returns the ref of an element stored at a given property name (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName)
	{
		return *GetDataByName<T>(InName);
	}

	// Returns the ref of an element stored at a given property index and a property path
	template<typename T>
	T& GetRef(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetData<T>(InPropertyIndex, InPropertyPath);
	}

	// Returns the ref of an element stored at a given property name and a property path (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetDataByName<T>(InName, InPropertyPath);
	}

	// Returns the ref of an element stored for a given operand
	template<typename T>
	T& GetRef(const FRigVMOperand& InOperand)
	{
		return *GetData<T>(InOperand);
	}

	// Returns the exported text for a given property index
	RIGVM_API FString GetDataAsString(int32 InPropertyIndex, int32 PortFlags = PPF_None);

	// Returns the exported text for given property name 
	FString GetDataAsStringByName(const FName& InName, int32 PortFlags = PPF_None)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsString(PropertyIndex, PortFlags);
	}

	// Returns the exported text for a given operand
	RIGVM_API FString GetDataAsString(const FRigVMOperand& InOperand, int32 PortFlags = PPF_None);

	// Returns the exported text for a given property index
	RIGVM_API FString GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags = PPF_None);

	// Returns the exported text for given property name 
	FString GetDataAsStringByNameSafe(const FName& InName, int32 PortFlags = PPF_None)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsStringSafe(PropertyIndex, PortFlags);
	}

	// Returns the exported text for a given operand
	RIGVM_API FString GetDataAsStringSafe(const FRigVMOperand& InOperand, int32 PortFlags = PPF_None);

	// Sets the content of a property by index given an exported string. Returns true if succeeded
	RIGVM_API bool SetDataFromString(int32 InPropertyIndex, const FString& InValue);

	// Sets the content of a property by name given an exported string. Returns true if succeeded
	bool SetDataFromStringByName(const FName& InName, const FString& InValue)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return SetDataFromString(PropertyIndex, InValue);
	}

	// Returns the handle for a given property by index (and optionally property path)
	RIGVM_API FRigVMMemoryHandle GetHandle(int32 InPropertyIndex, const FRigVMPropertyPath* InPropertyPath = nullptr);

	// Returns the handle for a given property by name (and optionally property path)
	FRigVMMemoryHandle GetHandleByName(const FName& InName, const FRigVMPropertyPath* InPropertyPath = nullptr)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetHandle(PropertyIndex, InPropertyPath);
	}

	/**
	 * Copies the content of a source property and memory into the memory of a target property
	 * @param InTargetProperty The target property to copy into
	 * @param InTargetPtr The memory of the target value to copy into
	 * @param InSourceProperty The source property to copy from
	 * @param InSourcePtr The memory of the source value to copy from
	 * @return true if the copy operations was successful
	 */ 
	static RIGVM_API bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant supports property paths for both target and source.
	 * @param InTargetProperty The target property to copy into
	 * @param InTargetPtr The memory of the target value to copy into
	 * @param InTargetPropertyPath The property path to use when traversing the target memory.
	 * @param InSourceProperty The source property to copy from
	 * @param InSourcePtr The memory of the source value to copy from
	 * @param InSourcePropertyPath The property path to use when traversing the source memory.
	 * @return true if the copy operations was successful
	 */ 
	static RIGVM_API bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FRigVMPropertyPath& InTargetPropertyPath,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr,
		const FRigVMPropertyPath& InSourcePropertyPath);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant retrieves the memory pointers given target and source storage objects.
	 * @param InTargetStorage The target property to copy into
	 * @param InTargetPropertyIndex The index of the value to use on the InTargetStorage
	 * @param InTargetPropertyPath The property path to use when traversing the target memory.
	 * @param InSourceStorage The source property to copy from
	 * @param InSourcePropertyIndex The index of the value to use on the InSourceStorage
	 * @param InSourcePropertyPath The property path to use when traversing the source memory.
	 * @return true if the copy operations was successful
	 */ 
	static RIGVM_API bool CopyProperty(
		URigVMMemoryStorage* InTargetStorage,
		int32 InTargetPropertyIndex,
		const FRigVMPropertyPath& InTargetPropertyPath,
		URigVMMemoryStorage* InSourceStorage,
		int32 InSourcePropertyIndex,
		const FRigVMPropertyPath& InSourcePropertyPath);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant retrieves the memory pointers given target and source storage objects.
	 * @param InTargetHandle The handle of the target memory to copy into
	 * @param InSourceHandle The handle of the source memory to copy from
	 * @return true if the copy operations was successful
	 */ 
	static RIGVM_API bool CopyProperty(
		FRigVMMemoryHandle& InTargetHandle,
		FRigVMMemoryHandle& InSourceHandle);

	RIGVM_API void CopyFrom(URigVMMemoryStorage* InSourceMemory);

private:
	
	static RIGVM_API const TArray<const FProperty*> EmptyProperties;
	static RIGVM_API const TArray<FRigVMPropertyPath> EmptyPropertyPaths;
};
