// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMCore/RigVMPropertyBag.h"
#include "RigVMCore/RigVMPropertyPath.h"
#include "StructUtils/PropertyBag.h"
#include "RigVMMemoryStorageStruct.generated.h"

#define UE_API RIGVM_API

USTRUCT()
struct FRigVMMemoryStorageStruct : public FInstancedPropertyBag
{
	GENERATED_BODY()

	UE_API FRigVMMemoryStorageStruct();
	UE_API explicit FRigVMMemoryStorageStruct(ERigVMMemoryType InMemoryType);
	UE_API FRigVMMemoryStorageStruct(ERigVMMemoryType InMemoryType, const TArray<FRigVMPropertyDescription>& InPropertyDescriptions, const TArray<FRigVMPropertyPathDescription>& InPropertyPaths = TArray<FRigVMPropertyPathDescription>());
	UE_API ~FRigVMMemoryStorageStruct();

	UE_API void InitializeFromNativeStruct(const UScriptStruct* InStruct);

	UE_API bool Serialize(FArchive& Ar);
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API void GetUserDefinedDependencies(TArray<const UObject*>& OutDependencies) const;
	UE_API void GetRequiredPlugins(TArray<FString>& OutPlugins) const;

	friend FArchive& operator<<(FArchive& Ar, FRigVMMemoryStorageStruct& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static bool IsClassOf(const FRigVMMemoryStorageStruct* InElement)
	{
		return true;
	}

	//---------------------------------------------------------------------------

	void SetMemoryType(ERigVMMemoryType InMemoryType)
	{
		MemoryType = InMemoryType;
	}
	ERigVMMemoryType GetMemoryType() const
	{
		return MemoryType;
	}

	//---------------------------------------------------------------------------

	/**
	 * Adds properties to the storage. If property of same name already exists, it will be replaced with the new type.
	 * Numeric property values will be converted if possible, when a property's type changes.
	 * @param Descriptors : Descriptors of new properties to add.
	 */
	UE_API void AddProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions, const TArray<FRigVMPropertyPathDescription>& InPropertyPathDescriptions = TArray<FRigVMPropertyPathDescription>());


	// Returns the number of properties stored in this instance
	int32 Num() const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties.Num();
	}

	// Returns true if a provided property index is valid
	bool IsValidIndex(int32 InIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties.IsValidIndex(InIndex);
	}


	// Returns the properties provided by this instance
	const TArray<const FProperty*>& GetProperties() const 
	{
		return LinkedProperties;
	}

	// Returns the property paths provided by this instance
	const TArray<FRigVMPropertyPath>& GetPropertyPaths() const 
	{
		return PropertyPaths;
	}

	// Returns the index of a property given the property itself
	UE_API int32 GetPropertyIndex(const FProperty* InProperty) const;

	// Returns the index of a property given its name
	UE_API int32 GetPropertyIndexByName(const FName& InName) const;

	// Returns a property given its index
	const FProperty* GetProperty(int32 InPropertyIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties[InPropertyIndex];
	}

	// Returns a property given its name (or nullptr if the name wasn't found)
	UE_API FProperty* FindPropertyByName(const FName& InName) const;

	// Creates and returns a new operand for a property (and optionally a property path)
	UE_API FRigVMOperand GetOperand(int32 InPropertyIndex, int32 InPropertyPathIndex = INDEX_NONE) const;

	// Creates and returns a new operand for a property (and optionally a property path)
	UE_API FRigVMOperand GetOperandByName(const FName& InName, int32 InPropertyPathIndex = INDEX_NONE) const;

	// Returns the raw memory storage pointer
	UE_API void* GetContainerPtr() const;

	UE_API uint32 GetMemoryHash() const;

	// Returns true if the property at a given index is a TArray
	bool IsArray(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FArrayProperty>();
	}

	// Returns true if the property at a given index is a TMap
	bool IsMap(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FMapProperty>();
	}

	//---------------------------------------------------------------------------

	// Returns the memory for a property given its index
	template<typename T>
	T* GetData(int32 InPropertyIndex) 
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		check(Properties.IsValidIndex(InPropertyIndex));
		return Properties[InPropertyIndex]->ContainerPtrToValuePtr<T>(GetMutableValue().GetMemory());
	}
	template<typename T>
	const T* GetData(int32 InPropertyIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		check(Properties.IsValidIndex(InPropertyIndex));
		return Properties[InPropertyIndex]->ContainerPtrToValuePtr<T>(GetValue().GetMemory());
	}

	// Returns the memory for a property given its name (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if (PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex);
	}

	// Returns the mutable memory for a given property (or nullptr if does not belong to this storage)
	template<typename T>
	T* GetData(const FProperty* Property)
	{
		if (const int32 PropertyIndex = GetPropertyIndex(Property); PropertyIndex != INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		return nullptr;
	}
	template<typename T>
	const T* GetData(const FProperty* Property) const
	{
		if (const int32 PropertyIndex = GetPropertyIndex(Property); PropertyIndex != INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		return nullptr;
	}

	// Returns the memory for a property given its index and a matching property path
	template<typename T>
	T* GetData(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		const FProperty* Property = GetProperty(InPropertyIndex);
		return InPropertyPath.GetData<T>(GetData<uint8>(InPropertyIndex), Property);
	}

	// Returns the memory for a property given its name and a matching property path (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if (PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex, InPropertyPath);
	}

	// Returns the memory for a property (and optionally a property path) given an operand
	template<typename T>
	T* GetData(const FRigVMOperand& InOperand)
	{
		const int32 PropertyIndex = InOperand.GetRegisterIndex();
		const int32 PropertyPathIndex = InOperand.GetRegisterOffset();

		check(GetProperties().IsValidIndex(PropertyIndex));

		if (PropertyPathIndex == INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		check(GetPropertyPaths().IsValidIndex(PropertyPathIndex));
		return GetData<T>(PropertyIndex, GetPropertyPaths()[PropertyPathIndex]);
	}

	// Returns the ref of an element stored at a given property index
	template<typename T>
	T& GetRef(int32 InPropertyIndex)
	{
		return *GetData<T>(InPropertyIndex);
	}

	// Returns the ref of an element stored at a given property name (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName)
	{
		return *GetDataByName<T>(InName);
	}

	// Returns the ref of an element stored at a given property index and a property path
	template<typename T>
	T& GetRef(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetData<T>(InPropertyIndex, InPropertyPath);
	}

	// Returns the ref of an element stored at a given property name and a property path (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetDataByName<T>(InName, InPropertyPath);
	}

	// Returns the ref of an element stored for a given operand
	template<typename T>
	T& GetRef(const FRigVMOperand& InOperand)
	{
		return *GetData<T>(InOperand);
	}

	//---------------------------------------------------------------------------

	// Returns the exported text for a given property index
	UE_API FString GetDataAsString(int32 InPropertyIndex, int32 PortFlags = PPF_None) const;

	// Returns the exported text for given property name 
	FString GetDataAsStringByName(const FName& InName, int32 PortFlags = PPF_None) const
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsString(PropertyIndex, PortFlags);
	}

	// Returns the exported text for a given operand
	UE_API FString GetDataAsString(const FRigVMOperand& InOperand, int32 PortFlags = PPF_None) const;

	// Returns the exported text for a given property index
	UE_API FString GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags = PPF_None) const;

	// Returns the exported text for given property name 
	FString GetDataAsStringByNameSafe(const FName& InName, int32 PortFlags = PPF_None) const
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsStringSafe(PropertyIndex, PortFlags);
	}

	// Returns the exported text for a given operand
	UE_API FString GetDataAsStringSafe(const FRigVMOperand& InOperand, int32 PortFlags = PPF_None) const;

	// Sets the content of a property by index given an exported string. Returns true if succeeded
	UE_API bool SetDataFromString(int32 InPropertyIndex, const FString& InValue);

	// Sets the content of a property by name given an exported string. Returns true if succeeded
	bool SetDataFromStringByName(const FName& InName, const FString& InValue)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return SetDataFromString(PropertyIndex, InValue);
	}

	//---------------------------------------------------------------------------

	// Returns the handle for a given property by index (and optionally property path)
	UE_API FRigVMMemoryHandle GetHandle(int32 InPropertyIndex, const FRigVMPropertyPath* InPropertyPath = nullptr);

	// Returns the handle for a given property by name (and optionally property path)
	FRigVMMemoryHandle GetHandleByName(const FName& InName, const FRigVMPropertyPath* InPropertyPath = nullptr)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetHandle(PropertyIndex, InPropertyPath);
	}

	//---------------------------------------------------------------------------

	void SetPropertyPathDescriptions(const TArray<FRigVMPropertyPathDescription>& InPropertyPathDescriptions)
	{
		PropertyPathDescriptions = InPropertyPathDescriptions;
	}

	void ResetPropertyPathDescriptions()
	{
		PropertyPathDescriptions.Reset();
	}

	UE_API void Refresh();
	UE_API void RefreshPropertyPaths();

	bool IsValidPropertyPathDescriptionIndex(int32 Index) const
	{
		return PropertyPathDescriptions.IsValidIndex(Index);
	}

	const FRigVMPropertyPathDescription* GetPropertyPathDescriptionByIndex(int32 Index) const
	{
		return &PropertyPathDescriptions[Index];
	}

	/**
	 * Copies the content of a source property and memory into the memory of a target property
	 * @param InTargetProperty The target property to copy into
	 * @param InTargetPtr The memory of the target value to copy into
	 * @param InSourceProperty The source property to copy from
	 * @param InSourcePtr The memory of the source value to copy from
	 * @return true if the copy operations was successful
	 */
	static UE_API bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant supports property paths for both target and source.
	 * @param InTargetProperty The target property to copy into
	 * @param InTargetPtr The memory of the target value to copy into
	 * @param InTargetPropertyPath The property path to use when traversing the target memory.
	 * @param InSourceProperty The source property to copy from
	 * @param InSourcePtr The memory of the source value to copy from
	 * @param InSourcePropertyPath The property path to use when traversing the source memory.
	 * @return true if the copy operations was successful
	 */
	static UE_API bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FRigVMPropertyPath& InTargetPropertyPath,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr,
		const FRigVMPropertyPath& InSourcePropertyPath);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant retrieves the memory pointers given target and source storage objects.
	 * @param InTargetStorage The target property to copy into
	 * @param InTargetPropertyIndex The index of the value to use on the InTargetStorage
	 * @param InTargetPropertyPath The property path to use when traversing the target memory.
	 * @param InSourceStorage The source property to copy from
	 * @param InSourcePropertyIndex The index of the value to use on the InSourceStorage
	 * @param InSourcePropertyPath The property path to use when traversing the source memory.
	 * @return true if the copy operations was successful
	 */
	static UE_API bool CopyProperty(
		FRigVMMemoryStorageStruct* InTargetStorage,
		int32 InTargetPropertyIndex,
		const FRigVMPropertyPath& InTargetPropertyPath,
		FRigVMMemoryStorageStruct* InSourceStorage,
		int32 InSourcePropertyIndex,
		const FRigVMPropertyPath& InSourcePropertyPath);

	/**
	 * Copies the content of a source property and memory into the memory of a target property.
	 * This variant retrieves the memory pointers given target and source storage objects.
	 * @param InTargetHandle The handle of the target memory to copy into
	 * @param InSourceHandle The handle of the source memory to copy from
	 * @return true if the copy operations was successful
	 */
	static UE_API bool CopyProperty(
		FRigVMMemoryHandle& InTargetHandle,
		FRigVMMemoryHandle& InSourceHandle);

	/**
	* Returns the PropertyBag struct using a standard function signature, so it can be used at IStructureDataProvider
	*/
	UE_API  const UStruct * GetStruct() const;

	// Helper function to generate a property bag desc from a FRigVMPropertyDescription
	static UE_API FPropertyBagPropertyDesc GeneratePropertyBagDescriptor(const FRigVMPropertyDescription& RigVMDescriptor);

protected:
	ERigVMMemoryType MemoryType = ERigVMMemoryType::Invalid;

	TArray<FRigVMPropertyPathDescription> PropertyPathDescriptions;

	// A cached list of all linked properties (created by RefreshLinkedProperties)
	TArray<const FProperty*> LinkedProperties;

	// A cached list of all property paths (created by RefreshPropertyPaths)
	TArray<FRigVMPropertyPath> PropertyPaths;

	// A list of decriptions for the property paths - used for serialization

	mutable int32 CachedMemoryHash = 0;

	static UE_API const TArray<const FProperty*> EmptyProperties;
	static UE_API const TArray<FRigVMPropertyPath> EmptyPropertyPaths;

	void ForEachObjectDependency(const TFunction<void(const UObject*)>& InCallBack) const;

	UE_API void RefreshLinkedProperties();

	UE_API void SetDefaultValues(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions);

	static UE_API bool GetPropertyTypeDataFromVMDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, EPropertyBagPropertyType& OutBagPropertyType, FPropertyBagContainerTypes& OutBagContainerTypes);
};

template<> struct TStructOpsTypeTraits<FRigVMMemoryStorageStruct> : public TStructOpsTypeTraitsBase2<FRigVMMemoryStorageStruct>
{
	enum
	{
		WithSerializer = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
