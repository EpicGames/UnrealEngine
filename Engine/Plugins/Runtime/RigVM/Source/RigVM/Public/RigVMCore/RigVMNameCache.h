// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class ERigVMNameOp : uint8
{
	None,
	Concat,
	Left,
    Right,
    LeftChop,
	RightChop,
	ReplaceCase,
	ReplaceNoCase,
	EndsWithCase,
	EndsWithNoCase,
    StartsWithCase,
    StartsWithNoCase,
    ContainsCase,
    ContainsNoCase
};

struct FRigVMNameOp
{
public:

	FRigVMNameOp()
	: A(INDEX_NONE)
	, B(INDEX_NONE)
	, C(INDEX_NONE)
	, Type(ERigVMNameOp::None)
	{}

	static uint32 GetTypeHash(const FName& InName)
	{
		return HashCombine(InName.GetComparisonIndex().ToUnstableInt(), uint32(InName.GetNumber()));
	}
	
	static RIGVM_API FRigVMNameOp Concat(const FName& InA, const FName& InB);
	static RIGVM_API FRigVMNameOp Left(const FName& InA, const uint32 InCount);
	static RIGVM_API FRigVMNameOp Right(const FName& InA, const uint32 InCount);
	static RIGVM_API FRigVMNameOp LeftChop(const FName& InA, const uint32 InCount);
	static RIGVM_API FRigVMNameOp RightChop(const FName& InA, const uint32 InCount);
	static RIGVM_API FRigVMNameOp Replace(const FName& InA, const FName& InB, const FName& InC, const ESearchCase::Type InSearchCase);
    static RIGVM_API FRigVMNameOp EndsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);
    static RIGVM_API FRigVMNameOp StartsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);
    static RIGVM_API FRigVMNameOp Contains(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);

	friend uint32 GetTypeHash(const FRigVMNameOp& Op)
	{
		return HashCombine(Op.A, HashCombine(Op.B, HashCombine(Op.C, (uint32)Op.Type)));
	}

	bool operator ==(const FRigVMNameOp& Other) const
	{
		return Type == Other.Type && A == Other.A && B == Other.B && C == Other.C;
	}

	bool operator !=(const FRigVMNameOp& Other) const
	{
		return Type != Other.Type || A != Other.A || B != Other.B || C != Other.C;
	}

	bool operator <(const FRigVMNameOp& Other) const
	{
		if (Type < Other.Type)
		{
			return true;
		}
		if(A < Other.A)
		{
			return true;
		}
		if(B < Other.B)
		{
			return true;
		}
		return C < Other.C;
	}

	bool operator >(const FRigVMNameOp& Other) const
	{
		if (Type > Other.Type)
		{
			return true;
		}
		if(A > Other.A)
		{
			return true;
		}
		if(B > Other.B)
		{
			return true;
		}
		return C > Other.C;
	}


private:

	uint32 A;
	uint32 B;
	uint32 C;
	ERigVMNameOp Type;
};

class FRigVMNameCache
{
public:

	void Reset()
	{
		NameCache.Reset();
		BoolCache.Reset();
	}

	RIGVM_API FName Concat(const FName& InA, const FName& InB);
	RIGVM_API FName Left(const FName& InA, const uint32 InCount);
	RIGVM_API FName Right(const FName& InA, const uint32 InCount);
	RIGVM_API FName LeftChop(const FName& InA, const uint32 InCount);
	RIGVM_API FName RightChop(const FName& InA, const uint32 InCount);
	RIGVM_API FName Replace(const FName& InA, const FName& InB, const FName& InC, const ESearchCase::Type InSearchCase);
	RIGVM_API bool EndsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);
	RIGVM_API bool StartsWith(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);
	RIGVM_API bool Contains(const FName& InA, const FName& InB, const ESearchCase::Type InSearchCase);

	const TMap<FRigVMNameOp, FName>& GetNameCache() const { return NameCache; }
	const TMap<FRigVMNameOp, bool>&  GetBoolCache() const { return BoolCache; }
	RIGVM_API TArray<FRigVMNameOp> GetNameOps() const;
	RIGVM_API TArray<FName> GetNameValues() const;
	RIGVM_API TArray<FRigVMNameOp> GetBoolOps() const;
	RIGVM_API TArray<bool> GetBoolValues() const;

private:

#if WITH_EDITOR
	bool CheckCacheSize() const;
#else
	bool CheckCacheSize() const { return true; } 
#endif

	TMap<FRigVMNameOp, FName> NameCache;
	TMap<FRigVMNameOp, bool> BoolCache;
};
