// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Reverse.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Logging/LogVerbosity.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/OutputDevice.h"
#include "RigVMDefines.h"
#include "RigVM.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMMemoryCommon.h"
#include "RigVMCore/RigVMMemoryStorage.h"
#include "RigVMCore/RigVMTraits.h"
#include "Templates/EnableIf.h"
#include "Templates/Models.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "RigVMNativized.generated.h"

#define UE_API RIGVM_API

class FArchive;
class UObject;
struct FRigVMInstructionArray;

#define RIGVMNATIVIZED_ARRAYTYPE_BODY(NativeType) \
operator TArray<NativeType>&() { return Array; } \
operator const TArray<NativeType>&() const { return Array; } \
void Reset() { Array.Reset(); } \
void Empty() { Array.Empty(); } \
void Reserve(int32 InSize) { Array.Reserve(InSize); } \
void SetNum(int32 InSize) { Array.SetNum(InSize); } \
void SetNumUninitialized(int32 InSize) { Array.SetNumUninitialized(InSize); } \
void SetNumZeroed(int32 InSize) { Array.SetNumZeroed(InSize); } \
bool IsValidIndex(int32 InIndex) const { return Array.IsValidIndex(InIndex); } \
int32 Num() const { return Array.Num(); } \
bool IsEmpty() const { return Array.IsEmpty(); } \
const NativeType& operator[](int32 InIndex) const { return Array[InIndex]; } \
NativeType& operator[](int32 InIndex) { return Array[InIndex]; } \
TArray<NativeType>::RangedForIteratorType      begin()       { return Array.begin(); } \
TArray<NativeType>::RangedForConstIteratorType begin() const { return Array.begin(); } \
TArray<NativeType>::RangedForIteratorType      end()         { return Array.end();   } \
TArray<NativeType>::RangedForConstIteratorType end() const   { return Array.end();   } \
const NativeType* GetData() const { return Array.GetData(); }

struct FRigVMNativizedContext
{
	UE_API FRigVMNativizedContext(FRigVMExtendedExecuteContext& InExecuteContext, FRigVMExecuteContext& InPublicContext);
	FRigVMExtendedExecuteContext& ExecuteContext;
	FRigVMExecuteContext& PublicContextBase;
};

template<typename ComputedType>
class TRigVMNativizedLazyValue : public TRigVMLazyValue<ComputedType>
{
public:
	TRigVMNativizedLazyValue(const ComputedType& Value, const FProperty* Property)
	: TRigVMLazyValue<ComputedType>(Value)
	, NativizedMemoryHandle((uint8*)&Value, Property, nullptr, nullptr)
	{
		TRigVMLazyValueBase::MemoryHandle = &NativizedMemoryHandle;
	}

	TRigVMNativizedLazyValue(const ComputedType& Value, const FProperty* Property, int32 InBranchIndex, const TFunction<ERigVMExecuteResult()>& ComputeCallback)
	: TRigVMLazyValue<ComputedType>(Value)
	, LazyBranch(InBranchIndex, ComputeCallback)
	, NativizedMemoryHandle((uint8*)&Value, Property, nullptr, &LazyBranch)
	{
		TRigVMLazyValueBase::MemoryHandle = &NativizedMemoryHandle;
	}

	TRigVMNativizedLazyValue(const ComputedType& Value, const FProperty* Property, const TFunction<ERigVMExecuteResult()>& ComputeCallback)
	: TRigVMNativizedLazyValue(Value, Property, INDEX_NONE, ComputeCallback)
	{
	}
	
	virtual ~TRigVMNativizedLazyValue() override
	{
	}

	operator const ComputedType& () const
	{
		return NativizedMemoryHandle.GetDataLazily<ComputedType>();
	}

	operator ComputedType& ()
	{
		return NativizedMemoryHandle.GetDataLazily<ComputedType>();
	}

private:

	FRigVMLazyBranch LazyBranch;
	FRigVMMemoryHandle NativizedMemoryHandle;
};

UCLASS(MinimalAPI, BlueprintType, Abstract)
class URigVMNativized : public URigVM
{
	GENERATED_BODY()

public:

	UE_API URigVMNativized();
	UE_API virtual ~URigVMNativized() override;

	UE_API static UClass* FindClassForHash(uint32 InVMHash);
	UE_API static void RegisterClassForHash(uint32 InVMHash, UClass* InClass);
	UE_API static TMap<uint32, UClass*>& GetNativizedClassMap();

	// URigVM interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override { return; }
	UE_API virtual void Reset(FRigVMExtendedExecuteContext& Context) override;
	virtual bool IsNativized() const override { return true; }
	virtual void Empty(FRigVMExtendedExecuteContext& Context) override { return; }
	UE_API virtual bool Initialize(FRigVMExtendedExecuteContext& Context) override;
	UE_API virtual ERigVMExecuteResult ExecuteVM(FRigVMExtendedExecuteContext& Context, const FName& InEntryName = NAME_None) override;
	virtual int32 AddRigVMFunction(UScriptStruct* InRigVMStruct, const FName& InMethodName) override { return INDEX_NONE; }
	virtual FString GetRigVMFunctionName(int32 InFunctionIndex) const override { return FString(); }
	UE_API virtual FRigVMMemoryStorageStruct* GetMemoryByType(FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType) override;
	UE_API virtual const FRigVMMemoryStorageStruct* GetMemoryByType(const FRigVMExtendedExecuteContext& Context, ERigVMMemoryType InMemoryType) const override;
	virtual void ClearMemory(FRigVMExtendedExecuteContext& Context) override { return; }
	UE_API virtual const FRigVMInstructionArray& GetInstructions() override;
	UE_API virtual const FRigVMByteCode& GetByteCode() const override;
	UE_API virtual const TArray<const FRigVMFunction*>& GetFunctions() const override;
	UE_API virtual const TArray<FName>& GetFunctionNames() const override;
	UE_API virtual const TArray<const FRigVMDispatchFactory*>& GetFactories() const override;
	virtual bool ContainsEntry(const FName& InEntryName) const override { return GetEntryNames().Contains(InEntryName); }
	virtual int32 FindEntry(const FName& InEntryName) const override { return GetEntryNames().Find(InEntryName); }
	virtual const TArray<FName>& GetEntryNames() const override { static const TArray<FName> EmptyEntries; return EmptyEntries; }

	UE_DEPRECATED(5.8, "Please, use with FRigVMNativizedContext param")
	[[nodiscard]] virtual bool SetInstructionIndex(FRigVMExtendedExecuteContext& Context, FRigVMExecuteContext& PublicData, int32 InInstructionIndex) const override
	{
		if (!Super::SetInstructionIndex(Context, PublicData, InInstructionIndex))
		{
			return false;
		}
#if WITH_EDITOR
		
		if (FRigVMInstructionVisitInfo* InstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
		{
			InstructionVisitInfo->SetInstructionVisitedDuringLastRun(InInstructionIndex);
			InstructionVisitInfo->AddInstructionIndexToVisitOrder(InInstructionIndex);
		}
#endif
		return true;
	}

	[[nodiscard]] bool SetInstructionIndex(FRigVMNativizedContext& Context, int32 InInstructionIndex) const
	{
		if (!Super::SetInstructionIndex(Context.ExecuteContext, Context.PublicContextBase, InInstructionIndex))
		{
			return false;
		}
#if WITH_EDITOR
		if (FRigVMInstructionVisitInfo* InstructionVisitInfo = Context.ExecuteContext.GetRigVMInstructionVisitInfo())
		{
			InstructionVisitInfo->SetInstructionVisitedDuringLastRun(InInstructionIndex);
			InstructionVisitInfo->AddInstructionIndexToVisitOrder(InInstructionIndex);
		}
#endif
		return true;
	}

	UE_API virtual URigVM* GetSourceVM() const;
#if WITH_EDITOR
	UE_API void SetSourceVM(URigVM* InSourceVM);
#endif

	template<typename T>
	T* SetupNativizedVariables(FRigVMExtendedExecuteContext& Context)
	{
		Context.NativizedVMVariables = MakeShared<T>();
		return (T*)Context.NativizedVMVariables.Get();
	}
	template<typename T>
	T* GetNativizedVariables(FRigVMExtendedExecuteContext& Context)
	{
		check(Context.NativizedVMVariables.IsValid());
		return (T*)Context.NativizedVMVariables.Get();
	}

	virtual void SetExternalVariablesInstanceData(FRigVMExtendedExecuteContext& Context, const TArray<FRigVMExternalVariable>& InExternalVariables, bool bAllowNullMemory = false) override {};

protected:

	template<typename ExecuteContextType = FRigVMExecuteContext>
	ExecuteContextType& GetPublicContext(FRigVMExtendedExecuteContext& Context, int32 InNumberInstructions, int32 InNumberCallables, const FName& InEntryName)
	{
		Context.ResetExecutionState();
		Context.SetVM(this);
		Context.Frame->SliceOffsetsPerInstruction.SetNumZeroed(InNumberInstructions);
		Context.Frame->SliceOffsetsPerCallable.SetNumZeroed(InNumberCallables);
		Context.GetPublicData<ExecuteContextType>().EventName = InEntryName;
		SetupInstructionTracking(Context, InNumberInstructions, InNumberCallables, InEntryName);
		return Context.GetPublicData<ExecuteContextType>();
	}

	UE_API const FRigVMFunction* FindDispatchFunction(const FString& InIdentifier) const;

#if WITH_EDITOR
	mutable TWeakObjectPtr<URigVM> WeakSourceVM;
#endif

	class FErrorPipe : public FOutputDevice
	{
	public:

		int32 NumErrors;

		FErrorPipe()
			: FOutputDevice()
			, NumErrors(0)
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			NumErrors++;
		}
	};

	void BeginLoopSlice(FRigVMNativizedContext& Context, int32 InCount, int32 InRelativeIndex = 0)
	{
		Context.ExecuteContext.BeginLoopSlice(InCount, InRelativeIndex);
	}

	void EndLoopSlice(FRigVMNativizedContext& Context)
	{
		Context.ExecuteContext.EndLoopSlice();
	}

	void BeginCallableSlice(FRigVMNativizedContext& Context, int32 InCallableIndex)
	{
		Context.ExecuteContext.BeginCallableSlice(InCallableIndex);
	}

	void EndCallableSlice(FRigVMNativizedContext& Context)
	{
		Context.ExecuteContext.EndCallableSlice();
	}

	bool IsValidArraySize(const FRigVMNativizedContext& Context, int32 InArraySize) const
	{
		return Context.ExecuteContext.IsValidArraySize(InArraySize);
	}

	template<typename T>
	bool IsValidArrayIndex(const FRigVMNativizedContext& Context, int32& InOutIndex, const TArray<T>& InArray) const
	{
		return Context.ExecuteContext.IsValidArrayIndex(InOutIndex, InArray.Num());
	}

	template<typename T>
	static const T& GetArrayElementSafe(const TArray<T>& InArray, const int32 InIndex)
	{
		if(InArray.IsValidIndex(InIndex))
		{
			return InArray[InIndex];
		}
		static const T EmptyElement;
		return EmptyElement;
	}

	template<typename T>
	static T& GetArrayElementSafe(TArray<T>& InArray, const int32 InIndex)
	{
		if(InArray.IsValidIndex(InIndex))
		{
			return InArray[InIndex];
		}
		static T EmptyElement;
		return EmptyElement;
	}

	template<typename A, typename B>
	static void CopyUnrelatedArrays(TArray<A>& Target, const TArray<B>& Source)
	{
		const int32 Num = Source.Num();
		Target.SetNum(Num);
		for(int32 Index = 0; Index < Num; Index++)
		{
			Target[Index] = (A)Source[Index];
		}
	}

	void BroadcastExecutionReachedExit(FRigVMExtendedExecuteContext& Context)
	{
		if(Context.EntriesBeingExecuted.Num() == 1)
		{
			if (Context.OnExecutionReachedExitCallback)
			{
				Context.OnExecutionReachedExitCallback(Context.GetPublicData<>().GetEventName());
			}
		}
		Context.GetPublicDataSafe<>().NumExecutions++;
	}

	template<typename T>
	T& GetExternalVariableRef(const FName& InExternalVariableName, const FName& InExpectedType, const TArray<FRigVMExternalVariable>& InExternalVariables, TMap<FName,int32>& InLookup) const
	{
		if (InLookup.IsEmpty())
		{
			for(int32 Index = 0; Index < InExternalVariables.Num(); Index++)
			{
				InLookup.Add(InExternalVariables[Index].GetName(), Index);
			}
		}

		const int32* Index = InLookup.Find(InExternalVariableName);
		if (InExternalVariables.IsValidIndex(*Index))
		{
			const FRigVMExternalVariable& ExternalVariable = InExternalVariables[*Index];
			if(ExternalVariable.GetExtendedCPPType() == InExpectedType)
			{
				if(ExternalVariable.IsValid(false))
				{
					return *(T*)ExternalVariable.GetMemory();
				}
			}
		}
		static T EmptyValue;
		return EmptyValue;
	}

public:
	
	template<typename TargetT, typename SourceT>
	static void Copy(const SourceT& InSource, TargetT& InTarget)
	{
		InTarget = InSource;
	}

	template<typename ArrayType>
	static typename ArrayType::ElementType& GetOperandSlice(FRigVMNativizedContext& Context, ArrayType& InOutArray, const typename ArrayType::ElementType* InDefaultValue = nullptr)
	{
		return GetOperandSlice(Context.ExecuteContext, InOutArray, InDefaultValue);
	}

	template<typename ArrayType>
	static typename ArrayType::ElementType& GetOperandSlice(FRigVMExtendedExecuteContext& Context, ArrayType& InOutArray, const typename ArrayType::ElementType* InDefaultValue = nullptr)
	{
		const int32 SliceIndex = Context.GetSlice().GetIndex();
		if(InOutArray.Num() <= SliceIndex)
		{
			const int32 FirstInsertedItem = InOutArray.AddDefaulted(1 + SliceIndex - InOutArray.Num());
			if (InDefaultValue)
			{
				for (int32 i=FirstInsertedItem; i<InOutArray.Num();++i)
				{
					InOutArray[i] = *InDefaultValue;
				}
			}
		}
		return InOutArray[SliceIndex];
	}
	
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static T GetStructConstant(const FString& InDefaultValue)
	{
		T Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			TBaseStructure<T>::Get()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
	>
	static T GetStructConstant(const FString& InDefaultValue)
	{
		T Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			T::StaticStruct()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static TArray<T> GetStructArrayConstant(const FString& InDefaultValue)
	{
		TArray<T> Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			TBaseStructure<T>::Get()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
	>
	static TArray<T> GetStructArrayConstant(const FString& InDefaultValue)
	{
		TArray<T> Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			T::StaticStruct()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	static TArray<TArray<T>> GetStructArrayArrayConstant(const FString& InDefaultValue)
	{
		TArray<TArray<T>> Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			TBaseStructure<T>::Get()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
	>
	static TArray<TArray<T>> GetStructArrayArrayConstant(const FString& InDefaultValue)
	{
		TArray<TArray<T>> Value;
		if(!InDefaultValue.IsEmpty())
		{
			FErrorPipe ErrorPipe;
			T::StaticStruct()->ImportText(*InDefaultValue, &Value, nullptr, PPF_None, &ErrorPipe, FString());
		}
		return Value;
	}

protected:

	class FDefaultValueImportErrorPipe : public FOutputDevice
	{
	public:

		FString Owner;
		int32 NumErrors;

		FDefaultValueImportErrorPipe(const FString& InOwner)
			: FOutputDevice()
			, Owner(InOwner)
			, NumErrors(0)
		{
		}

		UE_API virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override;
	};

	struct FRigVMNativizedCallstackGuard
	{
	public:
		FRigVMNativizedCallstackGuard(FRigVMNativizedContext& InContext, const FRigVMCallableInfo* InCallable = nullptr)
			: Context(InContext.ExecuteContext)
			, PublicData(InContext.PublicContextBase)
			, PreviousInstructionIndex(INDEX_NONE)
			, PreviousNumSlices(InContext.ExecuteContext.Frame->Slices.Num())
			, PreviousCallstackHash(Context.CallstackHash)
			, bIsCallable(false)
		{
			bIsCallable = InCallable != nullptr;
			if (bIsCallable)
			{
				PreviousInstructionIndex = Context.GetInstructionIndex();
				PublicData.InstructionIndex = InCallable->FirstInstruction;
				Context.Callstack.Push({PreviousInstructionIndex, InCallable});
				Context.CallstackHash = HashCombine(PreviousInstructionIndex, InCallable->Index, Context.CallstackHash);
				Context.BeginCallableSlice(InCallable->Index);
#if WITH_EDITOR
				if (FRigVMInstructionVisitInfo* RigVMInstructionVisitInfo = Context.GetRigVMInstructionVisitInfo())
				{
					RigVMInstructionVisitInfo->SetCallableVisitedDuringLastRun(InCallable->Index);
					RigVMInstructionVisitInfo->AddCallableIndexToVisitOrder(InCallable->Index);
				}
#endif
			}
		}

		~FRigVMNativizedCallstackGuard()
		{
			if (bIsCallable)
			{
				Context.EndCallableSlice();
				Context.CallstackHash = PreviousCallstackHash;
				Context.Callstack.Pop();
				PublicData.InstructionIndex = PreviousInstructionIndex;
			}
			else
			{
				while (Context.Frame->Slices.Num() > PreviousNumSlices)
				{
					if (Context.Frame->Slices.Last().IsLoop())
					{
						Context.EndLoopSlice();
					}
					else if (Context.Frame->Slices.Last().IsCallable())
					{
						Context.EndCallableSlice();
					}
				}
			}
		}
		
	private:
		FRigVMExtendedExecuteContext& Context;
		FRigVMExecuteContext& PublicData;
		int32 PreviousInstructionIndex;
		int32 PreviousNumSlices;
		uint32 PreviousCallstackHash;
		bool bIsCallable;
	};

	template <typename T,
			  typename TEnableIf<!TIsArray<T>::Value, int>::Type = 0>
	bool ImportDefaultValue(T& OutValue, const FProperty* Property, const FString& InBuffer)
	{
		return false;
	}

	template <typename T,
			  typename TEnableIf<TIsArray<T>::Value, int>::Type = 0>
	bool ImportDefaultValue(T& OutValue, const FProperty* Property, const FString& InBuffer)
	{
		return false;
	}
};

template<>
inline void URigVMNativized::Copy(const double& InSource, float& InTarget)
{
	InTarget = static_cast<float>(InSource);
}

template<>
inline void URigVMNativized::Copy(const float& InSource, double& InTarget)
{
	InTarget = static_cast<double>(InSource);
}

template<>
inline void URigVMNativized::Copy(const TArray<double>& InSource, TArray<float>& InTarget)
{
	InTarget.Reset();
	InTarget.Reserve(InSource.Num());
	for (const double& SourceValue : InSource)
	{
		InTarget.Add(static_cast<float>(SourceValue));
	}
}

template<>
inline void URigVMNativized::Copy(const TArray<float>& InSource, TArray<double>& InTarget)
{
	InTarget.Reset();
	InTarget.Reserve(InSource.Num());
	for (const float& SourceValue : InSource)
	{
		InTarget.Add(static_cast<double>(SourceValue));
	}
}

struct FPlaneRef
{
public:
	FPlaneRef(FMatrix& InMatrix, int32 InColumn)
		: Matrix(InMatrix)
		, Column(InColumn)
		, X(Matrix.M[0][InColumn])
		, Y(Matrix.M[1][InColumn])
		, Z(Matrix.M[2][InColumn])
		, W(Matrix.M[3][InColumn])
	{
	}
	
	FMatrix& Matrix;
	int32 Column;
	FMatrix::FReal& X;
	FMatrix::FReal& Y;
	FMatrix::FReal& Z;
	FMatrix::FReal& W;
};

#undef UE_API
