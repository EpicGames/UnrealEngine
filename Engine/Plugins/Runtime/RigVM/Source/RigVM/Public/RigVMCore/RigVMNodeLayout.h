// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMObjectVersion.h"
#include "RigVMNodeLayout.generated.h"

#define UE_API RIGVM_API

USTRUCT(BlueprintType)
struct FRigVMPinCategory
{
	GENERATED_BODY()
	
	FRigVMPinCategory()
	: Path()
	, Elements()
	, bExpandedByDefault(true)
	{}

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	FString Path;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	TArray<FString> Elements;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	bool bExpandedByDefault;

	UE_API FString GetName() const;

	friend uint32 GetTypeHash(const FRigVMPinCategory& Category)
	{
		uint32 Hash = GetTypeHash(Category.Path);
		for (const FString& Element : Category.Elements)
		{
			Hash = HashCombine(Hash, GetTypeHash(Element));
		}
		Hash = HashCombine(Hash, GetTypeHash(Category.bExpandedByDefault));
		return Hash;
	}

	bool operator < (const FRigVMPinCategory& Other) const
	{
		return FCString::Strcmp(*Path, *Other.Path) < 0;
	}

	friend FArchive& operator<<(FArchive& Ar, FRigVMPinCategory& Category)
	{
		Ar << Category.Path;
		Ar << Category.Elements;
		if(Ar.IsLoading())
		{
			if(Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::FunctionHeaderLayoutStoresCategoryExpansion)
			{
				Category.bExpandedByDefault = true;
			}
			else
			{
				Ar << Category.bExpandedByDefault;
			}
		}
		else
		{
			Ar << Category.bExpandedByDefault;
		}
		return Ar;
	}

	UE_API bool IsDefaultCategory() const;

	static UE_API const FString& GetDefaultCategoryName();
};

USTRUCT(BlueprintType)
struct FRigVMNodeLayout
{
	GENERATED_BODY()
	
	FRigVMNodeLayout()
	: Categories()
	, DisplayNames()
	{}

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	TArray<FRigVMPinCategory> Categories;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	TMap<FString, int32> PinIndexInCategory;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=FunctionArgument)
	TMap<FString, FString> DisplayNames;

	void Reset()
	{
		Categories.Reset();
		PinIndexInCategory.Reset();
		DisplayNames.Reset();
	}

	UE_API bool IsValid() const;

	friend uint32 GetTypeHash(const FRigVMNodeLayout& Layout)
	{
		uint32 Hash = 0;;
		for (const FRigVMPinCategory& Category : Layout.Categories)
		{
			Hash = HashCombine(Hash, GetTypeHash(Category));
		}
		for(const TPair<FString, int32>& Pair : Layout.PinIndexInCategory)
		{
			Hash = HashCombine(Hash, GetTypeHash(Pair));
		}
		for(const TPair<FString, FString>& Pair : Layout.DisplayNames)
		{
			Hash = HashCombine(Hash, GetTypeHash(Pair));
		}
		return Hash;
	}

	bool operator ==(const FRigVMNodeLayout& OtherLayout) const
	{
		return GetTypeHash(*this) == GetTypeHash(OtherLayout);
	}

	bool operator !=(const FRigVMNodeLayout& OtherLayout) const
	{
		return !(*this == OtherLayout);
	}
	
	friend RIGVM_API FArchive& operator<<(FArchive& Ar, FRigVMNodeLayout& Layout);

	UE_API const FString* FindCategory(const FString& InElement) const;
	UE_API const FString* FindDisplayName(const FString& InElement) const;
};

#undef UE_API
