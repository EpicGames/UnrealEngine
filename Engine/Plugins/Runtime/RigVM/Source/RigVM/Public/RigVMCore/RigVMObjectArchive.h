// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/ArchiveUObject.h"
#include "RigVMObjectArchive.generated.h"

#define UE_API RIGVM_API

USTRUCT()
struct FRigVMObjectArchive
{
public:
	GENERATED_BODY()

	FRigVMObjectArchive()
		: UncompressedSize(INDEX_NONE)
		, CompressedSize(INDEX_NONE)
		, bIsCompressed(false)
	{
	}
	
	UE_API void Reset();
	UE_API void Empty();
	UE_API bool IsEmpty() const;
	UE_API void Compress();
	UE_API bool Decompress();
	bool IsCompressed() const
	{
		return bIsCompressed;
	}

	UE_API FString ToString();
	UE_API bool SetFromString(const FString& InString);

	RIGVM_API friend FArchive& operator<<(FArchive& Ar, FRigVMObjectArchive& Data);

	RIGVM_API void SetVersionsFromArchive(const FArchive& Ar);

private:

	UPROPERTY()
	TMap<FGuid,int32> Versions;
	
	UPROPERTY()
	TArray<uint8> Buffer;

	UPROPERTY()
	int32 UncompressedSize;
	
	UPROPERTY()
	int32 CompressedSize;

	UPROPERTY()
	bool bIsCompressed;

	friend class FRigVMObjectArchiveWriter;
	friend class FRigVMObjectArchiveReader;
};

class FRigVMObjectArchiveWriter : public FArchiveUObject
{
public:
	UE_API FRigVMObjectArchiveWriter( FRigVMObjectArchive& InOutArchive, const UObject* InRoot );
	UE_API virtual ~FRigVMObjectArchiveWriter() override;
	UE_API virtual void Serialize( void* V, int64 Length ) override;
	UE_API virtual int64 Tell() override;
	UE_API virtual int64 TotalSize() override;
	UE_API virtual void Seek(int64 InPos) override;
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;

protected:
	
	FRigVMObjectArchive& Archive;
	int64 Offset;
	const UObject* Root;
	const FString RootPathName;
	TSet<UObject*> VisitedObjects;
	TMap<FName,int64> NameToOffset;
	
	static inline constexpr uint8 StoringNullptr = 0;
	static inline constexpr uint8 StoringFullObject = 1;
	static inline constexpr uint8 StoringArchiveLocalPath = 2;
	static inline constexpr uint8 StoringNameAsString = 0;
	static inline constexpr uint8 StoringNameAsOffset = 1;
};

class FRigVMObjectArchiveReader : public FRigVMObjectArchiveWriter
{
public:
	
	UE_API FRigVMObjectArchiveReader( FRigVMObjectArchive& InOutArchive, const UObject* InRoot );
	UE_API virtual void Serialize( void* V, int64 Length ) override;
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;

	struct FObjectHeader
	{
		FObjectHeader()
		{
			Class = nullptr;
			Name = OuterPathName = NAME_None;
			Flags = RF_NoFlags;
		}

		bool IsValid() const
		{
			return Class != nullptr && !Name.IsNone();   
		}
		
		UClass* Class;
		FName Name;
		EObjectFlags Flags;
		FName OuterPathName;
	};

	UE_API FObjectHeader ReadObjectHeader(int64 InPosition = INDEX_NONE);
	UE_API FObjectHeader GetRootObjectHeader();

protected:

	void ReadVersions();

	TMap<FName, UClass*> ReadClasses;
	TMap<FName, UObject*> ReadObjects;
	TArray<UObject*> DeserializedObjects;
	TMap<int64,FName> OffsetToName;
};

#undef UE_API