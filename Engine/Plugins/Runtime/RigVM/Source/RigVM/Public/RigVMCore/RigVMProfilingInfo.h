// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#include "RigVMProfilingInfo.generated.h"

#define UE_API RIGVM_API

USTRUCT()
struct FRigVMInstructionVisitInfo
{
	GENERATED_BODY()

	FRigVMInstructionVisitInfo()
	{
	}

	void Reset()
	{
		InstructionVisitedDuringLastRun.Reset();
		InstructionVisitOrder.Reset();
		CallableVisitedDuringLastRun.Reset();
		CallableVisitOrder.Reset();
		FirstEntryEventInQueue = NAME_None;
	}

	inline void ResetInstructionVisitedDuringLastRun(int32 NewSize = 0) { InstructionVisitedDuringLastRun.Reset(NewSize); }
	inline void SetNumInstructionVisitedDuringLastRunZeroed(int32 Num) { InstructionVisitedDuringLastRun.SetNumZeroed(Num); }
	inline void SetInstructionVisitedDuringLastRun(int32 InstructionIndex)
	{
		if (InstructionVisitedDuringLastRun.IsValidIndex(InstructionIndex))
		{
			InstructionVisitedDuringLastRun[InstructionIndex]++;
		}
	}
	inline int32 GetInstructionVisitedCountDuringLastRun(int32 InstructionIndex) const { return InstructionVisitedDuringLastRun.IsValidIndex(InstructionIndex) ? InstructionVisitedDuringLastRun[InstructionIndex] : 0; }
	inline const TArray<int32>& GetInstructionVisitedCountDuringLastRun() const { return InstructionVisitedDuringLastRun; }

	inline void ResetInstructionVisitOrder(int32 NewSize = 0) { InstructionVisitOrder.Reset(NewSize); }
	inline void AddInstructionIndexToVisitOrder(int32 InstructionIndex) { InstructionVisitOrder.Add(InstructionIndex); }
	inline const TArray<int32>& GetInstructionVisitOrder() const { return InstructionVisitOrder; }

	inline void ResetCallableVisitedDuringLastRun(int32 NewSize = 0) { CallableVisitedDuringLastRun.Reset(NewSize); }
	inline void SetNumCallableVisitedDuringLastRunZeroed(int32 Num) { CallableVisitedDuringLastRun.SetNumZeroed(Num); }
	inline void SetCallableVisitedDuringLastRun(int32 CallableIndex)
	{
		if (CallableVisitedDuringLastRun.IsValidIndex(CallableIndex))
		{
			CallableVisitedDuringLastRun[CallableIndex]++;
		}
	}
	inline int32 GetCallableVisitedCountDuringLastRun(int32 CallableIndex) const { return CallableVisitedDuringLastRun.IsValidIndex(CallableIndex) ? CallableVisitedDuringLastRun[CallableIndex] : 0; }
	inline const TArray<int32>& GetCallableVisitedCountDuringLastRun() const { return CallableVisitedDuringLastRun; }

	inline void ResetCallableVisitOrder(int32 NewSize = 0) { CallableVisitOrder.Reset(NewSize); }
	inline void AddCallableIndexToVisitOrder(int32 CallableIndex) { CallableVisitOrder.Add(CallableIndex); }
	inline const TArray<int32>& GetCallableVisitOrder() const { return CallableVisitOrder; }
	
	inline const void SetFirstEntryEventInEventQueue(const FName& InFirstEventName) { FirstEntryEventInQueue = InFirstEventName; }
	inline const FName& GetFirstEntryEventInEventQueue() const { return FirstEntryEventInQueue; }

	UE_API void SetupInstructionTracking(int32 InInstructionCount, int32 InCallableCount);

private:

	// stores the number of times each instruction was visited
	TArray<int32> InstructionVisitedDuringLastRun;
	TArray<int32> InstructionVisitOrder;

	// stores the number of times each callable was visited
	TArray<int32> CallableVisitedDuringLastRun;
	TArray<int32> CallableVisitOrder;

	// A RigVMHost can run multiple events per evaluation, such as the Backward&Forward Solve Mode,
	// store the first event such that we know when to reset data for a new round of rig evaluation
	FName FirstEntryEventInQueue = NAME_None;

	friend class URigVMHost;
	friend struct FFirstEntryEventGuard;
	friend class FRigVMTrace;
};

struct FFirstEntryEventGuard
{
public:
	FFirstEntryEventGuard(FRigVMInstructionVisitInfo* InVisitInfo, const FName& InFirstEvent)
		: VisitInfo(InVisitInfo)
	{
		OldEntry = VisitInfo->FirstEntryEventInQueue;
		VisitInfo->FirstEntryEventInQueue = InFirstEvent;
	}

	~FFirstEntryEventGuard()
	{
		VisitInfo->FirstEntryEventInQueue = OldEntry;
	}

	FName OldEntry;
	FRigVMInstructionVisitInfo* VisitInfo;
};

USTRUCT()
struct FRigVMProfilingInfo
{
	GENERATED_BODY()

	FRigVMProfilingInfo()
	{
	}

	void Reset()
	{
		InstructionCyclesDuringLastRun.Reset();
		StartCycles = 0;
		OverallCycles = 0;
	}

	inline uint64 GetStartCycles() const {	return StartCycles;	}
	inline void SetStartCycles(uint64 InStartCycles) { StartCycles = InStartCycles; }

	inline uint64 GetOverallCycles() const { return OverallCycles; }
	inline void SetOverallCycles(uint64 Cycles) { OverallCycles = Cycles; }
	inline void AddOverallCycles(uint64 Cycles) { OverallCycles += Cycles; }

	inline void ResetInstructionCyclesDuringLastRun(int32 NewSize = 0) { InstructionCyclesDuringLastRun.Reset(NewSize); }
	inline uint64 GetInstructionCyclesDuringLastRun(int32 InstructionIndex) const
	{
		return InstructionCyclesDuringLastRun.IsValidIndex(InstructionIndex) ? InstructionCyclesDuringLastRun[InstructionIndex] : UINT64_MAX;
	}
	inline void SetInstructionCyclesDuringLastRun(int32 InstructionIndex, uint64 CyclesDuringLastRun)
	{
		if (InstructionCyclesDuringLastRun.IsValidIndex(InstructionIndex))
		{
			InstructionCyclesDuringLastRun[InstructionIndex] = CyclesDuringLastRun;
		}
	}
	inline void AddInstructionCyclesDuringLastRun(int32 InstructionIndex, uint64 CyclesDuringLastRun)
	{
		if (InstructionCyclesDuringLastRun.IsValidIndex(InstructionIndex))
		{
			InstructionCyclesDuringLastRun[InstructionIndex] += CyclesDuringLastRun;
		}
	}
	inline void InitInstructionCyclesDuringLastRunValues(int32 NewSize, uint64 DefaultValue)
	{
		InstructionCyclesDuringLastRun.SetNumUninitialized(NewSize);
		for (uint64& Value : InstructionCyclesDuringLastRun)
		{
			Value = DefaultValue;
		}
	}

	inline void ResetCallableCyclesDuringLastRun(int32 NewSize = 0) { CallableCyclesDuringLastRun.Reset(NewSize); }
	inline uint64 GetCallableCyclesDuringLastRun(int32 CallableIndex) const
	{
		return CallableCyclesDuringLastRun.IsValidIndex(CallableIndex) ? CallableCyclesDuringLastRun[CallableIndex] : UINT64_MAX;
	}
	inline void SetCallableCyclesDuringLastRun(int32 CallableIndex, uint64 CyclesDuringLastRun)
	{
		if (CallableCyclesDuringLastRun.IsValidIndex(CallableIndex))
		{
			CallableCyclesDuringLastRun[CallableIndex] = CyclesDuringLastRun;
		}
	}
	inline void AddCallableCyclesDuringLastRun(int32 CallableIndex, uint64 CyclesDuringLastRun)
	{
		if (CallableCyclesDuringLastRun.IsValidIndex(CallableIndex))
		{
			CallableCyclesDuringLastRun[CallableIndex] += CyclesDuringLastRun;
		}
	}
	inline void InitCallableCyclesDuringLastRunValues(int32 NewSize, uint64 DefaultValue)
	{
		CallableCyclesDuringLastRun.SetNumUninitialized(NewSize);
		for (uint64& Value : CallableCyclesDuringLastRun)
		{
			Value = DefaultValue;
		}
	}
	
	UE_API void SetupInstructionTracking(int32 InInstructionCount, int32 InCallableCount, bool bEnableProfiling);

	double GetLastExecutionMicroSeconds() const { return LastExecutionMicroSeconds; }
	void SetLastExecutionMicroSeconds(double InLastExecutionMicroSeconds) { LastExecutionMicroSeconds = InLastExecutionMicroSeconds; }

	UE_API void StartProfiling(bool bEnableProfiling);
	UE_API void StopProfiling();

private:

	// stores the cycles for each instruction
	TArray<uint64> InstructionCyclesDuringLastRun;
	TArray<uint64> CallableCyclesDuringLastRun;

	uint64 StartCycles = 0;
	uint64 OverallCycles = 0;

	double LastExecutionMicroSeconds = 0.0;

	friend class FRigVMTrace;
};

#undef UE_API
