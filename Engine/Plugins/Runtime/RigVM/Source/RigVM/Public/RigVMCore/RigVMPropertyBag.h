// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMPropertyPath.h"
#include "StructUtils/PropertyBag.h"
#include "EdGraph/EdGraphNode.h"
#include "RigVMPropertyBag.generated.h"

#define UE_API RIGVM_API

namespace UE::RigVM::RigVMCore::Private
{
	inline constexpr TCHAR BraceFormat[] = TEXT("(%s)");
	inline const FString EmptyBraces = TEXT("()");
}

//////////////////////////////////////////////////////////////////////////////
/// Property Management
//////////////////////////////////////////////////////////////////////////////

/**
 * The property description is used to provide all required information
 * to create a property for the memory storage class.
 */
struct FRigVMPropertyDescription
{
	/** Metadata string that sets the display name */
	static const FName MD_DisplayName;
	
public:
	
	// The guid of the property to create
	FGuid Guid;
	
	// The name of the property to create
	FName Name;

	// The property to base a new property off of
	const FProperty* Property;

	// The property flags that should be set. These will be used in the PropertyBag hash
	// Changing a flag in the PropertyBag will change it for all the instances that share the same PropertyBag hash
	EPropertyFlags PropertyFlags;

	// The complete CPP type to base a new property off of (for ex: 'TArray<TArray<FVector>>')
	FString CPPType;

	// The tail CPP Type object, for example the UScriptStruct for a struct 
	UObject* CPPTypeObject;

	// A list of containers to use for this property, for example [Array, Array]
	TArray<EPinContainerType> Containers;

	// The default value to use for this property (for example: '(((X=1.000000, Y=2.000000, Z=3.000000)))')
	FString DefaultValue;

#if WITH_EDITOR
	// All the metadata on the PropertyBag. This will be used in the PropertyBag hash.
	TMap<FName, FString> MetaData;
#endif

	// Default constructor
	FRigVMPropertyDescription()
		: Name(NAME_None)
		, Property(nullptr)
		, CPPType()
		, CPPTypeObject(nullptr)
		, Containers()
		, DefaultValue()
	{}

	// Constructor from an existing property
	RIGVM_API FRigVMPropertyDescription(const FProperty* InProperty, const FString& InDefaultValue, const FName& InName = NAME_None, const bool bAllowSpacesInName = false);

	// Constructor from complete data
	RIGVM_API FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const bool bAllowSpacesInName = false);

#if WITH_EDITOR
	// Constructor from complete data
	RIGVM_API FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const EPropertyFlags InPropertyFlags, const TMap<FName, FString>& InMetaData, const bool bAllowSpacesInName = false);
#else
	// Constructor from complete data
	RIGVM_API FRigVMPropertyDescription(const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const EPropertyFlags InPropertyFlags, const bool bAllowSpacesInName = false);
#endif

	// Returns a sanitized, valid name to use for a new property 
	static RIGVM_API FName SanitizeName(const FName& InName, const bool bAllowSpaces = false);

	// Sanitizes a name using a string ref, creating a valid name to use for a new property 
	static RIGVM_API void SanitizeName(FString& InString, const bool bAllowSpaces = false);

	// Sanitize the name of this description in line
	RIGVM_API void SanitizeName(const bool bAllowSpaces = false);

	// Returns true if this property description is valid
	bool IsValid() const { return !Name.IsNone(); }

	// Returns the CPP type of the tail property, for ex: '[2].Translation' it is 'FVector'
	RIGVM_API FString GetTailCPPType() const;

private:
	
	static const inline TCHAR* ArrayPrefix = TEXT("TArray<");
	static const inline TCHAR* MapPrefix = TEXT("TMap<");
	static const inline TCHAR* ContainerSuffix = TEXT(">");
};

USTRUCT()
struct FRigVMPropertyBag : public FInstancedPropertyBag
{
	GENERATED_BODY()

	UE_API FRigVMPropertyBag();

	UE_API bool Serialize(FArchive& Ar);
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API void GetUserDefinedDependencies(TArray<const UObject*>& OutDependencies) const;
	UE_API void GetRequiredPlugins(TArray<FString>& OutPlugins) const;

	friend FArchive& operator<<(FArchive& Ar, FRigVMPropertyBag& P)
	{
		P.Serialize(Ar);
		return Ar;
	}

	static bool IsClassOf(const FRigVMPropertyBag* InElement)
	{
		return true;
	}

	//---------------------------------------------------------------------------

	/**
	 * Adds properties to the storage. If property of same name already exists, it will be replaced with the new type.
	 * Numeric property values will be converted if possible, when a property's type changes.
	 * @param InPropertyDescriptions : Descriptors of new properties to add.
	 */
	UE_API void AddProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions);

	/**
	 * Remove property from the storage. 
	 * @param InPropertyName : The name of the property to remove
	 * @return Success if the property was successfully removed
	 */
	UE_API EPropertyBagAlterationResult RemovePropertyByName(const FName& InPropertyName);


	// Returns the number of properties stored in this instance
	int32 Num() const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties.Num();
	}

	// Returns true if a provided property index is valid
	bool IsValidIndex(int32 InIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties.IsValidIndex(InIndex);
	}


	// Returns the properties provided by this instance
	const TArray<const FProperty*>& GetProperties() const 
	{
		return LinkedProperties;
	}

	// Returns the index of a property given the property itself
	UE_API int32 GetPropertyIndex(const FProperty* InProperty) const;

	// Returns the index of a property given its name
	UE_API int32 GetPropertyIndexByName(const FName& InName) const;

	// Returns a property given its index
	const FProperty* GetProperty(int32 InPropertyIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		return Properties[InPropertyIndex];
	}

	// Returns a property given its name (or nullptr if the name wasn't found)
	UE_API FProperty* FindPropertyByName(const FName& InName) const;

	// Returns the raw memory storage pointer
	UE_API void* GetContainerPtr() const;

	// Returns true if the property at a given index is a TArray
	bool IsArray(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FArrayProperty>();
	}

	// Returns true if the property at a given index is a TMap
	bool IsMap(int32 InPropertyIndex) const
	{
		return GetProperty(InPropertyIndex)->IsA<FMapProperty>();
	}

	//---------------------------------------------------------------------------

	// Returns the memory for a property given its index
	template<typename T>
	T* GetData(int32 InPropertyIndex) 
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		check(Properties.IsValidIndex(InPropertyIndex));
		return Properties[InPropertyIndex]->ContainerPtrToValuePtr<T>(GetMutableValue().GetMemory());
	}
	template<typename T>
	const T* GetData(int32 InPropertyIndex) const
	{
		const TArray<const FProperty*>& Properties = GetProperties();
		check(Properties.IsValidIndex(InPropertyIndex));
		return Properties[InPropertyIndex]->ContainerPtrToValuePtr<T>(GetValue().GetMemory());
	}

	// Returns the memory for a property given its name (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if (PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex);
	}

	// Returns the mutable memory for a given property (or nullptr if does not belong to this storage)
	template<typename T>
	T* GetData(const FProperty* Property)
	{
		if (const int32 PropertyIndex = GetPropertyIndex(Property); PropertyIndex != INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		return nullptr;
	}
	template<typename T>
	const T* GetData(const FProperty* Property) const
	{
		if (const int32 PropertyIndex = GetPropertyIndex(Property); PropertyIndex != INDEX_NONE)
		{
			return GetData<T>(PropertyIndex);
		}

		return nullptr;
	}

	// Returns the memory for a property given its index and a matching property path
	template<typename T>
	T* GetData(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		const FProperty* Property = GetProperty(InPropertyIndex);
		return InPropertyPath.GetData<T>(GetData<uint8>(InPropertyIndex), Property);
	}

	// Returns the memory for a property given its name and a matching property path (or nullptr)
	template<typename T>
	T* GetDataByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		if (PropertyIndex == INDEX_NONE)
		{
			return nullptr;
		}
		return GetData<T>(PropertyIndex, InPropertyPath);
	}

	// Returns the ref of an element stored at a given property index
	template<typename T>
	T& GetRef(int32 InPropertyIndex)
	{
		return *GetData<T>(InPropertyIndex);
	}

	// Returns the ref of an element stored at a given property name (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName)
	{
		return *GetDataByName<T>(InName);
	}

	// Returns the ref of an element stored at a given property index and a property path
	template<typename T>
	T& GetRef(int32 InPropertyIndex, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetData<T>(InPropertyIndex, InPropertyPath);
	}

	// Returns the ref of an element stored at a given property name and a property path (throws if name is invalid)
	template<typename T>
	T& GetRefByName(const FName& InName, const FRigVMPropertyPath& InPropertyPath)
	{
		return *GetDataByName<T>(InName, InPropertyPath);
	}

	//---------------------------------------------------------------------------

	// Returns the exported text for a given property index
	UE_API FString GetDataAsString(int32 InPropertyIndex, int32 PortFlags = PPF_None) const;

	// Returns the exported text for given property name 
	FString GetDataAsStringByName(const FName& InName, int32 PortFlags = PPF_None) const
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsString(PropertyIndex, PortFlags);
	}

	// Returns the exported text for a given property index
	UE_API FString GetDataAsStringSafe(int32 InPropertyIndex, int32 PortFlags = PPF_None) const;

	// Returns the exported text for given property name 
	FString GetDataAsStringByNameSafe(const FName& InName, int32 PortFlags = PPF_None) const
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return GetDataAsStringSafe(PropertyIndex, PortFlags);
	}

	// Sets the content of a property by index given an exported string. Returns true if succeeded
	UE_API bool SetDataFromString(int32 InPropertyIndex, const FString& InValue);

	// Sets the content of a property by name given an exported string. Returns true if succeeded
	bool SetDataFromStringByName(const FName& InName, const FString& InValue)
	{
		const int32 PropertyIndex = GetPropertyIndexByName(InName);
		return SetDataFromString(PropertyIndex, InValue);
	}

	//---------------------------------------------------------------------------
	UE_API void Refresh();

	/**
	 * Copies the content of a source property and memory into the memory of a target property
	 * @param InTargetProperty The target property to copy into
	 * @param InTargetPtr The memory of the target value to copy into
	 * @param InSourceProperty The source property to copy from
	 * @param InSourcePtr The memory of the source value to copy from
	 * @return true if the copy operations was successful
	 */
	static UE_API bool CopyProperty(
		const FProperty* InTargetProperty,
		uint8* InTargetPtr,
		const FProperty* InSourceProperty,
		const uint8* InSourcePtr);


	// Helper function to generate a property bag desc from a FRigVMPropertyDescription
	static UE_API FPropertyBagPropertyDesc GeneratePropertyBagDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, bool bComputeId = true);

protected:

	// A cached list of all linked properties (created by RefreshLinkedProperties)
	TArray<const FProperty*> LinkedProperties;


	mutable int32 CachedMemoryHash = 0;

	static UE_API const TArray<const FProperty*> EmptyProperties;

	void ForEachObjectDependency(const TFunction<void(const UObject*)>& InCallBack) const;

	UE_API void RefreshLinkedProperties();

	UE_API void SetDefaultValues(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions);

	// Set default values on the last properties added
	UE_API void SetDefaultValuesOnTailProperties(const TArray<FRigVMPropertyDescription>& InPropertyDescriptions);

	static UE_API bool GetPropertyTypeDataFromVMDescriptor(const FRigVMPropertyDescription& RigVMDescriptor, EPropertyBagPropertyType& OutBagPropertyType, FPropertyBagContainerTypes& OutBagContainerTypes);
};

template<> struct TStructOpsTypeTraits<FRigVMPropertyBag> : public TStructOpsTypeTraitsBase2<FRigVMPropertyBag>
{
	enum
	{
		WithSerializer = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
