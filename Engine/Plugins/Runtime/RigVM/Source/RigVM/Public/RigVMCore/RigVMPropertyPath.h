// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

#define UE_API RIGVM_API

class FProperty;

/**
 * The RigVMPropertyPathDescription is used to provide all of the information
 * necessary to describe a property path for creating it. This is used by
 * the RigVMCompiler, which collects all descriptions first and then passes
 * them onto the RigVMStorageGeneratorClass for construction. Furthermore
 * the RigVMPropertyPathDescription is used to serialize property paths and
 * recreate them on load.
 */
struct FRigVMPropertyPathDescription
{
	// The index of the property this property path belongs to.
	int32 PropertyIndex;

	// The CPP type of the head property.
	// For Transform.Translation.X the HeadCPPType is 'FTransform' 
	FString HeadCPPType;

	// The segment path of the properties below the head property.
	// For Transform.Translation.X the SegmentPath is 'Translation.X' 
	FString SegmentPath;

	// Default constructor
	FRigVMPropertyPathDescription()
		: PropertyIndex(INDEX_NONE)
		, HeadCPPType()
		, SegmentPath()
	{}

	// Constructor from complete data
	FRigVMPropertyPathDescription(int32 InPropertyIndex, const FString& InHeadCPPType, const FString& InSegmentPath)
		: PropertyIndex(InPropertyIndex)
		, HeadCPPType(InHeadCPPType)
		, SegmentPath(InSegmentPath)
	{}

	// returns true if this property path is valid
	bool IsValid() const { return PropertyIndex != INDEX_NONE && !HeadCPPType.IsEmpty() && !SegmentPath.IsEmpty(); }

	// Archive operator for serialization
	friend FArchive& operator<<(FArchive& Ar, FRigVMPropertyPathDescription& Path)
	{
		Ar << Path.PropertyIndex;
		Ar << Path.HeadCPPType;
		Ar << Path.SegmentPath;
		return Ar;
	}
};

/**
 * The RigVMPropertyPathSegmentType is used to differentiate
 * segments under a property path.
 */
enum ERigVMPropertyPathSegmentType
{
	StructMember, // The segment represents a member of a struct
	ArrayElement, // The segment represents an element of an array
	MapValue // The segments represents a value within a map. Typically maps supported use FName as the key.
};

/**
 * The RigVMPropertyPathSegment represents a single step needed to go from
 * the head property of a property path to the tail.
 */
struct FRigVMPropertyPathSegment
{
	// The type of segment / step.
	ERigVMPropertyPathSegmentType Type;

	// The name of the segment - can be the name of the struct member of key for a map element.
	FName Name;

	// The index of the segment - used to represent the array index for array elements.
	int32 Index;

	// The property for this segment - on the chain towards the tail property. 
	const FProperty* Property;
};

/**
 * The RigVMPropertyPath is used to access a memory pointer for a tail property
 * given the memory of the head property. A property path allows to traverse
 * struct members, array elements and map values.
 * For example given an array of transforms the property path
 * '[2].Translation.X'
 * is represented as three segments (1 array element and 2 struct members).
 */
class FRigVMPropertyPath
{
public:

	// Default constructor
	UE_API FRigVMPropertyPath();

	/**
	 * Constructor from complete data
	 * @param InProperty The head property to base this property path off of
	 * @param InSegmentPath A . separated string providing all of the segments of this path
	 */
	UE_API FRigVMPropertyPath(const FProperty* InProperty, const FString& InSegmentPath);

	// Returns the property path as a string (the sanitized SegmentPath)
	const FString& ToString() const { return Path; }

	// Returns a descriptive name
	UE_API FString GetName() const;

	// Returns the number of segments
	int32 Num() const { return Segments.Num(); }

	// Returns true if the property path is valid (has any segments)
	bool IsValid() const { return Num() > 0; }

	// Returns true if the property is empty (no segments)
	bool IsEmpty() const { return Num() == 0; }

	// Index based access operator for a segment
	const FRigVMPropertyPathSegment& operator[](int32 InIndex) const { return Segments[InIndex]; }

	// Returns the segments of this property path
	const TArray<FRigVMPropertyPathSegment>& GetSegments() const { return Segments; }

	bool operator ==(const FRigVMPropertyPath& Other) const
	{
		return GetTypeHash(this) == GetTypeHash(Other);
	}

	bool operator !=(const FRigVMPropertyPath& Other) const
	{
		return GetTypeHash(this) != GetTypeHash(Other);
	}

	bool operator >(const FRigVMPropertyPath& Other) const
	{
		return GetTypeHash(this) > GetTypeHash(Other);
	}

	bool operator <(const FRigVMPropertyPath& Other) const
	{
		return GetTypeHash(this) < GetTypeHash(Other);
	}

	/**
	 * Accessor for traversing the path and returning the memory of the tail
	 * @param InPtr The memory of the head property
	 * @param InProperty The head property to traverse off of
	 */
	template<typename T>
	T* GetData(uint8* InPtr, const FProperty* InProperty) const
	{
		return (T*)GetData_Internal(InPtr, InProperty);
	}

	// Returns a unique hash for the property path
	friend uint32 GetTypeHash(const FRigVMPropertyPath& InPropertyPath)
	{
		if(InPropertyPath.IsEmpty())
		{
			return 0;
		}
		
		return HashCombine(
			GetTypeHash(InPropertyPath[0].Property),
			GetTypeHash(InPropertyPath.ToString())
		);
	}

	// Return the owner's native type. the owner can be a property or a struct.
	UE_API FString GetHeadCPPType() const;

	// Returns the property of the last segment
	UE_API const FProperty* GetTailProperty() const;

	// Static empty property path (used for comparisons)
	static UE_API FRigVMPropertyPath Empty;

private:

	// Internal traversal method
	UE_API uint8* GetData_Internal(uint8* InPtr, const FProperty* InProperty) const;

	// Stored string representation of the segment path
	FString Path;

	// Resolved segments of the property path
	TArray<FRigVMPropertyPathSegment> Segments;
};

#undef UE_API
