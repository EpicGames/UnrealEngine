// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "RigVMRegistryHandle.h"
#include "RigVMCore/RigVMTraits.h"
#include "RigVMDispatchFactory.h"
#include "RigVMFunction.h"
#include "RigVMTemplate.h"
#include "RigVMTypeIndex.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnum.h"
#include "Templates/Models.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/GCObject.h"

#define UE_API RIGVM_API

class FProperty;
class IPlugin;
class UObject;
struct FRigVMDispatchFactory;

struct FRigVMRegistry_RWLock;
typedef FRigVMRegistry_RWLock FRigVMRegistry;

extern UE_API TAutoConsoleVariable<bool> CVarRigVMEnableLocalizedRegistry;

/**
 * The FRigVMRegistry is used to manage all known function pointers
 * for use in the RigVM. The Register method is called automatically
 * when the static struct is initially constructed for each USTRUCT
 * hosting a RIGVM_METHOD enabled virtual function.
 * 
 * Inheriting from FGCObject to ensure that all type objects cannot be GCed
 */
struct FRigVMRegistry_NoLock : public FGCObject, public TSharedFromThis<FRigVMRegistry_NoLock>
{
public:

	enum ELockType : uint8
	{
		LockType_Read,
		LockType_Write,
		LockType_Invalid
	};

	// returns true for the singleton global registry
	bool IsGlobalRegistry() const { return bIsGlobalRegistry; }

	// returns the handle referring to this hierarchy
	UE_API const FRigVMRegistryHandle& GetHandle_NoLock() const;
	UE_API FRigVMRegistryHandle& GetHandle_NoLock();

	// Constructor for a localized, independent registry
	UE_API static TSharedRef<FRigVMRegistry_NoLock> CreateLocalizedRegistry();

	// Constructor for a localized, independent registry based on an existing registry
	UE_API static TSharedRef<FRigVMRegistry_NoLock> CloneLocalizedRegistry(FRigVMRegistry_NoLock* InRegistry);

	DECLARE_MULTICAST_DELEGATE(FOnRigVMRegistryChanged);

	UE_API virtual ~FRigVMRegistry_NoLock() override;
	
	// FGCObject overrides
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	
	// Registers a function given its name.
	// The name will be the name of the struct and virtual method,
	// for example "FMyStruct::MyVirtualMethod"
	UE_API virtual void Register_NoLock(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct = nullptr, const TArray<FRigVMFunctionArgument>& InArguments = TArray<FRigVMFunctionArgument>());
	
	// Register many functions, as if calling Register_NoLock in a loop
	UE_API virtual void RegisterCompiledInStruct_NoLock(UScriptStruct* InStruct, TConstArrayView<FRigVMCompiledInFunction> InFunctions);
	
	// This keeps the function index stable - but assigns an invalid function
	UE_API virtual bool RemoveFunction_NoLock(int32 InFunctionIndex);

	// Registers a dispatch factory given its struct.
	UE_API virtual const FRigVMDispatchFactory* RegisterFactory_NoLock(UScriptStruct* InFactoryStruct, const TArray<FRigVMTemplateArgumentInfo>& InArgumentInfos = {});

	// Unregisters a factory given its script struct.
	// This keeps the template index stable
	UE_API virtual bool RemoveFactory_NoLock(UScriptStruct* InFactoryStruct);

	// Unregisters a template given its index.
	// This keeps the template index stable - but assigns an invalid template
	UE_API virtual bool RemoveTemplate_NoLock(int32 InTemplateIndex);

	// Register a predicate contained in the input struct
	UE_API virtual void RegisterPredicate_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const TArray<FRigVMFunctionArgument>& InArguments);

	// How to register an object's class when passed to RegisterObjectTypes
	enum class ERegisterObjectOperation
	{
		Class,

		ClassAndParents,

		ClassAndChildren,
	};

	// Register a set of allowed object types
	UE_API virtual void RegisterObjectTypes_NoLock(TConstArrayView<TPair<UClass*, ERegisterObjectOperation>> InClasses);

	// Register a set of allowed struct types
	UE_API virtual void RegisterStructTypes_NoLock(TConstArrayView<UScriptStruct*> InStructs);

	// Refreshes the list and finds the function pointers
	// based on the names.
	UE_API virtual void RefreshEngineTypes_NoLock();

	// Refreshes the registered functions and dispatches.
	UE_API virtual bool RefreshFunctionsAndDispatches_NoLock();

	// Refreshes the list and finds the function pointers
	// based on the names.
	UE_API virtual void RefreshEngineTypesIfRequired_NoLock();
	
	// Update the registry when old types are being unloaded
	UE_API virtual bool OnCleanupUnloadingObjects_NoLock(const TArrayView<UObject*> InObjects);

	// Update the registry when types are renamed
	UE_API virtual void OnAssetRenamed_NoLock(const FAssetData& InAssetData, const FString& InOldObjectPath);
	
	// Update the registry when old types are removed
	UE_API virtual bool OnAssetRemoved_NoLock(const FAssetData& InAssetData);

	// Returns the prefixes for all modules within a plugin
	UE_API static TArray<FString> GetPluginModulePrefixes(const IPlugin& InPlugin);

	// Returns true if a given object is part of a plugin
	UE_API static bool IsWithinPlugin(const IPlugin& InPlugin, const UObject* InObject);

	// Returns true if a given object is part of a plugin object path
	UE_API static bool IsWithinPlugin(const TArray<FString>& InModulePrefixes, const UObject* InObject);

	// May add factories and unit functions declared in the plugin 
	UE_API virtual bool OnPluginLoaded_NoLock(IPlugin& InPlugin);

	// Removes all types associated with a plugin that's being unloaded. 
	UE_API virtual bool OnPluginUnloaded_NoLock(IPlugin& InPlugin);

	// Removes all types associated with modules that are being unloaded.
	UE_API virtual bool OnModulesUnloaded_NoLock(TConstArrayView<FName> ModuleNames);
	
	// Update the registry when new types are added to the attribute system so that they can be selected
	// on Attribute Nodes
	UE_API virtual void OnAnimationAttributeTypesChanged_NoLock(const UScriptStruct* InStruct, bool bIsAdded);

	// Clear the registry
	UE_API virtual void Reset_NoLock();

	// Serializes the registry to an archive. This throws for the global registry.
	UE_API void Serialize_NoLock(FArchive& Ar, const UObject* Owner = nullptr);

	// Adds a type if it doesn't exist yet and returns its index.
	// This function is thread-safe
	UE_API virtual TRigVMTypeIndex FindOrAddType_NoLock(const FRigVMTemplateArgumentType& InType, bool bForce = false);

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	UE_API virtual bool RemoveType_NoLock(const FSoftObjectPath& InObjectPath);

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	UE_API virtual bool RemoveType_NoLock(TRigVMTypeIndex InTypeIndex);

	// Returns the type index given a type
	UE_API virtual TRigVMTypeIndex GetTypeIndex_NoLock(const FRigVMTemplateArgumentType& InType) const;

	// Returns the type index given a cpp type and a type object
	virtual TRigVMTypeIndex GetTypeIndex_NoLock(const FName& InCPPType, UObject* InCPPTypeObject) const
	{
		return GetTypeIndex_NoLock(FRigVMTemplateArgumentType(InCPPType, InCPPTypeObject));
	}

	// Returns the type index given an enum
	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
	>
	TRigVMTypeIndex GetTypeIndex_NoLock(bool bAsArray = false) const
	{
		FRigVMTemplateArgumentType Type(StaticEnum<T>());
		if(bAsArray)
		{
			Type.ConvertToArray();
		}
		return GetTypeIndex_NoLock(Type);
	}

	// Returns the type index given a struct
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
	>
	TRigVMTypeIndex GetTypeIndex_NoLock(bool bAsArray = false) const
	{
		FRigVMTemplateArgumentType Type(TBaseStructure<T>::Get());
		if(bAsArray)
		{
			Type.ConvertToArray();
		}
		return GetTypeIndex_NoLock(Type);
	}

	// Returns the type index given a struct
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	TRigVMTypeIndex GetTypeIndex_NoLock(bool bAsArray = false) const
	{
		FRigVMTemplateArgumentType Type(T::StaticStruct());
		if(bAsArray)
		{
			Type.ConvertToArray();
		}
		return GetTypeIndex_NoLock(Type);
	}

	// Returns the type index given an object
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	TRigVMTypeIndex GetTypeIndex_NoLock(bool bAsArray = false) const
	{
		FRigVMTemplateArgumentType Type(T::StaticClass(), RigVMTypeUtils::EClassArgType::AsObject);
		if(bAsArray)
		{
			Type.ConvertToArray();
		}
		return GetTypeIndex_NoLock(Type);
	}

	// Returns the type given its index
	UE_API virtual const FRigVMTemplateArgumentType& GetType_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns the number of types
	virtual int32 NumTypes_NoLock() const { return Types.Num(); }

	// Returns the type given only its cpp type
	UE_API virtual const FRigVMTemplateArgumentType& FindTypeFromCPPType_NoLock(const FString& InCPPType) const;

	// Returns the type index given only its cpp type
	UE_API virtual TRigVMTypeIndex GetTypeIndexFromCPPType_NoLock(const FString& InCPPType) const;

	// Returns true if the type is an array
	UE_API virtual bool IsArrayType_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns true if the type is an execute type
	UE_API virtual bool IsExecuteType_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Converts the given execute context type to the base execute context type
	UE_API virtual bool ConvertExecuteContextToBaseType_NoLock(TRigVMTypeIndex& InOutTypeIndex) const;

	// Returns the dimensions of the array 
	UE_API virtual int32 GetArrayDimensionsForType_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns true if the type is a wildcard type
	UE_API virtual bool IsWildCardType_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns true if the types can be matched.
	UE_API virtual bool CanMatchTypes_NoLock(TRigVMTypeIndex InTypeIndexA, TRigVMTypeIndex InTypeIndexB, bool bAllowFloatingPointCasts) const;

	// Returns the list of compatible types for a given type
	UE_API virtual const TArray<TRigVMTypeIndex>& GetCompatibleTypes_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns all compatible types given a category
	UE_API virtual const TArray<TRigVMTypeIndex>& GetTypesForCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory) const;

	// Returns the type index of the array matching the given element type index
	UE_API virtual TRigVMTypeIndex GetArrayTypeFromBaseTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns the type index of the element matching the given array type index
	UE_API virtual TRigVMTypeIndex GetBaseTypeFromArrayTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const;

	// Returns true if the given type is specifically allowed 
	UE_API virtual bool IsSpecificallyAllowedType_NoLock(const UObject* InObject) const;

	// Sets a given type as specifically allowed 
	UE_API virtual void AddSpecificallyAllowedType_NoLock(const UObject* InObject);

	// Returns the function given its name (or nullptr)
	UE_API virtual const FRigVMFunction* FindFunction_NoLock(const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InTypeResolver = FRigVMUserDefinedTypeResolver()) const;

	// Returns the function given its backing up struct and method name
	UE_API virtual const FRigVMFunction* FindFunction_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InResolvalInfo = FRigVMUserDefinedTypeResolver()) const;

	// Returns all current RigVM functions
	UE_API virtual const TChunkedArray<FRigVMFunction>& GetFunctions_NoLock() const;

	// Returns a template pointer given its notation (or nullptr)
	UE_API virtual const FRigVMTemplate* FindTemplate_NoLock(const FName& InNotation, bool bIncludeDeprecated = false, bool bRemapToCoreDispatch = true) const;

	// Returns all current RigVM functions
	UE_API virtual const TChunkedArray<FRigVMTemplate>& GetTemplates_NoLock() const;

	// Defines and retrieves a template given its arguments
	UE_API virtual const FRigVMTemplate* GetOrAddTemplateFromArguments_NoLock(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates);

	// Adds a new template given its arguments
	UE_API virtual const FRigVMTemplate* AddTemplateFromArguments_NoLock(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates);

	// Returns a dispatch factory given its name (or nullptr)
	UE_API virtual FRigVMDispatchFactory* FindDispatchFactory_NoLock(const FName& InFactoryName) const;

	// Returns a dispatch factory given its static struct (or nullptr)
	UE_API virtual FRigVMDispatchFactory* FindOrAddDispatchFactory_NoLock(UScriptStruct* InFactoryStruct);

	// Returns a dispatch factory given its static struct (or nullptr)
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FRigVMDispatchFactory* FindOrAddDispatchFactory_NoLock()
	{
		return FindOrAddDispatchFactory_NoLock(T::StaticStruct());
	}

	// Returns a dispatch factory's singleton function name if that exists
	UE_API virtual FString FindOrAddSingletonDispatchFunction_NoLock(UScriptStruct* InFactoryStruct);

	// Returns a dispatch factory's singleton function name if that exists
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FString FindOrAddSingletonDispatchFunction_NoLock()
	{
		return FindOrAddSingletonDispatchFunction_NoLock(T::StaticStruct());
	}

	// Returns all dispatch factories
	UE_API virtual const TArray<FRigVMDispatchFactory*>& GetFactories_NoLock() const;

	// Given a struct name, return the predicates
	UE_API virtual const TArray<FRigVMFunction>* GetPredicatesForStruct_NoLock(const FName& InStructName) const;

	static UE_API const TArray<UScriptStruct*>& GetMathTypes();


	// Returns a unique hash per type index
	UE_API virtual uint32 GetHashForType_NoLock(TRigVMTypeIndex InTypeIndex) const;
	UE_API virtual uint32 GetHashForScriptStruct_NoLock(const UScriptStruct* InScriptStruct, bool bCheckTypeIndex = true) const;
	UE_API virtual uint32 GetHashForStruct_NoLock(const UStruct* InStruct) const;
	UE_API virtual uint32 GetHashForEnum_NoLock(const UEnum* InEnum, bool bCheckTypeIndex = true) const;
	UE_API virtual uint32 GetHashForProperty_NoLock(const FProperty* InProperty) const;

	UE_API virtual void RebuildRegistry_NoLock(); 

	static inline const FLazyName TemplateNameMetaName = FLazyName(TEXT("TemplateName"));

	static UE_API void OnEngineInit();

protected:


	UE_API FRigVMRegistry_NoLock(bool InIsGlobalRegistry);

	// disable copy constructor
	FRigVMRegistry_NoLock(const FRigVMRegistry_NoLock&);
	// disable assignment operator
	FRigVMRegistry_NoLock& operator= (const FRigVMRegistry_NoLock &InOther) = delete;

	struct FTypeInfo
	{
		FTypeInfo()
			: Type()
			, BaseTypeIndex(INDEX_NONE)
			, ArrayTypeIndex(INDEX_NONE)
			, bIsArray(false)
			, bIsExecute(false)
			, Hash(UINT32_MAX)
		{}
		
		FRigVMTemplateArgumentType Type;
		TRigVMTypeIndex BaseTypeIndex;
		TRigVMTypeIndex ArrayTypeIndex;
		bool bIsArray;
		bool bIsExecute;
		mutable uint32 Hash;

		void Serialize_NoLock(FArchive& Ar);

		bool IsUnknownType() const
		{
			return Type.IsUnknownType();
		}
	};

	// Initialize the base types
	UE_API void InitializeBaseTypes_NoLock();

	static EObjectFlags DisallowedFlags()
	{
		return RF_BeginDestroyed | RF_FinishDestroyed;
	}

	static EObjectFlags NeededFlags()
	{
		return RF_Public;
	}

	UE_API bool OnModulesUnloadedInternal(const TArray<FString>& ModulePrefixes);

	UE_API bool IsAllowedType_NoLock(const FProperty* InProperty) const;
	UE_API bool IsAllowedType_NoLock(const UEnum* InEnum) const;
	UE_API bool IsAllowedType_NoLock(const UStruct* InStruct) const;
	UE_API bool IsAllowedType_NoLock(const UClass* InClass) const;
	static UE_API bool IsTypeOfByName(const UObject* InObject, const FName& InName);

	UE_API void RegisterTypeInCategory_NoLock(const FRigVMTemplateArgument::ETypeCategory InCategory, const TRigVMTypeIndex InTypeIndex);
	UE_API void PropagateTypeAddedToCategory_NoLock(const FRigVMTemplateArgument::ETypeCategory InCategory, const TRigVMTypeIndex InTypeIndex);
	UE_API void RemoveTypeInCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory, TRigVMTypeIndex InTypeIndex);

	// the handle storing this registry
	mutable FRigVMRegistryHandle ThisHandle;
	
	// true if this registry is the globally shared registry 
	bool bIsGlobalRegistry;

	// memory for all (known) types
	TArray<FTypeInfo> Types;
	TMap<FRigVMTemplateArgumentType, TRigVMTypeIndex> TypeToIndex;

	// memory for all functions
	// We use TChunkedArray because we need the memory locations to be stable, since we only ever add and never remove.
	TChunkedArray<FRigVMFunction> Functions;

	// memory for all non-deprecated templates
	TChunkedArray<FRigVMTemplate> Templates;

	// memory for all deprecated templates
	TChunkedArray<FRigVMTemplate> DeprecatedTemplates;

	// memory for all dispatch factories
	TArray<FRigVMDispatchFactory*> Factories;

	// name lookup for factories
	TMap<FName, FRigVMDispatchFactory*> FactoryNameToFactory;

	// struct lookup for factories
	TMap<const UScriptStruct*, FRigVMDispatchFactory*> FactoryStructToFactory;

	// name lookup for functions
	TMap<FName, int32> FunctionNameToIndex;

	// Previous value for name lookups - this can happen to functions which get deregistered
	TMap<FName, int32> PreviousFunctionNameToIndex;

	// lookup all the predicate functions of this struct
	TMap<FName, TArray<FRigVMFunction>> StructNameToPredicates;

	// name lookup for non-deprecated templates
	TMap<FName, int32> TemplateNotationToIndex;

	// previous name lookup for templates which got removed
	TMap<FName, int32> PreviousTemplateNotationToIndex;

	// name lookup for deprecated templates
	TMap<FName, int32> DeprecatedTemplateNotationToIndex;

	// Maps storing the default types per type category
	TMap<FRigVMTemplateArgument::ETypeCategory, TArray<TRigVMTypeIndex>> TypesPerCategory;

	// Lookup per type category to know which template to keep in sync
	TMap<FRigVMTemplateArgument::ETypeCategory, TArray<int32>> TemplatesPerCategory;

	// This type index triple represents the single value, array value, and 2d array value type indices.
	using TRigVMTypeIndexTriple = TTuple<TRigVMTypeIndex, TRigVMTypeIndex, TRigVMTypeIndex>; 

	// Name lookup for types since they can be deleted.
	// When that happens, it won't be safe to reload deleted assets so only type names are reliable
	TMap<FSoftObjectPath, TRigVMTypeIndexTriple> SoftObjectPathToTypeIndex;
	
	// All allowed classes
	TSet<TObjectPtr<const UClass>> AllowedClasses;

	// All allowed structs
	TSet<TObjectPtr<const UScriptStruct>> AllowedStructs;
	
	// If this is true the registry is currently refreshing all types
	// and we want to avoid propagating types to the templates
	bool bAvoidTypePropagation;

	// This is true if the engine has ever refreshed the engine types
	bool bEverRefreshedEngineTypes;

	// This is true if the dispatch factories and functions have been greedily loaded once during engine init
	bool bEverRefreshedDispatchFactoriesAfterEngineInit;

	friend struct FRigVMStruct;
	friend struct FRigVMTemplate;
	friend struct FRigVMTemplateArgument;
	friend struct FRigVMDispatchFactory;
};

/**
 * The FRigVMRegistry is used to manage all known function pointers
 * for use in the RigVM. The Register method is called automatically
 * when the static struct is initially constructed for each USTRUCT
 * hosting a RIGVM_METHOD enabled virtual function.
 * 
 * Inheriting from FGCObject to ensure that all type objects cannot be GCed
 */
struct FRigVMRegistry_RWLock final : public FRigVMRegistry_NoLock
{
	typedef FRigVMRegistry_NoLock Super;
	
protected:

	class FConditionalScopeLock : public FRigVMRegistryHandleLock
	{
	public:
		UE_NODISCARD_CTOR explicit FConditionalScopeLock(const FRigVMRegistry_RWLock& InRegistry, ELockType InLockType, bool bInLockEnabled = true)
			: FRigVMRegistryHandleLock(&InRegistry, bInLockEnabled)
			, Registry(const_cast<FRigVMRegistry_RWLock*>(&InRegistry))
			, DesiredLockType(InLockType)
		{
			if (!bLockEnabled)
			{
				return;
			}

			checkSlow(DesiredLockType == LockType_Read || DesiredLockType == LockType_Write);

			// First, take the actual underlying lock.
			if (DesiredLockType == LockType_Read)
			{
				// Only attempt to lock if we're not already holding a write lock.
				if (Registry->WriteLockOwnerThread == FPlatformTLS::GetCurrentThreadId())
				{
					bLockEnabled = false;
					return;
				}
				
				Registry->Lock.ReadLock();
			}
			else if (DesiredLockType == LockType_Write)
			{
				Registry->Lock.WriteLock();
				UE_AUTORTFM_OPEN
				{
					Registry->WriteLockOwnerThread.store(FPlatformTLS::GetCurrentThreadId());
				};
			}

			// Increment the matching lock count within the RigVM registry.
			int32 CurrentLockCount = 0;
			UE_AUTORTFM_OPEN
			{
				CurrentLockCount = ++Registry->LockCount;
			};

			// Handle rollback for the lock count.
			AutoRTFM::PushOnAbortHandler(Registry, [Registry = this->Registry]
			{
				int32 RollbackLockCount = --Registry->LockCount;

				// If we are rolling back the initial lock, invalidate the lock type.
				if (RollbackLockCount == 0)
				{
					Registry->LockType.store(LockType_Invalid);
				}
			});

			if (CurrentLockCount == 1)
			{
				// If we have taken the initial lock, update the lock type.
				UE_AUTORTFM_OPEN
				{
					Registry->LockType.store(DesiredLockType);
				};
			}
			else
			{
				// We should never have more than one write lock!
				ensure(DesiredLockType != LockType_Write);
			}
		}

		~FConditionalScopeLock()
		{
			if (!bLockEnabled)
			{
				return;
			}

			// Decrement the matching lock count within the RigVM registry.
			int32 CurrentLockCount = 0;
			UE_AUTORTFM_OPEN
			{
				CurrentLockCount = --Registry->LockCount;
			};

			// Since we have balanced out the increment above, remove our OnAbort handler.
			AutoRTFM::PopOnAbortHandler(Registry);

			// If we have released the final lock, invalidate the lock type.
			if (CurrentLockCount == 0)
			{
				UE_AUTORTFM_OPEN
				{
					Registry->LockType.store(LockType_Invalid);
				};
			}

			// Finally, release the actual underlying lock.
			if (DesiredLockType == LockType_Read)
			{
				ensure(CurrentLockCount >= 0);
				
				// If we were already holding the write lock, we didn't apply a subsequent read lock.
				ensure (Registry->WriteLockOwnerThread != FPlatformTLS::GetCurrentThreadId());

				Registry->Lock.ReadUnlock();
			}
			else if (DesiredLockType == LockType_Write)
			{
				ensure(CurrentLockCount == 0);
				UE_AUTORTFM_OPEN
				{
					Registry->WriteLockOwnerThread.store(static_cast<uint32>(-1));
				};
				Registry->Lock.WriteUnlock();
			}
		}

		FRigVMRegistry_NoLock& GetRegistry()
		{
			return *Registry;
		}

		const FRigVMRegistry_NoLock& GetRegistry() const
		{
			return *Registry;
		}

	private:
		FRigVMRegistry_RWLock* Registry;
		ELockType DesiredLockType;

		UE_NONCOPYABLE(FConditionalScopeLock);
	};

public:

	class FConditionalReadScopeLock : public FConditionalScopeLock 
	{
	public:
		UE_NODISCARD_CTOR explicit FConditionalReadScopeLock(const FRigVMRegistry_RWLock& InRegistry, bool bInLockEnabled = true)
		: FConditionalScopeLock(InRegistry, LockType_Read, bInLockEnabled)
		{
		}

		UE_NODISCARD_CTOR explicit FConditionalReadScopeLock(bool bInLockEnabled = true)
		: FConditionalReadScopeLock(FRigVMRegistry_RWLock::Get(), bInLockEnabled)
		{
		}

		UE_NONCOPYABLE(FConditionalReadScopeLock);
	};

	class FConditionalWriteScopeLock : public FConditionalScopeLock
	{
	public:
		UE_NODISCARD_CTOR explicit FConditionalWriteScopeLock(const FRigVMRegistry_RWLock& InRegistry, bool bInLockEnabled = true)
		: FConditionalScopeLock(InRegistry, LockType_Write, bInLockEnabled)
		{
		}

		UE_NODISCARD_CTOR explicit FConditionalWriteScopeLock(bool bInLockEnabled = true)
		: FConditionalWriteScopeLock(FRigVMRegistry_RWLock::Get(), bInLockEnabled)
		{
		}

		UE_NONCOPYABLE(FConditionalWriteScopeLock);
	};

	class FSuspendPluginUnmounting
	{
	public:
		UE_NODISCARD_CTOR FSuspendPluginUnmounting() { SuspendGlobalUnmount(); }
		~FSuspendPluginUnmounting() { ResumeGlobalUnmount(); }
	};

	// Returns the singleton registry
	static RIGVM_API FRigVMRegistry_RWLock& Get();

	static FRigVMRegistry_NoLock& Get(ELockType InLockType) = delete;
	static const FRigVMRegistry_NoLock& GetForRead() = delete;
	static FRigVMRegistry_NoLock& GetForWrite() = delete;

	// Registers a function given its name.
	// The name will be the name of the struct and virtual method,
	// for example "FMyStruct::MyVirtualMethod"
	void Register(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct = nullptr, const TArray<FRigVMFunctionArgument>& InArguments = TArray<FRigVMFunctionArgument>(), bool bLockRegistry = true)
	{
		FConditionalWriteScopeLock _(*this);
		Super::Register_NoLock(InName, InFunctionPtr, InStruct, InArguments);
	}

	void RegisterCompiledInStruct(UScriptStruct* InStruct, TConstArrayView<FRigVMCompiledInFunction> InFunctions)
	{
		FConditionalWriteScopeLock _(*this);
		Super::RegisterCompiledInStruct_NoLock(InStruct, InFunctions);
	}

	// Unregisters a function given its function index.
	// This keeps the function index stable - but assigns an invalid function
	bool RemoveFunction(int32 InFunctionIndex)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RemoveFunction_NoLock(InFunctionIndex);
	}

	// Registers a dispatch factory given its struct.
	const FRigVMDispatchFactory* RegisterFactory(UScriptStruct* InFactoryStruct)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RegisterFactory_NoLock(InFactoryStruct);
	}

	// Unregisters a factory given its script struct.
	// This keeps the template index stable
	bool RemoveFactory(UScriptStruct* InFactoryStruct)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RemoveFactory_NoLock(InFactoryStruct);
	}

	// Unregisters a template given its index.
	// This keeps the template index stable - but assigns an invalid template
	bool RemoveTemplate(int32 InTemplateIndex)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RemoveTemplate_NoLock(InTemplateIndex);
	}

	// Register a predicate contained in the input struct
	void RegisterPredicate(UScriptStruct* InStruct, const TCHAR* InName, const TArray<FRigVMFunctionArgument>& InArguments)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RegisterPredicate_NoLock(InStruct, InName, InArguments);
	}

	// Register a set of allowed object types
	void RegisterObjectTypes(TConstArrayView<TPair<UClass*, ERegisterObjectOperation>> InClasses)
	{
		FConditionalWriteScopeLock _(*this);
		Super::RegisterObjectTypes_NoLock(InClasses);
	}

	// Register a set of allowed struct types
	void RegisterStructTypes(TConstArrayView<UScriptStruct*> InStructs)
	{
		FConditionalWriteScopeLock _(*this);
		Super::RegisterStructTypes_NoLock(InStructs);
	}

	// Refreshes the list and finds the function pointers
	// based on the names.
	void RefreshEngineTypes()
	{
		FConditionalWriteScopeLock _(*this);
		Super::RefreshEngineTypes_NoLock();
	}

	// Refreshes the list and finds the function pointers
	// based on the names.
	void RefreshEngineTypesIfRequired()
	{
		FConditionalWriteScopeLock _(*this);
		Super::RefreshEngineTypesIfRequired_NoLock();
	}
	
	// Refreshes the registered functions and dispatches.
	bool RefreshFunctionsAndDispatches()
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RefreshFunctionsAndDispatches_NoLock();
	}

	// Update the registry when types are renamed
	void OnAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
	{
		FConditionalWriteScopeLock _(*this);
		Super::OnAssetRenamed_NoLock(InAssetData, InOldObjectPath);
	}

	// Update the registry when old types are being unloaded
	RIGVM_API void OnCleanupUnloadingObjects(const TArrayView<UObject*> InObjects);
	
	// Update the registry when old types are removed
    RIGVM_API void OnAssetRemoved(const FAssetData& InAssetData);

	// May add factories and unit functions declared in the plugin 
	RIGVM_API void OnPluginLoaded(IPlugin& InPlugin);

	// Removes all types associated with a plugin that's being unloaded. 
	RIGVM_API void OnPluginUnloaded(IPlugin& InPlugin);

	// Removes all types associated with modules that are being unloaded.
	RIGVM_API void OnModulesUnloaded(TConstArrayView<FName> ModuleNames);

	// Update the registry when new types are added to the attribute system so that they can be selected
	// on Attribute Nodes
	RIGVM_API void OnAnimationAttributeTypesChanged(const UScriptStruct* InStruct, bool bIsAdded);

	// Notifies other system that types have been added/removed, and template permutations have been updated
	FOnRigVMRegistryChanged& OnRigVMRegistryChanged() { return OnRigVMRegistryChangedDelegate; }
	
	// Clear the registry
	void Reset()
	{
		FConditionalWriteScopeLock _(*this);
		Super::Reset_NoLock();
	}

	// Adds a type if it doesn't exist yet and returns its index.
	// This function is thread-safe
	TRigVMTypeIndex FindOrAddType(const FRigVMTemplateArgumentType& InType, bool bForce = false)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindOrAddType_NoLock(InType, bForce);
	}

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	bool RemoveType(const FSoftObjectPath& InObjectPath)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RemoveType_NoLock(InObjectPath);
	}

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	bool RemoveType(TRigVMTypeIndex InTypeIndex)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::RemoveType_NoLock(InTypeIndex);
	}

	// Returns the type index given a type
	TRigVMTypeIndex GetTypeIndex(const FRigVMTemplateArgumentType& InType) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetTypeIndex_NoLock(InType);
	}

	// Returns the type index given a cpp type and a type object
	TRigVMTypeIndex GetTypeIndex(const FName& InCPPType, UObject* InCPPTypeObject) const
	{
		return GetTypeIndex(FRigVMTemplateArgumentType(InCPPType, InCPPTypeObject));
	}

	// Returns the type index given an enum, struct, or object
	template <typename T>
	TRigVMTypeIndex GetTypeIndex(bool bAsArray = false) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetTypeIndex_NoLock<T>(bAsArray);
	}

	// Returns the type given its index
	const FRigVMTemplateArgumentType& GetType(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetType_NoLock(InTypeIndex);
	}

	// Returns the number of types
	int32 NumTypes() const
	{
		FConditionalReadScopeLock _(*this);
		return Super::NumTypes_NoLock();
	}

	// Returns the type given only its cpp type
	const FRigVMTemplateArgumentType& FindTypeFromCPPType(const FString& InCPPType) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::FindTypeFromCPPType_NoLock(InCPPType);
	}

	// Returns the type index given only its cpp type
	TRigVMTypeIndex GetTypeIndexFromCPPType(const FString& InCPPType) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetTypeIndexFromCPPType_NoLock(InCPPType);
	}

	// Returns true if the type is an array
	bool IsArrayType(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::IsArrayType_NoLock(InTypeIndex);
	}

	// Returns true if the type is an execute type
	bool IsExecuteType(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::IsExecuteType_NoLock(InTypeIndex);
	} 

	// Converts the given execute context type to the base execute context type
	bool ConvertExecuteContextToBaseType(TRigVMTypeIndex& InOutTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::ConvertExecuteContextToBaseType_NoLock(InOutTypeIndex);
	} 

	// Returns the dimensions of the array 
	int32 GetArrayDimensionsForType(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetArrayDimensionsForType_NoLock(InTypeIndex);
	} 

	// Returns true if the type is a wildcard type
	bool IsWildCardType(TRigVMTypeIndex InTypeIndex) const
	{
		// no lock required
		return Super::IsWildCardType_NoLock(InTypeIndex);
	} 

	// Returns true if the types can be matched.
	bool CanMatchTypes(TRigVMTypeIndex InTypeIndexA, TRigVMTypeIndex InTypeIndexB, bool bAllowFloatingPointCasts) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::CanMatchTypes_NoLock(InTypeIndexA, InTypeIndexB, bAllowFloatingPointCasts);
	}

	// Returns the list of compatible types for a given type
	const TArray<TRigVMTypeIndex>& GetCompatibleTypes(TRigVMTypeIndex InTypeIndex) const
	{
		// no lock required
		return Super::GetCompatibleTypes_NoLock(InTypeIndex);
	}

	// Returns all compatible types given a category
	const TArray<TRigVMTypeIndex>& GetTypesForCategory(FRigVMTemplateArgument::ETypeCategory InCategory) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetTypesForCategory_NoLock(InCategory);
	}

	// Returns the type index of the array matching the given element type index
	TRigVMTypeIndex GetArrayTypeFromBaseTypeIndex(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex);
	}

	// Returns the type index of the element matching the given array type index
	TRigVMTypeIndex GetBaseTypeFromArrayTypeIndex(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex);
	}

	// Returns true if the given type is specifically allowed 
	bool IsSpecificallyAllowedType(const UObject* InObject) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::IsSpecificallyAllowedType_NoLock(InObject);
	}

	// Sets a given type as specifically allowed 
	void AddSpecificallyAllowedType(const UObject* InObject)
	{
		FConditionalWriteScopeLock _(*this);
		Super::AddSpecificallyAllowedType_NoLock(InObject);
	}
	
	// Returns the function given its name (or nullptr)
	const FRigVMFunction* FindFunction(const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InTypeResolver = FRigVMUserDefinedTypeResolver()) const
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindFunction_NoLock(InName, InTypeResolver);
	}

	// Returns the function given its backing up struct and method name
	const FRigVMFunction* FindFunction(UScriptStruct* InStruct, const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InResolvalInfo = FRigVMUserDefinedTypeResolver()) const
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindFunction_NoLock(InStruct, InName, InResolvalInfo);
	}

	// Returns all current RigVM functions
	const TChunkedArray<FRigVMFunction>& GetFunctions() const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetFunctions_NoLock();
	}

	// Returns a template pointer given its notation (or nullptr)
	const FRigVMTemplate* FindTemplate(const FName& InNotation, bool bIncludeDeprecated = false, bool bRemapToCoreDispatch = true) const
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindTemplate_NoLock(InNotation, bIncludeDeprecated, bRemapToCoreDispatch);
	}

	// Returns all current RigVM functions
	const TChunkedArray<FRigVMTemplate>& GetTemplates() const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetTemplates_NoLock();
	}

	// Defines and retrieves a template given its arguments
	const FRigVMTemplate* GetOrAddTemplateFromArguments(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::GetOrAddTemplateFromArguments_NoLock(InName, InInfos, InDelegates);
	}

	// Adds a new template given its arguments
	const FRigVMTemplate* AddTemplateFromArguments(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::AddTemplateFromArguments_NoLock(InName, InInfos, InDelegates);
	}

	// Returns a dispatch factory given its name (or nullptr)
	FRigVMDispatchFactory* FindDispatchFactory(const FName& InFactoryName) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::FindDispatchFactory_NoLock(InFactoryName);
	}

	// Returns a dispatch factory given its static struct (or nullptr)
	FRigVMDispatchFactory* FindOrAddDispatchFactory(UScriptStruct* InFactoryStruct)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindOrAddDispatchFactory_NoLock(InFactoryStruct);
	}

	// Returns a dispatch factory given its static struct (or nullptr)
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FRigVMDispatchFactory* FindOrAddDispatchFactory()
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindOrAddDispatchFactory_NoLock<T>();
	}

	// Returns a dispatch factory's singleton function name if that exists
	FString FindOrAddSingletonDispatchFunction(UScriptStruct* InFactoryStruct)
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindOrAddSingletonDispatchFunction_NoLock(InFactoryStruct);
	}

	// Returns a dispatch factory's singleton function name if that exists
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FString FindOrAddSingletonDispatchFunction()
	{
		FConditionalWriteScopeLock _(*this);
		return Super::FindOrAddSingletonDispatchFunction_NoLock<T>();
	}

	// Returns all dispatch factories
	const TArray<FRigVMDispatchFactory*>& GetFactories() const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetFactories_NoLock();
	}

	// Given a struct name, return the predicates
	const TArray<FRigVMFunction>* GetPredicatesForStruct(const FName& InStructName) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetPredicatesForStruct_NoLock(InStructName);
	}

	// Returns a unique hash per type index
	uint32 GetHashForType(TRigVMTypeIndex InTypeIndex) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetHashForType_NoLock(InTypeIndex);
	}
	 
	uint32 GetHashForScriptStruct(const UScriptStruct* InScriptStruct, bool bCheckTypeIndex = true) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetHashForScriptStruct_NoLock(InScriptStruct, bCheckTypeIndex);
	}
	 
	uint32 GetHashForStruct(const UStruct* InStruct) const
	{
		FConditionalReadScopeLock _(*this);
		return Super::GetHashForStruct_NoLock(InStruct);
	}
	 
	uint32 GetHashForEnum(const UEnum* InEnum, bool bCheckTypeIndex = true) const
	{
		const FConditionalReadScopeLock _(*this);
		return Super::GetHashForEnum_NoLock(InEnum, bCheckTypeIndex);
	}
	 
	uint32 GetHashForProperty(const FProperty* InProperty) const
	{
		const FConditionalReadScopeLock _(*this);
		return Super::GetHashForProperty_NoLock(InProperty);
	}

	void RebuildRegistry()
	{
		FConditionalWriteScopeLock _(*this);
		Super::RebuildRegistry_NoLock();
	}

protected:

	// Registers a function given its name.
	// The name will be the name of the struct and virtual method,
	// for example "FMyStruct::MyVirtualMethod"
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void Register_NoLock(const TCHAR* InName, FRigVMFunctionPtr InFunctionPtr, UScriptStruct* InStruct = nullptr, const TArray<FRigVMFunctionArgument>& InArguments = TArray<FRigVMFunctionArgument>()) override
	{
		Super::Register_NoLock(InName, InFunctionPtr, InStruct, InArguments);
	}

	// As above but registers a collection of functions for a struct
	virtual void RegisterCompiledInStruct_NoLock(UScriptStruct* InStruct, TConstArrayView<FRigVMCompiledInFunction> InFunctions) override
	{
		Super::RegisterCompiledInStruct_NoLock(InStruct, InFunctions);
	}

	// This keeps the function index stable - but assigns an invalid function
	virtual bool RemoveFunction_NoLock(int32 InFunctionIndex) override
	{
		return Super::RemoveFunction_NoLock(InFunctionIndex);
	}

	// Registers a dispatch factory given its struct.
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMDispatchFactory* RegisterFactory_NoLock(UScriptStruct* InFactoryStruct, const TArray<FRigVMTemplateArgumentInfo>& InArgumentInfos = {}) override
	{
		return Super::RegisterFactory_NoLock(InFactoryStruct, InArgumentInfos);
	}

	// Unregisters a factory given its script struct.
	// This keeps the template index stable
	virtual bool RemoveFactory_NoLock(UScriptStruct* InFactoryStruct) override
	{
		return Super::RemoveFactory_NoLock(InFactoryStruct);
	}

	// Unregisters a template given its index.
	// This keeps the template index stable - but assigns an invalid template
	virtual bool RemoveTemplate_NoLock(int32 InTemplateIndex) override
	{
		return Super::RemoveTemplate_NoLock(InTemplateIndex);
	}

	// Register a predicate contained in the input struct
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void RegisterPredicate_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const TArray<FRigVMFunctionArgument>& InArguments) override
	{
		return Super::RegisterPredicate_NoLock(InStruct, InName, InArguments);
	}

	// Register a set of allowed object types
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void RegisterObjectTypes_NoLock(TConstArrayView<TPair<UClass*, ERegisterObjectOperation>> InClasses) override
	{
		Super::RegisterObjectTypes_NoLock(InClasses);
	}

	// Refreshes the list and finds the function pointers
	// based on the names.
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void RefreshEngineTypes_NoLock() override
	{
		Super::RefreshEngineTypes_NoLock();
	}

	// Refreshes the registered functions and dispatches.
	virtual bool RefreshFunctionsAndDispatches_NoLock() override
	{
		return Super::RefreshFunctionsAndDispatches_NoLock();
	}

	// Refreshes the list and finds the function pointers
	// based on the names.
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void RefreshEngineTypesIfRequired_NoLock() override
	{
		Super::RefreshEngineTypesIfRequired_NoLock();
	}

	// Update the registry when old types are being unloaded
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool OnCleanupUnloadingObjects_NoLock(const TArrayView<UObject*> InObjects) override
	{
		return Super::OnCleanupUnloadingObjects_NoLock(InObjects);
	}

	// Update the registry when types are renamed
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void OnAssetRenamed_NoLock(const FAssetData& InAssetData, const FString& InOldObjectPath) override
	{
		Super::OnAssetRenamed_NoLock(InAssetData, InOldObjectPath);
	}
	
	// Update the registry when old types are removed
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool OnAssetRemoved_NoLock(const FAssetData& InAssetData) override
	{
		return Super::OnAssetRemoved_NoLock(InAssetData);
	}

	// May add factories and unit functions declared in the plugin 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool OnPluginLoaded_NoLock(IPlugin& InPlugin) override
	{
		return Super::OnPluginLoaded_NoLock(InPlugin);
	}

	// Removes all types associated with a plugin that's being unloaded. 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool OnPluginUnloaded_NoLock(IPlugin& InPlugin) override
	{
		return Super::OnPluginUnloaded_NoLock(InPlugin);
	}
	
	// Update the registry when new types are added to the attribute system so that they can be selected
	// on Attribute Nodes
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void OnAnimationAttributeTypesChanged_NoLock(const UScriptStruct* InStruct, bool bIsAdded) override
	{
		Super::OnAnimationAttributeTypesChanged_NoLock(InStruct, bIsAdded);
	}
	
	// Clear the registry
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void Reset_NoLock() override
	{
		Super::Reset_NoLock();
	}

	// Adds a type if it doesn't exist yet and returns its index.
	// This function is thread-safe
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex FindOrAddType_NoLock(const FRigVMTemplateArgumentType& InType, bool bForce = false) override
	{
		return Super::FindOrAddType_NoLock(InType, bForce);
	}

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool RemoveType_NoLock(const FSoftObjectPath& InObjectPath) override
	{
		return Super::RemoveType_NoLock(InObjectPath);
	}

	// Removes a type from the registry, and updates all dependent templates
	// which also creates invalid permutations in templates that we should ignore
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool RemoveType_NoLock(TRigVMTypeIndex InTypeIndex) override
	{
		return Super::RemoveType_NoLock(InTypeIndex);
	}

	// Returns the type index given a type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex GetTypeIndex_NoLock(const FRigVMTemplateArgumentType& InType) const override
	{
		return Super::GetTypeIndex_NoLock(InType);
	}

	// Returns the type index given a cpp type and a type object
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex GetTypeIndex_NoLock(const FName& InCPPType, UObject* InCPPTypeObject) const override
	{
		return Super::GetTypeIndex_NoLock(FRigVMTemplateArgumentType(InCPPType, InCPPTypeObject));
	}

	// Returns the type given its index
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMTemplateArgumentType& GetType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetType_NoLock(InTypeIndex);
	}

	// Returns the number of types
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual int32 NumTypes_NoLock() const override
	{
		return Super::NumTypes_NoLock();
	}

	// Returns the type given only its cpp type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMTemplateArgumentType& FindTypeFromCPPType_NoLock(const FString& InCPPType) const override
	{
		return Super::FindTypeFromCPPType_NoLock(InCPPType);
	}

	// Returns the type index given only its cpp type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex GetTypeIndexFromCPPType_NoLock(const FString& InCPPType) const override
	{
		return Super::GetTypeIndexFromCPPType_NoLock(InCPPType);
	}

	// Returns true if the type is an array
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool IsArrayType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::IsArrayType_NoLock(InTypeIndex);
	}

	// Returns true if the type is an execute type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool IsExecuteType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::IsExecuteType_NoLock(InTypeIndex);
	} 

	// Converts the given execute context type to the base execute context type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool ConvertExecuteContextToBaseType_NoLock(TRigVMTypeIndex& InOutTypeIndex) const override
	{
		return Super::ConvertExecuteContextToBaseType_NoLock(InOutTypeIndex);
	} 

	// Returns the dimensions of the array 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual int32 GetArrayDimensionsForType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetArrayDimensionsForType_NoLock(InTypeIndex);
	} 

	// Returns true if the type is a wildcard type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool IsWildCardType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::IsWildCardType_NoLock(InTypeIndex);
	} 

	// Returns true if the types can be matched.
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool CanMatchTypes_NoLock(TRigVMTypeIndex InTypeIndexA, TRigVMTypeIndex InTypeIndexB, bool bAllowFloatingPointCasts) const override
	{
		return Super::CanMatchTypes_NoLock(InTypeIndexA, InTypeIndexB, bAllowFloatingPointCasts);
	}

	// Returns the list of compatible types for a given type
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TArray<TRigVMTypeIndex>& GetCompatibleTypes_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetCompatibleTypes_NoLock(InTypeIndex);
	}

	// Returns all compatible types given a category
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TArray<TRigVMTypeIndex>& GetTypesForCategory_NoLock(FRigVMTemplateArgument::ETypeCategory InCategory) const override
	{
		return Super::GetTypesForCategory_NoLock(InCategory);
	}

	// Returns the type index of the array matching the given element type index
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex GetArrayTypeFromBaseTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetArrayTypeFromBaseTypeIndex_NoLock(InTypeIndex);
	}

	// Returns the type index of the element matching the given array type index
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual TRigVMTypeIndex GetBaseTypeFromArrayTypeIndex_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex);
	}

	// Returns true if the given type is specifically allowed 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual bool IsSpecificallyAllowedType_NoLock(const UObject* InObject) const override
	{
		return Super::IsSpecificallyAllowedType_NoLock(InObject);
	}

	// Sets a given type as specifically allowed 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void AddSpecificallyAllowedType_NoLock(const UObject* InObject) override
	{
		Super::AddSpecificallyAllowedType_NoLock(InObject);
	}

	// Returns the function given its name (or nullptr)
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMFunction* FindFunction_NoLock(const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InTypeResolver = FRigVMUserDefinedTypeResolver()) const override
	{
		return Super::FindFunction_NoLock(InName, InTypeResolver);
	}

	// Returns the function given its backing up struct and method name
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMFunction* FindFunction_NoLock(UScriptStruct* InStruct, const TCHAR* InName, const FRigVMUserDefinedTypeResolver& InResolvalInfo = FRigVMUserDefinedTypeResolver()) const override
	{
		return Super::FindFunction_NoLock(InStruct, InName, InResolvalInfo);
	}

	// Returns all current RigVM functions
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TChunkedArray<FRigVMFunction>& GetFunctions_NoLock() const override
	{
		return Super::GetFunctions_NoLock();
	}

	// Returns a template pointer given its notation (or nullptr)
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMTemplate* FindTemplate_NoLock(const FName& InNotation, bool bIncludeDeprecated = false, bool bRemapToCoreDispatch = true) const override
	{
		return Super::FindTemplate_NoLock(InNotation, bIncludeDeprecated, bRemapToCoreDispatch);
	}

	// Returns all current RigVM functions
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TChunkedArray<FRigVMTemplate>& GetTemplates_NoLock() const override
	{
		return Super::GetTemplates_NoLock();
	}

	// Defines and retrieves a template given its arguments
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMTemplate* GetOrAddTemplateFromArguments_NoLock(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates) override
	{
		return Super::GetOrAddTemplateFromArguments_NoLock(InName, InInfos, InDelegates);
	}

	// Adds a new template given its arguments
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const FRigVMTemplate* AddTemplateFromArguments_NoLock(
		const FName& InName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos,
		const FRigVMTemplateDelegates& InDelegates) override
	{
		return Super::AddTemplateFromArguments_NoLock(InName, InInfos, InDelegates);
	}

	// Returns a dispatch factory given its name (or nullptr)
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual FRigVMDispatchFactory* FindDispatchFactory_NoLock(const FName& InFactoryName) const override
	{
		return Super::FindDispatchFactory_NoLock(InFactoryName);
	}

	// Returns a dispatch factory given its static struct (or nullptr)
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual FRigVMDispatchFactory* FindOrAddDispatchFactory_NoLock(UScriptStruct* InFactoryStruct) override
	{
		return Super::FindOrAddDispatchFactory_NoLock(InFactoryStruct);
	}

	// Returns a dispatch factory's singleton function name if that exists
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual FString FindOrAddSingletonDispatchFunction_NoLock(UScriptStruct* InFactoryStruct) override
	{
		return Super::FindOrAddSingletonDispatchFunction_NoLock(InFactoryStruct);
	}

	// Returns all dispatch factories
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TArray<FRigVMDispatchFactory*>& GetFactories_NoLock() const override
	{
		return Super::GetFactories_NoLock();
	}

	// Given a struct name, return the predicates
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual const TArray<FRigVMFunction>* GetPredicatesForStruct_NoLock(const FName& InStructName) const override
	{
		return Super::GetPredicatesForStruct_NoLock(InStructName);
	}

	// Returns a unique hash per type index
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual uint32 GetHashForType_NoLock(TRigVMTypeIndex InTypeIndex) const override
	{
		return Super::GetHashForType_NoLock(InTypeIndex);
	}
	 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual uint32 GetHashForScriptStruct_NoLock(const UScriptStruct* InScriptStruct, bool bCheckTypeIndex = true) const override
	{
		return Super::GetHashForScriptStruct_NoLock(InScriptStruct, bCheckTypeIndex);
	}
	 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual uint32 GetHashForStruct_NoLock(const UStruct* InStruct) const override
	{
		return Super::GetHashForStruct_NoLock(InStruct);
	}
	 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual uint32 GetHashForEnum_NoLock(const UEnum* InEnum, bool bCheckTypeIndex = true) const override
	{
		return Super::GetHashForEnum_NoLock(InEnum, bCheckTypeIndex);
	}
	 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual uint32 GetHashForProperty_NoLock(const FProperty* InProperty) const override
	{
		return Super::GetHashForProperty_NoLock(InProperty);
	}
	 
	// Note: Only call NoLock methods on the FRigVMRegistry_NoLock
	virtual void RebuildRegistry_NoLock() override
	{
		Super::RebuildRegistry_NoLock();
	}

private:

	FRigVMRegistry_RWLock(bool bInIsGlobalRegistry);

	static void EnsureLocked(ELockType InLockType);

	UE_API void Initialize_NoLock();
	
	UE_API static void SuspendGlobalUnmount();
	UE_API static void ResumeGlobalUnmount();

	mutable FTransactionallySafeRWLock Lock;
	mutable std::atomic<ELockType> LockType = LockType_Invalid;
	mutable std::atomic<int32> LockCount = 0;
	mutable std::atomic<uint32> WriteLockOwnerThread = static_cast<uint32>(-1);
	

	// Notifies other system that types have been added/removed, and template permutations have been updated
	FOnRigVMRegistryChanged OnRigVMRegistryChangedDelegate;
	
	friend struct FRigVMStruct;
	friend struct FRigVMTemplate;
	friend struct FRigVMFunction;
	friend struct FRigVMTemplateArgument;
	friend struct FRigVMDispatchFactory;
	friend struct FRigVMRegistry_NoLock;
};

using FRigVMRegistryReadLock = FRigVMRegistry_RWLock::FConditionalReadScopeLock;
using FRigVMRegistryWriteLock = FRigVMRegistry_RWLock::FConditionalWriteScopeLock;

#if UE_WITH_CONSTINIT_UOBJECT
/** Structs used by UHT to register UScriptStructs that have RigVM functions */
struct FRigVMCompiledInStruct
{
	UScriptStruct* Struct = nullptr;
	TConstArrayView<FRigVMCompiledInFunction> Functions;
};

struct FRegisterRigVMStructs
{
	TConstArrayView<FRigVMCompiledInStruct> Structs;
	FRegisterRigVMStructs* Next = nullptr;

	UE_API FRegisterRigVMStructs(TConstArrayView<FRigVMCompiledInStruct> InStructs);
};
#endif // UE_WITH_CONSTINIT_UOBJECT

#undef UE_API
