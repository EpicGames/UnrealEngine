// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRigVMRegistry_NoLock;

#define UE_API RIGVM_API

struct FRigVMRegistryHandle
{
public:

	UE_API FRigVMRegistryHandle();

	UE_API bool IsValid() const;
	
	UE_API FRigVMRegistry_NoLock* operator->();
	UE_API const FRigVMRegistry_NoLock* operator->() const;

	UE_API FRigVMRegistry_NoLock* GetRegistry();
	UE_API const FRigVMRegistry_NoLock* GetRegistry() const;

	UE_NONCOPYABLE(FRigVMRegistryHandle);

protected:
	UE_API FRigVMRegistryHandle(const FRigVMRegistry_NoLock* InRegistry);

	FRigVMRegistry_NoLock* Registry;

	friend struct FRigVMRegistry_NoLock;
	friend class URigVM;
};

struct FRigVMRegistryHandleLock : public FRigVMRegistryHandle
{
	FRigVMRegistryHandleLock(const FRigVMRegistry_NoLock* InRegistry, bool bInLockEnabled)
		: FRigVMRegistryHandle(InRegistry)
		, bLockEnabled(bInLockEnabled)
	{
	}

	UE_NONCOPYABLE(FRigVMRegistryHandleLock);
	
protected:
	bool bLockEnabled;
};

#undef UE_API
