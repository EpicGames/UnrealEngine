// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMStatistics.generated.h"

USTRUCT(BlueprintType)
struct FRigVMMemoryStatistics
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 RegisterCount = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 DataBytes = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 TotalBytes = 0;
};

USTRUCT(BlueprintType)
struct FRigVMByteCodeStatistics
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 InstructionCount = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 DataBytes = 0;
};

USTRUCT(BlueprintType)
struct FRigVMStatistics
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 BytesForCDO = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 BytesPerInstance = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	FRigVMMemoryStatistics LiteralMemory;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	FRigVMMemoryStatistics WorkMemory;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	FRigVMMemoryStatistics DebugMemory;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	int32 BytesForCaching = 0;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Statistics")
	FRigVMByteCodeStatistics ByteCode;
};