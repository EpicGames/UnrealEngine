// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMStructUpgradeInfo.h"
#include "RigVMCore/RigVMTraits.h"
#include "RigVMCore/RigVMUserWorkflow.h"
#include "RigVMDefines.h"
#include "Templates/EnableIf.h"
#include "Templates/IsEnum.h"
#include "Templates/Models.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"

#include "RigVMStruct.generated.h"

class FProperty;
class UObject;

// delegates used for variable introspection / creation
DECLARE_DELEGATE_RetVal(TArray<FRigVMExternalVariable>, FRigVMGetExternalVariablesDelegate)
DECLARE_DELEGATE_RetVal_TwoParams(FName, FRigVMCreateExternalVariableDelegate, FRigVMExternalVariable, FString)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMBindPinToExternalVariableDelegate, FString, FString)

struct FRigVMStruct;
class URigVMController;
class IRigVMClientHost;

/** Context as of why the node was created */
enum class ERigVMNodeCreatedReason : uint8
{
	NodeSpawner,
	ScriptedEvent,
	Paste,
	BackwardsCompatibility,
	Unknown,
};

/**
 * A context struct passed to FRigVMStruct::OnUnitNodeCreated
 */
struct FRigVMUnitNodeCreatedContext
{
public:

	struct FScope
	{
	public:
		FScope(FRigVMUnitNodeCreatedContext& InContext, ERigVMNodeCreatedReason InReason, IRigVMClientHost* InHost)
			: Context(InContext)
			, PreviousReason(InContext.GetReason())
#if WITH_EDITOR
			, PreviousHost(InContext.GetHost())
#endif
		{
			Context.Reason = InReason;
#if WITH_EDITOR
			Context.Host = InHost;
#endif
		}

		~FScope()
		{
			Context.Reason = PreviousReason;
#if WITH_EDITOR
			Context.Host = PreviousHost;
#endif
		}

	private:
		FRigVMUnitNodeCreatedContext& Context;
		ERigVMNodeCreatedReason PreviousReason;
#if WITH_EDITOR
		IRigVMClientHost* PreviousHost = nullptr;
#endif
	};

	/** Returns true if this context is valid to use */
	RIGVM_API bool IsValid() const;

	/** Get the reason why this node was created */
	ERigVMNodeCreatedReason GetReason() const { return Reason; }

#if WITH_EDITOR
	URigVMController* GetController() const { return Controller; }
	IRigVMClientHost* GetHost() const { return Host; }
#endif

	/** Get the name of this node */
	FName GetNodeName() const { return NodeName; }

	/** Returns all currently existing external variables */
	RIGVM_API TArray<FRigVMExternalVariable> GetExternalVariables() const;

	/** Creates a new variable within the host of this VM */
	RIGVM_API FName AddExternalVariable(const FRigVMExternalVariable& InVariableToCreate, FString InDefaultValue = FString());

	/** Binds a pin to an external variable on the created node */
	RIGVM_API bool BindPinToExternalVariable(FString InPinPath, FString InVariablePath);

	/** Returns a variable given a name (or a non-valid variable if not found) */
	RIGVM_API FRigVMExternalVariable FindVariable(FName InVariableName) const;

	/** Returns the name of the first variable given a(or NAME_None if not found) */
	RIGVM_API FName FindFirstVariableOfType(FName InCPPTypeName) const;

	/** Returns the name of the first variable given a type (or NAME_None if not found) */
	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value>::Type * = nullptr
	>
	FName FindFirstVariableOfType() const
	{
		return FindFirstVariableOfType(TBaseStructure<T>::Get());
	}

	/** Returns the name of the first variable given a type (or NAME_None if not found) */
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FName FindFirstVariableOfType() const
	{
		return FindFirstVariableOfType(T::StaticStruct());
	}

	/** Returns the name of the first variable given a type (or NAME_None if not found) */
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUClass, T>>::Type * = nullptr
	>
	FName FindFirstVariableOfType() const
	{
		return FindFirstVariableOfType(T::StaticClass());
	}

	/** Returns the name of the first variable given a type (or NAME_None if not found) */
	template <
		typename T,
		typename TEnableIf<TIsEnum<T>::Value>::Type * = nullptr
	>
		FName FindFirstVariableOfType() const
	{
		return FindFirstVariableOfType(StaticEnum<T>());
	}

	FRigVMGetExternalVariablesDelegate& GetAllExternalVariablesDelegate() { return AllExternalVariablesDelegate; }
	FRigVMCreateExternalVariableDelegate& GetCreateExternalVariableDelegate() { return CreateExternalVariableDelegate; }
	FRigVMBindPinToExternalVariableDelegate& GetBindPinToExternalVariableDelegate() { return BindPinToExternalVariableDelegate; }

private:

#if WITH_EDITOR
	URigVMController* Controller = nullptr;
	IRigVMClientHost* Host = nullptr;
#endif
	FName NodeName = NAME_None;
	ERigVMNodeCreatedReason Reason = ERigVMNodeCreatedReason::Unknown;
	FRigVMGetExternalVariablesDelegate AllExternalVariablesDelegate;
	FRigVMCreateExternalVariableDelegate CreateExternalVariableDelegate;
	FRigVMBindPinToExternalVariableDelegate BindPinToExternalVariableDelegate;

	RIGVM_API FName FindFirstVariableOfType(UObject* InCPPTypeObject) const;

	friend class URigVMController;
	friend struct FScope;
};

/**
 * The base class for all RigVM enabled structs.
 */
USTRUCT()
struct FRigVMStruct
{
	GENERATED_BODY()

	virtual ~FRigVMStruct() = default;
	virtual FString ProcessPinLabelForInjection(const FString& InLabel) const { return InLabel; }
	virtual FName GetEventName() const { return NAME_None; }
	virtual bool CanOnlyExistOnce() const { return false; }
	virtual FString GetUnitLabel() const { return FString(); };
	virtual FString GetUnitSubTitle() const { return FString(); };

public:

	/** return the execute script struct this unit wants to use */
	virtual const UScriptStruct* GetExecuteContextStruct() const { return FRigVMExecuteContext::StaticStruct(); }

	/** initialize logic for this struct */
	virtual void Initialize() {}

	/** Execute logic for this struct */
	virtual void Execute() {}

	// control flow related
	virtual int32 GetNumSlices() const { return 1; }
#if WITH_EDITORONLY_DATA
	RIGVM_API bool IsForLoop() const;
	RIGVM_API bool IsControlFlowNode() const; 
	RIGVM_API const TArray<FName>& GetControlFlowBlocks() const;
#endif
	virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const { return false; }

	// node creation
	virtual void OnUnitNodeCreated(FRigVMUnitNodeCreatedContext& InContext) const {}

	// user workflow
	RIGVM_API TArray<FRigVMUserWorkflow> GetWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const; 

#if WITH_EDITOR
	RIGVM_API static bool ValidateStruct(UScriptStruct* InStruct, FString* OutErrorMessage);
	RIGVM_API static bool CheckPinType(UScriptStruct* InStruct, const FName& PinName, const FString& ExpectedType, FString* OutErrorMessage = nullptr);
	RIGVM_API static bool CheckPinDirection(UScriptStruct* InStruct, const FName& PinName, const FName& InDirectionMetaName);
	RIGVM_API static ERigVMPinDirection GetPinDirectionFromProperty(FProperty* InProperty);
	RIGVM_API static bool CheckPinExists(UScriptStruct* InStruct, const FName& PinName, const FString& ExpectedType = FString(), FString* OutErrorMessage = nullptr);
	RIGVM_API static bool CheckMetadata(UScriptStruct* InStruct, const FName& PinName, const FName& InMetadataKey, FString* OutErrorMessage = nullptr);
	RIGVM_API static bool CheckFunctionExists(UScriptStruct* InStruct, const FName& FunctionName, FString* OutErrorMessage = nullptr);
	RIGVM_API virtual bool ShouldCreatePinForProperty(const FProperty* InProperty) const; 
#endif
	RIGVM_API static FString ExportToFullyQualifiedText(const FProperty* InMemberProperty, const uint8* InMemberMemoryPtr, bool bUseQuotes = true);
	RIGVM_API static FString ExportToFullyQualifiedText(const UScriptStruct* InStruct, const uint8* InStructMemoryPtr, bool bUseQuotes = true);

	template <
		typename T,
		typename TEnableIf<TRigVMIsBaseStructure<T>::Value>::Type * = nullptr
	>
	static FString ExportToFullyQualifiedText(const T& InStructValue)
	{
		return ExportToFullyQualifiedText(TBaseStructure<T>::Get(), (const uint8*)&InStructValue);
	}
	
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	static FString ExportToFullyQualifiedText(const T& InStructValue)
	{
		return ExportToFullyQualifiedText(T::StaticStruct(), (const uint8*)&InStructValue);
	}

	RIGVM_API FString ExportToFullyQualifiedText(const UScriptStruct* InScriptStruct, const FName& InPropertyName, const uint8* InStructMemoryPointer = nullptr, bool bUseQuotes = true) const;
	
	RIGVM_API virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const;
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const { return FRigVMStructUpgradeInfo(); }

	static inline const FLazyName DeprecatedMetaName = FLazyName(TEXT("Deprecated"));
	static inline const FLazyName InputMetaName = FLazyName(TEXT("Input"));
	static inline const FLazyName OutputMetaName = FLazyName(TEXT("Output"));
	static inline const FLazyName IOMetaName = FLazyName(TEXT("IO"));
	static inline const FLazyName HiddenMetaName = FLazyName(TEXT("Hidden"));
	static inline const FLazyName VisibleMetaName = FLazyName(TEXT("Visible"));
	/** Meta tag for input pins that are mutable (passed by non-const reference). Must be combined with Input. */
	static inline const FLazyName ExternalVariableMetaName = FLazyName(TEXT("ExternalVariable"));
	static inline const FLazyName DetailsOnlyMetaName = FLazyName(TEXT("DetailsOnly"));
	static inline const FLazyName AbstractMetaName = FLazyName(TEXT("Abstract"));
	static inline const FLazyName CategoryMetaName = FLazyName(TEXT("Category"));
	static inline const FLazyName DisplayNameMetaName = FLazyName(TEXT("DisplayName"));
	static inline const FLazyName MenuDescSuffixMetaName = FLazyName(TEXT("MenuDescSuffix"));
	static inline const FLazyName ShowVariableNameInTitleMetaName = FLazyName(TEXT("ShowVariableNameInTitle"));
	static inline const FLazyName CustomWidgetMetaName = FLazyName(TEXT("CustomWidget"));
	static inline const FLazyName ConstantMetaName = FLazyName(TEXT("Constant"));
	static inline const FLazyName NoOpMetaName = FLazyName(TEXT("NoOp"));
	static inline const FLazyName TitleColorMetaName = FLazyName(TEXT("TitleColor"));
	static inline const FLazyName NodeColorMetaName = FLazyName(TEXT("NodeColor"));
	// icon meta name format: StyleSetName|StyleName|SmallStyleName|StatusOverlayStyleName
	// the last two names are optional, see FSlateIcon() for reference
	// Example: Icon="EditorStyle|GraphEditor.Sequence_16x"
	static inline const FLazyName IconMetaName = FLazyName(TEXT("Icon"));
	static inline const FLazyName KeywordsMetaName = FLazyName(TEXT("Keywords"));
	static inline const FLazyName FixedSizeArrayMetaName = FLazyName(TEXT("FixedSizeArray"));
	static inline const FLazyName ShowOnlySubPinsMetaName = FLazyName(TEXT("ShowOnlySubPins"));
	static inline const FLazyName HideSubPinsMetaName = FLazyName(TEXT("HideSubPins"));
	static inline const FLazyName ArraySizeMetaName = FLazyName(TEXT("ArraySize"));
	static inline const FLazyName AggregateMetaName = FLazyName(TEXT("Aggregate"));
	static inline const FLazyName ExpandPinByDefaultMetaName = FLazyName(TEXT("ExpandByDefault"));
	static inline const FLazyName DefaultArraySizeMetaName = FLazyName(TEXT("DefaultArraySize"));
	static inline const FLazyName VaryingMetaName = FLazyName(TEXT("Varying"));
	static inline const FLazyName PureMetaName = FLazyName(TEXT("Pure")); // The node will not modify the context
	static inline const FLazyName SingletonMetaName = FLazyName(TEXT("Singleton"));
	static inline const FLazyName SliceContextMetaName = FLazyName(TEXT("SliceContext"));
	static inline const FLazyName ExecuteName = FLazyName(TEXT("Execute"));
	static inline const FLazyName ExecuteContextName = FLazyName(TEXT("ExecuteContext"));
	static inline const FLazyName ExecutePinName = FLazyName(TEXT("ExecutePin"));
	static inline const FLazyName ForLoopCountPinName = FLazyName(TEXT("Count"));
	static inline const FLazyName ForLoopContinuePinName = FLazyName(TEXT("Continue"));
	static inline const FLazyName ForLoopCompletedPinName = FLazyName(TEXT("Completed"));
	static inline const FLazyName ForLoopIndexPinName = FLazyName(TEXT("Index"));
	static inline const FLazyName ComputeLazilyMetaName = FLazyName(TEXT("Lazy"));
	static inline const FLazyName ControlFlowBlockToRunName = FLazyName(TEXT("BlockToRun"));
	static inline const FLazyName ControlFlowCompletedName = FLazyName(TEXT("Completed"));
	static inline const FLazyName ControlFlowCountName = FLazyName(TEXT("Count"));
	static inline const FLazyName ControlFlowIndexName = FLazyName(TEXT("Index"));

#if WITH_EDITOR
	/**
	 * Checks if any struct in the hierarchy has the specified metadata.
	 * UScriptStruct metadata is not inherited automatically, so this walks
	 * up the hierarchy to check each struct in the chain.
	 */
	static bool HasMetaData(const UScriptStruct* InStruct, const FName& InMetaDataKey)
	{
		for (const UScriptStruct* Current = InStruct; Current; Current = Cast<UScriptStruct>(Current->GetSuperStruct()))
		{
			if (Current->HasMetaData(InMetaDataKey))
			{
				return true;
			}
		}
		return false;
	}
#endif

protected:

	RIGVM_API static float GetRatioFromIndex(int32 InIndex, int32 InCount);
	RIGVM_API TMap<FName, FString> GetDefaultValues(UScriptStruct* InScriptStruct) const;
	RIGVM_API static TMap<FName, FString> GetDefaultValues(const void* InValue, UScriptStruct* InScriptStruct);
	RIGVM_API bool ApplyUpgradeInfo(const FRigVMStructUpgradeInfo& InUpgradeInfo);
	virtual TArray<FRigVMUserWorkflow> GetSupportedWorkflows(const UObject* InSubject) const { return TArray<FRigVMUserWorkflow>(); }
	
#if WITH_EDITORONLY_DATA
	RIGVM_API virtual const TArray<FName>& GetControlFlowBlocks_Impl() const;
#endif

#if WITH_EDITOR
	RIGVM_API static void ValidateControlFlowBlocks(const TArray<FName>& InBlocks);
#endif

	friend struct FRigVMStructUpgradeInfo;
	friend class FRigVMGraphStructUpgradeInfoTest;
	friend class URigVMController;
	friend struct FRigVMDispatchFactory;
};

/**
 * The base mutable class for all RigVM enabled structs.
 */
USTRUCT()
struct FRigVMStructMutable : public FRigVMStruct
{
	GENERATED_BODY()

	virtual ~FRigVMStructMutable() = default;

public:


	/*
	 * This property is used to chain multiple mutable nodes together
	 */
	UPROPERTY(DisplayName = "Execute", Transient, meta = (Input, Output))
	FRigVMExecutePin ExecutePin;
};
