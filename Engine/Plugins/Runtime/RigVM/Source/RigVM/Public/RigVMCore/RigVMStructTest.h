// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "RigVMCore/RigVMStruct.h"

template<typename ExecuteContextType = FRigVMExecuteContext>
class FRigVMStructTestBase : public FAutomationTestBase
{
public:
	FRigVMStructTestBase(const FString& InName, bool bIsComplex)
		: FAutomationTestBase(InName, bIsComplex)
		, ExtendedExecuteContext(nullptr)
		, ExecuteContext(DefaultExecuteContext)
	{
	}

	~FRigVMStructTestBase() = default;

	virtual void Initialize()
	{
		ExtendedExecuteContext = FRigVMExtendedExecuteContext(ExecuteContextType::StaticStruct());
		ExecuteContext = ExtendedExecuteContext.GetPublicData<ExecuteContextType>(); 
	}
	
protected:
	FRigVMExtendedExecuteContext ExtendedExecuteContext;

public:
	ExecuteContextType& ExecuteContext;

private:
	ExecuteContextType DefaultExecuteContext;
};

#define RIGVMSTRUCT_TEST_STRINGIFY(Content) #Content
#define IMPLEMENT_RIGVMSTRUCT_AUTOMATION_TEST(TRigVMStruct) \
	class TRigVMStruct##Test : public FRigVMStructTestBase<FRigVMExecuteContext> \
	{ \
	public: \
		TRigVMStruct##Test( const FString& InName ) \
		:FRigVMStructTestBase<FRigVMExecuteContext>( InName, false ) {\
		} \
		virtual EAutomationTestFlags GetTestFlags() const override { return EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter; } \
		virtual bool IsStressTest() const { return false; } \
		virtual uint32 GetRequiredDeviceNum() const override { return 1; } \
		virtual FString GetTestSourceFileName() const override { return __FILE__; } \
		virtual int32 GetTestSourceFileLine() const override { return __LINE__; } \
	protected: \
		virtual void GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const override \
		{ \
			OutBeautifiedNames.Add(TEXT(RIGVMSTRUCT_TEST_STRINGIFY(RigVM.RigVMFunctions.TRigVMStruct))); \
			OutTestCommands.Add(FString()); \
		} \
		TRigVMStruct Unit; \
		virtual bool RunTest(const FString& Parameters) override \
		{ \
			Initialize(); \
			Unit = TRigVMStruct(); \
			return RunRigVMStructTest(Parameters); \
		} \
		virtual bool RunRigVMStructTest(const FString& Parameters); \
		virtual FString GetBeautifiedTestName() const override { return TEXT(RIGVMSTRUCT_TEST_STRINGIFY(RigVM.RigVMFunctions.TRigVMStruct)); } \
		void InitAndExecute() \
		{ \
			Unit.Initialize(); \
			Unit.Execute(ExecuteContext); \
		} \
		void Execute() \
		{ \
			Unit.Execute(ExecuteContext); \
		} \
	}; \
	namespace\
	{\
		TRigVMStruct##Test TRigVMStruct##AutomationTestInstance(TEXT(RIGVMSTRUCT_TEST_STRINGIFY(TRigVMStruct##Test))); \
	} \
	bool TRigVMStruct##Test::RunRigVMStructTest(const FString& Parameters)
