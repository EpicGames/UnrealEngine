// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMTemplate.h"

#define UE_API RIGVM_API

struct FRigVMStruct;

/** Structure used to upgrade to a new implementation of a node */
struct FRigVMStructUpgradeInfo
{
public:
	
	UE_API FRigVMStructUpgradeInfo();

	template<typename Old, typename New>
	FRigVMStructUpgradeInfo(const Old& InOld, const New& InNew)
	{
		OldStruct = Old::StaticStruct();
		NewStruct = New::StaticStruct();
		OldDispatchFunction = NAME_None;
		NewDispatchFunction = NAME_None;
		SetDefaultValues(&InNew);
	}

	FRigVMStructUpgradeInfo(UScriptStruct* InOldDispatchStruct, UScriptStruct* InNewDispatchStruct, const FName& InOldDispatchFunction, const FName& InNewDispatchFunction)
	{
		OldStruct = InOldDispatchStruct;
		NewStruct = InNewDispatchStruct;
		OldDispatchFunction = InOldDispatchFunction;
		NewDispatchFunction = InNewDispatchFunction;
	}

	static UE_API FRigVMStructUpgradeInfo MakeFromStructToFactory(UScriptStruct* InRigVMStruct, UScriptStruct* InFactoryStruct, const TMap<FName,FName>& InArgumentNameMap = TMap<FName,FName>());

	// returns true if this upgrade info can be applied
	UE_API bool IsValid() const;

	// returns the old struct trying to be upgraded
	UScriptStruct* GetOldStruct() const { return OldStruct; }

	// returns the new struct to upgrade to
	UScriptStruct* GetNewStruct() const { return NewStruct; }

	// returns the map for all default values
	const TMap<FName, FString>& GetDefaultValues() const { return DefaultValues; }

	// returns the default value for a given pin
	UE_API const FString& GetDefaultValueForPin(const FName& InPinName) const;

	// sets the default value for a given pin
	UE_API void SetDefaultValueForPin(const FName& InPinName, const FString& InDefaultValue);

	// adds a pin to be remapped
	UE_API void AddRemappedPin(const FString& InOldPinPath, const FString& InNewPinPath, bool bAsInput = true, bool bAsOutput = true);

	// remaps a pin path based on our internals
	UE_API FString RemapPin(const FString& InPinPath, bool bIsInput, bool bContainsNodeName) const;

	// adds a new aggregate pin
	UE_API FString AddAggregatePin(FString InPinName = FString());

	// returns the aggregate pins to add
	const TArray<FString>& GetAggregatePins() const { return AggregatePins; }

	// returns a type map representing the struct members
	static UE_API FRigVMTemplateTypeMap GetTypeMapFromStruct(UScriptStruct* InScriptStruct);

private:
	
	// The complete node path including models / collapse node.
	// The path may look like "RigGraph|CollapseNode1|Add"
	FString NodePath;
	
	// The old struct this upgrade info originates from
	UScriptStruct* OldStruct;

	// The new struct this upgrade info is targeting
	UScriptStruct* NewStruct;

	// The old optional function name this upgrade info originates from
	FName OldDispatchFunction;

	// The new optional function name this upgrade info is targeting
	FName NewDispatchFunction;

	// Remapping info for re-linking inputs 
	// Entries can be root pins or sub pins
	TMap<FString, FString> InputLinkMap;

	// Remapping info for re-linking outputs 
	// Entries can be root pins or sub pins
	TMap<FString, FString> OutputLinkMap;

	// New sets of default values
	TMap<FName, FString> DefaultValues;

	// Aggregate pins to add
	TArray<FString> AggregatePins;

	// sets the default values from the new struct.
	UE_API void SetDefaultValues(const void* InNewStructMemory);

	friend class URigVMController;
};

#undef UE_API
