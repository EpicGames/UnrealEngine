// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "RigVMFunction.h"
#include "RigVMTypeIndex.h"
#include "RigVMTypeUtils.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"

#include "RigVMTemplate.generated.h"

class FProperty;
struct FRigVMDispatchFactory;
struct FRigVMTemplate;
struct FRigVMDispatchContext;
struct FRigVMUserDefinedTypeResolver;
struct FRigVMRegistryHandle;
class FRigVMTypeCacheScope_NoLock;

typedef TMap<FName, TRigVMTypeIndex> FRigVMTemplateTypeMap;

// FRigVMTemplate_NewArgumentTypeDelegate is deprecated, use FRigVMTemplate_GetPermutationsFromArgumentTypeDelegate
DECLARE_DELEGATE_RetVal_ThreeParams(FRigVMTemplateTypeMap, FRigVMTemplate_NewArgumentTypeDelegate, const FName& /* InArgumentName */, TRigVMTypeIndex /* InTypeIndexToAdd */, const FRigVMRegistryHandle& /*InContext*/);
DECLARE_DELEGATE_RetVal_OneParam(FRigVMDispatchFactory*, FRigVMTemplate_GetDispatchFactoryDelegate, const FRigVMRegistryHandle& /* InContext */);

struct FRigVMTemplateDelegates
{
	FRigVMTemplate_NewArgumentTypeDelegate NewArgumentTypeDelegate;
	FRigVMTemplate_GetDispatchFactoryDelegate GetDispatchFactoryDelegate;
};

USTRUCT()
struct FRigVMTemplateArgumentType
{
	GENERATED_BODY()

	UPROPERTY()
	FName CPPType;
	
	UPROPERTY()
	TObjectPtr<UObject> CPPTypeObject; 

	FRigVMTemplateArgumentType()
		: CPPType(NAME_None)
		, CPPTypeObject(nullptr)
	{
		CPPType = RigVMTypeUtils::GetWildCardCPPTypeName();
		CPPTypeObject = RigVMTypeUtils::GetWildCardCPPTypeObject();
	}

	RIGVM_API FRigVMTemplateArgumentType(const FName& InCPPType, UObject* InCPPTypeObject = nullptr, bool bIsSerializing = false);
	
	FRigVMTemplateArgumentType(UClass* InClass, RigVMTypeUtils::EClassArgType InClassArgType = RigVMTypeUtils::EClassArgType::AsObject)
	: CPPType(*RigVMTypeUtils::CPPTypeFromObject(InClass, InClassArgType))
	, CPPTypeObject(InClass)
	{
	}

	FRigVMTemplateArgumentType(UScriptStruct* InScriptStruct)
	: CPPType(*RigVMTypeUtils::GetUniqueStructTypeName(InScriptStruct))
	, CPPTypeObject(InScriptStruct)
	{
	}

	FRigVMTemplateArgumentType(UEnum* InEnum)
	: CPPType(*RigVMTypeUtils::CPPTypeFromEnum(InEnum))
	, CPPTypeObject(InEnum)
	{
	}

	bool IsValid() const
	{
		return !CPPType.IsNone();
	}

	bool IsUnknownType() const
	{
		if (CPPTypeObject)
		{
			return CPPTypeObject == RigVMTypeUtils::GetWildCardCPPTypeObject();
		}
		return false;
	}
	
	void Serialize_NoLock(FArchive& Ar);

	static FRigVMTemplateArgumentType Array()
	{
		return FRigVMTemplateArgumentType(RigVMTypeUtils::GetWildCardArrayCPPTypeName(), RigVMTypeUtils::GetWildCardCPPTypeObject());
	}

	bool operator == (const FRigVMTemplateArgumentType& InOther) const
	{
		return CPPType == InOther.CPPType;
	}

	bool operator != (const FRigVMTemplateArgumentType& InOther) const
	{
		return CPPType != InOther.CPPType;
	}

	friend inline uint32 GetTypeHash(const FRigVMTemplateArgumentType& InType)
	{
		return GetTypeHash(InType.CPPType.ToString());
	}

	FName GetCPPTypeObjectPath() const
	{
		if(CPPTypeObject)
		{
			return *CPPTypeObject->GetPathName();
		}
		return NAME_None;
	}

	bool IsWildCard() const
	{
		return CPPTypeObject == RigVMTypeUtils::GetWildCardCPPTypeObject() ||
			CPPType == RigVMTypeUtils::GetWildCardCPPTypeName() ||
			CPPType == RigVMTypeUtils::GetWildCardArrayCPPTypeName();
	}

	bool IsArray() const
	{
		return RigVMTypeUtils::IsArrayType(CPPType.ToString());
	}

	FString GetBaseCPPType() const
	{
		if(IsArray())
		{
			return RigVMTypeUtils::BaseTypeFromArrayType(CPPType.ToString());
		}
		return CPPType.ToString();
	}

	FRigVMTemplateArgumentType& ConvertToArray() 
	{
		CPPType = *RigVMTypeUtils::ArrayTypeFromBaseType(CPPType.ToString());
		return *this;
	}

	FRigVMTemplateArgumentType& ConvertToBaseElement() 
	{
		CPPType = *RigVMTypeUtils::BaseTypeFromArrayType(CPPType.ToString());
		return *this;
	}
};

/**
 * The template argument represents a single parameter
 * in a function call and all of its possible types
 */
struct FRigVMTemplateArgument
{
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FTypeFilter, const TRigVMTypeIndex&, const FRigVMRegistryHandle&);
	
	enum EArrayType
	{
		EArrayType_SingleValue,
		EArrayType_ArrayValue,
		EArrayType_ArrayArrayValue,
		EArrayType_Mixed,
		EArrayType_Invalid
	};

	enum ETypeCategory : uint8
	{
		ETypeCategory_Execute,
		ETypeCategory_SingleAnyValue,
		ETypeCategory_ArrayAnyValue,
		ETypeCategory_ArrayArrayAnyValue,
		ETypeCategory_SingleSimpleValue,
		ETypeCategory_ArraySimpleValue,
		ETypeCategory_ArrayArraySimpleValue,
		ETypeCategory_SingleMathStructValue,
		ETypeCategory_ArrayMathStructValue,
		ETypeCategory_ArrayArrayMathStructValue,
		ETypeCategory_SingleScriptStructValue,
		ETypeCategory_ArrayScriptStructValue,
		ETypeCategory_ArrayArrayScriptStructValue,
		ETypeCategory_SingleEnumValue,
		ETypeCategory_ArrayEnumValue,
		ETypeCategory_ArrayArrayEnumValue,
		ETypeCategory_SingleObjectValue,
		ETypeCategory_ArrayObjectValue,
		ETypeCategory_ArrayArrayObjectValue,
		ETypeCategory_Invalid,
		
		ETypeCategory_Count,
	};

	// default constructor
	RIGVM_API FRigVMTemplateArgument();

	RIGVM_API FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection);
	RIGVM_API FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, TRigVMTypeIndex InType, const FRigVMRegistryHandle& InRegistry);
	RIGVM_API FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices, const FRigVMRegistryHandle& InRegistry);
	RIGVM_API FRigVMTemplateArgument(const FName& InName, ERigVMPinDirection InDirection, const TArray<ETypeCategory>& InTypeCategories, TFunction<bool(const TRigVMTypeIndex&, const FRigVMRegistryHandle&)> InFilterType, const FRigVMRegistryHandle& InRegistry);

	// returns the name of the argument
	const FName& GetName() const { return Name; }

	// returns the direction of the argument
	ERigVMPinDirection GetDirection() const { return Direction; }

#if WITH_EDITOR
	// returns true if this argument supports a given type across a set of permutations
	RIGVM_API bool SupportsTypeIndex(TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex = nullptr) const;
	RIGVM_API bool SupportsTypeIndex_NoLock(const FRigVMRegistryHandle& InRegistry, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex = nullptr) const;
#endif

	// returns the flat list of types (including duplicates) of this argument
	RIGVM_API void GetAllTypes(TArray<TRigVMTypeIndex>& OutTypes) const;
	RIGVM_API void GetAllTypes_NoLock(TArray<TRigVMTypeIndex>& OutTypes, const FRigVMRegistryHandle& InRegistry) const;

	RIGVM_API TRigVMTypeIndex GetTypeIndex(const int32 InIndex) const;
	RIGVM_API TRigVMTypeIndex GetTypeIndex_NoLock(const int32 InIndex, const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API TOptional<TRigVMTypeIndex> TryToGetTypeIndex(const int32 InIndex) const;
	RIGVM_API TOptional<TRigVMTypeIndex> TryToGetTypeIndex_NoLock(const int32 InIndex, const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API int32 GetNumTypes(const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API int32 GetNumTypes_NoLock(const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API void AddTypeIndex(const TRigVMTypeIndex InTypeIndex);
	RIGVM_API void RemoveType(const int32 InIndex);
	RIGVM_API void ForEachType(TFunction<bool(const TRigVMTypeIndex InType)>&& InCallback, const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API int32 FindTypeIndex(const TRigVMTypeIndex InTypeIndex, const FRigVMRegistryHandle& InRegistry) const;

	template <typename Predicate>
	int32 IndexOfByPredicate(Predicate Pred, const FRigVMRegistryHandle& InRegistry) const
	{
		if (!bUseCategories)
		{
			return TypeIndices.IndexOfByPredicate(Pred);
		}
		if (FilterType)
		{
			bool bFound = false;
			int32 ValidIndex = 0;
			CategoryViews(TypeCategories, InRegistry).ForEachType([this, &ValidIndex, &bFound, Pred, &InRegistry](const TRigVMTypeIndex Type)
			{
				if (FilterType(Type, InRegistry))
				{
					if (Pred(Type))
					{
						bFound = true;
						return false;
					}
					ValidIndex++;
				}
				return true;
			});
			if (bFound)
			{
				return ValidIndex;
			}
			return INDEX_NONE;
		}
		return CategoryViews(TypeCategories, InRegistry).IndexOfByPredicate(Pred);
	}

	// returns an array of all of the supported types
	RIGVM_API TArray<TRigVMTypeIndex> GetSupportedTypeIndices(const TArray<int32>& InPermutationIndices = TArray<int32>()) const;
	RIGVM_API TArray<TRigVMTypeIndex> GetSupportedTypeIndices_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const;

	// returns an array of all supported types as strings. this is used for automated testing only.
	RIGVM_API TArray<FString> GetSupportedTypeStrings(const TArray<int32>& InPermutationIndices = TArray<int32>()) const;
	RIGVM_API TArray<FString> GetSupportedTypeStrings_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const;
	
	// returns true if an argument is singleton (same type for all variants)
	RIGVM_API bool IsSingleton(const TArray<int32>& InPermutationIndices = TArray<int32>()) const;
	RIGVM_API bool IsSingleton_NoLock(const TArray<int32>& InPermutationIndices, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if this argument is an execute
	RIGVM_API bool IsExecute() const;
	RIGVM_API bool IsExecute_NoLock(const FRigVMRegistryHandle& InRegistry) const;
	
	// returns true if the argument uses an array container
	RIGVM_API EArrayType GetArrayType() const;
	RIGVM_API EArrayType GetArrayType_NoLock(const FRigVMRegistryHandle& InRegistry) const;

	RIGVM_API friend uint32 GetTypeHash_NoLock(const FRigVMTemplateArgument& InArgument, const FRigVMRegistryHandle& InRegistry);

#if WITH_EDITOR
	// Get the map of types to permutation indices
	RIGVM_API const TArray<int32>& GetPermutations(const TRigVMTypeIndex InType) const;
	RIGVM_API const TArray<int32>& GetPermutations_NoLock(const TRigVMTypeIndex InType, const FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API void InvalidatePermutations(const TRigVMTypeIndex InType);
#endif

protected:

	int32 Index = INDEX_NONE;
	FName Name = NAME_None;
	ERigVMPinDirection Direction = ERigVMPinDirection::IO;

	TArray<TRigVMTypeIndex> TypeIndices;
#if WITH_EDITOR
	mutable TMap<TRigVMTypeIndex, TArray<int32>> TypeToPermutations;
#endif

	bool bUseCategories = false;
	TArray<ETypeCategory> TypeCategories;
	TFunction<bool(const TRigVMTypeIndex&, const FRigVMRegistryHandle&)> FilterType;
	mutable TOptional<EArrayType> CachedArrayType;

	// constructor from a property. this forces the type to be created
	RIGVM_API FRigVMTemplateArgument(FProperty* InProperty, FRigVMRegistryHandle& InRegistry);

	// static make function using a lock to create it
	static RIGVM_API FRigVMTemplateArgument Make(FProperty* InProperty);

	// static make function using a lock to create it
	static RIGVM_API FRigVMTemplateArgument Make_NoLock(FProperty* InProperty);
	static RIGVM_API FRigVMTemplateArgument Make_NoLock(FProperty* InProperty, FRigVMRegistryHandle& InRegistry);

	RIGVM_API void EnsureValidExecuteType_NoLock(const FRigVMRegistryHandle& InRegistry);

#if WITH_EDITOR
	RIGVM_API void UpdateTypeToPermutationsSlow(const FRigVMRegistryHandle& InRegistry);
#endif

	friend struct FRigVMTemplate;
	friend struct FRigVMDispatchFactory;
	friend class URigVMController;
	friend struct FRigVMRegistry_NoLock;
	friend struct FRigVMStructUpgradeInfo;
	friend class URigVMCompiler;
	friend class FRigVMTypeCacheScope_NoLock;
	friend class URigVM;

private:
	struct CategoryViews
	{
		CategoryViews() = delete;
		CategoryViews(const TArray<ETypeCategory>& InCategories, const FRigVMRegistryHandle& InRegistry);
		
		void ForEachType(TFunction<bool(const TRigVMTypeIndex InType)>&& InCallback) const;

		TRigVMTypeIndex GetTypeIndex(int32 InIndex) const;
		
		int32 FindIndex(const TRigVMTypeIndex InTypeIndex) const;

		template <typename Predicate>
		int32 IndexOfByPredicate(Predicate Pred) const
		{
			int32 Offset = 0;
			for (const TArrayView<const TRigVMTypeIndex>& TypeView: Types)
			{
				const int32 Found = TypeView.IndexOfByPredicate(Pred);
				if (Found != INDEX_NONE)
				{
					return Found + Offset;
				}
				Offset += TypeView.Num();
			}
			return INDEX_NONE;
		}
		
	private:
		TArray<TArrayView<const TRigVMTypeIndex>, TInlineAllocator<(uint32)ETypeCategory::ETypeCategory_Count>> Types;
	};
};

/**
 * FRigVMTemplateArgumentInfo 
 */

struct FRigVMTemplateArgumentInfo
{
public:
	
	using ArgumentCallback = TFunction<FRigVMTemplateArgument(const FName /*InName*/, ERigVMPinDirection /*InDirection*/, const FRigVMRegistryHandle& /*InContext*/)>;
	using TypeFilterCallback = TFunction<bool(const TRigVMTypeIndex& /*InNewType*/, const FRigVMRegistryHandle& /*InContext*/)>;
	
	FName Name = NAME_None;
	ERigVMPinDirection Direction = ERigVMPinDirection::Invalid;
	ArgumentCallback FactoryCallback = [](const FName, ERigVMPinDirection, const FRigVMRegistryHandle&) { return FRigVMTemplateArgument(); };
	TArray<TRigVMTypeIndex> TypeIndices;
	
	RIGVM_API FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, TRigVMTypeIndex InTypeIndex);
	RIGVM_API FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices);
	RIGVM_API FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories, TypeFilterCallback InTypeFilter = nullptr);
	RIGVM_API FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection);
	RIGVM_API FRigVMTemplateArgumentInfo(const FName InName, ERigVMPinDirection InDirection, ArgumentCallback&& InCallback);

	RIGVM_API static FRigVMTemplateArgumentInfo MakeFlattenedFromTypeIndices(const FName InName, ERigVMPinDirection InDirection, const TArray<TRigVMTypeIndex>& InTypeIndices);

	RIGVM_API FRigVMTemplateArgument GetArgument() const;
	RIGVM_API FRigVMTemplateArgument GetArgument_NoLock(const FRigVMRegistryHandle& InRegistry) const;

	static RIGVM_API FName ComputeTemplateNotation(
		const FName InTemplateName,
		const TArray<FRigVMTemplateArgumentInfo>& InInfos);
	
	static RIGVM_API TArray<TRigVMTypeIndex> GetTypesFromCategories(
		const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories,
		const FRigVMTemplateArgument::FTypeFilter& InTypeFilter = {});

	static TArray<TRigVMTypeIndex> GetTypesFromCategories_NoLock(
    	const TArray<FRigVMTemplateArgument::ETypeCategory>& InTypeCategories,
    	const FRigVMTemplateArgument::FTypeFilter& InTypeFilter,
    	const FRigVMRegistryHandle& InRegistry);
};

/**
 * The template is used to group multiple rigvm functions
 * that share the same notation. Templates can then be used
 * to build polymorphic nodes (RigVMTemplateNode) that can
 * take on any of the permutations supported by the template.
 */
struct FRigVMTemplate
{
public:

	typedef FRigVMTemplateTypeMap FTypeMap;
	typedef TPair<FName, TRigVMTypeIndex> FTypePair;

	// Default constructor
	RIGVM_API FRigVMTemplate();

	// returns true if this is a valid template
	RIGVM_API bool IsValid() const;

	// Returns the notation of this template
	RIGVM_API const FName& GetNotation() const;

	// Returns the name of the template
	RIGVM_API FName GetName() const;

	// Returns the name to use for a node
	RIGVM_API FName GetNodeName() const;

	// returns true if this template can merge another one
	RIGVM_API bool Merge(const FRigVMTemplate& InOther, const FRigVMRegistryHandle& InRegistry);

	// returns the number of args of this template
	int32 NumArguments() const { return Arguments.Num(); }

	// returns an argument for a given index
	const FRigVMTemplateArgument* GetArgument(int32 InIndex) const { return &Arguments[InIndex]; }

	// returns an argument given a name (or nullptr)
	RIGVM_API const FRigVMTemplateArgument* FindArgument(const FName& InArgumentName) const;

	// returns the number of args of this template
	RIGVM_API int32 NumExecuteArguments(const FRigVMDispatchContext& InContext) const;

	// returns an argument for a given index
	RIGVM_API const FRigVMExecuteArgument* GetExecuteArgument(int32 InIndex, const FRigVMDispatchContext& InContext) const;

	// returns an argument given a name (or nullptr)
	RIGVM_API const FRigVMExecuteArgument* FindExecuteArgument(const FName& InArgumentName, const FRigVMDispatchContext& InContext) const;
	RIGVM_API const FRigVMExecuteArgument* FindExecuteArgument_NoLock(const FName& InArgumentName, const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const;

	// returns the top level execute context struct this template uses
	RIGVM_API const UScriptStruct* GetExecuteContextStruct() const;
	RIGVM_API const UScriptStruct* GetExecuteContextStruct_NoLock(const FRigVMRegistryHandle& InRegistry) const;

	// returns true if this template supports a given execute context struct
	RIGVM_API bool SupportsExecuteContextStruct(const UScriptStruct* InExecuteContextStruct) const;
	RIGVM_API bool SupportsExecuteContextStruct_NoLock(const UScriptStruct* InExecuteContextStruct, const FRigVMRegistryHandle& InRegistry) const;

#if WITH_EDITOR
	// returns true if a given arg supports a type
	RIGVM_API bool ArgumentSupportsTypeIndex(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex = nullptr) const;
#endif

	// returns the number of permutations supported by this template
	RIGVM_API int32 NumPermutations() const;
	int32 NumPermutations_NoLock(const FRigVMRegistryHandle& InRegistry) const { return Permutations.Num(); }

	// returns the first / primary permutation of the template
	RIGVM_API const FRigVMFunction* GetPrimaryPermutation() const;
	RIGVM_API const FRigVMFunction* GetPrimaryPermutation_NoLock(const FRigVMRegistryHandle& InRegistry) const;

	// returns a permutation given an index
	RIGVM_API const FRigVMFunction* GetPermutation(int32 InIndex) const;
	RIGVM_API const FRigVMFunction* GetPermutation_NoLock(int32 InIndex, const FRigVMRegistryHandle& InRegistry) const;

	// returns a permutation given an index and creates it using the backing factory if needed
	RIGVM_API const FRigVMFunction* GetOrCreatePermutation(int32 InIndex);
	RIGVM_API const FRigVMFunction* GetOrCreatePermutation_NoLock(int32 InIndex, FRigVMRegistryHandle& InRegistry);

	// returns true if a given function is a permutation of this template
	RIGVM_API bool ContainsPermutation(const FRigVMFunction* InPermutation) const;
	RIGVM_API bool ContainsPermutation_NoLock(const FRigVMFunction* InPermutation, const FRigVMRegistryHandle& InRegistry) const;

	// returns the index of the permutation within the template of a given function (or INDEX_NONE)
	RIGVM_API int32 FindPermutation(const FRigVMFunction* InPermutation) const;
	RIGVM_API int32 FindPermutation_NoLock(const FRigVMFunction* InPermutation, const FRigVMRegistryHandle& InRegistry) const;

	// returns the index of the permutation within the template of a given set of types
	RIGVM_API int32 FindPermutation(const FTypeMap& InTypes) const;
	RIGVM_API int32 FindPermutation_NoLock(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if the template was able to resolve to single permutation
	RIGVM_API bool FullyResolve(FTypeMap& InOutTypes, int32& OutPermutationIndex) const;
	RIGVM_API bool FullyResolve_NoLock(FTypeMap& InOutTypes, int32& OutPermutationIndex, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if the template was able to resolve to at least one permutation
	RIGVM_API bool Resolve(FTypeMap& InOutTypes, TArray<int32> & OutPermutationIndices, bool bAllowFloatingPointCasts) const;
	RIGVM_API bool Resolve_NoLock(FTypeMap& InOutTypes, TArray<int32>& OutPermutationIndices, bool bAllowFloatingPointCasts, const FRigVMRegistryHandle& InRegistry) const;

	// Will return the hash of the input type map, if it is a valid type map. Otherwise, will return 0.
	// It is only a valid type map if it includes all arguments, and non of the types is a wildcard.
	RIGVM_API uint32 GetTypesHashFromTypes(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if the template was able to resolve to at least one permutation
	RIGVM_API bool ContainsPermutation(const FTypeMap& InTypes) const;
	RIGVM_API bool ContainsPermutation_NoLock(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if the template can resolve an argument to a new type
	RIGVM_API bool ResolveArgument(const FName& InArgumentName, const TRigVMTypeIndex InTypeIndex, FTypeMap& InOutTypes) const;
	RIGVM_API bool ResolveArgument_NoLock(const FName& InArgumentName, const TRigVMTypeIndex InTypeIndex, FTypeMap& InOutTypes, const FRigVMRegistryHandle& InRegistry) const;

	// returns the types for a specific permutation
	RIGVM_API FRigVMTemplateTypeMap GetTypesForPermutation(const int32 InPermutationIndex) const;
	RIGVM_API FRigVMTemplateTypeMap GetTypesForPermutation_NoLock(const int32 InPermutationIndex, const FRigVMRegistryHandle& InRegistry) const;

	// returns true if a given argument is valid for a template
	static RIGVM_API bool IsValidArgumentForTemplate(const ERigVMPinDirection InDirection);

	// returns the prefix for an argument in the notation
	static RIGVM_API const FString& GetDirectionPrefix(const ERigVMPinDirection InDirection);

	// returns the notation of an argument
	static RIGVM_API FString GetArgumentNotation(const FName InName, const ERigVMPinDirection InDirection);

	// recomputes the notation from its arguments
	RIGVM_API void ComputeNotationFromArguments(const FString& InTemplateName);

	// returns an array of structs in the inheritance order of a given struct
	static RIGVM_API TArray<UStruct*> GetSuperStructs(UStruct* InStruct, bool bIncludeLeaf = true);

	// converts the types provided by a string (like "A:float,B:int32") into a type map
	RIGVM_API FTypeMap GetArgumentTypesFromString(const FString& InTypeString, const FRigVMUserDefinedTypeResolver* InTypeResolver = nullptr) const;

	// converts the types provided to a string (like "A:float,B:int32")
	static RIGVM_API FString GetStringFromArgumentTypes(const FTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry); 

#if WITH_EDITOR

	// Returns the color based on the permutation's metadata
	RIGVM_API FLinearColor GetColor(const TArray<int32>& InPermutationIndices = TArray<int32>()) const;

	// Returns the tooltip based on the permutation's metadata
	RIGVM_API FText GetTooltipText(const TArray<int32>& InPermutationIndices = TArray<int32>()) const;

	// Returns the display name text for an argument 
	RIGVM_API FText GetDisplayNameForArgument(const FName& InArgumentName, const TArray<int32>& InPermutationIndices = TArray<int32>()) const;

	// Returns meta data on the property of the permutations 
	RIGVM_API FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey, const TArray<int32>& InPermutationIndices = TArray<int32>()) const;

	RIGVM_API FString GetCategory() const;
	RIGVM_API FString GetKeywords() const;

#endif

	// Updates the template's argument types. This only affects templates which have category based
	// arguments and will resolve the other arguments to the expected types.
	RIGVM_API bool UpdateAllArgumentTypesSlow(FRigVMRegistryHandle& InRegistry);

	// Updates the template's argument types by adding a single new type
	RIGVM_API bool HandlePropagatedArgumentType(const TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry);

	// Invalidates template permutations whenever a type such as a user defined struct is removed
	RIGVM_API void HandleTypeRemoval(TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry);
	
	// Returns the delegate to be able to react to type changes dynamically
	// This delegate is deprecated
	FRigVMTemplate_NewArgumentTypeDelegate& OnNewArgumentType() { return Delegates.NewArgumentTypeDelegate; }

	// Returns the factory this template was created by
	RIGVM_API const FRigVMDispatchFactory* GetDispatchFactory() const;

	// Returns the factory this template was created by
	const FRigVMDispatchFactory* GetDispatchFactory_NoLock(const FRigVMRegistryHandle& InRegistry) const;
	
	// Returns true if this template is backed by a dispatch factory
	bool UsesDispatch() const
	{
		return Delegates.GetDispatchFactoryDelegate.IsBound();
	}

	RIGVM_API void RecomputeTypesHashToPermutations(const FRigVMRegistryHandle& InRegistry);
	RIGVM_API void RecomputeTypesHashToPermutations(const TArray<FRigVMTypeCacheScope_NoLock>& InTypeCaches, const FRigVMRegistryHandle& InRegistry);

	RIGVM_API void UpdateTypesHashToPermutation(const int32 InPermutation, const FRigVMRegistryHandle& InRegistry);

	RIGVM_API friend uint32 GetTypeHash(const FRigVMTemplate& InTemplate);
	RIGVM_API friend uint32 GetTypeHash_NoLock(const FRigVMTemplate& InTemplate, const FRigVMRegistryHandle& InRegistry);

	RIGVM_API TArray<FRigVMTemplateArgumentInfo> GetFlattenedArgumentInfos_NoLock(FRigVMRegistryHandle& InRegistry) const;
	RIGVM_API TArray<FRigVMTemplateArgumentInfo> GetFlattenedArgumentInfosForFunctions_NoLock(const TArray<const FRigVMFunction*>& InFunctions, FRigVMRegistryHandle& InRegistry) const;

	// Constructor from a struct, a template name and a function index
	RIGVM_API FRigVMTemplate(UScriptStruct* InStruct, const FString& InTemplateName, int32 InFunctionIndex, FRigVMRegistryHandle& InRegistry);

	// Constructor from a template name and argument infos
	RIGVM_API FRigVMTemplate(const FName& InTemplateName, const TArray<FRigVMTemplateArgumentInfo>& InInfos, const FRigVMRegistryHandle& InRegistry);

private:
	
	static FLinearColor GetColorFromMetadata(FString InMetadata);

	void UpdateTypesIndicesToPermutationIndices(int32 InPermutationIndex, const FRigVMRegistryHandle& InRegistry);
	void InvalidateHash() { Hash = UINT32_MAX; }
	RIGVM_API const TArray<FRigVMExecuteArgument>& GetExecuteArguments(const FRigVMDispatchContext& InContext) const;
	RIGVM_API const TArray<FRigVMExecuteArgument>& GetExecuteArguments_NoLock(const FRigVMDispatchContext& InContext, const FRigVMRegistryHandle& InRegistry) const;

	FTypeMap GetArgumentTypesFromString_NoLock(const FString& InTypeString, const FRigVMUserDefinedTypeResolver* InTypeResolver, FRigVMRegistryHandle& InRegistry) const;

	uint32 ComputeTypeHash(const FRigVMRegistryHandle& InRegistry) const;

	bool UpdateArgumentTypes_Impl(
		const FRigVMTemplateArgument& InPrimaryArgument,
		const TRigVMTypeIndex InPrimaryTypeIndex,
		FRigVMRegistryHandle& InRegistry, 
		const FRigVMDispatchFactory* InFactory,
		TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& InOutTypesArray);

	// returns true if this is a type which should be ignored by this template
	// since it got removed by unloading a plugin
	bool IsIgnoredType(const TRigVMTypeIndex& InTypeIndex) const;
	bool IsIgnoredType(const FName& InTypeName) const;

	int32 Index;
	FName Notation;
	TArray<FRigVMTemplateArgument> Arguments;
	mutable TArray<FRigVMExecuteArgument> ExecuteArguments;
	TArray<int32> Permutations;
	TMap<uint32, int32> TypesHashToPermutation;
	TMap<TRigVMTypeIndex, TArray<int32>> TypesIndicesToPermutationIndices;
	TArray<int32> RemovedPermutationIndices;
	mutable uint32 Hash;
	FRigVMRegistry_NoLock* OwnerRegistry;

	FRigVMTemplateDelegates Delegates;

	TArray<TRigVMTypeIndex> TypeIndicesToIgnore;
	TArray<FName> TypeNamesToIgnore;

	friend struct FRigVMRegistry_NoLock;
	friend class URigVMController;
	friend class URigVMLibraryNode;
	friend struct FRigVMDispatchFactory;
};

class FRigVMTypeCacheScope_NoLock
{
public:
	FRigVMTypeCacheScope_NoLock();
	FRigVMTypeCacheScope_NoLock(const FRigVMRegistryHandle& InRegistry, const FRigVMTemplateArgument& InArgument);
	~FRigVMTypeCacheScope_NoLock();

	bool IsValid() const { return Argument != nullptr; }
	const FRigVMTypeCacheScope_NoLock& UpdateIfRequired(const FRigVMRegistryHandle& InRegistry, const FRigVMTemplateArgument& InArgument);
	int32 GetNumTypes_NoLock() const;
	TRigVMTypeIndex GetTypeIndex_NoLock(int32 InIndex) const;
	
private:

	void UpdateTypesIfRequired() const;

	FRigVMRegistry_NoLock* Registry;
	const FRigVMTemplateArgument* Argument;
	bool bShouldCopyTypes;
	mutable TOptional<int32> NumTypes;
	mutable TOptional<TArray<TRigVMTypeIndex>> Types;
};
