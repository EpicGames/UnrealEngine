// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMDefines.h"
#include "RigVMCore/RigVMStruct.h"

#include "RigVMTrait.generated.h"

#if WITH_EDITOR

class URigVMController;
class URigVMNode;

#endif

/**
 * The base class for all RigVM traits.
 */
USTRUCT()
struct FRigVMTrait : public FRigVMStruct
{
	GENERATED_BODY()

public:

	FRigVMTrait()
	{}

	virtual ~FRigVMTrait() = default;

	// returns the name of the trait (the instance of it on the node)
	FString GetName() const { return Name; }

	// returns the display name of the trait
	virtual FString GetDisplayName() const
	{
		return FString();
	}

#if WITH_EDITOR

	// returns true if this trait can be added to a given node
	virtual bool CanBeAddedToNode(URigVMNode* InNode, FString* OutFailureReason) const { return true; }

	// allows the trait to react when added to a node
	virtual void OnTraitAdded(URigVMController* InController, URigVMNode* InNode) {}

	// allows the trait to return dynamic pins (parent pin index must be INDEX_NONE or point to a valid index of the parent pin in the OutPinArray)
	virtual void GetProgrammaticPins(URigVMController* InController, int32 InParentPinIndex, const URigVMPin* InTraitPin, const FString& InDefaultValue, struct FRigVMPinInfoArray& OutPinArray) const {}

	virtual UScriptStruct* GetTraitSharedDataStruct() const { return nullptr; }

	virtual bool ShouldCreatePinForProperty(const FProperty* InProperty) const override
	{
		if(!Super::ShouldCreatePinForProperty(InProperty))
		{
			return false;
		}
		return InProperty->GetFName() != GET_MEMBER_NAME_CHECKED(FRigVMTrait, Name);
	}

	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const { return FRigVMStructUpgradeInfo(); }
#endif

private:

	// The name of the trait on the node
	UPROPERTY()
	FString Name;
	
	friend class URigVMNode;
	friend class URigVMController;
	friend class FRigVMTraitScope;
};
