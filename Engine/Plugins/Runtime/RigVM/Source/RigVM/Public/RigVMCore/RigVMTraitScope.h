// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"

struct FRigVMTrait;
class UScriptStruct;
struct FRigVMMemoryHandle;

class FRigVMTraitScope
{
public:
	FRigVMTraitScope()
		: Trait(nullptr)
		, ScriptStruct(nullptr)
	{
	}

	FRigVMTraitScope(FRigVMTrait* InTrait, const UScriptStruct* InScriptStruct)
		: Trait(InTrait)
		, ScriptStruct(InScriptStruct)
	{
	}
	
	FRigVMTraitScope(FRigVMTrait* InTrait, const UScriptStruct* InScriptStruct, TConstArrayView<FRigVMMemoryHandle> InAdditionalMemoryHandles)
		: Trait(InTrait)
		, ScriptStruct(InScriptStruct)
		, AdditionalMemoryHandles(InAdditionalMemoryHandles)
	{
	}

	bool IsValid() const
	{
		return (Trait != nullptr) && (ScriptStruct != nullptr);
	}

	template<typename T>
	bool IsA() const
	{
		return ScriptStruct->IsChildOf(T::StaticStruct());
	}

	template<typename T = FRigVMTrait>
	const T* GetTrait() const
	{
		if(IsA<T>())
		{
			return static_cast<T*>(Trait);
		}
		return nullptr;
	}

	template<typename T = FRigVMTrait>
	const T* GetTraitChecked() const
	{
		check(IsA<T>());
		return static_cast<T*>(Trait);
	}

	template<typename T = FRigVMTrait>
	T* GetTrait()
	{
		if(IsA<T>())
		{
			return static_cast<T*>(Trait);
		}
		return nullptr;
	}

	template<typename T = FRigVMTrait>
	T* GetTraitChecked()
	{
		check(IsA<T>());
		return static_cast<T*>(Trait);
	}

	const UScriptStruct* GetScriptStruct() const
	{
		return ScriptStruct;
	}

	TConstArrayView<FRigVMMemoryHandle> GetAdditionalMemoryHandles() const
	{
		return AdditionalMemoryHandles;
	}

private:

	FRigVMTrait* Trait;
	const UScriptStruct* ScriptStruct;
	TConstArrayView<FRigVMMemoryHandle> AdditionalMemoryHandles;
};