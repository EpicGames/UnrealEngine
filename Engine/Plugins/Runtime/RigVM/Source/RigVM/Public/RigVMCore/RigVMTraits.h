// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "Templates/Models.h"
#include "RigVMDefines.h"

struct CRigVMUStruct
{
	template <typename T>
	auto Requires(UScriptStruct*& Val) -> decltype(
		Val = T::StaticStruct()
	);
};

struct CRigVMUClass
{
	template <typename T>
	auto Requires(UClass*& Val) -> decltype(
		Val = T::StaticClass()
	);
};

template <typename T>
struct TRigVMIsBool
{
	enum { Value = false };
};
template <> struct TRigVMIsBool<bool> { enum { Value = true }; };

template <typename T>
struct TRigVMIsFloat
{
	enum { Value = false };
};
template <> struct TRigVMIsFloat<float> { enum { Value = true }; };

template <typename T>
struct TRigVMIsDouble
{
	enum { Value = false };
};
template <> struct TRigVMIsDouble<double> { enum { Value = true }; };

template <typename T>
struct TRigVMIsInt32
{
	enum { Value = false };
};
template <> struct TRigVMIsInt32<int32> { enum { Value = true }; };

template <typename T>
struct TRigVMIsName
{
	enum { Value = false };
};
template <> struct TRigVMIsName<FName> { enum { Value = true }; };

template <typename T>
struct TRigVMIsString
{
	enum { Value = false };
};
template <> struct TRigVMIsString<FString> { enum { Value = true }; };

template <typename T>
struct TRigVMIsBaseStructure
{
	enum { Value = false };
};

template<> struct TRigVMIsBaseStructure<FRotator> { enum { Value = true }; };
template<> struct TRigVMIsBaseStructure<FQuat> { enum { Value = true }; };
template<> struct TRigVMIsBaseStructure<FTransform> { enum { Value = true }; };
struct FLinearColor;
template<> struct TRigVMIsBaseStructure<FLinearColor> { enum { Value = true }; };
struct FColor;
template<> struct TRigVMIsBaseStructure<FColor> { enum { Value = true }; };

template<> struct TRigVMIsBaseStructure<FPlane> { enum { Value = true }; };

template<> struct TRigVMIsBaseStructure<FVector> { enum { Value = true }; };

template<> struct TRigVMIsBaseStructure<FVector2D> { enum { Value = true }; };

template<> struct TRigVMIsBaseStructure<FVector4> { enum { Value = true }; };
struct FRandomStream;
template<> struct TRigVMIsBaseStructure<FRandomStream> { enum { Value = true }; };
struct FGuid;
template<> struct TRigVMIsBaseStructure<FGuid> { enum { Value = true }; };

template<> struct TRigVMIsBaseStructure<FBox2D> { enum { Value = true }; };
struct FFallbackStruct;
template<> struct TRigVMIsBaseStructure<FFallbackStruct> { enum { Value = true }; };
struct FFloatRangeBound;
template<> struct TRigVMIsBaseStructure<FFloatRangeBound> { enum { Value = true }; };
struct FFloatRange;
template<> struct TRigVMIsBaseStructure<FFloatRange> { enum { Value = true }; };
struct FInt32RangeBound;
template<> struct TRigVMIsBaseStructure<FInt32RangeBound> { enum { Value = true }; };
struct FInt32Range;
template<> struct TRigVMIsBaseStructure<FInt32Range> { enum { Value = true }; };
struct FFloatInterval;
template<> struct TRigVMIsBaseStructure<FFloatInterval> { enum { Value = true }; };
struct FInt32Interval;
template<> struct TRigVMIsBaseStructure<FInt32Interval> { enum { Value = true }; };
struct FFrameNumber;
template<> struct TRigVMIsBaseStructure<FFrameNumber> { enum { Value = true }; };
struct FSoftObjectPath;
template<> struct TRigVMIsBaseStructure<FSoftObjectPath> { enum { Value = true }; };
struct FSoftClassPath;
template<> struct TRigVMIsBaseStructure<FSoftClassPath> { enum { Value = true }; };
struct FPrimaryAssetType;
template<> struct TRigVMIsBaseStructure<FPrimaryAssetType> { enum { Value = true }; };
struct FPrimaryAssetId;
template<> struct TRigVMIsBaseStructure<FPrimaryAssetId> { enum { Value = true }; };
struct FDateTime;
template<> struct TRigVMIsBaseStructure<FDateTime> { enum { Value = true }; };
struct FPolyglotTextData;
template<> struct TRigVMIsBaseStructure<FPolyglotTextData> { enum { Value = true }; };

///////////////////////////////////////////////////////////////////////////////////////

template<
	typename T, 
	typename TEnableIf<TIsArithmetic<T>::Value, bool>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount = 1)
{
	ensure(InCount >= 0);
	FMemory::Memzero(InPtr, InCount * sizeof(T));
}

template<
	typename T,
	typename TEnableIf<TIsArithmetic<T>::Value, bool>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount = 1)
{
	// nothing to do
}

template<
	typename T,
	typename TEnableIf<TIsArithmetic<T>::Value, bool>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	ensure(InCount >= 0);
	FMemory::Memcpy(InTargetPtr, InSourcePtr, InCount * sizeof(T));
}

template<
	typename T,
	typename TEnableIf<TRigVMIsString<T>::Value>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	FMemory::Memzero(InPtr, InCount * sizeof(FString));

	FString* Values = (FString*)InPtr;
	for(int32 Index=0;Index<InCount;Index++)
	{
		Values[Index] = FString();
	}
}

template<
	typename T,
	typename TEnableIf<TRigVMIsString<T>::Value>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	FString* Values = (FString*)InPtr;
	for(int32 Index=0;Index<InCount;Index++)
	{
		Values[Index] = FString();
	}
}

template<
	typename T,
	typename TEnableIf<TRigVMIsString<T>::Value>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	ensure(InCount >= 0);

	FString* Target = (FString*)InTargetPtr;
	const FString* Source = (const FString*)InSourcePtr;
	for(int32 Index=0;Index<InCount;Index++)
	{
		Target[Index] = Source[Index];
	}
}

template<
	typename T,
	typename TEnableIf<TRigVMIsName<T>::Value, bool>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	FMemory::Memzero(InPtr, InCount * sizeof(FName));
}

template<
	typename T,
	typename TEnableIf<TRigVMIsName<T>::Value, bool>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	FMemory::Memzero(InPtr, InCount * sizeof(FName));
}

template<
	typename T,
	typename TEnableIf<TRigVMIsName<T>::Value, bool>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	ensure(InCount >= 0);
	FMemory::Memcpy(InTargetPtr, InSourcePtr, InCount * sizeof(FName));
}

template <
	typename T,
	typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	T::StaticStruct()->InitializeStruct(InPtr, InCount);
}

template <
	typename T,
	typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	T::StaticStruct()->DestroyStruct(InPtr, InCount);
}

template <
	typename T,
	typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	ensure(InCount >= 0);
	T::StaticStruct()->CopyScriptStruct(InTargetPtr, InSourcePtr, InCount);
}

template <
	typename T,
	typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount)
{
	FMemory::Memzero(InPtr, InCount * sizeof(T));
}

template <
	typename T,
	typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount)
{
	// nothing to do
}

template <
	typename T,
	typename TEnableIf<TIsEnum<T>::Value>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	FMemory::Memcpy(InTargetPtr, InSourcePtr, InCount * sizeof(T));
}
template <
	typename T,
	typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
>
void RigVMInitialize(void* InPtr, int32 InCount)
{
	ensure(InCount >= 0);
	TBaseStructure<T>::Get()->InitializeStruct(InPtr, InCount);
}

template <
	typename T,
	typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
>
void RigVMDestroy(void* InPtr, int32 InCount)
{
	// nothing to do
}

template <
	typename T,
	typename TEnableIf<TRigVMIsBaseStructure<T>::Value, T>::Type* = nullptr
>
void RigVMCopy(void* InTargetPtr, const void* InSourcePtr, int32 InCount = 1)
{
	ensure(InCount >= 0);
	FMemory::Memcpy(InTargetPtr, InSourcePtr, InCount * sizeof(T));
}
