// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMDefines.h"

#if UE_RIGVM_DEBUG_TYPEINDEX
class TRigVMTypeIndex
{
public:

	// default constructor provided just to avoid compilation error
	// in your code if you need to represent an invalid type, use INDEX_NONE instead of TRigVMTypeIndex()
	// the reason being that because we typedef int32 to TRigVMTypeIndex,
	// default constructor of TRigVMTypeIndex should be the same as int32, which defaults to 0 and is not an invalid type
	TRigVMTypeIndex()
	: Name(NAME_None)
	, Index(0)
	{}
	
	TRigVMTypeIndex(int32 InIndex)
		: Name(NAME_None)
		, Index(InIndex)
	{}

	int32 GetIndex() const { return Index; }
	const FName& GetName() const { return Name; }

	operator int() const
	{
		return Index;
	}

	bool operator ==(const TRigVMTypeIndex& Other) const
	{
		return Index == Other.Index;
	}

	bool operator ==(const int32& Other) const
	{
		return Index == Other;
	}

	bool operator !=(const TRigVMTypeIndex& Other) const
	{
		return Index != Other.Index;
	}

	bool operator !=(const int32& Other) const
	{
		return Index != Other;
	}

	bool operator >(const TRigVMTypeIndex& Other) const
	{
		return Index > Other.Index;
	}

	bool operator >(const int32& Other) const
	{
		return Index > Other;
	}

	bool operator <(const TRigVMTypeIndex& Other) const
	{
		return Index < Other.Index;
	}

	bool operator <(const int32& Other) const
	{
		return Index < Other;
	}

	friend uint32 GetTypeHash(const TRigVMTypeIndex& InIndex)
	{
		return GetTypeHash(InIndex.Index);
	}

protected:
	FName Name;
	int32 Index;

	friend struct FRigVMRegistry_NoLock;
	friend struct FRigVMTemplateArgument;
};
#else
typedef int32 TRigVMTypeIndex;
#endif
