// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RigVMDefines.h"
#include "UObject/ObjectMacros.h"

#include "RigVMUnknownType.generated.h"

/**
 * The unknown type is used to identify untyped nodes
 */
USTRUCT(meta=(DisplayName="Wildcard"))
struct FRigVMUnknownType
{
	GENERATED_BODY()

	FRigVMUnknownType()
		: Hash(0)
	{}

private:
	
	UPROPERTY()
	uint32 Hash;
};
