// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/AssertionMacros.h"
#include "RigVMCore/RigVMTraits.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include "RigVMUserWorkflow.generated.h"

#define UE_API RIGVM_API

class FProperty;
class URigVMUserWorkflowOptions;
struct FFrame;

DECLARE_DELEGATE_ThreeParams(FRigVMReportDelegate, EMessageSeverity::Type, UObject*, const FString&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMPerformUserWorkflowDelegate, const URigVMUserWorkflowOptions*, UObject*);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FRigVMPeformUserWorkflowDynamicDelegate, const URigVMUserWorkflowOptions*, InOptions, UObject*, InController);

// Types of workflows offered by a rigvm struct node
UENUM(BlueprintType)
enum class ERigVMUserWorkflowType : uint8
{
	Invalid = 0 UMETA(Hidden),
	NodeContext = 0x001,
	PinContext = 0x002,
	OnPinDefaultChanged  = 0x004,
	NodeContextButton  = 0x008,
	All = NodeContext | PinContext | OnPinDefaultChanged | NodeContextButton
};

USTRUCT(BlueprintType)
struct FRigVMUserWorkflow
{
	GENERATED_BODY()

public:

	FRigVMUserWorkflow()
	: Type(ERigVMUserWorkflowType::Invalid)
	, OptionsClass(nullptr)
	{}

	FRigVMUserWorkflow(
		const FString& InTitle,
		const FString& InTooltip,
		ERigVMUserWorkflowType InType,
		FRigVMPerformUserWorkflowDelegate InGetActionsDelegate,
		UClass* InOptionsClass)
	: Title(InTitle)
	, Tooltip(InTooltip)
	, Type(InType)
	, PerformDelegate(InGetActionsDelegate)
	, OptionsClass(InOptionsClass)
	{}
	
	virtual ~FRigVMUserWorkflow() = default;

	bool IsValid() const
	{
		return Type != ERigVMUserWorkflowType::Invalid &&
			GetOptionsClass() != nullptr &&
			(PerformDelegate.IsBound() || PerformDynamicDelegate.IsBound());
	}

	const FString& GetTitle() const { return Title; }
	const FString& GetTooltip() const { return Tooltip; }
	ERigVMUserWorkflowType GetType() const { return Type; }
	UClass* GetOptionsClass() const { return OptionsClass; }

	UE_API bool Perform(const URigVMUserWorkflowOptions* InOptions, UObject* InController) const;

protected:

	UE_API bool ValidateOptions(const URigVMUserWorkflowOptions* InOptions) const;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Workflow, meta = (AllowPrivateAccess = true))
	FString Title;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Workflow, meta = (AllowPrivateAccess = true))
	FString Tooltip;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Workflow, meta = (AllowPrivateAccess = true))
	ERigVMUserWorkflowType Type;

	FRigVMPerformUserWorkflowDelegate PerformDelegate;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Workflow, meta = (ScriptName = "OnPerformWorkflow", AllowPrivateAccess = true))
	FRigVMPeformUserWorkflowDynamicDelegate PerformDynamicDelegate;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Workflow, meta = (AllowPrivateAccess = true))
	TObjectPtr<UClass> OptionsClass;
};

UCLASS(MinimalAPI, BlueprintType)
class URigVMUserWorkflowOptions : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category=Options)
	bool IsValid() const { return Subject != nullptr; }
	
	UFUNCTION(BlueprintCallable, Category=Options)
	UE_API bool RequiresDialog() const;

	template<typename T = UObject>
	T* GetSubject() const { return Cast<T>(Subject.Get()); }

	UObject* GetSubjectChecked() const
	{
		UObject* Object = Subject.Get();
		check(Object);
		return Object;
	}

	template<typename T>
	T* GetSubjectChecked() const { return CastChecked<T>(Subject.Get()); }

	const FRigVMUserWorkflow& GetWorkflow() const { return Workflow; }

	UE_API void Report(EMessageSeverity::Type InSeverity, const FString& InMessage) const;

	template <typename... Types>
	void Reportf(EMessageSeverity::Type InSeverity, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		Report(InSeverity, FString::Printf(Fmt, Args...));
	}

	UFUNCTION(BlueprintCallable, Category=Options)
	void ReportError(const FString& InMessage)
	{
		Report(EMessageSeverity::Error, InMessage);
	}

	UFUNCTION(BlueprintCallable, Category=Options)
	void ReportWarning(const FString& InMessage)
	{
		Report(EMessageSeverity::Warning, InMessage);
	}

	UFUNCTION(BlueprintCallable, Category=Options)
	void ReportInfo(const FString& InMessage)
	{
		Report(EMessageSeverity::Info, InMessage);
	}

protected:

	UE_API virtual bool RequiresDialog(const FProperty* InProperty) const;

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = Options, meta = (AllowPrivateAccess = true))
	TObjectPtr<UObject> Subject;

	UPROPERTY(BlueprintReadOnly, EditDefaultsOnly, Category = Options, meta = (AllowPrivateAccess = true))
	FRigVMUserWorkflow Workflow;

	FRigVMReportDelegate ReportDelegate;

	friend class URigVMController;
};

#undef UE_API
