// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/Guid.h"

#include "RigVMVariableDescription.generated.h"

struct FRigVMExternalVariable;
struct FPropertyBagPropertyDesc;
struct FRigVMPropertyDescription;

/**
 * The variable description is used to convey information
 * about unique variables within a Graph. Multiple Variable
 * Nodes can share the same variable description.
 */
USTRUCT(BlueprintType)
struct FRigVMGraphVariableDescription
{
	GENERATED_BODY()

public:

	// comparison operator
	bool operator ==(const FRigVMGraphVariableDescription& Other) const
	{
		if (Guid.IsValid() && Other.Guid.IsValid())
		{
			return Guid == Other.Guid;
		}
		return Name == Other.Name;
	}
	
	// The guid of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FGuid Guid;

	// The name of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FName Name;

	// The C++ data type of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FString CPPType;

	// The Struct of the C++ data type of the variable (or nullptr)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	TObjectPtr<UObject> CPPTypeObject = nullptr;
	
	UPROPERTY()
	FName CPPTypeObjectPath;

	// The default value of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FString DefaultValue;

	// The category of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FText Category;

	// The tooltip of the variable
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	FText Tooltip;

	// Should this variable be exposed on spawn
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	bool bExposedOnSpawn = false;

	// Should this variable be exposed on spawn
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	bool bExposeToCinematics = false;

	// Is this variable public
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	bool bPublic = false;

	// This property should soon become deprecated, since it does not make sense for blueprint independent assets
	// Is this variable private
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = RigVMGraphVariableDescription)
	bool bPrivate = true;

	// Returns nullptr external variable matching this description
	RIGVM_API FRigVMExternalVariable ToExternalVariable() const;

	RIGVM_API bool ChangeType(const FString& InCPPType, UObject* InCPPTypeObject);
	
};

namespace RigVMVariableUtils
{
	RIGVM_API FRigVMExternalVariable ExternalVariableFromRigVMVariableDescription(const FRigVMGraphVariableDescription& InVariableDescription);

	RIGVM_API FRigVMGraphVariableDescription VariableDescriptionFromPropertyDesc(const FPropertyBagPropertyDesc& InPropertyDesc);

	RIGVM_API FRigVMPropertyDescription PropertyDescriptionFromVariableDescription(const FRigVMGraphVariableDescription& InVariableDescription, const bool bAllowSpacesInName);
}
