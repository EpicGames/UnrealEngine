// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMObjectVersion.h"
#include "HAL/IConsoleManager.h"
#include "RigVMVariant.generated.h"

struct FRigVMVariantRef;

extern RIGVM_API TAutoConsoleVariable<bool> CVarRigVMEnableVariants;

// User applied tag
USTRUCT(BlueprintType)
struct FRigVMTag
{
	GENERATED_BODY()

	FRigVMTag()
		: Name(NAME_None)
		, Label()
		, ToolTip()
		, Color(FLinearColor::White)
		, bShowInUserInterface(true)
		, bMarksSubjectAsInvalid(false)
	{}

	FRigVMTag(const FName&  InName, const FString& InLabel, const FText& InToolTip, const FLinearColor& InColor, bool InShowInUserInterface = true, bool InMarksSubjectAsInvalid = false)
	: Name(InName)
	, Label(InLabel)
	, ToolTip(InToolTip)
	, Color(InColor)
	, bShowInUserInterface(InShowInUserInterface)
	, bMarksSubjectAsInvalid(InMarksSubjectAsInvalid)
	{}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant)
	FName Name;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant)
	FString Label;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant)
	FText ToolTip;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant)
	FLinearColor Color;

	// Enabling this will show tags in the user interface within
	// the variant widgets and other places.
	// Disabling this will interpret this tag as automation / scripting only.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant)
	bool bShowInUserInterface;

	// Enabling this will mark the subject (asset, function etc) as (soft) deprecated.
	// The content referring to variants with this tag will continue to work, but the
	// user interface will suggest the user to upgrade to a later variant.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Variant, meta=(DisplayName="Mark Subject Deprecated"))
	bool bMarksSubjectAsInvalid;

	bool IsValid() const
	{
		return !Name.IsNone();
	}

	FString GetLabel() const
	{
		if(Label.IsEmpty())
		{
			return Name.ToString();
		}
		return Label;
	}

	friend uint32 GetTypeHash(const FRigVMTag& InTag)
	{
		uint32 Hash = GetTypeHash(InTag.Name.ToString());
		Hash = HashCombine(Hash, GetTypeHash(InTag.Label));
		Hash = HashCombine(Hash, GetTypeHash(InTag.ToolTip.ToString()));
		Hash = HashCombine(Hash, GetTypeHash(InTag.Color));
		Hash = HashCombine(Hash, GetTypeHash(InTag.bShowInUserInterface));
		Hash = HashCombine(Hash, GetTypeHash(InTag.bMarksSubjectAsInvalid));
		return Hash;
	}
	
	friend FArchive& operator<<(FArchive& Ar, FRigVMTag& Data)
	{
		Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

		Ar << Data.Name;
		Ar << Data.Label;
		Ar << Data.ToolTip;
		Ar << Data.Color;
		Ar << Data.bShowInUserInterface;
		Ar << Data.bMarksSubjectAsInvalid;
		return Ar;
	}

	friend bool operator==(const FRigVMTag& A, const FRigVMTag& B)
	{
		return A.Name == B.Name &&
			A.Label == B.Label &&
			A.ToolTip.EqualTo(B.ToolTip) &&
			A.Color == B.Color &&
			A.bShowInUserInterface == B.bShowInUserInterface &&
			A.bMarksSubjectAsInvalid == B.bMarksSubjectAsInvalid;
	}
};

USTRUCT(BlueprintType)
struct FRigVMVariant
{
	GENERATED_BODY()

	FRigVMVariant(const FRigVMVariant& OtherVariant)
	{
		Guid = OtherVariant.Guid;
		Tags = OtherVariant.Tags;
	}

	// Guid which is shared by all variants of the same element
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Variant)
	FGuid Guid;
	
	// Tags applied to this variant
	UPROPERTY(BlueprintReadWrite, VisibleAnywhere, Category=Variant)
	TArray<FRigVMTag> Tags;

	FRigVMVariant()
		: Guid(FGuid()) {}

	bool IsValid() const
	{
		return Guid.IsValid();
	}

	friend FArchive& operator<<(FArchive& Ar, FRigVMVariant& Data)
	{
		Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);
		
		Ar << Data.Guid;
		Ar << Data.Tags;
		return Ar;
	}

	static FGuid GenerateGUID(const FString InPath = FString())
	{
		if (!InPath.IsEmpty())
		{
			return FGuid::NewDeterministicGuid(InPath);
		}
		return FGuid::NewGuid();
	}
};

// This struct should not be serialized.
// It is generated on demand.
USTRUCT(BlueprintType)
struct FRigVMVariantRef
{
	GENERATED_BODY()
	
	FRigVMVariantRef(){}

	FRigVMVariantRef(const FSoftObjectPath& InPath, const FRigVMVariant& InVariant)
		: ObjectPath(InPath), Variant(InVariant) {}

	bool operator == (const FRigVMVariantRef& Other) const
	{
		return Variant.Guid == Other.Variant.Guid && ObjectPath == Other.ObjectPath; 
	}

	bool IsValid() const
	{
		return ObjectPath.IsValid() && Variant.IsValid();
	}

	friend uint32 GetTypeHash(const FRigVMVariantRef& InVariantRef)
	{
		return HashCombine(GetTypeHash(InVariantRef.ObjectPath.ToString()), GetTypeHash(InVariantRef.Variant.Guid));
	}

	UPROPERTY(BlueprintReadOnly, Category=RigVMVariantRef)
	FSoftObjectPath ObjectPath;

	UPROPERTY(BlueprintReadOnly, Category=RigVMVariantRef)
	FRigVMVariant Variant;
};
