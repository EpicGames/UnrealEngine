// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMDefines.h: Global defines
=============================================================================*/

#pragma once

#ifndef UE_RIGVM_UCLASS_BASED_STORAGE_DISABLED
	#define UE_RIGVM_UCLASS_BASED_STORAGE_DISABLED 0
#endif

#ifndef UE_RIGVM_UOBJECT_PROPERTIES_ENABLED
	#define UE_RIGVM_UOBJECT_PROPERTIES_ENABLED 1
#endif

#ifndef UE_RIGVM_UINTERFACE_PROPERTIES_ENABLED
	#define UE_RIGVM_UINTERFACE_PROPERTIES_ENABLED 1
#endif

#ifndef UE_RIGVM_AGGREGATE_NODES_ENABLED
	#define UE_RIGVM_AGGREGATE_NODES_ENABLED 1
#endif

#ifndef UE_RIGVM_DEBUG_TYPEINDEX
	#define UE_RIGVM_DEBUG_TYPEINDEX 1
#endif

// in shipping and test always disable debugging the type index
#if defined(UE_BUILD_SHIPPING) && UE_BUILD_SHIPPING || defined(UE_BUILD_TEST) && UE_BUILD_TEST
	#undef UE_RIGVM_DEBUG_TYPEINDEX
	#define UE_RIGVM_DEBUG_TYPEINDEX 0
#endif

#ifndef UE_RIGVM_DEBUG_EXECUTION
	#define UE_RIGVM_DEBUG_EXECUTION 0
#endif

#ifndef UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
	#define UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE 0
#endif

#ifndef UE_RIGVM_ARCHIVETRACE_ENABLE
	#define UE_RIGVM_ARCHIVETRACE_ENABLE 0
#endif

#if WITH_EDITOR
#else
	#undef UE_RIGVM_ARCHIVETRACE_ENABLE
	#define UE_RIGVM_ARCHIVETRACE_ENABLE 0
#endif

#if UE_RIGVM_ARCHIVETRACE_ENABLE

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

class FRigVMArchiveTrace
{
public:

	FRigVMArchiveTrace()
		: Counter(1)
		, Archive(nullptr)
	{
	}

private:
	
	int32 Counter;
	FArchive* Archive;

	static RIGVM_API FRigVMArchiveTrace* AddRefTrace(FArchive* InArchive);
	static RIGVM_API void DecRefTrace(FRigVMArchiveTrace* InTrace);

	static RIGVM_API TMap<FArchive*, TSharedPtr<FRigVMArchiveTrace>> ActiveTraces;
	static RIGVM_API FCriticalSection AddRemoveTracesMutex;
	
	friend class FRigVMArchiveTraceBracket;
};

class FRigVMArchiveTraceBracket
{
public:

	RIGVM_API FRigVMArchiveTraceBracket(FArchive& InArchive, const FString& InScope);
	RIGVM_API ~FRigVMArchiveTraceBracket();

	RIGVM_API void AddEntry(FArchive& InArchive, const FString& InScope);

	static RIGVM_API FString WhiteSpace(int32 InCount);
	static RIGVM_API FString ArchiveOffsetToString(int64 InOffset);
	
private:

	FRigVMArchiveTrace* Trace;
	int32 Indentation;
	int64 ArchivePos;
	int64 LastArchivePos;
	bool bEnabled;
	FString ArchiveName;
	FString ArchiveWhiteSpace;
	FString ArchivePrefix;
};
#endif

#ifndef UE_RIGVM_ARCHIVETRACE_SCOPE
	#if UE_RIGVM_ARCHIVETRACE_ENABLE
		#define UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, Scope) \
			FRigVMArchiveTraceBracket _ArchiveTraceBracket(Ar, FString(Scope));
	#else
		#define UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, Scope)
	#endif
#endif

// if you are hitting an error with this - you likely have to place UE_RIGVM_ARCHIVETRACE_SCOPE in the top of the scope
#ifndef UE_RIGVM_ARCHIVETRACE_ENTRY
	#if UE_RIGVM_ARCHIVETRACE_ENABLE
		#define UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, Scope) \
			_ArchiveTraceBracket.AddEntry(Ar, Scope);
	#else
		#define UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, Scope)
	#endif
#endif
