// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_AnimBase.generated.h"

/*
 * The base class for all animation related nodes
 */
USTRUCT(meta=(Abstract, Category = "Animation", NodeColor = "0.05 0.05 0.25", DocumentationPolicy = "Strict"))
struct FRigVMFunction_AnimBase : public FRigVMStruct
{
	GENERATED_BODY()
};

