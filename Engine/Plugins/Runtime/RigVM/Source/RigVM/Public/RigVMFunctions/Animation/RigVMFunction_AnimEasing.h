// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "RigVMFunction_AnimEasing.generated.h"

/**
 * A constant value of an easing type
 */
USTRUCT(meta = (DisplayName = "EaseType", Keywords = "Constant"))
struct FRigVMFunction_AnimEasingType : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()

	FRigVMFunction_AnimEasingType()
	{
		Type = ERigVMAnimEasingType::CubicEaseInOut;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The pass through constant easing type
	UPROPERTY(meta = (Input, Output))
	ERigVMAnimEasingType Type;
};

/**
 * Returns the eased version of the input value
 */
USTRUCT(meta=(DisplayName="Ease", Keywords="Easing,Profile,Smooth,Cubic"))
struct FRigVMFunction_AnimEasing : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AnimEasing()
	{
		Value = Result = 0.f;
		Type = ERigVMAnimEasingType::CubicEaseInOut;
		SourceMinimum = TargetMinimum = 0.f;
		SourceMaximum = TargetMaximum = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to ease using the easing functions
	UPROPERTY(meta=(Input))
	float Value;

	// The type of easing to apply
	UPROPERTY(meta=(Input))
	ERigVMAnimEasingType Type;

	// The minimum value of the input
	UPROPERTY(meta=(Input))
	float SourceMinimum;

	// The maximum value of the input
	UPROPERTY(meta=(Input))
	float SourceMaximum;

	// The minimum value of the output
	UPROPERTY(meta=(Input))
	float TargetMinimum;

	// The maximum value of the output
	UPROPERTY(meta=(Input))
	float TargetMaximum;

	// The resulting eased output value
	UPROPERTY(meta=(Output))
	float Result;
};

