// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "Curves/CurveFloat.h"
#include "RigVMFunction_AnimEvalRichCurve.generated.h"

#define UE_API RIGVM_API

/**
 * Evaluates the provided curve. Values are normalized between 0 and 1
 */
USTRUCT(meta=(DisplayName="Evaluate Curve", Keywords="Curve,Profile"))
struct FRigVMFunction_AnimEvalRichCurve : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AnimEvalRichCurve()
	{
		Value = Result = 0.f;
		Curve = FRuntimeFloatCurve();
		Curve.GetRichCurve()->AddKey(0.f, 0.f);
		Curve.GetRichCurve()->AddKey(1.f, 1.f);
		SourceMinimum = TargetMinimum = 0.f;
		SourceMaximum = TargetMaximum = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to evaluate the curve at
	UPROPERTY(meta=(Input))
	float Value;

	// The curve to evaluate
	UPROPERTY(meta=(Input))
	FRuntimeFloatCurve Curve;

	// The minimum range of the input value (typically 0.0)
	UPROPERTY(meta=(Input))
	float SourceMinimum;

	// The maximum range of the input value (typically 1.0)
	UPROPERTY(meta=(Input))
	float SourceMaximum;

	// The minimum range of the output
	UPROPERTY(meta=(Input))
	float TargetMinimum;

	// The maximum range of the output
	UPROPERTY(meta=(Input))
	float TargetMaximum;

	// The evaluated value of the curve at the input value
	UPROPERTY(meta=(Output))
	float Result;
};

#undef UE_API
