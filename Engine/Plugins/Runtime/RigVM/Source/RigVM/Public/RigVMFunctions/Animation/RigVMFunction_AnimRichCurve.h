// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "Curves/CurveFloat.h"
#include "RigVMFunction_AnimRichCurve.generated.h"

/**
 * Provides a constant curve to be used for multiple curve evaluations
 */
USTRUCT(meta=(DisplayName="Curve", Keywords="Curve,Profile"))
struct FRigVMFunction_AnimRichCurve : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AnimRichCurve()
	{
		Curve = FRuntimeFloatCurve();
		Curve.GetRichCurve()->AddKey(0.f, 0.f);
		Curve.GetRichCurve()->AddKey(1.f, 1.f);
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The pass through animation rich curve
	UPROPERTY(meta=(Input, Output, Constant))
	FRuntimeFloatCurve Curve;
};
