// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "RigVMFunction_GetDeltaTime.generated.h"

/**
 * Returns the time gone by from the previous evaluation
 */
USTRUCT(meta=(DisplayName="Delta Time", Varying))
struct FRigVMFunction_GetDeltaTime : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_GetDeltaTime()
	{
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The current delta time in seconds
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the time gone by since the start of the world / the time within sequencer 
 */
USTRUCT(meta=(DisplayName="Absolute Time", Varying))
struct FRigVMFunction_GetAbsoluteTime : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_GetAbsoluteTime()
	{
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The current absolute time in seconds
	UPROPERTY(meta=(Output))
	float Result;
};
