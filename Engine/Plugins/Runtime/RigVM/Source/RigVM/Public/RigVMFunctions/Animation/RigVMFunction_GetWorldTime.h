// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "RigVMFunction_GetWorldTime.generated.h"

/**
 * Returns the current time (year, month, day, hour, minute)
 */
USTRUCT(meta = (DisplayName = "Now", Keywords = "Time,Clock", Varying))
struct FRigVMFunction_GetWorldTime : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()

	FRigVMFunction_GetWorldTime()
	{
		Year = Month = Day = WeekDay = Hours = Minutes = Seconds = OverallSeconds = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The Year of the world time
	UPROPERTY(meta = (Output))
	float Year;

	// The Month of the world time
	UPROPERTY(meta = (Output))
	float Month;

	// The Day of the world time
	UPROPERTY(meta = (Output))
	float Day;

	// The WeekDay of the world time
	UPROPERTY(meta = (Output))
	float WeekDay;

	// The Hours of the world time
	UPROPERTY(meta = (Output))
	float Hours;

	// The Minutes of the world time
	UPROPERTY(meta = (Output))
	float Minutes;

	// The Seconds of the world time
	UPROPERTY(meta = (Output))
	float Seconds;

	// The OverallSeconds of the world time
	UPROPERTY(meta = (Output))
	float OverallSeconds;
};

