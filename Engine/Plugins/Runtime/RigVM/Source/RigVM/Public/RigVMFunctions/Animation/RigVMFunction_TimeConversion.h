// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_AnimBase.h"
#include "RigVMFunction_TimeConversion.generated.h"

/**
 * Converts frames to seconds based on the current frame rate
 */
USTRUCT(meta=(DisplayName="Frames to Seconds", Varying))
struct FRigVMFunction_FramesToSeconds : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_FramesToSeconds()
	{
		Seconds = Frames = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value representing a number of frames to convert to seconds
	UPROPERTY(meta=(Input))
	float Frames;

	// The resulting time in seconds
	UPROPERTY(meta=(Output))
	float Seconds;
};

/**
 * Converts seconds to frames based on the current frame rate
 */
USTRUCT(meta=(DisplayName="Seconds to Frames", Varying))
struct  FRigVMFunction_SecondsToFrames : public FRigVMFunction_AnimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_SecondsToFrames()
	{
		Seconds = Frames = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input time as seconds
	UPROPERTY(meta=(Input))
	float Seconds;

	// The resulting number of frames 
	UPROPERTY(meta=(Output))
	float Frames;
};
