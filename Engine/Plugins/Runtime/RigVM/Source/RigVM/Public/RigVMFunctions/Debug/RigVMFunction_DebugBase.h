// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMDebugDrawSettings.h"
#include "RigVMFunction_DebugBase.generated.h"

/*
 * The base class for all pure debug draw nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict"))
struct FRigVMFunction_DebugBase : public FRigVMStruct
{
	GENERATED_BODY()

	// The debug draw settings for this node
	UPROPERTY(meta = (Input, DetailsOnly, DisplayName = "Draw Settings"))
	FRigVMDebugDrawSettings DebugDrawSettings;
};

/*
 * The base class for all mutable debug draw nodes
 */
USTRUCT(meta=(Abstract, Category="Debug", NodeColor = "0.83077 0.846873 0.049707", DocumentationPolicy = "Strict", Pure))
struct FRigVMFunction_DebugBaseMutable : public FRigVMStructMutable
{
	GENERATED_BODY()

	// The debug draw settings for this node
	UPROPERTY(meta = (Input, DetailsOnly, DisplayName = "Draw Settings"))
	FRigVMDebugDrawSettings DebugDrawSettings;
};
