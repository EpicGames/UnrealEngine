// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_DebugLine.generated.h"

#define UE_API RIGVM_API

/**
 * Draws a line in the viewport given a start and end vector
 */
USTRUCT(meta=(DisplayName="Draw Line"))
struct FRigVMFunction_DebugLineNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugLineNoSpace()
	{
		A = B = FVector::ZeroVector;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The global start position of the line
	UPROPERTY(meta = (Input))
	FVector A;

	// The global end position of the line
	UPROPERTY(meta = (Input))
	FVector B;

	// The color to use for the drawing
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use for the drawing
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};

#undef UE_API