// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_DebugLineStrip.generated.h"

/**
 * Draws a line strip in the viewport given any number of points
 */
USTRUCT(meta=(DisplayName="Draw Line Strip"))
struct FRigVMFunction_DebugLineStripNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugLineStripNoSpace()
	{
		Color = FLinearColor::Red;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of input positions to treat as a line-strip
	UPROPERTY(meta = (Input))
	TArray<FVector> Points;

	// The color to use for the drawing
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use for the drawing
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;
	
	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};
