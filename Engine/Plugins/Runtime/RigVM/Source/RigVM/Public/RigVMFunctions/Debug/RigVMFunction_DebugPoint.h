// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_DebugPoint.generated.h"

UENUM(meta = (RigVMTypeAllowed))
enum class ERigUnitDebugPointMode : uint8
{
	/** Draw as point */
	Point,

	/** Draw as vector */
	Vector,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT(meta=(DisplayName="Draw Point In Place", Keywords="Draw Vector", Deprecated = "4.25.0"))
struct FRigVMFunction_DebugPoint : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_DebugPoint()
	{
		Vector = FVector::ZeroVector;
		Mode = ERigUnitDebugPointMode::Point;
		Color = FLinearColor::Red;
		Scale = 10.f;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FVector Vector;

	UPROPERTY(meta = (Input))
	ERigUnitDebugPointMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

USTRUCT(meta=(DisplayName="Draw Point", Keywords="Draw Vector", Deprecated = "4.25.0"))
struct FRigVMFunction_DebugPointMutable : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugPointMutable()
	{
		Vector = FVector::ZeroVector;
		Mode = ERigUnitDebugPointMode::Point;
		Color = FLinearColor::Red;
		Scale = 10.f;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FVector Vector;

	UPROPERTY(meta = (Input))
	ERigUnitDebugPointMode Mode;

	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Scale;

	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	UPROPERTY(meta = (Input))
	FTransform WorldOffset;



	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};