// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_DebugPrimitives.generated.h"

#define UE_API RIGVM_API

USTRUCT(meta=(DisplayName="Draw Rectangle", Keywords="Draw Square", Deprecated = "4.25"))
struct FRigVMFunction_DebugRectangle : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugRectangle()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The transform of the rectangle to draw
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The scale to apply to the debug draw
	UPROPERTY(meta = (Input))
	float Scale;

	// The line thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	FName Space;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws a rectangle in the viewport given a transform
 */
USTRUCT(meta=(DisplayName="Draw Rectangle", Keywords="Draw Square"))
struct FRigVMFunction_DebugRectangleNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugRectangleNoSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The transform of the rectangle to draw
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The scale to apply to the debug draw
	UPROPERTY(meta = (Input))
	float Scale;

	// The line thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};


USTRUCT(meta=(DisplayName="Draw Arc", Keywords="Draw Ellipse, Draw Circle", Deprecated = "4.25"))
struct FRigVMFunction_DebugArc : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugArc()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = MinimumDegrees = 0.f;
		Radius = 10.f;
		MaximumDegrees = 360.f;
		Detail = 16;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	UPROPERTY(meta = (Input))
	float Radius;

	UPROPERTY(meta = (Input))
	float MinimumDegrees;

	UPROPERTY(meta = (Input))
	float MaximumDegrees;

	// The line thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float Thickness;

	UPROPERTY(meta = (Input))
	int32 Detail;

	UPROPERTY(meta = (Input))
	FName Space;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input, Constant))
	bool bEnabled;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Draws an arc in the viewport, can take in different min and max degrees
 */
USTRUCT(meta=(DisplayName="Draw Arc", Keywords="Draw Ellipse, Draw Circle"))
struct FRigVMFunction_DebugArcNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugArcNoSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = MinimumDegrees = 0.f;
		Radius = 10.f;
		MaximumDegrees = 360.f;
		Detail = 16;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The transform of the arc to draw
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The radius of the arc to draw
	UPROPERTY(meta = (Input))
	float Radius;

	// The minimum angle in degrees of the arc
	UPROPERTY(meta = (Input))
	float MinimumDegrees;

	// The maximum angle in degrees of the arc
	UPROPERTY(meta = (Input))
	float MaximumDegrees;

	// The line thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float Thickness;

	// The detail to use when drawing the arc
	UPROPERTY(meta = (Input))
	int32 Detail;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};

/**
 * Draws a box in the viewport
 */
USTRUCT(meta=(DisplayName="Draw Box", Keywords="BoundingBox,Bbox"))
struct FRigVMFunction_DebugBoxNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugBoxNoSpace()
	{
		Box = FBox(EForceInit::ForceInit);
		WorldOffset = FTransform::Identity;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		WorldOffset = FTransform::Identity;
		bEnabled = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input bounding box to draw
	UPROPERTY(meta = (Input))
	FBox Box;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float Thickness;

	// The world offset to pre-multiply the positions with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};

#undef UE_API