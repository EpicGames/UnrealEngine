// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_DebugTransform.generated.h"

UENUM(meta = (RigVMTypeAllowed))
enum class ERigUnitDebugTransformMode : uint8
{
	/** Draw as point */
	Point,

	/** Draw as axes */
	Axes,

	/** Draw as box */
	Box,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

/**
 * Given a transform, will draw a point, axis, or a box in the viewport
 */
USTRUCT(meta=(DisplayName="Draw Transform"))
struct FRigVMFunction_DebugTransformMutableNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugTransformMutableNoSpace()
	{
		Transform = WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::Red;
		Thickness = 0.f;
		Scale = 10.f;
		bEnabled = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The transform to draw
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The mode to use when drawing the transform
	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	// The debug color to use 
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use
	UPROPERTY(meta = (Input))
	float Thickness;

	// The scale to scale the transform by
	UPROPERTY(meta = (Input))
	float Scale;

	// The world offset to pre-multiply the transform with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};

USTRUCT()
struct FRigVMFunction_DebugTransformArrayMutable_WorkData
{
	GENERATED_BODY()
		
	UPROPERTY()
	TArray<FTransform> DrawTransforms;
};

/**
* Given a transform array, will draw a point, axis, or a box in the viewport
*/
USTRUCT(meta=(DisplayName="Draw Transform Array"))
struct FRigVMFunction_DebugTransformArrayMutableNoSpace : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	FRigVMFunction_DebugTransformArrayMutableNoSpace()
	{
		WorldOffset = FTransform::Identity;
		Mode = ERigUnitDebugTransformMode::Axes;
		Color = FLinearColor::White;
		Thickness = 0.f;
		Scale = 10.f;
		bEnabled = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// An array of input transforms to draw
	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	// An array of parent indices for each transform.
	// If this is specified lines will be drawn from each child
	// to parent to represent a hierarchy.
	UPROPERTY(meta = (Input))
	TArray<int32> ParentIndices;

	// The mode to use when drawing the transforms
	UPROPERTY(meta = (Input))
	ERigUnitDebugTransformMode Mode;

	// The debug color to use
	UPROPERTY(meta = (Input))
	FLinearColor Color;

	// The line thickness to use
	UPROPERTY(meta = (Input))
	float Thickness;

	// The scale to scale the transforms by
	UPROPERTY(meta = (Input))
	float Scale;

	// The world offset to pre-multiply the transforms with
	UPROPERTY(meta = (Input))
	FTransform WorldOffset;

	// If False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;
};
