// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_VisualDebug.generated.h"

UENUM(meta = (RigVMTypeAllowed))
enum class ERigUnitVisualDebugPointMode : uint8
{
	/** Draw as point */
	Point,

	/** Draw as vector */
	Vector,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

USTRUCT(meta=(DisplayName = "Visual Debug Vector", Keywords = "Draw,Point", Deprecated = "4.25", Varying))
struct FRigVMFunction_VisualDebugVector : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugVector()
	{
		Value = FVector::ZeroVector;
		bEnabled = true;
		Mode = ERigUnitVisualDebugPointMode::Point;
		Color = FLinearColor::Red;
		Thickness = 10.f;
		Scale = 1.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FVector Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	ERigUnitVisualDebugPointMode Mode;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FLinearColor Color;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for a Point or Vector given a vector
 */
USTRUCT(meta=(DisplayName = "Visual Debug Vector", TemplateName="VisualDebug", Keywords = "Draw,Point", Varying))
struct FRigVMFunction_VisualDebugVectorNoSpace : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugVectorNoSpace()
	{
		Value = FVector::ZeroVector;
		bEnabled = true;
		Mode = ERigUnitVisualDebugPointMode::Point;
		Color = FLinearColor::Red;
		Thickness = 10.f;
		Scale = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The pass through vector to draw
	UPROPERTY(meta = (Input, Output))
	FVector Value;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;

	// The mode to draw the vector with
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	ERigUnitVisualDebugPointMode Mode;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FLinearColor Color;

	// The line thickness to use for the drawing
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	// The scale to apply to the vector when drawing
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;
};

USTRUCT(meta = (DisplayName = "Visual Debug Quat", Keywords = "Draw,Rotation", Deprecated = "4.25", Varying))
struct FRigVMFunction_VisualDebugQuat : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugQuat()
	{
		Value = FQuat::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FQuat Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for an Axis given a quaternion
 */
USTRUCT(meta = (DisplayName = "Visual Debug Quat", TemplateName="VisualDebug", Keywords = "Draw,Rotation", Varying))
struct FRigVMFunction_VisualDebugQuatNoSpace : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugQuatNoSpace()
	{
		Value = FQuat::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The pass through rotation to draw
	UPROPERTY(meta = (Input, Output))
	FQuat Value;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;

	// The line thickness to use for the drawing
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	// The scale to apply to the axes drawn
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;
};

USTRUCT(meta=(DisplayName="Visual Debug Transform", Keywords = "Draw,Axes", Deprecated = "4.25", Varying))
struct FRigVMFunction_VisualDebugTransform : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugTransform()
	{
		Value = FTransform::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
		BoneSpace = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output, Constant))
	FTransform Value;

	UPROPERTY(meta = (Input))
	bool bEnabled;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;

	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	FName BoneSpace;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Debug draw parameters for an Axis given a transform
 */
USTRUCT(meta=(DisplayName="Visual Debug Transform", TemplateName="VisualDebug", Keywords = "Draw,Axes", Varying))
struct FRigVMFunction_VisualDebugTransformNoSpace : public FRigVMFunction_DebugBase
{
	GENERATED_BODY()

	FRigVMFunction_VisualDebugTransformNoSpace()
	{
		Value = FTransform::Identity;
		bEnabled = true;
		Thickness = 0.f;
		Scale = 10.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The pass through transform to draw
	UPROPERTY(meta = (Input, Output, Constant))
	FTransform Value;

	// If set to False the debug drawing will be skipped
	UPROPERTY(meta = (Input))
	bool bEnabled;

	// The line thickness to use for the drawing
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Thickness;

	// The scale to apply to the axes drawn
	UPROPERTY(meta = (Input, EditCondition = "bEnabled"))
	float Scale;
};