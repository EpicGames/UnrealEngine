// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_DebugBase.h"
#include "RigVMFunction_VisualLog.generated.h"

/** Base RigVMFunction used for Visual Logger output */
USTRUCT(meta=(Abstract, Keywords = "Draw,Point"))
struct FRigVMFunction_VisualLogBase : public FRigVMFunction_DebugBaseMutable
{
	GENERATED_BODY()

	/** The text to log */ 
	UPROPERTY(meta=(Input))
	FString Text;

	/** The category of the logged text */
	UPROPERTY(meta=(Input))
	FName Category = TEXT("VisLogRigVM");
};

/** Logs simple text string with Visual Logger - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Text", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogText : public FRigVMFunction_VisualLogBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;
};

/** Base RigVMFunction for visual logging of objects */ 
USTRUCT(meta=(Abstract, Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogObject : public FRigVMFunction_VisualLogBase
{
	GENERATED_BODY()

	/** The color of the logged object */
	UPROPERTY(meta=(Input))
	FLinearColor ObjectColor = FLinearColor::Blue;
};

/** Logs location as sphere with given radius - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Location", Keywords = "Draw,String,Sphere"))
struct FRigVMFunction_VisualLogLocation : public FRigVMFunction_VisualLogObject
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The location to log */
	UPROPERTY(meta=(Input))
	FVector Location = FVector::ZeroVector;

	/** The radius of the sphere to log the location with */
	UPROPERTY(meta=(Input))
	float Radius = 10.0f;
};

/** Base RigVMFunction for visual logging of objects that can be wirefame */ 
USTRUCT(meta=(Abstract))
struct FRigVMFunction_VisualLogWireframeOptional : public FRigVMFunction_VisualLogObject
{
	GENERATED_BODY()

	/** Whether to display as wireframe */
	UPROPERTY(meta=(Input))
	bool bWireframe = false;
};

/** Logs sphere shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Sphere", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogSphere : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The centre of the sphere */
	UPROPERTY(meta=(Input))
	FVector Center = FVector::ZeroVector;

	/** The radius of the sphere */
	UPROPERTY(meta=(Input))
	float Radius = 10.0f;
};

/** Logs cone shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Cone", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogCone : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The origin of the cone */
	UPROPERTY(meta=(Input))
	FVector Origin = FVector::ZeroVector;

	/** The direction of the cone */
	UPROPERTY(meta=(Input))
	FVector Direction = FVector(0.0f, 0.0f, 1.0f);

	/** The length of the cone */
	UPROPERTY(meta=(Input))
	float Length = 10.0f;

	/** The angle of the cone */
	UPROPERTY(meta=(Input))
	float Angle = 10.0f;
};

/** Logs cylinder shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Cylinder", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogCylinder : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The start of the line segment forming the cylinder */
	UPROPERTY(meta=(Input))
	FVector Start = FVector::ZeroVector;

	/** The end of the line segment forming the cylinder */
	UPROPERTY(meta=(Input))
	FVector End = FVector::ZeroVector;

	/** The radius of the cylinder */
	UPROPERTY(meta=(Input))
	float Radius = 10.0f;
};

/** Logs capsule shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Capsule", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogCapsule : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The base or origin of the capsule */
	UPROPERTY(meta=(Input))
	FVector Base = FVector::ZeroVector;

	/** Half the height of the capsule */
	UPROPERTY(meta=(Input))
	float HalfHeight = 10.0f;

	/** The radius of the capsule */
	UPROPERTY(meta=(Input))
	float Radius = 10.0f;

	/** The orientation of the capsule */
	UPROPERTY(meta=(Input))
	FQuat Rotation = FQuat::Identity;
};

/** Logs box shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Box", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogBox : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The box to draw */
	UPROPERTY(meta=(Input))
	FBox Box = FBox(ForceInit);
};

/** Logs oriented box shape - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Oriented Box", Keywords = "Draw,String"))
struct FRigVMFunction_VisualLogOrientedBox : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The box to draw */
	UPROPERTY(meta=(Input))
	FBox Box = FBox(ForceInit);

	/** The transform of the box */
	UPROPERTY(meta=(Input))
	FTransform Transform;
};

/** Logs arrow - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Arrow", Keywords = "Draw,String,Direction"))
struct FRigVMFunction_VisualLogArrow : public FRigVMFunction_VisualLogObject
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The start of the arrow */
	UPROPERTY(meta=(Input))
	FVector SegmentStart = FVector::ZeroVector;

	/** The end of the arrow */
	UPROPERTY(meta=(Input))
	FVector SegmentEnd = FVector::ZeroVector;

	/** The size of the arrow head */
	UPROPERTY(meta=(Input))
	float ArrowHeadSize = 8.0f;
};

/** Logs circle - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Circle", Keywords = "Draw,String,Disc"))
struct FRigVMFunction_VisualLogCircle : public FRigVMFunction_VisualLogWireframeOptional
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The center of the circle */
	UPROPERTY(meta=(Input))
	FVector Center = FVector::ZeroVector;

	/** The up axis/normal of the circle's plane */
	UPROPERTY(meta=(Input))
	FVector UpAxis = FVector(0.0f, 0.0f, 1.0f);

	/** The radius of the circle */
	UPROPERTY(meta=(Input))
	float Radius = 10.0f;

	/** The thickness of the circle */
	UPROPERTY(meta=(Input))
	float Thickness = 0.0f;
};

/** Logs segment - recording for Visual Logs has to be enabled to record this data */
USTRUCT(meta=(DisplayName = "Visual Log Segment", Keywords = "Draw,String,Line"))
struct FRigVMFunction_VisualLogSegment : public FRigVMFunction_VisualLogObject
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	/** The start of the line segment */
	UPROPERTY(meta=(Input))
	FVector SegmentStart = FVector::ZeroVector;

	/** The end of the line segment */
	UPROPERTY(meta=(Input))
	FVector SegmentEnd = FVector::ZeroVector;

	/** The thickness of the segment */
	UPROPERTY(meta=(Input))
	float Thickness = 0.0f;
};
