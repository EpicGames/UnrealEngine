// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * =============================================================================
 * DESIGN INTENT
 * =============================================================================
 * GOAL:
 *   Defines FRigVMFunction_BreakFromLoop, a RigVM node that allows early exit
 *   from a For Loop or For Each loop body during execution.
 *
 * APPROACH:
 *   The node carries a singleton BlockToRun (FName) pin that the compiler
 *   aliases — at compile time — to the enclosing loop's BlockToRun memory slot.
 *   At runtime, Execute() writes ControlFlowCompletedName into that shared slot,
 *   causing the loop's JumpToBranch to route to the Completed branch on the next
 *   iteration check. The current iteration always completes in full.
 *   This node has no control flow blocks of its own (it does not branch); it is
 *   a pure "write-and-let-the-loop-read" passthrough.
 *
 * REVIEWER NOTES:
 *   The Singleton meta on BlockToRun is critical — it tells the compiler to alias
 *   this pin's memory to the enclosing loop's BlockToRun register rather than
 *   allocating new memory. Without Singleton, the write would go to an isolated
 *   slot the loop never reads, making Break a no-op at runtime.
 *   Do NOT add Hidden meta to BlockToRun — Hidden suppresses pin creation
 *   entirely, which would prevent the compiler from performing the aliasing.
 * =============================================================================
 */

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_BreakLoop.generated.h"

/**
 * Breaks out of the enclosing For Loop or For Each loop at the end of
 * the current iteration. When executed, it sets the enclosing loop's
 * BlockToRun to "Completed", causing the loop to skip remaining
 * iterations and proceed to its Completed branch.
 *
 * Must be placed inside a loop body; the compiler will emit an error
 * if this node is used outside a loop scope.
 */
USTRUCT(meta=(DisplayName="Break From Loop", Category="Execution", NodeColor = "0, 0, 0, 1", Keywords="Break,Stop,Exit,Loop", Icon="EditorStyle|GraphEditor.Macro.Loop_16x"))
struct FRigVMFunction_BreakFromLoop : public FRigVMStruct
{
	GENERATED_BODY()

	FRigVMFunction_BreakFromLoop()
	{
		BlockToRun = NAME_None;
	}

	/** Writes ControlFlowCompletedName into BlockToRun to signal the enclosing loop to exit. */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// Input-only execution pin — no output execution since the compiler emits
	// a JumpForward after this node, making any downstream nodes dead code.
	UPROPERTY(meta = (Input, DisplayName = "Execute"))
	FRigVMExecuteContext ExecuteContext;

	/**
	 * The name of the block the enclosing loop should run next.
	 * Singleton meta causes the compiler to alias this property to the enclosing
	 * loop's BlockToRun register — no separate memory is allocated for this node.
	 * At runtime, Execute() writes ControlFlowCompletedName here, and the loop's
	 * JumpToBranch reads it to decide whether to re-enter the body or jump to Completed.
	 * Do NOT add Hidden meta — it suppresses pin creation and prevents aliasing.
	 */
	UPROPERTY(meta = (Singleton))
	FName BlockToRun;
};
