// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_Context.generated.h"

/**
 * Returns true if a graph is run with its asset editor open. This is editor only - in shipping it always returns false.
 */
USTRUCT(meta=(DisplayName="Is Asset Editor Open", Category="Execution", TitleColor="1 0 0", NodeColor="1 1 1", Keywords="Debug,Open,Inspect"))
struct FRigVMFunction_IsHostBeingDebugged : public FRigVMStruct
{
	GENERATED_BODY()

	FRigVMFunction_IsHostBeingDebugged()
	: Result(false)
	{}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// True if the graph is currently open in the asset editor
	UPROPERTY(EditAnywhere, Transient, Category = "Execution", meta = (Output))
	bool Result;
};