// Copyright Epic Games, Inc. All Rights Reserved.

/*
 * =============================================================================
 * DESIGN INTENT
 * =============================================================================
 * GOAL:
 *   Defines FRigVMFunction_ContinueLoop, a RigVM node that skips the remaining
 *   body nodes in the current loop iteration and immediately re-enters the
 *   enclosing For Loop or For Each loop at the next iteration.
 *
 * APPROACH:
 *   Unlike BreakFromLoop, ContinueLoop has NO BlockToRun pin. It does not need
 *   to write any value to the loop's control-flow register. Instead, the skip
 *   behaviour is entirely a compiler artifact: the RigVM compiler emits a
 *   JumpForward instruction immediately after the CallExtern for this node,
 *   which jumps past the remaining body instructions to the loop's increment/
 *   re-entry point. The loop's own BlockToRun remains set to its body branch,
 *   so iteration continues normally — only the remainder of the *current*
 *   iteration is skipped.
 *
 * REVIEWER NOTES:
 *   Because the semantics are fully handled by the compiler's JumpForward
 *   emission, Execute() is intentionally empty — no runtime side effects are
 *   needed. The struct exists solely to provide a typed CallExtern target that
 *   the compiler recognizes as a continue directive.
 *   Do NOT add a BlockToRun property here. Doing so would cause the compiler
 *   to treat this node like BreakFromLoop and attempt Singleton aliasing,
 *   which is incorrect for continue semantics.
 * =============================================================================
 */

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_ContinueLoop.generated.h"

/**
 * Skips the remaining body nodes of the current loop iteration and
 * re-enters the enclosing For Loop or For Each loop at the next iteration.
 * The loop itself is not terminated — only the current iteration's remaining
 * nodes are bypassed.
 *
 * Must be placed inside a loop body; the compiler will emit an error
 * if this node is used outside a loop scope.
 */
USTRUCT(meta=(DisplayName="Continue Loop", Category="Execution", NodeColor = "0, 0, 0, 1", Keywords="Continue,Skip,Next,Loop", Icon="EditorStyle|GraphEditor.Macro.Loop_16x"))
struct FRigVMFunction_ContinueLoop : public FRigVMStruct
{
	GENERATED_BODY()

	/** Intentionally empty — continue semantics are handled by the compiler's JumpForward. */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// Input-only execution pin — no output execution since the compiler emits
	// a JumpForward after this node, making any downstream nodes dead code.
	UPROPERTY(meta = (Input, DisplayName = "Execute"))
	FRigVMExecuteContext ExecuteContext;
};
