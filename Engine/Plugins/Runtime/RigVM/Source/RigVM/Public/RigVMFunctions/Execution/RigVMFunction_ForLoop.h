// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_ForLoop.generated.h"

/**
 * Given a count, execute iteratively until the count is up
 */
USTRUCT(meta=(DisplayName="For Loop", Category="Execution", TitleColor="1 0 0", NodeColor="1 1 1", Keywords="Iterate", Icon="EditorStyle|GraphEditor.Macro.Loop_16x", DocumentationPolicy="Strict", Pure))
struct FRigVMFunction_ForLoopCount : public FRigVMStructMutable
{
	GENERATED_BODY()

	FRigVMFunction_ForLoopCount()
	{
		BlockToRun = NAME_None;
		Count = 1;
		Index = 0;
		Ratio = 0.f;
	}

	// FRigVMStruct overrides
#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {ExecutePinName, ForLoopCompletedPinName};
		return Blocks;
	}
#endif
	virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const { return InBlockName == ExecutePinName; }
	virtual int32 GetNumSlices() const override { return Count; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The internal block to run as this node progressed
	UPROPERTY(meta = (Singleton))
	FName BlockToRun;

	// The number of iterations for this loop
	UPROPERTY(meta = (Singleton, Input))
	int32 Count;

	// The index of the current iteration
	UPROPERTY(meta = (Singleton, Output))
	int32 Index;

	// The ratio of the current iteration (from 0.0 for the first to 1.0 for the last)
	UPROPERTY(meta = (Singleton, Output))
	float Ratio;

	// The completed branch to run once the loop has finished all iterations
	UPROPERTY(EditAnywhere, Transient, Category = "ForLoop", meta = (Output))
	FRigVMExecuteContext Completed;
};
