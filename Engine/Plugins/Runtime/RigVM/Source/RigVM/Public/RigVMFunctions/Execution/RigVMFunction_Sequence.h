// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_Sequence.generated.h"

/**
 * Allows for a single execution pulse to trigger a series of events in order.
 */
USTRUCT(meta=(DisplayName="Sequence", Category="Execution", TitleColor="1 0 0", NodeColor="1 1 1", Icon="EditorStyle|GraphEditor.Sequence_16x", Pure))
struct FRigVMFunction_Sequence : public FRigVMStruct
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The execution input
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "SequenceExecution", meta = (Input, Aggregate))
	FRigVMExecuteContext ExecuteContext;

	// The execution result A
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output, Aggregate))
	FRigVMExecuteContext A;

	// The execution result B
	UPROPERTY(EditAnywhere, Transient, Category = "SequenceExecution", meta = (Output, Aggregate))
	FRigVMExecuteContext B;
};