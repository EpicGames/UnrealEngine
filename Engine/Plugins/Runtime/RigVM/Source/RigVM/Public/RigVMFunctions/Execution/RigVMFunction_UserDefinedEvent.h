// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_UserDefinedEvent.generated.h"

#define UE_API RIGVM_API

/**
 * User Defined Event for running custom logic
 */
USTRUCT(meta=(DisplayName="User Defined Event", Category="Events", NodeColor="1, 0, 0", Keywords="Event,Entry,MyEvent"))
struct FRigVMFunction_UserDefinedEvent : public FRigVMStruct
{
	GENERATED_BODY()

	FRigVMFunction_UserDefinedEvent()
	{
		EventName = TEXT("My Event");
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UE_API virtual FString GetUnitLabel() const override;
	virtual FName GetEventName() const override { return EventName; }
	virtual bool CanOnlyExistOnce() const override { return false; }

	// The execution result
	UPROPERTY(EditAnywhere, Transient, DisplayName = "Execute", Category = "UserDefinedEvent", meta = (Output))
	FRigVMExecutePin ExecutePin;

	// The name of the event. Given this name the event can be invoked using the run event node.
	UPROPERTY(EditAnywhere, Transient, Category = "UserDefinedEvent", meta = (Input,Constant,DetailsOnly))
	FName EventName;
};

#undef UE_API
