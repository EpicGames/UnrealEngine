// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "RigVMFunction_MathBase.generated.h"

/*
 * The base class for all pure math nodes
 */
USTRUCT(meta=(Abstract, NodeColor = "0.05 0.25 0.05", DocumentationPolicy="Strict"))
struct FRigVMFunction_MathBase : public FRigVMStruct
{
	GENERATED_BODY()

	virtual void Execute() {};
};

/*
 * The base class for all mutable math nodes
 */
USTRUCT(meta=(Abstract, NodeColor = "0.05 0.25 0.05", DocumentationPolicy="Strict"))
struct FRigVMFunction_MathMutableBase : public FRigVMStructMutable
{
	GENERATED_BODY()

	virtual void Execute() {};
};
