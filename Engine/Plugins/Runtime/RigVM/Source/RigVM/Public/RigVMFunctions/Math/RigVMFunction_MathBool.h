// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathBool.generated.h"

/*
 * The base class for all pure bool math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Boolean"))
struct FRigVMFunction_MathBoolBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all bool math constants
 */
USTRUCT(meta=(Abstract, NoOp))
struct FRigVMFunction_MathBoolConstant : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolConstant()
	{
		Value = false;
	}

	// The value of the constant
	UPROPERTY(meta=(Output,Constant))
	bool Value;
};

/*
 * The base class for all unary bool math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathBoolUnaryOp : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolUnaryOp()
	{
		Value = Result = 0.f;
	}

	// The input value
	UPROPERTY(meta=(Input))
	bool Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	bool Result;
};

/*
 * The base class for all binary bool math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathBoolBinaryOp : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolBinaryOp()
	{
		A = B = Result = 0.f;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	bool A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	bool B;

	// The resulting value
	UPROPERTY(meta=(Output))
	bool Result;
};

/*
 * The base class for all aggregational bool math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathBoolBinaryAggregateOp : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolBinaryAggregateOp()
	{
		A = B = Result = 0.f;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	bool A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	bool B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	bool Result;
};

/**
 * A boolean constant
 */
USTRUCT(meta=(DisplayName="Bool", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathBoolMake : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolMake()
	{
		Value = false;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input, Output, Constant))
	bool Value;

	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true
 */
USTRUCT(meta=(DisplayName="True", Keywords="Yes"))
struct FRigVMFunction_MathBoolConstTrue : public FRigVMFunction_MathBoolConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathBoolConstTrue()
	{
		Value = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns false
 */
USTRUCT(meta=(DisplayName="False", Keywords="No"))
struct FRigVMFunction_MathBoolConstFalse : public FRigVMFunction_MathBoolConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathBoolConstFalse()
	{
		Value = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns true if the condition is false
 */
USTRUCT(meta=(DisplayName="Not", TemplateName="Not", Keywords="!"))
struct FRigVMFunction_MathBoolNot : public FRigVMFunction_MathBoolUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns true if both conditions are true
 */
USTRUCT(meta=(DisplayName="And", TemplateName="And", Keywords="&&"))
struct FRigVMFunction_MathBoolAnd : public FRigVMFunction_MathBoolBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns true if both conditions are false
 */
USTRUCT(meta=(DisplayName="Nand", Deprecated="5.1"))
struct FRigVMFunction_MathBoolNand : public FRigVMFunction_MathBoolBinaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns false if both conditions are true
 */
USTRUCT(meta=(DisplayName="Nand", TemplateName="Nand"))
struct FRigVMFunction_MathBoolNand2 : public FRigVMFunction_MathBoolBinaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns true if one of the conditions is true
 */
USTRUCT(meta=(DisplayName="Or", TemplateName="Or", Keywords="||"))
struct FRigVMFunction_MathBoolOr : public FRigVMFunction_MathBoolBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathBoolEquals : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolEquals()
	{
		A = B = false;
		Result = true;
	}

	UPROPERTY(meta=(Input))
	bool A;

	UPROPERTY(meta=(Input))
	bool B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=,Xor", Deprecated="5.1"))
struct FRigVMFunction_MathBoolNotEquals : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolNotEquals()
	{
		A = B = false;
		Result = false;
	}

	UPROPERTY(meta=(Input))
	bool A;

	UPROPERTY(meta=(Input))
	bool B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
* Returns true if the value has changed from the last run
*/
USTRUCT(meta=(DisplayName="Toggled", TemplateName="Toggled", Keywords="Changed,Different"))
struct FRigVMFunction_MathBoolToggled : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolToggled()
	{
		Value = Toggled = Initialized = LastValue = false;
	}

	// The value to check / compare
	UPROPERTY(meta=(Input))
	bool Value;

	// True if the input value has been toggled
	UPROPERTY(meta=(Output))
	bool Toggled;

	UPROPERTY()
	bool Initialized;

	UPROPERTY()
	bool LastValue;
};

/**
 * Returns true and false as a sequence.
 */
USTRUCT(meta=(DisplayName="FlipFlop", Keywords="Toggle,Changed,Different", Varying))
struct FRigVMFunction_MathBoolFlipFlop : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolFlipFlop()
	{
		Duration = TimeLeft = 0.f;
		StartValue = false;
		Result = LastValue = !StartValue;
	}

	// The initial value to use for the flag
	UPROPERTY(meta=(Visible))
	bool StartValue;

	/**
	 * The duration in seconds at which the result won't change.
     * Use 0 for a different result every time.
	 */
	UPROPERTY(meta = (Input))
	float Duration;

	// The resulting value
	UPROPERTY(meta=(Output))
	bool Result;

	UPROPERTY()
	bool LastValue;

	UPROPERTY()
	float TimeLeft;
};

/**
 * Returns true once the first time this node is hit
 */
USTRUCT(meta=(DisplayName="Once", Keywords="FlipFlop,Toggle,Changed,Different", Varying))
struct FRigVMFunction_MathBoolOnce : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolOnce()
	{
		Duration = TimeLeft = 0.f;
		Result = false;
		LastValue = true;
	}

	/**
	 * The duration in seconds at which the result is true
	 * Use 0 for a different result every time.
	 */
	UPROPERTY(meta = (Visible))
	float Duration;

	// The resulting value - true once the first time this node is hit
	UPROPERTY(meta=(Output))
	bool Result;

	UPROPERTY()
	bool LastValue;

	UPROPERTY()
	float TimeLeft;
};

/**
 * Turns the given bool into a float value
 */
USTRUCT(meta=(DisplayName="To Float", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathBoolToFloat : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolToFloat()
	{
		Value = false;
		Result = 0.f;
	}

	// The bool value to convert
	UPROPERTY(meta = (Input))
	bool Value;

	// The resulting float number (0 or 1)
	UPROPERTY(meta= (Output))
	float Result;
};

/**
 * Turns the given bool into an integer value
 */
USTRUCT(meta=(DisplayName="To Integer", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathBoolToInteger : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathBoolToInteger()
	{
		Value = false;
		Result = 0;
	}

	// The bool value to convert
	UPROPERTY(meta = (Input))
	bool Value;

	// The resulting int number (0 or 1)
	UPROPERTY(meta= (Output))
	int32 Result;
};

/**
* Returns true if any item in the array is true
*/
USTRUCT(meta = (DisplayName = "Any", Keywords = "Any"))
struct FRigVMFunction_MathBoolAny : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolAny()
	{
		Array.Reset();
		Result = false;
		IsEmpty = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	//The input array
	UPROPERTY(meta = (Input))
	TArray<bool> Array;

	//The resulting value. False if input array is empty
	UPROPERTY(meta = (Output))
	bool Result;

	//True if the array is empty
	UPROPERTY(meta = (Output))
	bool IsEmpty;

};

/**
* Returns true if all the items in the array are true
*/
USTRUCT(meta = (DisplayName = "All", Keywords = "All"))
struct FRigVMFunction_MathBoolAll : public FRigVMFunction_MathBoolBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoolAll()
	{
		Array.Reset();
		Result = false;
		IsEmpty = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	//The input array
	UPROPERTY(meta = (Input))
	TArray<bool> Array;

	//The resulting value. True if input array is empty
	UPROPERTY(meta = (Output))
	bool Result;

	//True if the array is empty
	UPROPERTY(meta = (Output))
	bool IsEmpty;

};