// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMMathLibrary.h"
#include "RigVMFunction_MathBox.generated.h"

/*
 * The base class for all pure box math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Box", MenuDescSuffix="(Box)"))
struct FRigVMFunction_MathBoxBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/**
 * Returns bounding box of the given array of positions
 */
USTRUCT(meta = (DisplayName = "Box from Array", Keywords = "ArrayBounds,CreateBox,CreateBoundingBox,NewBox,NewBoundingBoxMakeBox,MakeBoundingBox,BoundingBox,Bbox,Bounds"))
struct FRigVMFunction_MathBoxFromArray : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxFromArray()
	{
		Box = FBox(EForceInit::ForceInit);
		Minimum = Maximum = Center = Size = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of input positions
	UPROPERTY(meta = (Input))
	TArray<FVector> Array;

	// The resulting bounding box
	UPROPERTY(meta = (Output))
	FBox Box;

	// The resulting minimum value of the bounding box
	UPROPERTY(meta = (Output))
	FVector Minimum;

	// The resulting maximum value of the bounding box
	UPROPERTY(meta = (Output))
	FVector Maximum;

	// The resulting center value of the bounding box
	UPROPERTY(meta = (Output))
	FVector Center;

	// The resulting size value of the bounding box
	UPROPERTY(meta = (Output))
	FVector Size;
};

/**
 * Returns true if the box has any content / is valid
 */
USTRUCT(meta = (DisplayName = "Is Box Valid", Keywords = "IsValid,HasVolume,ContainsPoints,Bounds,BoundingBox,Bbox"))
struct FRigVMFunction_MathBoxIsValid : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxIsValid()
	{
		Box = FBox(EForceInit::ForceInit);
		Valid = false;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to check
	UPROPERTY(meta = (Input))
	FBox Box;

	// Returns true if the box has any content / is valid
	UPROPERTY(meta = (Output))
	bool Valid;
};

/**
 * Returns the center of a bounding box
 */
USTRUCT(meta = (DisplayName = "Get Box Center", Keywords = "Middle,Origin,Bounds,BoundingBox,Bbox"))
struct FRigVMFunction_MathBoxGetCenter : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxGetCenter()
	{
		Box = FBox(EForceInit::ForceInit);
		Center = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to retrieve the center for
	UPROPERTY(meta = (Input))
	FBox Box;

	// The center of the box
	UPROPERTY(meta = (Output))
	FVector Center;
};

/**
 * Returns the size of a bounding box
 */
USTRUCT(meta = (DisplayName = "Get Box Size", Keywords = "Middle,Origin,Bounds,BoundingBox,Bbox"))
struct FRigVMFunction_MathBoxGetSize : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxGetSize()
	{
		Box = FBox(EForceInit::ForceInit);
		Size = Extent = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to retrieve the size for
	UPROPERTY(meta = (Input))
	FBox Box;

	// the overall size of the box
	UPROPERTY(meta = (Output))
	FVector Size;

	// the half size of the box
	UPROPERTY(meta = (Output))
	FVector Extent;
};

/**
 * Move the box by a certain amount 
 */
USTRUCT(meta = (DisplayName = "Shift Box", Keywords = "Bbox,Translate,Move,BoundingBox"))
struct FRigVMFunction_MathBoxShift : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxShift()
	{
		Box = Result = FBox(EForceInit::ForceInit);
		Amount = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// the box to move
	UPROPERTY(meta = (Input))
	FBox Box;

	// the amount / vector to shift the box by
	UPROPERTY(meta = (Input))
	FVector Amount;

	// The resulting moved box
	UPROPERTY(meta = (Output))
	FBox Result;
};

/**
 * Moves the center of the box to a new location
 */
USTRUCT(meta = (DisplayName = "Move Box To", Keywords = "Bbox,Translate,Move,BoundingBox"))
struct FRigVMFunction_MathBoxMoveTo : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxMoveTo()
	{
		Box = Result = FBox(EForceInit::ForceInit);
		Center = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// the box to move
	UPROPERTY(meta = (Input))
	FBox Box;

	// the new center for the box
	UPROPERTY(meta = (Input))
	FVector Center;

	// The resulting moved box
	UPROPERTY(meta = (Output))
	FBox Result;
};

/**
 * Expands the size of the box by a given amount
 */
USTRUCT(meta = (DisplayName = "Expand Box", Keywords = "Bbox,Scale,Grow,Shrink,BoundingBox"))
struct FRigVMFunction_MathBoxExpand : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxExpand()
	{
		Box = Result = FBox(EForceInit::ForceInit);
		Amount = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// the box to expand
	UPROPERTY(meta = (Input))
	FBox Box;

	// the amount to grow / shrink the box by
	UPROPERTY(meta = (Input))
	FVector Amount;

	// the resulting expanded box
	UPROPERTY(meta = (Output))
	FBox Result;
};

/**
 * Transforms the box by a given transform
 */
USTRUCT(meta = (DisplayName = "Transform Box", Keywords = "Bbox,BoundingBox"))
struct FRigVMFunction_MathBoxTransform : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxTransform()
	{
		Box = Result = FBox(EForceInit::ForceInit);
		Transform = FTransform::Identity;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// the box to transform
	UPROPERTY(meta = (Input))
	FBox Box;

	// the transform to apply to the box
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// the resulting transformed box
	UPROPERTY(meta = (Output))
	FBox Result;
};

/**
 * Returns the distance to a given box
 */
USTRUCT(meta = (DisplayName = "Get Distance to Box", Keywords = "Bbox,Closest,BoundingBox"))
struct FRigVMFunction_MathBoxGetDistance : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxGetDistance()
	{
		Box = FBox(EForceInit::ForceInit);
		Position = FVector::ZeroVector;
		Square = true;
		Valid = false;
		Distance = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to measure the distance to
	UPROPERTY(meta = (Input))
	FBox Box;

	// The position to measure the distance to
	UPROPERTY(meta = (Input))
	FVector Position;

	// if true the distance will be returned square
	UPROPERTY(meta = (Input))
	bool Square;

	// Returns true if the computed distance is valid
	UPROPERTY(meta = (Output))
	bool Valid;

	// The distance between the box and the position
	UPROPERTY(meta = (Output))
	float Distance;
};

/**
 * Returns true if a point is inside a given box
 */
USTRUCT(meta = (DisplayName = "Is Inside Box", Keywords = "Bbox,Contains,Encompasses,BoundingBox"))
struct FRigVMFunction_MathBoxIsInside : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxIsInside()
	{
		Box = FBox(EForceInit::ForceInit);
		Position = FVector::ZeroVector;
		Result = false;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to test
	UPROPERTY(meta = (Input))
	FBox Box;

	// The position to check for
	UPROPERTY(meta = (Input))
	FVector Position;

	// Returns true if the given point is inside the given box
	UPROPERTY(meta = (Output))
	bool Result;
};


/**
 * Returns the volume of a given box
 */
USTRUCT(meta = (DisplayName = "Get Box Volume", Keywords = "Bbox,BoundingBox"))
struct FRigVMFunction_MathBoxGetVolume : public FRigVMFunction_MathBoxBase
{
	GENERATED_BODY()

	FRigVMFunction_MathBoxGetVolume()
	{
		Box = FBox(EForceInit::ForceInit);
		Volume = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The box to retrieve the volume for
	UPROPERTY(meta = (Input))
	FBox Box;

	// The volume of the box
	UPROPERTY(meta = (Output))
	float Volume;
};