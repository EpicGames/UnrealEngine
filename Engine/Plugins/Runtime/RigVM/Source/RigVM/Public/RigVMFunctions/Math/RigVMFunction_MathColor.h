// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathColor.generated.h"

/*
 * The base class for all pure color math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Color", MenuDescSuffix="(Color)"))
struct FRigVMFunction_MathColorBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all aggregational color math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathColorBinaryAggregateOp : public FRigVMFunction_MathColorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathColorBinaryAggregateOp()
	{
		A = B = Result = FLinearColor::Black;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FLinearColor A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FLinearColor B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	FLinearColor Result;
};

/**
 * Makes a color from its components
 */
USTRUCT(meta=(DisplayName="Make Color", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathColorMake : public FRigVMFunction_MathColorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathColorMake()
	{
		R = G = B = 0.f;
		A = 1.f;
		Result = FLinearColor::Black;
	}

	UPROPERTY(meta=(Input))
	float R;

	UPROPERTY(meta=(Input))
	float G;

	UPROPERTY(meta=(Input))
	float B;

	UPROPERTY(meta=(Input))
	float A;

	UPROPERTY(meta=(Output))
	FLinearColor Result;

	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a color from a single float
 */
USTRUCT(meta=(DisplayName="From Float", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathColorFromFloat : public FRigVMFunction_MathColorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathColorFromFloat()
	{
		Value = 0.f;
		Result = FLinearColor::Black;
	}

	// The float value to convert to a color
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting color
	UPROPERTY(meta=(Output))
	FLinearColor Result;
};

/**
 * Makes a color from a single double
 */
USTRUCT(meta=(DisplayName="From Double", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathColorFromDouble : public FRigVMFunction_MathColorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathColorFromDouble()
	{
		Value = 0.0;
		Result = FLinearColor::Black;
	}

	// The double value to convert to a color
	UPROPERTY(meta=(Input))
	double Value;
	
	// The resulting color
	UPROPERTY(meta=(Output))
	FLinearColor Result;
};

/**
 * Returns the sum of the two values
 */
USTRUCT(meta=(DisplayName="Add", TemplateName="Add", Keywords="Sum,+"))
struct FRigVMFunction_MathColorAdd : public FRigVMFunction_MathColorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the difference of the two values
 */
USTRUCT(meta=(DisplayName="Subtract", TemplateName="Subtract", Keywords="-"))
struct FRigVMFunction_MathColorSub : public FRigVMFunction_MathColorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathColorMul : public FRigVMFunction_MathColorBinaryAggregateOp
{
	GENERATED_BODY()

	FRigVMFunction_MathColorMul()
	{
		A = B = FLinearColor::White;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Linearly interpolates between A and B using the ratio T
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend"))
struct FRigVMFunction_MathColorLerp : public FRigVMFunction_MathColorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathColorLerp()
	{
		A = Result = FLinearColor::Black;
		B = FLinearColor::White;
		T = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first color to interpolate from
	UPROPERTY(meta=(Input))
	FLinearColor A;

	// The second color to interpolate to
	UPROPERTY(meta=(Input))
	FLinearColor B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	float T;

	// The resulting interpolated color
	UPROPERTY(meta=(Output))
	FLinearColor Result;
};

