// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathDouble.generated.h"

/*
 * The base class for all pure double math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Double", MenuDescSuffix="(Double)"))
struct FRigVMFunction_MathDoubleBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all double math constants
 */
USTRUCT(meta=(Abstract, NoOp))
struct FRigVMFunction_MathDoubleConstant : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleConstant()
	{
		Value = 0.0;
	}

	// The value of the constant
	UPROPERTY(meta=(Output,Constant))
	double Value;
};

/*
 * The base class for all unary double math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathDoubleUnaryOp : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleUnaryOp()
	{
		Value = Result = 0.0;
	}

	// The input value
	UPROPERTY(meta=(Input))
	double Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	double Result;
};

/*
 * The base class for all binary double math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathDoubleBinaryOp : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleBinaryOp()
	{
		A = B = Result = 0.0;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	double A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	double B;

	// The resulting value
	UPROPERTY(meta=(Output))
	double Result;
};

/*
 * The base class for all aggregational double math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathDoubleBinaryAggregateOp : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleBinaryAggregateOp()
	{
		A = B = Result = 0.0;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	double A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	double B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	double Result;
};

/**
 * A double constant
 */
USTRUCT(meta=(DisplayName="Double", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathDoubleMake : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleMake()
	{
		Value = 0.0;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input, Output, Constant))
	double Value;
	
	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns PI
 */
USTRUCT(meta=(DisplayName="Pi", TemplateName="Pi"))
struct FRigVMFunction_MathDoubleConstPi : public FRigVMFunction_MathDoubleConstant
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleConstPi()
	{
		Value = PI;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns PI * 0.5
 */
USTRUCT(meta=(DisplayName="Half Pi", TemplateName="HalfPi"))
struct FRigVMFunction_MathDoubleConstHalfPi : public FRigVMFunction_MathDoubleConstant
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleConstHalfPi()
	{
		Value = HALF_PI;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns PI * 2.0
 */
USTRUCT(meta=(DisplayName="Two Pi", TemplateName="TwoPi", Keywords="Tau"))
struct FRigVMFunction_MathDoubleConstTwoPi : public FRigVMFunction_MathDoubleConstant
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleConstTwoPi()
	{
		Value = 2.0 * PI;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns E
 */
USTRUCT(meta=(DisplayName="E", TemplateName="E", Keywords="Euler"))
struct FRigVMFunction_MathDoubleConstE : public FRigVMFunction_MathDoubleConstant
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleConstE()
	{
		Value = EULERS_NUMBER;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the sum of the two values
 */
USTRUCT(meta=(DisplayName="Add", TemplateName="Add", Keywords="Sum,+"))
struct FRigVMFunction_MathDoubleAdd : public FRigVMFunction_MathDoubleBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the difference of the two values
 */
USTRUCT(meta=(DisplayName="Subtract", TemplateName="Subtract", Keywords="-"))
struct FRigVMFunction_MathDoubleSub : public FRigVMFunction_MathDoubleBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathDoubleMul : public FRigVMFunction_MathDoubleBinaryAggregateOp
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleMul()
	{
		A = 1.0;
		B = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the division of the two values
 */
USTRUCT(meta=(DisplayName="Divide", TemplateName="Divide", Keywords="Division,Divisor,/"))
struct FRigVMFunction_MathDoubleDiv : public FRigVMFunction_MathDoubleBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleDiv()
	{
		B = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the modulo of the two values
 */
USTRUCT(meta=(DisplayName="Modulo", TemplateName="Modulo", Keywords="%,fmod"))
struct FRigVMFunction_MathDoubleMod : public FRigVMFunction_MathDoubleBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleMod()
	{
		A = 0.0;
		B = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the smaller of the two values
 */
USTRUCT(meta=(DisplayName="Minimum", TemplateName="Minimum"))
struct FRigVMFunction_MathDoubleMin : public FRigVMFunction_MathDoubleBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the larger of the two values
 */
USTRUCT(meta=(DisplayName="Maximum", TemplateName="Maximum"))
struct FRigVMFunction_MathDoubleMax : public FRigVMFunction_MathDoubleBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the value of A raised to the power of B.
 */
USTRUCT(meta=(DisplayName="Power", TemplateName="Power"))
struct FRigVMFunction_MathDoublePow : public FRigVMFunction_MathDoubleBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathDoublePow()
	{
		A = 1.0;
		B = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the square root of the given value
 */
USTRUCT(meta=(DisplayName="Sqrt", TemplateName="Sqrt", Keywords="Root,Square"))
struct FRigVMFunction_MathDoubleSqrt : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the negative value
 */
USTRUCT(meta=(DisplayName="Negate", TemplateName="Negate", Keywords="-,Abs"))
struct FRigVMFunction_MathDoubleNegate : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the absolute (positive) value
 */
USTRUCT(meta=(DisplayName="Absolute", TemplateName="Absolute", Keywords="Abs,Neg"))
struct FRigVMFunction_MathDoubleAbs : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns one minus the input value (1 - x).
 * Commonly used for inverting normalized values such as blend weights.
 */
USTRUCT(meta=(DisplayName="One Minus", TemplateName="OneMinus", Keywords="Invert,Inverse,Subtract,One,Minus,Negate,Flip"))
struct FRigVMFunction_MathDoubleOneMinus : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the closest lower full number (integer) of the value
 */
USTRUCT(meta=(DisplayName="Floor", TemplateName="Floor", Keywords="Round"))
struct FRigVMFunction_MathDoubleFloor : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleFloor()
	{
		Value = Result = 0.0;
		Int = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to apply the floor to
	UPROPERTY(meta=(Input))
	double Value;

	// The resulting closest lower full number (integer) of the input value
	UPROPERTY(meta=(Output))
	double Result;

	// The result as an integer value
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the closest higher full number (integer) of the value
 */
USTRUCT(meta=(DisplayName="Ceiling", TemplateName="Ceiling", Keywords="Round"))
struct FRigVMFunction_MathDoubleCeil : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleCeil()
	{
		Value = Result = 0.0;
		Int = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to apply the ceiling to
	UPROPERTY(meta=(Input))
	double Value;

	// The resulting closest higher full number (integer) of the input value
	UPROPERTY(meta=(Output))
	double Result;

	// The result as an integer value
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the rounded value of the given double number
 */
USTRUCT(meta=(DisplayName="Round", TemplateName="Round"))
struct FRigVMFunction_MathDoubleRound : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleRound()
	{
		Value = Result = 0.0;
		Int = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to round
	UPROPERTY(meta=(Input))
	double Value;

	// The rounded value of the given double number
	UPROPERTY(meta=(Output))
	double Result;

	// The result as an integer
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the double cast to an int (this uses floor)
 */
USTRUCT(meta=(DisplayName="To Int", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathDoubleToInt : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleToInt()
	{
		Value = 0.0;
		Result = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to floor to an int
	UPROPERTY(meta=(Input))
	double Value;

	// The resulting (floored) integer
	UPROPERTY(meta=(Output))
	int32 Result;
};

/**
 * Returns the double cast to a float
 */
USTRUCT(meta=(DisplayName="To Float", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathDoubleToFloat : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleToFloat()
	{
		Value = 0.0;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The double number to cast to float
	UPROPERTY(meta=(Input))
	double Value;

	// The resulting float number
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the sign of the value (+1 for >= 0.0, -1 for < 0.0)
 */
USTRUCT(meta=(DisplayName="Sign", TemplateName="Sign"))
struct FRigVMFunction_MathDoubleSign : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Clamps the given value within the range provided by minimum and maximum
 */
USTRUCT(meta=(DisplayName="Clamp", TemplateName="Clamp", Keywords="Range,Remap"))
struct FRigVMFunction_MathDoubleClamp : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleClamp()
	{
		Value = Minimum = Maximum = Result = 0.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The number to clamp
	UPROPERTY(meta=(Input))
	double Value;

	// The Minimum for the resulting range
	UPROPERTY(meta=(Input))
	double Minimum;

	// The Maximum for the resulting range
	UPROPERTY(meta=(Input))
	double Maximum;

	// The resulting clamped value
	UPROPERTY(meta=(Output))
	double Result;
};

/**
 * Wraps the given value to be within minimum and maximum, inclusive
 * When the value can wrap to both Min and Max, it will wrap to Min if it lies below the range and wrap to Max if it is above the range.
 */
USTRUCT(meta = (DisplayName = "Wrap", TemplateName = "Wrap", Keywords = "Range,Remap"))
struct FRigVMFunction_MathDoubleWrap : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleWrap()
	{
		Value = Result = 0.0;
		Minimum = -1.0;
		Maximum = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The number to wrap
	UPROPERTY(meta = (Input))
	double Value;

	// The Minimum for the resulting range
	UPROPERTY(meta = (Input))
	double Minimum;

	// The Maximum for the resulting range
	UPROPERTY(meta = (Input))
	double Maximum;

	// The resulting wrapped value
	UPROPERTY(meta = (Output))
	double Result;
};

/**
 * Linearly interpolates between A and B using the ratio T
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend"))
struct FRigVMFunction_MathDoubleLerp : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleLerp()
	{
		A = B = T = Result = 0.0;
		B = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to interpolate from
	UPROPERTY(meta=(Input))
	double A;

	// The second value to interpolate to
	UPROPERTY(meta=(Input))
	double B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	double T;

	// The resulting interpolated value
	UPROPERTY(meta=(Output))
	double Result;
};

/**
 * Remaps the given value from a source range to a target range.
 */
USTRUCT(meta=(DisplayName="Remap", TemplateName="Remap", Keywords="Rescale,Scale"))
struct FRigVMFunction_MathDoubleRemap : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleRemap()
	{
		Value = SourceMinimum = TargetMinimum = Result = 0.0;
		SourceMaximum = TargetMaximum = 1.0;
		bClamp = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to remap
	UPROPERTY(meta=(Input))
	double Value;

	// The minimum of the range of the input / source value
	UPROPERTY(meta=(Input))
	double SourceMinimum;

	// The maximum of the range of the input / source value
	UPROPERTY(meta=(Input))
	double SourceMaximum;

	// The minimum of the range of the output / target value
	UPROPERTY(meta=(Input))
	double TargetMinimum;

	// The maximum of the range of the output / target value
	UPROPERTY(meta=(Input))
	double TargetMaximum;

	/** If set to true the result is clamped to the target range */
	UPROPERTY(meta=(Input))
	bool bClamp;

	// The resulting remapped value
	UPROPERTY(meta=(Output))
	double Result;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathDoubleEquals : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathDoubleEquals()
	{
		A = B = 0.0;
		Result = true;
	}

	UPROPERTY(meta=(Input))
	double A;

	UPROPERTY(meta=(Input))
	double B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=", Deprecated="5.1"))
struct FRigVMFunction_MathDoubleNotEquals : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathDoubleNotEquals()
	{
		A = B = 0.0;
		Result = false;
	}

	UPROPERTY(meta=(Input))
	double A;

	UPROPERTY(meta=(Input))
	double B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A is greater than B
 */
USTRUCT(meta=(DisplayName="Greater", TemplateName="Greater", Keywords="Larger,Bigger,>"))
struct FRigVMFunction_MathDoubleGreater : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleGreater()
	{
		A = B = 0.0;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	double A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	double B;

	// True if the value A is greater than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than B
 */
USTRUCT(meta=(DisplayName="Less", TemplateName="Less", Keywords="Smaller,<"))
struct FRigVMFunction_MathDoubleLess : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathDoubleLess()
	{
		A = B = 0.0;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	double A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	double B;

	// True if the value A is less than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is greater than or equal to B
 */
USTRUCT(meta=(DisplayName="Greater Equal", TemplateName="GreaterEqual", Keywords="Larger,Bigger,>="))
struct FRigVMFunction_MathDoubleGreaterEqual : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleGreaterEqual()
	{
		A = B = 0.0;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	double A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	double B;

	// True if the value A is greater than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than or equal to B
 */
USTRUCT(meta=(DisplayName="Less Equal", TemplateName="LessEqual", Keywords="Smaller,<="))
struct FRigVMFunction_MathDoubleLessEqual : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleLessEqual()
	{
		A = B = 0.0;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	double A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	double B;

	// True if the value A is less than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value is nearly zero
 */
USTRUCT(meta=(DisplayName="Is Nearly Zero", TemplateName="IsNearlyZero", Keywords="AlmostZero,0"))
struct FRigVMFunction_MathDoubleIsNearlyZero : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathDoubleIsNearlyZero()
	{
		Value = Tolerance = 0.0;
		Result = true;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to check
	UPROPERTY(meta=(Input))
	double Value;

	// The tolerance to apply for the comparison
	UPROPERTY(meta=(Input))
	double Tolerance;

	// True if the value is nearly zero
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is almost equal to B
 */
USTRUCT(meta=(DisplayName="Is Nearly Equal", TemplateName="IsNearlyEqual", Keywords="AlmostEqual,=="))
struct FRigVMFunction_MathDoubleIsNearlyEqual : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleIsNearlyEqual()
	{
		A = B = Tolerance = 0.0;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	double A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	double B;

	// The tolerance to apply for the comparison
	UPROPERTY(meta=(Input))
	double Tolerance;

	// True if the value A is almost equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns the degrees of a given value in radians
 */
USTRUCT(meta=(DisplayName="Degrees", TemplateName="Degrees"))
struct FRigVMFunction_MathDoubleDeg : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the radians of a given value in degrees
 */
USTRUCT(meta=(DisplayName="Radians", TemplateName="Radians"))
struct FRigVMFunction_MathDoubleRad : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the trigonometric sine value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Sin", TemplateName="Sin"))
struct FRigVMFunction_MathDoubleSin : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the trigonometric cosine value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Cos", TemplateName="Cos"))
struct FRigVMFunction_MathDoubleCos : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the trigonometric tangent value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Tan", TemplateName="Tan"))
struct FRigVMFunction_MathDoubleTan : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the inverse sine value (in radians between -pi/2 and pi/2) of the given value
 */
USTRUCT(meta=(DisplayName="Asin", TemplateName="Asin", Keywords="Arcsin"))
struct FRigVMFunction_MathDoubleAsin : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the inverse cosine value (in radians between 0 and pi) of the given value
 */
USTRUCT(meta=(DisplayName="Acos", TemplateName="Acos", Keywords="Arccos"))
struct FRigVMFunction_MathDoubleAcos : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the inverse tangent value (in radians between -pi/2 and pi/2) of the given value
 */
USTRUCT(meta=(DisplayName="Atan", TemplateName="Atan", Keywords="Arctan"))
struct FRigVMFunction_MathDoubleAtan : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the inverse tangent value (in radians between -pi and pi) of A/B, using the signs of A and B to determine the correct quadrant.
 */
USTRUCT(meta=(DisplayName="Atan2", TemplateName="Atan2", Keywords="Arctan"))
struct FRigVMFunction_MathDoubleAtan2 : public FRigVMFunction_MathDoubleBinaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Computes the angles alpha, beta and gamma (in radians) between the three sides A, B and C
 */
USTRUCT(meta = (DisplayName = "Law Of Cosine", TemplateName="LawOfCosine"))
struct FRigVMFunction_MathDoubleLawOfCosine : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleLawOfCosine()
	{
		A = B = C = AlphaAngle = BetaAngle = GammaAngle = 0.0;
		bValid = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The length of the first edge of the triangle
	UPROPERTY(meta = (Input))
	double A;

	// The length of the second edge of the triangle
	UPROPERTY(meta = (Input))
	double B;

	// The length of the third edge of the triangle
	UPROPERTY(meta = (Input))
	double C;

	// The angle between B and C
	UPROPERTY(meta = (Output))
	double AlphaAngle;

	// The angle between A and C
	UPROPERTY(meta = (Output))
	double BetaAngle;

	// The angle between A and B
	UPROPERTY(meta = (Output))
	double GammaAngle;

	// True if the results are valid
	UPROPERTY(meta = (Output))
	bool bValid;
};

/**
 * Computes the base-e exponential of the given value 
 */
USTRUCT(meta = (DisplayName = "Exponential", TemplateName="Exponential"))
struct FRigVMFunction_MathDoubleExponential : public FRigVMFunction_MathDoubleUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the sum of the given array
 */
USTRUCT(meta = (DisplayName = "Array Sum", TemplateName = "ArraySum"))
struct FRigVMFunction_MathDoubleArraySum : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleArraySum()
	{
		Sum = 0.0;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of numbers to add up
	UPROPERTY(meta = (Input))
	TArray<double> Array;

	// The resulting sum of all numbers
	UPROPERTY(meta = (Output))
	double Sum;
};

/**
 * Returns the average of the given array
 */
USTRUCT(meta = (DisplayName = "Array Average", TemplateName = "ArrayAverage"))
struct FRigVMFunction_MathDoubleArrayAverage : public FRigVMFunction_MathDoubleBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDoubleArrayAverage()
	{
		Average = 0.0;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of numbers to average
	UPROPERTY(meta = (Input))
	TArray<double> Array;

	// The resulting average
	UPROPERTY(meta = (Output))
	double Average;
};
