// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathFloat.generated.h"

#define UE_API RIGVM_API

/*
 * The base class for all pure float math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Float", MenuDescSuffix="(Float)"))
struct FRigVMFunction_MathFloatBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all float math constants
 */
USTRUCT(meta=(Abstract, NoOp))
struct FRigVMFunction_MathFloatConstant : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatConstant()
	{
		Value = 0.f;
	}

	// The value of the constant
	UPROPERTY(meta=(Output,Constant))
	float Value;
};

/*
 * The base class for all unary float math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathFloatUnaryOp : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatUnaryOp()
	{
		Value = Result = 0.f;
	}

	// The input value
	UPROPERTY(meta=(Input))
	float Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	float Result;
};

/*
 * The base class for all binary float math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathFloatBinaryOp : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatBinaryOp()
	{
		A = B = Result = 0.f;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	float A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	float B;

	// The resulting value
	UPROPERTY(meta=(Output))
	float Result;
};

/*
 * The base class for all aggregational float math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathFloatBinaryAggregateOp : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatBinaryAggregateOp()
	{
		A = B = Result = 0.f;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	float A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	float B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	float Result;
};

/**
 * A float constant
 */
USTRUCT(meta=(DisplayName="Float", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathFloatMake : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatMake()
	{
		Value = 0.f;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input, Output, Constant))
	float Value;

	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns PI
 */
USTRUCT(meta=(DisplayName="Pi", TemplateName="Pi"))
struct FRigVMFunction_MathFloatConstPi : public FRigVMFunction_MathFloatConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatConstPi()
	{
		Value = PI;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns PI * 0.5
 */
USTRUCT(meta=(DisplayName="Half Pi", TemplateName="HalfPi"))
struct FRigVMFunction_MathFloatConstHalfPi : public FRigVMFunction_MathFloatConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatConstHalfPi()
	{
		Value = HALF_PI;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns PI * 2.0
 */
USTRUCT(meta=(DisplayName="Two Pi", TemplateName="TwoPi", Keywords="Tau"))
struct FRigVMFunction_MathFloatConstTwoPi : public FRigVMFunction_MathFloatConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatConstTwoPi()
	{
		Value = PI * 2.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns E
 */
USTRUCT(meta=(DisplayName="E", TemplateName="E", Keywords="Euler"))
struct FRigVMFunction_MathFloatConstE : public FRigVMFunction_MathFloatConstant
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatConstE()
	{
		Value = EULERS_NUMBER;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the sum of the two values
 */
USTRUCT(meta=(DisplayName="Add", TemplateName="Add", Keywords="Sum,+"))
struct FRigVMFunction_MathFloatAdd : public FRigVMFunction_MathFloatBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the difference of the two values
 */
USTRUCT(meta=(DisplayName="Subtract", TemplateName="Subtract", Keywords="-"))
struct FRigVMFunction_MathFloatSub : public FRigVMFunction_MathFloatBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathFloatMul : public FRigVMFunction_MathFloatBinaryAggregateOp
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatMul()
	{
		A = 1.f;
		B = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the division of the two values
 */
USTRUCT(meta=(DisplayName="Divide", TemplateName="Divide", Keywords="Division,Divisor,/"))
struct FRigVMFunction_MathFloatDiv : public FRigVMFunction_MathFloatBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatDiv()
	{
		B = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the modulo of the two values
 */
USTRUCT(meta=(DisplayName="Modulo", TemplateName="Modulo", Keywords="%,fmod"))
struct FRigVMFunction_MathFloatMod : public FRigVMFunction_MathFloatBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatMod()
	{
		A = 0.f;
		B = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the smaller of the two values
 */
USTRUCT(meta=(DisplayName="Minimum", TemplateName="Minimum"))
struct FRigVMFunction_MathFloatMin : public FRigVMFunction_MathFloatBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the larger of the two values
 */
USTRUCT(meta=(DisplayName="Maximum", TemplateName="Maximum"))
struct FRigVMFunction_MathFloatMax : public FRigVMFunction_MathFloatBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the value of A raised to the power of B.
 */
USTRUCT(meta=(DisplayName="Power", TemplateName="Power"))
struct FRigVMFunction_MathFloatPow : public FRigVMFunction_MathFloatBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatPow()
	{
		A = 1.f;
		B = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the square root of the given value
 */
USTRUCT(meta=(DisplayName="Sqrt", TemplateName="Sqrt", Keywords="Root,Square"))
struct FRigVMFunction_MathFloatSqrt : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the negative value
 */
USTRUCT(meta=(DisplayName="Negate", TemplateName="Negate", Keywords="-,Abs"))
struct FRigVMFunction_MathFloatNegate : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the absolute (positive) value
 */
USTRUCT(meta=(DisplayName="Absolute", TemplateName="Absolute", Keywords="Abs,Neg"))
struct FRigVMFunction_MathFloatAbs : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns one minus the input value (1 - x).
 * Commonly used for inverting normalized values such as blend weights.
 */
USTRUCT(meta=(DisplayName="One Minus", TemplateName="OneMinus", Keywords="Invert,Inverse,Subtract,One,Minus,Negate,Flip"))
struct FRigVMFunction_MathFloatOneMinus : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the closest lower full number (integer) of the value
 */
USTRUCT(meta=(DisplayName="Floor", TemplateName="Floor", Keywords="Round"))
struct FRigVMFunction_MathFloatFloor : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatFloor()
	{
		Value = Result = 0.f;
		Int = 0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to apply the floor to
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting closest lower full number (integer) of the input value
	UPROPERTY(meta=(Output))
	float Result;

	// The result as an integer value
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the closest higher full number (integer) of the value
 */
USTRUCT(meta=(DisplayName="Ceiling", TemplateName="Ceiling", Keywords="Round"))
struct FRigVMFunction_MathFloatCeil : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatCeil()
	{
		Value = Result = 0.f;
		Int = 0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to apply the ceiling to
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting closest higher full number (integer) of the input value
	UPROPERTY(meta=(Output))
	float Result;

	// The result as an integer value
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the rounded value of the given float number
 */
USTRUCT(meta=(DisplayName="Round", TemplateName="Round"))
struct FRigVMFunction_MathFloatRound : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatRound()
	{
		Value = Result = 0.f;
		Int = 0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to round
	UPROPERTY(meta=(Input))
	float Value;

	// The rounded value of the given float number
	UPROPERTY(meta=(Output))
	float Result;

	// The result as an integer
	UPROPERTY(meta=(Output))
	int32 Int;
};

/**
 * Returns the float cast to an int (this uses floor)
 */
USTRUCT(meta=(DisplayName="To Int", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathFloatToInt : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatToInt()
	{
		Value = 0.f;
		Result = 0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to floor to an int
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting (floored) integer
	UPROPERTY(meta=(Output))
	int32 Result;
};

/**
 * Returns the float cast to a double
 */
USTRUCT(meta=(DisplayName="To Double", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathFloatToDouble : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatToDouble()
	{
		Value = 0.f;
		Result = 0.0;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The float number to cast to double
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting double number
	UPROPERTY(meta=(Output))
	double Result;
};

/**
 * Returns the sign of the value (+1 for >= 0.f, -1 for < 0.f)
 */
USTRUCT(meta=(DisplayName="Sign", TemplateName="Sign"))
struct FRigVMFunction_MathFloatSign : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Clamps the given value within the range provided by minimum and maximum
 */
USTRUCT(meta=(DisplayName="Clamp", TemplateName="Clamp", Keywords="Range,Remap"))
struct FRigVMFunction_MathFloatClamp : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatClamp()
	{
		Value = Minimum = Maximum = Result = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The number to clamp
	UPROPERTY(meta=(Input))
	float Value;

	// The Minimum for the resulting range
	UPROPERTY(meta=(Input))
	float Minimum;

	// The Maximum for the resulting range
	UPROPERTY(meta=(Input))
	float Maximum;

	// The resulting clamped value
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Wraps the given value to be within minimum and maximum, inclusive
 * When the value can wrap to both Min and Max, it will wrap to Min if it lies below the range and wrap to Max if it is above the range.
 */
USTRUCT(meta = (DisplayName = "Wrap", TemplateName = "Wrap", Keywords = "Range,Remap"))
struct FRigVMFunction_MathFloatWrap : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatWrap()
	{
		Value = Result = 0.0f;
		Minimum = -1.0f;
		Maximum = 1.0f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The number to wrap
	UPROPERTY(meta = (Input))
	float Value;

	// The Minimum for the resulting range
	UPROPERTY(meta = (Input))
	float Minimum;

	// The Maximum for the resulting range
	UPROPERTY(meta = (Input))
	float Maximum;

	// The resulting wrapped value
	UPROPERTY(meta = (Output))
	float Result;
};

/**
 * Linearly interpolates between A and B using the ratio T
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend"))
struct FRigVMFunction_MathFloatLerp : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatLerp()
	{
		A = B = T = Result = 0.f;
		B = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to interpolate from
	UPROPERTY(meta=(Input))
	float A;

	// The second value to interpolate to
	UPROPERTY(meta=(Input))
	float B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	float T;

	// The resulting interpolated value
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Remaps the given value from a source range to a target range.
 */
USTRUCT(meta=(DisplayName="Remap", TemplateName="Remap", Keywords="Rescale,Scale"))
struct FRigVMFunction_MathFloatRemap : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatRemap()
	{
		Value = SourceMinimum = TargetMinimum = Result = 0.f;
		SourceMaximum = TargetMaximum = 1.f;
		bClamp = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to remap
	UPROPERTY(meta=(Input))
	float Value;

	// The minimum of the range of the input / source value
	UPROPERTY(meta=(Input))
	float SourceMinimum;

	// The maximum of the range of the input / source value
	UPROPERTY(meta=(Input))
	float SourceMaximum;

	// The minimum of the range of the output / target value
	UPROPERTY(meta=(Input))
	float TargetMinimum;

	// The maximum of the range of the output / target value
	UPROPERTY(meta=(Input))
	float TargetMaximum;

	/** If set to true the result is clamped to the target range */
	UPROPERTY(meta=(Input))
	bool bClamp;

	// The resulting remapped value
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathFloatEquals : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigVMFunction_MathFloatEquals()
	{
		A = B = 0.f;
		Result = true;
	}

	UPROPERTY(meta=(Input))
	float A;

	UPROPERTY(meta=(Input))
	float B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=", Deprecated="5.1"))
struct FRigVMFunction_MathFloatNotEquals : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigVMFunction_MathFloatNotEquals()
	{
		A = B = 0.f;
		Result = false;
	}

	UPROPERTY(meta=(Input))
	float A;

	UPROPERTY(meta=(Input))
	float B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A is greater than B
 */
USTRUCT(meta=(DisplayName="Greater", TemplateName="Greater", Keywords="Larger,Bigger,>"))
struct FRigVMFunction_MathFloatGreater : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatGreater()
	{
		A = B = 0.f;
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	float A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	float B;

	// True if the value A is greater than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than B
 */
USTRUCT(meta=(DisplayName="Less", TemplateName="Less", Keywords="Smaller,<"))
struct FRigVMFunction_MathFloatLess : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatLess()
	{
		A = B = 0.f;
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	float A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	float B;

	// True if the value A is less than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is greater than or equal to B
 */
USTRUCT(meta=(DisplayName="Greater Equal", TemplateName="GreaterEqual", Keywords="Larger,Bigger,>="))
struct FRigVMFunction_MathFloatGreaterEqual : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatGreaterEqual()
	{
		A = B = 0.f;
		Result = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	float A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	float B;

	// True if the value A is greater than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than or equal to B
 */
USTRUCT(meta=(DisplayName="Less Equal", TemplateName="LessEqual", Keywords="Smaller,<="))
struct FRigVMFunction_MathFloatLessEqual : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatLessEqual()
	{
		A = B = 0.f;
		Result = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	float A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	float B;

	// True if the value A is less than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value is nearly zero
 */
USTRUCT(meta=(DisplayName="Is Nearly Zero", TemplateName="IsNearlyZero", Keywords="AlmostZero,0"))
struct FRigVMFunction_MathFloatIsNearlyZero : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathFloatIsNearlyZero()
	{
		Value = Tolerance = 0.f;
		Result = true;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to compare
	UPROPERTY(meta=(Input))
	float Value;

	// The tolerance to apply for the comparison
	UPROPERTY(meta=(Input))
	float Tolerance;

	// True if the value is nearly zero
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is almost equal to B
 */
USTRUCT(meta=(DisplayName="Is Nearly Equal", TemplateName="IsNearlyEqual", Keywords="AlmostEqual,=="))
struct FRigVMFunction_MathFloatIsNearlyEqual : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatIsNearlyEqual()
	{
		A = B = Tolerance = 0.f;
		Result = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	float A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	float B;

	// The tolerance to apply for the comparison
	UPROPERTY(meta=(Input))
	float Tolerance;

	// True if the value A is almost equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Return one of the two values based on the condition
 */
USTRUCT(meta=(DisplayName="Select", Keywords="Pick,If", Deprecated = "4.26.0"))
struct FRigVMFunction_MathFloatSelectBool : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatSelectBool()
	{
		Condition = false;
		IfTrue = IfFalse = Result = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	bool Condition;

	UPROPERTY(meta=(Input))
	float IfTrue;

	UPROPERTY(meta=(Input))
	float IfFalse;

	UPROPERTY(meta=(Output))
	float Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the degrees of a given value in radians
 */
USTRUCT(meta=(DisplayName="Degrees", TemplateName="Degrees"))
struct FRigVMFunction_MathFloatDeg : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the radians of a given value in degrees
 */
USTRUCT(meta=(DisplayName="Radians", TemplateName="Radians"))
struct FRigVMFunction_MathFloatRad : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the trigonometric sine value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Sin", TemplateName="Sin"))
struct FRigVMFunction_MathFloatSin : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the trigonometric cosine value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Cos", TemplateName="Cos"))
struct FRigVMFunction_MathFloatCos : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the trigonometric tangent value of the given angle (in radians)
 */
USTRUCT(meta=(DisplayName="Tan", TemplateName="Tan"))
struct FRigVMFunction_MathFloatTan : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the inverse sine value (in radians between -pi/2 and pi/2) of the given value
 */
USTRUCT(meta=(DisplayName="Asin", TemplateName="Asin", Keywords="Arcsin"))
struct FRigVMFunction_MathFloatAsin : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the inverse cosine value (in radians between 0 and pi) of the given value
 */
USTRUCT(meta=(DisplayName="Acos", TemplateName="Acos", Keywords="Arccos"))
struct FRigVMFunction_MathFloatAcos : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the inverse tangent value (in radians between -pi/2 and pi/2) of the given value
 */
USTRUCT(meta=(DisplayName="Atan", TemplateName="Atan", Keywords="Arctan"))
struct FRigVMFunction_MathFloatAtan : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the inverse tangent value (in radians between -pi and pi) of A/B, using the signs of A and B to determine the correct quadrant.
 */
USTRUCT(meta=(DisplayName="Atan2", TemplateName="Atan2", Keywords="Arctan"))
struct FRigVMFunction_MathFloatAtan2 : public FRigVMFunction_MathFloatBinaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Computes the angles alpha, beta and gamma (in radians) between the three sides A, B and C
 */
USTRUCT(meta = (DisplayName = "Law Of Cosine", TemplateName="LawOfCosine"))
struct FRigVMFunction_MathFloatLawOfCosine : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatLawOfCosine()
	{
		A = B = C = AlphaAngle = BetaAngle = GammaAngle = 0.f;
		bValid = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The length of the first edge of the triangle
	UPROPERTY(meta = (Input))
	float A;

	// The length of the second edge of the triangle
	UPROPERTY(meta = (Input))
	float B;

	// The length of the third edge of the triangle
	UPROPERTY(meta = (Input))
	float C;

	// The angle of between B and C
	UPROPERTY(meta = (Output))
	float AlphaAngle;

	// The angle of between A and C
	UPROPERTY(meta = (Output))
	float BetaAngle;

	// The angle of between A and B
	UPROPERTY(meta = (Output))
	float GammaAngle;

	// True if the results are valid
	UPROPERTY(meta = (Output))
	bool bValid;
};

/**
 * Computes the base-e exponential of the given value 
 */
USTRUCT(meta = (DisplayName = "Exponential", TemplateName="Exponential"))
struct FRigVMFunction_MathFloatExponential : public FRigVMFunction_MathFloatUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the sum of the given array
 */
USTRUCT(meta = (DisplayName = "Array Sum", TemplateName = "ArraySum"))
struct FRigVMFunction_MathFloatArraySum : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatArraySum()
	{
		Sum = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The array of numbers to add up
	UPROPERTY(meta = (Input))
	TArray<float> Array;

	// The resulting sum of all numbers
	UPROPERTY(meta = (Output))
	float Sum;
};

/**
 * Returns the average of the given array
 */
USTRUCT(meta = (DisplayName = "Array Average", TemplateName = "ArrayAverage"))
struct FRigVMFunction_MathFloatArrayAverage : public FRigVMFunction_MathFloatBase
{
	GENERATED_BODY()

	FRigVMFunction_MathFloatArrayAverage()
	{
		Average = 0.f;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The array of numbers to average
	UPROPERTY(meta = (Input))
	TArray<float> Array;

	// The resulting average
	UPROPERTY(meta = (Output))
	float Average;
};

#undef UE_API 