// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathInt.generated.h"

/*
 * The base class for all pure int math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Int", MenuDescSuffix="(Int)"))
struct FRigVMFunction_MathIntBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all unary int math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathIntUnaryOp : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntUnaryOp()
	{
		Value = Result = 0;
	}

	// The input value
	UPROPERTY(meta=(Input))
	int32 Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	int32 Result;
};

/*
 * The base class for all binary int math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathIntBinaryOp : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntBinaryOp()
	{
		A = B = Result = 0;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	int32 A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	int32 B;

	// The resulting value
	UPROPERTY(meta=(Output))
	int32 Result;
};

/*
 * The base class for all aggregational int math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathIntBinaryAggregateOp : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntBinaryAggregateOp()
	{
		A = B = Result = 0;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	int32 A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	int32 B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	int32 Result;
};

/**
 * A integer constant
 */
USTRUCT(meta=(DisplayName="Integer", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathIntMake : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntMake()
	{
		Value = 0;
	}
	
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input, Output, Constant))
	int32 Value;

	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the sum of the two values
 */
USTRUCT(meta=(DisplayName="Add", TemplateName="Add", Keywords="Sum,+"))
struct FRigVMFunction_MathIntAdd : public FRigVMFunction_MathIntBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the difference of the two values
 */
USTRUCT(meta=(DisplayName="Subtract", TemplateName="Subtract", Keywords="-"))
struct FRigVMFunction_MathIntSub : public FRigVMFunction_MathIntBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathIntMul : public FRigVMFunction_MathIntBinaryAggregateOp
{
	GENERATED_BODY()

	FRigVMFunction_MathIntMul()
	{
		A = 1;
		B = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the division of the two values
 */
USTRUCT(meta=(DisplayName="Divide", TemplateName="Divide", Keywords="Division,Divisor,/"))
struct FRigVMFunction_MathIntDiv : public FRigVMFunction_MathIntBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathIntDiv()
	{
		B = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the modulo of the two values
 */
USTRUCT(meta=(DisplayName="Modulo", TemplateName="Modulo", Keywords="%,fmod"))
struct FRigVMFunction_MathIntMod : public FRigVMFunction_MathIntBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathIntMod()
	{
		A = 0;
		B = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the smaller of the two values
 */
USTRUCT(meta=(DisplayName="Minimum", TemplateName="Minimum"))
struct FRigVMFunction_MathIntMin : public FRigVMFunction_MathIntBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the larger of the two values
 */
USTRUCT(meta=(DisplayName="Maximum", TemplateName="Maximum"))
struct FRigVMFunction_MathIntMax : public FRigVMFunction_MathIntBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the value of A raised to the power of B.
 */
USTRUCT(meta=(DisplayName="Power", TemplateName="Power"))
struct FRigVMFunction_MathIntPow : public FRigVMFunction_MathIntBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathIntPow()
	{
		A = 1;
		B = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the negative value
 */
USTRUCT(meta=(DisplayName="Negate", TemplateName="Negate", Keywords="-,Abs"))
struct FRigVMFunction_MathIntNegate : public FRigVMFunction_MathIntUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the absolute (positive) value
 */
USTRUCT(meta=(DisplayName="Absolute", TemplateName="Absolute", Keywords="Abs,Neg"))
struct FRigVMFunction_MathIntAbs : public FRigVMFunction_MathIntUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the int cast to a float
 */
USTRUCT(meta=(DisplayName="To Float", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathIntToFloat : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntToFloat()
	{
		Value = 0;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The int to cast
	UPROPERTY(meta=(Input))
	int32 Value;

	// The resulting float value
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the int cast to a double
 */
USTRUCT(meta=(DisplayName="To Double", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext"))
struct FRigVMFunction_MathIntToDouble : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntToDouble()
	{
		Value = 0;
		Result = 0.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The int to cast to double
	UPROPERTY(meta=(Input))
	int32 Value;

	// The resulting double value
	UPROPERTY(meta=(Output))
	double Result;
};

/**
 * Returns the sign of the value (+1 for >= 0, -1 for < 0)
 */
USTRUCT(meta=(DisplayName="Sign", TemplateName="Sign"))
struct FRigVMFunction_MathIntSign : public FRigVMFunction_MathIntUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Clamps the given value within the range provided by minimum and maximum
 */
USTRUCT(meta=(DisplayName="Clamp", TemplateName="Clamp", Keywords="Range,Remap"))
struct FRigVMFunction_MathIntClamp : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntClamp()
	{
		Value = Minimum = Maximum = Result = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The number to clamp
	UPROPERTY(meta=(Input))
	int32 Value;

	// The Minimum for the resulting range
	UPROPERTY(meta=(Input))
	int32 Minimum;
	
	// The Maximum for the resulting range
	UPROPERTY(meta=(Input))
	int32 Maximum;

	// The resulting clamped value
	UPROPERTY(meta=(Output))
	int32 Result;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathIntEquals : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathIntEquals()
	{
		A = B = 0;
		Result = true;
	}

	UPROPERTY(meta=(Input))
	int32 A;

	UPROPERTY(meta=(Input))
	int32 B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=", Deprecated="5.1"))
struct FRigVMFunction_MathIntNotEquals : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathIntNotEquals()
	{
		A = B = 0;
		Result = false;
	}

	UPROPERTY(meta=(Input))
	int32 A;

	UPROPERTY(meta=(Input))
	int32 B;

	UPROPERTY(meta=(Output))
	bool Result;
	
	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A is greater than B
 */
USTRUCT(meta=(DisplayName="Greater", TemplateName="Greater", Keywords="Larger,Bigger,>"))
struct FRigVMFunction_MathIntGreater : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntGreater()
	{
		A = B = 0;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	int32 A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	int32 B;

	// True if the value A is greater than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than B
 */
USTRUCT(meta=(DisplayName="Less", TemplateName="Less", Keywords="Smaller,<"))
struct FRigVMFunction_MathIntLess : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathIntLess()
	{
		A = B = 0;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	int32 A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	int32 B;

	// True if the value A is less than B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is greater than or equal to B
 */
USTRUCT(meta=(DisplayName="Greater Equal", TemplateName="GreaterEqual", Keywords="Larger,Bigger,>="))
struct FRigVMFunction_MathIntGreaterEqual : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntGreaterEqual()
	{
		A = B = 0;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	int32 A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	int32 B;

	// True if the value A is greater than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the value A is less than or equal to B
 */
USTRUCT(meta=(DisplayName="Less Equal", TemplateName="LessEqual", Keywords="Smaller,<="))
struct FRigVMFunction_MathIntLessEqual : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntLessEqual()
	{
		A = B = 0;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first value to compare
	UPROPERTY(meta=(Input))
	int32 A;

	// The second value to compare
	UPROPERTY(meta=(Input))
	int32 B;

	// True if the value A is less than or equal to B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns the sum of the given array
 */
USTRUCT(meta = (DisplayName = "Array Sum", TemplateName = "ArraySum"))
struct FRigVMFunction_MathIntArraySum : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntArraySum()
	{
		Sum = 0;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of numbers to add up
	UPROPERTY(meta = (Input))
	TArray<int32> Array;

	// The resulting sum
	UPROPERTY(meta = (Output))
	int32 Sum;
};


/**
 * Returns the average of the given array
 */
USTRUCT(meta = (DisplayName = "Array Average", TemplateName = "ArrayAverage"))
struct FRigVMFunction_MathIntArrayAverage : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntArrayAverage()
	{
		Average = 0;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of numbers to average
	UPROPERTY(meta = (Input))
	TArray<int32> Array;

	// The resulting average
	UPROPERTY(meta = (Output))
	int32 Average;
};

/**
 * Converts an integer to a string
 */
USTRUCT(meta = (DisplayName = "Int to String", TemplateName="Int to Name"))
struct FRigVMFunction_MathIntToString : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntToString()
	{
		Number = PaddedSize = 0;
	}

	/** Execute logic for this function */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The number to convert
	UPROPERTY(meta = (Input))
	int32 Number;

	/*
	 * For positive numbers you can pad the result to a required width
	 * so rather than '13' return '00013' for a padded size of 5.
	 */
	UPROPERTY(meta = (Input))
	int32 PaddedSize;

	// The resulting potentially padded string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Converts an integer to a name
 */
USTRUCT(meta = (DisplayName = "Int to Name", TemplateName="Int to Name"))
struct FRigVMFunction_MathIntToName : public FRigVMFunction_MathIntBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntToName()
	{
		Number = PaddedSize = 0;
	}

	/** Execute logic for this function */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The number to convert
	UPROPERTY(meta = (Input))
	int32 Number;

	/*
	 * For positive numbers you can pad the result to a required width
	 * so rather than '13' return '00013' for a padded size of 5.
	 */
	UPROPERTY(meta = (Input))
	int32 PaddedSize;

	// The resulting potentially padded name
	UPROPERTY(meta = (Output))
	FName Result;
};