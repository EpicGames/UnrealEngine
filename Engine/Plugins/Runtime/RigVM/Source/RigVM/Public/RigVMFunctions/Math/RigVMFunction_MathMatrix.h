// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EulerTransform.h"
#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathMatrix.generated.h"

/*
 * The base class for all pure matrix math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Matrix", MenuDescSuffix="(Matrix)"))
struct FRigVMFunction_MathMatrixBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all unary matrix math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathMatrixUnaryOp : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixUnaryOp()
	{
		Value = Result = FMatrix::Identity;
	}

	// The input value
	UPROPERTY(meta=(Input))
	FMatrix Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	FMatrix Result;
};

/*
 * The base class for all binary matrix math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathMatrixBinaryOp : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixBinaryOp()
	{
		A = B = Result = FMatrix::Identity;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	FMatrix A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	FMatrix B;

	// The resulting value
	UPROPERTY(meta=(Output))
	FMatrix Result;
};

/*
 * The base class for all aggregational matrix math nodes
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathMatrixBinaryAggregateOp : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixBinaryAggregateOp()
	{
		A = B = Result = FMatrix::Identity;
	}

	// The first matrix for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FMatrix A;

	// The second matrix for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FMatrix B;

	// The resulting matrix
	UPROPERTY(meta=(Output, Aggregate))
	FMatrix Result;
};

/**
* Makes a transform from a matrix
*/
USTRUCT(meta=(DisplayName="To Transform", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathMatrixToTransform : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixToTransform()
	{
		Value = FMatrix::Identity;
		Result = FTransform::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The matrix to convert to a transform
	UPROPERTY(meta=(Input))
	FMatrix Value;

	// The resulting transform
	UPROPERTY(meta=(Output))
	FTransform Result;
};

/**
 * Makes a matrix from a transform
 */
USTRUCT(meta=(DisplayName="From Transform", Keywords="Make,Construct", Deprecated="5.0.1"))
struct FRigVMFunction_MathMatrixFromTransform : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixFromTransform()
	{
		Transform = FTransform::Identity;
		Result = FMatrix::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FTransform Transform;

	UPROPERTY(meta=(Output))
	FMatrix Result;
	
	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a matrix from a transform
 */
USTRUCT(meta=(DisplayName="From Transform", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathMatrixFromTransformV2 : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixFromTransformV2()
	{
		Value = FTransform::Identity;
		Result = FMatrix::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The transform to convert to a matrix
	UPROPERTY(meta=(Input))
	FTransform Value;

	// The resulting matrix
	UPROPERTY(meta=(Output))
	FMatrix Result;
};

/**
 * Converts the matrix to its vectors
 */
USTRUCT(meta=(DisplayName="To Vectors", Keywords="Make,Construct"))
struct FRigVMFunction_MathMatrixToVectors : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixToVectors()
	{
		Value = FMatrix::Identity;
		Origin = FVector::ZeroVector;
		X = FVector::XAxisVector;
		Y = FVector::YAxisVector;
		Z = FVector::ZAxisVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The matrix to split up to vectors
	UPROPERTY(meta=(Input))
	FMatrix Value;

	// The resulting origin
	UPROPERTY(meta=(Output))
	FVector Origin;

	// The resulting X component
	UPROPERTY(meta=(Output))
	FVector X;

	// The resulting Y component
	UPROPERTY(meta=(Output))
	FVector Y;

	// The resulting Z component
	UPROPERTY(meta=(Output))
	FVector Z;
};

/**
* Makes a matrix from its vectors
*/
USTRUCT(meta=(DisplayName="From Vectors", Keywords="Make,Construct"))
struct FRigVMFunction_MathMatrixFromVectors : public FRigVMFunction_MathMatrixBase
{
	GENERATED_BODY()

	FRigVMFunction_MathMatrixFromVectors()
	{
		Origin = FVector::ZeroVector;
		X = FVector::XAxisVector;
		Y = FVector::YAxisVector;
		Z = FVector::ZAxisVector;
		Result = FMatrix::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input origin for the matrix
	UPROPERTY(meta=(Input))
	FVector Origin;

	// The input X component for the matrix
	UPROPERTY(meta=(Input))
	FVector X;

	// The input Y component for the matrix
	UPROPERTY(meta=(Input))
	FVector Y;

	// The input Z component for the matrix
	UPROPERTY(meta=(Input))
	FVector Z;

	// The resulting matrix
	UPROPERTY(meta=(Output))
	FMatrix Result;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*,Global"))
struct FRigVMFunction_MathMatrixMul : public FRigVMFunction_MathMatrixBinaryAggregateOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the inverse value
 */
USTRUCT(meta=(DisplayName="Inverse", TemplateName="Inverse"))
struct FRigVMFunction_MathMatrixInverse : public FRigVMFunction_MathMatrixUnaryOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};
