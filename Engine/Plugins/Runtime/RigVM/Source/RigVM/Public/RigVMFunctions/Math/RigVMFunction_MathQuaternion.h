// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "EulerTransform.h"
#include "RigVMFunction_MathQuaternion.generated.h"

#define UE_API RIGVM_API

/*
 * The base class for all pure quaternion math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Quaternion", MenuDescSuffix="(Quaternion)"))
struct FRigVMFunction_MathQuaternionBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all unary quaternion math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathQuaternionUnaryOp : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionUnaryOp()
	{
		Value = Result = FQuat::Identity;
	}

	// The input value
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/*
 * The base class for all binary quaternion math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathQuaternionBinaryOp : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionBinaryOp()
	{
		A = B = Result = FQuat::Identity;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	FQuat A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	FQuat B;

	// The resulting value
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/*
 * The base class for all aggregational quaternion math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathQuaternionBinaryAggregateOp : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionBinaryAggregateOp()
	{
		A = B = Result = FQuat::Identity;
	}
	
	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FQuat A;
	
	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FQuat B;
	
	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	FQuat Result;
};

/**
 * Makes a quaternion from its components
 */
USTRUCT(meta=(DisplayName="Make Quat", Keywords="Make,Construct,Constant"))
struct FRigVMFunction_MathQuaternionMake : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigVMFunction_MathQuaternionMake()
	{
		X = Y = Z = W = 0.f;
		Result = FQuat::Identity;
	}

	// The X of the quaternion
	UPROPERTY(meta=(Input))
	float X;

	// The Y of the quaternion
	UPROPERTY(meta=(Input))
	float Y;

	// The Z of the quaternion
	UPROPERTY(meta=(Input))
	float Z;

	// The W of the quaternion
	UPROPERTY(meta=(Input))
	float W;

	// The resulting composed quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Makes a quaternion from an axis and an angle in radians
 */
USTRUCT(meta=(DisplayName="From Axis And Angle", TemplateName="FromAxisAndAngle", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionFromAxisAndAngle : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionFromAxisAndAngle()
	{
		Axis = FVector(1.f, 0.f, 0.f);
		Angle = 0.f;
		Result = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The axis to use for the quaternion
	UPROPERTY(meta=(Input))
	FVector Axis;

	// The angle in radians to use for the quaternion
	UPROPERTY(meta=(Input))
	float Angle;

	// The resulting composed quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Makes a quaternion from euler values in degrees
 */
USTRUCT(meta=(DisplayName="From Euler", TemplateName="FromEuler", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionFromEuler : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionFromEuler()
	{
		Euler = FVector::ZeroVector;
		RotationOrder = EEulerRotationOrder::ZYX;
		Result = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The euler representation of the rotation in degrees.
	UPROPERTY(meta = (Input))
	FVector Euler;

	// The rotation order to use when interpreting the euler angles
	UPROPERTY(meta = (Input))
	EEulerRotationOrder RotationOrder;

	// The resulting composed quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Makes a quaternion from a rotator
 */
USTRUCT(meta=(DisplayName="From Rotator", Keywords="Make,Construct", Deprecated="5.0.1"))
struct FRigVMFunction_MathQuaternionFromRotator : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathQuaternionFromRotator()
	{
		Rotator = FRotator::ZeroRotator;
		Result = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FRotator Rotator;

	UPROPERTY(meta=(Output))
	FQuat Result;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a quaternion from a rotator
 */
USTRUCT(meta=(DisplayName="From Rotator", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionFromRotatorV2 : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathQuaternionFromRotatorV2()
	{
		Value = FRotator::ZeroRotator;
		Result = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The rotator representation of the rotation
	UPROPERTY(meta=(Input))
	FRotator Value;

	// The resulting composed quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Makes a quaternion from two vectors, representing the shortest rotation between the two vectors.
 */
USTRUCT(meta=(DisplayName="From Two Vectors", TemplateName="FromTwoVectors", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionFromTwoVectors : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathQuaternionFromTwoVectors()
	{
		A = B = FVector(1.f, 0.f, 0.f);
		Result = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// A vector representing a direction prior to rotation
	UPROPERTY(meta=(Input))
	FVector A;

	// A vector representing a direction after a rotation
	UPROPERTY(meta=(Input))
	FVector B;

	// The quaternion representing the rotation from A to B
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Retrieves the axis and angle of a quaternion in radians
 */
USTRUCT(meta=(DisplayName="To Axis And Angle", TemplateName="ToAxisAndAngle", Keywords="Make,Construct,GetAxis,GetAngle"))
struct FRigVMFunction_MathQuaternionToAxisAndAngle : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionToAxisAndAngle()
	{
		Value = FQuat::Identity;
		Axis = FVector(1.f, 0.f, 0.f);
		Angle = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The axis of the quaternion
	UPROPERTY(meta=(Output))
	FVector Axis;

	// The angle in radians of the quaternion
	UPROPERTY(meta=(Output))
	float Angle;
};

/**
 * Retrieves the forward, right and up vectors of a quaternion
 */
USTRUCT(meta=(DisplayName="To Vectors", TemplateName="ToVectors", Keywords="Forward,Right,Up"))
struct FRigVMFunction_MathQuaternionToVectors : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionToVectors()
	{
		Value = FQuat::Identity;
		Forward = Right = Up = FVector(0.f, 0.f, 0.f);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The forward axis of the quaternion
	UPROPERTY(meta=(Output))
	FVector Forward;

	// The right axis of the quaternion
	UPROPERTY(meta=(Output))
	FVector Right;

	// The up axis of the quaternion
	UPROPERTY(meta=(Output))
	FVector Up;
};

/**
 * Scales a quaternion's angle
 */
USTRUCT(meta=(DisplayName="Scale", Keywords="Multiply,Angle,Scale", Constant, Deprecated = "5.0.1"))
struct FRigVMFunction_MathQuaternionScale : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionScale()
	{
		Value = FQuat::Identity;
		Scale = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input, Output))
	FQuat Value;

	UPROPERTY(meta=(Input))
	float Scale;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Scales a quaternion's angle
 */
USTRUCT(meta=(DisplayName="Scale", TemplateName="Scale", Keywords="Multiply,Angle,Scale", Constant))
struct FRigVMFunction_MathQuaternionScaleV2 : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionScaleV2()
	{
		Value = Result = FQuat::Identity;
		Factor = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The scale factor to scale the input by
	UPROPERTY(meta=(Input))
	float Factor;

	// The resulting scaled quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Retrieves the euler angles in degrees
 */
USTRUCT(meta=(DisplayName="To Euler", TemplateName="ToEuler", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionToEuler : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionToEuler()
	{
		Value = FQuat::Identity;
		RotationOrder = EEulerRotationOrder::ZYX;
		Result = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The rotation order to use when encoding the euler angles
	UPROPERTY(meta = (Input))
	EEulerRotationOrder RotationOrder;

	// The euler angles in degrees representing the quaternion in the given rotation order
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Retrieves the rotator
 */
USTRUCT(meta=(DisplayName="To Rotator", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathQuaternionToRotator : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionToRotator()
	{
		Value = FQuat::Identity;
		Result = FRotator::ZeroRotator;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The rotator representation of the rotation
	UPROPERTY(meta=(Output))
	FRotator Result;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathQuaternionMul : public FRigVMFunction_MathQuaternionBinaryAggregateOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the negative value
 */
USTRUCT(meta=(DisplayName="Inverse", TemplateName="Inverse"))
struct FRigVMFunction_MathQuaternionInverse : public FRigVMFunction_MathQuaternionUnaryOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Performs a spherical interpolation of the quaternions A and B based on the blend value T.
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend,Slerp,SphericalInterpolate"))
struct FRigVMFunction_MathQuaternionSlerp : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionSlerp()
	{
		A = B = Result = FQuat::Identity;
		T = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first quaternion to interpolate from
	UPROPERTY(meta=(Input))
	FQuat A;

	// The second quaternion to interpolate to
	UPROPERTY(meta=(Input))
	FQuat B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	float T;

	// The resulting interpolated quaternion
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathQuaternionEquals : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionEquals()
	{
		A = B = FQuat::Identity;
		Result = true;
	}	

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FQuat A;

	UPROPERTY(meta=(Input))
	FQuat B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=", Deprecated="5.1"))
struct FRigVMFunction_MathQuaternionNotEquals : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionNotEquals()
	{
		A = B = FQuat::Identity;
		Result = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FQuat A;

	UPROPERTY(meta=(Input))
	FQuat B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Return one of the two values based on the condition
 */
USTRUCT(meta=(DisplayName="Select", Keywords="Pick,If", Deprecated = "4.26.0"))
struct FRigVMFunction_MathQuaternionSelectBool : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionSelectBool()
	{
		IfTrue = IfFalse = Result = FQuat::Identity;
		Condition = true;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	bool Condition;

	UPROPERTY(meta=(Input))
	FQuat IfTrue;

	UPROPERTY(meta=(Input))
	FQuat IfFalse;

	UPROPERTY(meta=(Output))
	FQuat Result;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the dot product between two quaternions
 */
USTRUCT(meta=(DisplayName="Dot", Keywords="Dot,|"))
struct FRigVMFunction_MathQuaternionDot : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionDot()
	{
		A = B = FQuat::Identity;
		Result = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first quaternion to compute the dot product for
	UPROPERTY(meta=(Input))
	FQuat A;

	// The second quaternion to compute the dot product for
	UPROPERTY(meta=(Input))
	FQuat B;

	// The resulting dot product between two quaternions
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the normalized quaternion
 */
USTRUCT(meta=(DisplayName="Unit", TemplateName="Unit", Keywords="Normalize"))
struct FRigVMFunction_MathQuaternionUnit : public FRigVMFunction_MathQuaternionUnaryOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Rotates a given vector by the quaternion
 */
USTRUCT(meta=(DisplayName="Rotate Vector", TemplateName="Rotate Vector", Keywords="Transform,Multiply"))
struct FRigVMFunction_MathQuaternionRotateVector : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionRotateVector()
	{
		Transform = FQuat::Identity;
		Vector = Result = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion to rotate the input vector by
	UPROPERTY(meta=(Input))
	FQuat Transform;

	// The input vector to rotate
	UPROPERTY(meta=(Input))
	FVector Vector;

	// The resulting rotated vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Extracts an axis from a quaternion rotation
 */
USTRUCT(meta = (DisplayName = "Axis", TemplateName="Axis", Keywords = "GetAxis,xAxis,yAxis,zAxis"))
struct FRigVMFunction_MathQuaternionGetAxis: public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionGetAxis()
	{
		Quaternion = FQuat::Identity;
		Axis = EAxis::X;
		Result = FVector(1.f, 0.f, 0.f);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta = (Input))
	FQuat Quaternion;

	// The axis to extract
	UPROPERTY(meta = (Input))
	TEnumAsByte<EAxis::Type> Axis;

	// The resulting extracted axis
	UPROPERTY(meta = (Output))
	FVector Result;
};


/**
 * Computes the swing and twist components of a quaternion
 */
USTRUCT(meta = (DisplayName = "To Swing & Twist"))
struct FRigVMFunction_MathQuaternionSwingTwist : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionSwingTwist()
	{
		Input = Swing = Twist = FQuat::Identity;
		TwistAxis = FVector(1.f, 0.f, 0.f);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta = (Input))
	FQuat	Input;

	// The axis to treat as the twist axis
	UPROPERTY(meta = (Input))
	FVector TwistAxis;

	// The resulting swing quaternion of the input quaternion
	UPROPERTY(meta = (Output))
	FQuat Swing;

	// The resulting twist quaternion of the input quaternion
	UPROPERTY(meta = (Output))
	FQuat Twist;
};

/**
 * Enum of possible rotation orders
 */
USTRUCT(meta = (DisplayName = "Rotation Order", Category = "Math|Quaternion", Constant))
struct FRigVMFunction_MathQuaternionRotationOrder : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionRotationOrder()
	{
		RotationOrder = EEulerRotationOrder::ZYX;
	}

	// The constant rotation order
	UPROPERTY(meta = (Input, Output))
	EEulerRotationOrder RotationOrder;


	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the relative local rotation within a parent's rotation
 */
USTRUCT(meta=(DisplayName="Make Relative", TemplateName="Make Relative", Keywords="Local,Global,Absolute"))
struct FRigVMFunction_MathQuaternionMakeRelative : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionMakeRelative()
	{
		Global = Parent = Local = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The global rotation quaternion to make relative
	UPROPERTY(meta=(Input))
	FQuat Global;

	// The parent global rotation to make the input relative to
	UPROPERTY(meta=(Input))
	FQuat Parent;

	// The resulting relative rotation
	UPROPERTY(meta=(Output))
	FQuat Local;
};

/**
 * Returns the absolute global rotation within a parent's rotation
 */
USTRUCT(meta = (DisplayName = "Make Absolute", TemplateName="Make Absolute", Keywords = "Local,Global,Relative"))
struct FRigVMFunction_MathQuaternionMakeAbsolute : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionMakeAbsolute()
	{
		Global = Parent = Local = FQuat::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input local rotation to make absolute
	UPROPERTY(meta = (Input))
	FQuat Local;

	// The parent global rotation to make the local absolute to
	UPROPERTY(meta = (Input))
	FQuat Parent;

	// The resulting global rotation
	UPROPERTY(meta = (Output))
	FQuat Global;
};

/**
 * Mirror a rotation about a central transform.
 */
USTRUCT(meta=(DisplayName="Mirror", TemplateName="Mirror"))
struct FRigVMFunction_MathQuaternionMirrorTransform : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionMirrorTransform()
	{
		Value = Result = FQuat::Identity;
		MirrorAxis = EAxis::X;
		AxisToFlip = EAxis::Z;
		CentralTransform = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input quaternion
	UPROPERTY(meta=(Input))
	FQuat Value;

	// the axis to mirror against
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> MirrorAxis;

	// the axis to flip for rotations
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> AxisToFlip;

	// The transform about which to mirror
	UPROPERTY(meta=(Input))
	FTransform CentralTransform;

	// The resulting mirrored rotation
	UPROPERTY(meta=(Output))
	FQuat Result;
};

/**
 * Return the Quaternion orientation corresponding to the direction in which the vector points.
 * Similar to the blueprint "Rotation From X Vector" version, returns a result without roll such that it preserves the up vector.
 *
 * @note If you don't care about preserving the up vector and just want the most direct rotation, you can use the faster "From Two Vectors"
 */
USTRUCT(meta=(DisplayName="Rotation From X Vector", Keywords = "Orientation, From Axis"))
struct FRigVMFunction_MathQuaternionFromXVector : public FRigVMFunction_MathQuaternionBase
{
	GENERATED_BODY()

	FRigVMFunction_MathQuaternionFromXVector()
	{
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input direction vector
	UPROPERTY(meta=(Input))
	FVector Value = FVector::XAxisVector;

	// The resulting rotation
	UPROPERTY(meta=(Output))
	FQuat Result = FQuat::Identity;
};

#undef UE_API