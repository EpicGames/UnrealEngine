// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RBF/RBFInterpolator.h"

#include "RigVMFunction_MathRBFInterpolate.generated.h"

#define UE_API RIGVM_API

template<typename T> class TRBFInterpolator;
struct RIGVM_API FRigVMFunction_MathRBFQuatWeightFunctor;
struct RIGVM_API FRigVMFunction_MathRBFVectorWeightFunctor;


/** Function to use for each target falloff */
UENUM()
enum class ERBFKernelType : uint8
{
	Gaussian,
	Exponential,
	Linear,
	Cubic,
	Quintic
};

/** Function to use for computing distance between the input and target 
	quaternions. */
UENUM()
enum class ERBFQuatDistanceType : uint8
{
	Euclidean,
	ArcLength,
	SwingAngle,
	TwistAngle,
};

/** Function to use for computing distance between the input and target 
	quaternions. */
UENUM()
enum class ERBFVectorDistanceType : uint8
{
	Euclidean,
	Manhattan,
	ArcLength
};


USTRUCT()
struct FRigVMFunction_MathRBFInterpolateQuatWorkData
{
	GENERATED_BODY()

	// There's no current mechanism for detecting whether an entire input stream is
	// constant or variable in the RigVM. So what we do here is that the firs time
	// around we set up an interpolator 
	TRBFInterpolator<FQuat> Interpolator;
	TArray<FQuat> Targets;
	uint64 Hash = 0;
	bool bAreTargetsConstant = true;
};

USTRUCT()
struct FRigVMFunction_MathRBFInterpolateVectorWorkData
{
	GENERATED_BODY()

	TRBFInterpolator<FVector> Interpolator;
	TArray<FVector> Targets;
	uint64 Hash = 0;
	bool bAreTargetsConstant = true;
};


/**
 * The base class for all pure RBF nodes
 */
USTRUCT(meta = (Abstract, Category = "Math|RBF Interpolation"))
struct FRigVMFunction_MathRBFInterpolateBase : 
	public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/**
 * The base class for all quaternion RBF nodes
 */
USTRUCT(meta = (Abstract, TemplateName="RBF Quaternion", Keywords = "RBF,Interpolate,Quaternion"))
struct FRigVMFunction_MathRBFInterpolateQuatBase :
	public FRigVMFunction_MathRBFInterpolateBase
{
	GENERATED_BODY()

	// The input quaternion
	UPROPERTY(meta = (Input))
	FQuat Input = FQuat::Identity;

	// The distance function to use
	UPROPERTY(meta = (Input, Constant))
	ERBFQuatDistanceType DistanceFunction = ERBFQuatDistanceType::ArcLength;

	// The smoothing function to use
	UPROPERTY(meta = (Input, Constant))
	ERBFKernelType SmoothingFunction = ERBFKernelType::Gaussian;

	// The smoothing angle
	UPROPERTY(meta = (Input, Constant, UIMin = "0", UIMax = "180"))
	float SmoothingAngle = 45.0f;

	// If true the resulting output will be normalized
	UPROPERTY(meta = (Input, Constant))
	bool bNormalizeOutput = false;

	// The twist axis of the input quaternion
	UPROPERTY(meta = (Input, EditCondition = "DistanceFunction == ERBFQuatDistanceType::SwingAngle || DistanceFunction == ERBFQuatDistanceType::TwistAngle"))
	FVector TwistAxis = FVector::ForwardVector;

	UPROPERTY(transient)
	FRigVMFunction_MathRBFInterpolateQuatWorkData WorkData;

protected:
	
	template<typename T>
	static void GetInterpolatedWeights(
		FRigVMFunction_MathRBFInterpolateQuatWorkData& WorkData,
		const TArrayView<const T>& Targets,
		const FQuat& Input,
		ERBFQuatDistanceType DistanceFunction,
		ERBFKernelType SmoothingFunction,
		float SmoothingAngle,
		bool bNormalizeOutput,
		FVector TwistAxis,
		TArray<float>& Weights
	);

	static UE_API uint64 HashTargets(const TArrayView<const FQuat>& Targets);
};

/**
 * The base class for all vector RBF nodes
 */
USTRUCT(meta = (Abstract, TemplateName="RBF Vector", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorBase :
	public FRigVMFunction_MathRBFInterpolateBase
{
	GENERATED_BODY()

	// The input vector
	UPROPERTY(meta = (Input))
	FVector Input = FVector::ZeroVector;

	// The distance function to use
	UPROPERTY(meta = (Input, Constant))
	ERBFVectorDistanceType DistanceFunction = ERBFVectorDistanceType::Euclidean;

	// The smoothing function to use
	UPROPERTY(meta = (Input, Constant))
	ERBFKernelType SmoothingFunction = ERBFKernelType::Gaussian;

	// The radius to apply for the smoothing function
	UPROPERTY(meta = (Input, Constant))
	float SmoothingRadius = 5.0f;

	// If true the resulting output will be normalized
	UPROPERTY(meta = (Input, Constant))
	bool bNormalizeOutput = false;

	UPROPERTY(transient)
	FRigVMFunction_MathRBFInterpolateVectorWorkData WorkData;

protected:
	template<typename T>
	static void GetInterpolatedWeights(
		FRigVMFunction_MathRBFInterpolateVectorWorkData& WorkData,
		const TArrayView<const T>& Targets,
		const FVector& Input,
		ERBFVectorDistanceType DistanceFunction,
		ERBFKernelType SmoothingFunction,
		float SmoothingRadius,
		bool bNormalizeOutput,
		TArray<float>& Weights
	);

	static UE_API uint64 HashTargets(const TArrayView<const FVector>& Targets);
};

// The actual unit implementation declarations.

// Quat -> T

USTRUCT()
struct FMathRBFInterpolateQuatFloat_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	UPROPERTY(meta = (Input))
	float Value = 0.0f;
};

// A RBF interpolator from quaternion to float
USTRUCT(meta = (Abstract, DisplayName = "RBF Quaternion to Float"))
struct FRigVMFunction_MathRBFInterpolateQuatFloat : 
	public FRigVMFunction_MathRBFInterpolateQuatBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateQuatFloat_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	float Output = 0.0f;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateQuatVector_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	UPROPERTY(meta = (Input))
	FVector Value = FVector::ZeroVector;
};

// A RBF interpolator from quaternion to vector
USTRUCT(meta = (Abstract, DisplayName = "RBF Quaternion to Vector"))
struct FRigVMFunction_MathRBFInterpolateQuatVector :
	public FRigVMFunction_MathRBFInterpolateQuatBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateQuatVector_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FVector Output = FVector::ZeroVector;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateQuatColor_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	UPROPERTY(meta = (Input))
	FLinearColor Value = FLinearColor::Transparent;
};

// A RBF interpolator from quaternion to color
USTRUCT(meta = (Abstract, DisplayName = "RBF Quaternion to Color"))
struct FRigVMFunction_MathRBFInterpolateQuatColor :
	public FRigVMFunction_MathRBFInterpolateQuatBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateQuatColor_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FLinearColor Output = FLinearColor::Transparent;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateQuatQuat_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	UPROPERTY(meta = (Input))
	FQuat Value = FQuat::Identity;
};

// A RBF interpolator from quaternion to quaternion
USTRUCT(meta = (Abstract, DisplayName = "RBF Quaternion to Quaternion"))
struct FRigVMFunction_MathRBFInterpolateQuatQuat :
	public FRigVMFunction_MathRBFInterpolateQuatBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateQuatQuat_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FQuat Output = FQuat::Identity;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateQuatXform_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	UPROPERTY(meta = (Input))
	FTransform Value = FTransform::Identity;
};

// A RBF interpolator from quaternion to transform
USTRUCT(meta = (Abstract, DisplayName = "RBF Quaternion to Transform"))
struct FRigVMFunction_MathRBFInterpolateQuatXform :
	public FRigVMFunction_MathRBFInterpolateQuatBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateQuatXform_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FTransform Output = FTransform::Identity;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


/// Vector->T

USTRUCT()
struct FMathRBFInterpolateVectorFloat_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	UPROPERTY(meta = (Input))
	float Value = 0.0f;
};

// A RBF interpolator from vector to float
USTRUCT(meta = (Abstract, DisplayName = "RBF Vector to Float", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorFloat :
	public FRigVMFunction_MathRBFInterpolateVectorBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateVectorFloat_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	float Output = 0.0f;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateVectorVector_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	UPROPERTY(meta = (Input))
	FVector Value = FVector::ZeroVector;
};

// A RBF interpolator from vector to vector
USTRUCT(meta = (Abstract, DisplayName = "RBF Vector to Vector", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorVector :
	public FRigVMFunction_MathRBFInterpolateVectorBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateVectorVector_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FVector Output = FVector::ZeroVector;

	RIGVM_METHOD()
	UE_API void Execute() override;
};


USTRUCT()
struct FMathRBFInterpolateVectorColor_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	UPROPERTY(meta = (Input))
	FLinearColor Value = FLinearColor::Transparent;
};

// A RBF interpolator from vector to color
USTRUCT(meta = (Abstract, DisplayName = "RBF Vector to Color", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorColor :
	public FRigVMFunction_MathRBFInterpolateVectorBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateVectorColor_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FLinearColor Output = FLinearColor::Transparent;

	RIGVM_METHOD()
	UE_API void Execute() override;
};



USTRUCT()
struct FMathRBFInterpolateVectorQuat_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	UPROPERTY(meta = (Input))
	FQuat Value = FQuat::Identity;
};

// A RBF interpolator from vector to quaternion
USTRUCT(meta = (Abstract, DisplayName = "RBF Vector to Quat", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorQuat :
	public FRigVMFunction_MathRBFInterpolateVectorBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateVectorQuat_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FQuat Output = FQuat::Identity;

	RIGVM_METHOD()
	UE_API void Execute() override;
};

USTRUCT()
struct FMathRBFInterpolateVectorXform_Target
{
	GENERATED_BODY()

	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	UPROPERTY(meta = (Input))
	FTransform Value = FTransform::Identity;
};

// A RBF interpolator from vector to transform
USTRUCT(meta = (Abstract, DisplayName = "RBF Vector to Transform", Keywords = "RBF,Interpolate,Vector"))
struct FRigVMFunction_MathRBFInterpolateVectorXform :
	public FRigVMFunction_MathRBFInterpolateVectorBase
{
	GENERATED_BODY()

	// The array of targets to interpolate within
	UPROPERTY(meta = (Input))
	TArray<FMathRBFInterpolateVectorXform_Target> Targets;

	// The interpolated result
	UPROPERTY(meta = (Output))
	FTransform Output = FTransform::Identity;

	RIGVM_METHOD()
	UE_API void Execute() override;
};

#undef UE_API
