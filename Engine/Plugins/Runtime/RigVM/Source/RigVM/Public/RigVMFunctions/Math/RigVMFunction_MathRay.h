// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Ray.h"
#include "Math/Plane.h"
#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathRay.generated.h"

/*
 * The base class for all pure ray math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Ray", MenuDescSuffix="(Ray)"))
struct FRigVMFunction_MathRayBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/**
 * Returns the closest point intersection of a ray with another ray
 */
USTRUCT(meta = (DisplayName = "Intersect Ray", Keywords = "Closest,Ray,Cross"))
struct FRigVMFunction_MathRayIntersectRay : public FRigVMFunction_MathRayBase
{
	GENERATED_BODY()

	FRigVMFunction_MathRayIntersectRay()
	{
		A = B = FRay();
		Result = FVector::ZeroVector;
		Distance = RatioA = RatioB = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first ray to intersect
	UPROPERTY(meta = (Input))
	FRay A;

	// The second ray to intersect
	UPROPERTY(meta = (Input))
	FRay B;

	// The resulting intersection position.
	// This is either on the rays themselves
	// or at the closest position between the two.
	UPROPERTY(meta = (Output))
	FVector Result;

	// The distance between the two rays (or 0 if they touch)
	UPROPERTY(meta = (Output))
	float Distance;

	// The ratio on the first ray at the intersection
	UPROPERTY(meta = (Output))
	float RatioA;

	// The ratio on the second ray at the intersection
	UPROPERTY(meta = (Output))
	float RatioB;
};

/**
 * Returns the closest point intersection of a ray with a plane
 */
USTRUCT(meta = (DisplayName = "Intersect Plane", Keywords = "Closest,Ray,Cross"))
struct FRigVMFunction_MathRayIntersectPlane : public FRigVMFunction_MathRayBase
{
	GENERATED_BODY()

	FRigVMFunction_MathRayIntersectPlane()
	{
		Ray = FRay();
		PlanePoint = FVector::ZeroVector;
		PlaneNormal = FVector::UpVector;
		Result = FVector::ZeroVector;
		Distance = Ratio = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The ray to intersect with the plane
	UPROPERTY(meta = (Input))
	FRay Ray;

	// The point on the plane to intersect the ray with
	UPROPERTY(meta = (Input))
	FVector PlanePoint;

	// The normal of the plane to intersect the ray with
	UPROPERTY(meta = (Input))
	FVector PlaneNormal;

	// The resulting intersection position
	UPROPERTY(meta = (Output))
	FVector Result;

	// The distance of the ray origin to the plane
	UPROPERTY(meta = (Output))
	float Distance;

	// The ratio along the ray up to the intersection point
	UPROPERTY(meta = (Output))
	float Ratio;
};

/**
 * Returns the position on a ray
 */
USTRUCT(meta=(DisplayName="GetAt", Keywords="Ratio,Percentage"))
struct FRigVMFunction_MathRayGetAt : public FRigVMFunction_MathRayBase
{
	GENERATED_BODY()

	FRigVMFunction_MathRayGetAt()
	{
		Ray = FRay();
		Ratio = 0.f;
		Result = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The ray to query
	UPROPERTY(meta = (Input))
	FRay Ray;

	// The ratio to query the ray at
	UPROPERTY(meta = (Input))
	float Ratio;

	// The resulting position on the ray
	UPROPERTY(meta = (Output))
	FVector Result;
};

/**
 * Transforms a ray
 */
USTRUCT(meta=(DisplayName="Transform Ray", Keywords="Multiply,Project"))
struct FRigVMFunction_MathRayTransform : public FRigVMFunction_MathRayBase
{
	GENERATED_BODY()

	FRigVMFunction_MathRayTransform()
	{
		Ray = Result = FRay();
		Transform = FTransform::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The ray to transform
	UPROPERTY(meta = (Input))
	FRay Ray;

	// The transform to apply to the ray
	UPROPERTY(meta = (Input))
	FTransform Transform;

	// The resulting transformed ray
	UPROPERTY(meta = (Output))
	FRay Result;
};
