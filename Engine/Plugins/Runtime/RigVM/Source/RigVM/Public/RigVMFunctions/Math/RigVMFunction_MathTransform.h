// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EulerTransform.h"
#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_MathTransform.generated.h"

#define UE_API RIGVM_API

/*
 * The base class for all pure transform math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Transform", MenuDescSuffix="(Transform)"))
struct FRigVMFunction_MathTransformBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all mutable transform math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Transform", MenuDescSuffix="(Transform)"))
struct FRigVMFunction_MathTransformMutableBase : public FRigVMFunction_MathMutableBase
{
	GENERATED_BODY()
};

/*
 * The base class for all unary transform math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathTransformUnaryOp : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformUnaryOp()
	{
		Value = Result = FTransform::Identity;
	}

	// The input value
	UPROPERTY(meta=(Input))
	FTransform Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	FTransform Result;
};

/*
 * The base class for all binary transform math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathTransformBinaryOp : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformBinaryOp()
	{
		A = B = Result = FTransform::Identity;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	FTransform A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	FTransform B;

	// The resulting value
	UPROPERTY(meta=(Output))
	FTransform Result;
};

/*
 * The base class for all aggregational transform math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathTransformBinaryAggregateOp : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformBinaryAggregateOp()
	{
		A = B = Result = FTransform::Identity;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FTransform A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FTransform B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	FTransform Result;
};

/**
 * Makes a transform from its components
 */
USTRUCT(meta=(DisplayName="Make Transform", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathTransformMake : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigVMFunction_MathTransformMake()
	{
		Translation = FVector::ZeroVector;
		Rotation = FQuat::Identity;
		Scale = FVector::OneVector;
		Result = FTransform::Identity;
	}

	UPROPERTY(meta=(Input))
	FVector Translation;

	UPROPERTY(meta=(Input))
	FQuat Rotation;

	UPROPERTY(meta=(Input))
	FVector Scale;

	UPROPERTY(meta=(Output))
	FTransform Result;

	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a quaternion based transform from a euler based transform
 */
USTRUCT(meta=(DisplayName="From Euler Transform", TemplateName="FromEulerTransform", Keywords="Make,Construct", Deprecated="5.0.1"))
struct FRigVMFunction_MathTransformFromEulerTransform : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformFromEulerTransform()
	{
		EulerTransform = FEulerTransform::Identity;
		Result = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FEulerTransform EulerTransform;

	UPROPERTY(meta=(Output))
	FTransform Result;
	
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a quaternion based transform from a euler based transform
 */
USTRUCT(meta=(DisplayName="To Transform", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathTransformFromEulerTransformV2 : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformFromEulerTransformV2()
	{
		Value = FEulerTransform::Identity;
		Result = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input euler transform to convert
	UPROPERTY(meta=(Input))
	FEulerTransform Value;

	// The resulting composed transform
	UPROPERTY(meta=(Output))
	FTransform Result;
};

/**
 * Retrieves a euler based transform from a quaternion based transform
 */
USTRUCT(meta=(DisplayName="To Euler Transform", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathTransformToEulerTransform : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformToEulerTransform()
	{
		Value = FTransform::Identity;
		Result = FEulerTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform to convert
	UPROPERTY(meta=(Input))
	FTransform Value;

	// The resulting composed euler transform
	UPROPERTY(meta=(Output))
	FEulerTransform Result;
};

/**
 * Retrieves the forward, right and up vectors of a transform's quaternion
 */
USTRUCT(meta=(DisplayName="To Vectors", TemplateName="ToVectors", Keywords="Forward,Right,Up"))
struct FRigVMFunction_MathTransformToVectors : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformToVectors()
	{
		Value = FTransform::Identity;
		Forward = Right = Up = FVector(0.f, 0.f, 0.f);
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform
	UPROPERTY(meta=(Input))
	FTransform Value;

	// The rotation's forward axis
	UPROPERTY(meta=(Output))
	FVector Forward;

	// The rotation's right axis
	UPROPERTY(meta=(Output))
	FVector Right;

	// The rotation's up axis
	UPROPERTY(meta=(Output))
	FVector Up;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*,Global"))
struct FRigVMFunction_MathTransformMul : public FRigVMFunction_MathTransformBinaryAggregateOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Returns the relative local transform within a parent's transform
 */
USTRUCT(meta=(DisplayName="Make Relative", TemplateName="Make Relative", Keywords="Local,Global,Absolute"))
struct FRigVMFunction_MathTransformMakeRelative : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformMakeRelative()
	{
		Global = Parent = Local = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The global transform to make relative
	UPROPERTY(meta=(Input))
	FTransform Global;

	// The parent global transform to make the input relative to
	UPROPERTY(meta=(Input))
	FTransform Parent;

	// The resulting relative transform
	UPROPERTY(meta=(Output))
	FTransform Local;
};

/**
 * Returns the absolute global transform within a parent's transform
 */
USTRUCT(meta = (DisplayName = "Make Absolute", TemplateName="Make Absolute", Keywords = "Local,Global,Relative"))
struct FRigVMFunction_MathTransformMakeAbsolute : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformMakeAbsolute()
	{
		Global = Parent = Local = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input local transform to make absolute
	UPROPERTY(meta = (Input))
	FTransform Local;

	// The parent global transform to make the local absolute to
	UPROPERTY(meta = (Input))
	FTransform Parent;

	// The resulting global transform
	UPROPERTY(meta = (Output))
	FTransform Global;
};

/**
* Treats the provided transforms as a chain with global / local transforms, and
* projects each transform into the target space. Optionally you can provide
* a custom parent indices array, with which you can represent more than just chains.
*/
USTRUCT(meta=(DisplayName="Make Transform Array Relative", Keywords="Local,Global,Absolute,Array,Accumulate"))
struct FRigVMFunction_MathTransformAccumulateArray : public FRigVMFunction_MathTransformMutableBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformAccumulateArray()
	{
		TargetSpace = ERigVMTransformSpace::GlobalSpace;
		Root = FTransform::Identity;
	}

	UE_API virtual FString GetUnitLabel() const override;

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform array to accumulate
	UPROPERTY(meta=(Input, Output))
	TArray<FTransform> Transforms;

	/**
	* Defines the space to project to
	*/ 
	UPROPERTY(meta = (Input))
	ERigVMTransformSpace TargetSpace;

	/**
	 * Provides the parent transform for the root
	 */
	UPROPERTY(meta=(Input))
	FTransform Root;

	/**
	 * If this array is the same size as the transforms array the indices will be used
	 * to look up each transform's parent. They are expected to be in order.
	 */
	UPROPERTY(meta=(Input))
	TArray<int32> ParentIndices;
};

/**
 * Returns the inverted transform of the input
 */
USTRUCT(meta=(DisplayName="Inverse", TemplateName="Inverse"))
struct FRigVMFunction_MathTransformInverse : public FRigVMFunction_MathTransformUnaryOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	UE_API virtual void Execute() override;
};

/**
 * Linearly interpolates between A and B using the ratio T
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend"))
struct FRigVMFunction_MathTransformLerp : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformLerp()
	{
		A = B = Result = FTransform::Identity;
		T = 0.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The first transform to interpolate from
	UPROPERTY(meta=(Input))
	FTransform A;

	// The second transform to interpolate to
	UPROPERTY(meta=(Input))
	FTransform B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	float T;

	// The resulting interpolated transform
	UPROPERTY(meta=(Output))
	FTransform Result;
};

/**
 * Return one of the two values based on the condition
 */
USTRUCT(meta=(DisplayName="Select", Keywords="Pick,If", Deprecated = "4.26.0"))
struct FRigVMFunction_MathTransformSelectBool : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformSelectBool()
	{
		Condition = false;
		IfTrue = IfFalse = Result = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	bool Condition;

	UPROPERTY(meta=(Input))
	FTransform IfTrue;

	UPROPERTY(meta=(Input))
	FTransform IfFalse;

	UPROPERTY(meta=(Output))
	FTransform Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Rotates a given vector (direction) by the transform
 */
USTRUCT(meta=(DisplayName="Rotate Vector", TemplateName="Rotate Vector", Keywords="Transform,Direction,TransformDirection"))
struct FRigVMFunction_MathTransformRotateVector : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformRotateVector()
	{
		Transform = FTransform::Identity;
		Vector = Result = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;
	
	// The input transform to rotate the input vector by
	UPROPERTY(meta=(Input))
	FTransform Transform;

	// The input vector to rotate
	UPROPERTY(meta=(Input))
	FVector Vector;

	// The resulting rotated vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Multiplies a given vector (location) by the transform
 */
USTRUCT(meta=(DisplayName="Transform Location", Keywords="Multiply"))
struct FRigVMFunction_MathTransformTransformVector : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformTransformVector()
	{
		Transform = FTransform::Identity;
		Location = Result = FVector::ZeroVector;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform to transform the input vector by
	UPROPERTY(meta=(Input))
	FTransform Transform;

	// The input vector to transform
	UPROPERTY(meta=(Input))
	FVector Location;

	// The resulting transformed vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Composes a Transform (and Euler Transform) from its components.
 */
USTRUCT(meta=(DisplayName="Transform from SRT", Keywords ="EulerTransform,Scale,Rotation,Orientation,Translation,Location"))
struct FRigVMFunction_MathTransformFromSRT : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformFromSRT()
	{
		Location = FVector::ZeroVector;
		Rotation = FVector::ZeroVector;
		RotationOrder = EEulerRotationOrder::XYZ;
		Scale = FVector::OneVector;
		Transform = FTransform::Identity;
		EulerTransform = FEulerTransform::Identity;
	}
	
	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The location for the desired transform
	UPROPERTY(meta=(Input))
	FVector Location;

	// The euler angles in degrees for the desired transform
	UPROPERTY(meta=(Input))
	FVector Rotation;

	// The rotation order to interpret the euler angles by
	UPROPERTY(meta=(Input))
	EEulerRotationOrder RotationOrder;

	// The scale for the desired transform
	UPROPERTY(meta=(Input))
	FVector Scale;

	// The result as a transform
	UPROPERTY(meta=(Output))
	FTransform Transform;

	// The result as a euler transform
	UPROPERTY(meta=(Output))
	FEulerTransform EulerTransform;
};


/**
 * Decomposes a Transform Array to its components.
 */
USTRUCT(meta = (DisplayName = "Transform Array to SRT", Keywords = "EulerTransform,Scale,Rotation,Orientation,Translation,Location"))
struct FRigVMFunction_MathTransformArrayToSRT : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformArrayToSRT()
	{
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform array to decompose
	UPROPERTY(meta = (Input))
	TArray<FTransform> Transforms;

	// The array of translations - one for each input transform
	UPROPERTY(meta = (Output))
	TArray<FVector> Translations;

	// The array of rotation - one for each input transform
	UPROPERTY(meta = (Output))
	TArray<FQuat> Rotations;

	// The array of scale values - one for each input transform
	UPROPERTY(meta = (Output))
	TArray<FVector> Scales;
};

/**
 * Clamps a transform's position using a plane collision, cylindric collision or spherical collision.
 */
USTRUCT(meta = (DisplayName = "Clamp Spatially", TemplateName="ClampSpatially", Keywords = "Collide,Collision"))
struct FRigVMFunction_MathTransformClampSpatially : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformClampSpatially()
	{
		Value = Result = FTransform::Identity;
		Axis = EAxis::X;
		Type = ERigVMClampSpatialMode::Plane;
		Minimum = 0.f;
		Maximum = 100.f;
		Space = FTransform::Identity;
		bDrawDebug = false;
		DebugColor = FLinearColor::Red;
		DebugThickness = 1.f;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform to clamp
	UPROPERTY(meta = (Input))
	FTransform Value;

	// The axis to use for the filter
	UPROPERTY(meta = (Input))
	TEnumAsByte<EAxis::Type> Axis;

	// The filter / spatial mode to use
	UPROPERTY(meta = (Input))
	TEnumAsByte<ERigVMClampSpatialMode::Type> Type;

	// The minimum allowed distance at which a collision occurs. 
	// Note: For capsule this represents the radius.
	// Disable by setting to 0.0.
	UPROPERTY(meta = (Input))
	float Minimum;

	// This maximum allowed distance.
	// A collision will occur towards the center at this wall.
	// Note: For capsule this represents the length.
	// Disable by setting to 0.0.
	UPROPERTY(meta = (Input))
	float Maximum;

	// The space this spatial clamp happens within.
	// The input position will be projected into this space.
	UPROPERTY(meta = (Input))
	FTransform Space;

	// Draws debug information if True
	UPROPERTY(meta = (Input))
	bool bDrawDebug;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor DebugColor;

	// The thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float DebugThickness;

	// The resulting transform with a clamped position
	UPROPERTY(meta = (Output))
	FTransform Result;
};

/**
 * Mirror a transform about a central transform.
 */
USTRUCT(meta=(DisplayName="Mirror", TemplateName="Mirror"))
struct FRigVMFunction_MathTransformMirrorTransform : public FRigVMFunction_MathTransformBase
{
	GENERATED_BODY()

	FRigVMFunction_MathTransformMirrorTransform()
	{
		Value = Result = FTransform::Identity;
		MirrorAxis = EAxis::X;
		AxisToFlip = EAxis::Z;
		CentralTransform = FTransform::Identity;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input transform to mirror
	UPROPERTY(meta=(Input))
	FTransform Value;

	// the axis to mirror against
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> MirrorAxis;

	// the axis to flip for rotations
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> AxisToFlip;

	// The transform about which to mirror
	UPROPERTY(meta=(Input))
	FTransform CentralTransform;

	// The resulting mirrored transform
	UPROPERTY(meta=(Output))
	FTransform Result;
};

#undef UE_API 