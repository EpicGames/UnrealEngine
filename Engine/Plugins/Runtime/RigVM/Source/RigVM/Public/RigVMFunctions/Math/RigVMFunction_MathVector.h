// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMMathLibrary.h"
#include "RigVMFunction_MathVector.generated.h"

/*
 * The base class for all pure vector math nodes
 */
USTRUCT(meta=(Abstract, Category="Math|Vector", MenuDescSuffix="(Vector)"))
struct FRigVMFunction_MathVectorBase : public FRigVMFunction_MathBase
{
	GENERATED_BODY()
};

/*
 * The base class for all unary vector math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathVectorUnaryOp : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()
	
	FRigVMFunction_MathVectorUnaryOp()
	{
		Value = Result = FVector::ZeroVector;
	}

	// The input value
	UPROPERTY(meta=(Input))
	FVector Value;

	// The result of the operation
	UPROPERTY(meta=(Output))
	FVector Result;
};

/*
 * The base class for all binary vector math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathVectorBinaryOp : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorBinaryOp()
	{
		A = B = Result = FVector::ZeroVector;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input))
	FVector A;

	// The second value for the operation
	UPROPERTY(meta=(Input))
	FVector B;

	// The resulting value
	UPROPERTY(meta=(Output))
	FVector Result;
};

/*
 * The base class for all aggregational vector math operations
 */
USTRUCT(meta=(Abstract))
struct FRigVMFunction_MathVectorBinaryAggregateOp : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorBinaryAggregateOp()
	{
		A = B = Result = FVector::ZeroVector;
	}

	// The first value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FVector A;

	// The second value for the operation
	UPROPERTY(meta=(Input, Aggregate))
	FVector B;

	// The resulting value
	UPROPERTY(meta=(Output, Aggregate))
	FVector Result;
};

/**
 * Makes a vector from its components
 */
USTRUCT(meta=(DisplayName="Make Vector", Keywords="Make,Construct,Constant", Deprecated="5.7"))
struct FRigVMFunction_MathVectorMake : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathVectorMake()
	{
		X = Y = Z = 0.f;
		Result = FVector::ZeroVector;
	}

	UPROPERTY(meta=(Input))
	float X;

	UPROPERTY(meta=(Input))
	float Y;

	UPROPERTY(meta=(Input))
	float Z;

	UPROPERTY(meta=(Output))
	FVector Result;
		
	RIGVM_METHOD()
	RIGVM_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Makes a vector from a single float
 */
USTRUCT(meta=(DisplayName="From Float", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathVectorFromFloat : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathVectorFromFloat()
	{
		Value = 0.f;
		Result = FVector::ZeroVector;
	}

	// The input float value to use for all vector components 
	UPROPERTY(meta=(Input))
	float Value;

	// The resulting (Value,Value,Value) vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Makes a vector from a single double
 */
USTRUCT(meta=(DisplayName="From Double", TemplateName="Cast", ExecuteContext="FRigVMExecuteContext", Keywords="Make,Construct"))
struct FRigVMFunction_MathVectorFromDouble : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	FRigVMFunction_MathVectorFromDouble()
	{
		Value = 0.0;
		Result = FVector::ZeroVector;
	}

	// The input value to use for all vector components
	UPROPERTY(meta=(Input))
	double Value;

	// The resulting (Value,Value,Value) vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Returns the sum of the two values
 */
USTRUCT(meta=(DisplayName="Add", TemplateName="Add", Keywords="Sum,+"))
struct FRigVMFunction_MathVectorAdd : public FRigVMFunction_MathVectorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the difference of the two values
 */
USTRUCT(meta=(DisplayName="Subtract", TemplateName="Subtract", Keywords="-"))
struct FRigVMFunction_MathVectorSub : public FRigVMFunction_MathVectorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the product of the two values
 */
USTRUCT(meta=(DisplayName="Multiply", TemplateName="Multiply", Keywords="Product,*"))
struct FRigVMFunction_MathVectorMul : public FRigVMFunction_MathVectorBinaryAggregateOp
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMul()
	{
		A = B = FVector::OneVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the product of the vector and the float value
 */
USTRUCT(meta = (DisplayName = "Scale", TemplateName="Scale", Keywords = "Multiply,Product,*,ByScalar,ByFloat"))
struct FRigVMFunction_MathVectorScale : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

		FRigVMFunction_MathVectorScale()
	{
		Value = Result = FVector::ZeroVector;
		Factor = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to scale
	UPROPERTY(meta = (Input))
	FVector Value;

	// The factor to scale the input vector by
	UPROPERTY(meta = (Input))
	float Factor;

	// The resulting scaled vector
	UPROPERTY(meta = (Output))
	FVector Result;
};

/**
 * Returns the division of the two values
 */
USTRUCT(meta=(DisplayName="Divide", TemplateName="Divide", Keywords="Division,Divisor,/"))
struct FRigVMFunction_MathVectorDiv : public FRigVMFunction_MathVectorBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorDiv()
	{
		B = FVector::OneVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the modulo of the two values
 */
USTRUCT(meta=(DisplayName="Modulo", TemplateName="Modulo", Keywords="%,fmod"))
struct FRigVMFunction_MathVectorMod : public FRigVMFunction_MathVectorBinaryOp
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMod()
	{
		A = FVector::ZeroVector;
		B = FVector::OneVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the smaller of the two values for each component
 */
USTRUCT(meta=(DisplayName="Minimum", TemplateName="Minimum"))
struct FRigVMFunction_MathVectorMin : public FRigVMFunction_MathVectorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the larger of the two values each component
 */
USTRUCT(meta=(DisplayName="Maximum", TemplateName="Maximum"))
struct FRigVMFunction_MathVectorMax : public FRigVMFunction_MathVectorBinaryAggregateOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the negative value
 */
USTRUCT(meta=(DisplayName="Negate", TemplateName="Negate", Keywords="-,Abs"))
struct FRigVMFunction_MathVectorNegate : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the absolute (positive) value
 */
USTRUCT(meta=(DisplayName="Absolute", TemplateName="Absolute", Keywords="Abs,Neg"))
struct FRigVMFunction_MathVectorAbs : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the closest lower full number (integer) of the value for each component
 */
USTRUCT(meta=(DisplayName="Floor", Keywords="Round"))
struct FRigVMFunction_MathVectorFloor : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the closest higher full number (integer) of the value for each component
 */
USTRUCT(meta=(DisplayName="Ceiling", Keywords="Round"))
struct FRigVMFunction_MathVectorCeil : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the closest full number (integer) of the value for each component
 */
USTRUCT(meta=(DisplayName="Round"))
struct FRigVMFunction_MathVectorRound : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the sign of the value (+1 for >= FVector(0.f, 0.f, 0.f), -1 for < 0.f) for each component
 */
USTRUCT(meta=(DisplayName="Sign", TemplateName="Sign"))
struct FRigVMFunction_MathVectorSign : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Clamps the given value within the range provided by minimum and maximum for each component
 */
USTRUCT(meta=(DisplayName="Clamp", TemplateName="Clamp", Keywords="Range,Remap"))
struct FRigVMFunction_MathVectorClamp : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorClamp()
	{
		Value = Minimum = Result = FVector::ZeroVector;
		Maximum = FVector::OneVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to clamp
	UPROPERTY(meta=(Input))
	FVector Value;

	// The Minimum vector to clamp the input by
	UPROPERTY(meta=(Input))
	FVector Minimum;

	// The Maximum vector to clamp the input by
	UPROPERTY(meta=(Input))
	FVector Maximum;

	// The resulting clamped vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Linearly interpolates between A and B using the ratio T
 */
USTRUCT(meta=(DisplayName="Interpolate", TemplateName="Interpolate", Keywords="Lerp,Mix,Blend"))
struct FRigVMFunction_MathVectorLerp : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorLerp()
	{
		A = Result = FVector::ZeroVector;
		B = FVector::OneVector;
		T = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to interpolate from
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to interpolate to
	UPROPERTY(meta=(Input))
	FVector B;

	// The blend value for the interpolation
	UPROPERTY(meta=(Input, UIMin = "0", UIMax = "1"))
	float T;

	// The resulting interpolated vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Remaps the given value from a source range to a target range for each component
 */
USTRUCT(meta=(DisplayName="Remap", TemplateName="Remap", Keywords="Rescale,Scale"))
struct FRigVMFunction_MathVectorRemap : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorRemap()
	{
		Value = SourceMinimum = TargetMinimum = Result = FVector::ZeroVector;
		SourceMaximum = TargetMaximum = FVector::OneVector;
		bClamp = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to remap
	UPROPERTY(meta=(Input))
	FVector Value;

	// The minimum of the input vector for each component
	UPROPERTY(meta=(Input))
	FVector SourceMinimum;

	// The maximum of the input vector for each component
	UPROPERTY(meta=(Input))
	FVector SourceMaximum;

	// The minimum for the output vector for each component
	UPROPERTY(meta=(Input))
	FVector TargetMinimum;

	// The maximum for the output vector for each component
	UPROPERTY(meta=(Input))
	FVector TargetMaximum;

	/** If set to true the result is clamped to the target range */
	UPROPERTY(meta=(Input))
	bool bClamp;

	// The resulting remapped vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Returns true if the value A equals B
 */
USTRUCT(meta=(DisplayName="Equals", TemplateName="Equals", Keywords="Same,==", Deprecated="5.1"))
struct FRigVMFunction_MathVectorEquals : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorEquals()
	{
		A = B = FVector::ZeroVector;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FVector A;

	UPROPERTY(meta=(Input))
	FVector B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if the value A does not equal B
 */
USTRUCT(meta=(DisplayName="Not Equals", TemplateName="NotEquals", Keywords="Different,!=", Deprecated="5.1"))
struct FRigVMFunction_MathVectorNotEquals : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorNotEquals()
	{
		A = B = FVector::ZeroVector;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	FVector A;

	UPROPERTY(meta=(Input))
	FVector B;

	UPROPERTY(meta=(Output))
	bool Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns true if each component of the value is within a tolerance of zero
 */
USTRUCT(meta=(DisplayName="Is Nearly Zero", TemplateName="IsNearlyZero", Keywords="AlmostZero,0"))
struct FRigVMFunction_MathVectorIsNearlyZero : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorIsNearlyZero()
	{
		Value = FVector::ZeroVector;
		Tolerance = 0.f;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to check
	UPROPERTY(meta=(Input))
	FVector Value;

	// The tolerance to use when comparing each component with zero
	UPROPERTY(meta=(Input))
	float Tolerance;

	// True if the input vector is within Tolerance of zero in each component
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if value A is within a tolerance (in each component) of B
 */
USTRUCT(meta=(DisplayName="Is Nearly Equal", TemplateName="IsNearlyEqual", Keywords="AlmostEqual,=="))
struct FRigVMFunction_MathVectorIsNearlyEqual : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorIsNearlyEqual()
	{
		A = B = FVector::ZeroVector;
		Tolerance = 0.f;
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to compare
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to compare
	UPROPERTY(meta=(Input))
	FVector B;

	// The tolerance to use for the comparison
	UPROPERTY(meta=(Input))
	float Tolerance;

	// True if every component of A is within Tolerance of the corresponding component of B
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Return one of the two values based on the condition
 */
USTRUCT(meta=(DisplayName="Select", Keywords="Pick,If", Deprecated = "4.26.0"))
struct FRigVMFunction_MathVectorSelectBool : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorSelectBool()
	{
		Condition = false;
		IfTrue = IfFalse = Result = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta=(Input))
	bool Condition;

	UPROPERTY(meta=(Input))
	FVector IfTrue;

	UPROPERTY(meta=(Input))
	FVector IfFalse;

	UPROPERTY(meta=(Output))
	FVector Result;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Returns the degrees of a given value in radians
 */
USTRUCT(meta=(DisplayName="Degrees", TemplateName="Degrees"))
struct FRigVMFunction_MathVectorDeg : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the radians of a given value in degrees
 */
USTRUCT(meta=(DisplayName="Radians", TemplateName="Radians"))
struct FRigVMFunction_MathVectorRad : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the squared length of the vector
 */
USTRUCT(meta=(DisplayName="Length Squared", TemplateName="LengthSquared", Keywords="Length,Size,Magnitude"))
struct FRigVMFunction_MathVectorLengthSquared : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorLengthSquared()
	{
		Value = FVector::ZeroVector;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to compute the squared length of
	UPROPERTY(meta=(Input))
	FVector Value;

	// The resulting squared length of the vector
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the length of the vector
 */
USTRUCT(meta=(DisplayName="Length", TemplateName="Length", Keywords="Size,Magnitude"))
struct FRigVMFunction_MathVectorLength : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorLength()
	{
		Value = FVector::ZeroVector;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to compute the length for
	UPROPERTY(meta=(Input))
	FVector Value;

	// The resulting length of the vector
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the distance from A to B
 */
USTRUCT(meta=(DisplayName="Distance Between", TemplateName="Distance"))
struct FRigVMFunction_MathVectorDistance : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorDistance()
	{
		A = B = FVector::ZeroVector;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to measure the distance between
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to measure the distance between
	UPROPERTY(meta=(Input))
	FVector B;

	// The distance between the two vectors A and B
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the cross product between two vectors
 */
USTRUCT(meta=(DisplayName="Cross", TemplateName="Cross", Keywords="^"))
struct FRigVMFunction_MathVectorCross : public FRigVMFunction_MathVectorBinaryOp
{
	GENERATED_BODY()
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Returns the dot product between two vectors
 */
USTRUCT(meta=(DisplayName="Dot", Keywords="Dot,|"))
struct FRigVMFunction_MathVectorDot : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorDot()
	{
		A = B = FVector::ZeroVector;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to compute the dot product for
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to compute the dot product for
	UPROPERTY(meta=(Input))
	FVector B;

	// The resulting dot product between A and B
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns the normalized value
 */
USTRUCT(meta=(DisplayName="Unit", TemplateName="Unit", Keywords="Normalize"))
struct FRigVMFunction_MathVectorUnit : public FRigVMFunction_MathVectorUnaryOp
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
};

/**
 * Sets the length of a given vector
 */
USTRUCT(meta = (DisplayName = "SetLength", TemplateName="SetLength", Keywords = "Unit,Normalize,Scale"))
struct FRigVMFunction_MathVectorSetLength: public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorSetLength()
	{
		Value = Result = FVector(1.f, 0.f, 0.f);
		Length = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to set the length of
	UPROPERTY(meta = (Input))
	FVector Value;

	// The desired length of the vector
	UPROPERTY(meta = (Input))
	float Length;

	/* 
	 * The resulting scaled input vector
	 * (or 0,0,0 in case the input was also 0,0,0)
	 */
	UPROPERTY(meta = (Output))
	FVector Result;
};

/**
 * Clamps the length of a given vector between a minimum and maximum
 */
USTRUCT(meta = (DisplayName = "ClampLength", TemplateName="ClampLength", Keywords = "Unit,Normalize,Scale"))
struct FRigVMFunction_MathVectorClampLength: public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorClampLength()
	{
		Value = Result = FVector(1.f, 0.f, 0.f);
		MinimumLength = 0.f;
		MaximumLength = 1.f;
	}


	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to clamp the length of 
	UPROPERTY(meta = (Input))
	FVector Value;

	// The minimum length to clamp by
	UPROPERTY(meta = (Input))
	float MinimumLength;

	// The maximum length to clamp by
	UPROPERTY(meta = (Input))
	float MaximumLength;

	/* 
	 * The resulting clamped input vector
	 * (or 0,0,0 in case the input was also 0,0,0)
	 */
	UPROPERTY(meta = (Output))
	FVector Result;
};

/**
 * Mirror a vector about a normal vector.
 */
USTRUCT(meta=(DisplayName="Mirror on Normal"))
struct FRigVMFunction_MathVectorMirror : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMirror()
	{
		Value = Result = FVector::ZeroVector;
		Normal = FVector(1.f, 0.f, 0.f);
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to mirror
	UPROPERTY(meta=(Input))
	FVector Value;

	// The normal to mirror the vector by
	UPROPERTY(meta=(Input))
	FVector Normal;

	// The resulting mirrored vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Returns the angle between two vectors in radians
 */
USTRUCT(meta=(DisplayName="Angle Between", TemplateName="AngleBetween"))
struct FRigVMFunction_MathVectorAngle : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

		FRigVMFunction_MathVectorAngle()
	{
		A = B = FVector::ZeroVector;
		Result = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to measure the angle between
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to measure the angle between
	UPROPERTY(meta=(Input))
	FVector B;

	// The angle between A and B in radians
	UPROPERTY(meta=(Output))
	float Result;
};

/**
 * Returns true if the two vectors are parallel
 */
USTRUCT(meta=(DisplayName="Parallel", TemplateName="Parallel"))
struct FRigVMFunction_MathVectorParallel : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorParallel()
	{
		A = B = FVector(1.f, 0.f, 0.f);
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to compare
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to compare
	UPROPERTY(meta=(Input))
	FVector B;

	// True if the two vectors A and B are parallel
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns true if the two vectors are orthogonal
 */
USTRUCT(meta=(DisplayName="Orthogonal", TemplateName="Orthogonal"))
struct FRigVMFunction_MathVectorOrthogonal : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorOrthogonal()
	{
		A = FVector(1.f, 0.f, 0.f);
		B = FVector(0.f, 1.f, 0.f);
		Result = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first vector to compare
	UPROPERTY(meta=(Input))
	FVector A;

	// The second vector to compare
	UPROPERTY(meta=(Input))
	FVector B;

	// True if the two vectors A and B are orthogonal
	UPROPERTY(meta=(Output))
	bool Result;
};

/**
 * Returns the 4 point bezier interpolation
 */
USTRUCT(meta=(DisplayName="Bezier Four Point", Deprecated = "5.0"))
struct FRigVMFunction_MathVectorBezierFourPoint : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorBezierFourPoint()
	{
		Bezier = FRigVMFourPointBezier();
		T = 0.f;
		Result = Tangent = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input four point structure
	UPROPERTY(meta=(Input))
	FRigVMFourPointBezier Bezier;

	// The interpolation T value along the bezier to compute
	UPROPERTY(meta=(Input))
	float T;

	// The resulting position at value T on the bezier
	UPROPERTY(meta=(Output))
	FVector Result;

	// The resulting tangent at value T on the bezier
	UPROPERTY(meta=(Output))
	FVector Tangent;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Creates a bezier four point
 */
USTRUCT(meta = (DisplayName = "Make Bezier Four Point", Constant, Deprecated = "5.0"))
struct FRigVMFunction_MathVectorMakeBezierFourPoint : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMakeBezierFourPoint()
	{
		Bezier = FRigVMFourPointBezier();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input, Output))
	FRigVMFourPointBezier Bezier;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Clamps a position using a plane collision, cylindric collision or spherical collision.
 * The collision happens both towards an inner envelope (minimum) and towards an outer envelope (maximum).
 * You can disable the inner / outer envelope / collision by setting the minimum / maximum to 0.0.
 */
USTRUCT(meta = (DisplayName = "Clamp Spatially", TemplateName="ClampSpatially", Keywords="Collide,Collision"))
struct FRigVMFunction_MathVectorClampSpatially: public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorClampSpatially()
	{
		Value = Result = FVector::ZeroVector;
		Axis = EAxis::X;
		Type = ERigVMClampSpatialMode::Plane;
		Minimum = 0.f;
		Maximum = 100.f;
		Space = FTransform::Identity;
		bDrawDebug = false;
		DebugColor = FLinearColor::Red;
		DebugThickness = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input position to clamp
	UPROPERTY(meta = (Input))
	FVector Value;

	// The axis to use for the filter
	UPROPERTY(meta = (Input))
	TEnumAsByte<EAxis::Type> Axis;

	// The type of filter / mode to apply
	UPROPERTY(meta = (Input))
	TEnumAsByte<ERigVMClampSpatialMode::Type> Type;

	// The minimum allowed distance at which a collision occurs.
	// Note: For capsule this represents the radius.
	// Disable by setting to 0.0.
	UPROPERTY(meta = (Input))
	float Minimum;

	// This maximum allowed distance.
	// A collision will occur towards the center at this wall.
	// Note: For capsule this represents the length.
	// Disable by setting to 0.0.
	UPROPERTY(meta = (Input))
	float Maximum;

	// The space this spatial clamp happens within.
	// The input position will be projected into this space.
	UPROPERTY(meta = (Input))
	FTransform Space;

	// Draws debug information if True
	UPROPERTY(meta = (Input))
	bool bDrawDebug;

	// The color to use for the debug draw
	UPROPERTY(meta = (Input))
	FLinearColor DebugColor;

	// The thickness to use for the debug draw
	UPROPERTY(meta = (Input))
	float DebugThickness;

	// The resulting clamped position
	UPROPERTY(meta = (Output))
	FVector Result;
};

/**
 * Intersects a plane with a vector given a start and direction
 */
USTRUCT(meta = (DisplayName = "Intersect Plane", Keywords = "Collide,Intersect,Raycast"))
struct FRigVMFunction_MathIntersectPlane : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathIntersectPlane()
	{
		Start = PlanePoint = Result = FVector::ZeroVector;
		Direction = PlaneNormal = FVector(0.f, 0.f, 1.f);
		Distance = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The start of the ray to intersect with the plane
	UPROPERTY(meta = (Input))
	FVector Start;

	// The direction of the ray to intersect with the plane
	UPROPERTY(meta = (Input))
	FVector Direction;

	// The point on the plane to intersect with
	UPROPERTY(meta = (Input))
	FVector PlanePoint;

	// The normal of the plane to intersect with
	UPROPERTY(meta = (Input))
	FVector PlaneNormal;

	// The resulting position on the plane
	UPROPERTY(meta = (Output))
	FVector Result;

	// The distance along the ray to the intersection 
	UPROPERTY(meta = (Output))
	float Distance;
};

/**
 * Find the point on the plane that is closest to the given point and the distance between them.
 */
USTRUCT(meta = (DisplayName = "Distance To Plane", Keywords = "Distance,Plane,Closest,Project"))
struct FRigVMFunction_MathDistanceToPlane : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathDistanceToPlane()
	{
		Point = PlanePoint = ClosestPointOnPlane = FVector::ZeroVector;
		PlaneNormal = FVector(0.f, 0.f, 1.f);
		SignedDistance = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The point to measure against the plane
	UPROPERTY(meta = (Input))
	FVector Point;

	// A point on the plane to measure against
	UPROPERTY(meta = (Input))
	FVector PlanePoint;

	// The normal of the plane to measure against
	UPROPERTY(meta = (Input))
	FVector PlaneNormal;

	// The resulting closest point on the plane
	UPROPERTY(meta = (Output))
	FVector ClosestPointOnPlane;

	// The distance to the plane as a signed (positive or negative) value
	UPROPERTY(meta = (Output))
	float SignedDistance;
};

/**
 * Returns the relative local vector within a parent's vector
 */
USTRUCT(meta=(DisplayName="Make Relative", TemplateName="Make Relative", Keywords="Local,Global,Absolute"))
struct FRigVMFunction_MathVectorMakeRelative : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMakeRelative()
	{
		Global = Parent = Local = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
	
	// The global position to make relative
	UPROPERTY(meta=(Input))
	FVector Global;

	// The parent global position to make the input relative to
	UPROPERTY(meta=(Input))
	FVector Parent;

	// The resulting relative position
	UPROPERTY(meta=(Output))
	FVector Local;
};

/**
 * Returns the absolute global vector within a parent's vector
 */
USTRUCT(meta = (DisplayName = "Make Absolute", TemplateName="Make Absolute", Keywords = "Local,Global,Relative"))
struct FRigVMFunction_MathVectorMakeAbsolute : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMakeAbsolute()
	{
		Global = Parent = Local = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input local position to make absolute
	UPROPERTY(meta = (Input))
	FVector Local;

	// The parent global position to make the local absolute to
	UPROPERTY(meta = (Input))
	FVector Parent;

	// The resulting global position
	UPROPERTY(meta = (Output))
	FVector Global;
};

/**
 * Mirror a vector about a central transform.
 */
USTRUCT(meta=(DisplayName="Mirror", TemplateName="Mirror"))
struct FRigVMFunction_MathVectorMirrorTransform : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorMirrorTransform()
	{
		Value = Result = FVector::ZeroVector;
		MirrorAxis = EAxis::X;
		AxisToFlip = EAxis::Z;
		CentralTransform = FTransform::Identity;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input vector to mirror
	UPROPERTY(meta=(Input))
	FVector Value;

	// the axis to mirror against
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> MirrorAxis;

	// the axis to flip for rotations
	UPROPERTY(meta=(Input))
	TEnumAsByte<EAxis::Type> AxisToFlip;

	// The transform about which to mirror
	UPROPERTY(meta=(Input))
	FTransform CentralTransform;

	// The resulting mirrored vector
	UPROPERTY(meta=(Output))
	FVector Result;
};

/**
 * Returns the sum of the given array
 */
USTRUCT(meta = (DisplayName = "Array Sum", TemplateName = "ArraySum"))
struct FRigVMFunction_MathVectorArraySum : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorArraySum()
	{
		Sum = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of vector to add up
	UPROPERTY(meta = (Input))
	TArray<FVector> Array;

	// The sum of all vectors in the input array
	UPROPERTY(meta = (Output))
	FVector Sum;
};


/**
 * Returns the average of the given array
 */
USTRUCT(meta = (DisplayName = "Array Average", TemplateName = "ArrayAverage"))
struct FRigVMFunction_MathVectorArrayAverage : public FRigVMFunction_MathVectorBase
{
	GENERATED_BODY()

	FRigVMFunction_MathVectorArrayAverage()
	{
		Average = FVector::ZeroVector;
	}

	/** Execute logic for this rig unit */
	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The array of vectors to average
	UPROPERTY(meta = (Input))
	TArray<FVector> Array;

	// The resulting average vector of the input array
	UPROPERTY(meta = (Output))
	FVector Average;
};