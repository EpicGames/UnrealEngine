// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_Noise.generated.h"

/**
 * Generates a float through a noise fluctuation process between a min and a max through speed
 */
USTRUCT(meta=(DisplayName="Noise (Float)", Category="Math|Noise", TemplateName="Noise"))
struct FRigVMFunction_NoiseFloat : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_NoiseFloat()
	{
		Value = Minimum = Result = Time = 0.f;
		Speed = 0.1f;
		Frequency = Maximum = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to retrieve the noise for
	UPROPERTY(meta = (Input))
	float Value;

	// The speed of the noise field
	UPROPERTY(meta = (Input))
	float Speed;

	// The frequency of the noise field
	UPROPERTY(meta = (Input))
	float Frequency;

	// The minimum range of the noise
	UPROPERTY(meta = (Input))
	float Minimum;

	// The maximum range of the noise
	UPROPERTY(meta = (Input))
	float Maximum;

	// The resulting value in the noise field
	UPROPERTY(meta = (Output))
	float Result;

	UPROPERTY()
	float Time;
};

/**
 * Generates a double through a noise fluctuation process between a min and a max through speed
 */
USTRUCT(meta=(DisplayName="Noise (Double)", Category="Math|Noise", TemplateName="Noise"))
struct FRigVMFunction_NoiseDouble : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_NoiseDouble()
	{
		Value = Minimum = Result = Time = 0.0;
		Speed = 0.1;
		Frequency = Maximum = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input value to retrieve the noise for
	UPROPERTY(meta = (Input))
	double Value;

	// The speed of the noise field
	UPROPERTY(meta = (Input))
	double Speed;

	// The frequency of the noise field
	UPROPERTY(meta = (Input))
	double Frequency;

	// The minimum range of the noise
	UPROPERTY(meta = (Input))
	double Minimum;

	// The maximum range of the noise
	UPROPERTY(meta = (Input))
	double Maximum;

	// The resulting value in the noise field
	UPROPERTY(meta = (Output))
	double Result;

	UPROPERTY()
	double Time;
};

/**
 * Generates a vector through a noise fluctuation process between a min and a max through speed
 */
USTRUCT(meta = (DisplayName = "Noise (Vector)", Category = "Math|Noise", Deprecated = "5.0.0"))
struct FRigVMFunction_NoiseVector : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_NoiseVector()
	{
		Position = Result = Time = FVector::ZeroVector;
		Frequency = FVector::OneVector;
		Speed = FVector(0.1f, 0.1f, 0.1f);
		Minimum = 0.f;
		Maximum = 1.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	FVector Position;

	UPROPERTY(meta = (Input))
	FVector Speed;

	UPROPERTY(meta = (Input))
	FVector Frequency;

	UPROPERTY(meta = (Input))
	float Minimum;

	UPROPERTY(meta = (Input))
	float Maximum;

	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector Time;

	RIGVM_METHOD()
	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

/**
 * Generates a vector through a noise fluctuation process between a min and a max through speed
 */
USTRUCT(meta = (DisplayName = "Noise (Vector)", Category = "Math|Noise", TemplateName="Noise"))
struct FRigVMFunction_NoiseVector2 : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_NoiseVector2()
	{
		Value = Result = Time = FVector::ZeroVector;
		Frequency = FVector::OneVector;
		Speed = FVector(0.1f, 0.1f, 0.1f);
		Minimum = 0.0;
		Maximum = 1.0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input position to retrieve the noise for
	UPROPERTY(meta = (Input))
	FVector Value;

	// The speed of the noise field
	UPROPERTY(meta = (Input))
	FVector Speed;

	// The frequency of the noise field
	UPROPERTY(meta = (Input))
	FVector Frequency;

	// The minimum range of the noise
	UPROPERTY(meta = (Input))
	double Minimum;

	// The maximum range of the noise
	UPROPERTY(meta = (Input))
	double Maximum;

	// The resulting vector in the noise field
	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector Time;
};