// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_Random.generated.h"

/**
 * Generates a random integer between a min and a max (inclusive)
 */
USTRUCT(meta = (DisplayName = "Random (Int)", Category = "Math|Random", TemplateName = "Random"))
struct FRigVMFunction_RandomInt : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_RandomInt()
	{
		Seed = BaseSeed = LastSeed = 217;
		Minimum = Result = LastResult = 0;
		Maximum = 1;
		Duration = 0.f;
		TimeLeft = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The seed for the random number generator
	UPROPERTY(meta = (Input))
	int32 Seed;

	// The minimum value for the random number range
	UPROPERTY(meta = (Input))
	int32 Minimum = 0;

	// The maximum value for the random number range (inclusive)
	UPROPERTY(meta = (Input))
	int32 Maximum = 1;

	/**
	 * The duration at which the number won't change.
	 * Use 0 for a different number every time.
	 * A negative number (for ex: -1) results in an infinite duration.
	 */
	UPROPERTY(meta = (Input))
	float Duration;

	// The resulting random number
	UPROPERTY(meta = (Output))
	int32 Result;

	UPROPERTY()
	int32 LastResult;

	UPROPERTY()
	int32 LastSeed;

	UPROPERTY()
	int32 BaseSeed;

	UPROPERTY()
	float TimeLeft;
};

/**
 * Generates a random float between a min and a max
 */
USTRUCT(meta=(DisplayName="Random (Float)", Category="Math|Random", TemplateName="Random"))
struct FRigVMFunction_RandomFloat : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_RandomFloat()
	{
		Seed = BaseSeed = LastSeed = 217;
		Minimum = Result = LastResult = 0.f;
		Maximum = 1.f;
		Duration = 0.f;
		TimeLeft = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The seed for the random number generator
	UPROPERTY(meta = (Input))
	int32 Seed;

	// The minimum value for the random number range
	UPROPERTY(meta = (Input))
	float Minimum;

	// The maximum value for the random number range
	UPROPERTY(meta = (Input))
	float Maximum;

	/**
	 * The duration at which the number won't change.
	 * Use 0 for a different number every time.
	 * A negative number (for ex: -1) results in an infinite duration.
	 */
	UPROPERTY(meta = (Input))
	float Duration;

	// The resulting random number
	UPROPERTY(meta = (Output))
	float Result;

	UPROPERTY()
	float LastResult;

	UPROPERTY()
	int32 LastSeed;

	UPROPERTY()
	int32 BaseSeed;

	UPROPERTY()
	float TimeLeft;
};

/**
 * Generates a random vector between a min and a max
 */
USTRUCT(meta = (DisplayName = "Random (Vector)", Category = "Math|Random", TemplateName="Random"))
struct FRigVMFunction_RandomVector: public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	FRigVMFunction_RandomVector()
	{
		Seed = BaseSeed = LastSeed = 217;
		Minimum = 0.f;
		Maximum = 1.f;
		Duration = 0.f;
		TimeLeft = 0.f;
		Result = LastResult = FVector::ZeroVector;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The seed for the random number generator
	UPROPERTY(meta = (Input))
	int32 Seed;

	// The minimum value for the random number range for all components of the vector
	UPROPERTY(meta = (Input))
	float Minimum;

	// The maximum value for the random number range for all components of the vector
	UPROPERTY(meta = (Input))
	float Maximum;
	
	/**
	 * The duration at which the number won't change.
	 * Use 0 for a different number every time.
	 * A negative number (for ex: -1) results in an infinite duration.
	 */
	UPROPERTY(meta = (Input))
	float Duration;

	// The resulting random vector
	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector LastResult;

	UPROPERTY()
	int32 LastSeed;

	UPROPERTY()
	int32 BaseSeed;

	UPROPERTY()
	float TimeLeft;
};