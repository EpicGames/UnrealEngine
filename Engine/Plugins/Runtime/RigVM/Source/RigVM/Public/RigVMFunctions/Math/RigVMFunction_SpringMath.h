// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunction_MathBase.h"
#include "RigVMFunction_SpringMath.generated.h"

/**
 * Damps a float value using exponential decay damping
 */
USTRUCT(meta=(DisplayName="Damp (Float)", Category="Math|Damp", TemplateName="Damp"))
struct FRigVMFunction_DampFloat : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input))
	float Value = 0.0f;

	// The target value to damp towards
	UPROPERTY(meta = (Input))
	float Target = 0.0f;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;

	// Resulting damped value
	UPROPERTY(meta = (Output))
	float Result = 0.0f;
};

/**
 * Damps a vector value using exponential decay damping
 */
USTRUCT(meta=(DisplayName="Damp (Vector)", Category="Math|Damp", TemplateName="Damp"))
struct FRigVMFunction_DampVector : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input))
	FVector Value = FVector::ZeroVector;

	// The target value to damp towards
	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;

	// Resulting damped value
	UPROPERTY(meta = (Output))
	FVector Result = FVector::ZeroVector;
};

/**
 * Damps a quaternion value using exponential decay damping
 */
USTRUCT(meta=(DisplayName="Damp (Quaternion)", Category="Math|Damp", TemplateName="Damp"))
struct FRigVMFunction_DampQuaternion : public FRigVMFunction_MathBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input))
	FQuat Value = FQuat::Identity;

	// The target value to damp towards
	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;

	// Resulting damped value
	UPROPERTY(meta = (Output))
	FQuat Result = FQuat::Identity;
};

/**
 * Damps a float using a spring damper
 */
USTRUCT(meta=(DisplayName="Critical Spring Damp (Float)", Category="Math|Damp", TemplateName="Critical Spring Damp"))
struct FRigVMFunction_SpringDampFloat : public FRigVMFunction_MathMutableBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input, Output))
	float Value = 0.0f;

	// The velocity of the value
	UPROPERTY(meta = (Input, Output))
	float ValueVelocity = 0.0f;

	// The target to damp towards
	UPROPERTY(meta = (Input))
	float Target = 0.0f;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;
};

/**
 * Damps a vector using a spring damper
 */
USTRUCT(meta=(DisplayName="Critical Spring Damp (Vector)", Category="Math|Damp", TemplateName="Critical Spring Damp"))
struct FRigVMFunction_SpringDampVector : public FRigVMFunction_MathMutableBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input, Output))
	FVector Value = FVector::ZeroVector;

	// The velocity of the value
	UPROPERTY(meta = (Input, Output))
	FVector ValueVelocity = FVector::ZeroVector;

	// The target to damp towards
	UPROPERTY(meta = (Input))
	FVector Target = FVector::ZeroVector;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;
};

/**
 * Damps a quaternion using a spring damper
 */
USTRUCT(meta=(DisplayName="Critical Spring Damp (Quat)", Category="Math|Damp", TemplateName="Critical Spring Damp"))
struct FRigVMFunction_SpringDampQuat : public FRigVMFunction_MathMutableBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to damp
	UPROPERTY(meta = (Input, Output))
	FQuat Value = FQuat::Identity;

	// The velocity of the value
	UPROPERTY(meta = (Input, Output))
	FVector ValueVelocity = FVector::ZeroVector;

	// The target to damp towards
	UPROPERTY(meta = (Input))
	FQuat Target = FQuat::Identity;

	// The time to apply smoothing for
	UPROPERTY(meta = (Input, ClampMin = "0", UIMin = "0", ForceUnits = "s"))
	float SmoothingTime = 0.2f;
};
