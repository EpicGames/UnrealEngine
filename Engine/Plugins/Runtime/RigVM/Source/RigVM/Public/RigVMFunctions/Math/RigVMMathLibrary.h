// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "EulerTransform.h"
#include "RigVMFunctions/RigVMFunctionDefines.h"
#include "RigVMMathLibrary.generated.h"

UENUM(meta = (RigVMTypeAllowed))
enum class ERigVMAnimEasingType : uint8
{
	Linear,
	QuadraticEaseIn,
	QuadraticEaseOut,
	QuadraticEaseInOut,
	CubicEaseIn,
	CubicEaseOut,
	CubicEaseInOut,
	QuarticEaseIn,
	QuarticEaseOut,
	QuarticEaseInOut,
	QuinticEaseIn,
	QuinticEaseOut,
	QuinticEaseInOut,
	SineEaseIn,
	SineEaseOut,
	SineEaseInOut,
	CircularEaseIn,
	CircularEaseOut,
	CircularEaseInOut,
	ExponentialEaseIn,
	ExponentialEaseOut,
	ExponentialEaseInOut,
	ElasticEaseIn,
	ElasticEaseOut,
	ElasticEaseInOut,
	BackEaseIn,
	BackEaseOut,
	BackEaseInOut,
	BounceEaseIn,
	BounceEaseOut,
	BounceEaseInOut
};

USTRUCT(BlueprintType)
struct FRigVMFourPointBezier
{
	GENERATED_BODY()

	FRigVMFourPointBezier()
	{
		A = B = C = D = FVector::ZeroVector;
	}

	UPROPERTY(EditAnywhere, Category=Bezier)
	FVector A;

	UPROPERTY(EditAnywhere, Category=Bezier)
	FVector B;

	UPROPERTY(EditAnywhere, Category=Bezier)
	FVector C;

	UPROPERTY(EditAnywhere, Category=Bezier)
	FVector D;
};

USTRUCT(BlueprintType)
struct FRigVMMirrorSettings
{
	GENERATED_USTRUCT_BODY()

	FRigVMMirrorSettings()
	: MirrorAxis(EAxis::X)
	, AxisToFlip(EAxis::Z)
	{
	}

	// the axis to mirror against
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TEnumAsByte<EAxis::Type> MirrorAxis;

	// the axis to flip for rotations
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings)
	TEnumAsByte<EAxis::Type> AxisToFlip;

	// the string to search for
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, DisplayName = "Search")
	FString SearchString;

	// the string to replace the search occurrences with
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Settings, DisplayName = "Replace")
	FString ReplaceString;

	RIGVM_API FTransform MirrorTransform(const FTransform& InTransform) const;
	RIGVM_API FVector MirrorVector(const FVector& InVector) const;
};

UENUM(meta = (RigVMTypeAllowed))
enum class ERigVMSimPointIntegrateType : uint8
{
	Verlet,
	SemiExplicitEuler
};

USTRUCT(BlueprintType)
struct FRigVMSimPoint
{
	GENERATED_BODY()

	FRigVMSimPoint()
	{
		Mass = 1.f;
		Size = 0.f;
		LinearDamping = 0.01f;
		InheritMotion = 0.f;
		Position = LinearVelocity = FVector::ZeroVector;
	}

	/**
	 * The mass of the point
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Mass;

	/**
	 * Size of the point - only used for collision
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float Size;

	/**
	 * The linear damping of the point
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float LinearDamping;

	/**
	 * Defines how much the point will inherit motion from its input.
	 * This does not have an effect on passive (mass == 0.0) points.
	 * Values can be higher than 1 due to timestep - but they are clamped internally.
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	float InheritMotion;

	/**
	 * The position of the point
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	FVector Position;

	/**
	 * The velocity of the point per second
	 */
	UPROPERTY(EditAnywhere, Category=Simulation)
	FVector LinearVelocity;

	RIGVM_API FRigVMSimPoint IntegrateVerlet(const FVector& InForce, float InBlend, float InDeltaTime) const;
	RIGVM_API FRigVMSimPoint IntegrateSemiExplicitEuler(const FVector& InForce, float InDeltaTime) const;
};

class FRigVMMathLibrary
{
public:
	static RIGVM_API float AngleBetween(const FVector& A, const FVector& B);
	static RIGVM_API void FourPointBezier(const FVector& A, const FVector& B, const FVector& C, const FVector& D, float T, FVector& OutPosition, FVector& OutTangent);
	static RIGVM_API void FourPointBezier(const FRigVMFourPointBezier& Bezier, float T, FVector& OutPosition, FVector& OutTangent);
	static RIGVM_API float EaseFloat(float Value, ERigVMAnimEasingType Type);
	static RIGVM_API FTransform LerpTransform(const FTransform& A, const FTransform& B, float T);
	static RIGVM_API FVector ClampSpatially(const FVector& Value, EAxis::Type Axis, ERigVMClampSpatialMode::Type Type, float Minimum, float Maximum, FTransform Space);
	static RIGVM_API FQuat FindQuatBetweenVectors(const FVector& A, const FVector& B);
	static RIGVM_API FQuat FindQuatBetweenNormals(const FVector& A, const FVector& B);

	// See - "Computing Euler angles from a rotation matrix" by Gregory G. Slabaugh
	// Each spatial orientation can be mapped to two equivalent euler angles within range (-180, 180)
	static RIGVM_API FVector GetEquivalentEulerAngle(const FVector& InEulerAngle, const EEulerRotationOrder& InOrder);
	
	static RIGVM_API FVector& ChooseBetterEulerAngleForAxisFilter(const FVector& Base, FVector& A, FVector& B);
};