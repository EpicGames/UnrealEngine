// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.h"
#include "RigVMCore/RigVMByteCode.h"
#include "RigVMDispatch_Array.generated.h"

USTRUCT(meta=(Abstract, Category = "Array", Keywords = "List,Collection", NodeColor = "1,1,1,1"))
struct FRigVMDispatch_ArrayBase : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:
	virtual ERigVMOpCode GetOpCode() const { return ERigVMOpCode::Invalid; }
	static RIGVM_API UScriptStruct* GetFactoryDispatchForOpCode(ERigVMOpCode InOpCode);
	static RIGVM_API FName GetFactoryNameForOpCode(ERigVMOpCode InOpCode);
#if WITH_EDITOR
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual bool IsSingleton() const override { return true; } 

protected:
	static RIGVM_API FRigVMTemplateArgumentInfo CreateArgumentInfo(const FName& InName, ERigVMPinDirection InDirection);
	static RIGVM_API TMap<uint32, int32> GetArrayHash(FScriptArrayHelper& InArrayHelper, const FArrayProperty* InArrayProperty);

	static inline FLazyName ArrayName = FLazyName(TEXT("Array"));
	static inline FLazyName ValuesName = FLazyName(TEXT("Values"));
	static inline FLazyName NumName = FLazyName(TEXT("Num"));
	static inline FLazyName IsEmptyName = FLazyName(TEXT("IsEmpty"));
	static inline FLazyName IndexName = FLazyName(TEXT("Index"));
	static inline FLazyName ElementName = FLazyName(TEXT("Element"));
	static inline FLazyName SuccessName = FLazyName(TEXT("Success"));
	static inline FLazyName OtherName = FLazyName(TEXT("Other"));
	static inline FLazyName CloneName = FLazyName(TEXT("Clone"));
	static inline FLazyName CountName = FLazyName(TEXT("Count"));
	static inline FLazyName RatioName = FLazyName(TEXT("Ratio"));
	static inline FLazyName ResultName = FLazyName(TEXT("Result"));

	friend class URigVMController;
};

USTRUCT(meta=(Abstract))
struct FRigVMDispatch_ArrayBaseMutable : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	RIGVM_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
};

USTRUCT(meta=(DisplayName = "Make Array", Keywords = "Make,MakeArray,Constant,Reroute"))
struct FRigVMDispatch_ArrayMake : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayMake()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayMake::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Reset", Pure))
struct FRigVMDispatch_ArrayReset : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayReset()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayReset; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayReset::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Num"))
struct FRigVMDispatch_ArrayGetNum : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayGetNum()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayGetNum; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayGetNum::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Set Num", Pure))
struct FRigVMDispatch_ArraySetNum : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArraySetNum()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArraySetNum; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArraySetNum::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "IsEmpty"))
struct FRigVMDispatch_ArrayIsEmpty : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayIsEmpty()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayIsEmpty::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Init", Pure))
struct FRigVMDispatch_ArrayInit : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayInit()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayInit::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "At", Keywords = "Get Index,At Index,[]"))
struct FRigVMDispatch_ArrayGetAtIndex : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayGetAtIndex()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayGetAtIndex; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayGetAtIndex::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Set At", Pure))
struct FRigVMDispatch_ArraySetAtIndex : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArraySetAtIndex()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArraySetAtIndex; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArraySetAtIndex::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Add", Pure))
struct FRigVMDispatch_ArrayAdd : public FRigVMDispatch_ArraySetAtIndex
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayAdd()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayAdd; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayAdd::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Insert", Pure))
struct FRigVMDispatch_ArrayInsert : public FRigVMDispatch_ArraySetAtIndex
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayInsert()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayInsert; }
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayInsert::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Remove", Pure))
struct FRigVMDispatch_ArrayRemove : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayRemove()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayRemove; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayRemove::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Reverse", Pure))
struct FRigVMDispatch_ArrayReverse : public FRigVMDispatch_ArrayReset
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayReverse()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayReverse; }
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayReverse::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Find"))
struct FRigVMDispatch_ArrayFind : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayFind()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayFind; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayFind::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Append", Pure))
struct FRigVMDispatch_ArrayAppend : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayAppend()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayAppend; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayAppend::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Clone"))
struct FRigVMDispatch_ArrayClone : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayClone()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayClone; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayClone::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Union", Pure))
struct FRigVMDispatch_ArrayUnion : public FRigVMDispatch_ArrayAppend
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayUnion()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayUnion; }
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayUnion::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Difference"))
struct FRigVMDispatch_ArrayDifference : public FRigVMDispatch_ArrayBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayDifference()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayDifference; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayDifference::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "Intersection"))
struct FRigVMDispatch_ArrayIntersection : public FRigVMDispatch_ArrayDifference
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayIntersection()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayIntersection; }
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayIntersection::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

USTRUCT(meta=(DisplayName = "For Each", Icon="EditorStyle|GraphEditor.Macro.ForEach_16x", Pure))
struct FRigVMDispatch_ArrayIterator : public FRigVMDispatch_ArrayBaseMutable
{
	GENERATED_BODY()

public:
	FRigVMDispatch_ArrayIterator()
	{
		FactoryScriptStruct = StaticStruct();
	}

	virtual ERigVMOpCode GetOpCode() const override { return ERigVMOpCode::ArrayIterator; }
	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
#if WITH_EDITORONLY_DATA
	RIGVM_API virtual const TArray<FName>& GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const override;
#endif
	RIGVM_API virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
   	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
   	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_ArrayIterator::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};
