// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_CastEnum.generated.h"

struct FRigVMDispatch_CastEnumBase
{
	static inline const FLazyName ValueName = FLazyName(TEXT("Value"));
	static inline const FLazyName ResultName = FLazyName(TEXT("Result"));
};

USTRUCT(meta=(DisplayName = "Cast", Category = "Enum", Keywords = "As", NodeColor = "1,1,1,1"))
struct FRigVMDispatch_CastEnumToInt : public FRigVMDispatchFactory
#if CPP
	, public FRigVMDispatch_CastEnumBase
#endif
{
	GENERATED_BODY()

public:
	FRigVMDispatch_CastEnumToInt()
	{
		FactoryScriptStruct = StaticStruct();
	}
	
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual bool GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual bool IsSingleton() const override { return true; }
	
protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_CastEnumToInt::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};

USTRUCT(meta=(DisplayName = "Cast", Category = "Enum", Keywords = "As", NodeColor = "1,1,1,1"))
struct FRigVMDispatch_CastIntToEnum : public FRigVMDispatchFactory
#if CPP
	, public FRigVMDispatch_CastEnumBase
#endif
{
	GENERATED_BODY()

public:
	FRigVMDispatch_CastIntToEnum()
	{
		FactoryScriptStruct = StaticStruct();
	}
	
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual bool GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual bool IsSingleton() const override { return true; }
	
protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_CastIntToEnum::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);
};
