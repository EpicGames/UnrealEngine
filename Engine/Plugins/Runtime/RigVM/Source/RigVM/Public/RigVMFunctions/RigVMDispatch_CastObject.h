// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMDispatch_CastObject.generated.h"

USTRUCT(meta=(DisplayName = "Cast", Category = "Object", Keywords = "As", NodeColor = "1,1,1,1"))
struct FRigVMDispatch_CastObject : public FRigVMDispatchFactory
{
	GENERATED_BODY()

public:
	FRigVMDispatch_CastObject()
	{
		FactoryScriptStruct = StaticStruct();
	}
	
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual bool GetPermutationsFromArgumentType(const FName& InArgumentName, const TRigVMTypeIndex& InTypeIndex, TArray<FRigVMTemplateTypeMap, TInlineAllocator<1>>& OutPermutations, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual bool IsSingleton() const override { return true; }
	
protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_CastObject::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);

	static inline const FLazyName ValueName = FLazyName(TEXT("Value"));
	static inline const FLazyName ResultName = FLazyName(TEXT("Result"));
};
