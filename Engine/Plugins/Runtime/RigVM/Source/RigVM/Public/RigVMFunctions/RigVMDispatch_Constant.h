// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.h"
#include "RigVMCore/RigVMByteCode.h"
#include "RigVMDispatch_Constant.generated.h"

USTRUCT(meta=(DisplayName = "Constant", Category = "Core", Keywords = "Value,Reroute", NodeColor = "1,1,1,1"))
struct FRigVMDispatch_Constant : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:
	FRigVMDispatch_Constant()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FString GetNodeTitle(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetNodeTooltip(const FRigVMTemplateTypeMap& InTypes) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
#endif
	virtual bool IsSingleton() const override { return true; }
	virtual bool SupportsRenaming() const override { return true; }

protected:
	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override { return &FRigVMDispatch_Constant::Execute; }
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray RigVMBranches);

	static inline const FLazyName ValueName = FLazyName(TEXT("Value"));

	friend class URigVMController;
	friend class FRigVMParserAST;
};
