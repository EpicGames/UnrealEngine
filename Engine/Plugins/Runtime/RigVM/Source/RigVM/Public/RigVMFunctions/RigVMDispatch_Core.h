// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.generated.h"

USTRUCT(meta=(Abstract, Category = "Core", NodeColor = "0.762745, 1,0, 0.329412"))
struct FRigVMDispatch_CoreBase : public FRigVMDispatchFactory
{
	GENERATED_BODY()
};

/*
 * Compares any two values and returns true if they are identical
 */
USTRUCT(meta=(DisplayName = "Equals", Keywords = "Same,=="))
struct FRigVMDispatch_CoreEquals : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:

	FRigVMDispatch_CoreEquals()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	RIGVM_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static RIGVM_API bool AdaptResult(bool bResult, const FRigVMExtendedExecuteContext& InContext);
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);

	template<typename T>
	static void Equals(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		check(Handles[0].IsType<T>());
		check(Handles[1].IsType<T>());
		check(Handles[2].IsBool());
		const T& A = *(const T*)Handles[0].GetInputData();
		const T& B = *(const T*)Handles[1].GetInputData();
		bool& Result = *(bool*)Handles[2].GetOutputData();
		Result = AdaptResult(A == B, InContext);
	}

	static void NameEquals(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		check(Handles[0].IsType<FName>());
		check(Handles[1].IsType<FName>());
		check(Handles[2].IsBool());
		const FName& A = *(const FName*)Handles[0].GetInputData();
		const FName& B = *(const FName*)Handles[1].GetInputData();
		bool& Result = *(bool*)Handles[2].GetOutputData();
		Result = AdaptResult(A.IsEqual(B, ENameCase::CaseSensitive), InContext);
	}

	static void StringEquals(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		check(Handles[0].IsType<FString>());
		check(Handles[1].IsType<FString>());
		check(Handles[2].IsBool());
		const FString& A = *(const FString*)Handles[0].GetInputData();
		const FString& B = *(const FString*)Handles[1].GetInputData();
		bool& Result = *(bool*)Handles[2].GetOutputData();
		Result = AdaptResult(A.Equals(B, ESearchCase::CaseSensitive), InContext);
	}

	template<typename T>
	static void MathTypeEquals(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates)
	{
		check(Handles[0].IsType<T>());
		check(Handles[1].IsType<T>());
		check(Handles[2].IsBool());
		const T& A = *(const T*)Handles[0].GetInputData();
		const T& B = *(const T*)Handles[1].GetInputData();
		bool& Result = *(bool*)Handles[2].GetOutputData();
		Result = AdaptResult(A.Equals(B), InContext);
	}

	static inline FLazyName AName = FLazyName(TEXT("A"));
	static inline FLazyName BName = FLazyName(TEXT("B"));
	static inline FLazyName ResultName = FLazyName(TEXT("Result"));
};

/*
 * Compares any two values and returns true if they are not identical
 */
USTRUCT(meta=(DisplayName = "Not Equals", Keywords = "Different,!=,Xor"))
struct FRigVMDispatch_CoreNotEquals : public FRigVMDispatch_CoreEquals
{
	GENERATED_BODY()

	FRigVMDispatch_CoreNotEquals()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;

	// we are inheriting everything from the equals dispatch,
	// and due to the check of the factory within FRigVMDispatch_CoreEquals::Execute we can
	// rely on that completely. we only need this class for the displayname and
	// operation specific StaticStruct().
};
