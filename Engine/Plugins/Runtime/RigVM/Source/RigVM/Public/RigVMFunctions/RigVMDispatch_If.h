// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.h"
#include "RigVMDispatch_If.generated.h"

/*
 * Chooses between two values based on a condition
 */
USTRUCT(meta=(DisplayName = "If", Category = "Execution", Keywords = "Branch,Condition", NodeColor = "0,1,0,1", Pure))
struct FRigVMDispatch_If : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:

	FRigVMDispatch_If()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; } 

#if WITH_EDITOR
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	RIGVM_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);

	static inline const FLazyName ConditionName = FLazyName(TEXT("Condition"));
	static inline const FLazyName TrueName = FLazyName(TEXT("True"));
	static inline const FLazyName FalseName = FLazyName(TEXT("False"));
	static inline const FLazyName ResultName = FLazyName(TEXT("Result"));
};
