// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Print.generated.h"

/*
 * Prints any value to the log
 */
USTRUCT(meta=(DisplayName = "Print", NodeColor = "0.8, 0, 0.2, 1", Pure))
struct FRigVMDispatch_Print : public FRigVMDispatchFactory
{
	GENERATED_BODY()

public:

	FRigVMDispatch_Print()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
	virtual bool IsSingleton() const override { return true; } 

#if WITH_EDITOR
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override
	{
		return &FRigVMDispatch_Print::Execute;
	}
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);

	static inline const FLazyName PrefixName = FLazyName(TEXT("Prefix"));
	static inline const FLazyName ValueName = FLazyName(TEXT("Value"));
	static inline const FLazyName EnabledName = FLazyName(TEXT("Enabled"));
	static inline const FLazyName ScreenDurationName = FLazyName(TEXT("ScreenDuration"));
	static inline const FLazyName ScreenColorName = FLazyName(TEXT("ScreenColor"));
};
