// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.h"
#include "RigVMDispatch_Select.generated.h"

/*
 * Pick from a list of values based on an integer index
 */
USTRUCT(meta=(DisplayName = "Select", Category = "Execution", Keywords = "Switch,Case", NodeColor = "0,1,0,1", Pure))
struct FRigVMDispatch_SelectInt32 : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:

	FRigVMDispatch_SelectInt32()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;
#if WITH_EDITOR
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	virtual bool IsSingleton() const override { return true; } 

protected:

	RIGVM_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);

	static inline FLazyName IndexName = FLazyName(TEXT("Index"));
	static inline FLazyName ValuesName = FLazyName(TEXT("Values"));
	static inline FLazyName ResultName = FLazyName(TEXT("Result"));
};
