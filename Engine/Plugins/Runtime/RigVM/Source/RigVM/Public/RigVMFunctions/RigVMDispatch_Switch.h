// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMDispatch_Core.h"
#include "RigVMDispatch_Switch.generated.h"

USTRUCT(meta=(Abstract, Category = "Switch", NodeColor = "0.762745, 1,0, 0.329412"))
struct FRigVMDispatch_SwitchBase : public FRigVMDispatch_CoreBase
{
	GENERATED_BODY()

public:

	RIGVM_API virtual FName GetArgumentNameForOperandIndex(int32 InOperandIndex, int32 InTotalOperands, FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual const TArray<FRigVMExecuteArgument>& GetExecuteArguments_Impl(const FRigVMDispatchContext& InContext) const override;
	virtual bool IsSingleton() const override { return true; } 

#if WITH_EDITOR
	RIGVM_API virtual FString GetArgumentMetaData(const FName& InArgumentName, const FName& InMetaDataKey) const override;
	RIGVM_API virtual FString GetArgumentDefaultValue(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
	RIGVM_API virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif
	
protected:

	static RIGVM_API FName GetCaseName(int32 InIndex);

	static inline const FLazyName IndexName = FLazyName(TEXT("Index"));
	static inline const FLazyName CasesName = FLazyName(TEXT("Cases"));
};

/*
 * Run a branch based on an integer index
 */
USTRUCT(meta=(DisplayName = "Switch", Category = "Execution", Keywords = "Case", NodeColor = "0,1,0,1", Pure))
struct FRigVMDispatch_SwitchInt32 : public FRigVMDispatch_SwitchBase
{
	GENERATED_BODY()

public:

	FRigVMDispatch_SwitchInt32()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	RIGVM_API virtual FName GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const override;
#endif
	
#if WITH_EDITORONLY_DATA
	RIGVM_API virtual const TArray<FName>& GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const override;
#endif

protected:

	RIGVM_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
	static RIGVM_API FName GetCaseDisplayName(int32 InIndex);
};

/*
 * Run a branch based on an enum index
 */
USTRUCT(meta=(DisplayName = "Switch Enum", Category = "Execution", Keywords = "Case", NodeColor = "0,1,0,1", Pure))
struct FRigVMDispatch_SwitchEnum : public FRigVMDispatch_SwitchBase
{
	GENERATED_BODY()

public:

	FRigVMDispatch_SwitchEnum()
	{
		FactoryScriptStruct = StaticStruct();
	}

	RIGVM_API virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	RIGVM_API virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	RIGVM_API virtual FName GetDisplayNameForArgument(const FName& InArgumentName, const FRigVMTemplateTypeMap& InTypes) const override;
#endif
	
#if WITH_EDITORONLY_DATA
	RIGVM_API virtual const TArray<FName>& GetControlFlowBlocks_Impl(const FRigVMDispatchContext& InContext, const FRigVMTemplateTypeMap& InTypeMap) const override;
#endif

protected:

	RIGVM_API virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static RIGVM_API void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
	
#if WITH_EDITORONLY_DATA
	static RIGVM_API FName GetCaseDisplayName(int32 InIndex, const UEnum* InEnum = nullptr);
#endif
	
	friend class URigVMController;
};
