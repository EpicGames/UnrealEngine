// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "RigVMFunctionDefines.generated.h"

UENUM(meta = (RigVMTypeAllowed))
enum class ERigVMTransformSpace : uint8
{
	/** Apply in parent space */
	LocalSpace,

	/** Apply in rig space*/
	GlobalSpace,

	/** MAX - invalid */
	Max UMETA(Hidden),
};

UENUM(meta = (RigVMTypeAllowed))
namespace ERigVMClampSpatialMode
{
	enum Type : int
	{
		Plane,
		Cylinder,
		Sphere,
		Capsule
	};
}

#if WITH_EDITOR
#define UE_RIGVMSTRUCT_REPORT(Severity, Format, ...) \
if(ExecuteContext.GetLog() != nullptr) \
{ \
ExecuteContext.Report(EMessageSeverity::Severity, ExecuteContext.GetFunctionName(), ExecuteContext.GetInstructionIndex(), FString::Printf((Format), ##__VA_ARGS__)); \
}
#define UE_RIGVMSTRUCT_LOG_MESSAGE(Format, ...) UE_RIGVMSTRUCT_REPORT(Info, (Format), ##__VA_ARGS__)
#define UE_RIGVMSTRUCT_REPORT_WARNING(Format, ...) UE_RIGVMSTRUCT_REPORT(Warning, (Format), ##__VA_ARGS__)
#define UE_RIGVMSTRUCT_REPORT_ERROR(Format, ...) UE_RIGVMSTRUCT_REPORT(Error, (Format), ##__VA_ARGS__)
#else
#define UE_RIGVMSTRUCT_REPORT(Severity, Format, ...)
#define UE_RIGVMSTRUCT_LOG_MESSAGE(Format, ...)
#define UE_RIGVMSTRUCT_REPORT_WARNING(Format, ...)
#define UE_RIGVMSTRUCT_REPORT_ERROR(Format, ...)
#endif
