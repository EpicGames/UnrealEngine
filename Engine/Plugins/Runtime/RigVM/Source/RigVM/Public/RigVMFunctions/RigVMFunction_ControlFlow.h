// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_ControlFlow.generated.h"

/*
 * The base class for all control flow nodes
 */
USTRUCT(meta=(Abstract, Category="Execution", NodeColor = "0, 0, 0, 1", DocumentationPolicy = "Strict"))
struct FRigVMFunction_ControlFlowBase : public FRigVMStruct
{
	GENERATED_BODY()
};

/**
 * Executes either the True or False branch based on the condition
 */
USTRUCT(meta = (DisplayName = "Branch", Keywords = "If", Pure))
struct FRigVMFunction_ControlFlowBranch : public FRigVMFunction_ControlFlowBase
{
	GENERATED_BODY()

	FRigVMFunction_ControlFlowBranch()
	{
		Condition = false;
		BlockToRun = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, True),
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, False),
			ForLoopCompletedPinName
		};
		return Blocks;
	}
#endif

	// The input execution pin to hook up to the graph
	UPROPERTY(meta=(Input, DisplayName="Execute"))
	FRigVMExecuteContext ExecuteContext;

	// The condition based on which to pick between the True and False branches
	UPROPERTY(meta=(Input))
	bool Condition;

	// The branch to run if the condition is True
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext True;

	// The branch to run if the condition is False
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext False;

	// The execute flow to run when the True or False branch is complete
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext Completed;

	// The internal block to run as the node progresses
	UPROPERTY(meta=(Singleton))
	FName BlockToRun;
};

/**
 * Executes the Once branch only once
 */
USTRUCT(meta = (DisplayName = "Run Once", Keywords = "Branch,if"))
struct FRigVMFunction_ControlFlowRunOnce : public FRigVMFunction_ControlFlowBase
{
	GENERATED_BODY()

	FRigVMFunction_ControlFlowRunOnce()
	{
		HasRun = false;
		BlockToRun = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowRunOnce, Once),
			ForLoopCompletedPinName
		};
		return Blocks;
	}
#endif

	// The input execution pin to hook up to the graph
	UPROPERTY(meta=(Input, DisplayName="Execute"))
	FRigVMExecuteContext ExecuteContext;

	// The branch to run once
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext Once;

	// The execute flow to run every frame
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext Completed;

	// The internal block to run as the node progresses
	UPROPERTY(meta=(Singleton))
	FName BlockToRun;

	// The internal condition indicating if this has run once or now
	UPROPERTY()
	bool HasRun;
};

/**
 * Executes the output branches based on a timer
 */
USTRUCT(meta = (DisplayName = "Timer", Keywords = "Branch,If,Timeline,Execute"))
struct FRigVMFunction_ControlFlowTimer : public FRigVMFunction_ControlFlowBase
{
	GENERATED_BODY()

	FRigVMFunction_ControlFlowTimer()
	{
		BlockToRun = NAME_None;
		Reset = Repeats = bJustStarted = bIsRunning = bJustFinished = bIsInitialized = false;
		Speed = Duration = 1.f;
		Time = Ratio = AccumulatedValue = 0.f;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

#if WITH_EDITORONLY_DATA
	virtual const TArray<FName>& GetControlFlowBlocks_Impl() const override
	{
		static const TArray<FName> Blocks = {
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, JustStarted),
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, IsRunning),
			GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowTimer, JustFinished),
			ForLoopCompletedPinName
		};
		return Blocks;
	}
#endif

	// The input execution pin to hook up to the graph
	UPROPERTY(meta=(Input, DisplayName="Execute"))
	FRigVMExecuteContext ExecuteContext;

	// A flag to reset / restart the timer
	UPROPERTY(meta = (Input))
	bool Reset;

	// The speed to apply to the played back time
	UPROPERTY(meta = (Input))
	float Speed;

	// The duration in seconds the timer will take to complete
	UPROPERTY(meta = (Input))
	float Duration;

	// A flag indicating if the timer is a one-shot or if it should repeat / loop
	UPROPERTY(meta = (Input))
	bool Repeats;

	// The branch to run when the timer has just started
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext JustStarted;

	// The branch to run when the timer is running
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext IsRunning;

	// The branch to run when the timer has just finished
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext JustFinished;

	// The played back / accumulated time in seconds
	UPROPERTY(meta=(Output))
	float Time;

	// The played back / accumulated time as a ratio of time / duration (0.0 - 1.0)
	UPROPERTY(meta=(Output))
	float Ratio;

	// The execute flow to run every frame after all other branches
	UPROPERTY(meta=(Output))
	FRigVMExecuteContext Completed;

	// The internal block to run as the node progresses
	UPROPERTY(meta=(Singleton))
	FName BlockToRun;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;

	// True if the timer has just started
	UPROPERTY(meta=(Singleton))
	bool bJustStarted;

	// True if the timer is running
	UPROPERTY(meta=(Singleton))
	bool bIsRunning;

	// True if the timer has just finished
	UPROPERTY(meta=(Singleton))
	bool bJustFinished;
};