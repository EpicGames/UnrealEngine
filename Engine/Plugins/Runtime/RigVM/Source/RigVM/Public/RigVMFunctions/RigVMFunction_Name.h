// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_Name.generated.h"

/*
 * The base class for all name operation nodes
 */
USTRUCT(meta=(Abstract, Category="Core|Name", NodeColor = "0.462745, 1,0, 0.329412", DocumentationPolicy = "Strict"))
struct FRigVMFunction_NameBase : public FRigVMStruct
{
	GENERATED_BODY()

	virtual void Execute() {}
};

/**
 * Concatenates two names together to make a new name
 */
USTRUCT(meta = (DisplayName = "Concat", TemplateName = "Concat", Keywords = "Add,+,Combine,Merge,Append"))
struct FRigVMFunction_NameConcat : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_NameConcat()
	{
		A = B = Result = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The first / left name to concatenate
	UPROPERTY(meta=(Input, Aggregate))
	FName A;

	// The second / right name to concatenate
	UPROPERTY(meta=(Input, Aggregate))
	FName B;

	// The resulting concatenated name
	UPROPERTY(meta=(Output, Aggregate))
	FName Result;
};

/**
 * Returns the left or right most characters from the name chopping the given number of characters from the start or the end
 */
USTRUCT(meta = (DisplayName = "Chop", TemplateName = "Chop", Keywords = "Truncate,-,Remove,Subtract,Split"))
struct FRigVMFunction_NameTruncate : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_NameTruncate()
	{
		Name = NAME_None;
		Count = 1;
		FromEnd = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to chop
	UPROPERTY(meta=(Input))
	FName Name;

	// Number of characters to remove from left or right
	UPROPERTY(meta=(Input))
	int32 Count;

	// if set to true the characters will be removed from the end
	UPROPERTY(meta=(Input))
	bool FromEnd;

	// the part of the name without the chopped characters
	UPROPERTY(meta=(Output))
	FName Remainder;

	// the part of the name that has been chopped off
	UPROPERTY(meta = (Output))
	FName Chopped;
};

/**
 * Replace all occurrences of a subname in this name
 */
USTRUCT(meta = (DisplayName = "Replace", TemplateName = "Replace", Keywords = "Search,Emplace,Find"))
struct FRigVMFunction_NameReplace : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_NameReplace()
	{
		Name = Old = New = NAME_None;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to search within
	UPROPERTY(meta = (Input))
	FName Name;

	// The old name to search for
	UPROPERTY(meta = (Input))
	FName Old;

	// The new name to replace the old name with
	UPROPERTY(meta = (Input))
	FName New;

	// The resulting replaced name
	UPROPERTY(meta = (Output))
	FName Result;
};

/**
 * Tests whether this name ends with given name
 */
USTRUCT(meta = (DisplayName = "Ends With", TemplateName = "EndsWith", Keywords = "Right"))
struct FRigVMFunction_EndsWith : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_EndsWith()
	{
		Name = Ending = NAME_None;
		Result = false;
	}

	RIGVM_METHOD()
		RIGVM_API virtual void Execute() override;

	// The input name to search within
	UPROPERTY(meta = (Input))
	FName Name;

	// The ending name to look for
	UPROPERTY(meta = (Input))
	FName Ending;

	// True if the given input name ends with the given ending
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Tests whether this name starts with given name
 */
USTRUCT(meta = (DisplayName = "Starts With", TemplateName = "StartsWith", Keywords = "Left"))
struct FRigVMFunction_StartsWith : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_StartsWith()
	{
		Name = Start = NAME_None;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to search within
	UPROPERTY(meta = (Input))
	FName Name;

	// The start name to look for
	UPROPERTY(meta = (Input))
	FName Start;

	// True if the given input name starts with the given start string
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns true or false if a given name exists in another given name
 */
USTRUCT(meta = (DisplayName = "Contains", TemplateName = "Contains", Keywords = "Contains,Find,Has,Search"))
struct FRigVMFunction_Contains : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

		FRigVMFunction_Contains()
	{
		Name = Search = NAME_None;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to search within
	UPROPERTY(meta = (Input))
	FName Name;

	// The name to search for
	UPROPERTY(meta = (Input))
	FName Search;

	// True if a given name exists in another given name
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns true if this name is none
 */
USTRUCT(meta = (DisplayName = "Is None", TemplateName = "IsNone", Keywords = "Empty,Valid"))
struct FRigVMFunction_IsNone : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_IsNone()
	{
		Name = NAME_None;
		Result = false;
	}

	RIGVM_METHOD()
		RIGVM_API virtual void Execute() override;

	// The input name to check
	UPROPERTY(meta = (Input))
	FName Name;

	// True if the input name is None
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns true if this name is valid / is not none
 */
USTRUCT(meta = (DisplayName = "Is Valid", TemplateName = "IsValid", Keywords = "Empty,Valid"))
struct FRigVMFunction_IsNameValid : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_IsNameValid()
	{
		Value = NAME_None;
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to check
	UPROPERTY(meta = (Input))
	FName Value;

	// True if the name is valid/ is not None
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Tests whether this name ends with a numeric suffix
 */
USTRUCT(meta = (DisplayName = "Get Numeric Suffix", Keywords = "Suffix,Number,Integer"))
struct FRigVMFunction_GetNameNumericSuffix : public FRigVMFunction_NameBase
{
	GENERATED_BODY()

	FRigVMFunction_GetNameNumericSuffix()
	{
		Name = NAME_None;
		Suffix = INDEX_NONE;
		Success = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The input name to check for a numeric suffix
	UPROPERTY(meta = (Input))
	FName Name;

	// The resulting suffix (if it exists)
	UPROPERTY(meta = (Output))
	int32 Suffix;

	// True if the name ends with the given numeric suffix
	UPROPERTY(meta = (Output))
	bool Success;
};