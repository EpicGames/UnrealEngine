// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMFunction_String.generated.h"

/*
 * The base class for all string operation nodes
 */
USTRUCT(meta=(Abstract, Category="Core|String", NodeColor = "0.462745, 1,0, 0.329412", DocumentationPolicy = "Strict"))
struct FRigVMFunction_StringBase : public FRigVMStruct
{
	GENERATED_BODY()
};

/**
 * Concatenates two strings together to make a new string
 */
USTRUCT(meta = (DisplayName = "Concat", TemplateName = "Concat", Keywords = "Add,+,Combine,Merge,Append"))
struct FRigVMFunction_StringConcat : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringConcat()
	{
		A = B = Result = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The first / left string to concatenate
	UPROPERTY(meta=(Input, Aggregate))
	FString A;

	// The second / right string to concatenate
	UPROPERTY(meta=(Input, Aggregate))
	FString B;

	// The resulting concatenated string
	UPROPERTY(meta=(Output, Aggregate))
	FString Result;
};

/**
 * Returns the left or right most characters from the string chopping the given number of characters from the start or the end
 */
USTRUCT(meta = (DisplayName = "Chop", TemplateName = "Chop", Keywords = "Truncate,-,Remove,Subtract,Split"))
struct FRigVMFunction_StringTruncate : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringTruncate()
	{
		Name = FString();
		Count = 1;
		FromEnd = true;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to chop
	UPROPERTY(meta=(Input))
	FString Name;

	// Number of characters to remove from left or right
	UPROPERTY(meta=(Input))
	int32 Count;

	// if set to true the characters will be removed from the end
	UPROPERTY(meta=(Input))
	bool FromEnd;

	// the part of the string without the chopped characters
	UPROPERTY(meta=(Output))
	FString Remainder;

	// the part of the name that has been chopped off
	UPROPERTY(meta = (Output))
	FString Chopped;
};

/**
 * Replace all occurrences of a substring in this string
 */
USTRUCT(meta = (DisplayName = "Replace", TemplateName = "Replace", Keywords = "Search,Emplace,Find"))
struct FRigVMFunction_StringReplace : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringReplace()
	{
		Name = Old = New = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to search within
	UPROPERTY(meta = (Input))
	FString Name;

	// The old string to search for
	UPROPERTY(meta = (Input))
	FString Old;

	// The new string to replace the old string with
	UPROPERTY(meta = (Input))
	FString New;

	// The resulting replaced string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Tests whether this string ends with given string
 */
USTRUCT(meta = (DisplayName = "Ends With", TemplateName = "EndsWith", Keywords = "Right"))
struct FRigVMFunction_StringEndsWith : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringEndsWith()
	{
		Name = Ending = FString();
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to search within
	UPROPERTY(meta = (Input))
	FString Name;

	// The ending string to look for
	UPROPERTY(meta = (Input))
	FString Ending;

	// True if the given input string ends with the given ending
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Tests whether this string starts with given string
 */
USTRUCT(meta = (DisplayName = "Starts With", TemplateName = "StartsWith", Keywords = "Left"))
struct FRigVMFunction_StringStartsWith : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringStartsWith()
	{
		Name = Start = FString();
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to search within
	UPROPERTY(meta = (Input))
	FString Name;

	// The start string to look for
	UPROPERTY(meta = (Input))
	FString Start;

	// True if the given input string starts with the given start string
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns true or false if a given name exists in another given name
 */
USTRUCT(meta = (DisplayName = "Contains", TemplateName = "Contains", Keywords = "Contains,Find,Has,Search"))
struct FRigVMFunction_StringContains : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

		FRigVMFunction_StringContains()
	{
		Name = Search = FString();
		Result = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to search within
	UPROPERTY(meta = (Input))
	FString Name;

	// The string to search for
	UPROPERTY(meta = (Input))
	FString Search;

	// True if a given string exists in another given string
	UPROPERTY(meta = (Output))
	bool Result;
};

/**
 * Returns the length of a string 
 */
USTRUCT(meta = (DisplayName = "Length"))
struct FRigVMFunction_StringLength : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringLength()
	{
		Value = FString();
		Length = 0;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to retrieve the length for
	UPROPERTY(meta = (Input))
	FString Value;

	// The resulting length of the input string
	UPROPERTY(meta = (Output))
	int32 Length;
};

/**
 * Trims the whitespace from a string (start and end)
 */
USTRUCT(meta = (DisplayName = "Trim Whitespace", Keywords = "Space,WhiteSpace,Remove,Truncate"))
struct FRigVMFunction_StringTrimWhitespace : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringTrimWhitespace()
	{
		Value = Result = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to trim
	UPROPERTY(meta = (Input))
	FString Value;

	// The resulting trimmed string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Converts the string to upper case
 */
USTRUCT(meta = (DisplayName = "To Uppercase"))
struct FRigVMFunction_StringToUppercase : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringToUppercase()
	{
		Value = Result = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to convert to uppercase
	UPROPERTY(meta = (Input))
	FString Value;

	// The resulting uppercased string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Converts the string to lower case
 */
USTRUCT(meta = (DisplayName = "To Lowercase"))
struct FRigVMFunction_StringToLowercase : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringToLowercase()
	{
		Value = Result = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to convert to lowercase
	UPROPERTY(meta = (Input))
	FString Value;

	// The resulting lowercased string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Returns the reverse of the input string
 */
USTRUCT(meta = (DisplayName = "Reverse"))
struct FRigVMFunction_StringReverse : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringReverse()
	{
		Value = Reverse = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to reverse
	UPROPERTY(meta = (Input))
	FString Value;

	// The resulting reversed string
	UPROPERTY(meta = (Output))
	FString Reverse;
};

/**
 * Returns the left most characters of a string
 */
USTRUCT(meta = (DisplayName = "Left", Keywords = "Start,Begin"))
struct FRigVMFunction_StringLeft : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringLeft()
	{
		Value = Result = FString();
		Count = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to retrieve the left most characters of
	UPROPERTY(meta = (Input))
	FString Value;

	// The number of characters to retrieve
	UPROPERTY(meta = (Input))
	int32 Count;

	// The resulting left most characters of the input string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Returns the right most characters of a string
 */
USTRUCT(meta = (DisplayName = "Right", Keywords = "End"))
struct FRigVMFunction_StringRight : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringRight()
	{
		Value = Result = FString();
		Count = 1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to retrieve the right most characters of
	UPROPERTY(meta = (Input))
	FString Value;

	// The number of characters to retrieve
	UPROPERTY(meta = (Input))
	int32 Count;

	// The resulting right most characters of the input string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Returns the middle section of a string
 */
USTRUCT(meta = (DisplayName = "Middle", Keywords = "Within,Center"))
struct FRigVMFunction_StringMiddle : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringMiddle()
	{
		Value = Result = FString();
		Start = 0;
		Count = -1;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to retrieve a section of
	UPROPERTY(meta = (Input))
	FString Value;

	// the index of the first character
	UPROPERTY(meta = (Input))
	int32 Start;

	// if count is set to -1 all character from Start will be returned 
	UPROPERTY(meta = (Input))
	int32 Count;

	// The resulting section of the input string given start and count
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Finds a string within another string
 */
USTRUCT(meta = (DisplayName = "Find", Keywords = "IndexOf"))
struct FRigVMFunction_StringFind : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringFind()
	{
		Value = Search = FString();
		Found = false;
		Index = INDEX_NONE;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to search within
	UPROPERTY(meta = (Input))
	FString Value;

	// The string to search for
	UPROPERTY(meta = (Input))
	FString Search;

	// True if the search string was found in the input string
	UPROPERTY(meta = (Output))
    bool Found;

	// The index of the search string within the input string (or INDEX_NONE)
	UPROPERTY(meta = (Output))
	int32 Index;
};

/**
 * Splits a string into multiple sections given a separator
 */
USTRUCT(meta = (DisplayName = "Split"))
struct FRigVMFunction_StringSplit : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringSplit()
	{
		Value = Separator = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to split up
	UPROPERTY(meta = (Input))
	FString Value;

	// The separator to use to split up the input string
	UPROPERTY(meta = (Input))
	FString Separator;

	// The array of split up parts of the input string 
	UPROPERTY(meta = (Output))
	TArray<FString> Result;
};

/**
 * Joins a string into multiple sections given a separator
 */
USTRUCT(meta = (DisplayName = "Join", Keywords = "Combine"))
struct FRigVMFunction_StringJoin : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringJoin()
	{
		Result = Separator = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// An array of input strings to join
	UPROPERTY(meta = (Input))
	TArray<FString> Values;

	// The separator to use when joining the parts
	UPROPERTY(meta = (Input))
	FString Separator;

	// The resulting joined string
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Converts an integer number to a string with padding
 */
USTRUCT(meta = (DisplayName = "Pad Integer", Keywords = "FromInt,Number,LeadingZeroes"))
struct FRigVMFunction_StringPadInteger : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringPadInteger()
	{
		Value = 0;
		Digits = 4;
		Result = FString();
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input number to convert to a string (for example 4)
	UPROPERTY(meta = (Input))
	int32 Value;

	// The number of digits in total to use (for example 3)
	UPROPERTY(meta = (Input))
	int32 Digits;
 
	// The resulting string representing the padded integer (for example '004')
	UPROPERTY(meta = (Output))
	FString Result;
};

/**
 * Converts a string to an integer
 */
USTRUCT(meta = (DisplayName = "To Integer", Keywords = "ToInt,Number"))
struct FRigVMFunction_StringToInteger : public FRigVMFunction_StringBase
{
	GENERATED_BODY()

	FRigVMFunction_StringToInteger()
	{
		Value = FString();
		Result = INDEX_NONE;
		ChopLeft  = ChopRight = Success = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute();

	// The input string to convert to integer
	UPROPERTY(meta = (Input))
	FString Value;

	// chops non-digit characters from the left of the string
	UPROPERTY(meta = (Input))
	bool ChopLeft;

	// chops non-digit characters from the right of the string
	UPROPERTY(meta = (Input))
	bool ChopRight;

	// The resulting integer number (or INDEX_NONE)
	UPROPERTY(meta = (Output))
	int32 Result;

	// True if the string was successfully converted to integer
	UPROPERTY(meta = (Output))
	bool Success;
};

/*
 * Converts any value to string
 */
USTRUCT(meta=(DisplayName = "To String", NodeColor = "0.462745, 1,0, 0.329412"))
struct FRigDispatch_ToString : public FRigVMDispatchFactory
{
	GENERATED_BODY()

public:

	virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};

/*
 * Converts a string into any value
 */
USTRUCT(meta=(DisplayName = "From String", NodeColor = "0.462745, 1,0, 0.329412"))
struct FRigDispatch_FromString : public FRigVMDispatchFactory
{
	GENERATED_BODY()

public:

	virtual const TArray<FRigVMTemplateArgumentInfo>& GetArgumentInfos(FRigVMRegistryHandle& InRegistry) const override;
	virtual FRigVMTemplateTypeMap OnNewArgumentType(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const override;

#if WITH_EDITOR
	virtual FText GetArgumentTooltip(const FName& InArgumentName, TRigVMTypeIndex InTypeIndex) const override;
#endif

protected:

	virtual FRigVMFunctionPtr GetDispatchFunctionImpl(const FRigVMTemplateTypeMap& InTypes, const FRigVMRegistryHandle& InRegistry) const override;
	static void Execute(FRigVMExtendedExecuteContext& InContext, FRigVMMemoryHandleArray Handles, FRigVMPredicateBranchArray Predicates);
};