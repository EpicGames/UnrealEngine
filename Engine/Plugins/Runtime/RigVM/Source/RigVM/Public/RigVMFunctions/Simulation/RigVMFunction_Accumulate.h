// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunction_Accumulate.generated.h"

/*
 * The base class for all accumulation nodes
 */
USTRUCT(meta=(Category="Simulation|Accumulate"))
struct FRigVMFunction_AccumulateBase : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
};

/**
 * Adds a value over time over and over again
 */
USTRUCT(meta=(DisplayName="Accumulate Add (Float)", TemplateName="AccumulateAdd", Keywords="Simulate,++"))
struct FRigVMFunction_AccumulateFloatAdd : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateFloatAdd()
	{
		InitialValue = Increment = Result = AccumulatedValue = 0.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value added on every iteration (depending on the IntegrateDeltaTime flag)
	UPROPERTY(meta=(Input))
	float Increment;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	float InitialValue;

	// If True, the increment will be multiplied by the delta time as it is accumulated, treating
	// it as a rate of change. This is advisable when dealing with variable frame rates.
	UPROPERTY(meta=(Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Adds a vector over time over and over again
 */
USTRUCT(meta = (DisplayName="Accumulate Add (Vector)", TemplateName="AccumulateAdd", Keywords="Simulate,++"))
struct FRigVMFunction_AccumulateVectorAdd : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateVectorAdd()
	{
		InitialValue = Increment = Result = AccumulatedValue = FVector::ZeroVector;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value added on every iteration (depending on the IntegrateDeltaTime flag)
	UPROPERTY(meta = (Input))
	FVector Increment;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	// If True, the increment will be multiplied by the delta time as it is accumulated, treating
	// it as a rate of change. This is advisable when dealing with variable frame rates.
	UPROPERTY(meta = (Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Multiplies a value over time over and over again
 */
USTRUCT(meta=(DisplayName="Accumulate Mul (Float)", TemplateName="AccumulateMul", Keywords="Simulate,**"))
struct FRigVMFunction_AccumulateFloatMul : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateFloatMul()
	{
		InitialValue = Multiplier = Result = AccumulatedValue = 1.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The multiplier to apply on every iteration (depending on the IntegrateDeltaTime flag)
	UPROPERTY(meta=(Input))
	float Multiplier;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	float InitialValue;

	// If True, the multiplier will be scaled depending on the delta time as it is accumulated,
	// making the resulting accumulation less sensitive to varying frame rates.
	UPROPERTY(meta=(Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Multiplies a vector over time over and over again
 */
USTRUCT(meta = (DisplayName="Accumulate Mul (Vector)", TemplateName="AccumulateMul", Keywords="Simulate,**"))
struct FRigVMFunction_AccumulateVectorMul : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateVectorMul()
	{
		InitialValue = Multiplier = Result = AccumulatedValue = FVector::OneVector;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The multiplier to apply on every iteration (depending on the IntegrateDeltaTime flag)
	UPROPERTY(meta = (Input))
	FVector Multiplier;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	// If True, the multiplier will be scaled depending on the delta time as it is accumulated,
	// making the resulting accumulation less sensitive to varying frame rates.
	UPROPERTY(meta = (Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Multiplies a quaternion over time over and over again
 */
USTRUCT(meta=(DisplayName="Accumulate Mul (Quaternion)", TemplateName="AccumulateMul", Keywords="Simulate,**"))
struct FRigVMFunction_AccumulateQuatMul : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateQuatMul()
	{
		InitialValue = Multiplier = Result = AccumulatedValue = FQuat::Identity;
		bIntegrateDeltaTime = bFlipOrder = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The multiplier to apply on every iteration (depending on the IntegrateDeltaTime flag) 
	UPROPERTY(meta=(Input))
	FQuat Multiplier;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FQuat InitialValue;

	// If True the multiplier will be pre-multiplied, otherwise post-multiplied
	UPROPERTY(meta = (Input))
	bool bFlipOrder;

	// If True, the multiplier will be scaled depending on the delta time as it is accumulated,
	// making the resulting accumulation less sensitive to varying frame rates.
	UPROPERTY(meta=(Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta=(Output))
	FQuat Result;

	UPROPERTY()
	FQuat AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Multiplies a transform over time over and over again
 */
USTRUCT(meta = (DisplayName="Accumulate Mul (Transform)", TemplateName="AccumulateMul", Keywords="Simulate,**"))
struct FRigVMFunction_AccumulateTransformMul : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateTransformMul()
	{
		InitialValue = Multiplier = Result = AccumulatedValue = FTransform::Identity;
		bIntegrateDeltaTime = bFlipOrder = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The multiplier to apply for every iteration (depending on the IntegrateDeltaTime flag) 
	UPROPERTY(meta = (Input))
	FTransform Multiplier;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FTransform InitialValue;

	// If True the multiplier will be pre-multiplied, otherwise post-multiplied
	UPROPERTY(meta = (Input))
	bool bFlipOrder;

	// If True, the multiplier will be scaled depending on the delta time as it is accumulated,
	// making the resulting accumulation less sensitive to varying frame rates.
	UPROPERTY(meta = (Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta = (Output))
	FTransform Result;

	UPROPERTY()
	FTransform AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Interpolates two values over time over and over again
 */
USTRUCT(meta=(DisplayName="Accumulate Lerp (Float)", TemplateName="AccumulateLerp", Keywords="Simulate,Ramp"))
struct FRigVMFunction_AccumulateFloatLerp : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateFloatLerp()
	{
		InitialValue = TargetValue = Blend = Result = AccumulatedValue = 0.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The target value to interpolate towards
	UPROPERTY(meta=(Input))
	float TargetValue;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	float InitialValue;

	// The blend to use for the interpolation.
	// This value may be scaled down based on the Integrate Delta Time setting
	UPROPERTY(meta = (Input))
	float Blend;

	// If True the integration will be relying on the delta time
	// to create more deterministic results with varying framerates.
	UPROPERTY(meta=(Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Interpolates two vectors over time over and over again
 */
USTRUCT(meta = (DisplayName="Accumulate Lerp (Vector)", TemplateName="AccumulateLerp", Keywords="Simulate,Ramp"))
struct FRigVMFunction_AccumulateVectorLerp : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateVectorLerp()
	{
		InitialValue = TargetValue = Result = AccumulatedValue = FVector::ZeroVector;
		Blend = 0.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The target value to interpolate towards
	UPROPERTY(meta = (Input))
	FVector TargetValue;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FVector InitialValue;

	// The blend to use for the interpolation.
	// This value may be scaled down based on the Integrate Delta Time setting
	UPROPERTY(meta = (Input))
	float Blend;

	// If True the integration will be relying on the delta time
	// to create more deterministic results with varying framerates.
	UPROPERTY(meta = (Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta = (Output))
	FVector Result;

	UPROPERTY()
	FVector AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Interpolates two quaternions over time over and over again
 */
USTRUCT(meta=(DisplayName="Accumulate Lerp (Quaternion)", TemplateName="AccumulateLerp", Keywords="Simulate,Ramp"))
struct FRigVMFunction_AccumulateQuatLerp : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateQuatLerp()
	{
		InitialValue = TargetValue = Result = AccumulatedValue = FQuat::Identity;
		Blend = 0.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The target value to interpolate towards
	UPROPERTY(meta=(Input))
	FQuat TargetValue;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FQuat InitialValue;

	// The blend to use for the interpolation.
	// This value may be scaled down based on the Integrate Delta Time setting
	UPROPERTY(meta = (Input))
	float Blend;

	// If True the integration will be relying on the delta time
	// to create more deterministic results with varying framerates.
	UPROPERTY(meta=(Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta=(Output))
	FQuat Result;

	UPROPERTY()
	FQuat AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Interpolates two transforms over time over and over again
 */
USTRUCT(meta = (DisplayName="Accumulate Lerp (Transform)", TemplateName="AccumulateLerp", Keywords="Simulate,Ramp"))
struct FRigVMFunction_AccumulateTransformLerp : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateTransformLerp()
	{
		InitialValue = TargetValue = Result = AccumulatedValue = FTransform::Identity;
		Blend = 0.f;
		bIntegrateDeltaTime = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The target value to interpolate towards
	UPROPERTY(meta = (Input))
	FTransform TargetValue;

	// The initial value to start at
	UPROPERTY(meta = (Input))
	FTransform InitialValue;

	// The blend to use for the interpolation.
	// This value may be scaled down based on the Integrate Delta Time setting
	UPROPERTY(meta = (Input))
	float Blend;

	// If True the integration will be relying on the delta time
	// to create more deterministic results with varying framerates.
	UPROPERTY(meta = (Input))
	bool bIntegrateDeltaTime;

	// The resulting accumulated value
	UPROPERTY(meta = (Output))
	FTransform Result;

	UPROPERTY()
	FTransform AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Accumulates the min and max values over time
 */
USTRUCT(meta=(DisplayName="Accumulate Range (Float)", TemplateName="AccumulateRange", Keywords="Range"))
struct FRigVMFunction_AccumulateFloatRange : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AccumulateFloatRange()
	{
		Value = Minimum = Maximum = AccumulatedMinimum = AccumulatedMaximum = 0.f;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to accumulate the min and max value of
	UPROPERTY(meta = (Input))
	float Value;

	// The accumulated minimum value of the input
	UPROPERTY(meta= (Output))
	float Minimum;

	// The accumulated maximum value of the input
	UPROPERTY(meta= (Output))
	float Maximum;

	UPROPERTY()
	float AccumulatedMinimum;

	UPROPERTY()
	float AccumulatedMaximum;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Accumulates the min and max values over time
 */
USTRUCT(meta = (DisplayName="Accumulate Range (Vector)", TemplateName="AccumulateRange", Keywords="Range"))
struct FRigVMFunction_AccumulateVectorRange : public FRigVMFunction_AccumulateBase
{
	GENERATED_BODY()

	FRigVMFunction_AccumulateVectorRange()
	{
		Value = Minimum = Maximum = AccumulatedMinimum = AccumulatedMaximum = FVector::ZeroVector;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to accumulate the min and max value of
	UPROPERTY(meta = (Input))
	FVector Value;

	// The accumulated minimum value of the input (for each component)
	UPROPERTY(meta= (Output))
	FVector Minimum;

	// The accumulated maximum value of the input (for each component)
	UPROPERTY(meta= (Output))
	FVector Maximum;

	UPROPERTY()
	FVector AccumulatedMinimum;

	UPROPERTY()
	FVector AccumulatedMaximum;

	UPROPERTY()
	bool bIsInitialized;
};
