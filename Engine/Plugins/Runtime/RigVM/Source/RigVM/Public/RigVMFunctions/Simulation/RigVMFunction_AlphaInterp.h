// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "Animation/InputScaleBias.h"
#include "RigVMFunction_AlphaInterp.generated.h"

/**
 * Takes in a float value and outputs an accumulated value with a customized scale and clamp
 */
USTRUCT(meta=(DisplayName="Alpha Interpolate", Keywords="Alpha,Lerp,LinearInterpolate", Category = "Simulation|Time", TemplateName="AlphaInterp", MenuDescSuffix = "(Float)"))
struct FRigVMFunction_AlphaInterp : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AlphaInterp()
	{
		Value = Result = 0.f;
		ScaleBiasClamp = FInputScaleBiasClamp();
		bMapRange = ScaleBiasClamp.bMapRange;
		bClampResult = ScaleBiasClamp.bClampResult;
		bInterpResult = ScaleBiasClamp.bInterpResult;
		InRange = ScaleBiasClamp.InRange;
		OutRange = ScaleBiasClamp.OutRange;
		Scale = ScaleBiasClamp.Scale;
		Bias = ScaleBiasClamp.Bias;
		ClampMin = ScaleBiasClamp.ClampMin;
		ClampMax = ScaleBiasClamp.ClampMax;
		InterpSpeedIncreasing = ScaleBiasClamp.InterpSpeedIncreasing;
		InterpSpeedDecreasing = ScaleBiasClamp.InterpSpeedDecreasing;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	RIGVM_API virtual FString ProcessPinLabelForInjection(const FString& InLabel) const override;

	// The input value to interpolate
	UPROPERTY(meta=(Input))
	float Value;

	// The scale to apply to the interpolation
	UPROPERTY(meta = (Input))
	float Scale;

	// The bias to use for the interpolation  
	UPROPERTY(meta = (Input))
	float Bias;

	// If True the input value will be mapped using the min and max range 
	UPROPERTY(meta=(Input, Constant))
	bool bMapRange;

	// The minimum and maximum for the input range
	UPROPERTY(meta=(Input, EditCondition = "bMapRange"))
	FInputRange InRange;

	// The minimum and maximum for the output range
	UPROPERTY(meta=(Input, EditCondition = "bMapRange"))
	FInputRange OutRange;

	// If True the output value will be clamped by the min and max 
	UPROPERTY(meta = (Input, Constant))
	bool bClampResult;

	// The minimum for the output's clamp range 
	UPROPERTY(meta=(Input, EditCondition = "bClampResult"))
	float ClampMin;

	// The maximum for the output's clamp range 
	UPROPERTY(meta=(Input, EditCondition = "bClampResult"))
	float ClampMax;

	// If True to the output result will be further intepolated
	UPROPERTY(meta = (Input, Constant))
	bool bInterpResult;

	// The output interpolation increasing speed
	UPROPERTY(meta=(Input, EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing;

	// The output interpolation decreasing speed
	UPROPERTY(meta=(Input, EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing;

	// The resulting interpolated value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY()
	FInputScaleBiasClamp ScaleBiasClamp;
};

/**
 * Takes in a vector value and outputs an accumulated value with a customized scale and clamp
 */
USTRUCT(meta=(DisplayName="Alpha Interpolate", Keywords="Alpha,Lerp,LinearInterpolate", Category = "Simulation|Time", TemplateName="AlphaInterp", MenuDescSuffix = "(Vector)"))
struct FRigVMFunction_AlphaInterpVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AlphaInterpVector()
	{
		Value = Result = FVector::ZeroVector;
		ScaleBiasClamp = FInputScaleBiasClamp();
		bMapRange = ScaleBiasClamp.bMapRange;
		bClampResult = ScaleBiasClamp.bClampResult;
		bInterpResult = ScaleBiasClamp.bInterpResult;
		InRange = ScaleBiasClamp.InRange;
		OutRange = ScaleBiasClamp.OutRange;
		Scale = ScaleBiasClamp.Scale;
		Bias = ScaleBiasClamp.Bias;
		ClampMin = ScaleBiasClamp.ClampMin;
		ClampMax = ScaleBiasClamp.ClampMax;
		InterpSpeedIncreasing = ScaleBiasClamp.InterpSpeedIncreasing;
		InterpSpeedDecreasing = ScaleBiasClamp.InterpSpeedDecreasing;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	RIGVM_API virtual FString ProcessPinLabelForInjection(const FString& InLabel) const override;

	// The input value to interpolate
	UPROPERTY(meta=(Input))
	FVector Value;

	// The scale to apply to the interpolation
	UPROPERTY(meta = (Input))
	float Scale;

	// The bias to use for the interpolation  
	UPROPERTY(meta = (Input))
	float Bias;

	// If True the input value will be mapped using the min and max range 
	UPROPERTY(meta = (Input, Constant))
	bool bMapRange;

	// The minimum and maximum for the input range
	UPROPERTY(meta = (Input, EditCondition = "bMapRange"))
	FInputRange InRange;

	// The minimum and maximum for the output range
	UPROPERTY(meta = (Input, EditCondition = "bMapRange"))
	FInputRange OutRange;

	// If True the output value will be clamped by the min and max 
	UPROPERTY(meta = (Input, Constant))
	bool bClampResult;

	// The minimum for the output's clamp range 
	UPROPERTY(meta = (Input, EditCondition = "bClampResult"))
	float ClampMin;

	// The maximum for the output's clamp range 
	UPROPERTY(meta = (Input, EditCondition = "bClampResult"))
	float ClampMax;

	// If True to the output result will be further intepolated
	UPROPERTY(meta = (Input, Constant))
	bool bInterpResult;

	// The output interpolation increasing speed
	UPROPERTY(meta = (Input, EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing;

	// The output interpolation decreasing speed
	UPROPERTY(meta = (Input, EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing;

	// The resulting interpolated value
	UPROPERTY(meta=(Output))
	FVector Result;

	UPROPERTY()
	FInputScaleBiasClamp ScaleBiasClamp;
};

/**
 * Takes in a quaternion value and outputs an accumulated value with a customized scale and clamp
 */
USTRUCT(meta=(DisplayName="Alpha Interpolate", Keywords="Alpha,Lerp,LinearInterpolate", Category = "Simulation|Time", TemplateName="AlphaInterp", MenuDescSuffix = "(Quat)"))
struct FRigVMFunction_AlphaInterpQuat : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_AlphaInterpQuat()
	{
		Value = Result = FQuat::Identity;
		ScaleBiasClamp = FInputScaleBiasClamp();
		bMapRange = ScaleBiasClamp.bMapRange;
		bClampResult = ScaleBiasClamp.bClampResult;
		bInterpResult = ScaleBiasClamp.bInterpResult;
		InRange = ScaleBiasClamp.InRange;
		OutRange = ScaleBiasClamp.OutRange;
		Scale = ScaleBiasClamp.Scale;
		Bias = ScaleBiasClamp.Bias;
		ClampMin = ScaleBiasClamp.ClampMin;
		ClampMax = ScaleBiasClamp.ClampMax;
		InterpSpeedIncreasing = ScaleBiasClamp.InterpSpeedIncreasing;
		InterpSpeedDecreasing = ScaleBiasClamp.InterpSpeedDecreasing;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	RIGVM_API virtual FString ProcessPinLabelForInjection(const FString& InLabel) const override;

	// The input value to interpolate
	UPROPERTY(meta=(Input))
	FQuat Value;

	// The scale to apply to the interpolation
	UPROPERTY(meta = (Input))
	float Scale;

	// The bias to use for the interpolation  
	UPROPERTY(meta = (Input))
	float Bias;

	// If True the input value will be mapped using the min and max range 
	UPROPERTY(meta = (Input, Constant))
	bool bMapRange;

	// The minimum and maximum for the input range
	UPROPERTY(meta = (Input, EditCondition = "bMapRange"))
	FInputRange InRange;

	// The minimum and maximum for the output range
	UPROPERTY(meta = (Input, EditCondition = "bMapRange"))
	FInputRange OutRange;

	// If True the output value will be clamped by the min and max 
	UPROPERTY(meta = (Input, Constant))
	bool bClampResult;

	// The minimum for the output's clamp range 
	UPROPERTY(meta = (Input, EditCondition = "bClampResult"))
	float ClampMin;

	// The maximum for the output's clamp range 
	UPROPERTY(meta = (Input, EditCondition = "bClampResult"))
	float ClampMax;

	// If True to the output result will be further intepolated
	UPROPERTY(meta = (Input, Constant))
	bool bInterpResult;

	// The output interpolation increasing speed
	UPROPERTY(meta = (Input, EditCondition = "bInterpResult"))
	float InterpSpeedIncreasing;

	// The output interpolation decreasing speed
	UPROPERTY(meta = (Input, EditCondition = "bInterpResult"))
	float InterpSpeedDecreasing;

	// The resulting interpolated value
	UPROPERTY(meta=(Output))
	FQuat Result;

	UPROPERTY()
	FInputScaleBiasClamp ScaleBiasClamp;
};
