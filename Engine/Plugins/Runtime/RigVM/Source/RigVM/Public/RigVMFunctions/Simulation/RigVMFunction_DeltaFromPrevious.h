// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunction_DeltaFromPrevious.generated.h"

#define UE_API RIGVM_API

/**
 * Computes the difference from the previous value going through the node
 */
USTRUCT(meta=(DisplayName="DeltaFromPrevious (Float)", Category = "Simulation|Time", TemplateName="DeltaFromPrevious", Keywords="Difference,Velocity,Acceleration"))
struct FRigVMFunction_DeltaFromPreviousFloat : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_DeltaFromPreviousFloat()
	{
		Value = Delta = PreviousValue = Cache = 0.f;
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to compute the delta of
	UPROPERTY(meta = (Input))
	float Value;

	// The resulting delta from the previous iteration
	UPROPERTY(meta=(Output))
	float Delta;

	// The value from the previous iteration
	UPROPERTY(meta=(Output))
	float PreviousValue;

	UPROPERTY()
	float Cache;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Computes the difference from the previous value going through the node
 */
USTRUCT(meta=(DisplayName="DeltaFromPrevious (Vector)", Category = "Simulation|Time", TemplateName="DeltaFromPrevious", Keywords="Difference,Velocity,Acceleration"))
struct FRigVMFunction_DeltaFromPreviousVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_DeltaFromPreviousVector()
	{
		Value = Delta = PreviousValue = Cache = FVector::ZeroVector;
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to compute the delta of
	UPROPERTY(meta = (Input))
	FVector Value;

	// The resulting delta from the previous iteration
	UPROPERTY(meta=(Output))
	FVector Delta;

	// The value from the previous iteration
	UPROPERTY(meta=(Output))
	FVector PreviousValue;

	UPROPERTY()
	FVector Cache;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Computes the difference from the previous value going through the node
 */
USTRUCT(meta=(DisplayName="DeltaFromPrevious (Quaternion)", Category = "Simulation|Time", TemplateName="DeltaFromPrevious", Keywords="Difference,Velocity,Acceleration"))
struct FRigVMFunction_DeltaFromPreviousQuat : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_DeltaFromPreviousQuat()
	{
		Value = Delta = PreviousValue = Cache = FQuat::Identity;
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to compute the delta of
	UPROPERTY(meta = (Input))
	FQuat Value;

	// The resulting delta from the previous iteration
	UPROPERTY(meta=(Output))
	FQuat Delta;

	// The value from the previous iteration
	UPROPERTY(meta=(Output))
	FQuat PreviousValue;

	UPROPERTY()
	FQuat Cache;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Computes the difference from the previous value going through the node
 */
USTRUCT(meta=(DisplayName="DeltaFromPrevious (Transform)", Category = "Simulation|Time", TemplateName="DeltaFromPrevious", Keywords="Difference,Velocity,Acceleration"))
struct FRigVMFunction_DeltaFromPreviousTransform : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_DeltaFromPreviousTransform()
	{
		Value = Delta = PreviousValue = Cache = FTransform::Identity;
		bIsInitialized = false;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The input value to compute the delta of
	UPROPERTY(meta = (Input))
	FTransform Value;

	// The resulting delta from the previous iteration
	UPROPERTY(meta=(Output))
	FTransform Delta;

	// The value from the previous iteration
	UPROPERTY(meta=(Output))
	FTransform PreviousValue;

	UPROPERTY()
	FTransform Cache;

	UPROPERTY()
	bool bIsInitialized;
};

#undef UE_API
