// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunction_Kalman.generated.h"

/**
 * Averages a value over time.
 * This uses a Kalman Filter internally.
 */
USTRUCT(meta=(DisplayName="Average Over Time (Float)", Category = "Simulation|Time", TemplateName="KalmanFilter", Keywords="Average,Smooth,KalmanFilter"))
struct FRigVMFunction_KalmanFloat : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_KalmanFloat()
	{
		Value = Result = 0.f;
		BufferSize = 16;
		LastInsertIndex = 0;
	}

	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to filter
	UPROPERTY(meta = (Input))
	float Value;

	// The buffer size to use (the higher the more precise)
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	// The resulting averaged / filtered value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY(meta=(ArraySize = "FMath::Clamp<int32>(BufferSize, 1, 512)"))
	TArray<float> Buffer;

	UPROPERTY()
	int32 LastInsertIndex;
};

/**
 * Averages a value over time.
 * This uses a Kalman Filter internally.
 */
USTRUCT(meta=(DisplayName="Average Over Time (Vector)", Category = "Simulation|Time", TemplateName="KalmanFilter", Keywords="Average,Smooth,KalmanFilter"))
struct FRigVMFunction_KalmanVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_KalmanVector()
	{
		Value = Result = FVector::ZeroVector;
		BufferSize = 16;
		LastInsertIndex = 0;
	}
	
	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to filter
	UPROPERTY(meta = (Input))
	FVector Value;

	// The buffer size to use (the higher the more precise)
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	// The resulting averaged / filtered value
	UPROPERTY(meta=(Output))
	FVector Result;

	UPROPERTY(meta=(ArraySize = "FMath::Clamp<int32>(BufferSize, 1, 512)"))
	TArray<FVector> Buffer;

	UPROPERTY()
	int32 LastInsertIndex;
};

/**
 * Averages a transform over time.
 * This uses a Kalman Filter internally.
 */
USTRUCT(meta=(DisplayName="Average Over Time (Transform)", Category = "Simulation|Time", TemplateName="KalmanFilter", Keywords="Average,Smooth,KalmanFilter"))
struct FRigVMFunction_KalmanTransform : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_KalmanTransform()
	{
		Value = Result = FTransform::Identity;
		BufferSize = 16;
		LastInsertIndex = 0;
	}

	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The value to filter
	UPROPERTY(meta = (Input))
	FTransform Value;

	// The buffer size to use (the higher the more precise)
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	// The resulting averaged / filtered value
	UPROPERTY(meta=(Output))
	FTransform Result;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 1, 512)"))
	TArray<FTransform> Buffer;

	UPROPERTY()
	int32 LastInsertIndex;
};
