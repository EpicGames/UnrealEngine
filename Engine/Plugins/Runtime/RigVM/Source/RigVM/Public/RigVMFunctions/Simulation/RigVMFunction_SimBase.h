// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMStruct.h"
#include "RigVMFunction_SimBase.generated.h"

/*
 * The base class for all pure simulation nodes
 */
USTRUCT(meta=(Abstract, Category = "Simulation", NodeColor = "0.25 0.05 0.05", DocumentationPolicy = "Strict"))
struct FRigVMFunction_SimBase : public FRigVMStruct
{
	GENERATED_BODY()
};

/*
 * The base class for all mutable simulation nodes
 */
USTRUCT(meta=(Abstract, Category = "Simulation", NodeColor = "0.25 0.05 0.05", DocumentationPolicy = "Strict"))
struct FRigVMFunction_SimBaseMutable : public FRigVMStructMutable
{
	GENERATED_BODY()
};
