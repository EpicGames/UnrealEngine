// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunction_TimeOffset.generated.h"

/**
 * Records a value over time and can access the value from the past
 */
USTRUCT(meta=(DisplayName="Value Over Time (Float)", Category = "Simulation|Time", TemplateName="TimeOffset", Keywords="Buffer,Delta,History,Previous,TimeOffset,Delay"))
struct FRigVMFunction_TimeOffsetFloat : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_TimeOffsetFloat()
	{
		SecondsAgo = 0.5f;
		Value = Result = 0.f;
		BufferSize = 16;
		TimeRange = 1.f;
		LastInsertIndex = 0;
		UpperBound = 0;
	}

	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
	
	/** The value to record */
	UPROPERTY(meta = (Input))
	float Value;

	/** Seconds of time in the past you want to query the value for */
	UPROPERTY(meta = (Input))
	float SecondsAgo;

	/** The sampling precision of the buffer. The higher the more precise - the more memory usage. */
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	/** The maximum time required for offsetting in seconds. */
	UPROPERTY(meta=(Input, Constant))
	float TimeRange;

	// The resulting offset value
	UPROPERTY(meta=(Output))
	float Result;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<float> Buffer;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<float> DeltaTimes;

	UPROPERTY()
	int32 LastInsertIndex;

	UPROPERTY()
	int32 UpperBound;
};

/**
 * Records a value over time and can access the value from the past
 */
USTRUCT(meta=(DisplayName="Value Over Time (Vector)", Category = "Simulation|Time", TemplateName="TimeOffset", Keywords="Buffer,Delta,History,Previous,TimeOffset,Delay"))
struct FRigVMFunction_TimeOffsetVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_TimeOffsetVector()
	{
		SecondsAgo = 0.5f;
		Value = Result = FVector::ZeroVector;
		BufferSize = 16;
		TimeRange = 1.f;
		LastInsertIndex = 0;
		UpperBound = 0;
	}

	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
	
	/** The value to record */
	UPROPERTY(meta = (Input))
	FVector Value;

	/** Seconds of time in the past you want to query the value for */
	UPROPERTY(meta = (Input))
	float SecondsAgo;

	/** The sampling precision of the buffer. The higher the more precise - the more memory usage. */
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	/** The maximum time required for offsetting in seconds. */
	UPROPERTY(meta=(Input, Constant))
	float TimeRange;

	// The resulting offset value
	UPROPERTY(meta=(Output))
	FVector Result;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<FVector> Buffer;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<float> DeltaTimes;

	UPROPERTY()
	int32 LastInsertIndex;

	UPROPERTY()
	int32 UpperBound;
};

/**
 * Records a value over time and can access the value from the past
 */
USTRUCT(meta=(DisplayName="Value Over Time (Transform)", Category = "Simulation|Time", TemplateName="TimeOffset", Keywords="Buffer,Delta,History,Previous,TimeOffset,Delay"))
struct FRigVMFunction_TimeOffsetTransform : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_TimeOffsetTransform()
	{
		SecondsAgo = 0.5f;
		Value = Result = FTransform::Identity;
		BufferSize = 16;
		TimeRange = 1.f;
		LastInsertIndex = 0;
		UpperBound = 0;
	}

	virtual void Initialize() override { Buffer.Reset(); }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;
	
	/** The value to record */
	UPROPERTY(meta = (Input))
	FTransform Value;

	/** Seconds of time in the past you want to query the value for */
	UPROPERTY(meta = (Input))
	float SecondsAgo;

	/** The sampling precision of the buffer. The higher the more precise - the more memory usage. */
	UPROPERTY(meta=(Input, Constant))
	int32 BufferSize;

	/** The maximum time required for offsetting in seconds. */
	UPROPERTY(meta=(Input, Constant))
	float TimeRange;

	// The resulting offset value
	UPROPERTY(meta=(Output))
	FTransform Result;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<FTransform> Buffer;

	UPROPERTY(meta = (ArraySize = "FMath::Clamp<int32>(BufferSize, 2, 512)"))
	TArray<float> DeltaTimes;

	UPROPERTY()
	int32 LastInsertIndex;

	UPROPERTY()
	int32 UpperBound;
};
