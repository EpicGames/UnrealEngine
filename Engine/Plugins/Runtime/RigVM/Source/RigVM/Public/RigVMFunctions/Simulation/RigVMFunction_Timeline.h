// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunction_Timeline.generated.h"

/*
 * The base class for all time nodes
 */
USTRUCT(meta=(Abstract, Category="Simulation|Time"))
struct FRigVMFunction_TimeBase : public FRigVMFunction_SimBase
{
	GENERATED_BODY()

	virtual void Execute() {}
};

/**
 * Simulates a time value - can act as a timeline playing back
 */
USTRUCT(meta=(DisplayName="Accumulated Time", Keywords="Playback,Pause,Timeline"))
struct FRigVMFunction_Timeline : public FRigVMFunction_TimeBase
{
	GENERATED_BODY()
	
	FRigVMFunction_Timeline()
	{
		Speed = 1.f;
		Time = AccumulatedValue = 0.f;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The speed to apply to the played back time
	UPROPERTY(meta = (Input))
	float Speed;

	// The played back / accumulated time in seconds
	UPROPERTY(meta=(Output))
	float Time;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Simulates a time value - and outputs loop information
 */
USTRUCT(meta=(DisplayName="Time Loop", Keywords="Playback,Pause,Timeline"))
struct FRigVMFunction_TimeLoop : public FRigVMFunction_TimeBase
{
	GENERATED_BODY()
	
	FRigVMFunction_TimeLoop()
	{
		Speed = 1.f;
		Duration = 1.f;
		Normalize = false;
		Absolute = Relative = FlipFlop = AccumulatedAbsolute = AccumulatedRelative = 0.f;
		NumIterations = 0;
		Even = false;
		bIsInitialized = false;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The speed to play-back the time at
	UPROPERTY(meta = (Input))
	float Speed;

	// the duration of a single loop in seconds
	UPROPERTY(meta = (Input))
	float Duration;

	// if set to true the output relative and flipflop
	// will be normalized over the duration.
	UPROPERTY(meta = (Input))
	bool Normalize;

	// the overall time in seconds
	UPROPERTY(meta=(Output))
	float Absolute;

	// the relative time in seconds (within the loop)
	UPROPERTY(meta=(Output))
	float Relative;

	// the relative time in seconds (within the loop),
	// going from 0 to duration and then back from duration to 0,
	// or 0 to 1 and 1 to 0 if Normalize is turned on
	UPROPERTY(meta=(Output))
	float FlipFlop;

	// true if the iteration of the loop is even
	UPROPERTY(meta=(Output))
	bool Even;

	UPROPERTY()
	float AccumulatedAbsolute;

	UPROPERTY()
	float AccumulatedRelative;

	UPROPERTY()
	int32 NumIterations;

	UPROPERTY()
	bool bIsInitialized;
};

/**
 * Starts a timer and provides access to the time simulation
 */
USTRUCT(meta=(DisplayName="Timer Value", Keywords="Playback,Pause,Timeline"))
struct FRigVMFunction_TimerValue : public FRigVMFunction_TimeBase
{
	GENERATED_BODY()
	
	FRigVMFunction_TimerValue()
	{
		Reset = Repeats = JustStarted = IsRunning = JustFinished = bIsInitialized = false;
		Speed = Duration = 1.f;
		Time = Ratio = AccumulatedValue = 0.f;
	}

	virtual void Initialize() override { bIsInitialized = false; }

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// A flag to reset / restart the timer
	UPROPERTY(meta = (Input))
	bool Reset;

	// The speed to apply to the played back time
	UPROPERTY(meta = (Input))
	float Speed;

	// The duration in seconds the timer will take to complete
	UPROPERTY(meta = (Input))
	float Duration;

	// A flag indicating if the timer is a one-shot or if it should repeat / loop
	UPROPERTY(meta = (Input))
	bool Repeats;

	// True if the timer has just started
	UPROPERTY(meta=(Output))
	bool JustStarted;

	// True if the timer is running
	UPROPERTY(meta=(Output))
	bool IsRunning;

	// True if the timer has just finished
	UPROPERTY(meta=(Output))
	bool JustFinished;

	// The played back / accumulated time in seconds
	UPROPERTY(meta=(Output))
	float Time;

	// The played back / accumulated time as a ratio of time / duration (0.0 - 1.0)
	UPROPERTY(meta=(Output))
	float Ratio;

	UPROPERTY()
	float AccumulatedValue;

	UPROPERTY()
	bool bIsInitialized;
};
