// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMFunctions/Simulation/RigVMFunction_SimBase.h"
#include "RigVMFunctions/Math/RigVMMathLibrary.h"
#include "RigVMFunction_Verlet.generated.h"

/**
 * Simulates a single position over time using Verlet integration. It is recommended to use SpringInterp instead as it
 * is more accurate and stable, and has more meaningful parameters.
 */
USTRUCT(meta=(DisplayName="Verlet (Vector)", Category = "Simulation|Springs", TemplateName="Verlet", Keywords="Simulate,Integrate"))
struct FRigVMFunction_VerletIntegrateVector : public FRigVMFunction_SimBase
{
	GENERATED_BODY()
	
	FRigVMFunction_VerletIntegrateVector()
	{
		Target = Force = Position = Velocity = Acceleration = FVector::ZeroVector;
		Strength = 64.f;
		Damp = 0.01f;
		Blend = 5.f;
		Point = FRigVMSimPoint();
		bInitialized = false;
	}

	RIGVM_METHOD()
	RIGVM_API virtual void Execute() override;

	// The target value to interpolate / integrate towards
	UPROPERTY(meta = (Input))
	FVector Target;

	/** The strength of the verlet spring */
	UPROPERTY(meta = (Input))
	float Strength;

	/** The amount of damping to apply ( 0.0 to 1.0, but usually really low like 0.005 )*/
	UPROPERTY(meta = (Input))
	float Damp;

	/** The amount of blending to apply per second */
	UPROPERTY(meta = (Input))
	float Blend;

	/** The force feeding into the solver. Can be used for gravity. */
	UPROPERTY(meta = (Input))
	FVector Force;

	// The resulting simulated position
	UPROPERTY(meta = (Output))
	FVector Position;

	// The resulting simulated velocity
	UPROPERTY(meta = (Output))
	FVector Velocity;

	// The resulting simulated acceleration
	UPROPERTY(meta = (Output))
	FVector Acceleration;

	UPROPERTY(transient)
	FRigVMSimPoint Point;

	UPROPERTY(transient)
	bool bInitialized;
};

