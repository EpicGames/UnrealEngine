// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMTrace.h"
#include "Engine/AssetUserData.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#if WITH_EDITOR
#include "RigVMCore/RigVMDebugInfo.h"
#endif
#if RIGVM_TRACE_ENABLED
#include "RigVMCore/RigVMObjectArchive.h"
#endif
#include "RigVMHost.generated.h"

#define UE_API RIGVM_API 

// set this to something larger than 0 to profile N runs
#ifndef UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
#define UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM 0
#endif

class IRigVMRuntimeAssetInterface;

USTRUCT()
struct FRigVMUserDefinedTypesInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FSoftObjectPath> StructGuidToPathName;
	UPROPERTY()
	TMap<FString, FSoftObjectPath> EnumToPathName;
	UPROPERTY(transient)
	TSet<TObjectPtr<UObject>> TypesInUse;
};

UCLASS(Abstract, editinlinenew, MinimalAPI)
class URigVMHost : public UObject, public IInterface_AssetUserData
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = RigVM)
	static UE_API TArray<URigVMHost*> FindRigVMHosts(UObject* Outer, TSubclassOf<URigVMHost> OptionalClass);

	static UE_API bool IsGarbageOrDestroyed(const UObject* InObject);

	/** UObject interface */
	UE_API virtual UWorld* GetWorld() const override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
#if WITH_EDITOR
	UE_API virtual void PostEditImport() override;
#endif

	/** Gets the current absolute time */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	float GetAbsoluteTime() const { return AbsoluteTime; }

	/** Gets the current delta time */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	float GetDeltaTime() const { return DeltaTime; }

	/** Set the current delta time */
	UFUNCTION(BlueprintCallable, Category="RigVM")
	UE_API void SetDeltaTime(float InDeltaTime);

	/** Set the current absolute time */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API void SetAbsoluteTime(float InAbsoluteTime, bool InSetDeltaTimeZero = false);

	/** Set the current absolute and delta times */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API void SetAbsoluteAndDeltaTime(float InAbsoluteTime, float InDeltaTime);

	/** Set the current fps */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API void SetFramesPerSecond(float InFramesPerSecond);

	/** Returns the current frames per second (this may change over time) */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API float GetCurrentFramesPerSecond() const;

	/** Returns the public context script struct to use for this owner */
	virtual UScriptStruct* GetPublicContextStruct() const { return FRigVMExecuteContext::StaticStruct(); }

	/** Is valid for execution */
	UFUNCTION(BlueprintPure, Category="RigVM")
	UE_API virtual bool CanExecute() const;

	/** Initialize things for the RigVM owner */
	UE_API virtual void Initialize(bool bRequestInit = true);

	/** Initialize this Host VM Instance */
	UE_API virtual bool InitializeVM(const FName& InEventName);

	/** Evaluate at Any Thread */
	UE_API virtual void Evaluate_AnyThread();

	/** Locks for the scope of Evaluate_AnyThread */
	FTransactionallySafeCriticalSection& GetEvaluateMutex() { return EvaluateMutex; };

	/** Lock for editing the event queue to run once */
	FTransactionallySafeCriticalSection& GetEventQueueToRunOnceMutex() const { return EventQueueToRunOnceMutex; };

	/** 
	 * Returns true if this host is read only - so if it's not support to be mutated / changed
	 */
	UE_API virtual bool IsReadOnly() const;
	
#if RIGVM_TRACE_ENABLED
	/**
	 * Returns true if this host is currently playing back a rewind debugger trace
	 */
	UE_API virtual bool IsPlayingRewindDebugTrace() const;

	/**
	 * Starts a potentially connected rewind debug trace for playback
	 */
	UE_API virtual void StartPlayingRewindDebugTrace();

	/**
	 * Stops a potentially connected rewind debug trace from playing back
	 */
	UE_API virtual void StopPlayingRewindDebugTrace();

#endif

	/**
	 * Returns the bound outer this host is nested under.
	 */
	UE_API virtual UObject* GetBoundOuterForTrace() const;
	
	/** Returns the member properties as an external variable array */
	UE_API TArray<FRigVMExternalVariable> GetExternalVariables() const;

	/** Returns the public member properties as an external variable array */
	UE_API TArray<FRigVMExternalVariable> GetPublicVariables() const;
	
	/** The struct where the variable properties are defined (for the BP-based host, this would be GetClass, for the BP-independent, it would be the property bag struct) */
	UE_API UStruct* GetVariablesStruct() const;
	
	/** The memory where the variables are stored (for the BP-based host, the memory is this, for the BP-independent, it would be the raw ptr of the property bag) */
	UE_API uint8* GetVariablesMemory();

	/** Returns a variable given its name */
	UE_API FRigVMExternalVariable GetVariableByName(const FName& InVariableName) const;
	
	/** Returns a public variable given its name */
	UE_API FRigVMExternalVariable GetPublicVariableByName(const FName& InVariableName) const;

	/** Returns the names of variables accessible in scripting */
	UFUNCTION(BlueprintPure, Category = "RigVM", meta=(DisplayName="Get Variables"))
	UE_API TArray<FName> GetScriptAccessibleVariables() const;

	/** Returns the type of a given variable */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API FName GetVariableType(const FName& InVariableName) const;

	/** Returns the value of a given variable as a string */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API FString GetVariableAsString(const FName& InVariableName) const;

	/** Returns the value of a given variable as a string */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API bool SetVariableFromString(const FName& InVariableName, const FString& InValue);

	template<class T>
	T GetPublicVariableValue(const FName& InVariableName)
	{
		FRigVMExternalVariable Variable = GetPublicVariableByName(InVariableName);
		if (Variable.IsValid())
		{
			return Variable.GetValue<T>();
		}
		return T();
	}

	template<class T>
	void SetPublicVariableValue(const FName& InVariableName, const T& InValue)
	{
		FRigVMExternalVariable Variable = GetPublicVariableByName(InVariableName);
		if (Variable.IsValid())
		{
			Variable.SetValue<T>(InValue);
		}
	}

	virtual FString GetName() const
	{
		FString ObjectName = (GetClass()->GetName());
		ObjectName.RemoveFromEnd(TEXT("_C"));
		return ObjectName;
	}

	UPROPERTY()
	FRigVMRuntimeSettings VMRuntimeSettings;

	UE_API virtual void InvalidateCachedMemory();

	// Regenerates cached handles after a structural change (i.e. new UUserStruct)
	UE_API virtual void RecreateCachedMemory();

	/** Execute */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API virtual bool Execute(const FName& InEventName);

#if WITH_EDITOR

	/** Bindable event for external objects to be notified that a RigVMHost is fully end-loaded*/
	DECLARE_EVENT_OneParam(URigVMHost, FOnEndLoadPackage, URigVMHost*);

	// these are needed so that sequencer can have a chance to update its 
	// RigVMHost instances after the package is fully end-loaded
	void BroadCastEndLoadPackage() { EndLoadPackageEvent.Broadcast(this); }
	FOnEndLoadPackage& OnEndLoadPackage() { return EndLoadPackageEvent; }

private:
	FOnEndLoadPackage EndLoadPackageEvent;

#endif

protected:

	/** Initialize the CDO VM */
	UE_API virtual bool InitializeCDOVM();

	/** ExecuteUnits */
	UE_API virtual bool Execute_Internal(const FName& InEventName);

	static UE_API bool DisableExecution();

#if RIGVM_TRACE_ENABLED
	UE_API virtual void TraceConstantData(FRigVMTraceArchiveWriter& InArchive) const;
public:
	UE_API virtual bool MatchesTracedConstantData(FRigVMTraceArchiveReader& InArchive) const;
protected:
	UE_API virtual void LoadTracedConstantData(FRigVMTraceArchiveReader& InArchive);
	UE_API virtual void TraceExecuteData(FRigVMTraceArchiveWriter& InArchive) const;
	UE_API virtual void LoadTracedExecuteData(FRigVMTraceArchiveReader& InArchive);
#endif

public:

	template<class T>
	bool SupportsEvent() const
	{
		return SupportsEvent(T::EventName);
	}

	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API virtual bool SupportsEvent(const FName& InEventName) const;

	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API virtual const TArray<FName>& GetSupportedEvents() const;

	/** Execute a user defined event */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API bool ExecuteEvent(const FName& InEventName);

	/** Requests to perform an init during the next execution */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API virtual void RequestInit();

	/** Returns true if this host requires the VM memory to be initialized */
	UFUNCTION(BlueprintPure, Category = "RigVM")
	virtual bool IsInitRequired() const
	{
		return bRequiresInitExecution;
	} 

	/** Requests to run an event once 
	 * @param InEventName The event to run
	 * @param InEventIndex Deprecated argument. Not used.
	 */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API virtual void RequestRunOnceEvent(const FName& InEventName, int32 InEventIndex = -1);

	/** Removes an event running once */
	UFUNCTION(BlueprintCallable, Category = "RigVM")
	UE_API bool RemoveRunOnceEvent(const FName& InEventName);

	/** Returns true if an event is queued to run once */
	UE_API bool IsRunOnceEvent(const FName& InEventName) const;

	/** Returns the queue of events to run */
	const TArray<FName>& GetEventQueue() const { return EventQueue; }

	/** Sets the queue of events to run */
	UE_API void SetEventQueue(const TArray<FName>& InEventNames);

	/** Provides the chance to a subclass to modify the event queue as needed */
	virtual void AdaptEventQueueForEvaluate(TArray<FName>& InOutEventQueueToRun) {}

	/** Update the settings such as array bound and log facilities */
	UE_API void UpdateVMSettings();

	UFUNCTION(BlueprintPure, Category = "RigVM")
	UE_API URigVM* GetVM();
	

#if WITH_EDITOR
	const FRigVMLog* GetLog() const { return RigVMLog; }
	FRigVMLog* GetLog() { return RigVMLog; }
	void SetLog(FRigVMLog* InLog) { RigVMLog = InLog; }
#endif

	// Returns the compiler generated VM memory storage by type
	UE_API virtual const FRigVMMemoryStorageStruct* GetDefaultMemoryByType(ERigVMMemoryType InMemoryType) const;

	// Returns an instanced VM memory storage by type
	UE_API virtual FRigVMMemoryStorageStruct* GetMemoryByType(ERigVMMemoryType InMemoryType);
	UE_API virtual const FRigVMMemoryStorageStruct* GetMemoryByType(ERigVMMemoryType InMemoryType) const;

	// The instanced mutable work memory
	FRigVMMemoryStorageStruct* GetWorkMemory() { return GetMemoryByType(ERigVMMemoryType::Work); }
	const FRigVMMemoryStorageStruct* GetWorkMemory() const { return GetMemoryByType(ERigVMMemoryType::Work); }

	// The default const literal memory
	FRigVMMemoryStorageStruct* GetLiteralMemory() { return GetMemoryByType(ERigVMMemoryType::Literal); }
	const FRigVMMemoryStorageStruct* GetLiteralMemory() const { return GetMemoryByType(ERigVMMemoryType::Literal); }

	// The instanced debug watch memory
	UE_API FRigVMMemoryStorageStruct* GetDebugMemory(bool bCreateIfRequired = false);
	const FRigVMMemoryStorageStruct* GetDebugMemory() const { return GetMemoryByType(ERigVMMemoryType::Debug); }

	DECLARE_EVENT_TwoParams(URigVM, FRigVMExecutedEvent, class URigVMHost*, const FName&);
	FRigVMExecutedEvent& OnInitialized_AnyThread() { return InitializedEvent; }
	FRigVMExecutedEvent& OnExecuted_AnyThread() { return ExecutedEvent; }
	FRigVMExecutedEvent& OnPreExecuted_AnyThread() { return PreExecutedEvent; }

	enum EEarlyExitReason
	{
		EEarlyExitReason_NoEvaluationsLeft
	};

	DECLARE_EVENT_TwoParams(URigVMHost, FRigVMHostEarlyExitEvent, class URigVMHost*, EEarlyExitReason);
	FRigVMHostEarlyExitEvent& OnEarlyExit_AnyThread() { return EarlyExitEvent; }

	const FRigVMDrawInterface& GetDrawInterface() const { return DrawInterface; };
	FRigVMDrawInterface& GetDrawInterface() { return DrawInterface; };

	const FRigVMDrawContainer& GetDrawContainer() const { return DrawContainer; };
	FRigVMDrawContainer& GetDrawContainer() { return DrawContainer; };

	UE_API void DrawIntoPDI(FPrimitiveDrawInterface* PDI, const FTransform& InTransform);
	UE_API void DrawIntoCanvas(class FCanvas* Canvas, const class FSceneView* View, const FTransform& InTransform);

	UE_API virtual USceneComponent* GetOwningSceneComponent(); 

	UE_API virtual void PostInitInstanceIfRequired();
	UE_API void SwapVMToNativizedIfRequired(UClass* InNativizedClass = nullptr);
	UE_API bool AreNativizedVMsEnabled() const;
	UE_API bool CanSwapVMToNativized() const;

#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if UE_RIGVM_DEBUG_EXECUTION
	bool bDebugExecutionEnabled = false;
	UE_API const FString GetDebugExecutionString();
#endif

	/** Provide access to the ExtendedExecuteContext */
	UE_DEPRECATED(5.4, "Please, use GetRigVMExtendedExecuteContext")
	UFUNCTION(BlueprintCallable, Category = RigVM, meta = (DeprecatedFunction, DeprecationMessage = "This function has been deprecated and it is no longer supported."))
	virtual FRigVMExtendedExecuteContext& GetExtendedExecuteContext()
	{
		static FRigVMExtendedExecuteContext DummyContext;
		return DummyContext;
	};

	UE_API void SetRigVMExtendedExecuteContext(FRigVMExtendedExecuteContext* InRigVMExtendedExecuteContext);

	inline FRigVMExtendedExecuteContext& GetRigVMExtendedExecuteContext()
	{
		check(RigVMExtendedExecuteContext != nullptr);
		return *RigVMExtendedExecuteContext;
	};
	inline const FRigVMExtendedExecuteContext& GetRigVMExtendedExecuteContext() const
	{
		check(RigVMExtendedExecuteContext != nullptr);
		return *RigVMExtendedExecuteContext;
	};

	UE_API UObject* ResolveUserDefinedTypeById(const FString& InTypeName) const;

	UE_API const TArray<FString>& GetRequiredPlugins() const;
	
	TScriptInterface<IRigVMRuntimeAssetInterface> GetGeneratedByAsset() const { return GeneratedBy; }

#if WITH_EDITOR
	UE_API void EnableNativizedVM(bool bEnable = true);
#endif
	
protected:

	UE_API virtual void PostInitInstance(URigVMHost* InCDO);
	UE_API virtual void PostInitProperties() override;
	
	UE_API URigVM* GetSourceVM() const;

	/** Current delta time */
	float DeltaTime;

	/** Current delta time */
	float AbsoluteTime;

	/** Current delta time */
	float FramesPerSecond;

	/** true if we should increase the AbsoluteTime */
	bool bAccumulateTime;

#if WITH_EDITOR
	/** true if the instance is being debugged in an asset editor */
	bool bIsBeingDebugged;

	/** true if this instance is allowed to use nativized VMs */
	bool bEnableNativizedVMs;
#endif

	mutable TOptional<uint32> LastNativizedVMHash;
	mutable TOptional<TWeakObjectPtr<UClass>> WeakNativizedVMClass;

#if RIGVM_TRACE_ENABLED
	/** true if this instance is playing back a rewind debug trace */
	bool bIsPlayingRewindDebugTrace;
#endif

	UPROPERTY(Transient)
	TObjectPtr<URigVM> VM;

	UPROPERTY()
	FRigVMPropertyBag Variables;

	UPROPERTY()
	TScriptInterface<IRigVMRuntimeAssetInterface> GeneratedBy;

#if WITH_EDITOR
	FRigVMLog* RigVMLog;
	bool bEnableLogging;
#endif

	UE_API void GenerateUserDefinedDependenciesData(FRigVMExtendedExecuteContext& Context);
	UE_API TArray<const UObject*> GetUserDefinedDependencies(const TArray<const FRigVMMemoryStorageStruct*> InMemory);
	UE_API virtual void GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& Context);

	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedStructGuidToPathName;
	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedEnumToPathName;
	UPROPERTY()
	TArray<FString> RequiredPlugins;

private:
	UPROPERTY(transient)
	TSet<TObjectPtr<UObject>> UserDefinedTypesInUse;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FRigVMExtendedExecuteContext ExtendedExecuteContext_DEPRECATED;
#endif
	
	FRigVMExtendedExecuteContext* RigVMExtendedExecuteContext = nullptr;

protected:
	
	UE_API void HandleExecutionReachedExit(const FName& InEventName);
	
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariablesImpl(bool bFallbackToBlueprint) const;
	
	FProperty* GetVariableProperty(const FName& InVariableName) const
	{
		FProperty* Property = nullptr;
		if (GeneratedBy)
		{
			Property = Variables.FindPropertyByName(InVariableName);
		}
		else
		{
			Property = GetClass()->FindPropertyByName(InVariableName);
		}
		return Property;
	}

	FProperty* GetPublicVariableProperty(const FName& InVariableName) const
	{
		FProperty* Property = GetVariableProperty(InVariableName);
		if (Property != nullptr)
		{
			if (!Property->IsNative())
			{
				if (!Property->HasAllPropertyFlags(CPF_DisableEditOnInstance))
				{
					return Property;
				}
			}
		}
		return nullptr;
	}
	
	FGuid GetVariableGuid(const FName& InVariableName) const
	{
		const TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
		const FRigVMExternalVariable* Found = ExternalVariables.FindByPredicate([InVariableName](const FRigVMExternalVariable& Variable)
			{
				return InVariableName == Variable.GetName();
			});
		if (Found)
		{
			return Found->GetGuid();
		}
		return FGuid();
	}
	
public:

	UPROPERTY()
	FRigVMDrawContainer DrawContainer;

	/** The draw interface for the units to use */
	UPROPERTY(transient)
	FRigVMDrawInterface DrawInterface;

	/** The event name used during an update */
	UPROPERTY(transient)
	TArray<FName> EventQueue;
	TArray<FName> EventQueueToRun;
	TArray<FName> EventsToRunOnce;

	/** Returns true if Evaluate_AnyThread is currently executing */
	bool IsEvaluating() const { return !EventQueueToRun.IsEmpty(); }

	/** Returns true if this host can run at least once */
	bool HasEvaluationsLeft() const
	{
		return EvaluationsLeft == INDEX_NONE || EvaluationsLeft > 0;
	}

	/**
	 * Returns the number of evaluations left for this host
	 * or INDEX_NONE if the host can run indefinitely.
	 */
	int32 GetEvaluationsLeft() const
	{
		return EvaluationsLeft;
	}

	/** Request to limit this hosts number of evaluations */
	void SetEvaluationsLeft(int32 InEvaluationsLeft)
	{
		EvaluationsLeft = FMath::Max<int32>(INDEX_NONE, InEvaluationsLeft);
	}

	/** Reset the number of evaluations left and allow this host to run indefinitely */
	void ResetEvaluationsLeft()
	{
		SetEvaluationsLeft(INDEX_NONE);
	}

	/** Copy the VM from the default object */
	UE_API void InstantiateVMFromCDO();
	
	/** Copy the default values of external variables from the default object */
	UE_API void CopyExternalVariableDefaultValuesFromCDO();
	
	/** Broadcasts a notification whenever the RigVMHost's memory is initialized. */
	FRigVMExecutedEvent InitializedEvent;

	/** Broadcasts a notification whenever the RigVMHost is executed / updated. */
	FRigVMExecutedEvent ExecutedEvent;

	/** Broadcasts a notification before the RigVMHost is executed / updated. */
	FRigVMExecutedEvent PreExecutedEvent;

	/** Broadcasts when the host cannot execute and needs to early exit during an evaluation */
	FRigVMHostEarlyExitEvent EarlyExitEvent;

protected: 

	UE_API virtual void InitializeFromCDO();

	UE_API virtual void CopyVMMemory(FRigVMExtendedExecuteContext& TargetContext, const FRigVMExtendedExecuteContext& SourceContext);

public:
	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	UE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

protected:
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = "Default")
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = "Default")
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;
#endif

protected:

	// the number of evaluations left. INDEX_NONE represents unlimited
	int32 EvaluationsLeft;

	bool bRequiresInitExecution;

	int32 InitBracket;
	int32 ExecuteBracket;

	TWeakObjectPtr<USceneComponent> OuterSceneComponent;

	bool IsInitializing() const
	{
		return InitBracket > 0;
	}

	bool IsExecuting() const
	{
		return ExecuteBracket > 0;
	}

private:

#if WITH_EDITORONLY_DATA

	/** The current execution mode */
	UPROPERTY(transient)
	bool bIsInDebugMode;

#endif
	
#if WITH_EDITOR	

protected:
	
	FRigVMInstructionVisitInfo InstructionVisitInfo;
	TSharedRef<FRigVMDebugInfo> DebugInfo;
	FRigVMProfilingInfo ProfilingInfo;
	TMap<FString, bool> LoggedMessages;
	UE_API void LogOnce(EMessageSeverity::Type InSeverity, int32 InInstructionIndex, const FString& InMessage);
	
public:

	void SetIsInDebugMode(const bool bValue) { bIsInDebugMode = bValue; }
	bool IsInDebugMode() const { return bIsInDebugMode; }
	bool IsProfilingEnabled() const
	{
		return IsInDebugMode() || VMRuntimeSettings.bEnableProfiling;
	}

	const FRigVMOperand* GetDebugOperandForOperand(const FRigVMOperand& InOperand) const
	{
		return GetRigVMExtendedExecuteContext().GetDebugOperandForOperand(InOperand);
	}
	
	FRigVMDebugInfo& GetDebugInfo() { return DebugInfo.Get(); }
	const FRigVMDebugInfo& GetDebugInfo() const { return DebugInfo.Get(); }

	FRigVMProfilingInfo& GetProfilingInfo() { return ProfilingInfo; }
	const FRigVMProfilingInfo& GetProfilingInfo() const { return ProfilingInfo; }

#endif	

private:

	FTransactionallySafeCriticalSection EvaluateMutex;
	mutable FTransactionallySafeCriticalSection EventQueueToRunOnceMutex;

protected:

#if UE_RIGVM_PROFILE_EXECUTE_UNITS_NUM
	int32 ProfilingRunsLeft;
	uint64 AccumulatedCycles;
#endif

	friend class URigVMBlueprint;
	friend class IRigVMEditorAssetInterface;
	friend class URigVMBlueprintGeneratedClass;
	friend class FRigVMCompileSettingsDetails;
	friend class FRigVMTrace;
	friend class URigVM;
	friend class URigVMRuntimeAsset;
	friend class UControlRigRuntimeAsset;
	friend class URigVMEditorAsset;
	friend class FRigVMExampleAssetActions;
	friend class FControlRigAssetActions;
};

class FRigVMBracketScope
{
public:

	FRigVMBracketScope(int32& InBracket)
		: Bracket(InBracket)
	{
		Bracket++;
	}

	~FRigVMBracketScope()
	{
		Bracket--;
	}

private:

	int32& Bracket;
};

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SceneManagement.h"
#endif
