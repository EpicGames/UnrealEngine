// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMModule.h: Module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "RigVMTrace.h"
#include "RigVMDefines.h"

#define UE_API RIGVM_API

RIGVM_API DECLARE_LOG_CATEGORY_EXTERN(LogRigVM, Log, All);

/**
* The public interface to this module
*/
class FRigVMModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
private:

#if RIGVM_TRACE_ENABLED
	FRigVMTrace RewindDebuggerRigVMExtension;
#endif
};

namespace RigVMCore
{
	RIGVM_API bool SupportsUObjects();
	RIGVM_API bool SupportsUInterfaces();
}

#undef UE_API
