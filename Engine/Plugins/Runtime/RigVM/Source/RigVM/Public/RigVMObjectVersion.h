// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API RIGVM_API

// Custom serialization version for changes made in Dev-Anim stream
struct FRigVMObjectVersion
{
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,
		
		// ControlRig & RigVMHost compute and checks VM Hash
		AddedVMHashChecks,

		// Predicates added to execute operations
		PredicatesAddedToExecuteOps,

		// Storing paths to user defined structs map
		VMStoringUserDefinedStructMap,

		// Storing paths to user defined enums map
		VMStoringUserDefinedEnumMap,

		// Storing paths to user defined enums map
		HostStoringUserDefinedData,

		// VM Memory Storage Struct serialized
		VMMemoryStorageStructSerialized,

		// VM Memory Storage Defaults generated at VM
		VMMemoryStorageDefaultsGeneratedAtVM,

		// VM Bytecode Stores the Public Context Path
		VMBytecodeStorePublicContextPath,

		// Removing unused tooltip property from frunction header
		VMRemoveTooltipFromFunctionHeader,

		// Removing library node FSoftObjectPath from FRigVMGraphFunctionIdentifier
		RemoveLibraryNodeReferenceFromFunctionIdentifier,

		// Adding variant struct to function identifier
		AddVariantToFunctionIdentifier,

		// Adding variant to every RigVM asset
		AddVariantToRigVMAssets,

		// Storing user interface layout within function header
		FunctionHeaderStoresLayout,

		// Storing user interface relevant pin index in category
		FunctionHeaderLayoutStoresPinIndexInCategory,

		// Storing user interface relevant category expansion
		FunctionHeaderLayoutStoresCategoryExpansion,

		// Storing function graph collapse node content as part of the header
		RigVMSaveSerializedGraphInGraphFunctionDataAsByteArray,

		// VM Bytecode Stores the Public Context Path as a FTopLevelAssetPath 
     	VMBytecodeStorePublicContextPathAsTopLevelAssetPath,

		// Serialized instruction offsets are now int32 rather than uint16, NumBytes has been removed
		// from RigVMCopyOp
		ByteCodeCleanup,

		// The VM stores a local snapshot registry to use in cooked environments instead of the shared global registry
		LocalizedRegistry,

		// The VM stores a relative seek offset to be able to skip the registry during load
		LocalizedRegistryWithRelativeSeekOffset,

		// Function arguments can now represent an input variable (an external variable passed into a function)
		FunctionArgumentCanRepresentInputVariable,

		// Object archive is now storing the version container
		ObjectArchiveVersionContainerSerialization,

		// Debug operand mapping simplified and moved to context only
		DebugOperandMappingSimplified,

		// Introduction of callables to the rigvm bytecode
		RigVMCallables,
		
		// Referencing variables through Guids
		GuidForVariables,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FRigVMObjectVersion() {}
};

// This is used to version up cooked assets independent of the object version. 
namespace ERigVMCookVersion
{
	enum Type : int
	{
		// even when the localized registry is turned off,
		// make sure to always serialize it so that we can decide
		// in code if we want to utilize it.
		AlwaysSerializeLocalizedRegistry,
		
		LatestVersion
	};
}
#undef UE_API
