// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/UnrealType.h"


namespace RigVMPropertyUtils
{

/**
 * Given a property, return the C++ type name for it and the associated type object. If the type is an unambiguous
 * simple type, such as a bool or an int, the OutTypeObject value will be nullptr. If the property represents a
 * ambiguous type, like an object pointer, interface or an enum, then the OutTypeObject will be the class of the
 * pointed to object, or the enum definition. For container properties, the type object is always the contained type,
 * rather than the type of the container.
 * \param InProperty The property to get the type for.
 * \param OutTypeName The C++ type name.
 * \param OutTypeObject The object class, if the property type is complex or ambiguous. 
 */
RIGVM_API void GetTypeFromProperty(const FProperty* InProperty, FName& OutTypeName, UObject*& OutTypeObject);

/**
 * Given a property and a value storage for it, return a 32-bit hash value for the underlying value of tha type.
 * \note The resulting hash is not suitable for serialization, since the underlying hash combine function is not 
 *    guaranteed to be stable between releases.
 * \param InProperty The property to get a hash value for.
 * \param InMemory The underlying memory that the property is defined for.
 * \param InContainerType The type of memory being pointed at. For UObjects, use EPropertyPointerType::Container.
 *    If pointing directly at the value location in memory, use EPropertyPointerType::Direct.
 * \return The hash value of the value of the property at the memory location the property lives at.
 */
RIGVM_API uint32 GetPropertyHashFast(const FProperty* InProperty, const uint8* InMemory, EPropertyPointerType InContainerType = EPropertyPointerType::Container);

/**
 * Given a property and a value storage for it, return a 32-bit hash value for the underlying value of tha type.
 * \note This hash is suitable for serialization, since the underlying hash combine function is guaranteed to be stable
 *    between releases.
 * \param InProperty The property to get a hash value for.
 * \param InMemory The underlying UObject memory that the property is defined for.
* \param InContainerType The type of memory being pointed at. For UObjects, use EPropertyPointerType::Container.
 *    If pointing directly at the value location in memory, use EPropertyPointerType::Direct.
 * \return The hash value of the value of the property at the memory location the property lives at.
 */
RIGVM_API uint32 GetPropertyHashStable(const FProperty* InProperty, const uint8* InMemory, EPropertyPointerType InContainerType = EPropertyPointerType::Container);

}
