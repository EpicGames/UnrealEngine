// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMHost.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMCore/RigVMVariableDescription.h"

#include "RigVMRuntimeAsset.generated.h"

#define UE_API RIGVM_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRigVMPreVariablesChanged,  const FName);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRigVMPostVariablesChanged, const FName);


/**
 * Enumerates states a RigVMAsset can be in.
 * Copied from EBlueprintStatus.
 */
UENUM()
enum ERigVMAssetStatus : int
{
	/** Asset is in an unknown state. */
	RVMA_Unknown,
	/** Asset has been modified but not recompiled. */
	RVMA_Dirty,
	/** Asset tried but failed to be compiled. */
	RVMA_Error,
	/** Asset has been compiled since it was last modified. */
	RVMA_UpToDate,
	/** Asset is in the process of being created for the first time. */
	RVMA_BeingCreated,
	/** Asset has been compiled since it was last modified. There are warnings. */
	RVMA_UpToDateWithWarnings,
	RVMA_MAX,
};

UINTERFACE(MinimalAPI, BlueprintType)
class URigVMRuntimeAssetInterface : public UInterface
{
	GENERATED_BODY()
};

class IRigVMRuntimeAssetInterface
{
	GENERATED_BODY()

protected:
	mutable TArray<TWeakObjectPtr<UObject>> ArchetypeInstances;


public:
	UE_API static TScriptInterface<IRigVMRuntimeAssetInterface> GetInterfaceOuter(const UObject* InObject);

#if WITH_EDITORONLY_DATA
	virtual UObject* GetEditorOnlyData() = 0;
	virtual const UObject* GetEditorOnlyData() const = 0;
#endif // WITH_EDITORONLY_DATA

	virtual UClass* GetRigVMHostClass() const = 0;
	virtual URigVM* GetVM() = 0;

	virtual TArray<FRigVMExternalVariable> GetExternalVariables() = 0;
	virtual const UStruct* GetVariablesStruct() = 0;
	virtual uint8* GetVariablesMemory() = 0;
	virtual const uint8* GetVariablesMemory() const = 0;
	virtual FProperty* FindGeneratedPropertyByName(const FName& InName) = 0;
	virtual const FProperty* FindGeneratedPropertyByName(const FName& InName) const { return const_cast<IRigVMRuntimeAssetInterface*>(this)->FindGeneratedPropertyByName(InName); }
	
	virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() = 0;
	virtual FRigVMVariant& GetAssetVariant() = 0;
	virtual const FRigVMVariant& GetAssetVariant() const { return const_cast<IRigVMRuntimeAssetInterface*>(this)->GetAssetVariant(); }
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName() = 0;
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName() = 0;
	virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse() = 0;
	virtual TArray<FName>& GetSupportedEventNames() = 0;
	virtual void UpdateSupportedEventNames() = 0;
	virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeDerivedClass) const = 0;
	virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() = 0;
	virtual void RemoveInstance(UObject* InInstance) = 0;
	virtual URigVMHost* InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags) = 0;
	virtual bool InitializeInstance(URigVMHost* InInstance) = 0;
	virtual bool InitializeVariables(URigVMHost* InInstance) = 0;
	virtual bool IsObjectInstanceOfAsset(URigVMHost* InInstance) = 0;
	virtual FRigVMDrawContainer& GetDrawContainer() = 0;
	const FRigVMDrawContainer& GetDrawContainer() const { return const_cast<IRigVMRuntimeAssetInterface*>(this)->GetDrawContainer(); }
#if WITH_EDITOR
	virtual FRigVMDebugInfo& GetDebugInfo() = 0;
#endif
	virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const = 0;
	

	friend class IRigVMEditorAssetInterface;

};

UCLASS(MinimalAPI)
class URigVMRuntimeAsset : public UObject, public IRigVMRuntimeAssetInterface, public IRigVMGraphFunctionHost
{
	GENERATED_BODY()

protected:

	virtual FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() override { return &GraphFunctionStore; }
	virtual const FRigVMGraphFunctionStore* GetRigVMGraphFunctionStore() const override { return &GraphFunctionStore; }

	UE_API virtual void PostLoad() override;
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	
	UE_API virtual void MarkInstancesAsGarbage();

	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual UClass* GetRigVMHostClass() const override { return URigVMHost::StaticClass(); }
	virtual URigVM* GetVM() override { return VM; }

	// Variables Begin
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables() override;
	virtual const UStruct* GetVariablesStruct() override { return Variables.GetPropertyBagStruct(); }
	virtual uint8* GetVariablesMemory() override { return Variables.GetMutableValue().GetMemory(); }
	virtual const uint8* GetVariablesMemory() const override { return Variables.GetValue().GetMemory(); }
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual TArray<FRigVMGraphVariableDescription> GetAssetVariables() const;
	UE_API virtual FRigVMGraphVariableDescription FindAssetVariable(const FName& InName) const;
	UE_API int32 GetVariableIndex(const FName& InName) const;
	UE_API int32 GetVariableIndex(const FGuid& InGuid) const;
#if WITH_EDITOR
	FOnRigVMPreVariablesChanged OnPreVariablesChangedDelegate;
	FOnRigVMPostVariablesChanged OnPostVariablesChangedDelegate;
	virtual FOnRigVMPreVariablesChanged& OnPreVariablesChanged() { return OnPreVariablesChangedDelegate; }
	virtual FOnRigVMPreVariablesChanged& OnPostVariablesChanged() { return OnPostVariablesChangedDelegate; }
	bool bIsModifyingVariables = false;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool RemoveMemberVariable(const FName& InName);
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool BulkRemoveMemberVariables(const TArray<FName>& InNames);
	UE_API virtual bool RenameMemberVariable(const FName& InOldName, const FName& InNewName);
	UE_API virtual bool ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue);
	UE_API bool SetVariableIndex(const FName& InName, int32 NewIndex);
	UE_API virtual bool SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex);
	UE_API virtual FText GetVariableTooltip(const FName& InName) const;
	UE_API virtual bool SetVariableTooltip(const FName& InName, const FText& InTooltip);
	UE_API virtual FString GetVariableCategory(const FName& InName) const;
	UE_API virtual bool SetVariableCategory(const FName& InName, const FString& InCategory);
	UE_API virtual FString GetVariableMetadataValue(const FName& InName, const FName& InKey) const;
	UE_API virtual bool SetVariableMetadataValue(const FName& InName, const FName& InKey, const FString& InValue);
	UE_API virtual bool RemoveVariableMetadataValue(const FName& InName, const FName& InKey);
	UE_API virtual bool SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn);
	UE_API virtual bool SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics);
	UE_API virtual bool SetVariablePrivate(const FName& InName, const bool bInPrivate);
	UE_API virtual bool SetVariablePublic(const FName& InName, const bool bIsPublic);
	UE_API virtual FName AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue = FString());
	virtual FString GetVariableDefaultValue(const FName& InName) const { return FindAssetVariable(InName).DefaultValue; }
#endif
	// Variables End

	virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() override { return RuntimeSettings; }
	virtual FRigVMVariant& GetAssetVariant() override { return AssetVariant; }
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName() override { return UserDefinedStructGuidToPathName; }
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName() override { return UserDefinedEnumToPathName; }
	virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse() override { return UserDefinedTypesInUse; }
	UE_API virtual void UpdateSupportedEventNames() override;
	virtual ERigVMAssetStatus GetAssetStatus() const { return Status; }
	virtual void SetAssetStatus(const ERigVMAssetStatus& InStatus) { Status = InStatus; }
	UE_API virtual bool IsUpToDate() const;
	virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() override { return &ExecuteContext; }
	UE_API virtual void MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(nullptr));
	UE_API virtual void MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus);
	UE_API virtual const TArray<FString>& GetRequiredPlugins(bool bRefresh = true) const;
	UE_API virtual void GenerateRequiredPluginsData(FRigVMExtendedExecuteContext& InContext);
	virtual FRigVMPropertyBag* GetVariablesPropertyBag() { return &Variables; }
	virtual FRigVMDrawContainer& GetDrawContainer() override { return DrawContainer; }
#if WITH_EDITOR
	virtual FRigVMDebugInfo& GetDebugInfo() override { return DebugInfo; }
#endif
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;

public:
	UE_API virtual FProperty* FindGeneratedPropertyByName(const FName& InName) override;
	virtual TArray<FName>& GetSupportedEventNames() override { return SupportedEventNames; }
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Initialize(UClass* InEditorOnlyDataClass);
	UE_API virtual void RemoveInstance(UObject* InInstance) override;
	UE_API virtual URigVMHost* InstantiateObject(UObject* InOuter, FName InName, EObjectFlags InFlags) override;
	UE_API virtual bool InitializeInstance(URigVMHost* InInstance) override;
	UE_API virtual bool InitializeVariables(URigVMHost* InInstance) override;
	virtual bool IsObjectInstanceOfAsset(URigVMHost* InInstance) override { return InInstance->GeneratedBy == this; }
	UE_API virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeDerivedClass) const override;

#if WITH_EDITORONLY_DATA
	virtual UObject* GetEditorOnlyData() override { return EditorAsset.Get();}
	virtual const UObject* GetEditorOnlyData() const override { return EditorAsset.Get();}

	// Transient storage for pre-rename asset paths, used by EditorAsset::PostRename
	// These are set before EditorAsset->Rename() and cleared after
	FString PreRenameOldAssetPath;
	FString PreRenameNewAssetPath;
#endif
	
#if WITH_EDITOR
	// This is the reflection for python usage, we cannot add the UFUNCTION in the WITH_EDITORONLY_DATA section
	UFUNCTION(BlueprintCallable, Category="VM")
	virtual UObject* GetEditorAsset() { return EditorAsset.Get();}
#endif
	
	virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) { return CurrentObjectBeingDebugged.Get(bEvenIfPendingKill); }
	virtual const FString& GetObjectBeingDebuggedPath() const { return ObjectPathToDebug; }
	virtual void SetObjectBeingDebuggedPath(const FString& InPath) { ObjectPathToDebug = InPath; }
	virtual void SetObjectBeingDebugged(UObject* InObject) { CurrentObjectBeingDebugged = InObject; }
	virtual UWorld* GetWorldBeingDebugged() const { return CurrentWorldBeingDebugged.Get(); }
	virtual void SetWorldBeingDebugged(UWorld* NewWorld) { CurrentWorldBeingDebugged = NewWorld; }
	
#if WITH_EDITORONLY_DATA
	virtual TArray<FRigVMReferenceNodeData>& GetFunctionReferenceNodeData() { return FunctionReferenceNodeData; }
#endif
	virtual const TArray<FRigVMGraphFunctionHeader>& GetPublicGraphFunctions() const { return PublicGraphFunctions; }
	virtual void SetPublicGraphFunctions(const TArray<FRigVMGraphFunctionHeader>& InHeaders) { PublicGraphFunctions = InHeaders; }

protected:

	UPROPERTY()
	TObjectPtr<URigVM> VM;

	UPROPERTY()
	FRigVMExtendedExecuteContext ExecuteContext;

	UPROPERTY()
	FRigVMPropertyBag Variables;

	UPROPERTY()
	FRigVMGraphFunctionStore GraphFunctionStore;

	/** Asset searchable information about exposed public functions on this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FRigVMGraphFunctionHeader> PublicGraphFunctions;

	UPROPERTY(EditAnywhere, Category = "VM")
	FRigVMRuntimeSettings RuntimeSettings;

	UPROPERTY()
	FRigVMDrawContainer DrawContainer;

	/** Variant information about this asset */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = "Variant")
	FRigVMVariant AssetVariant;

	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedStructGuidToPathName;
	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedEnumToPathName;
	UPROPERTY(transient)
	TSet<TObjectPtr<UObject>> UserDefinedTypesInUse;

	/** The event names this rigvm blueprint contains */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FName> SupportedEventNames;

	/** The current status of this blueprint */
	UPROPERTY(transient)
	TEnumAsByte<ERigVMAssetStatus> Status;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UObject> EditorAsset; // The outer of this object is the runtime asset
	
	/** Asset searchable information function references in this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FRigVMReferenceNodeData> FunctionReferenceNodeData;

#endif

	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Dependencies")
	TArray<FString> RequiredPlugins;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = "Default")
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

#if WITH_EDITORONLY_DATA
	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = "Default")
	TArray<TObjectPtr<UAssetUserData>> AssetUserDataEditorOnly;
#endif
	
	/** Current object being debugged for this blueprint */
	TWeakObjectPtr< UObject > CurrentObjectBeingDebugged;

	/** Raw path of object to be debugged, this might have been spawned inside a specific PIE level so is not stored as an object path type */
	FString ObjectPathToDebug;

	/** Current world being debugged for this blueprint */
	TWeakObjectPtr< class UWorld > CurrentWorldBeingDebugged;
	
	FRigVMDebugInfo DebugInfo;


	friend class URigVMBlueprint;
	friend class FRigVMAssetVariablesChangeScope;
	friend struct FControlRigAssetStrongReference;
	friend class URigVMEditorAsset;
	friend class FRigVMEditorModule;
};

#if WITH_EDITOR
class FRigVMAssetVariablesChangeScope
{
public:
   
	UE_API FRigVMAssetVariablesChangeScope(URigVMRuntimeAsset* InAsset, const FName InVariableName);

	UE_API ~FRigVMAssetVariablesChangeScope();

private:

	const FName VariableName;
	URigVMRuntimeAsset* Asset;
};
#endif

#undef UE_API