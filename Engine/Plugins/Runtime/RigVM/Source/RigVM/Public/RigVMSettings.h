// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMSettings.h: Declares the RigVMEditorSettings class.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "RigVMCore/RigVMVariant.h"

#include "RigVMSettings.generated.h"

class UStaticMesh;

/**
 * Determines when to save RigVM assets after compilation.
 */
UENUM()
enum class ERigVMSaveOnCompile : uint8
{
	Never UMETA(DisplayName="Never"),
	SuccessOnly UMETA(DisplayName="On Success Only"),
	Always UMETA(DisplayName = "Always"),
};

/**
 * Customize RigVM Rig Editor.
 */
UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="RigVM Editor"), MinimalAPI)
class URigVMEditorSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return TEXT("ContentEditors"); }
	
#if WITH_EDITORONLY_DATA

	// Determines when the asset should be saved after compilation.
	// This applies to RigVM assets that are not blueprint-based (e.g., Control Rig Runtime Assets).
	UPROPERTY(EditAnywhere, config, Category = Compiler)
	ERigVMSaveOnCompile SaveOnCompile = ERigVMSaveOnCompile::Never;

	// with this checked similar nodes in the graph will be highlighted.
	// this highlighting can then also be used to functionize items
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bHighlightSimilarNodes;

	// fade out unrelated nodes
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bFadeOutUnrelatedNodes;

	// use a flash light to brighten up nodes
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bUseFlashLight;

	// When this is checked mutable nodes (nodes with an execute pin)
	// will be hooked up automatically.
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bAutoLinkMutableNodes;

	/** 
	 * When enabled:
	 * - Left Mouse Button drag drags nodes
	 * - Ctrl Left Mouse Button drag creates connections
	 * - Double-clicking near a spline in the Graph creates a reroute node
	 */
	UPROPERTY(EditAnywhere, config, Category = Interaction)
	bool bDirectRerouteNodeEditing = false;

	/** When the Blueprint graph context menu is invoked (e.g. by right-clicking in the graph or dragging off a pin), do not block the UI while populating the available actions list. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (DisplayName = "Enable Non-Blocking Context Menu"))
	bool bEnableContextMenuTimeSlicing;

	/**
	 * When right-clicking in a RigVM based graph (for example Control Rig)
	 * enabling this option will apply a fuzzy string search to find items
	 */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (DisplayName = "Enable fuzzy search in action menu"))
	bool bEnableFuzzySearchInActionMenu;

	/**
	 * Fuzzy searches will return a weight between 0.0 and 100.0. 
	 * By specifying the minimum weight you can remove unlikely matches.
	 */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (DisplayName = "Minimum Fuzzy Score", UIMin = "0.0", UIMax = "100.0"))
	float MinimumFuzzyScore;
#endif
};


UCLASS(config = Editor, meta=(DisplayName="RigVM Project Settings"), MinimalAPI)
class URigVMProjectSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, config, Category = Variants)
	TArray<FRigVMTag> VariantTags;

	UFUNCTION(BlueprintPure, Category= Variants)
	RIGVM_API FRigVMTag GetTag(FName InTagName) const;
	RIGVM_API const FRigVMTag* FindTag(FName InTagName) const;
};