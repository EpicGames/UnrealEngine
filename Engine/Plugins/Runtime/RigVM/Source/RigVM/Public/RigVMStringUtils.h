// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace RigVMStringUtils
{
	// Joins two strings given a separator 
	RIGVM_API FString JoinStrings(const FString& InStringA, const FString& InStringB, const TCHAR* InSeparator, const TCHAR* InPrefix = nullptr, const TCHAR* InSuffix = nullptr);

	// Joins a series of strings given a separator 
	RIGVM_API FString JoinStrings(const TArray<FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix = nullptr, const TCHAR* InSuffix = nullptr);
	RIGVM_API FString JoinStrings(const TArrayView<FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix = nullptr, const TCHAR* InSuffix = nullptr);
	RIGVM_API FString JoinStringsConst(const TArrayView<const FString>& InStrings, const TCHAR* InSeparator, const TCHAR* InPrefix = nullptr, const TCHAR* InSuffix = nullptr);

	// Splits a string up into its parts
	RIGVM_API bool SplitString(const FString& InString, const TCHAR* InSeparator, TArray<FString>& InOutParts);

	// Creates a template notation such as "MyTemplate(float,bool,float)"
	RIGVM_API FString MakeTemplateNotation(const FString& InTemplateName, const TArray<FString>& InArgumentNotations);

	// Splits a NodePath at the start, so for example "CollapseNodeA|CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA" and "CollapseNodeB|CollapseNodeC"
	RIGVM_API bool SplitNodePathAtStart(const FString& InNodePath, FString& LeftMost, FString& Right);

	// Splits a NodePath at the end, so for example "CollapseNodeA|CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA|CollapseNodeB" and "CollapseNodeC"
	RIGVM_API bool SplitNodePathAtEnd(const FString& InNodePath, FString& Left, FString& RightMost);

	// Splits a NodePath into all segments, so for example "Node.Color.R" becomes ["Node", "Color", "R"]
	RIGVM_API bool SplitNodePath(const FString& InNodePath, TArray<FString>& Parts);

	// Joins a NodePath from to segments, so for example "CollapseNodeA" and "CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA|CollapseNodeB|CollapseNodeC"
	RIGVM_API FString JoinNodePath(const FString& Left, const FString& Right);

	// Joins a NodePath from to segments, so for example ["CollapseNodeA", "CollapseNodeB", "CollapseNodeC"] becomes "CollapseNodeA|CollapseNodeB|CollapseNodeC"
	RIGVM_API FString JoinNodePath(const TArray<FString>& InParts);

	// Splits a PinPath at the start, so for example "Node.Color.R" becomes "Node" and "Color.R"
	RIGVM_API bool SplitPinPathAtStart(const FString& InPinPath, FString& LeftMost, FString& Right);
	RIGVM_API bool SplitPinPathAtStart(FStringView InPinPath, FStringView& LeftMost, FStringView& Right);

	// Splits a PinPath at the start, so for example "Node.Color.R" becomes "Node.Color" and "R"
	RIGVM_API bool SplitPinPathAtEnd(const FString& InPinPath, FString& Left, FString& RightMost);

	// Splits a PinPath into all segments, so for example "Node.Color.R" becomes ["Node", "Color", "R"]
	RIGVM_API bool SplitPinPath(const FString& InPinPath, TArray<FString>& Parts);

	// Joins a PinPath from to segments, so for example "Node.Color" and "R" becomes "Node.Color.R"
	RIGVM_API FString JoinPinPath(const FString& Left, const FString& Right);

	// Joins a PinPath from to segments, so for example ["Node", "Color", "R"] becomes "Node.Color.R"
	RIGVM_API FString JoinPinPath(const TArray<FString>& InParts);

	// Joins the name value pairs into a single default value string
	RIGVM_API FString JoinDefaultValue(const TArray<FString>& InParts);

	// Splits the default value into name-value pairs
	RIGVM_API TArray<FString> SplitDefaultValue(const FString& InDefaultValue);

	// Sanitizes a name as per ruleset
	RIGVM_API void SanitizeName(FString& InOutName, bool bAllowPeriod, bool bAllowSpace, int32 InMaxNameLength);
}