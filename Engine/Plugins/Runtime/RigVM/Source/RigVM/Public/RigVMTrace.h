// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Trace/Config.h"
#include "TraceFilter.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "RigVMTraceArchive.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

#define UE_API RIGVM_API

#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING && !UE_BUILD_TEST
#define RIGVM_TRACE_ENABLED 1
#else
#define RIGVM_TRACE_ENABLED 0
#endif

class UObject;
struct FRigVMExtendedExecuteContext;
struct FRigVMMemoryStorageStruct;

#if RIGVM_TRACE_ENABLED

UE_API UE_TRACE_CHANNEL_EXTERN(RigVMChannel)

/**
 * Base structure for all RigVM trace structures
 */
struct FRigVMTraceBaseData
{
	FRigVMTraceBaseData()
		: Cycles(0)
		, ProfileTime(0)
		, RecordingTime(0)
		, RigVMObjectVersion(0)
	{
	}

	// Cycles the frame took
	uint64 Cycles;

	// The time of the sample at machine time
	double ProfileTime;
	// The time of the sample at internal time of the host
	double RecordingTime;

	// The version used (FRigVMObjectVersion)
	int32 RigVMObjectVersion;
};

UE_API UE_TRACE_EVENT_BEGIN_EXTERN(RigVM, Literals)
	UE_TRACE_EVENT_FIELD(uint64, Cycles)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(int32, RigVMObjectVersion)
	UE_TRACE_EVENT_FIELD(uint64, HostId)
	UE_TRACE_EVENT_FIELD(uint64, BoundOuterId)
	UE_TRACE_EVENT_FIELD(uint32, VMHash)
	UE_TRACE_EVENT_FIELD(uint32, ByteCodeHash)
	UE_TRACE_EVENT_FIELD(uint8[], LiteralMemory)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, HostGeneratedBy)
	UE_TRACE_EVENT_FIELD(uint8[], HostConstantData)
UE_TRACE_EVENT_END()

/**
 * The constant data stored in the stream only once
 * when tracing a RigVMHost
 */
struct FRigVMTraceConstantData : FRigVMTraceBaseData
{
	FRigVMTraceConstantData()
		: FRigVMTraceBaseData()
		, HostId(0)
		, BoundOuterId(0)
		, VMHash(0)
		, ByteCodeHash(0)
	{
	}
	
	// The object id of the traced RigVM Host
	uint64 HostId;
	// The object id of the actor / component this host owned by
	uint64 BoundOuterId;
	// The hash of the traced VM
	uint32 VMHash;
	// The hash of the traced bytecode
	uint32 ByteCodeHash;
	// An archive storing the literal memory
	FRigVMTraceArchive LiteralMemory;
	// An archive storing auxiliary constant data from the host
	FRigVMTraceArchive HostConstantData;
};

UE_API UE_TRACE_EVENT_BEGIN_EXTERN(RigVM, Execute)
	UE_TRACE_EVENT_FIELD(uint64, Cycles)
	UE_TRACE_EVENT_FIELD(double, RecordingTime)
	UE_TRACE_EVENT_FIELD(int32, RigVMObjectVersion)
	UE_TRACE_EVENT_FIELD(double, AbsoluteTime)
	UE_TRACE_EVENT_FIELD(double, DeltaTime)
	UE_TRACE_EVENT_FIELD(uint64, WorldId)
	UE_TRACE_EVENT_FIELD(uint64, HostId)
	UE_TRACE_EVENT_FIELD(uint8[], WorkMemory)
	UE_TRACE_EVENT_FIELD(uint8[], DebugMemory)
	UE_TRACE_EVENT_FIELD(uint8[], ExternalMemory)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Entries)
	UE_TRACE_EVENT_FIELD(int32[], VisitedInstructions)
	UE_TRACE_EVENT_FIELD(uint64, OverallCycles)
	UE_TRACE_EVENT_FIELD(uint64[], InstructionCycles)
	UE_TRACE_EVENT_FIELD(uint8[], DrawInterface)
	UE_TRACE_EVENT_FIELD(uint8[], HostExecuteData)
UE_TRACE_EVENT_END()

/**
 * The data to be traced per frame for a RigVMHost.
 */
struct FRigVMTraceExecuteData : FRigVMTraceBaseData
{
	FRigVMTraceExecuteData()
		: FRigVMTraceBaseData()
		, AbsoluteTime(0)
		, DeltaTime(0)
		, WorldId(0)
		, HostId(0)
		, OverallCycles(0)
	{
	}

	// The sample's absolute time of the host
	double AbsoluteTime;
	// The sample's delta time of the host
	double DeltaTime;
	// The id of the world this host belongs to
	uint64 WorldId;
	// The id of the host
	uint64 HostId;
	// An archive storing the work memory of the VM
	FRigVMTraceArchive WorkMemory;
	// An archive storing the debug memory of the VM
	FRigVMTraceArchive DebugMemory;
	// An archive storing the external memory (variables) of the VM
	FRigVMTraceArchive ExternalMemory;
	// The names of the entries being executed
	FString Entries;
	// The instructions visited during this sample
	TArray<int32> VisitedInstructions;
	// The overall duration in cycles
	uint64 OverallCycles;
	// The cycles spent in each instruction
	TArray<uint64> InstructionCycles;
	// The content of the debug draw interface
	FRigVMTraceArchive DrawInterface;
	// Auxiliary data provided by the host's virtual TraceExecuteData
	FRigVMTraceArchive HostExecuteData;
};

/**
 * The base service to allow tracing of RigVM host instances
 */
class FRigVMTrace : public IRewindDebuggerRuntimeExtension
{
public:
	UE_API virtual void RecordingStarted() override;
	UE_API virtual void RecordingStopped() override;

	static UE_API void SetupRigVMEvaluation(FRigVMExtendedExecuteContext& InContext);
	static UE_API void TraceRigVMEvaluation(const FRigVMExtendedExecuteContext& InContext);
	static UE_API bool RestoreRigVMConstantData(FRigVMExtendedExecuteContext& InOutContext, const FRigVMTraceConstantData& InConstantData);
	static UE_API bool RestoreRigVMExecuteData(FRigVMExtendedExecuteContext& InOutContext, const FRigVMTraceExecuteData& InExecuteData);
};

#define TRACE_SETUP_RIGVM_EVALUATION(InContext) \
	FRigVMTrace::SetupRigVMEvaluation(InContext)

#define TRACE_RIGVM_EVALUATION(InContext) \
	FRigVMTrace::TraceRigVMEvaluation(InContext)

#else

#define TRACE_SETUP_RIGVM_EVALUATION(...)
#define TRACE_RIGVM_EVALUATION(...)

#endif

#undef UE_API
