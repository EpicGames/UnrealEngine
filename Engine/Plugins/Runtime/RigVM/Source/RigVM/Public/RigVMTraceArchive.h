// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Serialization/ArchiveUObject.h"

#define UE_API RIGVM_API

/**
 * The trace archive is used to store compressed data for a RewindDebugger stream.
 */
struct FRigVMTraceArchive
{
public:
	UE_API FRigVMTraceArchive();

	struct FHeader
	{
		bool bIsCompressed = false;
		int32 UncompressedSize = INDEX_NONE;
		int32 CompressedSize = INDEX_NONE;
		int32 NumVersions = INDEX_NONE;
		int64 VersionsOffset = INDEX_NONE;
	};
	
	UE_API void Reset();
	UE_API void Empty();
	UE_API bool IsPayloadEmpty() const;
	UE_API bool Compress();
	UE_API bool Decompress();

	UE_API const FHeader& GetHeader() const;
	UE_API bool IsCompressed() const;
	UE_API int32 GetUncompressedSize() const;
	UE_API int32 GetCompressedSize() const;

	UE_API uint8* GetPayloadData();
	UE_API const uint8* GetPayloadData() const;
	UE_API int32 PayloadNum() const;

	UE_API uint8* GetOverallData();
	UE_API const uint8* GetOverallData() const;
	UE_API int32 OverallNum() const;

	UE_API friend FArchive& operator<<(FArchive& Ar, FRigVMTraceArchive& Data);

protected:

	UE_API void SetOverallBuffer(const TArrayView<const uint8>& InBuffer);
	UE_API void SetOverallBuffer(const TArrayView<uint8>& InBuffer);

private:

	TArray<uint8, TSizedDefaultAllocator<32>> Buffer;

	FHeader& Header();

	friend class FRigVMTraceArchiveWriter;
	friend class FRigVMTraceArchiveReader;
	friend class FRigVMTraceAnalyzer;
};

/**
 * The trace archive writer is a wrapping archive used to write into
 * a trace archive.
 */
class FRigVMTraceArchiveWriter : public FArchiveUObject
{
public:
	UE_API FRigVMTraceArchiveWriter( FRigVMTraceArchive& InOutArchive );
	UE_API ~FRigVMTraceArchiveWriter();
	UE_API virtual void Serialize( void* V, int64 Length ) override;
	UE_API virtual int64 Tell() override;
	UE_API virtual int64 TotalSize() override;
	UE_API virtual void Seek(int64 InPos) override;
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	UE_API void WriteUObject(const UObject* Obj);
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;

protected:

	void WriteVersions();
	
	FRigVMTraceArchive& Archive;
	int64 Offset;
	TMap<FName,int64> NameToOffset;
	TMap<const UObject*, uint64> ObjectIdMap;

	static inline constexpr uint8 StoringStringAsString = 0;
	static inline constexpr uint8 StoringStringAsOffset = 1;
};

/**
 * The trace archive reader is a wrapping archive used to read from
 * a trace archive.
 */
class FRigVMTraceArchiveReader : public FRigVMTraceArchiveWriter
{
public:
	
	UE_API FRigVMTraceArchiveReader( FRigVMTraceArchive& InOutArchive );
	UE_API virtual void Serialize( void* V, int64 Length ) override;
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	void ReadUObject(UObject*& Obj);
	UE_API virtual FArchive& operator<<(UObject*& Obj) override;
	UE_API virtual FArchive& operator<<(FName& Value) override;
	UE_API virtual FArchive& operator<<(FText& Value) override;

protected:

	void ReadVersions();

	TMap<uint64, UObject*> ObjectIdMap;
	TMap<int64,FName> OffsetToName;
	TMap<int64,FString> OffsetToString;
};

#undef UE_API