// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMStringUtils.h"
#include "RigVMCore/RigVMTypeIndex.h"
#include "RigVMCore/RigVMUnknownType.h"
#include "UObject/Interface.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/CoreRedirects.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"

#define UE_API RIGVM_API

struct FRigVMExternalVariable;
struct FRigVMTemplateArgumentType;
class IPlugin;

struct FRigVMUserDefinedTypeResolver
{
	FRigVMUserDefinedTypeResolver() = default;
	explicit FRigVMUserDefinedTypeResolver(const TFunction<UObject*(const FString&)>& InResolver) : Resolver(InResolver) {}
	explicit FRigVMUserDefinedTypeResolver(TMap<FString, FSoftObjectPath>&& InObjectMap) : ObjectMap(InObjectMap) {} 
	
	UObject* GetTypeObjectByName(const FString& InTypeName) const
	{
		if (Resolver)
		{
			return Resolver(InTypeName);
		}
		
		if (const FSoftObjectPath *ObjectPath = ObjectMap.Find(InTypeName))
		{
			return ObjectPath->TryLoad();
		}
		return nullptr;
	}

	bool IsValid() const
	{
		return Resolver || !ObjectMap.IsEmpty();
	}
	
private:
	TFunction<UObject*(const FString&)> Resolver;
	TMap<FString, FSoftObjectPath> ObjectMap;
};

class FRigVMPluginsPrefixCache : public TSharedFromThis<FRigVMPluginsPrefixCache>
{

public:
	UE_API static TSharedRef<FRigVMPluginsPrefixCache> Get();
	UE_API virtual ~FRigVMPluginsPrefixCache();
	
	UE_API FString GetPluginName(const UObject* InObject) const;
	UE_API void Refresh();

private:
	UE_API FRigVMPluginsPrefixCache();
	
	FRigVMPluginsPrefixCache(const FRigVMPluginsPrefixCache&) = delete;
	FRigVMPluginsPrefixCache& operator=(const FRigVMPluginsPrefixCache&) = delete;
	
	UE_API void OnPluginMounted(IPlugin& InPlugin);
	UE_API void OnPluginUnmounted(IPlugin& InPlugin);
	
	struct FRigVMPluginReferences
	{
		FString PluginName;
		uint16 Count = 0;
	};
	
	TMap<FString, FRigVMPluginReferences> CachedPluginPrefixes;
	bool bInitialized = false;
	mutable FRWLock CacheLock;
	
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;
};

namespace RigVMTypeUtils
{
	constexpr TCHAR TArrayPrefix[] = TEXT("TArray<");
	constexpr TCHAR TObjectPtrPrefix[] = TEXT("TObjectPtr<");
	constexpr TCHAR TSubclassOfPrefix[] = TEXT("TSubclassOf<");
	constexpr TCHAR TScriptInterfacePrefix[] = TEXT("TScriptInterface<");
	constexpr TCHAR TArrayTemplate[] = TEXT("TArray<%s>");
	constexpr TCHAR TObjectPtrTemplate[] = TEXT("TObjectPtr<%s%s>");
	constexpr TCHAR TSubclassOfTemplate[] = TEXT("TSubclassOf<%s%s>");
	constexpr TCHAR TScriptInterfaceTemplate[] = TEXT("TScriptInterface<%s%s>");

	extern UE_API const TCHAR BoolType[];
	extern UE_API const TCHAR FloatType[];
	extern UE_API const TCHAR DoubleType[];
	extern UE_API const TCHAR IntType[];
	extern UE_API const TCHAR Int32Type[];
	extern UE_API const TCHAR Int64Type[];
	extern UE_API const TCHAR UInt8Type[];
	extern UE_API const TCHAR UInt32Type[];
	extern UE_API const TCHAR UInt64Type[];
	extern UE_API const TCHAR FNameType[];
	extern UE_API const TCHAR FStringType[];
	extern UE_API const TCHAR FTextType[];
	extern UE_API const TCHAR BoolArrayType[];
	extern UE_API const TCHAR FloatArrayType[];
	extern UE_API const TCHAR DoubleArrayType[];
	extern UE_API const TCHAR Int32ArrayType[];
	extern UE_API const TCHAR UInt32ArrayType[];
	extern UE_API const TCHAR UInt8ArrayType[];
	extern UE_API const TCHAR FNameArrayType[];
	extern UE_API const TCHAR FStringArrayType[];
	extern UE_API const TCHAR FTextArrayType[];

	extern UE_API const FLazyName BoolTypeName;
	extern UE_API const FLazyName FloatTypeName;
	extern UE_API const FLazyName DoubleTypeName;
	extern UE_API const FLazyName IntTypeName;
	extern UE_API const FLazyName Int32TypeName;
	extern UE_API const FLazyName Int64TypeName;
	extern UE_API const FLazyName UInt8TypeName;
	extern UE_API const FLazyName UInt32TypeName;
	extern UE_API const FLazyName UInt64TypeName;
	extern UE_API const FLazyName FNameTypeName;
	extern UE_API const FLazyName FStringTypeName;
	extern UE_API const FLazyName FTextTypeName;
	extern UE_API const FLazyName BoolArrayTypeName;
	extern UE_API const FLazyName FloatArrayTypeName;
	extern UE_API const FLazyName DoubleArrayTypeName;
	extern UE_API const FLazyName Int32ArrayTypeName;
	extern UE_API const FLazyName UInt32ArrayTypeName;
	extern UE_API const FLazyName UInt8ArrayTypeName;
	extern UE_API const FLazyName FNameArrayTypeName;
	extern UE_API const FLazyName FStringArrayTypeName;

	class TypeIndex
	{
	public:
		static UE_API TRigVMTypeIndex Execute;	
		static UE_API TRigVMTypeIndex ExecuteArray;	
		static UE_API TRigVMTypeIndex Bool;	
		static UE_API TRigVMTypeIndex Float;	
		static UE_API TRigVMTypeIndex Double;	
		static UE_API TRigVMTypeIndex Int32;	
		static UE_API TRigVMTypeIndex UInt32;	
		static UE_API TRigVMTypeIndex UInt8;	
		static UE_API TRigVMTypeIndex FName;	
		static UE_API TRigVMTypeIndex FString;
		static UE_API TRigVMTypeIndex WildCard;	
		static UE_API TRigVMTypeIndex BoolArray;	
		static UE_API TRigVMTypeIndex FloatArray;	
		static UE_API TRigVMTypeIndex DoubleArray;	
		static UE_API TRigVMTypeIndex Int32Array;	
		static UE_API TRigVMTypeIndex UInt32Array;	
		static UE_API TRigVMTypeIndex UInt8Array;	
		static UE_API TRigVMTypeIndex FNameArray;	
		static UE_API TRigVMTypeIndex FStringArray;	
		static UE_API TRigVMTypeIndex WildCardArray;	
	};

	// Returns true if the type specified is an array
	inline bool IsArrayType(const FString& InCPPType)
	{
		return InCPPType.StartsWith(TArrayPrefix);
	}

	inline FString ArrayTypeFromBaseType(const FString& InCPPType)
	{
		return RigVMStringUtils::JoinStrings(FString(), InCPPType, TArrayPrefix, nullptr, TEXT(">"));
	}

	inline FString BaseTypeFromArrayType(const FString& InCPPType)
	{
		return InCPPType.Mid(7 /* TArray< */, InCPPType.Len() - 8 /* TArray< and > */);
	}

	inline FString GetUniqueStructTypeName(const FGuid& InStructGuid)
	{
		static const FString UserDefinedStructPrefix = TEXT("FUserDefinedStruct_");
		return UserDefinedStructPrefix + InStructGuid.ToString();
	}

	inline FString GetUniqueStructTypeName(const UScriptStruct* InScriptStruct)
	{
		if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InScriptStruct))
		{
			return GetUniqueStructTypeName(UserDefinedStruct->GetCustomGuid());
		}

		return InScriptStruct->GetStructCPPName();
	}

	inline FString CPPTypeFromEnum(const UEnum* InEnum)
	{
		FString CPPType = InEnum->CppType;
		if(CPPType.IsEmpty()) // this might be a user defined enum
		{
			CPPType = FString::Printf(TEXT("EUserDefinedEnum_%s_%08x"), *InEnum->GetName(), GetTypeHash(InEnum->GetPathName()));
		}
		return CPPType;
	}

	inline bool IsUClassType(const FString& InCPPType)
	{
		return InCPPType.StartsWith(TSubclassOfPrefix);
	}

	inline bool IsUObjectType(const FString& InCPPType)
	{
		return InCPPType.StartsWith(TObjectPtrPrefix);
	}

	inline bool IsInterfaceType(const FString& InCPPType)
	{
		return InCPPType.StartsWith(TScriptInterfacePrefix);
	}

	static UScriptStruct* GetWildCardCPPTypeObject()
	{
		static UScriptStruct* WildCardTypeObject = FRigVMUnknownType::StaticStruct();
		return WildCardTypeObject;
	}

	static const FString& GetWildCardCPPType()
	{
		static const FString WildCardCPPType = GetUniqueStructTypeName(FRigVMUnknownType::StaticStruct()); 
		return WildCardCPPType;
	}

	static const FLazyName& GetWildCardCPPTypeName()
	{
		static const FLazyName WildCardCPPTypeName(*GetWildCardCPPType()); 
		return WildCardCPPTypeName;
	}

	static const FString& GetWildCardArrayCPPType()
	{
		static const FString WildCardArrayCPPType = ArrayTypeFromBaseType(GetWildCardCPPType()); 
		return WildCardArrayCPPType;
	}

	static const FLazyName& GetWildCardArrayCPPTypeName()
	{
		static const FLazyName WildCardArrayCPPTypeName(*GetWildCardArrayCPPType()); 
		return WildCardArrayCPPTypeName;
	}

	RIGVM_API bool RequiresCPPTypeObject(const FString& InCPPType);

	RIGVM_API UObject* FindObjectGlobally(const TCHAR* InObjectName, bool bUseRedirector);

	// A UClass argument is used to signify both the object type and its class type.
	// This argument differentiates between the two
	enum class EClassArgType
	{
		// This type signifies a class
		AsClass,

		// This type signifies an object 
		AsObject
	};

	RIGVM_API FString CPPTypeFromObject(const UObject* InCPPTypeObject, EClassArgType InClassArgType = EClassArgType::AsObject);

	// Finds the CPPTypeObject from a CPP type of a potentially missing / unloaded user defined struct or enum
	RIGVM_API UObject* UserDefinedTypeFromCPPType(FString& InOutCPPType, const FRigVMUserDefinedTypeResolver* InTypeResolver = nullptr);

	// Finds the CPPTypeObject from the CPPType. If not found, tries to use redirectors and modifies the InOutCPPType.
	RIGVM_API UObject* ObjectFromCPPType(FString& InOutCPPType, bool bUseRedirector = true, const FRigVMUserDefinedTypeResolver* InTypeResolver = nullptr);

	static void CleanupCPPType(FString& CPPType)
	{
		// make sure to clean up spacing errors with templated types
		if (CPPType.Contains(TEXT("<")) || CPPType.Contains(TEXT(">")))
		{
			CPPType.ReplaceInline(TEXT("< "), TEXT("<"));
			CPPType.ReplaceInline(TEXT(" >"), TEXT(">"));
			CPPType.ReplaceInline(TEXT(" <"), TEXT("<"));
			CPPType.ReplaceInline(TEXT("> "), TEXT(">"));
		}
	}

	static FString GetCPPTypeFromProperty(const FProperty* InProperty)
	{
		check(InProperty);
		FString ExtendedType;
		FString CPPType = InProperty->GetCPPType(&ExtendedType);
		CPPType += ExtendedType;

		CleanupCPPType(CPPType);
		return CPPType;
	}

	RIGVM_API UObject* GetCPPTypeObjectFromProperty(const FProperty* InProperty);

	RIGVM_API FString PostProcessCPPType(const FString& InCPPType, UObject* InCPPTypeObject = nullptr, const FRigVMUserDefinedTypeResolver* InResolvalInfo = nullptr, bool bIsSerializing = false);

	RIGVM_API bool FixCPPTypeAndObject(FString& InOutCPPType, TObjectPtr<UObject>& InOutCPPTypeObject, const FRigVMUserDefinedTypeResolver* InResolvalInfo = nullptr);

	// helper function to retrieve an object from a path
	RIGVM_API UObject* FindObjectFromCPPTypeObjectPath(const FString& InObjectPath);
	
	template<class T>
	static T* FindObjectFromCPPTypeObjectPath(const FString& InObjectPath)
	{
		return Cast<T>(FindObjectFromCPPTypeObjectPath(InObjectPath));
	}

	RIGVM_API bool AreCompatible(const FProperty* InSourceProperty, const FProperty* InTargetProperty);

	RIGVM_API FString GetPluginName(const UObject* InObject);
	RIGVM_API TArray<FString> GetPluginModulePrefixes(const IPlugin& InPlugin);

	RIGVM_API bool CPPTypeFromExternalVariable(const FRigVMExternalVariable& InExternalVariable, FString& OutCPPType, UObject** OutCPPTypeObject);
}

#undef UE_API
