// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class RigVM : ModuleRules
{
    public RigVM(ReadOnlyTargetRules Target) : base(Target)
    {
	    CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

        PrivateIncludePaths.Add(Path.Combine(EngineDirectory,"Source/ThirdParty/AHEasing/AHEasing-1.3.2"));

        PublicDependencyModuleNames.AddRange(
            new string[] {
				"AnimGraphRuntime",
				"AnimationCore",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Projects",
				"TraceLog",
				"RewindDebuggerRuntimeInterface",
			}
		);

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                    "BlueprintGraph",
					"StructUtilsEditor",
					"AssetTools"
				}
            );
        }
    }
}
