<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * RigVM Visualizers
  *
  -->
  
  <!-- RigVMCore Visualizers -->

  <Type Name="FRigVMExternalVariable">
    <Expand>
      <!-- CustomList just so we can have variables -->
      <CustomListItems>
        <Variable Name="DbgName" InitialValue="&amp;Name"/>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;bool&quot;, 4) == 0">*(bool*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;int32&quot;, 5) == 0">*(int32*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;float&quot;, 5) == 0">*(float*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;double&quot;, 6) == 0">*(double*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FName&quot;, 5) == 0">*(FName*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FString&quot;, 7) == 0">*(FString*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FVector&quot;, 7) == 0">*(FVector*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FRotator&quot;, 8) == 0">*(FRotator*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FQuat&quot;, 5) == 0">*(FQuat*)Memory</Item>
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FTransform&quot;, 10) == 0">*(FTransform*)Memory</Item>
        <!--<Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FEulerTransform&quot;, 15) == 0">*(FEulerTransform*)Memory</Item>-->
        <Item Name="[{*DbgName,sb}]" Condition="memcmp(&amp;TypeName[0], &quot;FMatrix&quot;, 7) == 0">*(FMatrix*)Memory</Item>
      </CustomListItems>
    </Expand>
  </Type>

</AutoVisualizer>
