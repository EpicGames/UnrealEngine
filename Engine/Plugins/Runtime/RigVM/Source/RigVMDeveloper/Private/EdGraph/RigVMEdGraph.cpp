// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraph.h"
#include "RigVMEditorAsset.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Editor/EditorEngine.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMObjectVersion.h"
#include "EdGraphNode_Comment.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "Stats/StatsHierarchical.h"
#include "Blueprint/BlueprintExtension.h"
#include "RigVMDeveloperTypeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEdGraph)

#if WITH_EDITOR
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMBlueprintUtils.h"
#include "BlueprintCompilationManager.h"
#include "Framework/Notifications/NotificationManager.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraph"

#if WITH_EDITOR
/** The editor object. */
extern UNREALED_API class UEditorEngine* GEditor;
#endif

URigVMEdGraph::URigVMEdGraph()
{
	bSuspendModelNotifications = false;
	bIsTemporaryGraphForCopyPaste = false;
	bIsSelecting = false;
	bIsFunctionDefinition = false;
}

FRigVMClient* URigVMEdGraph::GetRigVMClient() const
{
	if (const IRigVMClientHost* Host = GetImplementingOuter<IRigVMClientHost>())
	{
		return (FRigVMClient*)Host->GetRigVMClient();
	}
	return nullptr;
}

FString URigVMEdGraph::GetRigVMNodePath() const
{
	return ModelNodePath;
}

void URigVMEdGraph::HandleRigVMGraphRenamed(const FString& InOldNodePath, const FString& InNewNodePath)
{
	static constexpr TCHAR NodePathPrefixFormat[] = TEXT("%s|");
	const FString OldPrefix = FString::Printf(NodePathPrefixFormat, *InOldNodePath); 
	const FString NewPrefix = FString::Printf(NodePathPrefixFormat, *InNewNodePath);
	
	if(ModelNodePath == InOldNodePath)
	{
		Modify();
		ModelNodePath = InNewNodePath;

		FString GraphName;
		if(!ModelNodePath.Split(TEXT("|"), nullptr, &GraphName, ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			GraphName = ModelNodePath;
		}
		GraphName.RemoveFromEnd(TEXT("::"));
		GraphName.RemoveFromStart(FRigVMClient::RigVMModelPrefix);
		GraphName.TrimStartAndEndInline();

		if(GraphName.IsEmpty())
		{
			GraphName = URigVMEdGraphSchema::GraphName_RigVM.ToString(); 
		}
		GraphName = FRigVMClient::GetUniqueName(GetOuter(), *GraphName).ToString();

		Rename(*GraphName, nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	}
	else if(ModelNodePath.StartsWith(OldPrefix))
	{
		Modify();
		ModelNodePath = NewPrefix + ModelNodePath.RightChop(OldPrefix.Len());
	}
	else
	{
		return;
	}

	for(UEdGraphNode* Node : Nodes)
	{
		if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
		{
			if(RigNode->ModelNodePath.StartsWith(OldPrefix))
			{
				RigNode->Modify();
				RigNode->ModelNodePath = NewPrefix + RigNode->ModelNodePath.RightChop(OldPrefix.Len() - 1);
			}
		}
	}
}

const URigVMBlueprint* URigVMEdGraph::GetBlueprintDefaultObject() const
{
	return Cast<URigVMBlueprint>(GetAssetDefaultObject().GetObject());
}

const FRigVMEditorAssetInterfacePtr URigVMEdGraph::GetAssetDefaultObject() const
{
	if(RigVMBlueprintClass)
	{
		UObject* Obj = RigVMBlueprintClass->GetDefaultObject(false);
		if (Obj->Implements<URigVMEditorAssetInterface>())
		{
			return Obj;
		}
	}
	return nullptr;
}

void URigVMEdGraph::SetBlueprintClass(const UClass* InClass)
{
	RigVMBlueprintClass = InClass;
	if(const FRigVMEditorAssetInterfacePtr Blueprint = GetAssetDefaultObject())
	{
		Schema = Blueprint->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
	}
}

void URigVMEdGraph::InitializeFromAsset(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	SetBlueprintClass(InBlueprint->GetObject()->GetClass());
	InBlueprint->OnModified().RemoveAll(this);
	InBlueprint->OnModified().AddUObject(this, &URigVMEdGraph::HandleModifiedEvent);
	InBlueprint->OnVMCompiled().RemoveAll(this);
	InBlueprint->OnVMCompiled().AddUObject(this, &URigVMEdGraph::HandleVMCompiledEvent);
}

bool URigVMEdGraph::IsPreviewGraph() const
{
	// if we are not below a client host we are preview
	return GetImplementingOuter<IRigVMClientHost>() == nullptr;
}

const URigVMEdGraphSchema* URigVMEdGraph::GetRigVMEdGraphSchema()
{
	return CastChecked<const URigVMEdGraphSchema>(GetSchema());
}

#if WITH_EDITORONLY_DATA
void URigVMEdGraph::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if(Schema == nullptr || !Schema->IsChildOf(URigVMEdGraphSchema::StaticClass()))
		{
			Schema = URigVMEdGraphSchema::StaticClass();
		}
	}
}
#endif

#if WITH_EDITOR

void URigVMEdGraph::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	(void)HandleModifiedEvent_Internal(InNotifType, InGraph, InSubject);
}

bool URigVMEdGraph::HandleModifiedEvent_Internal(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (bSuspendModelNotifications)
	{
		return false;
	}

	// only make sure to receive notifs for this graph - unless
	// we are on a template graph (used by node spawners)
	if (GetModel() != InGraph)
	{
		return false;
	}

	// Make sure this EdGraph has a valid rigvm host
	const IRigVMClientHost* Host = GetImplementingOuter<IRigVMClientHost>();
	if (!Host)
	{
		return false;
	}


	if(URigVMEdGraphSchema* EdGraphSchema = (URigVMEdGraphSchema*)GetRigVMEdGraphSchema())
	{
		EdGraphSchema->HandleModifiedEvent(InNotifType, InGraph, InSubject);
	}

	// increment the node topology version for any interaction
	// with a node.
	{
		URigVMEdGraphNode* EdNode = nullptr;
		if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
		{
			EdNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName()));
		}
		else if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
		{
			EdNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName()));
		}

		if(EdNode)
		{
			EdNode->NodeTopologyVersion++;
		}
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::GraphChanged:
		{
			ModelNodePathToEdNode.Reset();

			for (URigVMNode* Node : InGraph->GetNodes())
			{
				UEdGraphNode* EdNode = FindNodeForModelNodeName(Node->GetFName(), false);
				if (EdNode != nullptr)
				{
					RemoveNode(EdNode);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeSelectionChanged:
		{
			if (bIsSelecting)
			{
				return false;
			}
			TGuardValue<bool> SelectionGuard(bIsSelecting, true);

			TSet<const UEdGraphNode*> NodeSelection;
			for (FName NodeName : InGraph->GetSelectNodes())
			{
				if (UEdGraphNode* EdNode = FindNodeForModelNodeName(NodeName))
				{
					NodeSelection.Add(EdNode);
				}
			}
			SelectNodeSet(NodeSelection);
			break;
		}
		case ERigVMGraphNotifType::NodeAdded:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (!ModelNode->IsVisibleInUI())
				{
					if (URigVMInjectionInfo* Injection = ModelNode->GetInjectionInfo())
					{
						if (URigVMPin* ModelPin = Injection->GetPin())
						{
							URigVMNode* ParentModelNode = ModelPin->GetNode();
							if (ParentModelNode)
							{
								UEdGraphNode* EdNode = FindNodeForModelNodeName(ParentModelNode->GetFName());
								if (EdNode)
								{
									if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(EdNode))
									{
										RigNode->ModelPinsChanged(true);
									}
								}
							}
						}
					}
					break;
				}
				else
				{
					// check if the node is already part of the graph
					if(FindNodeForModelNodeName(ModelNode->GetFName()) != nullptr)
					{
						break;
					}
				}

				if (URigVMCommentNode* CommentModelNode = Cast<URigVMCommentNode>(ModelNode))
				{
					UEdGraphNode_Comment* NewNode = NewObject<UEdGraphNode_Comment>(this, CommentModelNode->GetFName());
					AddNode(NewNode, false, false);

					NewNode->CreateNewGuid();
					NewNode->PostPlacedNewNode();
					NewNode->AllocateDefaultPins();

					NewNode->NodePosX = FMath::RoundToInt32(ModelNode->GetPosition().X);
					NewNode->NodePosY = FMath::RoundToInt32(ModelNode->GetPosition().Y);
					NewNode->NodeWidth = FMath::RoundToInt32(ModelNode->GetSize().X);
					NewNode->NodeHeight = FMath::RoundToInt32(ModelNode->GetSize().Y);
					NewNode->CommentColor = ModelNode->GetNodeColor();
					NewNode->NodeComment = CommentModelNode->GetCommentText();
					NewNode->FontSize = CommentModelNode->GetCommentFontSize();
					NewNode->bCommentBubbleVisible = CommentModelNode->GetCommentBubbleVisible();
					NewNode->bCommentBubbleVisible_InDetailsPanel = CommentModelNode->GetCommentBubbleVisible();
					NewNode->bCommentBubblePinned = CommentModelNode->GetCommentBubbleVisible();
					NewNode->bColorCommentBubble = CommentModelNode->GetCommentColorBubble();
					NewNode->SetFlags(RF_Transactional);
					(void)NewNode->GetNodesUnderComment();

					ModelNodePathToEdNode.Add(ModelNode->GetFName(), NewNode);
				}
				else // struct, library, parameter + variable
				{
					URigVMEdGraphNode* NewNode = NewObject<URigVMEdGraphNode>(this, GetRigVMEdGraphSchema()->GetGraphNodeClass(this), ModelNode->GetFName());
					AddNode(NewNode, false, false);

					NewNode->ModelNodePath = ModelNode->GetNodePath();
					NewNode->CreateNewGuid();
					NewNode->PostPlacedNewNode();
					NewNode->AllocateDefaultPins();
					NewNode->PostReconstructNode();

					NewNode->NodePosX = FMath::RoundToInt32(ModelNode->GetPosition().X);
					NewNode->NodePosY = FMath::RoundToInt32(ModelNode->GetPosition().Y);
					if (ModelNode->IsA<URigVMRerouteNode>())
					{
						if (URigVMPin* ModelPin = ModelNode->FindPin("Value"))
						{
							if (UEdGraphPin* ValuePin = NewNode->FindPin(ModelPin->GetPinPath()))
							{
								NewNode->SetColorFromModel(GetSchema()->GetPinTypeColor(ValuePin->PinType));
							}
						}
					}
					else
					{
						NewNode->SetColorFromModel(ModelNode->GetNodeColor());
					}
					NewNode->SetFlags(RF_Transactional);

					ModelNodePathToEdNode.Add(ModelNode->GetFName(), NewNode);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeRemoved:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMInjectionInfo* Injection = ModelNode->GetInjectionInfo())
				{
					if (URigVMPin* ModelPin = Injection->GetPin())
					{
						if (URigVMNode* ParentModelNode = ModelPin->GetNode())
						{
							if (UEdGraphNode* EdNode = FindNodeForModelNodeName(ParentModelNode->GetFName()))
							{
								if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(EdNode))
								{
									RigNode->ModelPinsChanged(true);
								}
							}
						}
					}
					break;
				}

				ModelNodePathToEdNode.Remove(ModelNode->GetFName());

				if (UEdGraphNode* EdNode = FindNodeForModelNodeName(ModelNode->GetFName(), false))
				{
					RemoveNode(EdNode);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodePositionChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				UEdGraphNode* EdNode = FindNodeForModelNodeName(ModelNode->GetFName());
				if (EdNode)
				{
					// No need to call Node->Modify(), since control rig has its own undo/redo system see RigVMControllerActions.cpp
					EdNode->NodePosX = (int32)ModelNode->GetPosition().X;
					EdNode->NodePosY = (int32)ModelNode->GetPosition().Y;
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeSizeChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				UEdGraphNode_Comment* EdNode = Cast<UEdGraphNode_Comment>(FindNodeForModelNodeName(ModelNode->GetFName()));
				if (EdNode)
				{
					// No need to call Node->Modify(), since control rig has its own undo/redo system see RigVMControllerActions.cpp
					EdNode->NodeWidth = (int32)ModelNode->GetSize().X;
					EdNode->NodeHeight = (int32)ModelNode->GetSize().Y;
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeDescriptionChanged:
		case ERigVMGraphNotifType::NodeCategoryChanged:
		case ERigVMGraphNotifType::NodeTitleChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->SyncGraphNodeTitleWithModelNodeTitle();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeColorChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->InvalidateNodeColor();
				}
				else if(UEdGraphNode_Comment * EdComment = Cast<UEdGraphNode_Comment>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					EdComment->CommentColor = ModelNode->GetNodeColor();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::CommentTextChanged:
		{
			if (URigVMCommentNode* ModelNode = Cast<URigVMCommentNode>(InSubject))
			{
				UEdGraphNode_Comment* EdNode = Cast<UEdGraphNode_Comment>(FindNodeForModelNodeName(ModelNode->GetFName()));
				if (EdNode)
				{
					EdNode->NodeComment = ModelNode->GetCommentText();
					EdNode->FontSize = ModelNode->GetCommentFontSize();
					EdNode->bCommentBubbleVisible = ModelNode->GetCommentBubbleVisible();
					EdNode->bCommentBubbleVisible_InDetailsPanel = ModelNode->GetCommentBubbleVisible();
					EdNode->bColorCommentBubble = ModelNode->GetCommentColorBubble();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::LinkAdded:
		case ERigVMGraphNotifType::LinkRemoved:
		{
			bool AddLink = InNotifType == ERigVMGraphNotifType::LinkAdded;

			if (URigVMLink* Link = Cast<URigVMLink>(InSubject))
			{
				URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();

				if (SourcePin)
				{
					SourcePin = SourcePin->GetOriginalPinFromInjectedNode();
				}
				if (TargetPin)
				{
					TargetPin = TargetPin->GetOriginalPinFromInjectedNode();
				}

				if (SourcePin && TargetPin && SourcePin != TargetPin)
				{
					URigVMEdGraphNode* SourceRigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(SourcePin->GetNode()->GetFName()));
					URigVMEdGraphNode* TargetRigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(TargetPin->GetNode()->GetFName()));

					if (SourceRigNode != nullptr && TargetRigNode != nullptr)
					{
						FString SourcePinPath = SourcePin->GetPinPath();
						FString TargetPinPath = TargetPin->GetPinPath();
						UEdGraphPin* SourceRigPin = SourceRigNode->FindPin(*SourcePinPath, EGPD_Output);
						UEdGraphPin* TargetRigPin = TargetRigNode->FindPin(*TargetPinPath, EGPD_Input);

						if (SourceRigPin != nullptr && TargetRigPin != nullptr)
						{
							if (AddLink)
							{
								SourceRigPin->MakeLinkTo(TargetRigPin);
							}
							else
							{
								SourceRigPin->BreakLinkTo(TargetRigPin);
							}

							SourceRigPin->LinkedTo.Remove(nullptr);
							TargetRigPin->LinkedTo.Remove(nullptr);
						}
					}

					if (const URigVMNode* SourceNode = SourcePin->GetNode())
					{
						if (SourceNode->IsA<URigVMVariableNode>() && TargetRigNode)
						{
							TargetRigNode->ModelPinsChanged(true);
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinDefaultValueChanged:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					UEdGraphPin* RigNodePin = RigNode->FindPin(ModelPin->GetPinPath());
					if (RigNodePin == nullptr)
					{
						if(ModelPin->GetNode()->IsEvent())
						{
							RigNode->SyncGraphNodeTitleWithModelNodeTitle();
						}
						break;
					}

					RigNode->SetupPinDefaultsFromModel(RigNodePin);

					if (Cast<URigVMVariableNode>(ModelPin->GetNode()))
					{
						if (ModelPin->GetName() == TEXT("Variable"))
						{
							RigNode->SyncGraphNodeTitleWithModelNodeTitle();
							RigNode->SynchronizeGraphPinValueWithModelPin(ModelPin);
						}
					}
					else if (Cast<URigVMUnitNode>(ModelPin->GetNode()))
					{
						RigNode->SyncGraphNodeTitleWithModelNodeTitle();
					}
				}
				else if (URigVMInjectionInfo* Injection = ModelPin->GetNode()->GetInjectionInfo())
				{
					if (Injection->InputPin != ModelPin->GetRootPin())
					{
						if (URigVMPin* InjectionPin = Injection->GetPin())
						{
							URigVMNode* ParentModelNode = InjectionPin->GetNode();
							if (ParentModelNode)
							{
								UEdGraphNode* HostEdNode = FindNodeForModelNodeName(ParentModelNode->GetFName());
								if (HostEdNode)
								{
									if (URigVMEdGraphNode* HostRigNode = Cast<URigVMEdGraphNode>(HostEdNode))
									{
										HostRigNode->ModelPinsChanged();
									}
								}
							}
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinAdded:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if(URigVMNode* ModelNode = ModelPin->GetNode())
				{
					if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
					{
						RigNode->ModelPinAdded(ModelPin);
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinRemoved:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if(URigVMNode* ModelNode = ModelPin->GetNode())
				{
					if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
					{
						RigNode->ModelPinRemoved(ModelPin);
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinArraySizeChanged:
		{
			// don't do anything here - the UI will update based on the
			// PinAdded and PinRemoved notifs
			break;
		}
		case ERigVMGraphNotifType::PinDirectionChanged:
		case ERigVMGraphNotifType::PinIndexChanged:
		case ERigVMGraphNotifType::PinBoundVariableChanged:
		{
        	if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
        	{
        		if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
        		{
        			RigNode->ModelPinsChanged();
        		}
        	}
        	break;
        }
		case ERigVMGraphNotifType::PinCategoryChanged:
		case ERigVMGraphNotifType::PinCategoriesChanged:
		{
			URigVMNode* ModelNode = nullptr;
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				ModelNode = ModelPin->GetNode();
			}
			else
			{
				ModelNode = Cast<URigVMNode>(InSubject);
			}
			if(ModelNode)
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->ModelPinsChanged(true);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::LibraryTemplateChanged:
		{
			if (URigVMNode* LibraryNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(LibraryNode->GetFName())))
				{
					RigNode->ModelPinsChanged(true);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinDisplayNameChanged:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					RigNode->SynchronizeGraphPinNameWithModelPin(ModelPin);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinTypeChanged:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					RigNode->SynchronizeGraphPinTypeWithModelPin(ModelPin);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinRenamed:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					RigNode->SynchronizeGraphPinNameWithModelPin(ModelPin);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeRenamed:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				ModelNodePathToEdNode.Remove(ModelNode->GetPreviousFName());
				UEdGraphNode* EdNode = FindNodeForModelNodeName(ModelNode->GetPreviousFName());
				ModelNodePathToEdNode.Remove(ModelNode->GetPreviousFName());
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(EdNode))
				{
					RigNode->SyncGraphNodeNameWithModelNodeName(ModelNode);
					ModelNodePathToEdNode.Add(ModelNode->GetFName(), RigNode);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::VariableRenamed:
		case ERigVMGraphNotifType::NodeReferenceChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->SyncGraphNodeTitleWithModelNodeTitle();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeSelected:
		{
			if (URigVMCommentNode* ModelNode = Cast<URigVMCommentNode>(InSubject))
			{
				// UEdGraphNode_Comment cannot access RigVMCommentNode's selection state, so we have to manually toggle its selection state
				// URigVMEdGraphNode does not need this step because it overrides the IsSelectedInEditor() method
				UEdGraphNode_Comment* EdNode = Cast<UEdGraphNode_Comment>(FindNodeForModelNodeName(ModelNode->GetFName()));
				if (EdNode)
				{
					EdNode->SetSelectionState(UEdGraphNode_Comment::ESelectionState::Selected);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeDeselected:
		{
			if (URigVMCommentNode* ModelNode = Cast<URigVMCommentNode>(InSubject))
			{
				UEdGraphNode_Comment* EdNode = Cast<UEdGraphNode_Comment>(FindNodeForModelNodeName(ModelNode->GetFName()));
				if (EdNode)
				{
					EdNode->SetSelectionState(UEdGraphNode_Comment::ESelectionState::Deselected);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinExpansionChanged:
		{
			if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelPin->GetNode()->GetFName())))
				{
					RigNode->OnNodePinExpansionChanged().Broadcast();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinCategoryExpansionChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->OnNodePinExpansionChanged().Broadcast();
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeEarlyExitChanged:
		{
			if (URigVMNode* ModelNode = Cast<URigVMNode>(InSubject))
			{
				if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FindNodeForModelNodeName(ModelNode->GetFName())))
				{
					RigNode->InvalidateNodeTitle();
					RigNode->InvalidateNodeColor();
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}

	return true;
}

int32 URigVMEdGraph::GetInstructionIndex(const URigVMEdGraphNode* InNode, bool bAsInput)
{
	if (const TPair<int32, int32>* FoundIndex = CachedInstructionIndices.Find(InNode->GetModelNode()))
	{
		return bAsInput ? FoundIndex->Key : FoundIndex->Value;
	}

	struct Local
	{
		static int32 GetInstructionIndex(URigVMNode* InModelNode, const FRigVMByteCode* InByteCode, TMap<URigVMNode*, TPair<int32, int32>>& Indices, bool bAsInput)
		{
			if (InModelNode == nullptr)
			{
				return INDEX_NONE;
			}

			if (const TPair<int32, int32>* ExistingIndex = Indices.Find(InModelNode))
			{
				const int32 Index = bAsInput ? ExistingIndex->Key : ExistingIndex->Value;
				if(Index != INDEX_NONE)
				{
					return Index;
				}
			}

			if(const URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(InModelNode))
			{
				int32 InstructionIndex = INDEX_NONE;
				if(bAsInput)
				{
					TArray<URigVMNode*> SourceNodes = RerouteNode->GetLinkedSourceNodes();
					for(URigVMNode* SourceNode : SourceNodes)
					{
						InstructionIndex = GetInstructionIndex(SourceNode, InByteCode, Indices, bAsInput);
						if(InstructionIndex != INDEX_NONE)
						{
							break;
						}
					}
					Indices.FindOrAdd(InModelNode).Key = InstructionIndex;
				}
				else
				{
					TArray<URigVMNode*> TargetNodes = RerouteNode->GetLinkedTargetNodes();
					for(URigVMNode* TargetNode : TargetNodes)
					{
						InstructionIndex = GetInstructionIndex(TargetNode, InByteCode, Indices, bAsInput);
						if(InstructionIndex != INDEX_NONE)
						{
							break;
						}
					}
					Indices.FindOrAdd(InModelNode).Value = InstructionIndex;
				}

				return InstructionIndex;
			}
			else if(URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(InModelNode))
			{
				int32 InstructionIndex = INDEX_NONE;
				if(!bAsInput)
				{
					TArray<URigVMNode*> TargetNodes = EntryNode->GetLinkedTargetNodes();
					for(URigVMNode* TargetNode : TargetNodes)
					{
						InstructionIndex = GetInstructionIndex(TargetNode, InByteCode, Indices, bAsInput);
						if(InstructionIndex != INDEX_NONE)
						{
							break;
						}
					}
					Indices.FindOrAdd(InModelNode).Key = InstructionIndex;
				}
				return InstructionIndex;
			}
			else if(URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(InModelNode))
			{
				int32 InstructionIndex = INDEX_NONE;
				if(bAsInput)
				{
					TArray<URigVMNode*> SourceNodes = ReturnNode->GetLinkedSourceNodes();
					for(URigVMNode* SourceNode : SourceNodes)
					{
						InstructionIndex = GetInstructionIndex(SourceNode, InByteCode, Indices, bAsInput);
						if(InstructionIndex != INDEX_NONE)
						{
							break;
						}
					}
					Indices.FindOrAdd(InModelNode).Key = InstructionIndex;
				}
				return InstructionIndex;
			}

			Indices.FindOrAdd(InModelNode, TPair<int32, int32>(INDEX_NONE, INDEX_NONE));

			int32 InstructionIndex = InByteCode->GetFirstInstructionIndexForSubject(InModelNode);
			if (InstructionIndex != INDEX_NONE)
			{
				if(bAsInput)
				{
					Indices.FindOrAdd(InModelNode).Key = InstructionIndex;
				}
				else
				{
					Indices.FindOrAdd(InModelNode).Value = InstructionIndex;
				}
				return InstructionIndex;
			}

			FRigVMInstructionArray Instructions = InByteCode->GetInstructions();
			for (int32 i = 0; i < Instructions.Num(); ++i)
			{
				const FRigVMASTProxy Proxy = FRigVMASTProxy::MakeFromCallPath(InByteCode->GetCallPathForInstruction(i), InModelNode->GetRootGraph());
				if (Proxy.GetCallstack().Contains(InModelNode))
				{
					if(bAsInput)
					{
						Indices.FindOrAdd(InModelNode).Key = i;
					}
					else
					{
						Indices.FindOrAdd(InModelNode).Value = i;
					}
					return i;
				}
			}

			if(bAsInput)
			{
				Indices.FindOrAdd(InModelNode).Key = INDEX_NONE;
			}
			else
			{
				Indices.FindOrAdd(InModelNode).Value = INDEX_NONE;
			}

			return INDEX_NONE;
		}
	};

	if (const URigVMController* Controller = GetController())
	{
		if (const FRigVMByteCode* ByteCode = Controller->GetCurrentByteCode())
		{
			const int32 SourceInstructionIndex = Local::GetInstructionIndex(InNode->GetModelNode(), ByteCode, CachedInstructionIndices, true);
			const int32 TargetInstructionIndex = Local::GetInstructionIndex(InNode->GetModelNode(), ByteCode, CachedInstructionIndices, false);
			return bAsInput ? SourceInstructionIndex : TargetInstructionIndex;
		}
	}

	return INDEX_NONE;
}

#if WITH_EDITOR

void URigVMEdGraph::CacheEntryNameList()
{
	EntryNameList.Reset();
	EntryNameList.Add(MakeShared<FRigVMStringWithTag>(FName(NAME_None).ToString()));

	if(const TScriptInterface<IRigVMClientHost> Blueprint = GetAsset()->GetRigVMClientHost())
	{
		const TArray<FName> EntryNames = Blueprint->GetRigVMClient()->GetEntryNames();
		for (const FName& EntryName : EntryNames)
		{
			EntryNameList.Add(MakeShared<FRigVMStringWithTag>(EntryName.ToString()));
		}
	}
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* URigVMEdGraph::GetEntryNameList(URigVMPin* InPin) const
{
	if (const URigVMEdGraph* OuterGraph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return OuterGraph->GetEntryNameList(InPin);
	}
	return &EntryNameList;
}

#endif

UEdGraphNode* URigVMEdGraph::FindNodeForModelNodeName(const FName& InModelNodeName, const bool bCacheIfRequired)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(UEdGraphNode** MappedNode = ModelNodePathToEdNode.Find(InModelNodeName))
	{
		return *MappedNode;
	}
	
	const FString InModelNodePath = InModelNodeName.ToString();
	for (UEdGraphNode* EdNode : Nodes)
	{
		if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(EdNode))
		{
			if (RigNode->ModelNodePath == InModelNodePath)
			{
				if (RigNode->GetOuter() == this)
				{
					if (bCacheIfRequired)
					{
						ModelNodePathToEdNode.Add(InModelNodeName, EdNode);
					}
					return EdNode;
				}
			}
		}
		else if (EdNode)
		{
			if (EdNode->GetFName() == InModelNodeName)
			{
				if (EdNode->GetOuter() == this)
				{
					if (bCacheIfRequired)
					{
						ModelNodePathToEdNode.Add(InModelNodeName, EdNode);
					}
					return EdNode;
				}
			}
		}
	}
	return nullptr;
}

URigVMBlueprint* URigVMEdGraph::GetBlueprint() const
{
	return Cast<URigVMBlueprint>(GetAsset().GetObject());
}

void URigVMEdGraph::AddLocalVariableSearchMetaDataInfo(const FName InVariableName, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const
{
	if (URigVMGraph* RigVMGraph = GetModel())
	{
		const TArray<FRigVMGraphVariableDescription> LocalVariables = RigVMGraph->GetLocalVariables(false);
		for (const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
		{
			if (LocalVariable.Name.IsEqual(InVariableName))
			{
				OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_Name, FText::FromName(InVariableName));
				
				const FEdGraphPinType Type = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(LocalVariable);
				OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinCategory, FText::FromName(Type.PinCategory));
				OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinSubCategory, FText::FromName(Type.PinSubCategory));
				if (Type.PinSubCategoryObject.IsValid())
				{
					OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_ObjectClass, FText::FromString(Type.PinSubCategoryObject->GetPathName()));
				}
				OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_IsArray, FText::Format(LOCTEXT("RigVMNodePinIsArray", "{0}"), Type.IsArray() ? 1 : 0));
			}
		}
	}
}

FRigVMEditorAssetInterfacePtr URigVMEdGraph::GetAsset() const
{
	if (URigVMEdGraph* OuterGraph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return OuterGraph->GetAsset();
	}
	return GetOuter();
}

URigVMGraph* URigVMEdGraph::GetModel() const
{
	if(CachedModelGraph.IsValid())
	{
		return CachedModelGraph.Get();
	}
	
	if (const FRigVMClient* Client = GetRigVMClient())
	{
		if (URigVMGraph* Model = Client->GetModel(this))
		{
			CachedModelGraph = Model;
			return Model;
		}
	}

	// for preview scenarios we'll nest the edgraph under the model graph
	if(URigVMGraph* Model = GetTypedOuter<URigVMGraph>())
	{
		CachedModelGraph = Model;
		return Model;
	}
	
	return nullptr;
}

URigVMController* URigVMEdGraph::GetController() const
{
	if (FRigVMClient* Client = GetRigVMClient())
	{
		if (URigVMController* Controller = Client->GetOrCreateController(GetModel()))
		{
			return Controller;
		}
	}
	
	// for preview scenarios we'll nest the edgraph under the model graph
	// and the model graph under the controller
	if(URigVMController* Controller = GetTypedOuter<URigVMController>())
	{
		return Controller;
	}
	
	return nullptr;
}

const URigVMEdGraph* URigVMEdGraph::GetRootGraph() const
{
	if(const URigVMEdGraph* ParentGraph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return ParentGraph->GetRootGraph();
	}
	return this;
}

void URigVMEdGraph::AddNode(UEdGraphNode* NodeToAdd, bool bUserAction, bool bSelectNewNode)
{
	// Comments are added outside of the ControlRigEditor, so we add here the node to the model
	bool bCommentNodeAdded = false;
	if (const UEdGraphNode_Comment* CommentNode = Cast<const UEdGraphNode_Comment>(NodeToAdd))
	{
		if (URigVMController* Controller = GetController())
		{
			if (GetModel()->FindNodeByName(NodeToAdd->GetFName()) == nullptr) // When recreating nodes at RebuildGraphFromModel, the model node already exists
			{
#if WITH_EDITOR
				if (GEditor && !GIsTransacting)
				{
					// FBlueprintActionMenuItem::PerformAction has a super high level FScopedTransaction
					// FScopedTransaction Transaction(LOCTEXT("AddNodeTransaction", "Add Node"));
					// That scoped transaction stores the whole graph change, which is not what we want, as it deletes the Subgraphs
					// when we perform an redo of a comment after performing the redo of an Add Pin of a Sequence.
					// The issue happens because the Redo of the Add Pin changes the Subgraph object, so when we apply 
					// the Comment Redo with all the Graph changes, it clears and serializes the SubGraphs array,
					// making it invalid, as the subgraph has changed during the previous AddPin Redo
					// For other node types, UControlRigUnitNodeSpawner::Invoke cancels the transaction when the node is added,
					GEditor->CancelTransaction(0);
				}
#endif // WITH_EDITOR
				TGuardValue<bool> BlueprintNotifGuard(GetRigVMClient()->bSuspendModelNotificationsForOthers, true);
				FVector2D NodePos(CommentNode->NodePosX, CommentNode->NodePosY);
				FVector2D NodeSize(CommentNode->NodeWidth, CommentNode->NodeHeight);
				FLinearColor NodeColor = CommentNode->CommentColor;
				Controller->AddCommentNode(CommentNode->NodeComment, NodePos, NodeSize, NodeColor, CommentNode->GetName(), !GIsTransacting, true);
				ModelNodePathToEdNode.Add(NodeToAdd->GetFName(), NodeToAdd);

				bCommentNodeAdded = true;
			}
		}
	}

	Super::AddNode(NodeToAdd, bUserAction, bSelectNewNode);

	// If a comment node was added the graph needs to be refreshed to re-sort its nodes
	// Otherwise when selecting nodes they won't be depth sorted correctly and the selection can be wrong
	if (bCommentNodeAdded)
	{
		NotifyGraphChanged();
	}
}

void URigVMEdGraph::RemoveNode(UEdGraphNode* InNode)
{
	// Make sure EdGraph is not part of the transaction
	TGuardValue<ITransaction*> TransactionGuard(GUndo, nullptr);

	if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InNode))
	{
		RigNode->OnNodeBeginRemoval().Broadcast();
	}

	// clear out the pin relationships
	for(UEdGraphPin* Pin : InNode->Pins)
	{
		Pin->MarkAsGarbage();
	}
	InNode->Pins.Reset();
					
	// Rename the soon to be deleted object to a unique name, so that other objects can use
	// the old name
	FString DeletedName;
	{
		UObject* ExistingObject;
		static int32 DeletedIndex = FMath::Rand();
		do
		{
			DeletedName = FString::Printf(TEXT("EdGraph_%s_Deleted_%d"), *InNode->GetName(), DeletedIndex++); 
			ExistingObject = StaticFindObject(/*Class=*/ nullptr, this, *DeletedName, EFindObjectFlags::ExactClass);
		}
		while (ExistingObject);
	}
	InNode->Rename(*DeletedName, GetTransientPackage(), REN_DontCreateRedirectors);	

	// this also subsequently calls NotifyGraphChanged
	Super::RemoveNode(InNode);

	PostEditChange();
}

void URigVMEdGraph::HandleVMCompiledEvent(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext)
{
	CachedInstructionIndices.Reset();
}

#endif

#undef LOCTEXT_NAMESPACE

