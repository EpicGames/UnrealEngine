// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "KismetCompiler.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#if WITH_RIGVMLEGACYEDITOR
#include "FindInBlueprintManager.h"
#else
#include "RigVMEditor/Private/Editor/Kismet/RigVMBlueprintCompilationManager.h"
#include "RigVMEditor/Private/Editor/Kismet/RigVMFindInBlueprintManager.h"
#endif
#include "RigVMEditorModule.h"
#include "Textures/SlateIcon.h"
#include "RigVMEditorAsset.h"
#include "PropertyPathHelpers.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMStringUtils.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMFunctions/RigVMDispatch_CastEnum.h"
#include "RigVMFunctions/RigVMDispatch_CastObject.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "Stats/StatsHierarchical.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEdGraphNode)

#define LOCTEXT_NAMESPACE "RigVMEdGraphNode"

TAutoConsoleVariable<bool> CVarRigVMDisableCompactNodes(TEXT("RigVM.Graph.DisableCompactNodes"), false, TEXT("When true all nodes are going to be drawn as full nodes."));

URigVMEdGraphNode::URigVMEdGraphNode()
: Dimensions(0.0f, 0.0f)
, NodeTitle(FText::GetEmpty())
, FullNodeTitle(FText::GetEmpty())
, bSubTitleEnabled(true)
, NodeTopologyVersion(INDEX_NONE)
, CachedTitleColor(FLinearColor(0.f, 0.f, 0.f, 0.f))
, CachedNodeColor(FLinearColor(0.f, 0.f, 0.f, 0.f))
#if WITH_EDITOR
#endif
, CachedTemplate(nullptr)
, bRenameIsPending(false)
{
	bHasCompilerMessage = false;
	ErrorType = (int32)EMessageSeverity::Info + 1;
	
#if WITH_EDITOR
	UpdateVisualSettings();
#endif
}

FText URigVMEdGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if(NodeTitle.IsEmpty())
	{
		FString SubTitle;
		if(URigVMNode* ModelNode = GetModelNode())
		{
			if (bCanRenameNode)
			{
				FString ModelNodeTitle = ModelNode->GetNodeTitleRaw();
				if (ModelNodeTitle.IsEmpty())
				{
					ModelNodeTitle = ModelNode->GetName();
					ModelNodeTitle.RemoveFromStart(TEXT("DISPATCH_"));
					ModelNodeTitle.RemoveFromStart(TEXT("RigVMFunction_"));
					ModelNodeTitle.RemoveFromStart(TEXT("RigVMDispatch_"));
				}
				NodeTitle = FText::FromString(ModelNodeTitle);
				SubTitle = ModelNode->GetNodeSubTitle();
			}

			else if (const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
			{
				const UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct();
				if (ScriptStruct && ScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()))
				{
					if (const TSharedPtr<FStructOnScope> StructOnScope = UnitNode->ConstructStructInstance())
					{
						const FRigVMStruct* RigVMStruct = (const FRigVMStruct*)StructOnScope->GetStructMemory();
						NodeTitle = FText::FromString(RigVMStruct->GetUnitLabel());
						SubTitle = UnitNode->GetNodeSubTitle();
					}
				}
			}

			else if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
			{
				const FRigVMGraphFunctionHeader& ReferencedHeader = FunctionReferenceNode->GetReferencedFunctionHeader();
				{
					TSoftObjectPtr<URigVMFunctionReferenceNode> RefNodePtr(FunctionReferenceNode);

					const FSoftObjectPath NodeSoftPath = ReferencedHeader.LibraryPointer.GetNodeSoftPath();
					const FString& PackagePath = NodeSoftPath.GetLongPackageName();
					if(PackagePath != RefNodePtr.GetLongPackageName())
					{
						const FString AssetName = NodeSoftPath.GetAssetName();
						SubTitle = FText::Format(LOCTEXT("AssetNameSubTitle", "From {0}"), FText::FromString(AssetName)).ToString();
					}
					else
					{
						SubTitle = LOCTEXT("LocalFunctionSubTitle", "Local Function").ToString();
					}
				}
			}

			else if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelNode))
			{
				if(!CollapseNode->IsA<URigVMAggregateNode>())
				{
					SubTitle = LOCTEXT("CollapsedGraphSubtitle", "Collapsed Graph").ToString();
				}
			}

			else if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelNode))
			{
				SubTitle = VariableNode->GetNodeSubTitle();

				if (SubTitle.Len() == 0)
				{
					if(FRigVMEditorAssetInterfacePtr Blueprint = GetAsset())
					{
						const FName VariableName = VariableNode->GetVariableName();
						for(const FRigVMGraphVariableDescription& NewVariable : Blueprint->GetAssetVariables())
						{
							if(NewVariable.Name == VariableName)
							{
								FString DefaultValue = NewVariable.DefaultValue;
								if(DefaultValue.IsEmpty())
								{
									SubTitle = LOCTEXT("VariableSubtitle", "Variable").ToString();;
								}
								else
								{
									const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(NewVariable);
									// Change the order of values in rotators so that they match the pin order
									if (!PinType.IsContainer() && PinType.PinSubCategoryObject == TBaseStructure<FRotator>::Get())
									{
										TArray<FString> Values;
										DefaultValue.ParseIntoArray(Values, TEXT(","));
										if (Values.Num() == 3)
										{
											Values.Swap(0, 1);
											Values.Swap(0, 2);
										}
										DefaultValue = FString::Join(Values, TEXT(","));										
									}
									SubTitle = FText::Format(LOCTEXT("DefaultValueSubTitle", "Default {0}"), FText::FromString(DefaultValue)).ToString();
								}
								break;
							}
						}
					}
				}

				if(SubTitle.Len() > 40)
				{
					SubTitle = SubTitle.Left(36) + TEXT(" ...");
				}
			}

			if (NodeTitle.IsEmpty())
			{
				NodeTitle = FText::FromString(ModelNode->GetNodeTitle());
			}
		}

		if(IsDeprecated())
		{
			NodeTitle = FText::Format(LOCTEXT("DeprecatedNodeTitle", "{0} (Deprecated)"), NodeTitle);
		}

		if(IsOutDated())
		{
			NodeTitle = FText::Format(LOCTEXT("OutDatedNodeTitle", "{0} (OutDated)"), NodeTitle);
		}

		FullNodeTitle = NodeTitle;

		if(!SubTitle.IsEmpty() && bSubTitleEnabled)
		{
			FullNodeTitle = FText::FromString(FString::Printf(TEXT("%s\n%s"), *NodeTitle.ToString(), *SubTitle));
		}
	}

	if(TitleType == ENodeTitleType::FullTitle)
	{
		return FullNodeTitle;
	}
	return NodeTitle;
}

void URigVMEdGraphNode::ReconstructNode()
{
	ReconstructNode_Internal();
}

void URigVMEdGraphNode::ReconstructNode_Internal(bool bForce)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetOuter()); 
	if (RigGraph && !bForce)
	{
		if (RigGraph->bIsTemporaryGraphForCopyPaste)
		{
			return;
		}
	}

#if WITH_EDITOR
	UpdateVisualSettings();
#endif

	// Clear previously set messages
	ClearErrorInfo();
	
	// Move the existing pins to a saved array.
	// This way we can reuse them later
	LastEdGraphPins = Pins;

	// Also make sure to clean up any indirections
	// on them.
	for(UEdGraphPin* LastEdGraphPin : LastEdGraphPins)
	{
		LastEdGraphPin->ParentPin = nullptr;
		LastEdGraphPin->SubPins.Reset();
	}
	Pins.Reset();

	// Recreate the new pins
	ReallocatePinsDuringReconstruction(LastEdGraphPins);

	// Maintain watches up to date
	if (const URigVMNode* Node = GetModelNode())
	{
		FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetAsset();
		for (UEdGraphPin* NewPin : Pins)
		{
			const FString PinName = NewPin->GetName();
			FString Left, Right = PinName;
			URigVMPin::SplitPinPathAtStart(PinName, Left, Right);
			if (URigVMPin* ModelPin = Node->FindPin(Right))
			{
				if (ModelPin->RequiresWatch())
				{
					RigVMBlueprint->AddPinWatch(NewPin);
				}
			}
		}
	}
	
	// LogBlueprint: Warning: UEdGraphPin::DestroyImpl BasicIKSetup_0.UpperBone? causing an issue?
	// we may need to clean these pins in the opposite order? child pins first?
	RewireOldPinsToNewPins(LastEdGraphPins, Pins);
	LastEdGraphPins.Reset();

	DrawAsCompactNodeCache.Reset();

	// Let subclasses do any additional work
	PostReconstructNode();

	if(bForce)
	{
		if (RigGraph)
		{
			RigGraph->NotifyGraphChanged();
		}
	}
	else
	{
		InvalidateNodeTitle();
		OnNodePinsChanged().Broadcast();
	}
}

bool URigVMEdGraphNode::IsDeprecated() const
{
	// we longer consider nodes to be deprecated ever,
	// instead we refer to them as outdated
	return false;
}

bool URigVMEdGraphNode::IsOutDated() const
{
	if(URigVMNode* ModelNode = GetModelNode())
	{
		return ModelNode->IsOutDated();
	}
	return false;
}

FEdGraphNodeDeprecationResponse URigVMEdGraphNode::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);

	if(URigVMNode* ModelNode = GetModelNode())
	{
		const FString DeprecatedMetadata = ModelNode->GetDeprecatedMetadata();
		if (!DeprecatedMetadata.IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DeprecatedMetadata"), FText::FromString(DeprecatedMetadata));
			Response.MessageText = FText::Format(LOCTEXT("RigVMEdGraphNodeDeprecationMessage", "Warning: This node is deprecated from: {DeprecatedMetadata}"), Args);
		}
	}

	return Response;
}

void URigVMEdGraphNode::ReallocatePinsDuringReconstruction(const TArray<UEdGraphPin*>& OldPins)
{
	AllocateDefaultPins();
}

void URigVMEdGraphNode::RewireOldPinsToNewPins(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	for(UEdGraphPin* OldPin : InOldPins)
	{
		const FString Name = OldPin->PinName.ToString();
		FString Left, Right;
		if (!URigVMPin::SplitPinPathAtStart(Name, Left, Right))
		{
			Left = Name;
		}
		const FString OrphanName = FString::Printf(TEXT("%s.%s%s"), *Left, URigVMPin::OrphanPinPrefix, *Right);

		for(UEdGraphPin* NewPin : InNewPins)
		{
			if((OldPin->PinName == NewPin->PinName || NewPin->PinName == OrphanName)
				&& OldPin->Direction == NewPin->Direction)
			{
				if (OldPin->PinType == NewPin->PinType ||
					OldPin->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject() ||
					NewPin->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject())
				{
					// make sure to remove invalid entries from the linked to list
					OldPin->LinkedTo.Remove(nullptr);
				
					NewPin->MovePersistentDataFromOldPin(*OldPin);
					break;
				}
			}
			else
			{
				continue;
			}
		}
	}

	DestroyPinList(InOldPins);
}

void URigVMEdGraphNode::DestroyPinList(TArray<UEdGraphPin*>& InPins)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const FRigVMEditorAssetInterfacePtr Blueprint = GetAsset();
	bool bNotify = false;
	if (Blueprint != nullptr)
	{
		bNotify = !Blueprint->IsRegeneratingOnLoad();
	}

	// Throw away the original pins
	for (UEdGraphPin* Pin : InPins)
	{
		// previously serialized graphs potentially contain
		// nullptr in the subpins
		Pin->SubPins.Remove(nullptr);
		
		Pin->BreakAllPinLinks(bNotify);
		Pin->SubPins.Remove(nullptr);
		UEdGraphNode::DestroyPin(Pin);
	}
}

void URigVMEdGraphNode::PostReconstructNode()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	for (UEdGraphPin* Pin : Pins)
	{
		SetupPinDefaultsFromModel(Pin);
	}

	bCanRenameNode = false;

	if(URigVMController* Controller = GetController())
	{
		if(const URigVMSchema* Schema = Controller->GetSchema())
		{
			bCanRenameNode = Schema->CanSetNodeTitle(Controller, GetModelNode());
		}
	}

	InvalidateNodeColor();
}

void URigVMEdGraphNode::SetColorFromModel(const FLinearColor& InColor)
{
	static const FLinearColor TitleToNodeColor(0.35f, 0.35f, 0.35f, 1.f);
	CachedNodeColor = InColor * TitleToNodeColor;
	CachedTitleColor = InColor;
}

void URigVMEdGraphNode::HandleClearArray(FString InPinPath)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(URigVMController* Controller = GetController())
	{
		Controller->ClearArrayPin(InPinPath);
	}
}

void URigVMEdGraphNode::HandleAddArrayElement(FString InPinPath)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigVMController* Controller = GetController())
	{
		Controller->OpenUndoBracket(TEXT("Add Array Pin"));
		FString PinPath = Controller->AddArrayPin(InPinPath, FString(), true, true);
		Controller->SetPinExpansion(InPinPath, true);
		Controller->SetPinExpansion(PinPath, true);
		Controller->CloseUndoBracket();
	}
}

void URigVMEdGraphNode::HandleRemoveArrayElement(FString InPinPath)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigVMController* Controller = GetController())
	{
		Controller->RemoveArrayPin(InPinPath, true, true);
	}
}

void URigVMEdGraphNode::HandleInsertArrayElement(FString InPinPath)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigVMController* Controller = GetController())
	{
		if (const URigVMPin* ArrayElementPin = GetModelPinFromPinPath(InPinPath))
		{
			if (ArrayElementPin->GetParentPin())
			{
				Controller->OpenUndoBracket(TEXT("Add Array Pin"));
				const FString PinPath = Controller->InsertArrayPin(InPinPath, ArrayElementPin->GetPinIndex() + 1, FString(), true, true);
				Controller->SetPinExpansion(InPinPath, true);
				Controller->SetPinExpansion(PinPath, true);
				Controller->CloseUndoBracket();
			}
		}
	}
}

int32 URigVMEdGraphNode::GetInstructionIndex(bool bAsInput) const
{
	if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetGraph()))
	{
		return RigGraph->GetInstructionIndex(this, bAsInput);
	}
	return INDEX_NONE;
}

bool URigVMEdGraphNode::IsExcludedFromPreview() const
{
	if(const URigVMNode* ModelNode = Cast<URigVMNode>(GetModelNode()))
	{
		return ModelNode->IsExcludedByEarlyExit();
	}
	return false;
}

const FRigVMTemplate* URigVMEdGraphNode::GetTemplate() const
{
	if(CachedTemplate == nullptr)
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		if(URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(GetModelNode()))
		{
			CachedTemplate = TemplateNode->GetTemplate();
		}
		else if(const FRigVMTemplate* Template = Registry.FindTemplate(*ModelNodePath))
		{
			CachedTemplate = Template; 
		}
	}
	return CachedTemplate;
}

void URigVMEdGraphNode::ClearErrorInfo()
{
	bHasCompilerMessage = false;
	// SRigVMGraphNode only updates if the error types do not match so we have
	// clear the error type as well, see SRigVMGraphNode::RefreshErrorInfo()
	ErrorType = (int32)EMessageSeverity::Info + 1;
	ErrorMsg = FString();
	ErrorMessageHashes.Reset();
}

void URigVMEdGraphNode::AddErrorInfo(const EMessageSeverity::Type& InSeverity, const FString& InMessage)
{
	if (ErrorType < InSeverity)
	{
		return;
	}

	const uint32 MessageHash = GetTypeHash(InMessage);
	if (ErrorType == InSeverity)
	{
		if (ErrorMessageHashes.Contains(MessageHash))
		{
			return;
		}
		ErrorMessageHashes.Add(MessageHash);
		ErrorMsg = FString::Printf(TEXT("%s\n%s"), *ErrorMsg, *InMessage);
	}
	else
	{
		ErrorMessageHashes.Reset();
		ErrorMessageHashes.Add(MessageHash);
		ErrorMsg = InMessage;
		ErrorType = InSeverity;
	}
}

void URigVMEdGraphNode::SetErrorInfo(const EMessageSeverity::Type& InSeverity, const FString& InMessage)
{
	ErrorMessageHashes.Reset();
	ErrorMessageHashes.Add(GetTypeHash(InMessage));
	ErrorMsg = InMessage;
	ErrorType = InSeverity;
}

URigVMPin* URigVMEdGraphNode::FindModelPinFromGraphPin(const UEdGraphPin* InGraphPin) const
{
	if(InGraphPin == nullptr)
	{
		return nullptr;
	}
	
	for(const auto& Pair : CachedPins)
	{
		if(Pair.Key.IsValid())
		{
			if(Pair.Value.InputPin == InGraphPin ||
				Pair.Value.OutputPin == InGraphPin)
			{
				return Pair.Key.Get();
			}
		}
	}

	return GetModelPinFromPinPath(InGraphPin->GetName());
}

UEdGraphPin* URigVMEdGraphNode::FindGraphPinFromModelPin(const URigVMPin* InModelPin, bool bAsInput) const
{
	if(InModelPin == nullptr)
	{
		return nullptr;
	}
	
	if(const FPinPair* Pair = CachedPins.Find(InModelPin))
	{
		return bAsInput ? Pair->InputPin : Pair->OutputPin;
	}

	const FString PinPath = InModelPin->GetPinPath();
	for(UEdGraphPin* GraphPin : Pins)
	{
		if((GraphPin->Direction == EGPD_Input) == bAsInput)
		{
			if(GraphPin->GetName() == PinPath)
			{
				return GraphPin;
			}
		}
	}
	
	return nullptr;
}

UEdGraphPin* URigVMEdGraphNode::FindGraphPinFromCategory(const FString& InCategory, bool bAsInput) const
{
	if(const FPinPair* Pair = CachedCategoryPins.Find(InCategory))
	{
		return bAsInput ? Pair->InputPin : Pair->OutputPin;
	}
	return nullptr;
}

void URigVMEdGraphNode::SynchronizeGraphPinNameWithModelPin(const URigVMPin* InModelPin, bool bNotify)
{
	auto SyncGraphPinLambda = [this](const URigVMPin* InModelPin, bool bAsInput) -> bool
	{
		if(UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin, bAsInput))
		{
			const FString OldPinName = GraphPin->PinName.ToString();
			const FString NewPinName = InModelPin->GetPinPath();
			const FText DisplayName = FText::FromName(InModelPin->GetDisplayName());

			if(OldPinName != NewPinName)
			{
				PinPathToModelPin.Remove(OldPinName);
				
				GraphPin->PinName = *NewPinName;
				GraphPin->PinFriendlyName = DisplayName;
				GraphPin->bOrphanedPin = InModelPin->IsOrphanPin() ? 1 : 0;

				PinPathToModelPin.Add(NewPinName, (URigVMPin*)InModelPin);
				return true;
			}

			if(!GraphPin->PinFriendlyName.EqualTo(DisplayName))
			{
				GraphPin->PinFriendlyName = DisplayName;
				return true;
			}
		}

		return false;
	};

	bool bResult = false;
	
	if(SyncGraphPinLambda(InModelPin, true))
	{
		bResult = true;
	}
	if(SyncGraphPinLambda(InModelPin, false))
	{
		bResult = true;
	}

	if(bResult)
	{
		for (const URigVMPin* ModelSubPin : InModelPin->GetSubPins())
		{
			SynchronizeGraphPinNameWithModelPin(ModelSubPin, false);
		}

		if(bNotify)
		{
			// Notify the node widget that the pins have changed.
			OnNodePinsChanged().Broadcast();
		}
	}
}

void URigVMEdGraphNode::SynchronizeGraphPinValueWithModelPin(const URigVMPin* InModelPin)
{
	auto SyncGraphPinLambda = [this](const URigVMPin* InModelPin, bool bAsInput)
	{
		// If the pin has sub-pins, we may need to remove or rebuild the sub-pins.
		if (UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin, bAsInput))
		{
			SetupPinDefaultsFromModel(GraphPin, InModelPin);
		}
	};

	SyncGraphPinLambda(InModelPin, true);
	SyncGraphPinLambda(InModelPin, false);
}

void URigVMEdGraphNode::SynchronizeGraphPinTypeWithModelPin(const URigVMPin* InModelPin)
{
	bool bNotify = false;

	auto SyncGraphPinLambda = [this, &bNotify](const URigVMPin* InModelPin, bool bAsInput)
	{
		// If the pin has sub-pins, we may need to remove or rebuild the sub-pins.
		if (UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin, bAsInput))
		{
			const FEdGraphPinType NewGraphPinType = GetPinTypeForModelPin(InModelPin);
			if(NewGraphPinType != GraphPin->PinType)
			{
				GraphPin->PinType = NewGraphPinType;
				ConfigurePin(GraphPin, InModelPin);

				// Create new sub-pins, if required, to reflect the new type.
				TArray<UEdGraphPin*> GraphSubPinsToKeep;
				if (!InModelPin->GetSubPins().IsEmpty())
				{
					for (const URigVMPin* ModelSubPin : InModelPin->GetSubPins())
					{
						if(UEdGraphPin* GraphSubPin = FindGraphPinFromModelPin(ModelSubPin, bAsInput))
						{
							GraphSubPinsToKeep.Add(GraphSubPin);

							const FEdGraphPinType NewGraphSubPinType = GetPinTypeForModelPin(ModelSubPin);
							if(NewGraphSubPinType != GraphSubPin->PinType)
							{
								GraphSubPin->PinType = NewGraphSubPinType;
								ConfigurePin(GraphSubPin, ModelSubPin);
							}
						}
						else
						{
							CreateGraphPinFromModelPin(ModelSubPin, GraphPin->Direction, GraphPin);
						}
					}
				}

				// If the graph node had other sub-pins, we need to remove those.
				RemoveGraphSubPins(GraphPin, GraphSubPinsToKeep);

				bNotify = true;
			}
		}
	};

	SyncGraphPinLambda(InModelPin, true);
	SyncGraphPinLambda(InModelPin, false);

	if(bNotify)
	{
		// Notify the node widget that the pins have changed.
		OnNodePinsChanged().Broadcast();
	}
}

void URigVMEdGraphNode::SynchronizeGraphPinExpansionWithModelPin(const URigVMPin* InModelPin)
{
	OnNodePinExpansionChanged().Broadcast();
}

void URigVMEdGraphNode::SyncGraphNodeTitleWithModelNodeTitle()
{
	InvalidateNodeTitle();
}

void URigVMEdGraphNode::SyncGraphNodeNameWithModelNodeName(const URigVMNode* InModelNode)
{
	Rename(*InModelNode->GetName());
	ModelNodePath = InModelNode->GetNodePath();
	SyncGraphNodeTitleWithModelNodeTitle();

	TArray<URigVMPin*> AllModelPins = InModelNode->GetAllPinsRecursively();
	for(const URigVMPin* ModelPin : AllModelPins)
	{
		SynchronizeGraphPinNameWithModelPin(ModelPin);
	}
}

bool URigVMEdGraphNode::CreateGraphPinFromModelPin(const URigVMPin* InModelPin, EEdGraphPinDirection InDirection,
	UEdGraphPin* InParentPin)
{
	// don't create output pins for array elements
	if(InDirection == EGPD_Output && InModelPin->IsArrayElement() && !InModelPin->GetParentPin()->IsFixedSizeArray())
	{
		return false;
	}

	// obey to the schema. certain pins may be hidden as per schema contract.
	if (const URigVMEdGraphSchema* Schema = Cast<URigVMEdGraphSchema>(GetSchema()))
	{
		if (Schema->ShouldHidePinOnNode(InModelPin))
		{
			return false;
		}
	}

	const FString Category = InModelPin->GetCategory();
	if(!Category.IsEmpty())
	{
		(void)CreateGraphPinFromCategory(Category, EGPD_Input);
		if(UEdGraphPin* CategoryPin = FindGraphPinFromCategory(Category, true))
		{
			InParentPin = CategoryPin;
		}
	}

	const FPinPair PairConst = CachedPins.FindOrAdd((URigVMPin*)InModelPin);

	auto CreatePinLambda = [this](const URigVMPin* InModelPin, EEdGraphPinDirection InDirection, UEdGraphPin* InParentPin) -> UEdGraphPin*
	{
		// check if we already have a pin like this in the last pins arrays
		const FString PinPath = InModelPin->GetPinPath();
		const FEdGraphPinType PinType = GetPinTypeForModelPin(InModelPin);
		const bool bShouldDisplayAsMutableRef = InModelPin->IsWildCard() || InModelPin->IsDefinedAsInputVariable();

		UEdGraphPin* GraphPin = nullptr;
		UEdGraphPin** ExistingEdGraphPinPtr = LastEdGraphPins.FindByPredicate([PinPath, InDirection, PinType, bShouldDisplayAsMutableRef](const UEdGraphPin* ExistingPin) -> bool
		{
			return !ExistingPin->bWasTrashed &&
				ExistingPin->GetName() == PinPath &&
				ExistingPin->Direction == InDirection &&
				ExistingPin->PinType == PinType &&
				ExistingPin->bDisplayAsMutableRef == bShouldDisplayAsMutableRef;
		});
		if(ExistingEdGraphPinPtr)
		{
			GraphPin = *ExistingEdGraphPinPtr;
			GraphPin->ParentPin = nullptr;
			GraphPin->SubPins.Reset();
			ExistingEdGraphPinPtr = nullptr;
			LastEdGraphPins.Remove(GraphPin);
			Pins.Add(GraphPin);
		}

		if(GraphPin == nullptr)
		{
			GraphPin = CreatePin(InDirection, GetPinTypeForModelPin(InModelPin), FName(*InModelPin->GetPinPath()));
		}
		
		if (GraphPin)
		{
			ConfigurePin(GraphPin, InModelPin);

			if (InParentPin)
			{
				InParentPin->SubPins.Add(GraphPin);
				GraphPin->ParentPin = InParentPin;
			}
			
			for(const URigVMPin* ModelSubPin : InModelPin->GetSubPins())
			{
				CreateGraphPinFromModelPin(ModelSubPin, InDirection, GraphPin);
			}

			SetupPinDefaultsFromModel(GraphPin, InModelPin);
		}

		return GraphPin;
	};

	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;

	bool bResult = false;
	if (InDirection == EGPD_Input && PairConst.InputPin == nullptr)
	{
		InputPin = CreatePinLambda(InModelPin, EGPD_Input, InParentPin);
		bResult = true;
	}
	if (InDirection == EGPD_Output && PairConst.OutputPin == nullptr)
	{
		OutputPin = CreatePinLambda(InModelPin, EGPD_Output, InParentPin);
		bResult = true;
	}

	FPinPair& Pair = CachedPins.FindChecked((URigVMPin*)InModelPin);
	Pair.InputPin = InputPin != nullptr ? InputPin : Pair.InputPin;
	Pair.OutputPin = OutputPin != nullptr ? OutputPin : Pair.OutputPin;
	return Pair.IsValid() && bResult;
}

bool URigVMEdGraphNode::CreateGraphPinFromCategory(const FString& InCategory, EEdGraphPinDirection InDirection)
{
	const FPinPair PairConst = CachedCategoryPins.FindOrAdd(InCategory);

	auto CreatePinLambda = [this](const FString& InCategory, const FString& InCategoryName, EEdGraphPinDirection InDirection, UEdGraphPin* InParentPin) -> UEdGraphPin*
	{
		// check if we already have a pin like this in the last pins arrays
		static const FEdGraphPinType PinType = GetPinTypeForCategoryPin();

		UEdGraphPin* GraphPin = nullptr;
		UEdGraphPin** ExistingEdGraphPinPtr = LastEdGraphPins.FindByPredicate([InCategory, InDirection](const UEdGraphPin* ExistingPin) -> bool
		{
			return !ExistingPin->bWasTrashed &&
				ExistingPin->GetName() == InCategory &&
				ExistingPin->Direction == InDirection &&
				ExistingPin->PinType == PinType;
		});
		if(ExistingEdGraphPinPtr)
		{
			GraphPin = *ExistingEdGraphPinPtr;
			GraphPin->ParentPin = nullptr;
			GraphPin->SubPins.Reset();
			ExistingEdGraphPinPtr = nullptr;
			LastEdGraphPins.Remove(GraphPin);
			Pins.Add(GraphPin);
		}

		if(GraphPin == nullptr)
		{
			GraphPin = CreatePin(InDirection, PinType, FName(*InCategory));
		}
		
		if (GraphPin)
		{
			GraphPin->bHidden = false;
			GraphPin->PinFriendlyName = FText::FromString(InCategoryName);
			GraphPin->bNotConnectable = true;
			GraphPin->bOrphanedPin = false; 
			GraphPin->bDisplayAsMutableRef = false;

			if (InParentPin)
			{
				InParentPin->SubPins.Add(GraphPin);
				GraphPin->ParentPin = InParentPin;
			}
		}

		return GraphPin;
	};

	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;
	UEdGraphPin* ParentPin = nullptr;

	FString CategoryName = InCategory;
	TArray<FString> Categories;
	if(RigVMStringUtils::SplitNodePath(InCategory, Categories))
	{
		CategoryName = Categories.Pop();
		if(Categories.Num() > 0)
		{
			const FString ParentCategory = RigVMStringUtils::JoinNodePath(Categories);
			(void)CreateGraphPinFromCategory(ParentCategory, InDirection);
			ParentPin = FindGraphPinFromCategory(ParentCategory, InDirection == EGPD_Input);
		}
	}

	bool bResult = false;
	if (InDirection == EGPD_Input && PairConst.InputPin == nullptr)
	{
		InputPin = CreatePinLambda(InCategory, CategoryName, EGPD_Input, ParentPin);
		bResult = true;
	}
	if (InDirection == EGPD_Output && PairConst.OutputPin == nullptr)
	{
		OutputPin = CreatePinLambda(InCategory, CategoryName, EGPD_Output, ParentPin);
		bResult = true;
	}

	FPinPair& Pair = CachedCategoryPins.FindChecked(InCategory);
	Pair.InputPin = InputPin != nullptr ? InputPin : Pair.InputPin;
	Pair.OutputPin = OutputPin != nullptr ? OutputPin : Pair.OutputPin;
	return Pair.IsValid() && bResult;
}

void URigVMEdGraphNode::RemoveGraphSubPins(UEdGraphPin* InParentPin, const TArray<UEdGraphPin*>& InPinsToKeep)
{
	TArray<UEdGraphPin*> SubPins = InParentPin->SubPins;

	TArray<URigVMPin*> ModelSubPins;
	TArray<FString> CategorySubPins;
	for (const UEdGraphPin* SubPin: SubPins)
	{
		if(InPinsToKeep.Contains(SubPin))
		{
			continue;
		}
		for(const auto& Pair : CachedPins)
		{
			if(Pair.Key.IsValid())
			{
				if(Pair.Value.InputPin == SubPin ||
					Pair.Value.OutputPin == SubPin)
				{
					ModelSubPins.Add(Pair.Key.Get());
				}
			}
		}
		for(const auto& Pair : CachedCategoryPins)
		{
			if(Pair.Value.InputPin == SubPin ||
				Pair.Value.OutputPin == SubPin)
			{
				CategorySubPins.Add(Pair.Key);
			}
		}
	}

	for(const URigVMPin* ModelSubPin : ModelSubPins)
	{
		PinPathToModelPin.Remove(ModelSubPin->GetPinPath());

		if(const FPinPair* PinPair = CachedPins.Find(ModelSubPin))
		{
			auto Traverse = [this](UEdGraphPin* SubPin)
			{
				if(SubPin == nullptr)
				{
					return;
				}
				
				// Remove this pin from our owned pins
				Pins.Remove(SubPin);

				if (!SubPin->SubPins.IsEmpty())
				{
					RemoveGraphSubPins(SubPin);
				}

				SubPin->MarkAsGarbage();
			};
			
			Traverse(PinPair->InputPin);
			Traverse(PinPair->OutputPin);
		}

		CachedPins.Remove(ModelSubPin);
	}

	for(const FString& CategorySubPin : CategorySubPins)
	{
		PinPathToModelPin.Remove(CategorySubPin);

		if(const FPinPair* PinPair = CachedCategoryPins.Find(CategorySubPin))
		{
			auto Traverse = [this](UEdGraphPin* SubPin)
			{
				if(SubPin == nullptr)
				{
					return;
				}
				
				// Remove this pin from our owned pins
				Pins.Remove(SubPin);

				if (!SubPin->SubPins.IsEmpty())
				{
					RemoveGraphSubPins(SubPin);
				}

				SubPin->MarkAsGarbage();
			};
			
			Traverse(PinPair->InputPin);
			Traverse(PinPair->OutputPin);
		}

		CachedCategoryPins.Remove(CategorySubPin);
	}
	
	InParentPin->SubPins.RemoveAll([InPinsToKeep](const UEdGraphPin* Pin) -> bool
	{
		return !InPinsToKeep.Contains(Pin);
	});
}

bool URigVMEdGraphNode::ModelPinsChanged(bool bForce)
{
	UpdatePinLists();

	// check if any of the pins need to be added / removed
	auto AddMissingPins = [this](const TArray<URigVMPin*>& InModelPins)
	{
		int32 PinsAdded = 0;
		for(const URigVMPin* ModelPin : InModelPins)
		{
			if(FindGraphPinFromModelPin(ModelPin, true) == nullptr && 
				FindGraphPinFromModelPin(ModelPin, false))
			{
				PinsAdded += ModelPinAdded_Internal(ModelPin) ? 1 : 0;
			}
		}
		return PinsAdded;
	};

	auto AddMissingCategoryPins = [this]()
	{
		int32 PinsAdded = 0;
		if(const URigVMNode* ModelNode = GetModelNode())
		{
			for(const FString& Category : ModelNode->GetPinCategories())
			{
				if(FindGraphPinFromCategory(Category, true) == nullptr && 
					FindGraphPinFromCategory(Category, false) == nullptr)
				{
					const TArray<URigVMPin*> PinsForCategory = ModelNode->GetPinsForCategory(Category);
					if(!PinsForCategory.IsEmpty())
					{
						const ERigVMPinDirection PinDirection = PinsForCategory[0]->GetDirection();
						if(PinDirection == ERigVMPinDirection::Input ||
							PinDirection == ERigVMPinDirection::Visible ||
							PinDirection == ERigVMPinDirection::IO)
						{
							PinsAdded += CategoryPinAdded_Internal(Category, EGPD_Input) ? 1 : 0;
						}
						if(PinDirection == ERigVMPinDirection::Output ||
							PinDirection == ERigVMPinDirection::IO)
						{
							PinsAdded += CategoryPinAdded_Internal(Category, EGPD_Output) ? 1 : 0;
						}
					}
				}
			}
		}
		return PinsAdded;
	};
	auto RemoveObsoletePins = [this]()
	{
		CachedPins = CachedPins.FilterByPredicate([](const TPair<TWeakObjectPtr<URigVMPin>, FPinPair>& InPair) -> bool
		{
			return InPair.Key.IsValid();
		});
		
		TArray<URigVMPin*> PinsToRemove; 
		for(const auto& Pair : CachedPins)
		{
			URigVMPin* ModelPin = Pair.Key.Get();
			TArray<URigVMPin*>& List = PinListForPin(ModelPin);
			bool bRemove = true;
			if (URigVMPin** FoundPin = List.FindByKey(ModelPin->GetRootPin()))
			{
				if (ModelPin->GetParentPin() == nullptr ||
					(*FoundPin)->FindSubPin(ModelPin->GetSegmentPath()))
				{
					bRemove = false;
				}
			}
			if(bRemove)
			{
				PinsToRemove.Add(ModelPin);
			}
		}

		int32 PinsRemoved = 0;
		for(const URigVMPin* ModelPin : PinsToRemove)
		{
			PinsRemoved += ModelPinRemoved_Internal(ModelPin) ? 1 : 0;
		}

		if(const URigVMNode* ModelNode = GetModelNode())
		{
			TArray<FString> Categories;
			CachedCategoryPins.GetKeys(Categories);
			
			for(const FString& Category : Categories)
			{
				if(!ModelNode->GetPinCategories().Contains(Category))
				{
					PinsRemoved += CategoryPinRemoved_Internal(Category);
				}
				else if(ModelNode->GetPinsForCategory(Category).IsEmpty())
				{
					PinsRemoved += CategoryPinRemoved_Internal(Category);
				}
			}
		}
		return PinsRemoved;
	};

	auto OrderPins = [this](const TArray<URigVMPin*>& InModelPins) -> int32
	{
		if(InModelPins.Num() < 2)
		{
			return 0;
		}

		TArray<UEdGraphPin*> OrderedGraphPins;
		OrderedGraphPins.Reserve(InModelPins.Num() * 2);
		int32 LastInputIndex = INDEX_NONE;
		int32 LastOutputIndex = INDEX_NONE;
		int32 PinsToReorder = 0;

		for(const URigVMPin* ModelPin : InModelPins)
		{
			if(UEdGraphPin* InputGraphPin = FindGraphPinFromModelPin(ModelPin, true))
			{
				OrderedGraphPins.Add(InputGraphPin);
				const int32 InputPinIndex = Pins.Find(InputGraphPin);
				if(LastInputIndex > InputPinIndex)
				{
					PinsToReorder++;
				}
				LastInputIndex = InputPinIndex;
			}
			
			if(UEdGraphPin* OutputGraphPin = FindGraphPinFromModelPin(ModelPin, false))
			{
				OrderedGraphPins.Add(OutputGraphPin);
				const int32 OutputPinIndex = Pins.Find(OutputGraphPin);
				if(LastOutputIndex > OutputPinIndex)
				{
					PinsToReorder++;
				}
				LastOutputIndex = OutputPinIndex;
			}
		}

		if(PinsToReorder > 0)
		{
			for(UEdGraphPin* GraphPinInOrder : OrderedGraphPins)
			{
				Pins.Remove(GraphPinInOrder);
			}

			OrderedGraphPins.Append(Pins);
			Swap(OrderedGraphPins, Pins);
		}

		return PinsToReorder;
	};

	auto ReparentPins = [this](const TArray<URigVMPin*>& InModelPins) -> int32
	{
		int32 PinsReparented = 0;
		for(const URigVMPin* ModelPin : InModelPins)
		{
			for(int32 Phase = 0; Phase < 2; Phase++)
			{
				if(UEdGraphPin* EdGraphPin = FindGraphPinFromModelPin(ModelPin, Phase == 0))
				{
					UEdGraphPin* ParentEdGraphPin = nullptr;

					const FString Category = ModelPin->GetCategory();
					if(Category.IsEmpty())
					{
						if(const URigVMPin* ParentModelPin = ModelPin->GetParentPin())
						{
							ParentEdGraphPin = FindGraphPinFromModelPin(ParentModelPin, Phase == 0);
						}
					}
					else
					{
						ParentEdGraphPin = FindGraphPinFromCategory(Category, Phase == 0);
					}

					if(EdGraphPin->ParentPin != ParentEdGraphPin)
					{
						if(EdGraphPin->ParentPin)
						{
							EdGraphPin->ParentPin->SubPins.Remove(EdGraphPin);
							EdGraphPin->ParentPin = nullptr;
						}
						if(ParentEdGraphPin)
						{
							ParentEdGraphPin->SubPins.Add(EdGraphPin);
							EdGraphPin->ParentPin = ParentEdGraphPin;
						}
						PinsReparented++;
					}
				}
			}
		}
		return PinsReparented;
	};

	auto ReparentCategoryPins = [this](const TArray<FString>& InCategories) -> int32
	{
		int32 PinsReparented = 0;
		for(const FString& Category : InCategories)
		{
			for(int32 Phase = 0; Phase < 2; Phase++)
			{
				if(UEdGraphPin* EdGraphPin = FindGraphPinFromCategory(Category, Phase == 0))
				{
					UEdGraphPin* ParentEdGraphPin = nullptr;
					
					FString ParentCategory, CategoryName;
					if(RigVMStringUtils::SplitNodePathAtEnd(Category, ParentCategory, CategoryName))
					{
						ParentEdGraphPin = FindGraphPinFromCategory(ParentCategory, Phase == 0);
					}

					if(EdGraphPin->ParentPin != ParentEdGraphPin)
					{
						if(EdGraphPin->ParentPin)
						{
							EdGraphPin->ParentPin->SubPins.Remove(EdGraphPin);
							EdGraphPin->ParentPin = nullptr;
						}
						if(ParentEdGraphPin)
						{
							ParentEdGraphPin->SubPins.Add(EdGraphPin);
							EdGraphPin->ParentPin = ParentEdGraphPin;
						}
						PinsReparented++;
					}
				}
			}
		}
		return PinsReparented;
	};

	int32 PinsAdded = 0;
	PinsAdded += AddMissingCategoryPins();
	PinsAdded += AddMissingPins(ExecutePins);
	PinsAdded += AddMissingPins(OutputPins);
	PinsAdded += AddMissingPins(InputOutputPins);
	PinsAdded += AddMissingPins(InputPins);

	int32 PinsReparented = 0;
	if(const URigVMNode* ModelNode = GetModelNode())
	{
		PinsReparented += ReparentCategoryPins(ModelNode->GetPinCategories());
	}
	PinsReparented += ReparentPins(InputPins);
	PinsReparented += ReparentPins(InputOutputPins);
	PinsReparented += ReparentPins(OutputPins);
	PinsReparented += ReparentPins(ExecutePins);

	const int32 PinsRemoved = RemoveObsoletePins();

	// working through it in the opposite order
	// due to the use of Append within the lambda
	int32 PinsReordered = 0;
	PinsReordered += OrderPins(InputPins);
	PinsReordered += OrderPins(InputOutputPins);
	PinsReordered += OrderPins(OutputPins);
	PinsReordered += OrderPins(ExecutePins);

	int32 PinLabelsChanged = 0;
	auto SyncPinLabel = [](const URigVMPin* InModelPin, const FPinPair& OutPinPair)
	{
		int32 PinLabelsChanged = 0;
		if(InModelPin)
		{
			const FName DesiredFriendlyName = InModelPin->GetDisplayName();
			if(!DesiredFriendlyName.IsNone())
			{
				const FText DesiredFriendlyNameText = FText::FromName(DesiredFriendlyName);
				for(int32 Phase = 0; Phase < 2; Phase++)
				{
					UEdGraphPin* Pin = Phase == 0 ? OutPinPair.InputPin : OutPinPair.OutputPin;
					if(Pin)
					{
						if(!Pin->PinFriendlyName.EqualTo(DesiredFriendlyNameText))
						{
							Pin->PinFriendlyName = DesiredFriendlyNameText;
							PinLabelsChanged++;
						}
					}
				}
			}
		}
		return PinLabelsChanged;
	};
	for(const TPair<FString,FPinPair>& Pair : CachedCategoryPins)
	{
		if(const URigVMPin* ModelPin = FindModelPinFromGraphPin(Pair.Value.InputPin != nullptr ? Pair.Value.InputPin : Pair.Value.OutputPin))
		{
			PinLabelsChanged += SyncPinLabel(ModelPin, Pair.Value);
		}
	}
	for(const TPair<TWeakObjectPtr<URigVMPin>,FPinPair>& Pair : CachedPins)
	{
		if(Pair.Key.IsValid())
		{
			PinLabelsChanged += SyncPinLabel(Pair.Key.Get(), Pair.Value);
		}
	}

	const bool bResult = (PinsAdded > 0) || (PinLabelsChanged > 0) || (PinsRemoved > 0) || (PinsReordered > 0) || (PinsReparented > 0); 
	if(bResult || bForce)
	{
		OnNodePinsChanged().Broadcast();
	}
	return bResult;
}

bool URigVMEdGraphNode::ModelPinAdded(const URigVMPin* InModelPin)
{
	if(InModelPin == nullptr)
	{
		return false;
	}

	UpdatePinLists();

	bool bResult = ModelPinAdded_Internal(InModelPin);
	if(bResult)
	{
		OnNodePinsChanged().Broadcast();
	}
	return bResult;
}

bool URigVMEdGraphNode::ModelPinAdded_Internal(const URigVMPin* InModelPin)
{
	bool bResult = false;

	// conversion nodes don't show sub pins
	if(!InModelPin->IsRootPin())
	{
		if(DrawAsCompactNode())
		{
			return false;
		}
	}

	UEdGraphPin* InputParentPin = FindGraphPinFromModelPin(InModelPin->GetParentPin(), true);
	UEdGraphPin* OutputParentPin = FindGraphPinFromModelPin(InModelPin->GetParentPin(), false);

	if(InModelPin->GetDirection() == ERigVMPinDirection::Input ||
		InModelPin->GetDirection() == ERigVMPinDirection::Visible ||
		InModelPin->GetDirection() == ERigVMPinDirection::IO)
	{
		if(CreateGraphPinFromModelPin(InModelPin, EGPD_Input, InputParentPin))
		{
			bResult = true;
		}
	}
	
	if(InModelPin->GetDirection() == ERigVMPinDirection::Output ||
		InModelPin->GetDirection() == ERigVMPinDirection::IO)
	{
		if(CreateGraphPinFromModelPin(InModelPin, EGPD_Output, OutputParentPin))
		{
			bResult = true;
		}
	}
	
	return bResult;
}

bool URigVMEdGraphNode::ModelPinRemoved(const URigVMPin* InModelPin)
{
	if(InModelPin == nullptr)
	{
		return false;
	}

	UpdatePinLists();

	const bool bResult = ModelPinRemoved_Internal(InModelPin);
	if(bResult)
	{
		OnNodePinsChanged().Broadcast();
	}
	return bResult;
}

bool URigVMEdGraphNode::DrawAsCompactNode() const
{
	if(CVarRigVMDisableCompactNodes.GetValueOnAnyThread())
	{
		return false;
	}
	
	if(!DrawAsCompactNodeCache.IsSet())
	{
		DrawAsCompactNodeCache = false;
		if(const URigVMTemplateNode* TemplateModelNode = Cast<URigVMTemplateNode>(GetModelNode()))
		{
			if(TemplateModelNode->GetNotation() == RigVMTypeUtils::GetCastTemplateNotation() ||
				TemplateModelNode->GetNotation() == FRigVMDispatch_CastObject().GetTemplateNotation() ||
				TemplateModelNode->GetNotation() == FRigVMDispatch_CastIntToEnum().GetTemplateNotation() ||
				TemplateModelNode->GetNotation() == FRigVMDispatch_CastEnumToInt().GetTemplateNotation())
			{
				DrawAsCompactNodeCache = true;
				
				// if the node has links on any subpin - we can't draw it as a compact node.
				const TArray<URigVMLink*> Links = TemplateModelNode->GetLinks();
				if(Links.ContainsByPredicate([TemplateModelNode](const URigVMLink* Link)
				{
					if(const URigVMPin* SourcePin = Link->GetSourcePin())
					{
						if(SourcePin->GetNode() == TemplateModelNode)
						{
							return !SourcePin->IsRootPin();
						}
					}
					if(const URigVMPin* TargetPin = Link->GetTargetPin())
					{
						if(TargetPin->GetNode() == TemplateModelNode)
						{
							return !TargetPin->IsRootPin();
						}
					}
					return false;
				}))
				{
					DrawAsCompactNodeCache = false;
				}

				// if the node has any injected nodes - we can't draw it as compact
				if(TemplateModelNode->GetPins().ContainsByPredicate([](const URigVMPin* Pin)
				{
					return Pin->HasInjectedNodes();
				}))
				{
					DrawAsCompactNodeCache = false;
				}
			}
		}
	}
	return DrawAsCompactNodeCache.GetValue();
}

void URigVMEdGraphNode::SetModelNode(URigVMNode* InModelNode)
{
	ModelNodePath = InModelNode->GetNodePath();
	CachedModelNode = InModelNode;
	SyncGraphNodeTitleWithModelNodeTitle();
	//AllocateDefaultPins();
	ReconstructNode();
	//PostReconstructNode();
}

bool URigVMEdGraphNode::IsFadedOut() const
{
	return GetFadedOutState() < 1.f - SMALL_NUMBER;
}

float URigVMEdGraphNode::GetFadedOutState() const
{
#if WITH_EDITOR
	if (FadedOutOverride.IsSet())
	{
		return FadedOutOverride.GetValue();
	}
	
	bool bIsVariableUsed = false;
	if (URigVMNode* ModelNode = GetModelNode())
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelNode))
		{
			if (VariableNode->IsGetter())
			{
				if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetGraph()))
				{
					TArray<URigVMNode*> TargetNodes = VariableNode->GetLinkedTargetNodes();
					for (URigVMNode* TargetNode : TargetNodes)
					{
						if (URigVMEdGraphNode* TargetEdNode = Cast<URigVMEdGraphNode>(Graph->FindNodeForModelNodeName(TargetNode->GetFName())))
						{
							if (!TargetEdNode->IsFadedOut())
							{
								bIsVariableUsed = true;
								break;
							}
						}
					}
				}
			}
		}
	}
	
	if (!bIsVariableUsed)
	{
		if(GetInstructionIndex(true) == INDEX_NONE)
		{
			if (GetDefault<URigVMEditorSettings>()->bFadeOutUnrelatedNodes)
			{
				return 0.25f;
			}
		}
	}
	
	if (IsExcludedFromPreview())
	{
		bool bWasRunDuringLastEval = false;
		if (CachedModelNode.IsValid())
		{
			if(IRigVMEditorAssetInterface* Blueprint = GetImplementingOuter<IRigVMEditorAssetInterface>())
			{
				if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
				{
					const int32 Count = CachedModelNode->GetInstructionVisitedCount(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
					bWasRunDuringLastEval = Count > 0;
				}
			}
		}

		if (!bWasRunDuringLastEval)
		{
			return 0.4f;
		}
	}
#endif
	return 1.f;
}

void URigVMEdGraphNode::OverrideFadeOutState(TOptional<float> InFadedOutState)
{
	FadedOutOverride = InFadedOutState;
}

bool URigVMEdGraphNode::ModelPinRemoved_Internal(const URigVMPin* InModelPin)
{
	auto RemoveGraphPin = [this](const URigVMPin* InModelPin, bool bAsInput)
	{
		if(UEdGraphPin* GraphPin = FindGraphPinFromModelPin(InModelPin, bAsInput))
		{
			RemoveGraphSubPins(GraphPin);
			Pins.Remove(GraphPin);
			GraphPin->MarkAsGarbage();
		}
	};
	RemoveGraphPin(InModelPin, true);
	RemoveGraphPin(InModelPin, false);
		
	const bool bResult = PinPathToModelPin.Remove(InModelPin->GetPinPath()) > 0;
	CachedPins.Remove(InModelPin);
	return bResult;
}

bool URigVMEdGraphNode::CategoryPinAdded_Internal(const FString& InCategory, EEdGraphPinDirection InDirection)
{
	return CreateGraphPinFromCategory(InCategory, InDirection);
}

bool URigVMEdGraphNode::CategoryPinRemoved_Internal(const FString& InCategory)
{
	auto RemoveGraphPin = [this](const FString& InCategory, bool bAsInput)
	{
		if(UEdGraphPin* GraphPin = FindGraphPinFromCategory(InCategory, bAsInput))
		{
			check(GraphPin->SubPins.IsEmpty());
			Pins.Remove(GraphPin);
			GraphPin->MarkAsGarbage();
		}
	};
	RemoveGraphPin(InCategory, true);
	RemoveGraphPin(InCategory, false);
		
	const bool bResult = PinPathToModelPin.Remove(InCategory) > 0;
	CachedCategoryPins.Remove(InCategory);
	return bResult;
}

void URigVMEdGraphNode::UpdatePinLists()
{
	ExecutePins.Reset();
	OutputPins.Reset();
	InputOutputPins.Reset();
	InputPins.Reset();

	if (const URigVMNode* ModelNode = GetModelNode())
	{
		for(int32 PinListIndex=0; PinListIndex<2; PinListIndex++)
		{
			const TArray<URigVMPin*>& ModelPins = PinListIndex == 0 ? ModelNode->GetPins() : ModelNode->GetOrphanedPins();
			for (URigVMPin* ModelPin : ModelPins)
			{
				if (ModelPin->ShowInDetailsPanelOnly())
				{
					continue;
				}
				if (ModelPin->GetDirection() == ERigVMPinDirection::IO)
				{
					if (ModelPin->IsStruct())
					{
						const UScriptStruct* ScriptStruct = ModelPin->GetScriptStruct();
						if (ScriptStruct && ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
						{
							ExecutePins.Add(ModelPin);
							continue;
						}
					}
					InputOutputPins.Add(ModelPin);
				}
				else if (ModelPin->GetDirection() == ERigVMPinDirection::Input || 
					ModelPin->GetDirection() == ERigVMPinDirection::Visible)
				{
					InputPins.Add(ModelPin);
				}
				else if (ModelPin->GetDirection() == ERigVMPinDirection::Output)
				{
					OutputPins.Add(ModelPin);
				}
			}
		}
	}
}

void URigVMEdGraphNode::AllocateDefaultPins()
{
	UpdatePinLists();

	CachedPins.Reset();
	CachedCategoryPins.Reset();
	PinPathToModelPin.Reset();

	auto CreateGraphPins = [this](const TArray<URigVMPin*>& InModelPins)
	{
		for (const URigVMPin* ModelPin : InModelPins)
		{
			ModelPinAdded_Internal(ModelPin);
		}
	};

	CreateGraphPins(ExecutePins);
	CreateGraphPins(OutputPins);
	CreateGraphPins(InputOutputPins);
	CreateGraphPins(InputPins);

	// Fill the variable list
	ExternalVariables.Reset();
	if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(GetModelNode()))
	{
		if(FunctionReferenceNode->RequiresVariableRemapping())
		{
			TArray<FRigVMExternalVariable> CurrentExternalVariables = FunctionReferenceNode->GetExternalVariables();
			for(const FRigVMExternalVariable& CurrentExternalVariable : CurrentExternalVariables)
			{
				ExternalVariables.Add(MakeShared<FRigVMExternalVariable>(CurrentExternalVariable));
			}
		}
	}
}

FLinearColor URigVMEdGraphNode::GetNodeTitleColor() const
{
	return CachedTitleColor;
}

FLinearColor URigVMEdGraphNode::GetNodeBodyTintColor() const
{
#if WITH_EDITOR
	if(ProfilingColor.IsSet())
	{
		return ProfilingColor.GetValue();
	}
	if (HighlightingColor.IsSet())
	{
		return HighlightingColor.GetValue();
	}
#endif
	return CachedNodeColor;
}

bool URigVMEdGraphNode::ShowPaletteIconOnNode() const
{
	if (URigVMNode* ModelNode = GetModelNode())
	{
		return ModelNode->IsEvent() ||
			ModelNode->IsA<URigVMInvokeEntryNode>() ||
			ModelNode->IsA<URigVMFunctionInterfaceNode>() ||
			ModelNode->IsA<URigVMFunctionReferenceNode>() ||
			ModelNode->IsA<URigVMCollapseNode>() ||
			ModelNode->IsA<URigVMUnitNode>() ||
			ModelNode->IsA<URigVMDispatchNode>() ||
			ModelNode->IsLoopNode();
	}
	return false;
}

FSlateIcon URigVMEdGraphNode::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = FLinearColor::White;

	static FSlateIcon FunctionIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	static FSlateIcon EventIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	static FSlateIcon EntryReturnIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Default_16x");
	static FSlateIcon CollapsedNodeIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.SubGraph_16x");
	static FSlateIcon TemplateNodeIcon("RigVMEditorStyle", "RigVM.Template");

	if (URigVMNode* ModelNode = GetModelNode())
	{
		if (ModelNode->IsEvent() || ModelNode->IsA<URigVMInvokeEntryNode>())
		{
			return EventIcon;
		}

		URigVMNode* InnerNode = ModelNode;
		while(const URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(InnerNode))
		{
			InnerNode = AggregateNode->GetFirstInnerNode();
		}
		if (InnerNode)
		{
			ModelNode = InnerNode;
		}
		else
		{
			if(!ModelNode->HasAnyFlags(RF_NeedLoad))
			{
				const FString Message = FString::Printf(TEXT("Could not find first inner node in aggregate node %s in package %s"), *ModelNode->GetPathName(), *GetPackage()->GetPathName());
				FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *Message, *FString());
			}
		}

		if (ModelNode->IsA<URigVMFunctionReferenceNode>())
		{ 
			return FunctionIcon;
		}

		if (ModelNode->IsA<URigVMCollapseNode>())
		{
			return CollapsedNodeIcon;
		}

		if (ModelNode->IsA<URigVMFunctionInterfaceNode>())
		{
			return EntryReturnIcon;
		}

		const UScriptStruct* MetadataScriptStruct = nullptr;
		if (const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
		{
			MetadataScriptStruct = UnitNode->GetScriptStruct();
		}
		else if (const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
		{
			if (const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
			{
				MetadataScriptStruct = Factory->GetScriptStruct();
			}
		}

		if(MetadataScriptStruct && MetadataScriptStruct->HasMetaDataHierarchical(FRigVMStruct::IconMetaName))
		{
			FString IconPath;
			const int32 NumOfIconPathNames = 4;
			
			FName IconPathNames[NumOfIconPathNames] = {
				NAME_None, // StyleSetName
				NAME_None, // StyleName
				NAME_None, // SmallStyleName
				NAME_None  // StatusOverlayStyleName
			};

			// icon path format: StyleSetName|StyleName|SmallStyleName|StatusOverlayStyleName
			// the last two names are optional, see FSlateIcon() for reference
			MetadataScriptStruct->GetStringMetaDataHierarchical(FRigVMStruct::IconMetaName, &IconPath);

			int32 NameIndex = 0;

			while (!IconPath.IsEmpty() && NameIndex < NumOfIconPathNames)
			{
				FString Left;
				FString Right;

				if (!IconPath.Split(TEXT("|"), &Left, &Right))
				{
					Left = IconPath;
				}

				IconPathNames[NameIndex] = FName(*Left);

				NameIndex++;
				IconPath = Right;
			}
			
			return FSlateIcon(IconPathNames[0], IconPathNames[1], IconPathNames[2], IconPathNames[3]);
		}

		if (const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelNode))
		{
			if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
			{
				if(Template->NumPermutations() > 1)
				{
					return TemplateNodeIcon;
				}
			}
		}

		return ModelNode->GetNodeIcon();
	}

	return FunctionIcon;
}

void URigVMEdGraphNode::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
#if WITH_EDITOR
	const URigVMEdGraphSchema* Schema = Cast<URigVMEdGraphSchema>(GetSchema());
	if(const FRigVMEditorAssetInterfacePtr Blueprint = GetAsset())
	{
		return Blueprint->GetEditorModule()->GetContextMenuActions(Schema, Menu, Context);
	}
	IRigVMEditorModule::Get().GetContextMenuActions(Schema, Menu, Context);
#endif
}

bool URigVMEdGraphNode::IsPinExpanded(const FString& InPinPath)
{
	if (URigVMPin* ModelPin = GetModelPinFromPinPath(InPinPath))
	{
		return ModelPin->IsExpanded();
	}
	return false;
}

void URigVMEdGraphNode::DestroyNode()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		BreakAllNodeLinks();

		if(IRigVMEditorAssetInterface* RigVMBlueprint = Graph->GetImplementingOuter<IRigVMEditorAssetInterface>())
		{
			if(PropertyName_DEPRECATED.IsValid())
			{
				FRigVMBlueprintUtils::RemoveMemberVariableIfNotUsed(RigVMBlueprint->GetObject(), PropertyName_DEPRECATED);
			}
		}
	}

	UEdGraphNode::DestroyNode();
}

void URigVMEdGraphNode::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	CopyPinDefaultsToModel(Pin, true, true);
}

void URigVMEdGraphNode::CopyPinDefaultsToModel(UEdGraphPin* Pin, bool bUndo, bool bPrintPythonCommand)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Pin->Direction != EGPD_Input)
	{
		return;
	}

	if (const URigVMPin* ModelPin = FindModelPinFromGraphPin(Pin))
	{
		if (ModelPin->GetSubPins().Num() > 0)
		{
			return;
		}

		FString DefaultValue = Pin->DefaultValue;

		if(DefaultValue.IsEmpty() && (
			Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
			Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class ||
			Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject ||
			Pin->PinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes ||
			Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface
			))
		{
			if(Pin->DefaultObject)
			{
				DefaultValue = Pin->DefaultObject->GetPathName();
			}
		}
		
		if (DefaultValue == FName(NAME_None).ToString() && Pin->PinType.PinSubCategory == UEdGraphSchema_K2::PC_Name)
		{
			DefaultValue = FString();
		}

		if (ModelPin->GetDefaultValue() != DefaultValue)
		{
			if (URigVMController* Controller = GetController())
			{
				Controller->SetPinDefaultValue(ModelPin->GetPinPath(), DefaultValue, false, true, false, bPrintPythonCommand);
			}
		}
	}
}

URigVMBlueprint* URigVMEdGraphNode::GetBlueprint() const
{
	if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return Cast<URigVMBlueprint>(Graph->GetAsset().GetObject());
	}
	return nullptr;
}

FRigVMEditorAssetInterfacePtr URigVMEdGraphNode::GetAsset() const
{
	if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return Graph->GetAsset();
	}
	return nullptr;
}

URigVMGraph* URigVMEdGraphNode::GetModel() const
{
	if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return Graph->GetModel();
	}
	return nullptr;
}

URigVMController* URigVMEdGraphNode::GetController() const
{
	if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		return Graph->GetController();
	}
	return nullptr;
}

URigVMNode* URigVMEdGraphNode::GetModelNode() const
{
	URigVMEdGraphNode* MutableThis = (URigVMEdGraphNode*)this;
	if (CachedModelNode.IsValid())
	{
		if (CachedModelNode.Get()->GetOuter() == GetTransientPackage())
		{
			MutableThis->CachedModelNode.Reset();
		}
		else
		{
			return CachedModelNode.Get();
		}
	}

	if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetOuter()))
	{
		if (URigVMGraph* Model = GetModel())
		{
			MutableThis->CachedModelNode = TWeakObjectPtr<URigVMNode>(Model->FindNode(ModelNodePath));
			return MutableThis->CachedModelNode.Get();
		}
	}

	return nullptr;
}

FName URigVMEdGraphNode::GetModelNodeName() const
{
	if (URigVMNode* ModelNode = GetModelNode())
	{
		return ModelNode->GetFName();
	}
	return NAME_None;
}

const FString& URigVMEdGraphNode::GetModelNodePath() const
{
	return ModelNodePath;
}

URigVMPin* URigVMEdGraphNode::GetModelPinFromPinPath(const FString& InPinPath) const
{
	if (TWeakObjectPtr<URigVMPin> const* CachedModelPinPtr = PinPathToModelPin.Find(InPinPath))
	{
		if(CachedModelPinPtr->IsValid())
		{
			URigVMPin* CachedModelPin = CachedModelPinPtr->Get();
			if (!CachedModelPin->HasAnyFlags(RF_Transient) && CachedModelPin->GetNode())
			{
				return CachedModelPin;
			}
		}
	}

	if (const URigVMNode* ModelNode = GetModelNode())
	{
		const FString PinPath = InPinPath.RightChop(ModelNode->GetNodePath().Len() + 1);
		URigVMPin* ModelPin = ModelNode->FindPin(PinPath);
		if (ModelPin)
		{
			URigVMEdGraphNode* MutableThis = (URigVMEdGraphNode*)this;
			MutableThis->PinPathToModelPin.FindOrAdd(InPinPath) = ModelPin;
		}
		return ModelPin;
	}
	
	return nullptr;
}

void URigVMEdGraphNode::HandleAddAggregateElement(const FString& InNodePath)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (URigVMController* Controller = GetController())
	{
		Controller->AddAggregatePin(InNodePath, FString(), FString(), true, true);
	}	
}

void URigVMEdGraphNode::SetupPinDefaultsFromModel(UEdGraphPin* Pin, const URigVMPin* InModelPin)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Pin->Direction != EGPD_Input)
	{
		return;
	}

	if(InModelPin == nullptr)
	{
		InModelPin = FindModelPinFromGraphPin(Pin);
	}

	// remove stale subpins
	Pin->SubPins.Remove(nullptr);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if (InModelPin && IsValid(InModelPin))
	{
		if (InModelPin->GetSubPins().Num() > 0)
		{
			return;
		}

		FString DefaultValueString = InModelPin->GetDefaultValue();
		if (DefaultValueString.IsEmpty() && InModelPin->GetCPPType() == TEXT("FName"))
		{
			DefaultValueString = FName(NAME_None).ToString();
		}
		K2Schema->GetPinDefaultValuesFromString(Pin->PinType, Pin->GetOwningNodeUnchecked(), DefaultValueString, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue);
	}
}

FText URigVMEdGraphNode::GetTooltipText() const
{
	if(URigVMNode* ModelNode = GetModelNode())
	{
		FText Tooltip = ModelNode->GetToolTipText();
		if(IsOutDated())
		{
			return FText::Format(
				LOCTEXT("OutDatedTooltipFormat", "This node is outdated and can potentially be upgraded.\nFor this right click and choose 'Upgrade Nodes'.\n\n{0}"),
					Tooltip);
		}
		return Tooltip;;
	}
	return FText::FromString(ModelNodePath);
}

void URigVMEdGraphNode::InvalidateNodeTitle() const
{
	NodeTitle = FText();
	FullNodeTitle = FText();
	NodeTitleDirtied.Broadcast();
}

void URigVMEdGraphNode::InvalidateNodeColor()
{
	if(const URigVMNode* ModelNode = GetModelNode())
	{
		if (ModelNode->HasEarlyExitMarker())
		{
			static constexpr FLinearColor PreviewColor = FLinearColor(0.f,0.f,1.f,1.f);
			SetColorFromModel(PreviewColor);
		}
		else if (ModelNode->IsOutDated())
		{
			static const FLinearColor WarningColor = FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor");
			SetColorFromModel(WarningColor);
		}
		else
		{
			SetColorFromModel(ModelNode->GetNodeColor());
		}
	}

}

bool URigVMEdGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const
{
	return InSchema->IsA<URigVMEdGraphSchema>();
}

void URigVMEdGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	Super::AutowireNewNode(FromPin);

	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	Schema->AutowireNewNode(this, FromPin);
}

bool URigVMEdGraphNode::IsSelectedInEditor() const
{
	URigVMNode* ModelNode = GetModelNode();
	if (ModelNode)
	{
		return ModelNode->IsSelected();
	}
	return false;
}

bool URigVMEdGraphNode::ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const
{
	if (Cast<URigVMRerouteNode>(GetModelNode()))
	{
		OutInputPinIndex = 0;
		OutOutputPinIndex = Pins.Num()/2;

		if (Pins[OutOutputPinIndex]->Direction != EGPD_Output || Pins[OutOutputPinIndex]->ParentPin != nullptr)
		{
			for (int32 i=0; i<Pins.Num(); ++i)
			{
				if (Pins[i]->Direction == EGPD_Output && Pins[i]->ParentPin == nullptr)
				{
					OutOutputPinIndex = i;
					break;
				}
			}
		}
		return true;
	}
	return false;
}

void URigVMEdGraphNode::BeginDestroy()
{
	for(UEdGraphPin* Pin : Pins)
	{
		Pin->SubPins.Remove(nullptr);
	}
	Super::BeginDestroy();
}

#if WITH_EDITOR
#if WITH_RIGVMLEGACYEDITOR
void URigVMEdGraphNode::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);
	if(const URigVMPin* ModelPin = FindModelPinFromGraphPin(Pin))
	{
		OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinCategory, FText::FromName(Pin->PinType.PinCategory));
		OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinSubCategory, FText::FromName(Pin->PinType.PinSubCategory));
		if (Pin->PinType.PinSubCategoryObject.IsValid())
		{
			OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_ObjectClass, FText::FromString(Pin->PinType.PinSubCategoryObject->GetPathName()));
		}
		OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_IsArray, FText::Format(LOCTEXT("RigVMNodePinIsArray", "{0}"), Pin->PinType.IsArray() ? 1 : 0));
		
		if (ModelPin->IsBoundToVariable())
		{
			OutTaggedMetaData.Add(FSearchTagDataPair(FRigVMSearchTags::FiB_PinBinding, FText::FromString(ModelPin->GetBoundVariablePath())));
		}
	}
}
#else
void URigVMEdGraphNode::AddRigVMSearchMetaDataInfo(TArray<struct UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const
{
	// Copied from EdGraphNode::AddSearchMetaDataInfo
	if (GetSchema() == nullptr)
	{
		ensure(false);
		return;
	}

	// Searchable - Primary label for the item in the search results
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_Name, GetNodeTitle(ENodeTitleType::ListView)));

	// Searchable - As well as being searchable, this displays in the tooltip for the node
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_ClassName, FText::FromString(GetClass()->GetName())));

	// Non-searchable - Used to lookup the node when attempting to jump to it
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_NodeGuid, FText::FromString(NodeGuid.ToString(EGuidFormats::Digits))));

	// Non-searchable - Important for matching pin types with icons and colors, stored here so that each pin does not store it
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_SchemaName, FText::FromString(GetSchema()->GetClass()->GetName())));

	// Non-Searchable - Used to display the icon and color for this node for better visual identification.
	FLinearColor GlyphColor = FLinearColor::White;
	FSlateIcon Icon = GetIconAndTint(GlyphColor);
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_Glyph, FText::FromName(Icon.GetStyleName())));
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_GlyphStyleSet, FText::FromName(Icon.GetStyleSetName())));
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_GlyphColor, FText::FromString(GlyphColor.ToString())));
	// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_Comment, FText::FromString(NodeComment)));
}

void URigVMEdGraphNode::AddRigVMPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const
{
	// Copied from EdGraphNode::AddPinSearchMetaDataInfo
	{
		//Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);
		// Searchable - Primary label for the item in the search results
		// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_Name, Pin->GetSchema()->GetPinDisplayName(Pin)));

		// Searchable - The pin's default value as a text string
		// OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FRigVMFindInBlueprintSearchTags::FiB_DefaultValue, Pin->GetDefaultAsText()));
	}
	
	if(const URigVMPin* ModelPin = FindModelPinFromGraphPin(Pin))
	{
		OutTaggedMetaData.Emplace(FText::FromString(TEXT("Data Type")), FText::FromString(ModelPin->GetCPPType()));
		
		if (ModelPin->IsBoundToVariable())
		{
			OutTaggedMetaData.Add(UBlueprintExtension::FSearchTagDataPair(FText::FromString(TEXT("Binding")), FText::FromString(ModelPin->GetBoundVariablePath())));
		}
	}
}

#endif

#endif

FEdGraphPinType URigVMEdGraphNode::GetPinTypeForModelPin(const URigVMPin* InModelPin)
{
	FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*InModelPin->GetCPPType(), InModelPin->GetCPPTypeObject());
	PinType.bIsConst = InModelPin->IsDefinedAsConstant();
	PinType.bIsReference = InModelPin->IsDefinedAsInputVariable();
	return PinType;
}

FEdGraphPinType URigVMEdGraphNode::GetPinTypeForCategoryPin()
{
	static const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(NAME_None, FRigVMPinCategory::StaticStruct());
	return PinType;
}

void URigVMEdGraphNode::ConfigurePin(UEdGraphPin* EdGraphPin, const URigVMPin* ModelPin) const
{
	const bool bConnectable =
		ModelPin->GetDirection() == ERigVMPinDirection::Input ||
		ModelPin->GetDirection() == ERigVMPinDirection::Output || 
		ModelPin->GetDirection() == ERigVMPinDirection::IO;

	// hide sub-pins on a compacted (knot) reroute node
	bool bHidden = ModelPin->GetDirection() == ERigVMPinDirection::Hidden;
	if(!bHidden && !ModelPin->IsRootPin())
	{
		if(Cast<URigVMRerouteNode>(ModelPin->GetNode()))
		{
			bHidden = true;
		}
	}

	EdGraphPin->bHidden = bHidden;
	EdGraphPin->PinFriendlyName = FText::FromName(ModelPin->GetDisplayName());
	EdGraphPin->bNotConnectable = !bConnectable  || ModelPin->IsFixedSizeArray();
	EdGraphPin->bOrphanedPin = ModelPin->IsOrphanPin() ? 1 : 0; 
	EdGraphPin->bDisplayAsMutableRef = ModelPin->IsWildCard() || ModelPin->IsDefinedAsInputVariable();
	if (!EdGraphPin->bDisplayAsMutableRef)
	{
		if(ModelPin->GetDirection() == ERigVMPinDirection::Output && ModelPin->IsRootPin())
		{
			if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelPin->GetNode()))
			{
				if (!VariableNode->IsInputArgument())
				{
					if (VariableNode->GetValuePin() == ModelPin)
					{
						EdGraphPin->bDisplayAsMutableRef = true;
					}
				}
			}
		}
	}
}

TArray<URigVMPin*>& URigVMEdGraphNode::PinListForPin(const URigVMPin* InModelPin)
{
	if(IsValid(InModelPin))
	{
		if(InModelPin->IsExecuteContext() && InModelPin->GetDirection() == ERigVMPinDirection::IO)
		{
			return ExecutePins;
		}
		if(InModelPin->GetDirection() == ERigVMPinDirection::Input || InModelPin->GetDirection() == ERigVMPinDirection::Visible)
		{
			return InputPins;
		}
		if(InModelPin->GetDirection() == ERigVMPinDirection::IO)
		{
			return InputOutputPins;
		}
		if(InModelPin->GetDirection() == ERigVMPinDirection::Output)
		{
			return OutputPins;
		}
	}
	static TArray<URigVMPin*> EmptyList;
	return EmptyList;
}

#if WITH_EDITOR

void URigVMEdGraphNode::UpdateVisualSettings()
{
	bIsHighlighted.Reset();
	ProfilingColor.Reset();
	HighlightingColor.Reset();
	bEnableProfiling.Reset();
	
	if(const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetOuter()))
	{
		if(const FRigVMEditorAssetInterfacePtr RigBlueprint = RigGraph->GetAsset())
		{
			bEnableProfiling = RigBlueprint->GetVMRuntimeSettings().bEnableProfiling;
			if (bEnableProfiling.Get(false))
			{
				if(IRigVMEditorAssetInterface* Blueprint = GetImplementingOuter<IRigVMEditorAssetInterface>())
				{
					if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
					{
						if(const URigVMNode* ModelNode = GetModelNode())
						{
							const double CurrentMicroSeconds = ModelNode->GetInstructionMicroSeconds(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
							MicroSeconds = Blueprint->GetRigGraphDisplaySettings().AggregateAverage(MicroSecondsFrames, MicroSeconds.Get(0), CurrentMicroSeconds);
							if(MicroSeconds.GetValue() >= 0.0)
							{
								if(Blueprint->GetRigGraphDisplaySettings().bAutoDetermineRange)
								{
									if(CurrentMicroSeconds < Blueprint->GetRigGraphDisplaySettings().MinMicroSeconds)
									{
										Blueprint->GetRigGraphDisplaySettings().MinMicroSeconds = CurrentMicroSeconds;
									}
									if(CurrentMicroSeconds > Blueprint->GetRigGraphDisplaySettings().MaxMicroSeconds)
									{
										Blueprint->GetRigGraphDisplaySettings().MaxMicroSeconds = CurrentMicroSeconds;
									}
								}
							
								const double MinMicroSeconds = Blueprint->GetRigGraphDisplaySettings().LastMinMicroSeconds;
								const double MaxMicroSeconds = Blueprint->GetRigGraphDisplaySettings().LastMaxMicroSeconds;
								if(MaxMicroSeconds > MinMicroSeconds)
								{
									const FLinearColor& MinColor = Blueprint->GetRigGraphDisplaySettings().MinDurationColor;
									const FLinearColor& MaxColor = Blueprint->GetRigGraphDisplaySettings().MaxDurationColor;

									const double T = (MicroSeconds.GetValue() - MinMicroSeconds) / (MaxMicroSeconds - MinMicroSeconds);
									ProfilingColor = FMath::Lerp<FLinearColor>(MinColor, MaxColor, (float)T);
								}
							}
						}
					}
				}
			}
		}
	}

	if (!bEnableProfiling.Get(false))
	{
		MicroSeconds.Reset();
		MicroSecondsFrames.Reset();
	}

	if (const URigVMNode* ModelNode = GetModelNode())
	{
		bIsHighlighted = ModelNode->IsHighlighted();
	}
	if (bIsHighlighted.Get(false))
	{
		HighlightingColor = FSlateColor(EStyleColor::Highlight).GetSpecifiedColor();
	}
}

#endif

FString URigVMEdGraphNode::GetPinMetaData(FName InPinName, FName InKey)
{
	URigVMNode* ModelNode = GetModelNode();
	if(ModelNode == nullptr)
	{
		return FString();
	}

	FString Left, Right = InPinName.ToString();
	URigVMPin::SplitPinPathAtStart(InPinName.ToString(), Left, Right);
	URigVMPin* Pin = ModelNode->FindPin(Right);
	if(Pin == nullptr)
	{
		return FString();
	}

	return Pin->GetMetaData(InKey);
}

TSharedPtr<INameValidatorInterface> URigVMEdGraphNode::MakeNameValidator() const
{
	const URigVMController* Controller = GetController();
	check(Controller);
	return MakeShareable(new FRigVMGraphNodeNameValidator(Controller));
}

void URigVMEdGraphNode::RequestRename(float InDelay)
{
	if(GetCanRenameNode())
	{
		if(RequestRenameDelegate.IsBound())
		{
			if(IsInGameThread())
			{
				RequestRenameDelegate.Execute();
			}
			else
			{
				TWeakObjectPtr<URigVMEdGraphNode> WeakThis = this;
				auto RequestRenameLambda = [WeakThis]()
				{
					if(WeakThis.IsValid())
					{
						(void)WeakThis->RequestRenameDelegate.ExecuteIfBound();
						WeakThis->bRenameIsPending = false;
					}
				};
				
				if(InDelay < SMALL_NUMBER)
				{
					ExecuteOnGameThread(UE_SOURCE_LOCATION, RequestRenameLambda);
				}
				else
				{
					FTimerHandle RequestRenameHandle;
					GEditor->GetTimerManager()->SetTimer(RequestRenameHandle, FTimerDelegate::CreateLambda([RequestRenameLambda]()
					{
						RequestRenameLambda();
					}), 1, false, InDelay);
				}
			}
		}
		else
		{
			bRenameIsPending = true;
		}
	}
}

FRigVMGraphNodeNameValidator::FRigVMGraphNodeNameValidator(const URigVMController* InController)
: Controller(InController)
{
}

EValidatorResult FRigVMGraphNodeNameValidator::IsValid(const FString& Name, bool bOriginal)
{
	if(Name.IsEmpty())
	{
		return EValidatorResult::EmptyName;
	}
	
	const URigVMGraph* Graph = Controller->GetGraph();
	check(Graph);

	if(Graph->FindNode(Name))
	{
		return EValidatorResult::AlreadyInUse;
	}

	const URigVMSchema* Schema = Graph->GetSchema();
	check(Schema);

	const FString SanitizedName = Schema->GetValidNodeName(Controller->GetGraph(), Name);

	if(SanitizedName.Len() < Name.Len())
	{
		return EValidatorResult::TooLong;
	}
	
	if(!SanitizedName.Equals(Name, ESearchCase::CaseSensitive))
	{
		return EValidatorResult::ContainsInvalidCharacters;
	}

	return EValidatorResult::Ok;
}

EValidatorResult FRigVMGraphNodeNameValidator::IsValid(const FName& Name, bool bOriginal)
{
	if(Name.IsNone())
	{
		return EValidatorResult::EmptyName;
	}
	return IsValid(Name.ToString(), bOriginal);
}

#undef LOCTEXT_NAMESPACE

