// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraphSchema.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
//#include "IControlRigEditorModule.h"
#include "RigVMCore/RigVMStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2_Actions.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphEditorActions.h"
#include "RigVMHost.h"
#include "RigVMBlueprintLegacy.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RigVMStringUtils.h"
#include "Curves/CurveFloat.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMCore/RigVMVariableDescription.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Algo/Count.h"
#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_MakeStruct.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMEditorAsset.h"
#include "RigVMDeveloperTypeUtils.h"
#include "Algo/LevenshteinDistance.h"
#if WITH_RIGVMLEGACYEDITOR
#include "BlueprintEditor.h"
#else
#include "Editor/RigVMNewEditor.h"
#endif

#include "RigVMBlueprintUtils.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEdGraphSchema)

#if WITH_EDITOR
#include "RigVMEditorModule.h"
#include "Misc/MessageDialog.h"
#include "Editor/Transactor.h"
#endif

#define LOCTEXT_NAMESPACE "CRigVMGraphSchema"

const FText FRigVMSearchTags::FiB_Name = LOCTEXT("Name", "Name");
const FText FRigVMSearchTags::FiB_ClassName = LOCTEXT("ClassName", "ClassName");
const FText FRigVMSearchTags::FiB_NodeGuid = LOCTEXT("NodeGuid", "NodeGuid");
const FText FRigVMSearchTags::FiB_PinCategory = LOCTEXT("PinCategory", "PinCategory");
const FText FRigVMSearchTags::FiB_PinSubCategory = LOCTEXT("SubCategory", "SubCategory");
const FText FRigVMSearchTags::FiB_PinBinding = LOCTEXT("Binding", "Binding");
const FText FRigVMSearchTags::FiB_ObjectClass = LOCTEXT("ObjectClass", "ObjectClass");
const FText FRigVMSearchTags::FiB_IsArray = LOCTEXT("IsArray", "IsArray");
const FText FRigVMSearchTags::FiB_Glyph = LOCTEXT("Glyph", "Glyph");
const FText FRigVMSearchTags::FiB_GlyphStyleSet = LOCTEXT("GlyphStyleSet", "GlyphStyleSet");
const FText FRigVMSearchTags::FiB_GlyphColor = LOCTEXT("GlyphColor", "GlyphColor");

FRigVMLocalVariableNameValidator::FRigVMLocalVariableNameValidator(const FRigVMEditorAssetInterfacePtr InAsset, const URigVMGraph* Graph, FName InExistingName)
	: FStringSetNameValidator(InExistingName.ToString())
{
	if (const UBlueprint* Blueprint = Cast<UBlueprint>(InAsset.GetObject()))
	{
		TSet<FName> NamesTemp;
		// We allow local variables with same name as blueprint variable
		
		FBlueprintEditorUtils::GetFunctionNameList(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetAllGraphNames(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetSCSVariableNameList(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetImplementingBlueprintsFunctionNameList(Blueprint, NamesTemp);

		for (FName & Name : NamesTemp)
		{
			Names.Add(Name.ToString());
		}
	}
	else if (InAsset)
	{
		FRigVMClient* Client = InAsset->GetRigVMClient();
		TArray<URigVMGraph*> Graphs = Client->GetAllModels(true, true);
		for (URigVMGraph* ItGraph : Graphs)
		{
			Names.Add(ItGraph->GetName());
		}
	}

	if (Graph)
	{
		for (const FRigVMGraphVariableDescription& LocalVariable : Graph->GetLocalVariables())
		{
			Names.Add(LocalVariable.Name.ToString());
		}

		for (const FRigVMGraphVariableDescription& InputArgument : Graph->GetInputArguments())
		{
			Names.Add(InputArgument.Name.ToString());
		}

		for (const FRigVMGraphVariableDescription& OutputArgument : Graph->GetOutputArguments())
		{
			Names.Add(OutputArgument.Name.ToString());
		}
	}
}

EValidatorResult FRigVMLocalVariableNameValidator::IsValid(const FString& Name, bool bOriginal)
{
	const EValidatorResult Result = FStringSetNameValidator::IsValid(Name, bOriginal);
	if (Result == EValidatorResult::Ok)
	{
		const URigVMSchema* RigSchema = Cast<URigVMSchema>(URigVMSchema::StaticClass()->GetDefaultObject(true));
		if (RigSchema->GetSanitizedName(Name, false, true) == Name)
		{
			return Result;
		}

		return EValidatorResult::ContainsInvalidCharacters;
	}
	return Result;
}

EValidatorResult FRigVMLocalVariableNameValidator::IsValid(const FName& Name, bool bOriginal)
{
	return IsValid(Name.ToString(), bOriginal);
}

FRigVMNameValidator::FRigVMNameValidator(const UBlueprint* Blueprint, const UStruct* ValidationScope, FName InExistingName)
: FStringSetNameValidator(InExistingName.ToString())
{
	if (Blueprint)
	{
		TSet<FName> NamesTemp;
		FBlueprintEditorUtils::GetClassVariableList(Blueprint, NamesTemp, true);
		FBlueprintEditorUtils::GetFunctionNameList(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetAllGraphNames(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetSCSVariableNameList(Blueprint, NamesTemp);
		FBlueprintEditorUtils::GetImplementingBlueprintsFunctionNameList(Blueprint, NamesTemp);

		for (FName& Name : NamesTemp)
		{
			Names.Add(Name.ToString());
		}
	}
}

FRigVMNameValidator::FRigVMNameValidator(const FRigVMEditorAssetInterfacePtr Blueprint, const UStruct* ValidationScope, FName InExistingName)
: FStringSetNameValidator(InExistingName.ToString())
{
	if (Blueprint)
	{
		TSet<FName> NamesTemp;
		TArray<FRigVMGraphVariableDescription> Variables = Blueprint->GetAssetVariables();
		TArray<URigVMGraph*> Graphs = Blueprint->GetRigVMClient()->GetAllModels(true, true);

		for (const FRigVMGraphVariableDescription& Variable : Variables)
		{
			NamesTemp.Add(Variable.Name);
		}
		for (const URigVMGraph* Graph : Graphs)
		{
			if (URigVMCollapseNode* Node = Cast<URigVMCollapseNode>(Graph->GetOuter()))
			{
				NamesTemp.Add(Node->GetFName());
			}
			else
			{
				NamesTemp.Add(Graph->GetFName());
			}
		}

		for (FName& Name : NamesTemp)
		{
			Names.Add(Name.ToString());
		}
	}
}

EValidatorResult FRigVMNameValidator::IsValid(const FString& Name, bool bOriginal)
{
	const EValidatorResult Result = FStringSetNameValidator::IsValid(Name, bOriginal);
	if (Result == EValidatorResult::Ok)
	{
		const URigVMSchema* RigSchema = Cast<URigVMSchema>(URigVMSchema::StaticClass()->GetDefaultObject(true));
		if (RigSchema->GetSanitizedName(Name, false, true) == Name)
		{
			return Result;
		}

		return EValidatorResult::ContainsInvalidCharacters;
	}
	return Result;
}

EValidatorResult FRigVMNameValidator::IsValid(const FName& Name, bool bOriginal)
{
	return IsValid(Name.ToString(), bOriginal);
}

FEdGraphPinType FRigVMEdGraphSchemaAction_LocalVar::GetPinType() const
{
	if (const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
		for (FRigVMGraphVariableDescription Variable : Graph->GetModel()->GetLocalVariables())
		{
			if (Variable.Name == GetVariableName())
			{
				return RigVMTypeUtils::PinTypeFromRigVMVariableDescription(Variable);
			}
		}

		for (FRigVMGraphVariableDescription Variable : Graph->GetModel()->GetInputArguments())
		{
			if (Variable.Name == GetVariableName())
			{
				return RigVMTypeUtils::PinTypeFromRigVMVariableDescription(Variable);
			}
		}
	}

	return FEdGraphPinType();
}

void FRigVMEdGraphSchemaAction_LocalVar::ChangeVariableType(const FEdGraphPinType& NewPinType)
{
	if (const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
		FString NewCPPType;
		UObject* NewCPPTypeObject = nullptr;
		RigVMTypeUtils::CPPTypeFromPinType(NewPinType, NewCPPType, &NewCPPTypeObject);
		Graph->GetController()->SetLocalVariableType(GetVariableName(), NewCPPType, NewCPPTypeObject, true, true);
	}
}

void FRigVMEdGraphSchemaAction_LocalVar::RenameVariable(const FName& NewName)
{
	const FName OldName = GetVariableName();
	if (OldName == NewName)
	{
		return;
	}
	
	if (const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
		if (Graph->GetController()->RenameLocalVariable(OldName, NewName, true, true))
		{
			SetVariableInfo(NewName, GetVariableScope(), GetPinType().PinCategory == TEXT("bool"));		
		}
	}	
}

bool FRigVMEdGraphSchemaAction_LocalVar::IsValidName(const FName& NewName, FText& OutErrorMessage) const
{
	if (const URigVMEdGraph* EdGraphGraph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
		FRigVMLocalVariableNameValidator NameValidator(EdGraphGraph->GetAsset(), EdGraphGraph->GetModel(), GetVariableName());
		const EValidatorResult Result = NameValidator.IsValid(NewName.ToString(), false);
		if (Result != EValidatorResult::Ok && Result != EValidatorResult::ExistingName)
		{
			OutErrorMessage = FText::FromString(TEXT("Name with invalid format"));
			return false;
		}
	}
	return true;
}

void FRigVMEdGraphSchemaAction_LocalVar::DeleteVariable() 
{
	if (const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
#if WITH_EDITOR

		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}
	
#endif
		
		Graph->GetController()->RemoveLocalVariable(GetVariableName(), true, true);
	}
}

bool FRigVMEdGraphSchemaAction_LocalVar::IsVariableUsed()
{
	if (const URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(GetVariableScope()))
	{
		const FString VarNameStr = GetVariableName().ToString();
		for (URigVMNode* Node : EdGraph->GetModel()->GetNodes())
		{
			if (const URigVMVariableNode* VarNode = Cast<URigVMVariableNode>(Node))
			{
				if (VarNode->FindPin(TEXT("Variable"))->GetDefaultValue() == VarNameStr)
				{
					return true;		
				}
			}
		}
	}
	return false;
}

FRigVMEdGraphSchemaAction_PromoteToVariable::FRigVMEdGraphSchemaAction_PromoteToVariable(UEdGraphPin* InEdGraphPin, bool InLocalVariable)
: FEdGraphSchemaAction(	FText(), 
						InLocalVariable ? LOCTEXT("PromoteToLocalVariable", "Promote to local variable") : LOCTEXT("PromoteToVariable", "Promote to variable"),
						InLocalVariable ? LOCTEXT("PromoteToLocalVariable", "Promote to local variable") : LOCTEXT("PromoteToVariable", "Promote to variable"),
						1)
, EdGraphPin(InEdGraphPin)
, bLocalVariable(InLocalVariable)
{
}

UEdGraphNode* FRigVMEdGraphSchemaAction_PromoteToVariable::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin,
	const FVector2f& Location, bool bSelectNewNode)
{
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(RigGraph == nullptr)
	{
		return nullptr;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = RigGraph->GetAsset();
	URigVMGraph* Model = RigGraph->GetModel();
	URigVMController* Controller = RigGraph->GetController();
	if((Blueprint == nullptr) ||
		(Model == nullptr) ||
		(Controller == nullptr))
	{
		return nullptr;
	}
	
	URigVMPin* ModelPin = Model->FindPin(FromPin->GetName());

	FName VariableName(NAME_None);

	const FScopedTransaction Transaction(
		bLocalVariable ?
		LOCTEXT("GraphEd_PromoteToLocalVariable", "Promote Pin To Local Variable") :
		LOCTEXT("GraphEd_PromoteToVariable", "Promote Pin To Variable"));
	

	bool bSuccess = false;
	if(bLocalVariable)
	{
		bSuccess = Controller->PromotePinToLocalVariable(ModelPin->GetPinPath(), true, FDeprecateSlateVector2D(Location), true, true);
	}
	else
	{
		bSuccess = Controller->PromotePinToVariable(ModelPin->GetPinPath(), true, FDeprecateSlateVector2D(Location), true, true);
	}

	if(bSuccess)
	{
		TArray<URigVMPin*> SourcePins = ModelPin->GetLinkedSourcePins();
		if (!SourcePins.IsEmpty())
		{
			return RigGraph->FindNodeForModelNodeName(SourcePins[0]->GetNode()->GetFName());
		}
	}
	
	return nullptr;
}

FRigVMEdGraphSchemaAction_PromoteToExposedPin::FRigVMEdGraphSchemaAction_PromoteToExposedPin(UEdGraphPin* InEdGraphPin)
: FEdGraphSchemaAction(	FText(), 
						LOCTEXT("PromoteToExposedPin", "Promote to exposed pin"),
						LOCTEXT("PromoteToExposedPinTooltip", "Adds an exposed pin and wires it to this pin. For inputs if you hold shift this will insert a Get node."),
						1)
, EdGraphPin(InEdGraphPin)
{
}

UEdGraphNode* FRigVMEdGraphSchemaAction_PromoteToExposedPin::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(RigGraph == nullptr)
	{
		return nullptr;
	}

	const  URigVMGraph* Model = RigGraph->GetModel();
	URigVMController* Controller = RigGraph->GetController();
	if(Model == nullptr || Controller == nullptr)
	{
		return nullptr;
	}
	
	URigVMPin* ModelPin = Model->FindPin(FromPin->GetName());
	if (ModelPin == nullptr)
	{
		return nullptr;
	}

	const FScopedTransaction Transaction(
		LOCTEXT("GraphEd_PromoteToExposedPin", "Promote To Exposed Pin"));
	
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}	
#endif

	Controller->OpenUndoBracket(TEXT("Promote to Exposed Pin"));

	// Graph functions don't really support IO pins, so if we are exposing an IO pin, we will assume it is an input
	ERigVMPinDirection Direction = ModelPin->GetDirection();
	if (Direction == ERigVMPinDirection::IO)
	{
		Direction = ERigVMPinDirection::Input;
	}
	
	UObject* CPPTypeObject = ModelPin->GetCPPTypeObject();
	const FString PinName = Controller->AddExposedPin(
		*ModelPin->GetName(),
		Direction,
		ModelPin->GetCPPType(),
		CPPTypeObject ? (FName)*CPPTypeObject->GetPathName() : NAME_None,
		ModelPin->GetDefaultValue(),
		true,
		true
	).ToString();

	UEdGraphNode* Result = nullptr;
	if(!PinName.IsEmpty())
	{
		if (Direction == ERigVMPinDirection::Input)
		{
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				URigVMVariableNode* VariableNode = Controller->AddVariableNode(*PinName, ModelPin->GetCPPType(), CPPTypeObject, true, ModelPin->GetDefaultValue(), FDeprecateSlateVector2D(Location), FString(), true, true);
				if (!VariableNode)
				{
					return nullptr;
				}
				Controller->AddLink(VariableNode->GetValuePin(), ModelPin, true);
				Result = RigGraph->FindNodeForModelNodeName(VariableNode->GetFName());
			}
			else
			{
				Controller->AddLink(Model->GetEntryNode()->FindPin(PinName), ModelPin, true);
				Result = RigGraph->FindNodeForModelNodeName(Model->GetEntryNode()->GetFName());
			}
		}
		else if(Direction == ERigVMPinDirection::Output)
		{
			Controller->AddLink(ModelPin, Model->GetReturnNode()->FindPin(PinName), true);
			Result = RigGraph->FindNodeForModelNodeName(Model->GetReturnNode()->GetFName());
		}
	}

	Controller->CloseUndoBracket();
	
	return Result;
}

FRigVMEdGraphSchemaAction_Event::FRigVMEdGraphSchemaAction_Event(const FName& InEventName, const FString& InNodePath, const FText& InNodeCategory)
: FEdGraphSchemaAction(	InNodeCategory, 
						FText::FromName(InEventName),
						FText(),
						1)
, NodePath(InNodePath)
{
}

FReply FRigVMEdGraphSchemaAction_Event::OnDoubleClick(UBlueprint* InBlueprint)
{
	if (FRigVMEditorAssetInterfacePtr Blueprint = InBlueprint)
	{
		if(const URigVMNode* ModelNode = Blueprint->GetRigVMClient()->FindNode(NodePath))
		{
			Blueprint->OnRequestJumpToHyperlink().Execute(ModelNode);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();;
}

FSlateBrush const* FRigVMEdGraphSchemaAction_Event::GetPaletteIcon() const
{
	static FSlateIcon EventIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	return EventIcon.GetIcon();
}

FReply FRigVMFunctionDragDropAction::DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	// For local variables
	if (SourceAction->GetTypeId() == FRigVMEdGraphSchemaAction_LocalVar::StaticGetTypeId())
	{
		if (URigVMEdGraph* TargetRigGraph = Cast<URigVMEdGraph>(&Graph))
		{
			if (TargetRigGraph == SourceRigGraph)
			{
				FRigVMEdGraphSchemaAction_LocalVar* VarAction = (FRigVMEdGraphSchemaAction_LocalVar*) SourceAction.Get();
				for (FRigVMGraphVariableDescription LocalVariable : TargetRigGraph->GetModel()->GetLocalVariables())
				{
					if (LocalVariable.Name == VarAction->GetVariableName())
					{
						URigVMController* Controller = TargetRigGraph->GetController();
						FMenuBuilder MenuBuilder(true, nullptr);
						const FText VariableNameText = FText::FromName( LocalVariable.Name );

						MenuBuilder.BeginSection("BPVariableDroppedOn", VariableNameText );

						MenuBuilder.AddMenuEntry(
							FText::Format( LOCTEXT("CreateGetVariable", "Get {0}"), VariableNameText ),
							FText::Format( LOCTEXT("CreateVariableGetterToolTip", "Create Getter for variable '{0}'\n(Ctrl-drag to automatically create a getter)"), VariableNameText ),
							FSlateIcon(),
							FUIAction(
							FExecuteAction::CreateLambda([Controller, LocalVariable, GraphPosition]()
							{
								Controller->AddVariableNode(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, true, LocalVariable.DefaultValue, FDeprecateSlateVector2D(GraphPosition), FString(), true, true);
							}),
							FCanExecuteAction()));

						MenuBuilder.AddMenuEntry(
							FText::Format( LOCTEXT("CreateSetVariable", "Set {0}"), VariableNameText ),
							FText::Format( LOCTEXT("CreateVariableSetterToolTip", "Create Setter for variable '{0}'\n(Alt-drag to automatically create a setter)"), VariableNameText ),
							FSlateIcon(),
							FUIAction(
							FExecuteAction::CreateLambda([Controller, LocalVariable, GraphPosition]()
							{
								Controller->AddVariableNode(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, false, LocalVariable.DefaultValue, FDeprecateSlateVector2D(GraphPosition), FString(), true, true);
							}),
							FCanExecuteAction()));

						TSharedRef< SWidget > PanelWidget = Panel;
						// Show dialog to choose getter vs setter
						FSlateApplication::Get().PushMenu(
							PanelWidget,
							FWidgetPath(),
							MenuBuilder.MakeWidget(),
							ScreenPosition,
							FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu)
							);

						MenuBuilder.EndSection();
					}	
				}				
			}
		}
	}
	// For functions
	else if (URigVMEdGraph* TargetRigGraph = Cast<URigVMEdGraph>(&Graph))
	{
		if (FRigVMEditorAssetInterfacePtr TargetRigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(TargetRigGraph))
		{
			if (URigVMGraph* FunctionDefinitionGraph = SourceRigBlueprint->GetModel(SourceRigGraph))
			{
				if (URigVMLibraryNode* FunctionDefinitionNode = Cast<URigVMLibraryNode>(FunctionDefinitionGraph->GetOuter()))
				{
					if(URigVMController* TargetController = TargetRigBlueprint->GetController(TargetRigGraph))
					{
						if(URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(FunctionDefinitionNode->GetOuter()))
						{
							if(FRigVMEditorAssetInterfacePtr FunctionRigBlueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(FunctionLibrary))
							{
#if WITH_EDITOR
								if(FunctionRigBlueprint != TargetRigBlueprint)
								{
									if(!FunctionRigBlueprint->GetRigVMClientHost()->IsFunctionPublic(FunctionDefinitionNode->GetFName()))
									{
										FRigVMGraphFunctionIdentifier FunctionIdentifier = FunctionDefinitionNode->GetFunctionIdentifier();
										TargetRigBlueprint->BroadcastRequestLocalizeFunctionDialog(FunctionIdentifier);
										FunctionDefinitionNode = TargetRigBlueprint->GetLocalFunctionLibrary()->FindPreviouslyLocalizedFunction(FunctionIdentifier);
									}
								}
#endif
								TargetController->AddFunctionReferenceNode(FunctionDefinitionNode, FDeprecateSlateVector2D(GraphPosition), FString(), true, true);
							}
						}
					}
				}
			}
		}
	}

	
	
	return FReply::Unhandled();
}

FReply FRigVMFunctionDragDropAction::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	return FReply::Unhandled();
}

FReply FRigVMFunctionDragDropAction::DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action)
{
	return FReply::Unhandled();
}

FReply FRigVMFunctionDragDropAction::DroppedOnCategory(FText Category)
{
	// todo
	/*
	if (SourceAction.IsValid())
	{
		SourceAction->MovePersistentItemToCategory(Category);
	}
	*/
	return FReply::Unhandled();
}

void FRigVMFunctionDragDropAction::HoverTargetChanged()
{
	// todo - see FMyBlueprintItemDragDropAction
	FGraphSchemaActionDragDropAction::HoverTargetChanged();

	// check for category + graph, everything else we won't allow for now.

	bDropTargetValid = true;
}

FRigVMFunctionDragDropAction::FRigVMFunctionDragDropAction()
	: FGraphSchemaActionDragDropAction()
	, SourceRigBlueprint(nullptr)
	, SourceRigGraph(nullptr)
	, bControlDrag(false)
	, bAltDrag(false)
{
}

TSharedRef<FRigVMFunctionDragDropAction> FRigVMFunctionDragDropAction::New(TSharedPtr<FEdGraphSchemaAction> InAction, FRigVMEditorAssetInterfacePtr InRigBlueprint, URigVMEdGraph* InRigGraph)
{
	TSharedRef<FRigVMFunctionDragDropAction> Action = MakeShareable(new FRigVMFunctionDragDropAction);
	Action->SourceAction = InAction;
	Action->SourceRigBlueprint = InRigBlueprint;
	Action->SourceRigGraph = InRigGraph;
	Action->Construct();
	return Action;
}

URigVMEdGraphSchema::URigVMEdGraphSchema()
	: UEdGraphSchema()
	, EditorSettingsClass(URigVMEditorSettings::StaticClass())
	, bLastPinWasInput(false)
{
}

void URigVMEdGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{

}

void URigVMEdGraphSchema::GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	/*
	// this seems to be taken care of by ControlRigGraphNode
#if WITH_EDITOR
	return IControlRigEditorModule::Get().GetContextMenuActions(this, Menu, Context);
#else
	check(0);
#endif
	*/
}

bool URigVMEdGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
#if WITH_EDITOR

	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
	
#endif

	if (PinA == PinB)
	{
		return false;
	}

	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return false;
	}

	IRigVMClientHost* Host = PinA->GetOwningNode()->GetImplementingOuter<IRigVMClientHost>();
	if (Host != nullptr)
	{
		if (URigVMController* Controller = Host->GetRigVMClient()->GetOrCreateController(PinA->GetOwningNode()->GetGraph()))
		{
			ERigVMPinDirection UserLinkDirection = ERigVMPinDirection::Output;
			if (PinA->Direction == EGPD_Input)
			{
				Swap(PinA, PinB);
				UserLinkDirection = ERigVMPinDirection::Input;
			}

			const bool bPinAIsWildCard = PinA->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject();
			const bool bPinBIsWildCard = PinB->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject();
			if(bPinAIsWildCard != bPinBIsWildCard)
			{
				// switch the user link direction if only one of the pins is a wildcard 
				if(UserLinkDirection == ERigVMPinDirection::Input && bPinBIsWildCard)
				{
					UserLinkDirection = ERigVMPinDirection::Output;
				}
				else if(UserLinkDirection == ERigVMPinDirection::Output && bPinAIsWildCard)
				{
					UserLinkDirection = ERigVMPinDirection::Input;
				}
			}

#if WITH_EDITOR

			static const FText LoopMessage = LOCTEXT("LinkingLoopNotRecommended", "Linking a function return within a loop is not recommended.\nAre you sure?");
			
			// check if we are trying to connect a loop iteration pin to a return
			if(const URigVMGraph* Graph = Controller->GetGraph())
			{
				if(const URigVMPin* TargetPin = Graph->FindPin(PinB->GetName()))
				{
					if(TargetPin->IsExecuteContext() && TargetPin->GetNode()->IsA<URigVMFunctionReturnNode>())
					{
						bool bIsInLoopIteration = false;
						if(const URigVMPin* SourcePin = Graph->FindPin(PinA->GetName()))
						{
							while(SourcePin)
							{
								if(!SourcePin->IsExecuteContext())
								{
									break;
								}
								const URigVMPin* CurrentSourcePin = SourcePin;
								SourcePin = nullptr;
								
								if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(CurrentSourcePin->GetNode()))
								{
									TSharedPtr<FStructOnScope> UnitScope = UnitNode->ConstructStructInstance();
									if(UnitScope.IsValid())
									{
										const FRigVMStruct* Unit = (FRigVMStruct*)UnitScope->GetStructMemory();
										if(Unit->IsForLoop())
										{
											if(CurrentSourcePin->GetFName() != FRigVMStruct::ForLoopCompletedPinName)
											{
												bIsInLoopIteration = true;
												break;
											}
										}
									}
									
								}

								for(const URigVMPin* PinOnSourceNode : CurrentSourcePin->GetNode()->GetPins())
								{
									if(!PinOnSourceNode->IsExecuteContext())
									{
										continue;
									}

									if(PinOnSourceNode->GetDirection() != ERigVMPinDirection::Input &&
										PinOnSourceNode->GetDirection() != ERigVMPinDirection::IO)
									{
										continue;
									}

									TArray<URigVMPin*> NextSourcePins = PinOnSourceNode->GetLinkedSourcePins();
									if(NextSourcePins.Num() > 0)
									{
										SourcePin = NextSourcePins[0];
										break;
									}
								}
							}
						}

						if(bIsInLoopIteration)
						{
							const EAppReturnType::Type Answer = FMessageDialog::Open( EAppMsgType::YesNo, LoopMessage );
							if(Answer == EAppReturnType::No)
							{
								return false;
							}
						}
					}
				}
			}
#endif

			const bool bCreateCastNode = !FSlateApplication::Get().GetModifierKeys().IsAltDown();
			return Controller->AddLink(PinA->GetName(), PinB->GetName(), true, true, UserLinkDirection, bCreateCastNode);
		}
	}
	return false;
}

static bool HasParentConnection_Recursive(const UEdGraphPin* InPin)
{
	if(InPin->ParentPin)
	{
		return InPin->ParentPin->LinkedTo.Num() > 0 || HasParentConnection_Recursive(InPin->ParentPin);
	}

	return false;
}

static bool HasChildConnection_Recursive(const UEdGraphPin* InPin)
{
	for(const UEdGraphPin* SubPin : InPin->SubPins)
	{
		if(SubPin->LinkedTo.Num() > 0 || HasChildConnection_Recursive(SubPin))
		{
			return true;
		}
	}

	return false;
}

const FPinConnectionResponse URigVMEdGraphSchema::CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const
{
	if (A == nullptr || B == nullptr)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("One of the Pins is NULL"));
	}
	
	const URigVMEdGraphNode* RigNodeA = Cast<URigVMEdGraphNode>(A->GetOwningNode());
	const URigVMEdGraphNode* RigNodeB = Cast<URigVMEdGraphNode>(B->GetOwningNode());

	if (RigNodeA && RigNodeB && RigNodeA != RigNodeB)
	{
		URigVMPin* PinA = RigNodeA->GetModelPinFromPinPath(A->GetName());
		if (PinA)
		{
			PinA = PinA->GetPinForLink();
			RigNodeA->GetModel()->PrepareCycleChecking(PinA, A->Direction == EGPD_Input);
		}

		URigVMPin* PinB = RigNodeB->GetModelPinFromPinPath(B->GetName());
		if (PinB)
		{
			PinB = PinB->GetPinForLink();
		}

		ERigVMPinDirection UserLinkDirection = ERigVMPinDirection::Output;
		if (A->Direction == EGPD_Input)
		{
			Swap(PinA, PinB);
			UserLinkDirection = ERigVMPinDirection::Input;
		}

		if (!PinA)
		{
			return FPinConnectionResponse(ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW, FString::Printf(TEXT("Pin %s not found"), *A->GetName()));
		}

		if (!PinB)
		{
			return FPinConnectionResponse(ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW, FString::Printf(TEXT("Pin %s not found"), *B->GetName()));
		}

		if(PinA->IsWildCard() != PinB->IsWildCard())
		{
			// switch the user link direction if only one of the pins is a wildcard 
			if(UserLinkDirection == ERigVMPinDirection::Input && PinB->IsWildCard())
			{
				UserLinkDirection = ERigVMPinDirection::Output;
			}
			else if(UserLinkDirection == ERigVMPinDirection::Output && PinA->IsWildCard())
			{
				UserLinkDirection = ERigVMPinDirection::Input;
			}
		}

		const FRigVMByteCode* ByteCode = RigNodeA->GetController()->GetCurrentByteCode();

		const bool bEnableTypeCasting = !FSlateApplication::Get().GetModifierKeys().IsAltDown();
		FString FailureReason;
		const bool bResult = RigNodeA->GetModel()->CanLink(PinA, PinB, &FailureReason, ByteCode, UserLinkDirection, bEnableTypeCasting);
		if (!bResult)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FText::FromString(FailureReason));
		}
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, LOCTEXT("ConnectResponse_Allowed", "Connect"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectResponse_Disallowed_Unexpected", "Unexpected error"));
}

FLinearColor URigVMEdGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	const FName& TypeName = PinType.PinCategory;
	if (TypeName == UEdGraphSchema_K2::PC_Struct)
	{
		if (const UStruct* Struct = Cast<UStruct>(PinType.PinSubCategoryObject))
		{
			if (Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				return FLinearColor::White;
			}

			if (Struct->IsChildOf(RigVMTypeUtils::GetWildCardCPPTypeObject()))
			{
				return FLinearColor(FVector3f::OneVector * 0.25f);
			}
		}
	}
	
	return GetDefault<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);
}

void URigVMEdGraphSchema::InsertAdditionalActions(TArray<UBlueprint*> InBlueprints, TArray<UEdGraph*> EdGraphs,
	TArray<UEdGraphPin*> EdGraphPins, FGraphActionListBuilderBase& OutAllActions) const
{
	Super::InsertAdditionalActions(InBlueprints, EdGraphs, EdGraphPins, OutAllActions);
	TArray<FRigVMEditorAssetInterfacePtr> Assets;
	Algo::Transform(InBlueprints, Assets, [](UBlueprint* Blueprint)
		{
			return TScriptInterface<IRigVMEditorAssetInterface>(Blueprint);
		});
	InsertAdditionalActions(Assets, EdGraphs, EdGraphPins, OutAllActions);
}

void URigVMEdGraphSchema::InsertAdditionalActions(TArray<FRigVMEditorAssetInterfacePtr> InAssets, TArray<UEdGraph*> InGraphs,
	TArray<UEdGraphPin*> InPins, FGraphActionListBuilderBase& OutAllActions) const
{
	if(InPins.Num() > 0)
	{
		if(const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPins[0]->GetOwningNode()))
		{
			if(const URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(InPins[0]->GetName()))
			{
				const bool bIsRootGraph = ModelPin->GetGraph()->IsRootGraph();
				if(!ModelPin->IsExecuteContext() && !ModelPin->IsWildCard())
				{
					if(!ModelPin->GetNode()->IsA<URigVMVariableNode>())
					{
						OutAllActions.AddAction(TSharedPtr<FRigVMEdGraphSchemaAction_PromoteToVariable>(
							new FRigVMEdGraphSchemaAction_PromoteToVariable(InPins[0], false)
						));

						if(!bIsRootGraph && !ModelPin->IsWildCard())
						{
							OutAllActions.AddAction(TSharedPtr<FRigVMEdGraphSchemaAction_PromoteToVariable>(
								new FRigVMEdGraphSchemaAction_PromoteToVariable(InPins[0], true)
							));
						}
					}
				}

				if (!bIsRootGraph)
				{
					if (!ModelPin->GetGraph()->GetRootGraph()->IsA<URigVMFunctionLibrary>() || !ModelPin->IsWildCard())
					{
						OutAllActions.AddAction(TSharedPtr<FRigVMEdGraphSchemaAction_PromoteToExposedPin>(
								   new FRigVMEdGraphSchemaAction_PromoteToExposedPin(InPins[0])
							   ));
					}
				}
			}
		}
	}
}

TSharedPtr<INameValidatorInterface> URigVMEdGraphSchema::GetNameValidator(const UBlueprint* BlueprintObj, const FName& OriginalName, const UStruct* ValidationScope, const FName& ActionTypeId) const
{
	if (ActionTypeId == FRigVMEdGraphSchemaAction_LocalVar::StaticGetTypeId())
	{
		// this cast will always fail, URigVMEdGraph is not a UStruct
		/*
		if (const URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(ValidationScope))
		{
			if (const URigVMGraph* Graph = EdGraph->GetModel())
			{
				return MakeShareable(new FRigVMLocalVariableNameValidator(BlueprintObj, Graph, OriginalName));
			}
		}
		*/
	}

	return MakeShareable(new FRigVMNameValidator(BlueprintObj, ValidationScope, OriginalName));		
}

TSharedPtr<INameValidatorInterface> URigVMEdGraphSchema::GetNameValidator(const FRigVMEditorAssetInterfacePtr Asset, const FName& OriginalName, const UStruct* ValidationScope) const
{
	return MakeShareable(new FRigVMNameValidator(Asset, ValidationScope, OriginalName));
}

bool URigVMEdGraphSchema::CanPromotePinToVariable(const UEdGraphPin& Pin, bool bCond) const
{
	return true;
}

bool URigVMEdGraphSchema::SupportsPreviewFromHereOnNode(const UEdGraphNode* InNode) const
{
	return false;
}

bool URigVMEdGraphSchema::AllowsFuzzyActionFiltering(const FEdGraphSchemaAction& InCurrentAction) const
{
	const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>(EditorSettingsClass);
	return Settings->bEnableFuzzySearchInActionMenu;
}

float URigVMEdGraphSchema::GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms,
                                                   const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>(EditorSettingsClass);
	if (!Settings->bEnableFuzzySearchInActionMenu)
	{
		return UEdGraphSchema::GetActionFilteredWeight(InCurrentAction, InFilterTerms, InSanitizedFilterTerms, DraggedFromPins);
	}
	
	TArray<FString> SanitizedFilterTerms = InSanitizedFilterTerms;
	for (FString& SanitizedFilterTerm : SanitizedFilterTerms)
	{
		SanitizedFilterTerm.ToLowerInline();
	}
	TArray<FString> MenuDescriptions = InCurrentAction.GetMenuDescriptionArray();
	for (FString& MenuDescription : MenuDescriptions)
	{
		MenuDescription.ToLowerInline();
	}
	TArray<FString> MenuDescriptionsReversed = MenuDescriptions;
	Algo::Reverse(MenuDescriptionsReversed);

	TArray<FString> KeywordsArray = InCurrentAction.GetSearchKeywordsArray();
	FString KeywordsJoined = RigVMStringUtils::JoinStrings(KeywordsArray,TEXT(",")).ToLower();
	KeywordsArray.Reset();
	if (!RigVMStringUtils::SplitString(KeywordsJoined, TEXT(","), KeywordsArray))
	{
		KeywordsArray = InCurrentAction.GetSearchKeywordsArray();
	}
	const TSet<FString> UniqueKeywords = TSet<FString>(KeywordsArray);
	KeywordsArray = UniqueKeywords.Array();

	auto LevenshteinSingle = [](const FStringView& A, const FStringView& B, const int32& MinLength) -> float
		{
			if (A.Len() < MinLength || B.Len() < MinLength)
			{
				return A.Equals(B, ESearchCase::CaseSensitive) ? 1.f : 0.f;
			}
			
			const float WorstCase = static_cast<float>(FMath::Max<int32>(A.Len(), B.Len()) - MinLength);
			if (WorstCase < 1.f - SMALL_NUMBER)
			{
				return 0.f;
			}

			const int32 Distance = Algo::LevenshteinDistance(A, B);
			return FMath::Max(0.f, 1.0f - (static_cast<float>(Distance) / WorstCase));
		};

	// returns the maximum weight for a given clause within an array of candidates
	auto LevenshteinSingleMaxInArray = [LevenshteinSingle](const FString& A, const TArrayView<const FString>& B, const int32& MinLength) -> float
		{
			float Weight = 0.f;
			for (const FString& ElementInB : B)
			{
				Weight = FMath::Max(Weight, LevenshteinSingle(A, ElementInB, MinLength));
			}
			return Weight;
		};

	// returns the maximum weight for an array of clauses within an array of candidates
	auto LevenshteinMaxInArray = [LevenshteinSingleMaxInArray](const TArrayView<const FString>& A, const TArrayView<const FString>& B, const int32& MinLength) -> float
		{
			float Weight = 0.f;
			for (const FString& ElementInA : A)
			{
				Weight = FMath::Max(Weight, LevenshteinSingleMaxInArray(ElementInA, B, MinLength));
			}
			return Weight;
		};

	// returns the maximum weight for all clauses for a matching array of candidates
	auto LevenshteinSameSizeArrays = [LevenshteinSingle](const TArrayView<const FString>& A, const TArrayView<const FString>& B, const int32& MinLength) -> float
		{
			if (A.Num() != B.Num() || A.IsEmpty() || B.IsEmpty())
			{
				return 0.f;
			}
			float Weight = 0.f;
			for (int32 Index = 0; Index < A.Num(); Index++)
			{
				const float SingleWeight = LevenshteinSingle(A[Index], B[Index], MinLength);
				if (SingleWeight < SMALL_NUMBER)
				{
					return 0;
				}
				Weight += SingleWeight;
			}
			return Weight / static_cast<float>(A.Num());
		};

	// returns the maximum weight of an array of clauses for any matching subarrays within a larger array of candidates
	auto LevenshteinArbitraryArrays = [LevenshteinSameSizeArrays](const TArrayView<const FString>& A, const TArrayView<const FString>& B, const int32& MinLength) -> float
		{
			if (A.IsEmpty() || B.IsEmpty() || A.Num() > B.Num())
			{
				return 0.f;
			}
			float Weight = 0.f;
			for (int32 Offset = 0; Offset <= B.Num() - A.Num(); Offset++)
			{
				const TArrayView<const FString> PartOfB = B.Slice(Offset, A.Num());
				Weight = FMath::Max(Weight, LevenshteinSameSizeArrays(A, PartOfB, MinLength));
			}
			return Weight;
		};

	const FString JoinedSanitizedFilterTerms = RigVMStringUtils::JoinStrings(SanitizedFilterTerms, TEXT(""));
	const FString JoinedMenuDescriptions = RigVMStringUtils::JoinStrings(MenuDescriptions, TEXT(""));
	const TArray<FString>& Categories = InCurrentAction.GetCategoryChain();
	const FString JoinedCategories = RigVMStringUtils::JoinStrings(Categories, TEXT(""));

	enum EScoreType : int32
	{
		Description,
		Keywords,
		ConstantCategoryWeight,
		CategoriesFuzzyMatch,
		NumberOfWordsDifference,
		TotalNumberOfScoreTypes
	};

	static constexpr uint64 Divisor = 32;
	static constexpr uint32 MinimumFuzzyLength = 3;

	// build up the score with a set of priorities
	TArray<float, TInlineAllocator<TotalNumberOfScoreTypes>> Scores;
	Scores.AddZeroed(TotalNumberOfScoreTypes);

	// exact match start has the highest weight
	if (JoinedMenuDescriptions.Equals(JoinedSanitizedFilterTerms, ESearchCase::IgnoreCase))
	{
		Scores[Description] = 1;
	}
	else if (JoinedMenuDescriptions.StartsWith(JoinedSanitizedFilterTerms, ESearchCase::IgnoreCase))
	{
		Scores[Description] = 0.9f;
	}
	else if (JoinedMenuDescriptions.Contains(JoinedSanitizedFilterTerms, ESearchCase::IgnoreCase))
	{
		Scores[Description] = 0.8f;
	}
	else
	{
		Scores[Description] = 0.7f * LevenshteinSingle(JoinedSanitizedFilterTerms, JoinedMenuDescriptions, MinimumFuzzyLength);
		if (Scores[Description] < SMALL_NUMBER)
		{
			const FString CategoryAndDescription = JoinedCategories + JoinedMenuDescriptions;
			Scores[Description] = 0.6f * LevenshteinSingle(JoinedSanitizedFilterTerms, CategoryAndDescription, MinimumFuzzyLength);
		}
		if (Scores[Description] < SMALL_NUMBER)
		{
			Scores[Description] = 0.5f * FMath::Max(LevenshteinArbitraryArrays(SanitizedFilterTerms, MenuDescriptions, MinimumFuzzyLength), LevenshteinArbitraryArrays(SanitizedFilterTerms, MenuDescriptionsReversed, MinimumFuzzyLength));
		}
	}

	if (Scores[Description] < 0.25f)
	{
		// exact keyword match start has the second-highest weight
		if (KeywordsArray.ContainsByPredicate([JoinedSanitizedFilterTerms](const FString& Keyword) -> bool
		{
			return Keyword.Equals(JoinedSanitizedFilterTerms, ESearchCase::IgnoreCase);
		}))
		{
			Scores[Keywords] = 1.f;
		}
		else if (KeywordsArray.ContainsByPredicate([JoinedSanitizedFilterTerms](const FString& Keyword) -> bool
		{
			return Keyword.Contains(JoinedSanitizedFilterTerms, ESearchCase::IgnoreCase);
		}))
		{
			Scores[Keywords] = 0.9f;
		}
		else
		{
			// fuzzy match within the keywords
			Scores[Keywords] = 0.5f * LevenshteinMaxInArray(SanitizedFilterTerms, KeywordsArray, MinimumFuzzyLength);
		}
	}

	// if we find multiple matches with the same weight - let's add the category in as well
	static const TMap<FString,float> CategoryWeights = {
		{TEXT("math"), 0.5f},
		{TEXT("array"), 0.4f},
		{TEXT("core"), 0.3f},
		{TEXT("execution"), 0.2f},
		{TEXT("simulation"), 0.1f},
	};

	if ((Scores[Description] + Scores[Keywords]) > SMALL_NUMBER)
	{
		if (!Categories.IsEmpty())
		{
			if (const float* CategoryWeight = CategoryWeights.Find(Categories[0].ToLower()))
			{
				Scores[ConstantCategoryWeight] = *CategoryWeight;
			}
		}
	}

	// fuzzy match within the categories
	Scores[CategoriesFuzzyMatch] = LevenshteinSingle(JoinedSanitizedFilterTerms, JoinedCategories, MinimumFuzzyLength);

	if ((Scores[Description] + Scores[Keywords] + Scores[CategoriesFuzzyMatch]) > SMALL_NUMBER)
	{
		// additional weight based on the distance in the number of terms and the number of menu descriptions (get transform vs get transform array)
		Scores[NumberOfWordsDifference] = 1.f - (FMath::Max(0.f, static_cast<float>(MenuDescriptions.Num() - SanitizedFilterTerms.Num())) / static_cast<float>(FMath::Max3(MenuDescriptions.Num(), SanitizedFilterTerms.Num(), 1)));
	}

	uint64 OverallScore = 0;
	uint64 MaximumPossibleScore = 0;
	uint64 Multiplier = 1;
	for (int32 ScoreIndex = Scores.Num() - 1; ScoreIndex >= 0; --ScoreIndex)
	{
		const int32 ScoreInt32 = FMath::Min(static_cast<int32>(Divisor) - 1, FMath::FloorToInt32(Scores[ScoreIndex] * (static_cast<float>(Divisor) - 0.1f)));
		OverallScore += static_cast<uint64>(ScoreInt32) * Multiplier;
		MaximumPossibleScore += Divisor * Multiplier;
		Multiplier *= Divisor;
	}

	const double OverallScoreDouble = static_cast<double>(OverallScore);
	const double MaximumScoreDouble = static_cast<double>(MaximumPossibleScore);
	const double MinimumScoreDouble = static_cast<double>(Settings->MinimumFuzzyScore);
	const double WeightAsPercent = 100.0 * OverallScoreDouble / MaximumScoreDouble;
	if (WeightAsPercent < MinimumScoreDouble)
	{
		return -1.f;
	}
	return static_cast<float>(WeightAsPercent);
}

bool URigVMEdGraphSchema::SupportsPinType(const UScriptStruct* ScriptStruct) const
{
	if(!ScriptStruct)
	{
		return false;
	}

	if(ScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		return false;
	}

	if(ScriptStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()))
	{
		return false;
	}

	// todo: Validate ExecuteContext structs to match URigVMBlueprint::GetExecuteContextStruct()

	for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
	{
		FProperty* Property = *It;

		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			Property = ArrayProperty->Inner;
		}

		FString CPPType = Property->GetCPPType();
		if (CPPType == TEXT("bool") ||
			CPPType == TEXT("float") ||
			CPPType == TEXT("double") ||
			CPPType == TEXT("int32") ||
			CPPType == TEXT("FString") ||
			CPPType == TEXT("FName") ||
			CPPType == TEXT("uint16"))
		{
			continue;
		}		
		
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (SupportsPinType(StructProperty->Struct))
			{
				continue;
			}
		}
		else if (Property->IsA<FEnumProperty>())
		{
			continue;
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (ByteProperty->Enum)
			{
				continue;
			}
		}
		else if (CastField<FObjectProperty>(Property) && RigVMCore::SupportsUObjects())
		{
			continue;
		}
		else if (CastField<FInterfaceProperty>(Property) && RigVMCore::SupportsUInterfaces())
		{
			continue;
		}
	
		return false;
	}
	
	return true;
}

bool URigVMEdGraphSchema::SupportsPinType(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType) const
{
	if (PinType.IsContainer())
	{
		return false;
	}
	
	const FName TypeName = PinType.PinCategory;

	if (TypeName == UEdGraphSchema_K2::PC_Boolean ||
		TypeName == UEdGraphSchema_K2::PC_Int ||
		TypeName == UEdGraphSchema_K2::PC_Real ||
		TypeName == UEdGraphSchema_K2::PC_Name ||
		TypeName == UEdGraphSchema_K2::PC_String ||
		TypeName == UEdGraphSchema_K2::PC_Enum)
	{
		return true;
	}

	if(RigVMCore::SupportsUObjects())
	{
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
			PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject ||
			PinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes)
		{
			if (PinType.PinSubCategoryObject.IsValid())
			{
				return PinType.PinSubCategoryObject->IsA<UClass>();
			}
		}
	}

	if (RigVMCore::SupportsUInterfaces())
	{
		if (PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			if (PinType.PinSubCategoryObject.IsValid())
			{
				return PinType.PinSubCategoryObject->IsA<UInterface>();
			}
		}
	}

	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(PinType.PinSubCategoryObject))
		{
			if(SchemaAction.IsValid() && SchemaAction.Pin()->IsAVariable())
			{
				if(ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					return false;
				}
			}
			return SupportsPinType(ScriptStruct);
		}
		else if (PinType.PinSubCategoryObject == UUserDefinedStruct::StaticClass())
		{
			// if a user defined struct hasn't been loaded yet,
			// its PinSubCategoryObject equals UUserDefinedStruct::StaticClass()
			// and since it is not practical to load every user defined struct to check if they only contain supported types,
			// we always return true so that they at least show up in the drop down menu
			// if they contain members of unsupported type, the spawned node will generate error
			return true;
		}
	}

	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		if (PinType.PinSubCategoryObject.IsValid())
		{
			return PinType.PinSubCategoryObject->IsA<UEnum>();
		}
	}

	return false;
}

bool URigVMEdGraphSchema::SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction,
	const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const
{
	// Do not allow containers for execute context type
	if(const UScriptStruct* ExecuteContextScriptStruct = Cast<UScriptStruct>(PinType.PinSubCategoryObject))
	{
		if (ExecuteContextScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
		{
			return ContainerType == EPinContainerType::None;
		}
	}
	
	return ContainerType == EPinContainerType::None || ContainerType == EPinContainerType::Array;
}

void URigVMEdGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	//const FScopedTransaction Transaction( LOCTEXT("GraphEd_BreakPinLinks", "Break Pin Links") );

	// cache this here, as BreakPinLinks can trigger a node reconstruction invalidating the TargetPin referenceS
	if (const URigVMEdGraphNode* Node = Cast< URigVMEdGraphNode>(TargetPin.GetOwningNode()))
	{
		Node->GetController()->BreakAllLinks(TargetPin.GetName(), TargetPin.Direction == EGPD_Input, true, true);
	}
}

void URigVMEdGraphSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	//const FScopedTransaction Transaction(LOCTEXT("GraphEd_BreakSinglePinLink", "Break Pin Link") );

	if (const URigVMEdGraphNode* Node = Cast< URigVMEdGraphNode>(TargetPin->GetOwningNode()))
	{
		if (SourcePin->Direction == EGPD_Input)
		{
			UEdGraphPin* Temp = TargetPin;
			TargetPin = SourcePin;
			SourcePin = Temp;
		}
		
		Node->GetController()->BreakLink(SourcePin->GetName(), TargetPin->GetName(), true, true);
	}
}

bool URigVMEdGraphSchema::CanGraphBeDropped(TSharedPtr<FEdGraphSchemaAction> InAction) const
{
	if (!InAction.IsValid())
	{
		return false;
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		const FEdGraphSchemaAction_K2Graph* FuncAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();
		if (Cast<URigVMEdGraph>(FuncAction->EdGraph))
		{
			return true;
		}
	}
	else if (InAction->GetTypeId() == FRigVMEdGraphSchemaAction_LocalVar::StaticGetTypeId())
	{
		const FRigVMEdGraphSchemaAction_LocalVar* VarAction = (FRigVMEdGraphSchemaAction_LocalVar*)InAction.Get();
		if (Cast<URigVMEdGraph>((UEdGraph*)VarAction->GetVariableScope()))
		{
			return true;
		}
	}
	
	return false;
}

FString URigVMEdGraphSchema::GetFindReferenceSearchTerm(const FEdGraphSchemaAction* InGraphAction) const
{
	if(InGraphAction)
	{
		if(InGraphAction->GetTypeId() == FEdGraphSchemaAction_K2Var::StaticGetTypeId())
		{
			const FEdGraphSchemaAction_K2Var* VarAction = (const FEdGraphSchemaAction_K2Var*)InGraphAction;
			return VarAction->GetVariableName().ToString();
		}
		else if (InGraphAction->GetTypeId() == FRigVMEdGraphSchemaAction_LocalVar::StaticGetTypeId())
		{
			const FRigVMEdGraphSchemaAction_LocalVar* VarAction = (const FRigVMEdGraphSchemaAction_LocalVar*)InGraphAction;
			return VarAction->GetVariableName().ToString();
		}
	}
	return Super::GetFindReferenceSearchTerm(InGraphAction);
}

FReply URigVMEdGraphSchema::BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent) const
{
	if (!InAction.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InAction->GetTypeId() == FEdGraphSchemaAction_K2Graph::StaticGetTypeId())
	{
		FEdGraphSchemaAction_K2Graph* FuncAction = (FEdGraphSchemaAction_K2Graph*)InAction.Get();
		if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(FuncAction->EdGraph))
		{
			if (FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(RigGraph))
			{
				TSharedRef<FRigVMFunctionDragDropAction> Action = FRigVMFunctionDragDropAction::New(InAction, RigBlueprint, RigGraph);
				Action->SetAltDrag(MouseEvent.IsAltDown());
				Action->SetCtrlDrag(MouseEvent.IsControlDown());
				return FReply::Handled().BeginDragDrop(Action);
			}
		}
	}
	else if(InAction->GetTypeId() == FRigVMEdGraphSchemaAction_LocalVar::StaticGetTypeId())
	{
		FRigVMEdGraphSchemaAction_LocalVar* VarAction = (FRigVMEdGraphSchemaAction_LocalVar*)InAction.Get();
		if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(VarAction->GetVariableScope()))
		{
			if (FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(RigGraph))
			{
				TSharedRef<FRigVMFunctionDragDropAction> Action = FRigVMFunctionDragDropAction::New(InAction, RigBlueprint, RigGraph);
				Action->SetAltDrag(MouseEvent.IsAltDown());
				Action->SetCtrlDrag(MouseEvent.IsControlDown());
				return FReply::Handled().BeginDragDrop(Action);
			}
		}
	}
	return FReply::Unhandled();
}

FConnectionDrawingPolicy* URigVMEdGraphSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
#if WITH_EDITOR
	if (const FRigVMEditorAssetInterfacePtr Blueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraphObj))
	{
		return Blueprint->GetEditorModule()->CreateConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}	
	return IRigVMEditorModule::Get().CreateConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
#else
	check(0);
	return nullptr;
#endif
}

bool URigVMEdGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	// we should hide default values if any of our parents are connected
	if(HasParentConnection_Recursive(Pin))
	{
		return true;
	}

	if(const URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(Pin->GetOwningNode()))
	{
		if(RigGraphNode->DrawAsCompactNode())
		{
			return true;
		}
	}

	return false;
}

bool URigVMEdGraphSchema::IsPinBeingWatched(UEdGraphPin const* Pin) const
{
	if (const URigVMEdGraphNode* Node = Cast< URigVMEdGraphNode>(Pin->GetOwningNode()))
	{
		if (const URigVMPin* ModelPin = Node->GetModel()->FindPin(Pin->GetName()))
		{
			return ModelPin->RequiresWatch();
		}
	}
	return false;
}

void URigVMEdGraphSchema::ClearPinWatch(UEdGraphPin const* Pin) const
{
	if (const URigVMEdGraphNode* Node = Cast< URigVMEdGraphNode>(Pin->GetOwningNode()))
	{
		Node->GetController()->SetPinIsWatched(Pin->GetName(), false);
	}
}

void URigVMEdGraphSchema::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	if (const URigVMEdGraphNode* Node = Cast< URigVMEdGraphNode>(PinA->GetOwningNode()))
	{
		if (URigVMLink* Link = Node->GetModel()->FindLink(FString::Printf(TEXT("%s -> %s"), *PinA->GetName(), *PinB->GetName())))
		{
			Node->GetController()->AddRerouteNodeOnLink(Link, FDeprecateSlateVector2D(GraphPosition), FString(), true, true);
		}
	}
}

bool URigVMEdGraphSchema::MarkBlueprintDirtyFromNewNode(UBlueprint* InBlueprint, UEdGraphNode* InEdGraphNode) const
{
	if (InBlueprint == nullptr || InEdGraphNode == nullptr)
	{
		return false;
	}
	return true;
}

bool URigVMEdGraphSchema::IsStructEditable(UStruct* InStruct) const
{
	if (InStruct == TBaseStructure<FQuat>::Get())
	{
		return true;
	}
	if (InStruct == FRuntimeFloatCurve::StaticStruct())
	{
		return true;
	}
	return false;
}

void URigVMEdGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const
{
	return SetNodePosition(Node, FVector2D(Position), true);
}

void URigVMEdGraphSchema::SetNodePosition(UEdGraphNode* Node, const FVector2D& Position, bool bSetupUndo) const
{
	StartGraphNodeInteraction(Node);

	if (const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
	{
		if (URigVMController* Controller = RigNode->GetController())
		{
			if  (!URigVMHost::IsGarbageOrDestroyed(Controller))
			{
				Controller->SetNodePosition(RigNode->GetModelNode(), Position, bSetupUndo, false, false);
			}
		}
	}
	
	if (const UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
	{
		if(const URigVMEdGraph* Graph = CommentNode->GetTypedOuter<URigVMEdGraph>())
		{
			if (URigVMController* Controller = Graph->GetController())
			{
				if  (!URigVMHost::IsGarbageOrDestroyed(Controller))
				{
					Controller->SetNodePositionByName(CommentNode->GetFName(), Position, bSetupUndo, false, false);
				}
			}
		}
	}
}

void URigVMEdGraphSchema::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	Super::GetGraphDisplayInformation(Graph, DisplayInfo);

	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)&Graph))
	{
		if(const URigVMGraph* Model = RigGraph->GetModel())
		{
			TArray<FString> NodePathParts;
			if (URigVMNode::SplitNodePath(RigGraph->ModelNodePath, NodePathParts))
			{
				if(NodePathParts.Num() > 1)
				{
					DisplayInfo.DisplayName = FText::FromString(NodePathParts.Last());
					DisplayInfo.PlainName = DisplayInfo.DisplayName;

					static const FText LocalFunctionText = FText::FromString(TEXT("A local function.")); 
					DisplayInfo.Tooltip = LocalFunctionText;
				}

				// if this is a riggraph within a collapse node - let's use that for the tooltip
				if(const URigVMCollapseNode* CollapseNode = Model->GetTypedOuter<URigVMCollapseNode>())
				{
					DisplayInfo.Tooltip = CollapseNode->GetToolTipText();
				}
			}

			if(Model->IsRootGraph())
			{
				// let's see if there is only one event
				FString EventName;
				if(Algo::CountIf(Model->GetNodes(), [&EventName](const URigVMNode* NodeToCount) -> bool
				{
					if(NodeToCount->IsEvent() && NodeToCount->CanOnlyExistOnce())
					{
						if(EventName.IsEmpty())
						{
							if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(NodeToCount))
							{
								if(UnitNode->GetScriptStruct())
								{
									EventName = UnitNode->GetScriptStruct()->GetDisplayNameText().ToString();
								}
							}
						}
						return true;
					}
					return false;
				}) == 1)
				{
					static constexpr TCHAR EventGraphNameFormat[] = TEXT("%s Graph");
					const FString DesiredGraphName = FString::Printf(EventGraphNameFormat, *EventName);
					DisplayInfo.DisplayName = FText::FromString(DesiredGraphName);
					DisplayInfo.PlainName = DisplayInfo.DisplayName;
				}
				else
				{
					static const FText MainGraphText = FText::FromString(TEXT("A top level graph for the rig."));
					DisplayInfo.Tooltip = MainGraphText;

					if(!NodePathParts.IsEmpty())
					{
						static constexpr TCHAR NodePathSuffix[] = TEXT("::");
						FString NodePath = NodePathParts[0];
						NodePath.RemoveFromEnd(NodePathSuffix);
						NodePath.RemoveFromStart(FRigVMClient::RigVMModelPrefix);
						NodePath.TrimStartAndEndInline();
						if (NodePath.IsEmpty())
						{
							NodePath = TEXT("Rig Graph");
						}
						DisplayInfo.DisplayName = FText::FromString(NodePath);
						DisplayInfo.PlainName = DisplayInfo.DisplayName;
					}
				}
			}
		}
	}
}

bool URigVMEdGraphSchema::GetLocalVariables(const UEdGraph* InGraph, TArray<FBPVariableDescription>& OutLocalVariables) const
{
	OutLocalVariables.Reset();
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)InGraph))
	{
		if (const URigVMGraph* Model = RigGraph->GetModel())
		{
			TArray<FRigVMGraphVariableDescription> LocalVariables = Model->GetLocalVariables();
			for (FRigVMGraphVariableDescription LocalVariable : LocalVariables)
			{
				FBPVariableDescription VariableDescription;
				VariableDescription.VarName = LocalVariable.Name;
				VariableDescription.FriendlyName = LocalVariable.Name.ToString();
				VariableDescription.DefaultValue = LocalVariable.DefaultValue;
				VariableDescription.VarType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(LocalVariable);
				VariableDescription.PropertyFlags |= CPF_BlueprintVisible;
				OutLocalVariables.Add(VariableDescription);
			}
		}
	}
	return true;
}

TSharedPtr<FEdGraphSchemaAction> URigVMEdGraphSchema::MakeActionFromVariableDescription(const UEdGraph* InEdGraph,
	const FBPVariableDescription& Variable) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)InEdGraph))
	{
		FText Category = Variable.Category;
		if (Variable.Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory))
		{
			Category = FText::GetEmpty();
		}
#if WITH_RIGVMLEGACYEDITOR
		TSharedPtr<FRigVMEdGraphSchemaAction_LocalVar> Action = MakeShareable(new FRigVMEdGraphSchemaAction_LocalVar(Category, FText::FromName(Variable.VarName), FText::GetEmpty(), 0, NodeSectionID::LOCAL_VARIABLE));
#else
		TSharedPtr<FRigVMEdGraphSchemaAction_LocalVar> Action = MakeShareable(new FRigVMEdGraphSchemaAction_LocalVar(Category, FText::FromName(Variable.VarName), FText::GetEmpty(), 0, RigVMNodeSectionID::LOCAL_VARIABLE));
#endif
		Action->SetVariableInfo(Variable.VarName, RigGraph, Variable.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
		return Action;
	}
	return nullptr;
}

FText URigVMEdGraphSchema::GetGraphCategory(const UEdGraph* InGraph) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)InGraph))
	{
		if (const URigVMGraph* Model = RigGraph->GetModel())
		{
			if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				return FText::FromString(CollapseNode->GetNodeCategory());
			}
		}
	}
	return FText();
}

EGraphType URigVMEdGraphSchema::GetGraphType(const UEdGraph* TestEdGraph) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)TestEdGraph))
	{
		if (FRigVMBlueprintUtils::FindAssetForGraph(RigGraph))
		{
			if (const URigVMGraph* Model = RigGraph->GetModel())
			{
				if(Model->IsRootGraph())
				{
					if(Model->IsA<URigVMFunctionLibrary>())
					{
						return EGraphType::GT_Function;
					}
					return EGraphType::GT_Ubergraph;
				}

				if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
				{
					if(LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
					{
						return EGraphType::GT_Function;
					}
					// collapse nodes show up as uber graphs
					return EGraphType::GT_Ubergraph;
				}
			}
		}
	}
	return Super::GetGraphType(TestEdGraph);
}

FReply URigVMEdGraphSchema::TrySetGraphCategory(const UEdGraph* InGraph, const FText& InCategory) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>((UEdGraph*)InGraph))
	{
		if (const FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(RigGraph))
		{
			if (const URigVMGraph* Model = RigGraph->GetModel())
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
				{
					if (URigVMController* Controller = RigBlueprint->GetOrCreateController(CollapseNode->GetGraph()))
					{
						if (Controller->SetNodeCategory(CollapseNode, InCategory.ToString(), true, false, true))
						{
							return FReply::Handled();
						}
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

bool URigVMEdGraphSchema::TryDeleteGraph(UEdGraph* GraphToDelete) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GraphToDelete))
	{
		if (const FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(RigGraph))
		{
			if (const URigVMGraph* Model = RigBlueprint->GetModel(GraphToDelete))
			{
				if(Model->IsRootGraph())
				{
					RigBlueprint->RemoveModel(Model->GetNodePath());
					return true;
				}
				else if (URigVMCollapseNode* LibraryNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
				{
					if (URigVMController* Controller = RigBlueprint->GetOrCreateController(LibraryNode->GetGraph()))
					{
						// check if there is a "bulk remove function" transaction going on.
						// which implies that a category is being deleted
						if (GEditor->CanTransact())
						{
							if (GEditor->Trans->GetQueueLength() > 0)
							{
								const FTransaction* LastTransaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - 1);
								if (LastTransaction)
								{
									if (LastTransaction->GetTitle().ToString() == TEXT("Bulk Remove Functions"))
									{
										// instead of deleting the graph, let's set its category to none
										// and thus moving it to the top of the tree
										return Controller->SetNodeCategory(LibraryNode, FString());
									}
								}
							}
						}

						bool bSetupUndoRedo = true;

						// if the element to remove is a function, check if it is public and referenced. If so,
						// warn the user about a bulk remove
						if (URigVMFunctionLibrary* Library = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
						{
							const FName& FunctionName = LibraryNode->GetFName();
							if (RigBlueprint->IsFunctionPublic(FunctionName))
							{
								for (auto Reference : Library->GetReferencesForFunction(FunctionName))
								{
									if (Reference.IsValid())
									{
										const FRigVMEditorAssetInterfacePtr OtherBlueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(Reference.Get());
										if (OtherBlueprint != RigBlueprint)
										{
											if(RigBlueprint->OnRequestBulkEditDialog().IsBound())
											{
												URigVMController* FunctionController = RigBlueprint->GetController(LibraryNode->GetContainedGraph());
												const FRigVMController_BulkEditResult Result = RigBlueprint->OnRequestBulkEditDialog().Execute(RigBlueprint, FunctionController, LibraryNode, ERigVMControllerBulkEditType::RemoveFunction);
												if(Result.bCanceled)
												{
													return false;
												}
												bSetupUndoRedo = Result.bSetupUndoRedo;
											}
											break;	
										}
									}
								}
							}
						}
						
						return Controller->RemoveNode(LibraryNode, bSetupUndoRedo, false);
					}
				}
			}
		}
	}
	return false;
}

bool URigVMEdGraphSchema::TryRenameGraph(UEdGraph* GraphToRename, const FName& InNewName) const
{
	if (const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GraphToRename))
	{
		if (IRigVMClientHost* ClientHost = RigGraph->GetImplementingOuter<IRigVMClientHost>())
		{
			if (const URigVMGraph* Model = RigGraph->GetModel())
			{
				if(Model->IsRootGraph())
				{
					const FString NewName = FString::Printf(TEXT("%s %s"), FRigVMClient::RigVMModelPrefix, *InNewName.ToString()); 
					ClientHost->RenameGraph(Model->GetNodePath(), *NewName);
				}
				else if (const URigVMGraph* RootModel = Model->GetRootGraph())
				{
					URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(RootModel->FindNode(RigGraph->ModelNodePath));
					if (LibraryNode)
					{
						if (URigVMController* Controller = ClientHost->GetOrCreateController(LibraryNode->GetGraph()))
						{
							Controller->RenameNode(LibraryNode, InNewName, true, true);
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

bool URigVMEdGraphSchema::TryToGetChildEvents(const UEdGraph* Graph, const int32 SectionId,
	TArray<TSharedPtr<FEdGraphSchemaAction>>& Actions, const FText& ParentCategory) const
{
	check(Graph);

	if(const URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph))
	{
		if(const URigVMGraph* Model = RigGraph->GetModel())
		{
			TArray<FName> EventNames;
			TMap<FName, const URigVMNode*> EventNameToNode;
			for(const URigVMNode* Node : Model->GetNodes())
			{
				if(Node->IsEvent())
				{
					if(!EventNames.Contains(Node->GetEventName()))
					{
						EventNames.Add(Node->GetEventName());
						EventNameToNode.Add(Node->GetEventName(), Node);
					}
				}
			}

			if(!EventNames.IsEmpty())
			{
				FGraphDisplayInfo EdGraphDisplayInfo;
				GetGraphDisplayInformation(*Graph, EdGraphDisplayInfo);

				FText EdGraphDisplayName = EdGraphDisplayInfo.DisplayName;
				FText ActionCategory;
				if (!ParentCategory.IsEmpty())
				{
					ActionCategory = FText::Format(FText::FromString(TEXT("{0}|{1}")), ParentCategory, EdGraphDisplayName);
				}
				else
				{
					ActionCategory = MoveTemp(EdGraphDisplayName);
				}

				for(const FName& EventName : EventNames)
				{
					const URigVMNode* Node = EventNameToNode.FindChecked(EventName);
					TSharedPtr<FEdGraphSchemaAction> Action = MakeShareable(
						new FRigVMEdGraphSchemaAction_Event(EventName, Node->GetNodePath(true), ActionCategory)
					);
					Action->SectionID = SectionId;
					Actions.Add(Action);
				}
			}
			return true;
		}
	}
	return false;
}

UEdGraphPin* URigVMEdGraphSchema::DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const
{
	if (FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForNode(InTargetNode))
	{
		if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InTargetNode))
		{
			if (URigVMNode* ModelNode = RigNode->GetModelNode())
			{
				FString NewPinName;
				
				const URigVMGraph* Model = nullptr;
				ERigVMPinDirection PinDirection = InSourcePinDirection == EGPD_Input ? ERigVMPinDirection::Input : ERigVMPinDirection::Output;

				if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelNode))
				{
					Model = CollapseNode->GetContainedGraph();
					PinDirection = PinDirection == ERigVMPinDirection::Output ? ERigVMPinDirection::Input : ERigVMPinDirection::Output;
				}
				else if (ModelNode->IsA<URigVMFunctionInterfaceNode>())
				{
					Model = ModelNode->GetGraph();
				}

				if (Model)
				{
					ensure(!Model->IsTopLevelGraph());

					const FRigVMExternalVariable ExternalVar = RigVMTypeUtils::ExternalVariableFromPinType(FGuid(), InSourcePinName, InSourcePinType);
					if (ExternalVar.IsValid(true /* allow null memory */))
					{
						if (URigVMController* Controller = RigBlueprint->GetController(Model))
						{
							FName SourcePinName = InSourcePinName; 
							FString TypeName = ExternalVar.GetExtendedCPPType().ToString();
							FName TypeObjectPathName = NAME_None;
							if (ExternalVar.GetCPPTypeObject())
							{
								TypeObjectPathName = *ExternalVar.GetCPPTypeObject()->GetPathName();

								if(const UScriptStruct* CPPTypeStruct = Cast<UScriptStruct>(ExternalVar.GetCPPTypeObject()))
								{
									if(CPPTypeStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
									{
										SourcePinName = FRigVMStruct::ExecuteContextName;
										PinDirection = ERigVMPinDirection::IO;										
									}
								}
							}

							FString DefaultValue;
							bool bIsDefinedAsInputVariable = false;
							if (PinBeingDropped)
							{
								if (const URigVMEdGraphNode* SourceNode = Cast<URigVMEdGraphNode>(PinBeingDropped->GetOwningNode()))
								{
									if (const URigVMPin* SourcePin = SourceNode->GetModelPinFromPinPath(PinBeingDropped->GetName()))
									{
										DefaultValue = SourcePin->GetDefaultValue();
										bIsDefinedAsInputVariable = SourcePin->IsDefinedAsInputVariable();
									}
								}
							}

							const FName ExposedPinName = Controller->AddExposedPin(
								SourcePinName,
								PinDirection,
								TypeName,
								TypeObjectPathName,
								DefaultValue,
								true,
								true
							);
							
							if (!ExposedPinName.IsNone())
							{
								NewPinName = ExposedPinName.ToString();
								Controller->MarkExposedPinAsInputVariable(ExposedPinName, bIsDefinedAsInputVariable);
							}
						}
					}
				}

				if (!NewPinName.IsEmpty())
				{
					if (const URigVMPin* NewModelPin = ModelNode->FindPin(NewPinName))
					{
						return RigNode->FindPin(*NewModelPin->GetPinPath());
					}
				}
			}
		}
	}

	return nullptr;
}

bool URigVMEdGraphSchema::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	if (const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InTargetNode))
	{
		if(const URigVMNode* ModelNode = RigNode->GetModelNode())
		{
			if (ModelNode->IsA<URigVMFunctionEntryNode>())
			{
				if (InSourcePinDirection == EGPD_Output)
				{
					OutErrorMessage = LOCTEXT("AddPinToReturnNode", "Add Pin to Return Node");
					return false;
				}
				return true;
			}
			else if (ModelNode->IsA<URigVMFunctionReturnNode>())
			{
				if (InSourcePinDirection == EGPD_Input)
				{
					OutErrorMessage = LOCTEXT("AddPinToEntryNode", "Add Pin to Entry Node");
					return false;
				}
				return true;
			}
			else if (ModelNode->IsA<URigVMCollapseNode>())
			{
				return true;
			}
		}
	}

	return false;
}

URigVMEdGraphNode* URigVMEdGraphSchema::CreateGraphNode(URigVMEdGraph* InGraph, const FName& InPropertyName) const
{
	const bool bSelectNewNode = true;
	FGraphNodeCreator<URigVMEdGraphNode> GraphNodeCreator(*InGraph);
	URigVMEdGraphNode* EdGraphNode = GraphNodeCreator.CreateNode(bSelectNewNode, GetGraphNodeClass(InGraph));
	EdGraphNode->ModelNodePath = InPropertyName.ToString();
	GraphNodeCreator.Finalize();

	return EdGraphNode;
}

void URigVMEdGraphSchema::TrySetDefaultValue(UEdGraphPin& InPin, const FString& InNewDefaultValue, bool bMarkAsModified) const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif
	GetDefault<UEdGraphSchema_K2>()->TrySetDefaultValue(InPin, InNewDefaultValue, false);
}

void URigVMEdGraphSchema::TrySetDefaultObject(UEdGraphPin& InPin, UObject* InNewDefaultObject, bool bMarkAsModified) const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif
	GetDefault<UEdGraphSchema_K2>()->TrySetDefaultObject(InPin, InNewDefaultObject, false);
}

void URigVMEdGraphSchema::TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified) const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif
	GetDefault<UEdGraphSchema_K2>()->TrySetDefaultText(InPin, InNewDefaultText, false);
}

bool URigVMEdGraphSchema::ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext, bool bIgnoreArray /*= false*/) const
{
	// filter out pins which have a parent
	if (PinB->ParentPin != nullptr)
	{
		return false;
	}

	// if we are looking at a polymorphic node
	if((PinA->PinType.ContainerType == PinB->PinType.ContainerType) ||
		(PinA->PinType.PinSubCategoryObject != PinB->PinType.PinSubCategoryObject))
	{
		auto IsPinCompatibleWithType = [](const UEdGraphPin* InPin, const FEdGraphPinType& InPinType) -> bool
		{
			if(const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPin->GetOwningNode()))
			{
				FString CPPType;
				UObject* CPPTypeObject = nullptr;
				if(RigVMTypeUtils::CPPTypeFromPinType(InPinType, CPPType, &CPPTypeObject))
				{
					FString PinPath, PinName;
					URigVMPin::SplitPinPathAtEnd(InPin->GetName(), PinPath, PinName);
						
					if((InPin->ParentPin != nullptr) &&
						(InPin->ParentPin->ParentPin == nullptr) &&
						(InPin->ParentPin->PinType.ContainerType == EPinContainerType::Array))
					{
						URigVMPin::SplitPinPathAtEnd(InPin->ParentPin->GetName(), PinPath, PinName);
						CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(CPPType);
					}

					const FRigVMTemplateArgumentType Type(*CPPType, CPPTypeObject);
					const TRigVMTypeIndex TypeIndex = FRigVMRegistry::Get().GetTypeIndex(Type);
					if(const FRigVMTemplate* Template = RigNode->GetTemplate())
					{
						if(const FRigVMTemplateArgument* Argument = Template->FindArgument(*PinName))
						{
							if(Argument->SupportsTypeIndex(TypeIndex))
							{
								return true;
							}
					
							const TArray<TRigVMTypeIndex>& AvailableCasts = RigVMTypeUtils::GetAvailableCasts(TypeIndex, InPin->Direction == EGPD_Output);
							for(const TRigVMTypeIndex& AvailableCast : AvailableCasts)
							{
								if(Argument->SupportsTypeIndex(AvailableCast))
								{
									return true;
								}
							}
						}
					}
				}
			}
			return false;
		};

		auto IsTemplateNodePin = [](const UEdGraphPin* InPin)
		{
			if(const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPin->GetOwningNode()))
			{
				return RigNode->GetTemplate() != nullptr;
			};
			return false;
		};
		
		if(PinA->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject() || IsTemplateNodePin(PinA))
		{
			if(IsPinCompatibleWithType(PinA, PinB->PinType))
			{
				URigVMEdGraphSchema* MutableThis = (URigVMEdGraphSchema*)this;
				MutableThis->LastPinForCompatibleCheck = PinB;
				MutableThis->bLastPinWasInput = PinB->Direction == EGPD_Input;
				return true;
			}
		}
 		if(PinB->PinType.PinSubCategoryObject == RigVMTypeUtils::GetWildCardCPPTypeObject() || IsTemplateNodePin(PinB))
		{
 			if(IsPinCompatibleWithType(PinB, PinA->PinType))
 			{
 				URigVMEdGraphSchema* MutableThis = (URigVMEdGraphSchema*)this;
 				MutableThis->LastPinForCompatibleCheck = PinA;
 				MutableThis->bLastPinWasInput = PinA->Direction == EGPD_Input;
 				return true;
 			}
		}
	}

	// for large world coordinate support we should allow connections
	// between float and double
	if(PinA->PinType.ContainerType == EPinContainerType::None &&
		PinB->PinType.ContainerType == EPinContainerType::None)
	{
		if((PinA->PinType.PinCategory == UEdGraphSchema_K2::PC_Float &&
			PinB->PinType.PinCategory == UEdGraphSchema_K2::PC_Double) ||
			(PinA->PinType.PinCategory == UEdGraphSchema_K2::PC_Double &&
			PinB->PinType.PinCategory == UEdGraphSchema_K2::PC_Float))
		{
			return true;
		}
	}

	if(PinA->PinType.ContainerType == EPinContainerType::None &&
		PinB->PinType.ContainerType == EPinContainerType::None &&
		PinA->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
		PinB->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if(const UScriptStruct* ScriptStructA = Cast<UScriptStruct>(PinA->PinType.PinSubCategoryObject))
		{
			if(const UScriptStruct* ScriptStructB = Cast<UScriptStruct>(PinB->PinType.PinSubCategoryObject))
			{
				if(ScriptStructA->IsChildOf(FRigVMExecutePin::StaticStruct()) &&
					ScriptStructB->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					return true;
				}
			}
		}
	}

	if(GetDefault<UEdGraphSchema_K2>()->ArePinsCompatible(PinA, PinB, CallingContext, bIgnoreArray))
	{
		return true;
	}

	// also check if there's a cast available for the type
	const TRigVMTypeIndex TypeIndexA = RigVMTypeUtils::TypeIndexFromPinType(PinA->Direction == EGPD_Output ? PinA->PinType : PinB->PinType);
	const TRigVMTypeIndex TypeIndexB = RigVMTypeUtils::TypeIndexFromPinType(PinA->Direction == EGPD_Output ? PinB->PinType : PinA->PinType);
	return RigVMTypeUtils::CanCastTypes(TypeIndexA, TypeIndexB);
}

void URigVMEdGraphSchema::RenameNode(URigVMEdGraphNode* Node, const FName& InNewNodeName) const
{
	Node->NodeTitle = FText::FromName(InNewNodeName);
	Node->Modify();
}

void URigVMEdGraphSchema::ResetPinDefaultsRecursive(UEdGraphPin* InPin) const
{
	URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPin->GetOwningNode());
	if (RigNode == nullptr)
	{
		return;
	}

	RigNode->CopyPinDefaultsToModel(InPin);
	for (UEdGraphPin* SubPin : InPin->SubPins)
	{
		ResetPinDefaultsRecursive(SubPin);
	}
}

void URigVMEdGraphSchema::GetVariablePinTypes(TArray<FEdGraphPinType>& PinTypes) const
{
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, FName(NAME_None), nullptr, EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Real, FName(NAME_None), nullptr, EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Int, FName(NAME_None), nullptr, EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, FName(NAME_None), TBaseStructure<FVector>::Get(), EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, FName(NAME_None), TBaseStructure<FVector2D>::Get(), EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, FName(NAME_None), TBaseStructure<FRotator>::Get(), EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, FName(NAME_None), TBaseStructure<FTransform>::Get(), EPinContainerType::None, false, FEdGraphTerminalType()));
	PinTypes.Add(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, FName(NAME_None), TBaseStructure<FLinearColor>::Get(), EPinContainerType::None, false, FEdGraphTerminalType()));
}

bool URigVMEdGraphSchema::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const
{
	if (const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
	{
		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}
		return RigNode->GetController()->RemoveNode(RigNode->GetModelNode(), true, true);
	}
	return false;
}

bool URigVMEdGraphSchema::CanVariableBeDropped(UEdGraph* InGraph, FProperty* InVariableToDrop) const
{
	const FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(FGuid(), InVariableToDrop, nullptr);
	return ExternalVariable.IsValid(true /* allow nullptr */);
}

bool URigVMEdGraphSchema::RequestFunctionDropOnPanel(UEdGraph* InGraph, const FRigVMGraphFunctionIdentifier& InFunction, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const
{
	if (FRigVMEditorAssetInterfacePtr TargetRigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph))
	{
		if(URigVMController* TargetController = TargetRigBlueprint->GetController(InGraph))
		{
			FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(InFunction);
			if (IRigVMClientHost* SourceClientHost = Cast<IRigVMClientHost>(InFunction.HostObject.ResolveObject()))
			{
				if (TScriptInterface<IRigVMGraphFunctionHost> SourceFunctionHost = SourceClientHost->GetRigVMGraphFunctionHost())
				{
					if (FRigVMGraphFunctionStore* SourceStore = SourceFunctionHost->GetRigVMGraphFunctionStore())
					{
#if WITH_EDITOR
						if(InFunction.HostObject != TargetRigBlueprint->GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetObject())
						{
							if (!SourceStore->IsFunctionPublic(InFunction))
							{
								TargetRigBlueprint->BroadcastRequestLocalizeFunctionDialog(InFunction);
								URigVMLibraryNode* FunctionDefinitionNode = TargetRigBlueprint->GetLocalFunctionLibrary()->FindPreviouslyLocalizedFunction(InFunction);
								Header = FunctionDefinitionNode->GetFunctionHeader();
							}
						}
#endif
					}
				}
			}
			return TargetController->AddFunctionReferenceNodeFromDescription(Header, InDropPosition, FString(), true, true) != nullptr;
		}
	}
	
	return false;
}

bool URigVMEdGraphSchema::RequestVariableDropOnPanel(UEdGraph* InGraph, FProperty* InVariableToDrop, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const
{
#if WITH_EDITOR
	if (CanVariableBeDropped(InGraph, InVariableToDrop))
	{
		FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph);
		if (RigBlueprint != nullptr)
		{
			RigBlueprint->OnVariableDropped().Broadcast(InGraph, InVariableToDrop, FDeprecateSlateVector2D(InDropPosition), FDeprecateSlateVector2D(InScreenPosition));
			return true;
		}
	}
#endif

	return false;
}

bool URigVMEdGraphSchema::RequestVariableDropOnPin(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphPin* InPin, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const
{
#if WITH_EDITOR
	if (CanVariableBeDropped(InGraph, InVariableToDrop))
	{
		if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InGraph))
		{
			if (const URigVMPin* ModelPin = Graph->GetModel()->FindPin(InPin->GetName()))
			{
				URigVMController* Controller = Graph->GetController();
				const FRigVMExternalVariable ExternalVariable = Controller->GetVariableByProperty(InVariableToDrop);
				ensure(ExternalVariable.GetProperty() == InVariableToDrop);
				return RequestVariableDropOnPin(InGraph, ExternalVariable, InPin, FDeprecateSlateVector2D(InDropPosition), FDeprecateSlateVector2D(InScreenPosition));
			}
		}
	}
#endif

	return false;
}

bool URigVMEdGraphSchema::RequestVariableDropOnPin(UEdGraph* InGraph, const FRigVMExternalVariable& InVariableToDrop, UEdGraphPin* InPin, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const
{
#if WITH_EDITOR
	if (InVariableToDrop.IsValid(true /* allow nullptr */))
	{
		if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InGraph))
		{
			if (const URigVMPin* ModelPin = Graph->GetModel()->FindPin(InPin->GetName()))
			{
				if (ModelPin->CanBeBoundToVariable(InVariableToDrop))
				{
					const FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
					if (KeyState.IsAltDown())
					{
						return Graph->GetController()->BindPinToVariable(ModelPin->GetPinPath(), InVariableToDrop.GetName().ToString(), true, true);
					}
					else
					{
						Graph->GetController()->OpenUndoBracket(TEXT("Bind Variable to Pin"));
						if (const URigVMVariableNode* VariableNode = Graph->GetController()->AddVariableNode(InVariableToDrop.GetName(), InVariableToDrop.GetExtendedCPPType().ToString(), const_cast<UObject*>(InVariableToDrop.GetCPPTypeObject()), true, FString(), InDropPosition + FVector2D(0.f, -34.f)))
						{
							Graph->GetController()->AddLink(VariableNode->FindPin(TEXT("Value"))->GetPinPath(), ModelPin->GetPinPath(), true);
						}
						Graph->GetController()->CloseUndoBracket();
						return true;
					}
				}
			}
		}
	}
#endif
	return false;
}

void URigVMEdGraphSchema::StartGraphNodeInteraction(UEdGraphNode* InNode) const
{
#if WITH_EDITOR

	check(InNode);

	if(NodesBeingInteracted.Contains(InNode))
	{
		return;
	}
	
	NodePositionsDuringStart.Reset();
	NodesBeingInteracted.Reset();

	const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InNode->GetOuter());
	if (Graph == nullptr)
	{
		return;
	}

	check(Graph->GetController());
	check(Graph->GetModel());

	NodesBeingInteracted = GetNodesToMoveForNode(InNode);

	for (const UEdGraphNode* NodeToMove : NodesBeingInteracted)
	{
		FName NodeName = NodeToMove->GetFName();
		if (const URigVMNode* ModelNode = Graph->GetModel()->FindNodeByName(NodeName))
		{
			NodePositionsDuringStart.FindOrAdd(NodeName, ModelNode->GetPosition());
		}
	}

#endif
}

void URigVMEdGraphSchema::EndGraphNodeInteraction(UEdGraphNode* InNode) const
{
#if WITH_EDITOR

	URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InNode->GetOuter());
	if (Graph == nullptr)
	{
		return;
	}

	check(Graph->GetController());
	check(Graph->GetModel());

	TArray<UEdGraphNode*> NodesToMove = GetNodesToMoveForNode(InNode);
	
	bool bMovedSomething = false;

	FGuardSkipDirtyBlueprintStatus GuardDirtyBlueprintStatus(Graph->GetAsset(), true);

	Graph->GetController()->OpenUndoBracket(TEXT("Move Nodes"));

	for (const UEdGraphNode* NodeToMove : NodesToMove)
	{
		FName NodeName = NodeToMove->GetFName();
		if (Graph->GetModel()->FindNodeByName(NodeName))
		{
			FVector2D NewPosition(NodeToMove->NodePosX, NodeToMove->NodePosY);

			if(const FVector2D* OldPosition = NodePositionsDuringStart.Find(NodeName))
			{
				TGuardValue<bool> SuspendNotification(Graph->bSuspendModelNotifications, true);
				Graph->GetController()->SetNodePositionByName(NodeName, *OldPosition, false, false);
			}
			
			if(Graph->GetController()->SetNodePositionByName(NodeName, NewPosition, true, false, true))
			{
				bMovedSomething = true;
			}
		}
	}

	if (bMovedSomething)
	{
		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}

		Graph->GetController()->CloseUndoBracket();
	}
	else
	{
		Graph->GetController()->CancelUndoBracket();
	}

	NodesBeingInteracted.Reset();
	NodePositionsDuringStart.Reset();

#endif
}

TArray<UEdGraphNode*> URigVMEdGraphSchema::GetNodesToMoveForNode(UEdGraphNode* InNode)
{
	TArray<UEdGraphNode*> NodesToMove;

#if WITH_EDITOR

	URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InNode->GetOuter());
	if (Graph == nullptr)
	{
		return NodesToMove;
	}

	NodesToMove.Add(InNode);

	for (UEdGraphNode* SelectedGraphNode : Graph->Nodes)
	{
		if (SelectedGraphNode->IsSelected())
		{
			NodesToMove.AddUnique(SelectedGraphNode);
		}
	}

	for (int32 NodeIndex = 0; NodeIndex < NodesToMove.Num(); NodeIndex++)
	{
		if (const UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(NodesToMove[NodeIndex]))
		{
			if (CommentNode->MoveMode == ECommentBoxMode::GroupMovement)
			{
				for (FCommentNodeSet::TConstIterator NodeIt(CommentNode->GetNodesUnderComment()); NodeIt; ++NodeIt)
				{
					if (UEdGraphNode* NodeUnderComment = Cast<UEdGraphNode>(*NodeIt))
					{
						NodesToMove.AddUnique(NodeUnderComment);
					}
				}
			}
		}
	}

#endif

	return NodesToMove;
}

FVector2D URigVMEdGraphSchema::GetNodePositionAtStartOfInteraction(const UEdGraphNode* InNode) const
{
#if WITH_EDITOR
	if(InNode)
	{
		if(const FVector2D* Position = NodePositionsDuringStart.Find(InNode->GetFName()))
		{
			return *Position;
		}

		return FVector2D(InNode->NodePosX, InNode->NodePosY);
	}
#endif

	return FVector2D::ZeroVector;
}

bool URigVMEdGraphSchema::AutowireNewNode(URigVMEdGraphNode* NewNode, UEdGraphPin* FromPin) const
{
	// copying high level information into a local array since the try create connection below
	// may cause the pin array to be destroyed / changed
	TArray<TPair<FName, EEdGraphPinDirection>> PinsToVisit;
	for(UEdGraphPin* Pin : NewNode->Pins)
	{
		PinsToVisit.Emplace(Pin->GetFName(), Pin->Direction);
	}

	TArray<UEdGraphPin*> OldLinkedTo = FromPin->LinkedTo;
	for(const TPair<FName, EEdGraphPinDirection>& PinToVisit : PinsToVisit)
	{
		UEdGraphPin* Pin = NewNode->FindPin(PinToVisit.Key, PinToVisit.Value);
		if(Pin == nullptr)
		{
			continue;
		}
		
		if (Pin->ParentPin != nullptr)
		{
			continue;
		}

		FPinConnectionResponse ConnectResponse = CanCreateConnection(FromPin, Pin);
		if(ConnectResponse.Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
		{
			if (TryCreateConnection(FromPin, Pin))
			{
				// It might have been linked in a different direction. Find pin with the correct direction
				if (Pin->LinkedTo.IsEmpty())
				{
					for (UEdGraphPin* Linked : FromPin->LinkedTo)
					{
						if (Linked->GetOwningNode() == Pin->GetOwningNode())
						{
							Pin = Linked;
							break;
						}
					}
				}

				// If the pin is an execute context, try to create a link to the pin which was previously connected to FromPin
				if (const URigVMPin* ModelPin = NewNode->FindModelPinFromGraphPin(Pin))
				{
					if (!OldLinkedTo.IsEmpty() && ModelPin->IsExecuteContext())
					{
						const bool bIsInput = Pin->Direction == EEdGraphPinDirection::EGPD_Input;

						// If the pin is an input to a control flow node, the output used should be the completed pin
						if (ModelPin->GetNode()->IsControlFlowNode())
						{
							if (bIsInput)
							{
								ModelPin = ModelPin->GetNode()->FindPin(FRigVMStruct::ControlFlowCompletedName.ToString());
							}
						}

						// Try to create the second connection
						if (UEdGraphPin* OppositePin = NewNode->FindGraphPinFromModelPin(ModelPin, !bIsInput))
						{
							if (CanCreateConnection(OppositePin, OldLinkedTo[0]).Response != ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW)
							{
								TryCreateConnection(OppositePin, OldLinkedTo[0]);
							}
						}
					}

					// copy the default value over if the node is a reroute,
					// a make array, make struct or make constant
					if(!ModelPin->IsExecuteContext() && FromPin->Direction == EGPD_Input)
					{
						if(const URigVMPin* OtherModelPin = ModelPin->GetGraph()->FindPin(FromPin->GetName()))
						{
							FString PinNameToSet;
							if(ModelPin->GetNode()->IsA<URigVMRerouteNode>())
							{
								PinNameToSet = TEXT("Value");
							}
							else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelPin->GetNode()))
							{
								if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
								{
									if(Factory->GetFactoryName() == FRigVMDispatch_Constant().GetFactoryName())
									{
										PinNameToSet = TEXT("Value");
									}
									else if(Factory->GetFactoryName() == FRigVMDispatch_MakeStruct().GetFactoryName())
									{
										PinNameToSet = TEXT("Elements");
									}
									else if(Factory->GetFactoryName() == FRigVMDispatch_ArrayMake().GetFactoryName())
									{
										PinNameToSet = TEXT("Values");
									}
								}
							}

							if(!PinNameToSet.IsEmpty())
							{
								const FString DefaultValue = OtherModelPin->GetDefaultValue();
								if(!DefaultValue.IsEmpty())
								{
									NewNode->GetController()->SetPinDefaultValue(RigVMStringUtils::JoinPinPath(NewNode->GetName(), PinNameToSet), DefaultValue, true, true, false, true);
								}
							}
						}
					}
				}
				return true;
			}
		}
	}
	return false;
}

void URigVMEdGraphSchema::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph,
                                                 UObject* InSubject)
{
	switch(InNotifType)
	{
		case ERigVMGraphNotifType::NodeAdded:
		case ERigVMGraphNotifType::NodeRemoved:
		case ERigVMGraphNotifType::PinAdded:
		case ERigVMGraphNotifType::PinRemoved:
		case ERigVMGraphNotifType::PinRenamed:
		case ERigVMGraphNotifType::PinArraySizeChanged:
		case ERigVMGraphNotifType::PinTypeChanged:
		case ERigVMGraphNotifType::LinkAdded:
		case ERigVMGraphNotifType::LinkRemoved:
		{
			LastPinForCompatibleCheck = nullptr;
			break;
		}
		default:
		{
			break;
		}
	}
}

TSubclassOf<URigVMEdGraphNode> URigVMEdGraphSchema::GetGraphNodeClass(const URigVMEdGraph* InGraph) const
{
	if (FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph))
	{
		return RigBlueprint->GetRigVMClientHost()->GetRigVMEdGraphNodeClass();
	}
	return nullptr;
}

bool URigVMEdGraphSchema::IsRigVMDefaultEvent(const FName& InEventName) const
{
	return false;
}

bool URigVMEdGraphSchema::ShouldHidePinOnNode(const URigVMPin* InModelPin) const
{
	return false;
}

#undef LOCTEXT_NAMESPACE

