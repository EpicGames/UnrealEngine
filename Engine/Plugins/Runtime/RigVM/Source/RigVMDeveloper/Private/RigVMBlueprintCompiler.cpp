// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMBlueprintCompiler.h"
#include "RigVMHost.h"
#include "KismetCompiler.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMCore/RigVMStruct.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Stats/StatsHierarchical.h"

DEFINE_LOG_CATEGORY_STATIC(LogRigVMCompiler, Log, All);
#define LOCTEXT_NAMESPACE "RigVMBlueprintCompiler"

void FRigVMBlueprintCompiler::Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMBlueprintCompilerContext Compiler(Blueprint, Results, CompileOptions);
	Compiler.Compile();
}

void FRigVMBlueprintCompilerContext::MarkCompilationFailed(const FString& Message)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigVMBlueprint* RigVMBlueprint = Cast<URigVMBlueprint>(Blueprint);
	if (RigVMBlueprint)
	{
		Blueprint->Status = BS_Error;
		Blueprint->MarkPackageDirty();
		UE_LOGF(LogRigVMCompiler, Error, "%ls", *Message);
		MessageLog.Error(*Message);

#if WITH_EDITOR
		FNotificationInfo Info(FText::FromString(Message));
		Info.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Error"));
		Info.bFireAndForget = true;
		Info.FadeOutDuration = 5.0f;
		Info.ExpireDuration = 5.0f;
		TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
#endif
	}
}

void FRigVMBlueprintCompilerContext::OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!Context.bIsSkeletonOnly)
	{
		URigVMBlueprint* RigVMBlueprint = Cast<URigVMBlueprint>(Blueprint);
		if (RigVMBlueprint)
		{
			if (!RigVMBlueprint->bIsRegeneratingOnLoad)
			{
				// todo: can we find out somehow if we are cooking?
				RigVMBlueprint->RecompileVM();
			}
		}
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER(FKismetCompilerContext::OnPostCDOCompiled)
		FKismetCompilerContext::OnPostCDOCompiled(Context);
	}
}

void FRigVMBlueprintCompilerContext::CopyTermDefaultsToDefaultObject(UObject* DefaultObject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FKismetCompilerContext::CopyTermDefaultsToDefaultObject(DefaultObject);

	URigVMBlueprint* RigVMBlueprint = Cast<URigVMBlueprint>(Blueprint);
	if (RigVMBlueprint)
	{
		// here, CDO is initialized from BP,
		// and in URigVMHost::InitializeFromCDO,
		// other instances are then initialized From the CDO 
		URigVMHost* Host = CastChecked<URigVMHost>(DefaultObject);
		Host->PostInitInstanceIfRequired();
	}
}

void FRigVMBlueprintCompilerContext::EnsureProperGeneratedClass(UClass*& TargetUClass)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if( TargetUClass && !((UObject*)TargetUClass)->IsA(URigVMBlueprintGeneratedClass::StaticClass()) )
	{
		FKismetCompilerUtilities::ConsignToOblivion(TargetUClass, Blueprint->bIsRegeneratingOnLoad);
		TargetUClass = nullptr;
	}
}

void FRigVMBlueprintCompilerContext::SpawnNewClass(const FString& NewClassName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	NewRigVMBlueprintGeneratedClass = FindObject<URigVMBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if (NewRigVMBlueprintGeneratedClass == nullptr)
	{
		NewRigVMBlueprintGeneratedClass = NewObject<URigVMBlueprintGeneratedClass>(Blueprint->GetOutermost(), FName(*NewClassName), RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewRigVMBlueprintGeneratedClass);
	}
	NewClass = NewRigVMBlueprintGeneratedClass;
}

void FRigVMBlueprintCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* ClassToUse)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	NewRigVMBlueprintGeneratedClass = CastChecked<URigVMBlueprintGeneratedClass>(ClassToUse);
}

void FRigVMBlueprintCompilerContext::CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FKismetCompilerContext::CleanAndSanitizeClass(ClassToClean, InOldCDO);

	// Make sure our typed pointer is set
	check(ClassToClean == NewClass && NewRigVMBlueprintGeneratedClass == NewClass);
}

void FRigVMBlueprintCompilerContext::PreCompileUpdateBlueprintOnLoad(UBlueprint* BP)
{
	if (URigVMBlueprint* RigBlueprint = Cast<URigVMBlueprint>(BP))
	{
		RigBlueprint->CreateMemberVariablesOnLoad();
	}
}


#undef LOCTEXT_NAMESPACE
