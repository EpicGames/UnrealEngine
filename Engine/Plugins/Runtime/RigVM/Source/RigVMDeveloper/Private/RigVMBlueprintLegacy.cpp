// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMBlueprintLegacy.h"

#include "RigVMBlueprintGeneratedClass.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphNode_Comment.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"
#include "RigVMObjectVersion.h"
#include "BlueprintCompilationManager.h"
#include "IMessageLogListing.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "RigVMPythonUtils.h"
#include "RigVMTypeUtils.h"
#include "Algo/Count.h"
#include "Misc/StringOutputDevice.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Stats/StatsHierarchical.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMBlueprintLegacy)

#if WITH_EDITOR
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMBlueprintUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "RigVMEditorModule.h"
#include "ScopedTransaction.h"
#if !WITH_RIGVMLEGACYEDITOR
#include "RigVMEditor/Private/Editor/Kismet/RigVMBlueprintCompilationManager.h"
#endif
#endif//WITH_EDITOR

#define LOCTEXT_NAMESPACE "RigVMBlueprint"

FEdGraphPinType FRigVMOldPublicFunctionArg::GetPinType() const
{
	UObject* CPPTypeObject = nullptr;
	if(CPPTypeObjectPath.IsValid())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(CPPTypeObjectPath.ToString());
	}
	FRigVMExternalVariable Variable = FRigVMExternalVariable::Make(FGuid(), Name, CPPType.ToString(), CPPTypeObject, false, false);
	return RigVMTypeUtils::PinTypeFromExternalVariable(Variable);
}

FRigVMOldPublicFunctionData::~FRigVMOldPublicFunctionData() = default;

bool FRigVMOldPublicFunctionData::IsMutable() const
{
	for(const FRigVMOldPublicFunctionArg& Arg : Arguments)
	{
		if(!Arg.CPPTypeObjectPath.IsNone())
		{
			if(UScriptStruct* Struct = Cast<UScriptStruct>(
				RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(Arg.CPPTypeObjectPath.ToString())))
			{
				if(Struct->IsChildOf(FRigVMExecuteContext::StaticStruct()))
				{
					return true;
				}
			}
		}
	}
	return false;
}

void URigVMBlueprint::BeginDestroy()
{
	Super::BeginDestroy();
	IRigVMEditorAssetInterface::BeginDestroy();
}

void URigVMBlueprint::UpdateSupportedEventNames()
{
	SupportedEventNames.Reset();
	if (URigVMHost* CDO = Cast<URigVMHost>(GetDefaultsObject()))
	{
		SupportedEventNames = CDO->GetSupportedEvents();
	}
}

UObject* URigVMBlueprint::GetDefaultsObject()
{
	if (GeneratedClass)
	{
		return GeneratedClass->GetDefaultObject();
	}
	return nullptr;
}

void URigVMBlueprint::PostEditChangeBlueprintActors()
{
	FBlueprintEditorUtils::PostEditChangeBlueprintActors(this);
}

FCompilerResultsLog URigVMBlueprint::CompileBlueprint()
{
	FCompilerResultsLog LogResults;
	LogResults.SetSourcePath(GetPathName());
	LogResults.BeginEvent(TEXT("Compile"));
		
	// TODO: sara-s remove once blueprint backend is replaced
	{
		EBlueprintCompileOptions CompileOptions = EBlueprintCompileOptions::None;

		// If compilation is enabled during PIE/simulation, references to the CDO might be held by a script variable.
		// Thus, we set the flag to direct the compiler to allow those references to be replaced during reinstancing.
		if (GEditor->PlayWorld != nullptr)
		{
			CompileOptions |= EBlueprintCompileOptions::IncludeCDOInReferenceReplacement;
		}
		
		FKismetEditorUtilities::CompileBlueprint(this, CompileOptions, &LogResults);
	}

	LogResults.EndEvent();

	// CachedNumWarnings = LogResults.NumWarnings;
	// CachedNumErrors = LogResults.NumErrors;

	if (UpgradeNotesLog.IsValid())
	{
		for (TSharedRef<FTokenizedMessage> Message :UpgradeNotesLog->Messages)
		{
			LogResults.AddTokenizedMessage(Message);
		}
	}

	return LogResults;
}

URigVMBlueprint::URigVMBlueprint()
	: IRigVMEditorAssetInterface()
{
}

URigVMBlueprint::URigVMBlueprint(const FObjectInitializer& ObjectInitializer)
	: IRigVMEditorAssetInterface(ObjectInitializer)
{

#if WITH_EDITORONLY_DATA
	ReferencedObjectPathsStored = false;
#endif

	bRecompileOnLoad = 0;
	SupportedEventNames.Reset();
	VMCompileSettings.ASTSettings.ReportDelegate.BindUObject(this, &IRigVMEditorAssetInterface::HandleReportFromCompiler);

	bUpdatingExternalVariables = false;
	
#if WITH_EDITOR
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		CompileLog.bSilentMode = true;
	}
	CompileLog.SetSourcePath(GetPathName());
#endif

	if(GetClass() == URigVMBlueprint::StaticClass())
	{
		CommonInitialization(ObjectInitializer);
	}

	UBlueprint::OnChanged().AddUObject(this, &URigVMBlueprint::OnBlueprintChanged);
	UBlueprint::OnSetObjectBeingDebugged().AddUObject(this, &URigVMBlueprint::OnSetObjectBeingDebuggedReceived);
}

URigVMBlueprintGeneratedClass* URigVMBlueprint::GetRigVMBlueprintGeneratedClass() const
{
	URigVMBlueprintGeneratedClass* Result = Cast<URigVMBlueprintGeneratedClass>(*GeneratedClass);
	return Result;
}

void URigVMBlueprint::PostSerialize(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		if(Model_DEPRECATED || FunctionLibrary_DEPRECATED)
		{
			TGuardValue<bool> DisableClientNotifs(GetRigVMClient()->bSuspendNotifications, true);
			GetRigVMClient()->SetFromDeprecatedData(Model_DEPRECATED, FunctionLibrary_DEPRECATED);
		}
	}
}

UClass* URigVMBlueprint::GetBlueprintClass() const
{
	return URigVMBlueprintGeneratedClass::StaticClass();
}

UClass* URigVMBlueprint::RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO)
{
	UClass* Result;
	{
		TGuardValue<bool> NotificationGuard(bSuspendAllNotifications, true);
		Result = Super::RegenerateClass(ClassToRegenerate, PreviousCDO);
		if (URigVMBlueprintGeneratedClass* Generated = Cast<URigVMBlueprintGeneratedClass>(Result))
		{
			Generated->SupportedEventNames = SupportedEventNames;
			Generated->AssetVariant = AssetVariant;
		}
	}
	return Result;
}

TArray<FRigVMExternalVariable> URigVMBlueprint::GetExternalVariables(bool bFallbackToBlueprint) const
{
	TArray<FRigVMExternalVariable> ExternalVariables;
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		const bool bCreateCDO = !IsAsyncLoading();
		if (URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */)))
		{
			return CDO->GetExternalVariablesImpl(bFallbackToBlueprint /* rely on variables within blueprint */);
		}
	}
	return ExternalVariables;
}

URigVM* URigVMBlueprint::GetVM(bool bCreateIfNeeded) const
{
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		if (URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateIfNeeded)))
		{
			return CDO->GetVM();
		}
	}
	return nullptr;
}

FRigVMExtendedExecuteContext* URigVMBlueprint::GetRigVMExtendedExecuteContext()
{
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		if (URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(false)))
		{
			return &CDO->GetRigVMExtendedExecuteContext();
		}
	}
	return nullptr;
}

void URigVMBlueprint::SetAssetStatus(const ERigVMAssetStatus& InStatus)
{
	switch (InStatus)
	{
		case RVMA_Dirty: Status = BS_Dirty; break;
		case RVMA_Error: Status = BS_Error; break;
		case RVMA_UpToDate: Status = BS_UpToDate; break;
		case RVMA_BeingCreated: Status = BS_BeingCreated; break;
		case RVMA_UpToDateWithWarnings: Status = BS_UpToDateWithWarnings; break;
		default: Status = BS_Unknown; break;
	}
}

ERigVMAssetStatus URigVMBlueprint::GetAssetStatus() const
{
	switch (Status)
	{
		case BS_Dirty:
		case BS_Unknown:
		{
			// Check if the asset is actually up to date by comparing the hashes.
			// This handles cases where status is incorrectly set to dirty or defaults to unknown.
			if (GetRigVMClient())
			{
				if (GetRigVMClient()->GetStructureHash() == GetRigVMClient()->GetSerializedStructureHash())
				{
					// Hashes match, asset is up to date.
					// Set the Status property so UI sees the correct state.
					const_cast<URigVMBlueprint*>(this)->Status = BS_UpToDate;
					return RVMA_UpToDate;
				}
			}
			return Status == BS_Dirty ? RVMA_Dirty : RVMA_Unknown;
		}
		case BS_Error: return RVMA_Error;
		case BS_UpToDate: return RVMA_UpToDate;
		case BS_BeingCreated: return RVMA_BeingCreated;
		case BS_UpToDateWithWarnings: return RVMA_UpToDateWithWarnings;
		default: return RVMA_Unknown;
	}
}


TArray<UObject*> URigVMBlueprint::GetArchetypeInstances(bool bIncludeCDO, bool bIncludeDerivedClasses) const
{
	TArray<UObject*> ArchetypeInstances;
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		if (bIncludeCDO)
		{
			URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(false /* create if needed */));
			ArchetypeInstances.Add(CDO);
		}
		//CDO->GetArchetypeInstances(ArchetypeInstances);
		GetObjectsOfClass(RigClass, ArchetypeInstances, bIncludeDerivedClasses);
	}
	return ArchetypeInstances;
}

FRigVMDebugInfo& URigVMBlueprint::GetDebugInfo()
{
	URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass();
	URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(false));
	return CDO->GetDebugInfo();
}

URigVMHost* URigVMBlueprint::CreateRigVMHostSuper(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	return NewObject<URigVMHost>(InOuter, GetRigVMHostClass(), InName, InFlags);
}

void URigVMBlueprint::MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus)
{
	const TEnumAsByte<EBlueprintStatus> OldStatus = Status;
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(this);
	if (bSkipDirtyAssetStatus)
	{
		Status = OldStatus;
	}
}

void URigVMBlueprint::MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent)
{
	FBlueprintEditorUtils::MarkBlueprintAsModified(this, PropertyChangedEvent);
}

void URigVMBlueprint::AddUbergraphPage(URigVMEdGraph* RigVMEdGraph)
{
	FBlueprintEditorUtils::AddUbergraphPage(this, RigVMEdGraph);
}

void URigVMBlueprint::AddLastEditedDocument(URigVMEdGraph* RigVMEdGraph)
{
	LastEditedDocuments.AddUnique(RigVMEdGraph);
}

void URigVMBlueprint::Compile()
{
#if WITH_RIGVMLEGACYEDITOR
	FBPCompileRequest Request(this, EBlueprintCompileOptions::None, nullptr);
	FBlueprintCompilationManager::CompileSynchronously(Request);
#else
	// FRigVMBPCompileRequest Request(this, EBlueprintCompileOptions::None, nullptr);
	// FRigVMBlueprintCompilationManager::CompileSynchronously(Request);
#endif
}

void URigVMBlueprint::PatchVariableNodesOnLoad()
{
	IRigVMEditorAssetInterface::PatchVariableNodesOnLoad();
#if WITH_EDITOR
	LastNewVariables = NewVariables;
#endif
}

void URigVMBlueprint::AddPinWatch(UEdGraphPin* InPin)
{
	if (!FKismetDebugUtilities::IsPinBeingWatched(this, InPin))
	{
		FKismetDebugUtilities::AddPinWatch(this, FBlueprintWatchedPin(InPin));
	}
}

void URigVMBlueprint::RemovePinWatch(UEdGraphPin* InPin)
{
	FKismetDebugUtilities::RemovePinWatch(this, InPin);
}

void URigVMBlueprint::ClearPinWatches()
{
	FKismetDebugUtilities::ClearPinWatches(this);
}

bool URigVMBlueprint::IsPinBeingWatched(const UEdGraphPin* InPin) const
{
	return FKismetDebugUtilities::IsPinBeingWatched(this, InPin);
}

bool URigVMBlueprint::NodeContainsWatchedPins(const UEdGraphNode* InNode) const
{
	UEdGraphPin* WatchedPin = FKismetDebugUtilities::FindPinWatchByPredicate(this, [InNode](const UEdGraphPin* InPin)
		{
			return InPin->GetOuter() == InNode;
		});
	return WatchedPin != nullptr;
}

void URigVMBlueprint::ForeachPinWatch(TFunctionRef<void(UEdGraphPin*)> Task)
{
	FKismetDebugUtilities::ForeachPinWatch(this, Task);
}

TMap<FString, FSoftObjectPath>& URigVMBlueprint::GetUserDefinedStructGuidToPathName(bool bFromCDO)
{
	if (bFromCDO)
	{
		URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass();
		const bool bCreateCDO = !IsAsyncLoading();
		URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */));
		if (CDO)
		{
			return CDO->UserDefinedStructGuidToPathName;
		}
	}
	return UserDefinedStructGuidToPathName;
}

TMap<FString, FSoftObjectPath>& URigVMBlueprint::GetUserDefinedEnumToPathName(bool bFromCDO)
{
	if (bFromCDO)
	{
		URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass();
		const bool bCreateCDO = !IsAsyncLoading();
		URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */));
		if (CDO)
		{
			return CDO->UserDefinedEnumToPathName;
		}
	}
	return UserDefinedEnumToPathName;
}

TSet<TObjectPtr<UObject>>& URigVMBlueprint::GetUserDefinedTypesInUse(bool bFromCDO)
{
	if (bFromCDO)
	{
		URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass();
		const bool bCreateCDO = !IsAsyncLoading();
		URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */));
		if (CDO)
		{
			return CDO->UserDefinedTypesInUse;
		}
	}
	return UserDefinedTypesInUse;
}

const TArray<FString>& URigVMBlueprint::GetRequiredPlugins(bool bRefresh) const
{
	if (!bRefresh)
	{
		return RequiredPlugins;
	}
	
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		const bool bCreateCDO = !IsAsyncLoading();
		if (URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */)))
		{
			CDO->GenerateRequiredPluginsData(CDO->GetRigVMExtendedExecuteContext());
			return CDO->GetRequiredPlugins();
		}
	}
	// fall back to the serialized data if we don't have a CDO yet
	return RequiredPlugins;
}

#if WITH_EDITORONLY_DATA
void URigVMBlueprint::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);

	// we append the object version to the hash so that we invalidate
	// cooked assets based on this.
	int32 ObjectVersion = (int32)FRigVMObjectVersion::LatestVersion;
	Context.Update(&ObjectVersion, sizeof(ObjectVersion));

	// we also append the cook version to the hash so that we invalidate
	// cooked assets based on the enum independent from the object version.
	int32 CookVersion = (int32)ERigVMCookVersion::LatestVersion;
	Context.Update(&CookVersion, sizeof(CookVersion));
	
	// Also make sure to invalidate the cooked assets based on if we want to use the 
	// localize registry at runtime. If so - we'll need to make sure that all assets are
	// cooked like that.
	bool bUseLocalizedRegistry = CVarRigVMEnableLocalizedRegistry.GetValueOnAnyThread();
	Context.Update(&bUseLocalizedRegistry, sizeof(bUseLocalizedRegistry));
}
#endif

#if WITH_EDITOR
bool URigVMBlueprint::CopyBlueprintToAsset(URigVMBlueprint* InBlueprint, URigVMRuntimeAsset* InAsset)
{
	// Copy Variables
	{
		TArray<FRigVMGraphVariableDescription> Variables = InBlueprint->GetAssetVariables();
		for (FRigVMGraphVariableDescription& Variable : Variables)
		{
			FRigVMExternalVariable ExternalVariable = Variable.ToExternalVariable();
			const FName Name = InAsset->AddHostMemberVariableFromExternal(ExternalVariable, Variable.DefaultValue);
			InAsset->SetVariableCategory(Name, Variable.Category.ToString());
			InAsset->SetVariableExposeOnSpawn(Name, Variable.bExposedOnSpawn);
			InAsset->SetVariableExposeToCinematics(Name, Variable.bExposeToCinematics);
			InAsset->SetVariablePrivate(Name, Variable.bPrivate);
			InAsset->SetVariablePublic(Name, Variable.bPublic);
			InAsset->SetVariableTooltip(Name, Variable.Tooltip);
			if (URigVMBlueprintGeneratedClass* GeneratedClass = InBlueprint->GetRigVMBlueprintGeneratedClass())
			{
				if (FProperty* Property = GeneratedClass->FindPropertyByName(Variable.Name))
				{
					for (const TTuple<FName, FString>& MetaDataPair : *Property->GetMetaDataMap())
					{
						InAsset->SetVariableMetadataValue(Name, MetaDataPair.Key, MetaDataPair.Value);
					}
				}
			}
		}
	}
	InAsset->RuntimeSettings = InBlueprint->GetVMRuntimeSettings();
	InAsset->PublicGraphFunctions = InBlueprint->PublicGraphFunctions;
	InAsset->AssetVariant = InBlueprint->GetAssetVariant();
	InAsset->AssetUserData = Cast<URigVMHost>(InBlueprint->GetRigVMBlueprintGeneratedClass()->GetDefaultObject())->AssetUserData;
	InAsset->AssetUserDataEditorOnly = Cast<URigVMHost>(InBlueprint->GetRigVMBlueprintGeneratedClass()->GetDefaultObject())->AssetUserDataEditorOnly;
	InAsset->SupportedEventNames = InBlueprint->GetSupportedEventNames();
	InAsset->UserDefinedStructGuidToPathName = InBlueprint->UserDefinedStructGuidToPathName;
	InAsset->UserDefinedEnumToPathName = InBlueprint->UserDefinedEnumToPathName;
	InAsset->UserDefinedTypesInUse = InBlueprint->UserDefinedTypesInUse;
	InAsset->RequiredPlugins = InBlueprint->RequiredPlugins;

	if (URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(InAsset->EditorAsset))
	{
		FRigVMClient* TargetClient = &EditorAsset->RigVMClient;

		TObjectPtr<URigVMFunctionLibrary> SourceFunctionLibrary = InBlueprint->GetRigVMClient()->FunctionLibrary;
		FObjectDuplicationParameters DupParams = InitStaticDuplicateObjectParams(SourceFunctionLibrary, InAsset->GetEditorOnlyData(), SourceFunctionLibrary->GetFName(), RF_Transient);
		TargetClient->FunctionLibrary = Cast<URigVMFunctionLibrary>(StaticDuplicateObjectEx(DupParams));
		TargetClient->FunctionLibrary->SetSchemaClass(EditorAsset->GetRigVMSchemaClass());

		if (!TargetClient->FunctionLibrary->GetFunctionHostObjectPathDelegate.IsBound())
		{
			TargetClient->SetGetFunctionHostObjectPathDelegate(TargetClient->FunctionLibrary);
		}

		auto ReplaceIdentifier = [InBlueprint, InAsset, TargetClient](FRigVMGraphFunctionIdentifier& Identifier)
			{
				if (Identifier.HostObject == InBlueprint->GetRigVMBlueprintGeneratedClass())
				{
					Identifier.HostObject = InAsset;
					if (URigVMLibraryNode* FunctionNode = TargetClient->GetFunctionLibrary()->FindFunction(Identifier.GetFunctionFName()))
					{
						Identifier.SetLibraryNodePath(FSoftObjectPath(FunctionNode).ToString());
					}
				}
			};

		// Lets add the functions to the function store
		FRigVMGraphFunctionStore* FunctionStore = InAsset->GetRigVMGraphFunctionStore();
		URigVMBuildData* BuildData = URigVMBuildData::Get();
		for (URigVMNode* Node : TargetClient->FunctionLibrary->GetNodes())
		{
			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Node))
			{
				const bool bPublicFunction = TargetClient->FunctionLibrary->IsFunctionPublic(LibraryNode->GetFName());
				FRigVMGraphFunctionData* NewData = FunctionStore->AddFunction(LibraryNode->GetFunctionHeader(InAsset), bPublicFunction);

				if (IRigVMGraphFunctionHost* OldFunctionStoreHost = Cast<IRigVMGraphFunctionHost>(InBlueprint->GetRigVMBlueprintGeneratedClass()))
				{
					if (FRigVMGraphFunctionData* FunctionData = OldFunctionStoreHost->GetRigVMGraphFunctionStore()->FindFunctionByName(LibraryNode->GetFName()))
					{
						TMap<FRigVMGraphFunctionIdentifier, uint32> NewDependencies = FunctionData->Header.Dependencies;
						for (TTuple<FRigVMGraphFunctionIdentifier, unsigned>& Dependency : NewDependencies)
						{
							ReplaceIdentifier(Dependency.Key);
							Dependency.Value = 0;
						}
						FunctionStore->UpdateDependencies(NewData->Header.LibraryPointer, NewDependencies);
					}
				}
			}
		}
		
		TargetClient->Models.Empty();
		TargetClient->Models.Reserve(InBlueprint->GetRigVMClient()->Models.Num());
		for (URigVMGraph* Graph : InBlueprint->GetRigVMClient()->Models)
		{
			DupParams = InitStaticDuplicateObjectParams(Graph, InAsset->GetEditorOnlyData(), Graph->GetFName(), RF_Transient);
			TargetClient->Models.Add(Cast<URigVMGraph>(StaticDuplicateObjectEx(DupParams)));
		}
		
		// Replace local function references to point to new asset
		TArray<URigVMGraph*> Graphs = TargetClient->GetAllModels(true, true);
		for (URigVMGraph* Graph : Graphs)
		{
			Graph->SetSchemaClass(EditorAsset->GetRigVMSchemaClass());
			Graph->SetDefaultFunctionLibrary(TargetClient->FunctionLibrary);
			for (URigVMNode* Node : Graph->GetNodes())
			{
				if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
				{
					ReplaceIdentifier(FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer);

					for (TPair<FRigVMGraphFunctionIdentifier, uint32>& Dependency : FunctionReferenceNode->ReferencedFunctionHeader.Dependencies)
					{
						ReplaceIdentifier(Dependency.Key);
						Dependency.Value = 0;
					}

					BuildData->RegisterFunctionReference(FunctionReferenceNode->GetFunctionIdentifier(), FunctionReferenceNode);
				}
			}
		}

		// Only create top-level EdGraphs via CreateEdGraph (which adds to RootEdGraphs).
		// Nested collapse/aggregate sub-graphs are created as SubGraphs on their parent
		// EdGraph via CreateEdGraphForCollapseNodeIfNeeded below. This matches how a
		// freshly authored (non-converted) asset is built. Iterating GetAllModels(true, true)
		// with CreateEdGraph would incorrectly add nested graphs to RootEdGraphs and surface
		// them through GetUberGraphs(), causing the editor to focus an aggregate sub-graph
		// on open.
		for (URigVMGraph* Graph : TargetClient->Models)
		{
			EditorAsset->CreateEdGraph(Graph);
		}

		// Create sub-graph EdGraphs so the LastEditedDocuments remapping below can resolve
		// entries that point at aggregate/collapse/function sub-graphs. Iteration order from
		// GetAllModels(true, true) is top-level first, then contained graphs depth-first, so
		// each parent EdGraph already exists by the time we process its children.
		//
		// CreateEdGraphForCollapseNodeIfNeeded calls ResendAllNotifications on the contained
		// graph's controller, which drives the RigVM change cascade through
		// RequestAutoVMRecompilation and would re-compile the asset once per collapse node
		// while bAutoRecompileVM is on. Suppress it for the duration of the creation loop so
		// we compile exactly once at the end via CompileBlueprint().
		{
			TGuardValue<bool> DisableAutoCompile(EditorAsset->bAutoRecompileVM, false);
			for (URigVMGraph* Graph : TargetClient->GetAllModels(true, true))
			{
				for (URigVMNode* Node : Graph->GetNodes())
				{
					if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
					{
						EditorAsset->CreateEdGraphForCollapseNodeIfNeeded(CollapseNode);
					}
				}
			}
		}

		// Copy LastEditedDocuments from source blueprint, remapping to new EdGraphs
		EditorAsset->GetLastEditedDocuments().Reset();
		for (const FEditedDocumentInfo& SourceDoc : InBlueprint->GetLastEditedDocuments())
		{
			if (URigVMEdGraph* SourceEdGraph = Cast<URigVMEdGraph>(SourceDoc.EditedObjectPath.ResolveObject()))
			{
				// Find the corresponding EdGraph in the new asset by ModelNodePath
				if (UEdGraph* NewEdGraph = EditorAsset->GetEdGraph(SourceEdGraph->ModelNodePath))
				{
					FEditedDocumentInfo NewDoc;
					NewDoc.EditedObjectPath = NewEdGraph;
					NewDoc.SavedViewOffset = SourceDoc.SavedViewOffset;
					NewDoc.SavedZoomAmount = SourceDoc.SavedZoomAmount;
					EditorAsset->GetLastEditedDocuments().Add(NewDoc);
				}
			}
		}

		EditorAsset->GraphDisplaySettings = InBlueprint->GetRigGraphDisplaySettings();
		EditorAsset->CompileSettings = InBlueprint->GetVMCompileSettings();
		
		InAsset->FunctionReferenceNodeData = EditorAsset->GetReferenceNodeData();
		
		// Register functions
		for (const FRigVMGraphFunctionData& Function : FunctionStore->PublicFunctions)
		{
			BuildData->RegisterPublicFunction(Function.Header.LibraryPointer, Function.Header);
		}
		
		EditorAsset->CompileBlueprint();
	}
	
	return true;
}
#endif

void URigVMBlueprint::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	UBlueprint::PostEditChangeChainProperty(PropertyChangedEvent);
	IRigVMEditorAssetInterface::PostEditChangeChainProperty(PropertyChangedEvent);
}

void URigVMBlueprint::PostRename(UObject* OldOuter, const FName OldName)
{
	UBlueprint::PostRename(OldOuter, OldName);

	// Whenever the asset is renamed/moved, generated classes parented to the old package
	// are not moved to the new package automatically (see FAssetRenameManager), so we
	// have to manually perform the move/rename, to avoid invalid reference to the old package
	
	// Note: while asset duplication doesn't duplicate the classes either, it is not a problem there
	// because we always recompile in post duplicate.
	TArray<UObject*> Objects;
	GetObjectsWithOuter(OldOuter->GetPackage(), Objects, EGetObjectsFlags::None);
	for (UObject* Object : Objects)
	{
		if (URigVMMemoryStorageGeneratorClass* MemoryClass = Cast<URigVMMemoryStorageGeneratorClass>(Object))
		{
			MemoryClass->Rename(nullptr, GetPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
		}
	}

	// Update function identifiers to reflect the new asset path
	const FString NewAssetPath = FString::Printf(TEXT("%s.%s"), *GetOuter()->GetPathName(), *GetFName().ToString());
	const FString OldAssetPath = FString::Printf(TEXT("%s.%s"), *OldOuter->GetPathName(), *OldName.ToString());

	const FString NewFunctionHostPath = FSoftObjectPath(GetRigVMGraphFunctionHost().GetObject()).ToString();
	const FString NewFunctionLibraryPath = FSoftObjectPath(GetRigVMClient()->GetFunctionLibrary()).ToString();
	const FString OldFunctionHostPath = NewFunctionHostPath.Replace(*NewAssetPath, *OldAssetPath);
	const FString OldFunctionLibraryPath = NewFunctionLibraryPath
		.Replace(*NewAssetPath, *OldAssetPath)
		.Replace(*GetFName().ToString(), *OldName.ToString());
	ReplaceFunctionIdentifiers(OldFunctionHostPath, NewFunctionHostPath, OldFunctionLibraryPath, NewFunctionLibraryPath);
}

void URigVMBlueprint::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	UBlueprint::GetPreloadDependencies(OutDeps);
	IRigVMEditorAssetInterface::GetPreloadDependencies(OutDeps);
}

void URigVMBlueprint::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

	// Make sure all the tags are accounted for in the TypeActions after we save
	if (FBlueprintActionDatabase* ActionDatabase = FBlueprintActionDatabase::TryGet())
	{
		ActionDatabase->ClearAssetActions(GetClass());
		ActionDatabase->RefreshClassActions(GetClass());
	}
}

UObject* URigVMBlueprint::GetObjectBeingDebugged(bool bEvenIfPendingKill)
{
	EGetObjectOrWorldBeingDebuggedFlags Flags = (bEvenIfPendingKill) ? EGetObjectOrWorldBeingDebuggedFlags::IgnorePendingKill : EGetObjectOrWorldBeingDebuggedFlags::None;
	return UBlueprint::GetObjectBeingDebugged(Flags);
}

void URigVMBlueprint::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	UBlueprint::PreSave(ObjectSaveContext);
	IRigVMEditorAssetInterface::PreSave(ObjectSaveContext);
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		const bool bCreateCDO = !IsAsyncLoading();
		if (const URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */)))
		{
			RigClass->SupportedEventNames = SupportedEventNames;
		}
		RigClass->AssetVariant = AssetVariant;
	}

	RequiredPlugins = GetRequiredPlugins();
}

void URigVMBlueprint::PostLoad()
{
	UBlueprint::PostLoad();
	IRigVMEditorAssetInterface::PostLoad();

	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &URigVMBlueprint::HandlePreVariableChange);
	
	UBlueprint::OnChanged().RemoveAll(this);
	UBlueprint::OnChanged().AddUObject(this, &URigVMBlueprint::HandlePostVariableChange);
}

#if WITH_EDITORONLY_DATA

void URigVMBlueprint::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	UBlueprint::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	IRigVMEditorAssetInterface::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
}
#endif


void URigVMBlueprint::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	IRigVMEditorAssetInterface::OnChanged().Broadcast(InBlueprint);
}

void URigVMBlueprint::OnSetObjectBeingDebuggedReceived(UObject* InObject)
{
	IRigVMEditorAssetInterface::OnSetObjectBeingDebugged().Broadcast(InObject);
}

void URigVMBlueprint::PreCompile()
{
	if (URigVMBlueprintGeneratedClass* RigClass = GetRigVMBlueprintGeneratedClass())
	{
		const bool bCreateCDO = !IsAsyncLoading();
		if (URigVMHost* CDO = Cast<URigVMHost>(RigClass->GetDefaultObject(bCreateCDO /* create if needed */)))
		{
			SetupDefaultObjectDuringCompilation(CDO);
			if (!this->HasAnyFlags(RF_Transient | RF_Transactional))
			{
				CDO->Modify(false);
			}
		}
	}
}

FProperty* URigVMBlueprint::FindGeneratedPropertyByName(const FName& InName)
{
	return SkeletonGeneratedClass->FindPropertyByName(InName);
}

bool URigVMBlueprint::SetVariableTooltip(const FName& InName, const FText& InTooltip)
{
	FBlueprintEditorUtils::SetBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_Tooltip, InTooltip.ToString());
	return true;
}

FText URigVMBlueprint::GetVariableTooltip(const FName& InName) const
{
	FString Result;
	FBlueprintEditorUtils::GetBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_Tooltip, Result);
	return FText::FromString(Result);
}

bool URigVMBlueprint::SetVariableCategory(const FName& InName, const FString& InCategory)
{
	FBlueprintEditorUtils::SetBlueprintVariableCategory(this, InName, nullptr, FText::FromString(InCategory), true);
	return true;
}

FString URigVMBlueprint::GetVariableCategory(const FName& InName) 
{
	return FBlueprintEditorUtils::GetBlueprintVariableCategory(this, InName, nullptr).ToString();
}

FString URigVMBlueprint::GetVariableMetadataValue(const FName& InName, const FName& InKey)
{
	FString Result;
	FBlueprintEditorUtils::GetBlueprintVariableMetaData(this, InName, nullptr, InKey, /*out*/ Result);
	return Result;
}

bool URigVMBlueprint::SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn)
{
	if(bInExposeOnSpawn)
	{
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_ExposeOnSpawn, TEXT("true"));
	}
	else
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_ExposeOnSpawn);
	} 
	return true;
}

bool URigVMBlueprint::SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics)
{
	FBlueprintEditorUtils::SetInterpFlag(this, InName, bInExposeToCinematics);
	return true;
}

bool URigVMBlueprint::SetVariablePrivate(const FName& InName, const bool bInPrivate)
{
	if(bInPrivate)
	{
		FBlueprintEditorUtils::SetBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_Private, TEXT("true"));
	}
	else
	{
		FBlueprintEditorUtils::RemoveBlueprintVariableMetaData(this, InName, nullptr, FBlueprintMetadata::MD_Private);
	} 
	return true;
}

bool URigVMBlueprint::SetVariablePublic(const FName& InName, const bool bIsPublic)
{
	FBlueprintEditorUtils::SetBlueprintOnlyEditableFlag(this, InName, !bIsPublic);
	return true;
}

FString URigVMBlueprint::OnCopyVariable(const FName& InName) const
{
	FString OutputString;
	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
	if (VarIndex != INDEX_NONE)
	{
		// make a copy of the Variable description so we can set the default value
		FBPVariableDescription Description = NewVariables[VarIndex];

		//Grab property of blueprint's current CDO
		UObject* GeneratedCDO = GeneratedClass->GetDefaultObject();

		if (FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedClass, Description.VarName))
		{
			// Grab the address of where the property is actually stored (UObject* base, plus the offset defined in the property)
			if (void* OldPropertyAddr = TargetProperty->ContainerPtrToValuePtr<void>(GeneratedCDO))
			{
				TargetProperty->ExportTextItem_Direct(Description.DefaultValue, OldPropertyAddr, OldPropertyAddr, nullptr, PPF_SerializedAsImportText);
			}
		}

		FBPVariableDescription::StaticStruct()->ExportText(OutputString, &Description, &Description, nullptr, 0, nullptr, false);
		OutputString = TEXT("BPVar") + OutputString;
	}

	return OutputString;
}

bool URigVMBlueprint::OnPasteVariable(const FString& InText)
{
	if (!ensure(InText.StartsWith(TEXT("BPVar"), ESearchCase::CaseSensitive)))
	{
		return false;
	}

	FBPVariableDescription Description;
	FStringOutputDevice Errors;
	const TCHAR* Import = InText.GetCharArray().GetData() + FCString::Strlen(TEXT("BPVar"));
	FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, PPF_None, &Errors, FBPVariableDescription::StaticStruct()->GetName());
	if (Errors.IsEmpty())
	{
		FBPVariableDescription NewVar = FBlueprintEditorUtils::DuplicateVariableDescription(this, Description);
		if (NewVar.VarGuid.IsValid())
		{
			FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteVariable", "Paste Variable: {0}"), FText::FromName(NewVar.VarName)));
			Modify();
			NewVariables.Add(NewVar);

			// Potentially adjust variable names for any child blueprints
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(this, NewVar.VarName);
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(this);
			return true;
		}
	}
	return false;
}

TArray<FRigVMGraphVariableDescription> URigVMBlueprint::GetAssetVariables() const
{
	TArray<FRigVMGraphVariableDescription> Variables;
	for (const FBPVariableDescription& BPVariable : NewVariables)
	{
		FRigVMGraphVariableDescription NewVariable;
		NewVariable.Guid = BPVariable.VarGuid;
		NewVariable.Name = BPVariable.VarName;
		NewVariable.DefaultValue = BPVariable.DefaultValue;
		if (NewVariable.DefaultValue.IsEmpty() && GeneratedClass)
		{
			if (const FProperty* Property = GeneratedClass->FindPropertyByName(NewVariable.Name))
			{
				const bool bCreateCDO = !IsAsyncLoading();
				if (const URigVMHost* CDO = Cast<URigVMHost>(GeneratedClass->GetDefaultObject(bCreateCDO))) 
				{
					if (const void* PropertyAddr = Property->ContainerPtrToValuePtr<void>(CDO))
					{
						Property->ExportTextItem_Direct(NewVariable.DefaultValue, PropertyAddr, PropertyAddr, nullptr, PPF_SerializedAsImportText);
					}
				}
			}
		}
		NewVariable.Category = BPVariable.Category;
		FString CPPType;
		UObject* CPPTypeObject;
		RigVMTypeUtils::CPPTypeFromPinType(BPVariable.VarType, CPPType, &CPPTypeObject);
		NewVariable.CPPType = CPPType;
		NewVariable.CPPTypeObject = CPPTypeObject;
		if (NewVariable.CPPTypeObject)
		{
			NewVariable.CPPTypeObjectPath = *NewVariable.CPPTypeObject->GetPathName();
		}
		if(BPVariable.HasMetaData(FBlueprintMetadata::MD_Tooltip))
		{
			NewVariable.Tooltip = FText::FromString(BPVariable.GetMetaData(FBlueprintMetadata::MD_Tooltip));
		}
		if(BPVariable.HasMetaData(FBlueprintMetadata::MD_ExposeOnSpawn))
		{
			NewVariable.bExposedOnSpawn = BPVariable.GetMetaData(FBlueprintMetadata::MD_ExposeOnSpawn) == TEXT("true");
		}
		if (BPVariable.PropertyFlags & CPF_Interp)
		{
			NewVariable.bExposeToCinematics = true;
		}
		if (!(BPVariable.PropertyFlags & CPF_DisableEditOnInstance))
		{
			NewVariable.bPublic = true;
		}
		if(BPVariable.HasMetaData(FBlueprintMetadata::MD_Private))
		{
			NewVariable.bPrivate = BPVariable.GetMetaData(FBlueprintMetadata::MD_Private) == TEXT("true");
		}
		Variables.Add(NewVariable);
	}

	return Variables;
}

#if WITH_EDITOR


// FName URigVMBlueprint::AddAssetVariable(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
// {
// 	FBPVariableDescription NewVar;
// 	const FRigVMExternalVariable Variable = RigVMTypeUtils::ExternalVariableFromCPPTypePath(InName, InCPPType, bIsPublic, bIsReadOnly);
// 	if (!Variable.IsValid(true))
// 	{
// 		return NAME_None;
// 	}
//
// 	return AddAssetVariableFromExternal(Variable, InDefaultValue);
// }
//
// FName URigVMBlueprint::AddAssetVariableFromExternal(const FRigVMExternalVariable& Variable, const FString& InDefaultValue)
// {
// 	FBPVariableDescription NewVar;
// 	TSharedPtr<FKismetNameValidator> NameValidator = MakeShareable(new FKismetNameValidator(this, NAME_None, nullptr));
// 	const FName VarName = FindHostMemberVariableUniqueName(NameValidator, Variable.Name.ToString());
// 	NewVar.VarName = VarName;
// 	NewVar.VarGuid = FGuid::NewGuid();
// 	
// 	NewVar.VarType = RigVMTypeUtils::PinTypeFromCPPType(Variable.TypeName, Variable.TypeObject);
// 	if (!NewVar.VarType.PinCategory.IsValid())
// 	{
// 		return NAME_None;
// 	}
// 	NewVar.FriendlyName = FName::NameToDisplayString(Variable.Name.ToString(), (NewVar.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false);
//
// 	NewVar.PropertyFlags |= (CPF_Edit | CPF_BlueprintVisible | CPF_DisableEditOnInstance);
//
// 	if (Variable.bIsPublic)
// 	{
// 		NewVar.PropertyFlags &= ~CPF_DisableEditOnInstance;
// 	}
//
// 	if (Variable.bIsReadOnly)
// 	{
// 		NewVar.PropertyFlags |= CPF_BlueprintReadOnly;
// 	}
//
// 	NewVar.ReplicationCondition = COND_None;
//
// 	NewVar.Category = UEdGraphSchema_K2::VR_DefaultCategory;
//
// 	// user created variables should be none of these things
// 	NewVar.VarType.bIsConst = false;
// 	NewVar.VarType.bIsWeakPointer = false;
// 	NewVar.VarType.bIsReference = false;
//
// 	// Text variables, etc. should default to multiline
// 	NewVar.SetMetaData(TEXT("MultiLine"), TEXT("true"));
//
// 	NewVar.DefaultValue = InDefaultValue;
//
// 	Modify();
// 	NewVariables.Add(NewVar);
// 	MarkAssetAsModified();
// 	
// #if WITH_RIGVMLEGACYEDITOR
// 	FBPCompileRequest Request(this, EBlueprintCompileOptions::None, nullptr);
// 	FBlueprintCompilationManager::CompileSynchronously(Request);
// #endif
// 	
// 	return NewVar.VarName;
// }
//
// bool URigVMBlueprint::RemoveAssetVariable(const FName& InName)
// {
// 	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
// 	if (VarIndex == INDEX_NONE)
// 	{
// 		return false;
// 	}
// 	
// 	FBlueprintEditorUtils::RemoveMemberVariable(this, InName);
// 	return true;
// }
//
// bool URigVMBlueprint::RenameAssetVariable(const FName& InOldName, const FName& InNewName)
// {
// 	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InOldName);
// 	if (VarIndex == INDEX_NONE)
// 	{
// 		return false;
// 	}
//
// 	VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InNewName);
// 	if (VarIndex != INDEX_NONE)
// 	{
// 		return false;
// 	}
// 	
// 	FBlueprintEditorUtils::RenameMemberVariable(this, InOldName, InNewName);
// 	return true;
// }
//
// bool URigVMBlueprint::ChangeAssetVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
// {
// 	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
// 	if (VarIndex == INDEX_NONE)
// 	{
// 		return false;
// 	}
//
// 	FRigVMExternalVariable Variable;
// 	Variable.Name = InName;
// 	Variable.bIsPublic = bIsPublic;
// 	Variable.bIsReadOnly = bIsReadOnly;
//
// 	FString CPPType = InCPPType;
// 	if (CPPType.StartsWith(TEXT("TMap<")))
// 	{
// 		UE_LOGF(LogRigVMDeveloper, Warning, "TMap Variables are not supported.");
// 		return false;
// 	}
//
// 	Variable.bIsArray = RigVMTypeUtils::IsArrayType(CPPType);
// 	if (Variable.bIsArray)
// 	{
// 		CPPType = RigVMTypeUtils::BaseTypeFromArrayType(CPPType);
// 	}
//
// 	if (CPPType == TEXT("bool"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(bool);
// 	}
// 	else if (CPPType == TEXT("float"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(float);
// 	}
// 	else if (CPPType == TEXT("double"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(double);
// 	}
// 	else if (CPPType == TEXT("int32"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(int32);
// 	}
// 	else if (CPPType == TEXT("FString"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(FString);
// 	}
// 	else if (CPPType == TEXT("FName"))
// 	{
// 		Variable.TypeName = *CPPType;
// 		Variable.Size = sizeof(FName);
// 	}
// 	else if(UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(CPPType))
// 	{
// 		Variable.TypeName = *RigVMTypeUtils::GetUniqueStructTypeName(ScriptStruct);
// 		Variable.TypeObject = ScriptStruct;
// 		Variable.Size = ScriptStruct->GetStructureSize();
// 	}
// 	else if (UEnum* Enum= RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UEnum>(CPPType))
// 	{
// 		Variable.TypeName = *RigVMTypeUtils::CPPTypeFromEnum(Enum);
// 		Variable.TypeObject = Enum;
// 		Variable.Size = static_cast<int32>(Enum->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal));
// 	}
//
// 	FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(Variable);
// 	if (!PinType.PinCategory.IsValid())
// 	{
// 		return false;
// 	}
//
// 	ChangeAssetVariableType(InName, PinType);
//
// 	return true;
// }

FString URigVMBlueprint::GetVariableDefaultValue(const FName& InName, bool bFromDebuggedObject) const
{
	FString DefaultValue;
	const bool bCreateCDO = !IsAsyncLoading();
	UObject* ObjectContainer = bFromDebuggedObject ? GetObjectBeingDebugged() : GeneratedClass->GetDefaultObject(bCreateCDO);
	if(ObjectContainer)
	{
		FProperty* TargetProperty = FindFProperty<FProperty>(GeneratedClass, InName);

		if (TargetProperty)
		{
			const uint8* Container = (const uint8*)ObjectContainer;
			FBlueprintEditorUtils::PropertyValueToString(TargetProperty, Container, DefaultValue, nullptr);
			return DefaultValue;
		}
	}
	return DefaultValue;
}

// bool URigVMBlueprint::ChangeAssetVariableType(const FName& InName, const FEdGraphPinType& InType)
// {
// 	FBlueprintEditorUtils::ChangeMemberVariableType(this, InName, InType);
// 	return true;
// }

TArray<FString> URigVMBlueprint::GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset)
{
	TArray<FString> InternalCommands;
	return InternalCommands;
}

FName URigVMBlueprint::AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	FRigVMExternalVariable Variable = RigVMTypeUtils::ExternalVariableFromCPPTypePath(FGuid::NewGuid(), InName, InCPPType, bIsPublic, bIsReadOnly);
	FName Result = AddHostMemberVariableFromExternal(Variable, InDefaultValue);
	if (!Result.IsNone())
	{
#if WITH_RIGVMLEGACYEDITOR
		FBPCompileRequest Request(this, EBlueprintCompileOptions::None, nullptr);
		FBlueprintCompilationManager::CompileSynchronously(Request);
#else
		// FRigVMBPCompileRequest Request(this, EBlueprintCompileOptions::None, nullptr);
		// FRigVMBlueprintCompilationManager::CompileSynchronously(Request);
#endif
	}
	return Result;
}

bool URigVMBlueprint::RemoveMemberVariable(const FName& InName)
{
	const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
	if (VarIndex == INDEX_NONE)
	{
		return false;
	}
	
	FBlueprintEditorUtils::RemoveMemberVariable(this, InName);
	return true;
}

bool URigVMBlueprint::BulkRemoveMemberVariables(const TArray<FName>& InNames)
{
	for (const FName& Name : InNames)
	{
		const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, Name);
		if (VarIndex == INDEX_NONE)
		{
			return false;
		}
	}

	FBlueprintEditorUtils::BulkRemoveMemberVariables(this, InNames);
	return true;
}

bool URigVMBlueprint::RenameMemberVariable(const FName& InOldName, const FName& InNewName)
{
	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InOldName);
	if (VarIndex == INDEX_NONE)
	{
		return false;
	}

	VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InNewName);
	if (VarIndex != INDEX_NONE)
	{
		return false;
	}
	
	FBlueprintEditorUtils::RenameMemberVariable(this, InOldName, InNewName);
	return true;
}

bool URigVMBlueprint::ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic,
	bool bIsReadOnly, FString InDefaultValue)
{
	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
	if (VarIndex == INDEX_NONE)
	{
		return false;
	}

	FString BaseCPPType = InCPPType;
	if (RigVMTypeUtils::IsArrayType(BaseCPPType))
	{
		BaseCPPType = RigVMTypeUtils::BaseTypeFromArrayType(BaseCPPType);
	}

	UObject* CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(BaseCPPType);
	const FRigVMExternalVariable Variable = FRigVMExternalVariable::Make(FGuid(), InName, InCPPType, CPPTypeObject, bIsPublic, bIsReadOnly);
	FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(Variable);
	if (!PinType.PinCategory.IsValid())
	{
		return false;
	}

	FBlueprintEditorUtils::ChangeMemberVariableType(this, InName, PinType);

	return true;
}

bool URigVMBlueprint::ChangeMemberVariableType(const FName& InName, const FEdGraphPinType& InType)
{
	int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(this, InName);
	if (VarIndex == INDEX_NONE)
	{
		return false;
	}
	
	FBlueprintEditorUtils::ChangeMemberVariableType(this, InName, InType);
	return true;
}

bool URigVMBlueprint::SetVariableIndex(const FName& InName, int32 NewIndex)
{
	// Find the variable index by name
	const FGuid VariableGuid = FBlueprintEditorUtils::FindMemberVariableGuidByName(this, InName);
	if (!VariableGuid.IsValid())
	{
		return false;
	}

	return SetVariableIndex(VariableGuid, NewIndex);
}

bool URigVMBlueprint::SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex)
{
	// Find the variable by GUID
	int32 OldIndex = INDEX_NONE;
	for (int32 Index = 0; Index < NewVariables.Num(); Index++)
	{
		if (NewVariables[Index].VarGuid == InVariableGuid)
		{
			OldIndex = Index;
			break;
		}
	}

	// Variable not found
	if (OldIndex == INDEX_NONE)
	{
		return false;
	}

	// Validate new index
	if (NewIndex < 0 || NewIndex >= NewVariables.Num())
	{
		return false;
	}

	// No change needed
	if (OldIndex == NewIndex)
	{
		return true;
	}

	// Modify for undo/redo
	Modify();

	// Reorder the array
	FBPVariableDescription VariableToMove = NewVariables[OldIndex];
	NewVariables.RemoveAt(OldIndex);
	NewVariables.Insert(VariableToMove, NewIndex);

	// Broadcast property change events
	FPropertyChangedEvent PropertyChangedEvent(FindFProperty<FProperty>(GetClass(), GET_MEMBER_NAME_CHECKED(UBlueprint, NewVariables)), EPropertyChangeType::ArrayMove);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);

	// Mark as modifice
	FBlueprintEditorUtils::MarkBlueprintAsModified(this);

	return true;
}

FName URigVMBlueprint::FindHostMemberVariableUniqueName(TSharedPtr<FKismetNameValidator> InNameValidator, const FString& InBaseName)
{
	FString BaseName = InBaseName;
	if (InNameValidator->IsValid(BaseName) == EValidatorResult::ContainsInvalidCharacters)
	{
		for (TCHAR& TestChar : BaseName)
		{
			for (TCHAR BadChar : UE_BLUEPRINT_INVALID_NAME_CHARACTERS)
			{
				if (TestChar == BadChar)
				{
					TestChar = TEXT('_');
					break;
				}
			}
		}
	}

	FString KismetName = BaseName;

	int32 Suffix = 0;
	while (InNameValidator->IsValid(KismetName) != EValidatorResult::Ok)
	{
		KismetName = FString::Printf(TEXT("%s_%d"), *BaseName, Suffix);
		Suffix++;
	}


	return *KismetName;
}

int32 URigVMBlueprint::AddHostMemberVariable(URigVMBlueprint* InBlueprint, const FGuid InVarGuid, const FName& InVarName, FEdGraphPinType InVarType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	FBPVariableDescription NewVar;

	NewVar.VarName = InVarName;
	NewVar.VarGuid = InVarGuid;

	// [GuidDedupe] If the caller passed a Guid that's already in use by another variable in this
	// blueprint (e.g. a duplicate-paste of a variable description carried the source's Guid forward),
	// regenerate it. Two NewVariables sharing a Guid corrupts tagged-property load via the
	// PropertyGuid-based name redirect — see URigVMRuntimeAsset::AddHostMemberVariableFromExternal
	// for the full rationale. This is the symmetric guard for the legacy blueprint path.
	// InBlueprint is assumed non-null by the existing function contract — the unconditional
	// InBlueprint->NewVariables.Add(NewVar) at the bottom of this function relies on it.
	if (NewVar.VarGuid.IsValid())
	{
		for (const FBPVariableDescription& ExistingVar : InBlueprint->NewVariables)
		{
			if (ExistingVar.VarGuid == NewVar.VarGuid)
			{
				// Reaching this branch IS the bug we want surfaced. Symmetric to the runtime-asset
				// guard in URigVMRuntimeAsset::AddHostMemberVariableFromExternal.
				ensureMsgf(false, TEXT("AddHostMemberVariable: incoming variable '%s' shared Guid %s with existing variable '%s'; regenerating Guid."),
					*InVarName.ToString(), *NewVar.VarGuid.ToString(), *ExistingVar.VarName.ToString());
				UE_LOGF(LogRigVMDeveloper, Warning,
					"AddHostMemberVariable: incoming variable '%ls' shared Guid %ls with existing variable '%ls'; regenerating Guid.",
					*InVarName.ToString(), *NewVar.VarGuid.ToString(), *ExistingVar.VarName.ToString());
				NewVar.VarGuid = FGuid::NewGuid();
				break;
			}
		}
	}
	NewVar.FriendlyName = FName::NameToDisplayString(InVarName.ToString(), (InVarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false);
	NewVar.VarType = InVarType;

	NewVar.PropertyFlags |= (CPF_Edit | CPF_BlueprintVisible | CPF_DisableEditOnInstance);

	if (bIsPublic)
	{
		NewVar.PropertyFlags &= ~CPF_DisableEditOnInstance;
	}

	if (bIsReadOnly)
	{
		NewVar.PropertyFlags |= CPF_BlueprintReadOnly;
	}

	NewVar.ReplicationCondition = COND_None;

	NewVar.Category = UEdGraphSchema_K2::VR_DefaultCategory;

	// user created variables should be none of these things
	NewVar.VarType.bIsConst = false;
	NewVar.VarType.bIsWeakPointer = false;
	NewVar.VarType.bIsReference = false;

	// Text variables, etc. should default to multiline
	NewVar.SetMetaData(TEXT("MultiLine"), TEXT("true"));

	NewVar.DefaultValue = InDefaultValue;

	return InBlueprint->NewVariables.Add(NewVar);
}

FName URigVMBlueprint::AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue)
{
	FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(InVariableToCreate);
	if (!PinType.PinCategory.IsValid())
	{
		return NAME_None;
	}

	Modify();

	TSharedPtr<FKismetNameValidator> NameValidator = MakeShareable(new FKismetNameValidator(this, NAME_None, nullptr));
	FName VarName = FindHostMemberVariableUniqueName(NameValidator, InVariableToCreate.GetName().ToString());
	int32 VariableIndex = AddHostMemberVariable(this, InVariableToCreate.GetGuid(), VarName, PinType, InVariableToCreate.IsPublic(), InVariableToCreate.IsReadOnly(), InDefaultValue);
	if (VariableIndex != INDEX_NONE)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(this);
		return VarName;
	}

	return NAME_None;
}


void URigVMBlueprint::HandlePreVariableChange(UObject* InObject)
{
	OnPreVariablesChangedDelegate.Broadcast(NAME_None);
	if (InObject != this)
	{
		return;
	}
	LastNewVariables = NewVariables;
}

void URigVMBlueprint::HandlePostVariableChange(UBlueprint* InBlueprint)
{
	if (InBlueprint != this)
	{
		return;
	}

	if (bUpdatingExternalVariables)
	{
		return;
	}

	TGuardValue<bool> UpdatingVariablesGuard(bUpdatingExternalVariables, true);
	TArray<FBPVariableDescription> LocalLastNewVariables = LastNewVariables;

	TMap<FGuid, int32> NewVariablesByGuid;
	for (int32 VarIndex = 0; VarIndex < NewVariables.Num(); VarIndex++)
	{
		NewVariablesByGuid.Add(NewVariables[VarIndex].VarGuid, VarIndex);
	}

	TMap<FGuid, int32> OldVariablesByGuid;
	for (int32 VarIndex = 0; VarIndex < LocalLastNewVariables.Num(); VarIndex++)
	{
		OldVariablesByGuid.Add(LocalLastNewVariables[VarIndex].VarGuid, VarIndex);
	}

	for (const FBPVariableDescription& OldVariable : LocalLastNewVariables)
	{
		if (!NewVariablesByGuid.Contains(OldVariable.VarGuid))
		{
			OnVariableRemoved(OldVariable.VarGuid);
			continue;
		}
	}

	for (const FBPVariableDescription& NewVariable : NewVariables)
	{
		if (!OldVariablesByGuid.Contains(NewVariable.VarGuid))
		{
			OnVariableAdded(NewVariable.VarName);
			continue;
		}

		int32 OldVarIndex = OldVariablesByGuid.FindChecked(NewVariable.VarGuid);
		const FBPVariableDescription& OldVariable = LocalLastNewVariables[OldVarIndex];
		if (OldVariable.VarName != NewVariable.VarName)
		{
			OnVariableRenamed(OldVariable.VarGuid, NewVariable.VarName);
		}

		if (OldVariable.VarType != NewVariable.VarType)
		{
			OnVariableTypeChanged(NewVariable.VarGuid, OldVariable.VarType, NewVariable.VarType);
		}
	}

	LastNewVariables = NewVariables;
	
	OnPostVariablesChangedDelegate.Broadcast(NAME_None);
}

void URigVMBlueprint::GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#if WITH_EDITOR
	GetEditorModule()->GetTypeActions(FRigVMEditorAssetInterfacePtr(const_cast<URigVMBlueprint*>(this)), ActionRegistrar);
#endif
}

void URigVMBlueprint::GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#if WITH_EDITOR
	GetEditorModule()->GetInstanceActions(FRigVMEditorAssetInterfacePtr(const_cast<URigVMBlueprint*>(this)), ActionRegistrar);
#endif
}


#endif

#undef LOCTEXT_NAMESPACE


