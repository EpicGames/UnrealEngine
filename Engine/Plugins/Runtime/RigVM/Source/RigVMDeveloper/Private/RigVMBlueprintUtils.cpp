// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMBlueprintUtils.h"

#include "BlueprintActionDatabase.h"
#include "INotifyFieldValueChanged.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMCore/RigVMStruct.h"
#include "UObject/UObjectIterator.h"
#include "RigVMEditorAsset.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Stats/StatsHierarchical.h"

#define LOCTEXT_NAMESPACE "RigVMBlueprintUtils"

FName FRigVMBlueprintUtils::ValidateName(FRigVMEditorAssetInterfacePtr InBlueprint, const FString& InName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FString Name = InName;
	if (Name.StartsWith(TEXT("RigUnit_")))
	{
		Name.RightChopInline(8, EAllowShrinking::No);
	}
	else if (Name.StartsWith(TEXT("RigVMStruct_")))
	{
		Name.RightChopInline(12, EAllowShrinking::No);
	}

	if (UBlueprint* Blueprint = Cast<UBlueprint>(InBlueprint.GetObject()))
	{
		TSharedPtr<FKismetNameValidator> NameValidator;
		NameValidator = MakeShareable(new FKismetNameValidator(Blueprint));

		// Clean up BaseName to not contain any invalid characters, which will mean we can never find a legal name no matter how many numbers we add
		if (NameValidator->IsValid(Name) == EValidatorResult::ContainsInvalidCharacters)
		{
			for (TCHAR& TestChar : Name)
			{
				for (TCHAR BadChar : UE_BLUEPRINT_INVALID_NAME_CHARACTERS)
				{
					if (TestChar == BadChar)
					{
						TestChar = TEXT('_');
						break;
					}
				}
			}
		}

		if (UClass* ParentClass = Blueprint->ParentClass)
		{
			FFieldVariant ExistingField = FindUFieldOrFProperty(ParentClass, *Name);
			if (ExistingField)
			{
				Name = FString::Printf(TEXT("%s_%d"), *Name, 0);
			}
		}

		int32 Count = 0;
		FString BaseName = Name;
		while (NameValidator->IsValid(Name) != EValidatorResult::Ok)
		{
			// Calculate the number of digits in the number, adding 2 (1 extra to correctly count digits, another to account for the '_' that will be added to the name
			int32 CountLength = Count > 0 ? (int32)log((double)Count) + 2 : 2;

			// If the length of the final string will be too long, cut off the end so we can fit the number
			if (CountLength + BaseName.Len() > NameValidator->GetMaximumNameLength())
			{
				BaseName.LeftInline(NameValidator->GetMaximumNameLength() - CountLength);
			}
			Name = FString::Printf(TEXT("%s_%d"), *BaseName, Count);
			Count++;
		}
	}

	return *Name;
}

UEdGraphNode* FRigVMBlueprintUtils::GetNodeByGUID(const FRigVMEditorAssetInterfacePtr InBlueprint, const FGuid& InNodeGuid)
{
	TArray<UEdGraphNode*> Nodes;
	TArray<UEdGraph*> AllGraphs;
	InBlueprint->GetAllEdGraphs(AllGraphs);
	for(int32 i=0; i<AllGraphs.Num(); i++)
	{
		check(AllGraphs[i] != NULL);
		TArray<UEdGraphNode*> GraphNodes;
		AllGraphs[i]->GetNodesOfClass(GraphNodes);
		Nodes.Append(GraphNodes);
	}

	for(UEdGraphNode* Node : Nodes)
	{
		if(Node->NodeGuid == InNodeGuid)
		{
			return Node;
		}
	}
	return nullptr;
}

FRigVMEditorAssetInterfacePtr FRigVMBlueprintUtils::FindAssetForGraph(const UEdGraph* Graph)
{
	for (UObject* TestOuter = Graph ? Graph->GetOuter() : nullptr; TestOuter; TestOuter = TestOuter->GetOuter())
	{
		if (TestOuter->Implements<URigVMEditorAssetInterface>())
		{
			return TestOuter;
		}
	}

	return nullptr;
}

FRigVMEditorAssetInterfacePtr FRigVMBlueprintUtils::FindAssetForNode(const UEdGraphNode* Node)
{
	if (Node)
	{
		if (UEdGraph* Graph = Node->GetGraph())
		{
			return FindAssetForGraph(Node->GetGraph());
		}
	}
	return nullptr;
}

FName FRigVMBlueprintUtils::FindUniqueVariableName(const IRigVMEditorAssetInterface* InBlueprint, const FString& InBaseName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	const TArray<FRigVMGraphVariableDescription> Variables = InBlueprint->GetAssetVariables();
	FString NewName = InBaseName;
	int32 Index = 0;
	while (Variables.ContainsByPredicate([NewName](const FRigVMGraphVariableDescription& Variable)
		{
			return Variable.Name == NewName;
		}))
	{
		Index++;
		NewName = FString::Printf(TEXT("%s_%d"), *InBaseName, Index);
	}

	return *NewName;
}

void FRigVMBlueprintUtils::ForAllRigVMStructs(TFunction<void(UScriptStruct*)> InFunction)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// Run over all unit types
	for(TObjectIterator<UStruct> StructIt; StructIt; ++StructIt)
	{
		if(StructIt->IsChildOf(FRigVMStruct::StaticStruct()) && !StructIt->HasMetaData(FRigVMStruct::AbstractMetaName))
		{
			if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(*StructIt))
			{
				InFunction(ScriptStruct);
			}
		}
	}
}

void FRigVMBlueprintUtils::HandleReconstructAllBlueprintNodes(UBlueprint* InBlueprint)
{
	if (FRigVMEditorAssetInterfacePtr Interface = InBlueprint)
	{
		HandleReconstructAllBlueprintNodes(Interface);
	}
}

void FRigVMBlueprintUtils::HandleReconstructAllBlueprintNodes(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	return HandleRefreshAllNodes(InBlueprint);
}

void FRigVMBlueprintUtils::HandleRefreshAllNodes(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	InBlueprint->RefreshAllNodes();
}

void FRigVMBlueprintUtils::HandleRefreshAllBlueprintNodes(UBlueprint* InBlueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (FRigVMEditorAssetInterfacePtr Blueprint = InBlueprint)
	{
		Blueprint->RefreshAllNodes();
	}
}

void FRigVMBlueprintUtils::HandleAssetDeleted(const FAssetData& InAssetData)
{
	if (InAssetData.GetClass() && InAssetData.GetClass()->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()))
	{
		// Make sure any RigVMBlueprint removes any TypeActions related to this asset (e.g. public functions)
		if (FBlueprintActionDatabase* ActionDatabase = FBlueprintActionDatabase::TryGet())
		{
			ActionDatabase->ClearAssetActions(InAssetData.GetClass());
		}
	}
}

void FRigVMBlueprintUtils::RemoveMemberVariableIfNotUsed(FRigVMEditorAssetInterfacePtr Blueprint, const FName VarName)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (Blueprint->GetObject()->IsA<UBlueprint>())
	{
		FBlueprintEditorUtils::RemoveMemberVariable(Cast<UBlueprint>(Blueprint->GetObject()), VarName);
	}
}

URigVMEdGraph* FRigVMBlueprintUtils::CreateNewGraph(UObject* ParentScope, const FName& GraphName, TSubclassOf<class URigVMEdGraph> GraphClass, TSubclassOf<class URigVMEdGraphSchema> SchemaClass)
{
	// Copy paste from FBlueprintEditorUtils::CreateNewGraph
	
	URigVMEdGraph* NewGraph = nullptr;
	bool bRename = false;

	// Ensure this name isn't already being used for a graph
	if (GraphName != NAME_None)
	{
		if (UObject* ExistingObject = FindObject<UObject>(ParentScope, *(GraphName.ToString())))
		{
			if (ExistingObject->IsA<URigVMEdGraph>())
			{
				// Rename the old graph out of the way - this may confuse the user somewhat - and even
				// break their logic. But name collisions are not avoidable e.g. someone can add
				// a function to an interface that conflicts with something in a class hierarchy
				ExistingObject->Rename(nullptr, ExistingObject->GetOuter(), REN_DoNotDirty);
			}
		}

		// Construct new graph with the supplied name
		NewGraph = NewObject<URigVMEdGraph>(ParentScope, GraphClass, NAME_None, RF_Transactional);
		bRename = true;
	}
	else
	{
		// Construct a new graph with a default name
		NewGraph = NewObject<URigVMEdGraph>(ParentScope, GraphClass, NAME_None, RF_Transactional);
	}

	NewGraph->Schema = SchemaClass;

	// Now move to where we want it to. Workaround to ensure transaction buffer is correctly utilized
	if (bRename)
	{
		NewGraph->Rename(*(GraphName.ToString()), ParentScope, REN_DoNotDirty);
	}
	return NewGraph;
}
#undef LOCTEXT_NAMESPACE