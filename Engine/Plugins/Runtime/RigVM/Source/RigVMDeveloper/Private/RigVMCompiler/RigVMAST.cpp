// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMAST.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/Nodes/RigVMParameterNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMModel/Nodes/RigVMCommentNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "RigVMModel/Nodes/RigVMEnumNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Stats/StatsHierarchical.h"
#include "RigVMDeveloperModule.h"
#include "VisualGraphUtils.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "UObject/FieldIterator.h"
#include "Algo/Sort.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_MakeStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMAST)

FRigVMExprAST::FRigVMExprAST(EType InType, const FRigVMASTProxy& InProxy)
	: Name(NAME_None)
	, Type(InType)
	, Index(INDEX_NONE)
	, ParserPtr(nullptr)
{
}

FName FRigVMExprAST::GetTypeName() const
{
	switch (GetType())
	{
		case EType::Block:
		{
			return TEXT("[.Block.]");
		}
		case EType::Entry:
		{
			return TEXT("[.Entry.]");
		}
		case EType::CallExtern:
		{
			return TEXT("[.Call..]");
		}
		case EType::InlineFunction:
		{
			return TEXT("[.Inline..]");
		}
		case EType::NoOp:
		{
			return TEXT("[.NoOp..]");
		}
		case EType::Var:
		{
			return TEXT("[.Var...]");
		}
		case EType::Literal:
		{
			return TEXT("[Literal]");
		}
		case EType::Assign:
		{
			return TEXT("[.Assign]");
		}
		case EType::Copy:
		{
			return TEXT("[.Copy..]");
		}
		case EType::CachedValue:
		{
			return TEXT("[.Cache.]");
		}
		case EType::Exit:
		{
			return TEXT("[.Exit..]");
		}
		case EType::Invalid:
		{
			return TEXT("[Invalid]");
		}
		default:
		{
			ensure(false);
		}
	}
	return NAME_None;
}

const FRigVMExprAST* FRigVMExprAST::GetParent() const
{
	if (Parents.Num() > 0)
	{
		return ParentAt(0);
	}
	return nullptr;
}

const FRigVMExprAST* FRigVMExprAST::GetFirstParentOfType(EType InExprType) const
{
	if(const FRigVMParserAST* Parser = GetParser())
	{
		if(FirstParentOfTypeCacheVersion.Get(INDEX_NONE) == Parser->CacheVersion)
		{
			if(const FRigVMExprAST* const* ExistingFirstParent = CachedFirstParentOfType.Find(InExprType))
			{
				return *ExistingFirstParent;
			}
		}
		else
		{
			// if the cache version no longer matches,
			// clear the map to make sure we re-retrieve the first parent accordingly.
			CachedFirstParentOfType.Reset();
		}
	}

	const FRigVMExprAST* FirstParent = nullptr;
	
	for(const FRigVMExprAST* Parent : Parents)
	{
		if (Parent->IsA(InExprType))
		{
			FirstParent = Parent;
			break;
		}
	}

	if(FirstParent == nullptr)
	{
		for (const FRigVMExprAST* Parent : Parents)
		{
			if (const FRigVMExprAST* GrandParent = Parent->GetFirstParentOfType(InExprType))
			{
				FirstParent = GrandParent;
				break;
			}
		}
	}

	if(const FRigVMParserAST* Parser = GetParser())
	{
		FirstParentOfTypeCacheVersion = Parser->CacheVersion;
		CachedFirstParentOfType.FindOrAdd(InExprType, nullptr) = FirstParent;
	}

	return FirstParent;
}

bool FRigVMExprAST::IsParentedTo(const FRigVMExprAST* InParentExpr) const
{
	check(InParentExpr);

	for(const FRigVMExprAST* Parent : Parents)
	{
		if(Parent == InParentExpr)
		{
			return true;
		}
		if(Parent->IsParentedTo(InParentExpr))
		{
			return true;
		}
	}
	return false;
}

bool FRigVMExprAST::IsParentOf(const FRigVMExprAST* InChildExpr) const
{
	check(InChildExpr);
	return InChildExpr->IsParentedTo(this);
}

const FRigVMExprAST* FRigVMExprAST::GetFirstChildOfType(EType InExprType) const
{
	if(const FRigVMParserAST* Parser = GetParser())
	{
		if(FirstChildOfTypeCacheVersion.Get(INDEX_NONE) == Parser->CacheVersion)
		{
			if(const FRigVMExprAST* const* ExistingFirstChild = CachedFirstChildOfType.Find(InExprType))
			{
				return *ExistingFirstChild;
			}
		}
		else
		{
			// if the cache version no longer matches,
			// clear the map to make sure we re-retrieve the first child accordingly.
			CachedFirstChildOfType.Reset();
		}
	}

	const FRigVMExprAST* FirstChild = nullptr;
	
	for (const FRigVMExprAST* Child : Children)
	{
		if (Child->IsA(InExprType))
		{
			FirstChild = Child;
			break;
		}
	}

	if(FirstChild == nullptr)
	{
		for (const FRigVMExprAST* Child : Children)
		{
			if (const FRigVMExprAST* GrandChild = Child->GetFirstChildOfType(InExprType))
			{
				FirstChild = GrandChild;
				break;
			}
		}
	}

	if(const FRigVMParserAST* Parser = GetParser())
	{
		FirstChildOfTypeCacheVersion = Parser->CacheVersion;
		CachedFirstChildOfType.FindOrAdd(InExprType, nullptr) = FirstChild;
	}

	return FirstChild;
}

const FRigVMBlockExprAST* FRigVMExprAST::GetBlock() const
{
	if (Parents.Num() == 0)
	{
		if (IsA(EType::Block))
		{
			return To<FRigVMBlockExprAST>();
		}
		return ParserPtr->GetObsoleteBlock();
	}

	const FRigVMExprAST* Parent = GetParent();
	if (Parent->IsA(EType::Block))
	{
		return Parent->To<FRigVMBlockExprAST>();
	}

	return Parent->GetBlock();
}

const FRigVMBlockExprAST* FRigVMExprAST::GetRootBlock() const
{
	const FRigVMBlockExprAST* Block = GetBlock();

	if (IsA(EType::Block))
	{
		if (Block && NumParents() > 0)
		{
			return Block->GetRootBlock();
		}
		return To<FRigVMBlockExprAST>();
	}

	if (Block)
	{
		return Block->GetRootBlock();
	}

	return nullptr;
}

FRigVMExprAST::FRigVMBlockArray FRigVMExprAST::GetBlocks(bool bSortByDepth) const
{
	FRigVMBlockArray Blocks;
	GetBlocksImpl(Blocks);

	if(bSortByDepth)
	{
		auto SortByDepth = [](const FRigVMExprAST* A, const FRigVMExprAST* B) -> bool
		{
			if(A->GetMaximumDepth() > B->GetMaximumDepth())
			{
				return true;
			}
			return A->Index < B->Index;
		};
		Algo::Sort(Blocks, SortByDepth);
	}

	// remove the obsolete block to avoid combining expressions on
	// a branch which are supposed to be obsolete.
	if(const FRigVMParserAST* Parser = GetParser())
	{
		Blocks.RemoveAll([](const FRigVMBlockExprAST* Block) -> bool
		{
			return Block->IsObsolete();
		});
	}

	return Blocks;
}

TOptional<uint32> FRigVMExprAST::GetBlockCombinationHash() const
{
	if(IsA(EType::Literal))
	{
		return TOptional<uint32>();
	}

	// if this expression is a node which is mutable, we are not part of a lazy block
	if(IsNode())
	{
		const FRigVMNodeExprAST* NodeExpr = To<FRigVMNodeExprAST>();
		check(NodeExpr);
		if(const URigVMNode* Node = NodeExpr->GetNode())
		{
			if(Node->IsMutable())
			{
				return TOptional<uint32>();
			}
		}
	}

	// if this expression is a var expression on a mutable node - we are not part of a lazy block
	if(IsVar())
	{
		const FRigVMVarExprAST* VarExpr = To<FRigVMVarExprAST>();
		check(VarExpr);
		
		if(const URigVMPin* Pin = VarExpr->GetPin())
		{
			if(const URigVMNode* Node = Pin->GetNode())
			{
				if(Node->IsMutable())
				{
					return TOptional<uint32>();
				}

				// Entry node output pins may be shared across many blocks,
				// since they represent data which is not considered scoped.
				if (Node->IsA<URigVMFunctionEntryNode>())
				{
					if (Pin->GetDirection() == ERigVMPinDirection::Output)
					{
						return TOptional<uint32>();
					}
				}

				// variable node output pins may be shared across many blocks,
				// since they represent data which is not considered scoped.
				if(Node->IsA<URigVMVariableNode>())
				{
					if(Pin->GetName() == URigVMVariableNode::ValueName)
					{
						if(Pin->GetDirection() == ERigVMPinDirection::Output)
						{
							return TOptional<uint32>();
						}
					}
				}
			}
		}
	}

	if(BlockCombinationHash.IsSet())
	{
		return BlockCombinationHash.GetValue();
	}
	
	// only consider cached / computed values for lazy blocks
	if (IsA(FRigVMExprAST::CachedValue))
	{
		// find the first node expr child
		FRigVMExprAST* const* NodeChildPtr = Children.FindByPredicate([](const FRigVMExprAST* Child)
		{
			return Child->IsNode();
		});

		if(NodeChildPtr)
		{
			const FRigVMExprAST* NodeExpression = *NodeChildPtr;
			
			// collect all of the blocks this expression is in
			const FRigVMExprAST::FRigVMBlockArray Blocks = NodeExpression->GetBlocks(true);
			if(Blocks.Num() > 1)
			{
				// compute a hash for the blocks
				uint32 ComputedHash = 0;
				for(const FRigVMBlockExprAST* Block : Blocks)
				{
					ComputedHash = HashCombine(ComputedHash, GetTypeHash(Block->Index));
				}

				BlockCombinationHash = TOptional<uint32>(ComputedHash);

				if(const FRigVMParserAST* Parser = GetParser())
				{
					// compute a unique name for the block combination
					if(!Parser->BlockCombinationHashToName.Contains(ComputedHash))
					{
						FString BlockCombinationName;
						for(const FRigVMBlockExprAST* Block : Blocks)
						{
							static constexpr TCHAR BlockNameFormat[] = TEXT("%s%d");
							const FString BlockNameAndIndex = FString::Printf(BlockNameFormat, *Block->GetName().ToString(), Block->Index);

							if(BlockCombinationName.IsEmpty())
							{
								BlockCombinationName = BlockNameAndIndex;
							}
							else
							{
								static constexpr TCHAR JoinFormat[] = TEXT("%s_%s");
								BlockCombinationName = FString::Printf(JoinFormat, *BlockCombinationName, *BlockNameAndIndex);
							}
						}
						Parser->BlockCombinationHashToName.Add(ComputedHash, BlockCombinationName);
					}
				}

				return BlockCombinationHash.GetValue();
			}
		}
	}

	// if we still don't have a hash - rely on the parents
	if(!BlockCombinationHash.IsSet())
	{
		for(int32 ParentIndex = 0; ParentIndex < NumParents(); ParentIndex++)
		{
			const TOptional<uint32> ParentHash = ParentAt(ParentIndex)->GetBlockCombinationHash();
			if(ParentHash.IsSet())
			{
				BlockCombinationHash = ParentHash;
				return ParentHash;
			}
		}
	}

	BlockCombinationHash = TOptional<uint32>();
	return BlockCombinationHash.GetValue();
}

const FString& FRigVMExprAST::GetBlockCombinationName() const
{
	const TOptional<uint32> CombinationHash = GetBlockCombinationHash();
	if(CombinationHash.IsSet())
	{
		if(const FRigVMParserAST* Parser = GetParser())
		{
			if(const FString* CombinationName = Parser->BlockCombinationHashToName.Find(CombinationHash.GetValue()))
			{
				return *CombinationName;
			}
		}
	}

	static const FString EmptyString;
	return EmptyString;
}

void FRigVMExprAST::GetBlocksImpl(FRigVMBlockArray& InOutBlocks) const
{
	if (IsA(EType::Block))
	{
		InOutBlocks.AddUnique(this->To<FRigVMBlockExprAST>());
		return;
	}
	
	const FRigVMParserAST* Parser = GetParser();
	check(Parser);
	
	if(BlocksCacheVersion.Get(INDEX_NONE) != Parser->CacheVersion)
	{
		CachedBlocks.Reset();
		for(int32 ParentIndex = 0; ParentIndex < NumParents(); ParentIndex++)
		{
			const FRigVMExprAST* ParentExpression = ParentAt(ParentIndex);
			ParentExpression->GetBlocksImpl(CachedBlocks);
		}
		BlocksCacheVersion = Parser->CacheVersion;
	}

	if(InOutBlocks.IsEmpty())
	{
		InOutBlocks = CachedBlocks;
	}
	else
	{
		for(const FRigVMBlockExprAST* Block : CachedBlocks)
		{
			InOutBlocks.AddUnique(Block);
		}
	}
}


int32 FRigVMExprAST::GetMinChildIndexWithinParent(const FRigVMExprAST* InParentExpr) const
{
	int32 MinIndex = INDEX_NONE;

	const TTuple<const FRigVMExprAST*, const FRigVMExprAST*> MapKey(InParentExpr, this);

	const FRigVMParserAST* Parser = GetParser();
	if(Parser)
	{
		if(const int32* IndexPtr = Parser->MinIndexOfChildWithinParent.Find(MapKey))
		{
			return *IndexPtr;
		}
	}

	for (const FRigVMExprAST* Parent : Parents)
	{
		int32 ChildIndex = INDEX_NONE;

		if (Parent == InParentExpr)
		{
			Parent->Children.Find((FRigVMExprAST*)this, ChildIndex);
		}
		else
		{
			ChildIndex = Parent->GetMinChildIndexWithinParent(InParentExpr);
		}

		if (ChildIndex != INDEX_NONE)
		{
			if (ChildIndex < MinIndex || MinIndex == INDEX_NONE)
			{
				MinIndex = ChildIndex;
			}
		}
	}

	if(Parser)
	{
		Parser->MinIndexOfChildWithinParent.Add(MapKey, MinIndex);
	}
	return MinIndex;
}

void FRigVMExprAST::AddParent(FRigVMExprAST* InParent)
{
	ensure(IsValid());
	ensure(InParent->IsValid());
	ensure(InParent != this);
	ensure(!InParent->IsA(FRigVMExprAST::EType::Literal));
	
	if (Parents.Contains(InParent))
	{
		return;
	}

	InParent->Children.Add(this);
	Parents.Add(InParent);

	InvalidateCaches();
}

void FRigVMExprAST::RemoveParent(FRigVMExprAST* InParent)
{
	ensure(IsValid());
	ensure(InParent->IsValid());
	
	if (Parents.Remove(InParent) > 0)
	{
		const int32 ExistingIndex = InParent->Children.Find(this);
		if(ExistingIndex != INDEX_NONE)
		{
			FName NameToRemove(NAME_None);
			for(TPair<FName, int32>& Pair: InParent->PinNameToChildIndex)
			{
				if(Pair.Value > ExistingIndex)
				{
					Pair.Value--;
				}
				else if(Pair.Value == ExistingIndex)
				{
					NameToRemove = Pair.Key;
				}
			}
			InParent->PinNameToChildIndex.Remove(NameToRemove);
		}
		InParent->Children.Remove(this);
	}
	
	InvalidateCaches();
}

void FRigVMExprAST::RemoveChild(FRigVMExprAST* InChild)
{
	ensure(IsValid());
	ensure(InChild->IsValid());
	
	InChild->RemoveParent(this);
}

void FRigVMExprAST::ReplaceParent(FRigVMExprAST* InCurrentParent, FRigVMExprAST* InNewParent, TArray<int32>* OutParentIndices)
{
	ensure(IsValid());
	ensure(InCurrentParent->IsValid());
	ensure(InNewParent->IsValid());
	
	for (int32 ParentIndex = 0; ParentIndex < Parents.Num(); ParentIndex++)
	{
		if (Parents[ParentIndex] == InCurrentParent)
		{
			Parents[ParentIndex] = InNewParent;
			InCurrentParent->Children.Remove(this);
			InNewParent->Children.Add(this);
			InvalidateCaches();

			if (OutParentIndices)
			{
				OutParentIndices->Add(ParentIndex);
			}
		}
	}
}

void FRigVMExprAST::ReplaceChild(FRigVMExprAST* InCurrentChild, FRigVMExprAST* InNewChild, TArray<int32>* OutChildIndices)
{
	ensure(IsValid());
	ensure(InCurrentChild->IsValid());
	ensure(InNewChild->IsValid());

	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		if (Children[ChildIndex] == InCurrentChild)
		{
			Children[ChildIndex] = InNewChild;
			InCurrentChild->Parents.Remove(this);
			InNewChild->Parents.Add(this);

			TArray<bool> InvalidateCachesProcessed;
			InvalidateCachesProcessed.AddZeroed(GetParser()->Expressions.Num());
			InCurrentChild->InvalidateCachesImpl(InvalidateCachesProcessed);
			InNewChild->InvalidateCachesImpl(InvalidateCachesProcessed);
			InvalidateCachesImpl(InvalidateCachesProcessed);

			if (OutChildIndices)
			{
				OutChildIndices->Add(ChildIndex);
			}
		}
	}
}

void FRigVMExprAST::ReplaceBy(FRigVMExprAST* InReplacement)
{
	TArray<FRigVMExprAST*> PreviousParents;
	PreviousParents.Append(Parents);

	for (FRigVMExprAST* PreviousParent : PreviousParents)
	{
		PreviousParent->ReplaceChild(this, InReplacement);
	}
}

bool FRigVMExprAST::IsConstant() const
{
	for (FRigVMExprAST* ChildExpr : Children)
	{
		if (!ChildExpr->IsConstant())
		{
			return false;
		}
	}
	return true;
}

int32 FRigVMExprAST::GetMaximumDepth() const
{
	if(MaximumDepth.IsSet())
	{
		return MaximumDepth.GetValue();
	}
	int32 Depth = 0;
	for(int32 ParentIndex = 0; ParentIndex < NumParents(); ParentIndex++)
	{
		const FRigVMExprAST* ParentExpression = ParentAt(ParentIndex);
		Depth = FMath::Max<int32>(Depth, ParentExpression->GetMaximumDepth() + 1);
	}
	MaximumDepth = Depth;
	return Depth;
}

FString FRigVMExprAST::DumpText(const FString& InPrefix) const
{
	FString Result;
	if (Name.IsNone())
	{
		Result = FString::Printf(TEXT("%s%s"), *InPrefix, *GetTypeName().ToString());
	}
	else
	{
		Result = FString::Printf(TEXT("%s%s %s"), *InPrefix, *GetTypeName().ToString(), *Name.ToString());
	}

	if (Children.Num() > 0)
	{
		FString Prefix = InPrefix;
		if (Prefix.IsEmpty())
		{
			Prefix = TEXT("-- ");
		}
		else
		{
			Prefix = TEXT("---") + Prefix;
		}
		for (FRigVMExprAST* Child : Children)
		{
			Result += TEXT("\n") + Child->DumpText(Prefix);
		}
	}
	return Result;
}

void FRigVMExprAST::InvalidateCaches()
{
	TArray<bool> Processed;
	Processed.AddZeroed(GetParser()->Expressions.Num());
	InvalidateCachesImpl(Processed);
}

void FRigVMExprAST::InvalidateCachesImpl(TArray<bool>& OutProcessed)
{
	if(OutProcessed[Index])
	{
		return;
	}
	
	BlockCombinationHash.Reset();
	BlocksCacheVersion.Reset();
	FirstChildOfTypeCacheVersion.Reset();
	FirstParentOfTypeCacheVersion.Reset();
	MaximumDepth.Reset();
	OutProcessed[Index] = true;

	if(const FRigVMParserAST* Parser = GetParser())
	{
		Parser->IncrementCacheVersion();
	}

	for(FRigVMExprAST* ChildExpression : Children)
	{
		ChildExpression->InvalidateCachesImpl(OutProcessed);
	}
}

bool FRigVMBlockExprAST::ShouldExecute() const
{
	return ContainsEntry();
}

bool FRigVMBlockExprAST::ContainsEntry() const
{
	if (IsA(FRigVMExprAST::EType::Entry))
	{
		return true;
	}
	for (FRigVMExprAST* Expression : *this)
	{
		if (Expression->IsA(EType::Entry))
		{
			return true;
		}
	}
	return false;
}

bool FRigVMBlockExprAST::Contains(const FRigVMExprAST* InExpression, TMap<const FRigVMExprAST*, bool>* ContainedExpressionsCache) const
{
	if (InExpression == this)
	{
		return true;
	}

	if (ContainedExpressionsCache)
	{
		if (bool* Result = ContainedExpressionsCache->Find(InExpression))
		{
			return *Result;
		}
	}

	for (int32 ParentIndex = 0; ParentIndex < InExpression->NumParents(); ParentIndex++)
	{
		const FRigVMExprAST* ParentExpr = InExpression->ParentAt(ParentIndex);
		if (Contains(ParentExpr, ContainedExpressionsCache))
		{
			if (ContainedExpressionsCache)
			{
				ContainedExpressionsCache->Add(InExpression, true);
			}
			return true;
		}
	}

	if (ContainedExpressionsCache)
	{
		ContainedExpressionsCache->Add(InExpression, false);
	}
	return false;
}

bool FRigVMBlockExprAST::IsObsolete() const
{
	if(bIsObsolete)
	{
		return true;
	}

	struct Local
	{
		static bool HasOnlyObsoleteParents(const FRigVMExprAST* InExpr)
		{
			for(int32 ParentIndex = 0; ParentIndex < InExpr->NumParents(); ParentIndex++)
			{
				const FRigVMExprAST* ParentExpr = InExpr->ParentAt(ParentIndex);
				if(!IsObsoleteExpr(ParentExpr))
				{
					return false;
				}
			}
			return true;
		}
		
		static bool IsObsoleteExpr(const FRigVMExprAST* InExpr)
		{
			if(InExpr->IsA(EType::Block))
			{
				const FRigVMBlockExprAST* Block = InExpr->To<FRigVMBlockExprAST>();
				if(!Block->bIsObsolete) // avoid further recursion here
				{
					return false;
				}
			}

			return HasOnlyObsoleteParents(InExpr);
		}
	};

	if(NumParents() == 0)
	{
		return Local::IsObsoleteExpr(this);
	}
	return Local::HasOnlyObsoleteParents(this);
}

bool FRigVMNodeExprAST::IsConstant() const
{
	if (URigVMNode* CurrentNode = GetNode())
	{
		if (CurrentNode->IsDefinedAsConstant())
		{
			return true;
		}
		else if (CurrentNode->IsDefinedAsVarying())
		{
			return false;
		}

		TArray<URigVMPin*> AllPins = CurrentNode->GetAllPinsRecursively();
		for (URigVMPin* Pin : AllPins)
		{
			// don't flatten pins which have a watch
			if(Pin->RequiresWatch(false))
			{
				return false;
			}
		}
	}
	return FRigVMExprAST::IsConstant();
}

const FRigVMExprAST* FRigVMNodeExprAST::FindExprWithPinName(const FName& InPinName) const
{
	if(const int32* PinIndex = PinNameToChildIndex.Find(InPinName))
	{
		return ChildAt(*PinIndex);
	}
	
	if(URigVMNode* CurrentNode = GetNode())
	{
		if(const URigVMPin* Pin = CurrentNode->FindPin(InPinName.ToString()))
		{
			const int32 PinIndex = Pin->GetPinIndex();
			if(PinIndex <= NumChildren())
			{
				return ChildAt(PinIndex);
			}
		}
	}
	return nullptr;
}

const FRigVMVarExprAST* FRigVMNodeExprAST::FindVarWithPinName(const FName& InPinName) const
{
	const FRigVMExprAST* Child = FindExprWithPinName(InPinName);
	if (Child && Child->IsA(FRigVMExprAST::Var))
	{
		return Child->To<FRigVMVarExprAST>();
	}
	return nullptr;
}

const FName& FRigVMNodeExprAST::GetPinNameForChildIndex(int32 InChildIndex) const
{
	if (const FName* PinName = PinNameToChildIndex.FindKey(InChildIndex))
	{
		return *PinName;
	}
	static const FName EmptyString = NAME_None;
	return EmptyString;
}

void FRigVMNodeExprAST::SetSegmentPathForChild(int32 InChildIndex, const FString& InSegmentPath)
{
	check(Children.IsValidIndex(InChildIndex));
	SegmentPathForChild.FindOrAdd(InChildIndex) = InSegmentPath;
}

const FString& FRigVMNodeExprAST::GetSegmentPathForChild(int32 InChildIndex) const
{
	check(Children.IsValidIndex(InChildIndex));
	if(const FString* ExistingSegmentPath = SegmentPathForChild.Find(InChildIndex))
	{
		return *ExistingSegmentPath;
	}
	static const FString EmptySegmentPath;
	return EmptySegmentPath;
}

FRigVMNodeExprAST::FRigVMNodeExprAST(EType InType, const FRigVMASTProxy& InNodeProxy)
	: FRigVMBlockExprAST(InType)
	, Proxy(InNodeProxy)
{
}

FName FRigVMEntryExprAST::GetEventName() const
{
	if (URigVMNode* EventNode = GetNode())
	{
		return EventNode->GetEventName();
	}
	return NAME_None;
}

bool FRigVMVarExprAST::IsConstant() const
{
	if (GetPin()->IsExecuteContext())
	{
		return false;
	}

	if (GetPin()->IsDefinedAsConstant())
	{
		return true;
	}

	if (SupportsSoftLinks())
	{
		return false;
	}

	ERigVMPinDirection Direction = GetPin()->GetDirection();
	if (Direction == ERigVMPinDirection::Hidden)
	{
		if (Cast<URigVMVariableNode>(GetPin()->GetNode()))
		{
			if (GetPin()->GetName() == URigVMVariableNode::VariableName)
			{
				return true;
			}
		}
		return false;
	}

	if (GetPin()->GetDirection() == ERigVMPinDirection::IO ||
		GetPin()->GetDirection() == ERigVMPinDirection::Output)
	{
		if (GetPin()->GetNode()->IsDefinedAsVarying())
		{
			return false;
		}
	}

	return FRigVMExprAST::IsConstant();
}

FString FRigVMVarExprAST::GetCPPType() const
{
	return GetPin()->GetCPPType();
}

UObject* FRigVMVarExprAST::GetCPPTypeObject() const
{
	return GetPin()->GetCPPTypeObject();
}

ERigVMPinDirection FRigVMVarExprAST::GetPinDirection() const
{
	return GetPin()->GetDirection();
}

FString FRigVMVarExprAST::GetDefaultValue() const
{
	return GetPin()->GetDefaultValue(URigVMPin::FPinOverride(GetProxy(), GetParser()->GetPinOverrides()));
}

bool FRigVMVarExprAST::IsExecuteContext() const
{
	return GetPin()->IsExecuteContext();
}

bool FRigVMVarExprAST::IsGraphVariable() const
{
	if (Cast<URigVMVariableNode>(GetPin()->GetNode()))
	{
		return GetPin()->GetName() == URigVMVariableNode::ValueName;
	}
	return false;
}

bool FRigVMVarExprAST::IsEnumValue() const
{
	if (Cast<URigVMEnumNode>(GetPin()->GetNode()))
	{
		return GetPin()->GetName() == TEXT("EnumIndex");
	}
	return false;
}

bool FRigVMVarExprAST::SupportsSoftLinks() const
{
	if (const URigVMNode* Node = Cast<URigVMNode>(GetPin()->GetNode()))
	{
		if (Node->IsControlFlowNode())
		{
			return !Node->GetControlFlowBlocks().Contains(GetPin()->GetFName());
		}
	}
	return false;
}

void FRigVMParserASTSettings::Report(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage) const
{
	if (ReportDelegate.IsBound())
	{
		ReportDelegate.Execute(InSeverity, InSubject, InMessage);
	}
	else
	{
		if (InSeverity == EMessageSeverity::Error)
		{
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *InMessage, *FString());
		}
		else if (InSeverity == EMessageSeverity::Warning)
		{
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *InMessage, *FString());
		}
		else
		{
			UE_LOGF(LogRigVMDeveloper, Display, "%ls", *InMessage);
		}
	}
}

const TArray<FRigVMASTProxy> FRigVMParserAST::EmptyProxyArray;

FRigVMParserAST::FRigVMParserAST(TArray<URigVMGraph*> InGraphs, URigVMController* InController, const FRigVMParserASTSettings& InSettings, const TArray<FRigVMExternalVariable>& InExternalVariables)
	: CacheVersion(0)
	, LibraryNodeBeingCompiled(nullptr)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InGraphs.Num() == 1 && InGraphs[0]->GetTypedOuter<URigVMFunctionLibrary>() != nullptr)
	{
		LibraryNodeBeingCompiled = Cast<URigVMLibraryNode>(InGraphs[0]->GetOuter());
	}

	Settings = InSettings;
	ObsoleteBlock = nullptr;
	LastCycleCheckExpr = nullptr;
	LinksToSkip = InSettings.LinksToSkip;

	check(!InGraphs.IsEmpty());

	// construct the inlined nodes and links information
	Inline(InGraphs);

	if (LibraryNodeBeingCompiled == nullptr)
	{
		for (const FRigVMASTProxy& NodeProxy : NodeProxies)
		{
			URigVMNode* Node = NodeProxy.GetSubjectChecked<URigVMNode>();
			if(Node->IsEvent())
			{
				TraverseMutableNode(NodeProxy, nullptr);
			}
		}
	}
	else
	{
		URigVMFunctionEntryNode* Entry = LibraryNodeBeingCompiled->GetEntryNode();
		URigVMFunctionReturnNode* Return = LibraryNodeBeingCompiled->GetReturnNode();
		if (Entry && Entry->IsMutable())
		{
			for (const FRigVMASTProxy& NodeProxy : NodeProxies)
			{
				URigVMNode* Node = NodeProxy.GetSubjectChecked<URigVMNode>();
				if(Node == Entry)
				{
					TraverseMutableNode(NodeProxy, nullptr);
					break;
				}
			}
		}
		else if (Return)
		{
			for (const FRigVMASTProxy& NodeProxy : NodeProxies)
			{
				URigVMNode* Node = NodeProxy.GetSubjectChecked<URigVMNode>();
				if(Node == Return)
				{
					TraverseNode(NodeProxy, nullptr);
					break;
				}
			}
		}
		else
		{
			return;
		}
	}

	// traverse all remaining mutable nodes,
	// followed by a pass for all remaining non-mutable nodes
	for (int32 PassIndex = 0; PassIndex < 2; PassIndex++)
	{
		const bool bTraverseMutable = PassIndex == 0;
		for (int32 NodeIndex = 0; NodeIndex < NodeProxies.Num(); NodeIndex++)
		{
			if (const int32* ExprIndex = NodeExpressionIndex.Find(NodeProxies[NodeIndex]))
			{
				if (*ExprIndex != INDEX_NONE)
				{
					continue;
				}
			}

			URigVMNode* Node = NodeProxies[NodeIndex].GetSubjectChecked<URigVMNode>();
			if (Node->IsMutable() == bTraverseMutable)
			{
				if (bTraverseMutable)
				{
					TraverseMutableNode(NodeProxies[NodeIndex], GetObsoleteBlock());
				}
				else
				{
					TraverseNode(NodeProxies[NodeIndex], GetObsoleteBlock());
				}
			}
		}
	}

	FoldEntries();
	InjectExitsToEntries();
	FoldNoOps();

	if (InSettings.bFoldAssignments)
	{
		FoldAssignments();
	}

	if (InSettings.bFoldSubPinCopies)
	{
		FoldSubPinCopies();
	}

	if (InSettings.bFoldLiterals)
	{
		FoldLiterals();
	}
}

FRigVMParserAST::~FRigVMParserAST()
{
	for (FRigVMExprAST* Expression : Expressions)
	{
		delete(Expression);
	}
	Expressions.Empty();

	for (FRigVMExprAST* Expression : DeletedExpressions)
	{
		delete(Expression);
	}
	DeletedExpressions.Empty();

	// root expressions are a subset of the
	// expressions array, so no cleanup necessary
	RootExpressions.Empty();
}

FRigVMExprAST* FRigVMParserAST::TraverseMutableNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr)
{
	if (SubjectToExpression.Contains(InNodeProxy))
	{
		return SubjectToExpression.FindChecked(InNodeProxy);
	}

	URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
	if(Node->HasOrphanedPins())
	{
		return nullptr;
	}

	FRigVMExprAST* NodeExpr = CreateExpressionForNode(InNodeProxy, InParentExpr);
	if (NodeExpr)
	{
		if (InParentExpr == nullptr)
		{
			InParentExpr = NodeExpr;
		}

		TraversePins(InNodeProxy, NodeExpr);
		if (NodeExpr->IsNode())
		{
			TraverseNodeExpressionForRequiredVariableClone(NodeExpr->To<FRigVMNodeExprAST>());
		}

		TArray<URigVMPin*> SourcePins = Node->GetPins().FilterByPredicate([](const URigVMPin* InPin) -> bool
		{
			return (InPin->GetDirection() == ERigVMPinDirection::Output || InPin->GetDirection() == ERigVMPinDirection::IO) && InPin->IsExecuteContext();
		});

		for(int32 SourcePinIndex = 0; SourcePinIndex < SourcePins.Num(); SourcePinIndex++)
		{
			URigVMPin* SourcePin = SourcePins[SourcePinIndex];
			FRigVMASTProxy SourcePinProxy = InNodeProxy.GetSibling(SourcePin);

			FRigVMExprAST* ParentExpr = InParentExpr;

			if(SourcePin->IsFixedSizeArray())
			{
				// also process elements of execute fixed arrays
				SourcePins.Append(SourcePin->GetSubPins());
			}
			else if (Node->IsControlFlowNode())
			{
				if (FRigVMExprAST** PinExpr = SubjectToExpression.Find(SourcePinProxy))
				{
					FRigVMBlockExprAST* BlockExpr = MakeExpr<FRigVMBlockExprAST>(FRigVMExprAST::EType::Block, FRigVMASTProxy());
					BlockExpr->AddParent(*PinExpr);
					BlockExpr->Name = SourcePin->GetFName();
					ParentExpr = BlockExpr;
				}
			}

			const TArray<int32>& LinkIndices = GetTargetLinkIndices(SourcePinProxy);
			for(const int32 LinkIndex : LinkIndices)
			{
				const FRigVMASTLinkDescription& Link = Links[LinkIndex];
				if(ShouldLinkBeSkipped(Link))
				{
					continue;
				}

				URigVMNode* TargetNode = Link.TargetProxy.GetSubjectChecked<URigVMPin>()->GetNode();
				FRigVMASTProxy TargetNodeProxy = Link.TargetProxy.GetSibling(TargetNode);
				TraverseMutableNode(TargetNodeProxy, ParentExpr);
			}
		}
	}

	return NodeExpr;
}

FRigVMExprAST* FRigVMParserAST::TraverseNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr)
{
	URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
	if (Cast<URigVMCommentNode>(Node))
	{
		return nullptr;
	
	}

	if(Node->HasOrphanedPins())
	{
		return nullptr;
	}
	
	if (SubjectToExpression.Contains(InNodeProxy))
	{
		FRigVMExprAST* NodeExpr = SubjectToExpression.FindChecked(InNodeProxy);
		NodeExpr->AddParent(InParentExpr);
		return NodeExpr;
	}

	// if we hit a mutable node here that hasn't been traversed yet,
	// it means that the node is not wired up correctly.
	if(Node->IsMutable())
	{
		struct Local
		{
			static bool IsNodeWiredToEvent(const FRigVMASTProxy& InNodeProxy)
			{
				const URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
				if(Node->IsEvent())
				{
					return true;
				}

				for(const URigVMPin* Pin : Node->GetPins())
				{
					if(Pin->GetDirection() != ERigVMPinDirection::Input &&
						Pin->GetDirection() != ERigVMPinDirection::IO)
							
					{
						continue;
					}
					if(!Pin->IsExecuteContext())
					{
						continue;
					}
					
					const TArray<URigVMPin*> SourcePins = Pin->GetLinkedSourcePins();
					for(const URigVMPin* SourcePin : SourcePins)
					{
						const FRigVMASTProxy SourceNodeProxy = InNodeProxy.GetSibling(SourcePin->GetNode());
						if(IsNodeWiredToEvent(SourceNodeProxy))
						{
							return true;
						}
					}
				}

				// if we hit an entry node - we need to continue the search a level up.
				// events cannot be placed inside of a function / collapse node so
				// there's no need to dive into library nodes.
				if(Node->IsA<URigVMFunctionEntryNode>())
				{
					return IsNodeWiredToEvent(InNodeProxy.GetParent());
				}
				return false;
			}
		};

		if(!Local::IsNodeWiredToEvent(InNodeProxy))
		{
			bool bIsInObsoleteBlock = false;
			if(InParentExpr)
			{
				if(InParentExpr->GetBlock() == GetObsoleteBlock())
				{
					bIsInObsoleteBlock = true;
				}
			}

			if(!bIsInObsoleteBlock)
			{
				Settings.Report(
					EMessageSeverity::Error,
					Node,
					TEXT("Node @@ is not linked to execution."));
			}
		}
	}

	FRigVMExprAST* NodeExpr = CreateExpressionForNode(InNodeProxy, InParentExpr);
	if (NodeExpr)
	{
		TraversePins(InNodeProxy, NodeExpr);
		if (NodeExpr->IsNode())
		{
			TraverseNodeExpressionForRequiredVariableClone(NodeExpr->To<FRigVMNodeExprAST>());
		}
	}

	return NodeExpr;
}

FRigVMExprAST* FRigVMParserAST::CreateExpressionForNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr)
{
	URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();

	bool bIsEntry = Node->IsEvent();
	FName NodeName = Node->GetEventName();
	if (LibraryNodeBeingCompiled)
	{
		if (Node->GetTypedOuter<URigVMLibraryNode>() == LibraryNodeBeingCompiled)
		{
			if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(Node))
			{
				if (EntryNode->IsMutable())
				{
					bIsEntry = true;
					NodeName = TEXT("Function Entry");
				}
			}
			if (URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(Node))
			{
				if (!ReturnNode->IsMutable())
				{
					bIsEntry = true;
					NodeName = TEXT("Function Return");
				}
			}
		}
	}

	FRigVMExprAST* NodeExpr = nullptr;
	if (bIsEntry)
	{
		NodeExpr = MakeExpr<FRigVMEntryExprAST>(InNodeProxy);
		NodeExpr->Name = NodeName;
	}
	else
	{
		if (InNodeProxy.IsA<URigVMFunctionReferenceNode>())
		{
			NodeExpr = MakeExpr<FRigVMInlineFunctionExprAST>(InNodeProxy);
		}
		else if (InNodeProxy.IsA<URigVMRerouteNode>() ||
			InNodeProxy.IsA<URigVMVariableNode>() ||
			InNodeProxy.IsA<URigVMEnumNode>() ||
			InNodeProxy.IsA<URigVMLibraryNode>() ||
			InNodeProxy.IsA<URigVMFunctionInterfaceNode>() ||
			IsNoOpUnitNode(Node) ||
			IsConstantDispatchNode(Node) ||
			IsMakeStructDispatchNode(Node) ||
			IsBreakStructDispatchNode(Node))
		{
			NodeExpr = MakeExpr<FRigVMNoOpExprAST>(InNodeProxy);
		}
		else if (InNodeProxy.IsA<URigVMInvokeEntryNode>())
		{
			NodeExpr = MakeExpr<FRigVMInvokeEntryExprAST>(InNodeProxy);
		}
		else
		{
			NodeExpr = MakeExpr<FRigVMCallExternExprAST>(InNodeProxy);
		}
		NodeExpr->Name = Node->GetFName();
	}

	if (InParentExpr != nullptr)
	{
		NodeExpr->AddParent(InParentExpr);
	}
	else
	{
		RootExpressions.Add(NodeExpr);
	}
	SubjectToExpression.Add(InNodeProxy, NodeExpr);
	NodeExpressionIndex.Add(InNodeProxy, NodeExpr->GetIndex());

	return NodeExpr;
}

TArray<FRigVMExprAST*> FRigVMParserAST::TraversePins(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr)
{
	URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
	TArray<FRigVMExprAST*> PinExpressions;

	TArray<URigVMPin*> Pins;

	// traverse the pins on a unit node in the order of the property definitions
	if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
	{
		if(UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
		{
			TArray<UStruct*> Structs = FRigVMTemplate::GetSuperStructs(ScriptStruct, true);
			for(const UStruct* Struct : Structs)
			{
				for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIterationFlags::None); PropertyIt; ++PropertyIt)
				{
					if(URigVMPin* Pin = UnitNode->FindPin(PropertyIt->GetName()))
					{
						Pins.Add(Pin);
					}
				}
			}
		}

		// We might have extra non-native pins, add them afterwards
		if (UnitNode->HasNonNativePins())
		{
			for (URigVMPin* Pin : Node->GetPins())
			{
				// We skip trait pins as we don't want to traverse them
				if (!Pin->IsTraitPin()  || Settings.bSetupTraits)
				{
					Pins.AddUnique(Pin);
				}
			}
		}
	}

	if (Pins.IsEmpty())
	{
		// Just grab whatever pins we have
		Pins = Node->GetPins();
	}

	// dispatch nodes may contain a fixed size array
	if(Node->IsA<URigVMDispatchNode>() || Node->IsA<UDEPRECATED_RigVMSelectNode>())
	{
		TArray<URigVMPin*> FlattenedPins;
		FlattenedPins.Reserve(Pins.Num());
		for(URigVMPin* Pin : Pins)
		{
			if(Pin->IsFixedSizeArray())
			{
				FlattenedPins.Append(Pin->GetSubPins());
			}
			else
			{
				FlattenedPins.Add(Pin);
			}
		}
		Swap(Pins, FlattenedPins);
	}

	for (URigVMPin* Pin : Pins)
	{
		FRigVMASTProxy PinProxy = InNodeProxy.GetSibling(Pin);
		PinExpressions.Add(TraversePin(PinProxy, InParentExpr));

		if (InParentExpr && !Pin->IsRootPin())
		{
			const FString PinName = FRigVMBranchInfo::GetFixedArrayLabel(Pin->GetParentPin()->GetName(), Pin->GetName());
			const int32 ChildIndex = InParentExpr->Children.Find(PinExpressions.Last());
			InParentExpr->PinNameToChildIndex.FindOrAdd(*PinName) = ChildIndex;
		}

		if(InParentExpr)
		{
			const int32 ChildIndex = InParentExpr->Children.Find(PinExpressions.Last());
			InParentExpr->PinNameToChildIndex.FindOrAdd(Pin->GetFName()) = ChildIndex;
		}
	}

	return PinExpressions;
}

FRigVMExprAST* FRigVMParserAST::TraversePin(const FRigVMASTProxy& InPinProxy, FRigVMExprAST* InParentExpr)
{
	ensure(!SubjectToExpression.Contains(InPinProxy));

	URigVMPin* Pin = InPinProxy.GetSubjectChecked<URigVMPin>();

	URigVMPin::FPinOverride PinOverride(InPinProxy, PinOverrides);

	TArray<int32> LinkIndices = GetSourceLinkIndices(InPinProxy, true);
	
	if (LinksToSkip.Num() > 0)
	{
		LinkIndices.RemoveAll([this](int32 LinkIndex)
			{
				return this->ShouldLinkBeSkipped(Links[LinkIndex]);
			}
		);
	}

	bool bForceUseOfLiteralExpression = false;
	if(IsConstantDispatchValuePin(Pin) || IsMakeStructDispatchElementsPin(Pin) || IsBreakStructDispatchStructPin(Pin))
	{
		// if this is a constant with no input links at all - just use a literal for it
		const TArray<URigVMPin*> AllLinkedPins = Pin->GetRootPin()->GetLinkedSourcePins(true);
		if(AllLinkedPins.IsEmpty())
		{
			bForceUseOfLiteralExpression = true;
		}
	}

	FRigVMExprAST* PinExpr = nullptr;

	if (Cast<URigVMVariableNode>(Pin->GetNode()))
	{
		if (Pin->GetName() == URigVMVariableNode::VariableName)
		{
			return nullptr;
		}
	}
	else if (Cast<URigVMEnumNode>(Pin->GetNode()))
	{
		if (Pin->GetDirection() == ERigVMPinDirection::Visible)
		{
			return nullptr;
		}
	}

	if (Pin->IsDefinedAsInputVariable())
	{
		PinExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, InPinProxy);
	}
	else if(Pin->IsTraitPin())
	{
		PinExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, InPinProxy);

		if(Settings.bSetupTraits)
		{
			// Traits can generate their own programmatic pins via FRigVMTrait::GetProgrammaticPins. We account for these as additional expressions if the
			// pin is not part of the set of sub-pins exposed on the struct
			for(URigVMPin* SubPin : Pin->SubPins)
			{
				if(SubPin->IsProgrammaticPin())
				{
					// Not a pin from the struct - add a synthetic var for this parent too
					FRigVMASTProxy SubPinProxy = InPinProxy.GetSibling(SubPin);
					FRigVMExprAST* SubPinExpr = TraversePin(SubPinProxy, InParentExpr);
					const int32 ChildIndex = InParentExpr->Children.Find(SubPinExpr);
					InParentExpr->PinNameToChildIndex.FindOrAdd(SubPinExpr->GetName()) = ChildIndex;
				}
			}
		}
	}
	else if (bForceUseOfLiteralExpression)
	{
		PinExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
	}
	else if ((Pin->GetDirection() == ERigVMPinDirection::Input ||
		Pin->GetDirection() == ERigVMPinDirection::Visible) &&
		LinkIndices.Num() == 0)
	{
		if (Cast<URigVMVariableNode>(Pin->GetNode()) ||
			(LibraryNodeBeingCompiled && Cast<URigVMFunctionReturnNode>(Pin->GetNode())))
		{
			PinExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, InPinProxy);
			FRigVMExprAST* PinLiteralExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
			PinLiteralExpr->Name = PinExpr->Name;
			const FRigVMASTLinkDescription LiteralLink(InPinProxy, InPinProxy, FString());
			FRigVMExprAST* PinCopyExpr = MakeExpr<FRigVMCopyExprAST>(LiteralLink);
			PinCopyExpr->AddParent(PinExpr);
			PinLiteralExpr->AddParent(PinCopyExpr);
		}
		else
		{
			PinExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
		}
	}
	else if (Cast<URigVMEnumNode>(Pin->GetNode()))
	{
		PinExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
	}
	else if (IsNoOpUnitNode(Pin->GetNode()))
	{
		PinExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
	}
	else
	{
		PinExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, InPinProxy);
	}

	PinExpr->AddParent(InParentExpr);
	PinExpr->Name = *Pin->GetPinPath();
	SubjectToExpression.Add(InPinProxy, PinExpr);

	if (Pin->IsExecuteContext())
	{
		return PinExpr;
	}

	if ((Pin->GetDirection() == ERigVMPinDirection::IO ||
		Pin->GetDirection() == ERigVMPinDirection::Input)
		&& !Pin->IsExecuteContext()
		&& !PinExpr->IsA(FRigVMExprAST::EType::Literal))
	{
		bool bHasSourceLinkToRoot = false;
		URigVMPin* RootPin = Pin->GetRootPin();
		for (const int32 LinkIndex : LinkIndices)
		{
			if (Links[LinkIndex].TargetProxy.GetSubject() == RootPin)
			{
				bHasSourceLinkToRoot = true;
				break;
			}
		}

		if (!bHasSourceLinkToRoot && 
			GetSourceLinkIndices(InPinProxy, false).Num() == 0 &&
			(Pin->GetDirection() == ERigVMPinDirection::IO ||
			(LinkIndices.Num() > 0 && Pin->IsArray())))
		{
			FRigVMLiteralExprAST* LiteralExpr = MakeExpr<FRigVMLiteralExprAST>(InPinProxy);
			const FRigVMASTLinkDescription LiteralLink(InPinProxy, InPinProxy, FString());
			FRigVMCopyExprAST* LiteralCopyExpr = MakeExpr<FRigVMCopyExprAST>(LiteralLink);
			LiteralCopyExpr->Name = *GetLinkAsString(LiteralCopyExpr->GetLink());
			LiteralCopyExpr->AddParent(PinExpr);
			LiteralExpr->AddParent(LiteralCopyExpr);
			LiteralExpr->Name = *Pin->GetPinPath();

			SubjectToExpression[InPinProxy] = LiteralExpr;
		}
	}

	FRigVMExprAST* ParentExprForLinks = PinExpr;

	if(LinkIndices.Num() > 0)
	{
		if (Pin->IsLazy())
		{
			// create a block for each lazily executing pin
			FRigVMBlockExprAST* BlockExpr = MakeExpr<FRigVMBlockExprAST>(FRigVMExprAST::EType::Block, FRigVMASTProxy());
			BlockExpr->Name = Pin->GetFName();
			BlockExpr->AddParent(PinExpr);
			ParentExprForLinks = BlockExpr;
		}
		else if (Pin->GetNode()->HasLazyPin(true))
		{
			// for greedy pins on nodes containing a lazy pin - we need to also create a block for the node itself
			FRigVMBlockExprAST* BlockExpr = nullptr;

			if(const FRigVMExprAST* NodeExpr = PinExpr->GetFirstParentOfType(FRigVMExprAST::CallExtern))
			{
				const FRigVMCallExternExprAST* CallExternExpr = NodeExpr->To<FRigVMCallExternExprAST>();
				
				// try to find the block under all non-lazy pins.
				// this is the block that is run before anything else - so for example
				// if you have a lazy interplate with values A and B being lazy and a greedy T blend pin,
				// you need a block to store the instructions related to computing T. once T is computed,
				// the callextern can run and lazily pull on A or B.
				for(const URigVMPin* NodePin : Pin->GetNode()->GetPins())
				{
					if(NodePin == Pin)
					{
						continue;
					}
					
					if(!NodePin->IsLazy() &&
						(NodePin->GetDirection() == ERigVMPinDirection::Input ||
						NodePin->GetDirection() == ERigVMPinDirection::IO))
					{
						if(const FRigVMExprAST* NodePinExpr = CallExternExpr->FindExprWithPinName(NodePin->GetFName()))
						{
							if(const FRigVMExprAST* ExistingBlockExpr = NodePinExpr->GetFirstChildOfType(FRigVMExprAST::Block))
							{
								BlockExpr = (FRigVMBlockExprAST*)ExistingBlockExpr->To<FRigVMBlockExprAST>();
								break;
							}
						}
					}
				}
			}

			if(BlockExpr == nullptr)
			{
				BlockExpr = MakeExpr<FRigVMBlockExprAST>(FRigVMExprAST::EType::Block, FRigVMASTProxy());
				BlockExpr->Name = Pin->GetFName();
			}

			if(BlockExpr)
			{
				BlockExpr->AddParent(PinExpr);
				ParentExprForLinks = BlockExpr;
			}
		}
	}

	for (const int32 LinkIndex : LinkIndices)
	{
		TraverseLink(LinkIndex, ParentExprForLinks);
	}

	return PinExpr;
}

FRigVMExprAST* FRigVMParserAST::TraverseLink(int32 InLinkIndex, FRigVMExprAST* InParentExpr)
{
	const FRigVMASTLinkDescription& Link = Links[InLinkIndex];
	const FRigVMASTProxy& SourceProxy = Link.SourceProxy;
	const FRigVMASTProxy& TargetProxy = Link.TargetProxy;
	URigVMPin* SourcePin = SourceProxy.GetSubjectChecked<URigVMPin>();
	URigVMPin* TargetPin = TargetProxy.GetSubjectChecked<URigVMPin>();
	URigVMPin* SourceRootPin = SourcePin->GetRootPin();
	URigVMPin* TargetRootPin = TargetPin->GetRootPin();
	FRigVMASTProxy SourceNodeProxy = SourceProxy.GetSibling(SourcePin->GetNode());

	bool bRequiresCopy = SourceRootPin != SourcePin || TargetRootPin != TargetPin || !Link.SegmentPath.IsEmpty();
	if (!bRequiresCopy)
	{
		if(Cast<URigVMVariableNode>(TargetRootPin->GetNode()))
		{
			bRequiresCopy = true;
		}
	}

	if (!bRequiresCopy)
	{
		// Connections between entry and return in a function requires a copy
		if (LibraryNodeBeingCompiled)
		{
			if (SourceRootPin->GetNode()->IsA<URigVMFunctionEntryNode>() && TargetRootPin->GetNode()->IsA<URigVMFunctionReturnNode>())
			{
				bRequiresCopy = true;
			}
		}
	}

	if (!bRequiresCopy)
	{
		if (SourcePin->GetTypeIndex() != TargetPin->GetTypeIndex())
		{
			bRequiresCopy = true;
		}
	}

	if (!bRequiresCopy)
	{
		// Due to the unpredictability of lazy branches, we need to make sure that non-lazy pins are not
		// affected by the execution of lazy evaluation.
		if (!TargetRootPin->IsLazy() && TargetRootPin->GetNode()->HasLazyPin(true))
		{
			bRequiresCopy = true;
		}
	}

	if (!bRequiresCopy)
	{
		// Programmatic pins always require a copy
		if(TargetPin->IsProgrammaticPin())
		{
			bRequiresCopy = true;
		}
	}

	FRigVMAssignExprAST* AssignExpr = nullptr;
	if (bRequiresCopy)
	{
		AssignExpr = MakeExpr<FRigVMCopyExprAST>(Link);
	}
	else
	{
		AssignExpr = MakeExpr<FRigVMAssignExprAST>(FRigVMExprAST::EType::Assign, Link);
	}

	AssignExpr->Name = *GetLinkAsString(Link);
	AssignExpr->AddParent(InParentExpr);

	FRigVMExprAST* NodeExpr = TraverseNode(SourceNodeProxy, AssignExpr);
	if (NodeExpr)
	{
		// if this is a copy expression - we should require the copy to use a ref instead
		if (NodeExpr->IsA(FRigVMExprAST::EType::CallExtern) ||
			NodeExpr->IsA(FRigVMExprAST::EType::InlineFunction))
		{
			for (FRigVMExprAST* ChildExpr : *NodeExpr)
			{
				if (ChildExpr->IsA(FRigVMExprAST::EType::Var))
				{
					FRigVMVarExprAST* VarExpr = ChildExpr->To<FRigVMVarExprAST>();
					if (VarExpr->GetPin() == SourceRootPin)
					{
						if (VarExpr->SupportsSoftLinks())
						{
							AssignExpr->ReplaceChild(NodeExpr, VarExpr);
							return AssignExpr;
						}

						FRigVMCachedValueExprAST* CacheExpr = nullptr;
						for (FRigVMExprAST* VarExprParent : VarExpr->Parents)
						{
							if (VarExprParent->IsA(FRigVMExprAST::EType::CachedValue))
							{
								CacheExpr = VarExprParent->To<FRigVMCachedValueExprAST>();
								break;
							}
						}

						if (CacheExpr == nullptr)
						{
							CacheExpr = MakeExpr<FRigVMCachedValueExprAST>(FRigVMASTProxy());
							CacheExpr->Name = AssignExpr->GetName();
							VarExpr->AddParent(CacheExpr);
							NodeExpr->AddParent(CacheExpr);
						}

						AssignExpr->ReplaceChild(NodeExpr, CacheExpr);
						return AssignExpr;
					}
				}
			}
			checkNoEntry();
		}
		else if (LibraryNodeBeingCompiled &&
			SourceRootPin->GetNode()->IsA<URigVMFunctionEntryNode>() &&
			SourceRootPin->GetTypedOuter<URigVMLibraryNode>() == LibraryNodeBeingCompiled)
		{
			FRigVMASTProxy SourceRootProxy = SourceProxy;
			if (SourceRootPin != SourcePin)
			{
				SourceRootProxy = FRigVMASTProxy::MakeFromUObject(SourceRootPin);
			}
			
			FRigVMVarExprAST* SourcePinExpr = nullptr;
			if (NodeExpr->IsNode())
			{
				SourcePinExpr = const_cast<FRigVMVarExprAST*>(NodeExpr->To<FRigVMNodeExprAST>()->FindVarWithPinName(SourceRootPin->GetFName()));

				if (SourcePinExpr)
				{
					if (const FRigVMVarExprAST* ClonedSourcePinExpr = SourcePinExpr->GetVariableCloneExpr())
					{
						SourcePinExpr = const_cast<FRigVMVarExprAST*>(ClonedSourcePinExpr);
					}
				}
			}

			if (!SourcePinExpr)
			{
				SourcePinExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, SourceRootProxy);
			}

			AssignExpr->ReplaceChild(NodeExpr, SourcePinExpr);
			return AssignExpr;
		}			
	}

	return AssignExpr;
}

bool FRigVMParserAST::TraverseNodeExpressionForRequiredVariableClone(FRigVMNodeExprAST* InNodeExpr)
{
	const URigVMNode* Node = InNodeExpr->GetNode();
	if (!Node)
	{
		return false;
	}
	
	const URigVMVariableNode* SourceVariableNode = Cast<URigVMVariableNode>(Node);
	const URigVMFunctionEntryNode* SourceEntryNode = Cast<URigVMFunctionEntryNode>(Node);
	if (SourceVariableNode || SourceEntryNode)
	{
		TArray<TTuple<URigVMPin*, FName>> PinsToCheck;
		if (SourceVariableNode && SourceVariableNode->IsGetter())
		{
			PinsToCheck.Emplace(SourceVariableNode->GetValuePin(), SourceVariableNode->GetVariableName());
		}
		else if (SourceEntryNode)
		{
			for (URigVMPin* Pin : SourceEntryNode->GetPins())
			{
				if (Pin->GetDirection() != ERigVMPinDirection::Output)
				{
					continue;
				}
				if (!Pin->IsDefinedAsInputVariable())
				{
					continue;
				}
				PinsToCheck.Emplace(Pin, Pin->GetFName());
			}
		}

		for (int32 PinToCheckIndex = 0; PinToCheckIndex < PinsToCheck.Num(); PinToCheckIndex++)
		{
			URigVMPin* PinToCheck = PinsToCheck[PinToCheckIndex].Get<0>();
			const FName& VariableName = PinsToCheck[PinToCheckIndex].Get<1>();

			bool bRequiresCloningVariableValue = false;
			FRigVMASTProxy PinProxyToCheck = InNodeExpr->GetProxy().GetSibling(PinToCheck);

			// for variable / entry nodes we need to deal with a specific case where a sub pin
			// of the pin is wired to a variable setter node with the same variable
			// on a different sub pin. so for example Get MyVector.X -> Set MyVector.Y.
			// In that case we need to inject a clone of the variable in the getter.
			// Since arrays don't have sub pins on the getter this can only happen to non-arrays.
			const TArray<int32>& LinkIndices = GetTargetLinkIndices(PinProxyToCheck, true);
			for(const int32 LinkIndex : LinkIndices)
			{
				const FRigVMASTLinkDescription& Link = Links[LinkIndex];
				const URigVMPin* SourcePin = Link.SourceProxy.GetSubjectChecked<URigVMPin>();
				const URigVMPin* TargetPin = Link.TargetProxy.GetSubjectChecked<URigVMPin>();
				if (TargetPin && !Link.SegmentPath.IsEmpty())
				{
					TargetPin = TargetPin->FindSubPin(Link.SegmentPath);
				}
				if (!SourcePin || !TargetPin)
				{
					continue;
				}
				if (TargetPin->IsRootPin())
				{
					continue;
				}
				const URigVMVariableNode* TargetVariableNode = Cast<URigVMVariableNode>(TargetPin->GetNode());
				if (!TargetVariableNode)
				{
					continue;
				}
				if (VariableName != TargetVariableNode->GetVariableName())
				{
					continue;
				}

				if (SourcePin->GetSegmentPath() != TargetPin->GetSegmentPath())
				{
					bRequiresCloningVariableValue = true;
					break;
				}
			}

			if (bRequiresCloningVariableValue)
			{
				if (const FRigVMVarExprAST* ConstVariableVarExpr = InNodeExpr->FindVarWithPinName(PinToCheck->GetFName()))
				{
					FRigVMVarExprAST* VariableVarExpr = const_cast<FRigVMVarExprAST*>(ConstVariableVarExpr);
			
					FRigVMASTProxy PinProxy = VariableVarExpr->GetProxy();
					FRigVMVarExprAST* ClonedVariableVarExpr = MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, PinProxy);
					ClonedVariableVarExpr->Name = *FString::Printf(TEXT("%s::Clone"), *VariableVarExpr->Name.ToString());
					const FRigVMASTLinkDescription CloneLink(PinProxy, PinProxy, FString());
					FRigVMExprAST* PinCopyExpr = MakeExpr<FRigVMCopyExprAST>(CloneLink);
					PinCopyExpr->Name = *FString::Printf(TEXT("Cloning Variable %s"), *VariableVarExpr->Name.ToString());
					PinCopyExpr->AddParent(ClonedVariableVarExpr);
					VariableVarExpr->AddParent(PinCopyExpr);
					VariableVarExpr->ClonedVarExpr = ClonedVariableVarExpr;
					InNodeExpr->ReplaceChild(VariableVarExpr, ClonedVariableVarExpr);
				}
			}
		}
	}

	return true;
}

void FRigVMParserAST::FoldEntries()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TArray<FRigVMExprAST*> FoldRootExpressions;
	TArray<FRigVMExprAST*> ExpressionsToRemove;
	TMap<FName, FRigVMEntryExprAST*> EntryByName;

	for (FRigVMExprAST* RootExpr : RootExpressions)
	{
		if (RootExpr->IsA(FRigVMExprAST::EType::Entry))
		{
			FRigVMEntryExprAST* Entry = RootExpr->To<FRigVMEntryExprAST>();
			if (EntryByName.Contains(Entry->GetEventName()))
			{
				FRigVMEntryExprAST* FoldEntry = EntryByName.FindChecked(Entry->GetEventName());

				// replace the original entry with a noop
				FRigVMNoOpExprAST* NoOpExpr = MakeExpr<FRigVMNoOpExprAST>(Entry->GetProxy());
				NoOpExpr->AddParent(FoldEntry);
				NoOpExpr->Name = Entry->Name;
				SubjectToExpression.FindChecked(Entry->GetProxy()) = NoOpExpr;

				TArray<FRigVMExprAST*> Children = Entry->Children; // copy since the loop changes the array
				for (FRigVMExprAST* ChildExpr : Children)
				{
					ChildExpr->RemoveParent(Entry);
					if (ChildExpr->IsA(FRigVMExprAST::Var))
					{
						if (ChildExpr->To<FRigVMVarExprAST>()->IsExecuteContext())
						{
							ExpressionsToRemove.AddUnique(ChildExpr);
							continue;
						}
					}
					ChildExpr->AddParent(FoldEntry);
				}
				ExpressionsToRemove.AddUnique(Entry);
			}
			else
			{
				FoldRootExpressions.Add(Entry);
				EntryByName.Add(Entry->GetEventName(), Entry);
			}
		}
		else
		{
			FoldRootExpressions.Add(RootExpr);
		}
	}

	RootExpressions = FoldRootExpressions;

	RemoveExpressions(ExpressionsToRemove);
}

void FRigVMParserAST::InjectExitsToEntries()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (LibraryNodeBeingCompiled)
	{
		return;
	}
	
	for (FRigVMExprAST* RootExpr : RootExpressions)
	{
		if (RootExpr->IsA(FRigVMExprAST::EType::Entry))
		{
			bool bHasExit = false;
			if (RootExpr->Children.Num() > 0)
			{
				if (RootExpr->Children.Last()->IsA(FRigVMExprAST::EType::Exit))
				{
					bHasExit = true;
					break;
				}
			}

			if (!bHasExit)
			{
				FRigVMExprAST* ExitExpr = MakeExpr<FRigVMExitExprAST>(FRigVMASTProxy());
				ExitExpr->AddParent(RootExpr);
			}
		}
	}
}

void FRigVMParserAST::RefreshExprIndices()
{
	for (int32 Index = 0; Index < Expressions.Num(); Index++)
	{
		Expressions[Index]->Index = Index;
	}
}

void FRigVMParserAST::FoldNoOps()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	for (FRigVMExprAST* Expression : Expressions)
	{
		if (Expression->IsA(FRigVMExprAST::EType::NoOp))
		{
			if (URigVMNode* Node = Expression->To<FRigVMNoOpExprAST>()->GetNode())
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
				{
					if (!VariableNode->IsGetter())
					{
						continue;
					}
				}
				if (LibraryNodeBeingCompiled != nullptr)
				{
					if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(Node))
					{
						if (EntryNode->GetTypedOuter<URigVMLibraryNode>() == LibraryNodeBeingCompiled &&
							EntryNode->IsMutable())
						{
							continue;
						}
					}
					if (URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(Node))
					{
						if (ReturnNode->GetTypedOuter<URigVMLibraryNode>() == LibraryNodeBeingCompiled &&
							!ReturnNode->IsMutable())
						{
							continue;
						}
					}
				}

				// for make struct and break struct nodes we'll always refer to the input
				// pin's expression since the nodes are considered no-op.
				if(IsMakeStructDispatchNode(Node) || IsBreakStructDispatchNode(Node))
				{
					check(Expression->Children.Num() == 2);

					FRigVMExprAST* ChildToKeep = Expression->Children[0];
					FRigVMExprAST* ChildToRemove = Expression->Children[1];

					TArray<FRigVMExprAST*> Parents = ChildToRemove->Parents;
					for(FRigVMExprAST* Parent : Parents)
					{
						if(Parent != Expression)
						{
							Parent->ReplaceChild(ChildToRemove, ChildToKeep);
						}
					}
				}
			}

			// copy since we are changing the content during iteration below
			TArray<FRigVMExprAST*> Children = Expression->Children;
			TArray<FRigVMExprAST*> Parents = Expression->Parents;

			for (FRigVMExprAST* Parent : Parents)
			{
				Expression->RemoveParent(Parent);
			}

			for (FRigVMExprAST* Child : Children)
			{
				Child->RemoveParent(Expression);
				for (FRigVMExprAST* Parent : Parents)
				{
					Child->AddParent(Parent);
				}
			}
		}
	}
}

void FRigVMParserAST::FoldAssignments()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TArray<FRigVMExprAST*> ExpressionsToRemove;

	// first - Fold all assignment chains
	for (FRigVMExprAST* Expression : Expressions)
	{
		if (Expression->Parents.Num() == 0)
		{
			continue;
		}

		if (Expression->GetType() != FRigVMExprAST::EType::Assign)
		{
			continue;
		}

		FRigVMAssignExprAST* AssignExpr = Expression->To<FRigVMAssignExprAST>();
		if (AssignExpr->Parents.Num() != 1 || AssignExpr->Children.Num() != 1)
		{
			continue;
		}

		URigVMPin* SourcePin = AssignExpr->GetSourcePin();
		URigVMPin* TargetPin = AssignExpr->GetTargetPin();

		// it's possible that we'll see copies from an element onto an array
		// here. we'll need to ignore these links and leave them since they
		// represent copies.
		if(SourcePin->IsArray() != TargetPin->IsArray())
		{
			continue;
		}

		// in case the assign has different types for left and right - we need to avoid folding
		// since this assign represents a cast operation
		if(SourcePin->GetCPPTypeObject() != TargetPin->GetCPPTypeObject())
		{
			continue;
		}
		else if(SourcePin->GetCPPTypeObject() == nullptr)
		{
			if(SourcePin->GetCPPType() != TargetPin->GetCPPType())
			{
				continue;
			}
		}

		// if we copy onto a sub pin - let's not fold this copy expression.
		// we also don't allow to assign onto variable nodes
		if(AssignExpr->GetType() == FRigVMExprAST::EType::Copy)
		{
			if(!TargetPin->IsRootPin())
			{
				continue;
			}
			if(TargetPin->GetNode()->IsA<URigVMVariableNode>())
			{
				continue;
			}
		}
		
		// non-input pins on anything but a reroute (passthrough or literal) or array node should be skipped
		const bool bIsReroute = TargetPin->GetNode()->IsA<URigVMRerouteNode>();

		// a copy / assign to a constant node root pin can be folded 
		const bool bIsConstantValueTarget = IsConstantDispatchValuePin(TargetPin);
		
		if (TargetPin->GetDirection() != ERigVMPinDirection::Input && !bIsReroute && !bIsConstantValueTarget)
		{
			continue;
		}

		// if this node is a loop node - let's skip the folding
		if (const URigVMNode* ModelNode = TargetPin->GetNode())
		{
			if (ModelNode->IsControlFlowNode())
			{
				continue;
			}
		}

		// if this node is a variable node and the pin requires a watch... skip this
		if (Cast<URigVMVariableNode>(SourcePin->GetNode()))
		{
			if(SourcePin->RequiresWatch(true))
			{
				continue;
			}
		}

		// Skip folding for programmatic pins as we expect them to always exist in work memory
		if(TargetPin->IsProgrammaticPin())
		{
			continue;
		}
		
		FRigVMExprAST* Parent = AssignExpr->Parents[0];
		if (!Parent->IsA(FRigVMExprAST::EType::Var))
		{
			continue;
		}

		// To prevent bad assignments in LWC for VMs compiled in non LWC, we do not allow folding of assignments
		// to/from external variables of type float
		if(IsFloatOrFloatArrayPinOnVariable(SourcePin) || IsFloatOrFloatArrayPinOnVariable(TargetPin))
		{
			continue;
		}

		// If the assign is between external variable pins, don't fold it
		if (TargetPin->GetRootPin()->IsDefinedAsInputVariable())
		{
			continue;
		}

		FRigVMExprAST* Child = AssignExpr->Children[0];
		AssignExpr->RemoveParent(Parent);
		Child->RemoveParent(AssignExpr);

		TArray<FRigVMExprAST*> GrandParents = Parent->Parents;
		for (FRigVMExprAST* GrandParent : GrandParents)
		{
			GrandParent->ReplaceChild(Parent, Child);
			if (GrandParent->IsA(FRigVMExprAST::EType::Assign))
			{
				FRigVMAssignExprAST* GrandParentAssign = GrandParent->To<FRigVMAssignExprAST>();

				const URigVMPin* OldSourcePin = GrandParentAssign->GetSourcePin();
				const FString SourceSegmentPath = OldSourcePin->GetSegmentPath();
				URigVMPin* NewSourcePin = AssignExpr->GetSourcePin();

				if(IsMakeStructDispatchStructPin(NewSourcePin))
				{
					NewSourcePin = NewSourcePin->GetNode()->FindRootPinByName(FRigVMDispatch_MakeStruct::ElementsName.Resolve());
					check(NewSourcePin);
				}
				
				if(!SourceSegmentPath.IsEmpty())
				{
					NewSourcePin = NewSourcePin->FindSubPin(SourceSegmentPath);
					if(NewSourcePin == nullptr)
					{
						Settings.Reportf(
							EMessageSeverity::Error,
							AssignExpr->GetSourcePin()->GetNode(),
							TEXT("Cannot resolve segment path '%s' on pin '%s'."),
							*SourceSegmentPath,
							*AssignExpr->GetSourcePin()->GetPinPath());
						continue;
					}
				}
				
				GrandParentAssign->Link = FRigVMASTLinkDescription(
					GrandParentAssign->GetSourceProxy().GetSibling(NewSourcePin),
					GrandParentAssign->GetTargetProxy(), FString());
				GrandParentAssign->Name = *GetLinkAsString(GrandParentAssign->GetLink());
			}
		}

		ExpressionsToRemove.AddUnique(AssignExpr);
		if (Parent->Parents.Num() == 0)
		{
			ExpressionsToRemove.AddUnique(Parent);
		}
	}

	RemoveExpressions(ExpressionsToRemove);
}

void FRigVMParserAST::FoldSubPinCopies()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TArray<FRigVMExprAST*> ExpressionsToRemove;

	// first - Fold all assignment chains
	for (FRigVMExprAST* Expression : Expressions)
	{
		if (Expression->Parents.Num() == 0)
		{
			continue;
		}

		if (Expression->GetType() != FRigVMExprAST::EType::Copy)
		{
			continue;
		}

		FRigVMCopyExprAST* CopyExpr = Expression->To<FRigVMCopyExprAST>();
		if (CopyExpr->Parents.Num() != 1 ||
			CopyExpr->Children.Num() != 1 ||
			!CopyExpr->Link.SegmentPath.IsEmpty())
		{
			continue;
		}

		URigVMPin* SourcePin = CopyExpr->GetSourcePin();
		URigVMPin* TargetPin = CopyExpr->GetTargetPin();

		// we only consider copies from a sub pin to a root pin in this phase
		if(SourcePin->IsRootPin() || !TargetPin->IsRootPin())
		{
			continue;
		}

		// if the target pin is lazy - don't fold the copy
		if(TargetPin->IsLazy())
		{
			continue;
		}

		// in case the assign has different types for left and right - we need to avoid folding
		// since this assign represents a cast operation
		if(SourcePin->GetCPPTypeObject() != TargetPin->GetCPPTypeObject())
		{
			continue;
		}
		if(SourcePin->GetCPPTypeObject() == nullptr)
		{
			if(SourcePin->GetCPPType() != TargetPin->GetCPPType())
			{
				continue;
			}
		}

		// if the source is a sub-pin of a fixed size array, don't fold the copy
		if (SourcePin->GetRootPin()->IsFixedSizeArray())
		{
			continue;
		}

		// non-input pins on anything but a reroute (passthrough or literal)
		const bool bIsReroute = TargetPin->GetNode()->IsA<URigVMRerouteNode>();

		if (TargetPin->GetDirection() != ERigVMPinDirection::Input && !bIsReroute)
		{
			continue;
		}

		// if this node is a loop node - let's skip the folding
		if (const URigVMNode* ModelNode = TargetPin->GetNode())
		{
			if (ModelNode->IsControlFlowNode())
			{
				continue;
			}
		}

		// if the source or target node is a variable node, don't fold this copy
		if (Cast<URigVMVariableNode>(SourcePin->GetNode()) || Cast<URigVMVariableNode>(TargetPin->GetNode()))
		{
			continue;
		}

		// if the source node is a reroute - skip this
		if (Cast<URigVMRerouteNode>(SourcePin->GetNode()))
		{
			continue;
		}

		// Skip folding for programmatic pins as we expect them to always exist in work memory
		if(TargetPin->IsProgrammaticPin())
		{
			continue;
		}

		// Skip folding onto a return node
		if (TargetPin->GetNode()->IsA<URigVMFunctionReturnNode>())
		{
			continue;
		}
		
		FRigVMExprAST* Parent = CopyExpr->Parents[0];
		if (!Parent->IsA(FRigVMExprAST::EType::Var))
		{
			continue;
		}

		// To prevent bad assignments in LWC for VMs compiled in non LWC, we do not allow folding of assignments
		// to/from external variables of type float
		if(IsFloatOrFloatArrayPinOnVariable(SourcePin) || IsFloatOrFloatArrayPinOnVariable(TargetPin))
		{
			continue;
		}

		// avoid referencing elements of arrays - since the array may change in the interim.
		const URigVMPin* SourceArrayPin = SourcePin;
		bool bIsArrayPin = false;
		while (SourceArrayPin && !SourceArrayPin->IsRootPin())
		{
			if (SourceArrayPin->IsArray() || SourceArrayPin->IsArrayElement())
			{
				bIsArrayPin = true;
				break;
			}
			SourceArrayPin = SourceArrayPin->GetParentPin();
		}
		if (bIsArrayPin)
		{
			continue;
		}

		FRigVMExprAST* Child = CopyExpr->Children[0];
		CopyExpr->RemoveParent(Parent);
		Child->RemoveParent(CopyExpr);

		TArray<FRigVMExprAST*> GrandParents = Parent->Parents;
		for (FRigVMExprAST* GrandParent : GrandParents)
		{
			TArray<int32> ChildIndices;
			GrandParent->ReplaceChild(Parent, Child, &ChildIndices);

			if (GrandParent->IsA(FRigVMExprAST::EType::CallExtern) ||
				GrandParent->IsA(FRigVMExprAST::EType::InlineFunction))
			{
				const URigVMPin* CopySourcePin = CopyExpr->GetSourcePin();
				check(!CopySourcePin->IsRootPin());

				FRigVMNodeExprAST* GrandParentNodeExpr = GrandParent->To<FRigVMNodeExprAST>();
				if(ensure(GrandParentNodeExpr))
				{
					for (int32 ChildIndex : ChildIndices)
					{
						GrandParentNodeExpr->SetSegmentPathForChild(ChildIndex, CopySourcePin->GetSegmentPath());
					}
				}
			}
		}

		ExpressionsToRemove.AddUnique(CopyExpr);
		if (Parent->Parents.Num() == 0)
		{
			ExpressionsToRemove.AddUnique(Parent);
		}
	}

	RemoveExpressions(ExpressionsToRemove);
}

void FRigVMParserAST::FoldLiterals()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TMap<FString, FRigVMLiteralExprAST*> ValueToLiteral;
	TArray<FRigVMExprAST*> ExpressionsToRemove;

	// Returns true if the literal's owning node is marked as interactive (e.g. has DM active).
	// Interactive nodes must not share literal registers with other nodes.
	auto IsInteractive = [this](const FRigVMLiteralExprAST* InLiteral) -> bool
	{
		if (Settings.InteractiveNodes.IsEmpty())
		{
			return false;
		}
		if (const URigVMPin* Pin = InLiteral->GetPin())
		{
			if (const URigVMNode* Node = Pin->GetNode())
			{
				return Settings.InteractiveNodes.Contains(FObjectKey(Node));
			}
		}
		return false;
	};

	for (int32 ExpressionIndex = 0; ExpressionIndex < Expressions.Num(); ExpressionIndex++)
	{
		FRigVMExprAST* Expression = Expressions[ExpressionIndex];
		if (Expression->Parents.Num() == 0)
		{
			continue;
		}

		if (Expression->GetType() == FRigVMExprAST::EType::Literal)
		{
			ensure(Expression->Children.Num() == 0);

			FRigVMLiteralExprAST* LiteralExpr = Expression->To<FRigVMLiteralExprAST>();
			FString DefaultValue = LiteralExpr->GetDefaultValue();
			if (DefaultValue.IsEmpty())
			{
				if (LiteralExpr->GetCPPType() == TEXT("bool"))
				{
					DefaultValue = TEXT("False");
				}
				else if (LiteralExpr->GetCPPType() == TEXT("float"))
				{
					DefaultValue = TEXT("0.000000");
				}
				else if (LiteralExpr->GetCPPType() == TEXT("double"))
				{
					DefaultValue = TEXT("0.000000");
				}
				else if (LiteralExpr->GetCPPType() == TEXT("int32"))
				{
					DefaultValue = TEXT("0");
				}
				else
				{
					continue;
				}
			}

			FString Hash = FString::Printf(TEXT("[%s] %s"), *LiteralExpr->GetCPPType(), *DefaultValue);

			FRigVMLiteralExprAST* const* MappedExpr = ValueToLiteral.Find(Hash);
			if (MappedExpr)
			{
				// Skip folding if either this literal or the canonical belongs to an
				// interactive node — each interactive node must keep its own register.
				if (!IsInteractive(LiteralExpr) && !IsInteractive(*MappedExpr))
				{
					TArray<FRigVMExprAST*> Parents = Expression->Parents;
					for (FRigVMExprAST* Parent : Parents)
					{
						Parent->ReplaceChild(Expression, *MappedExpr);
					}
					ExpressionsToRemove.AddUnique(Expression);
				}
			}
			else
			{
				// Don't use an interactive node's literal as the canonical for a value —
				// doing so would allow non-interactive nodes to share its register.
				if (!IsInteractive(LiteralExpr))
				{
					ValueToLiteral.Add(Hash, LiteralExpr);
				}
			}
		}
	}

	RemoveExpressions(ExpressionsToRemove);
}

const FRigVMExprAST* FRigVMParserAST::GetExprForSubject(const FRigVMASTProxy& InProxy) const
{
	if (FRigVMExprAST* const* ExpressionPtr = SubjectToExpression.Find(InProxy))
	{
		return *ExpressionPtr;
	}
	return nullptr;
}

TArray<const FRigVMExprAST*> FRigVMParserAST::GetExpressionsForSubject(UObject* InSubject) const
{
	TArray<const FRigVMExprAST*> ExpressionsForSubject;
	
	for (TPair<FRigVMASTProxy, FRigVMExprAST*> Pair : SubjectToExpression)
	{
		if(Pair.Key.GetCallstack().Last() == InSubject)
		{
			ExpressionsForSubject.Add(Pair.Value);
		}
	}

	return ExpressionsForSubject;
}

void FRigVMParserAST::PrepareCycleChecking(URigVMPin* InPin)
{
	if (InPin == nullptr)
	{
		LastCycleCheckExpr = nullptr;
		CycleCheckFlags.Reset();
		return;
	}
	FRigVMASTProxy NodeProxy = FRigVMASTProxy::MakeFromUObject(InPin->GetNode());

	const FRigVMExprAST* Expression = nullptr;
	if (FRigVMExprAST* const* ExpressionPtr = SubjectToExpression.Find(NodeProxy))
	{
		Expression = *ExpressionPtr;
	}
	else
	{
		return;
	}

	if (LastCycleCheckExpr != Expression)
	{
		LastCycleCheckExpr = Expression;
		CycleCheckFlags.SetNumZeroed(Expressions.Num());
		CycleCheckFlags[LastCycleCheckExpr->GetIndex()] = ETraverseRelationShip_Self;
	}
}

bool FRigVMParserAST::CanLink(URigVMPin* InSourcePin, URigVMPin* InTargetPin, FString* OutFailureReason)
{
	if (InSourcePin == nullptr || InTargetPin == nullptr || InSourcePin == InTargetPin)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FString(TEXT("Provided objects contain nullptr."));
		}
		return false;
	}

	URigVMNode* SourceNode = InSourcePin->GetNode();
	URigVMNode* TargetNode = InTargetPin->GetNode();
	if (SourceNode == TargetNode)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FString(TEXT("Source and Target Nodes are identical."));
		}
		return false;
	}

	if(SourceNode->IsA<URigVMRerouteNode>())
	{
		TArray<URigVMPin*> SourcePins;
		for (URigVMPin* Pin : SourceNode->GetPins())
		{
			SourcePins.Append(Pin->GetLinkedSourcePins(true));
		}
		
		for (URigVMPin* SourcePin : SourcePins)
		{
			if (!CanLink(SourcePin, InTargetPin, OutFailureReason))
			{
				return false;
			}
		}
		return true;
	}

	if(TargetNode->IsA<URigVMRerouteNode>())
	{
		TArray<URigVMPin*> TargetPins;
		for (URigVMPin* Pin : TargetNode->GetPins())
		{
			TargetPins.Append(Pin->GetLinkedTargetPins(true));
		}
		
		for (URigVMPin* TargetPin : TargetPins)
		{
			if (!CanLink(InSourcePin, TargetPin, OutFailureReason))
			{
				return false;
			}
		}
		return true;
	}

	const FRigVMASTProxy SourceNodeProxy = FRigVMASTProxy::MakeFromUObject(SourceNode);
	const FRigVMASTProxy TargetNodeProxy = FRigVMASTProxy::MakeFromUObject(TargetNode);

	const FRigVMExprAST* SourceExpression = nullptr;
	if (FRigVMExprAST* const* SourceExpressionPtr = SubjectToExpression.Find(SourceNodeProxy))
	{
		SourceExpression = *SourceExpressionPtr;
	}
	else
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FString(TEXT("Source node is not part of AST."));
		}
		return false;
	}

	const FRigVMVarExprAST* SourceVarExpression = nullptr;
	if (FRigVMExprAST* const* SourceVarExpressionPtr = SubjectToExpression.Find(SourceNodeProxy.GetSibling(InSourcePin->GetRootPin())))
	{
		if ((*SourceVarExpressionPtr)->IsA(FRigVMExprAST::EType::Var))
		{
			SourceVarExpression = (*SourceVarExpressionPtr)->To<FRigVMVarExprAST>();
		}
	}

	const FRigVMExprAST* TargetExpression = nullptr;
	if (FRigVMExprAST* const* TargetExpressionPtr = SubjectToExpression.Find(TargetNodeProxy))
	{
		TargetExpression = *TargetExpressionPtr;
	}
	else
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FString(TEXT("Target node is not part of AST."));
		}
		return false;
	}

	const FRigVMBlockExprAST* SourceBlock = SourceExpression->GetBlock();
	const FRigVMBlockExprAST* TargetBlock = TargetExpression->GetBlock();
	if (SourceBlock == nullptr || TargetBlock == nullptr)
	{
		return false;
	}

	// If the source node is an entry of a function, no need to cycle check
	if (SourceNode->IsA<URigVMFunctionEntryNode>() && SourceNode->GetTypedOuter<URigVMFunctionLibrary>() != nullptr)
	{
		return true;
	}
	const FRigVMBlockExprAST* SourceRoot = SourceBlock->GetRootBlock();
	const FRigVMBlockExprAST* TargetRoot = TargetBlock->GetRootBlock();
	
	bool bNeedsCycleChecking = (SourceBlock == TargetBlock);

	// check if source block/root contains the target
	if (!bNeedsCycleChecking)
	{
		TMap<const FRigVMExprAST*, bool> SourceCache;

		// check root first
		bNeedsCycleChecking = SourceRoot->Contains(TargetBlock, &SourceCache);

		// check block if needed (avoid doing it twice)
		if (!bNeedsCycleChecking && SourceBlock != SourceRoot)
		{
			bNeedsCycleChecking = SourceBlock->Contains(TargetBlock, &SourceCache);
		}		
	}

	// check if target block/root contains the source
	if (!bNeedsCycleChecking)
	{
		TMap<const FRigVMExprAST*, bool> TargetCache;

		// check root first
		bNeedsCycleChecking = TargetRoot->Contains(SourceBlock, &TargetCache);

		// check block if needed (avoid doing it twice)
		if (!bNeedsCycleChecking && TargetBlock != TargetRoot)
		{
			bNeedsCycleChecking = TargetBlock->Contains(SourceBlock, &TargetCache);
		}
	}

	if (bNeedsCycleChecking)
	{
		if (SourceVarExpression && SourceVarExpression->SupportsSoftLinks())
		{
			return true;
		}

		if (const URigVMVariableNode* TargetVariableNode = Cast<URigVMVariableNode>(TargetNode))
		{
			if (TargetVariableNode->GetValuePin() == InTargetPin->GetRootPin())
			{
				return true;
			}
		}

		if (LastCycleCheckExpr != SourceExpression && LastCycleCheckExpr != TargetExpression)
		{
			PrepareCycleChecking(InSourcePin);
		}

		TArray<ETraverseRelationShip>& Flags = CycleCheckFlags;
		TraverseParents(LastCycleCheckExpr, [&Flags](const FRigVMExprAST* InExpr) -> bool {
			if (Flags[InExpr->GetIndex()] == ETraverseRelationShip_Self)
			{
				return true;
			}
			if (Flags[InExpr->GetIndex()] != ETraverseRelationShip_Unknown)
			{
				return false;
			}
			if (InExpr->IsA(FRigVMExprAST::EType::Var))
			{
				if (InExpr->To<FRigVMVarExprAST>()->SupportsSoftLinks())
				{
					return false;
				}
			}
			Flags[InExpr->GetIndex()] = ETraverseRelationShip_Parent;
			return true;
		});

		TraverseChildren(LastCycleCheckExpr, [&Flags](const FRigVMExprAST* InExpr) -> bool {
			if (Flags[InExpr->GetIndex()] == ETraverseRelationShip_Self)
			{
				return true;
			}
			if (Flags[InExpr->GetIndex()] != ETraverseRelationShip_Unknown)
			{
				return false;
			}
			if (InExpr->IsA(FRigVMExprAST::EType::Var))
			{
				if (InExpr->To<FRigVMVarExprAST>()->SupportsSoftLinks())
				{
					return false;
				}
			}
			Flags[InExpr->GetIndex()] = ETraverseRelationShip_Child;
			return true;
		});

		bool bFoundCycle = false;
		if (LastCycleCheckExpr == SourceExpression)
		{
			bFoundCycle = Flags[TargetExpression->GetIndex()] == ETraverseRelationShip_Child;
		}
		else
		{
			bFoundCycle = Flags[SourceExpression->GetIndex()] == ETraverseRelationShip_Parent;
		}

		if (bFoundCycle)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString(TEXT("Cycles are not allowed."));
			}
			return false;
		}
	}
	else
	{
		// if one of the blocks is not part of the current 
		// execution - that's fine.
		if (SourceRoot->ContainsEntry() != TargetRoot->ContainsEntry())
		{
			return true;
		}

		if (OutFailureReason)
		{
			*OutFailureReason = FString::Printf(TEXT("You cannot combine nodes from \"%s\" and \"%s\"."), *SourceBlock->GetName().ToString(), *TargetBlock->GetName().ToString());
		}
		return false;
	}

	return true;
}

FString FRigVMParserAST::DumpText() const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FString Result;
	for (FRigVMExprAST* RootExpr : RootExpressions)
	{
		if(RootExpr == GetObsoleteBlock(false /* create */))
		{
			continue;
		}
		Result += TEXT("\n") + RootExpr->DumpText();
	}
	return Result;
}

FString FRigVMParserAST::DumpDot() const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TArray<bool> OutExpressionDefined;
	OutExpressionDefined.AddZeroed(Expressions.Num());

	FVisualGraph VisualGraph(TEXT("AST"));

	VisualGraph.AddSubGraph(TEXT("AST"), FName(TEXT("AST")));
	VisualGraph.AddSubGraph(TEXT("unused"), FName(TEXT("Unused")));

	struct Local
	{
		const TArray<FColor> BlockCombinationColors =
		{
			FColor(255,235,205),
			FColor(220,20,60),
			FColor(46,139,87),
			FColor(0,0,205),
			FColor(255,228,225),
			FColor(255,218,185),
			FColor(255,140,0),
			FColor(255,228,181),
			FColor(216,191,216),
			FColor(210,105,30),
			FColor(240,248,255),
			FColor(30,144,255),
			FColor(47,79,79),
			FColor(245,245,220),
			FColor(165,42,42),
			FColor(255,245,238),
			FColor(112,128,144),
			FColor(220,220,220),
			FColor(123,104,238),
			FColor(139,0,139),
			FColor(255,182,193),
			FColor(250,128,114),
			FColor(148,0,211),
			FColor(224,255,255),
			FColor(255,165,0),
			FColor(255,250,240),
			FColor(0,128,128),
			FColor(175,238,238),
			FColor(147,112,219),
			FColor(255,160,122),
			FColor(0,255,127),
			FColor(255,240,245),
			FColor(211,211,211),
			FColor(173,255,47),
			FColor(0,100,0),
			FColor(0,128,0),
			FColor(32,178,170),
			FColor(123,104,238),
			FColor(240,230,140),
			FColor(139,69,19),
			FColor(153,50,204),
			FColor(219,112,147),
			FColor(138,43,226),
			FColor(245,245,245),
			FColor(255,255,240),
			FColor(255,69,0),
			FColor(135,206,235),
			FColor(240,255,255),
			FColor(205,92,92),
			FColor(255,250,205),
			FColor(105,105,105),
			FColor(255,250,250),
			FColor(72,61,139),
			FColor(255,248,220),
			FColor(255,192,203),
			FColor(222,184,135),
			FColor(245,222,179),
			FColor(0,0,128),
			FColor(245,255,250),
			FColor(25,25,112),
			FColor(244,164,96),
			FColor(238,130,238),
			FColor(240,255,240),
			FColor(34,139,34),
		};

		int32 BlockCombinationColorIndex = 0;
		TMap<uint32, int32> BlockCombinationHashToColor; 

		TArray<int32> VisitChildren(const FRigVMExprAST* InExpr, int32 InSubGraphIndex, FVisualGraph& OutGraph)
		{
			TArray<int32> ChildNodeIndices;
			for (FRigVMExprAST* Child : InExpr->Children)
			{
				ChildNodeIndices.Add(VisitExpr(Child, InSubGraphIndex, OutGraph));
			}
			return ChildNodeIndices;
		}
		
		int32 VisitExpr(const FRigVMExprAST* InExpr, int32 InSubGraphIndex, FVisualGraph& OutGraph)
		{
			const FName NodeName = *FString::Printf(TEXT("node_%d"), InExpr->GetIndex());

			int32 NodeIndex = OutGraph.FindNode(NodeName); 
			if(NodeIndex != INDEX_NONE)
			{
				return NodeIndex;
			}

			FString Label = InExpr->GetName().ToString();
			TOptional<EVisualGraphShape> Shape = EVisualGraphShape::Ellipse;
			int32 SubGraphIndex = InSubGraphIndex;
			
			switch (InExpr->GetType())
			{
				case FRigVMExprAST::EType::Literal:
				{
					Label = FString::Printf(TEXT("%s(Literal)"), *InExpr->To<FRigVMLiteralExprAST>()->GetPin()->GetName());
					break;
				}
				case FRigVMExprAST::EType::Var:
				{
					if (InExpr->To<FRigVMVarExprAST>()->IsGraphVariable())
					{
						URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InExpr->To<FRigVMVarExprAST>()->GetPin()->GetNode());
						check(VariableNode);
						Label = FString::Printf(TEXT("Variable %s"), *VariableNode->GetVariableName().ToString());
					}
					else if (InExpr->To<FRigVMVarExprAST>()->IsEnumValue())
					{
						URigVMEnumNode* EnumNode = Cast<URigVMEnumNode>(InExpr->To<FRigVMVarExprAST>()->GetPin()->GetNode());
						check(EnumNode);
						Label = FString::Printf(TEXT("Enum %s"), *EnumNode->GetCPPType());
					}
					else
					{
						const URigVMPin* Pin = InExpr->To<FRigVMVarExprAST>()->GetPin();
						check(Pin);
						if (Pin->IsDefinedAsInputVariable())
						{
							if (InExpr->Name.ToString().EndsWith(TEXT("::Clone")))
							{
								Label = FString::Printf(TEXT("Cloned Input Variable %s"), *Pin->GetName());
							}
							else
							{
								Label = FString::Printf(TEXT("Input Variable %s"), *Pin->GetName());
							}
						}
						else
						{
							Label = Pin->GetPinPath(true);
						}
					}
						
					if (InExpr->To<FRigVMVarExprAST>()->IsExecuteContext())
					{
						Shape = EVisualGraphShape::House;
						Label = TEXT("Execute");
					}
						
					break;
				}
				case FRigVMExprAST::EType::Block:
				{
					if (InExpr->GetParent() == nullptr)
					{
						Label = TEXT("Unused");
						SubGraphIndex = OutGraph.FindSubGraph(TEXT("unused"));;
					}
					else
					{
						Label = TEXT("Block");
					}
					break;
				}
				case FRigVMExprAST::EType::Assign:
				{
					Label = TEXT("=");
					break;
				}
				case FRigVMExprAST::EType::Copy:
				{
					if (InExpr->Name.ToString().EndsWith(TEXT("::Clone")))
					{
						Label = TEXT("Clone");
					}
					else
					{
						Label = TEXT("Copy");
					}
					break;
				}
				case FRigVMExprAST::EType::CachedValue:
				{
					Label = TEXT("Cache");
					break;
				}
				case FRigVMExprAST::EType::CallExtern:
				{
					if (URigVMUnitNode* Node = Cast<URigVMUnitNode>(InExpr->To<FRigVMCallExternExprAST>()->GetNode()))
					{
						Label = Node->GetScriptStruct()->GetName();
					}
					break;
				}
				case FRigVMExprAST::EType::InlineFunction:
				{
					if (URigVMFunctionReferenceNode* Node = Cast<URigVMFunctionReferenceNode>(InExpr->To<FRigVMInlineFunctionExprAST>()->GetNode()))
					{
						Label = FString::Printf(TEXT("Inline %s"), *Node->GetReferencedFunctionHeader().Name.ToString());
					}
					break;
				}
				case FRigVMExprAST::EType::NoOp:
				{
					Label = TEXT("NoOp");
					break;
				}
				case FRigVMExprAST::EType::Exit:
				{
					Label = TEXT("Exit");
					break;
				}
				case FRigVMExprAST::EType::Entry:
				{
					SubGraphIndex = OutGraph.FindSubGraph(InExpr->GetName());
					if(SubGraphIndex == INDEX_NONE)
					{
						const int32 ASTGraphIndex = OutGraph.FindSubGraph(TEXT("AST"));
						FString SanitizedNameString = InExpr->GetName().ToString();
						SanitizedNameString.RemoveSpacesInline();
						SubGraphIndex = OutGraph.AddSubGraph(*SanitizedNameString, InExpr->GetName(), ASTGraphIndex);
					}
					break;
				}
				default:
				{
					break;
				}
			}

			switch (InExpr->GetType())
			{
				case FRigVMExprAST::EType::Entry:
				case FRigVMExprAST::EType::Exit:
				case FRigVMExprAST::EType::Block:
				{
					Shape = EVisualGraphShape::Diamond;
					break;
				}
				case FRigVMExprAST::EType::Assign:
				case FRigVMExprAST::EType::Copy:
				case FRigVMExprAST::EType::CallExtern:
				case FRigVMExprAST::EType::InlineFunction:
				case FRigVMExprAST::EType::NoOp:
				{
					Shape = EVisualGraphShape::Box;
					break;
				}
				default:
				{
					break;
				}
			}

			if (!Label.IsEmpty())
			{
				TOptional<FLinearColor> Color;
				const TOptional<uint32> BlockCombinationHash = InExpr->GetBlockCombinationHash();
				if(BlockCombinationHash.IsSet())
				{
					const uint32 Hash = BlockCombinationHash.GetValue();
					if(!BlockCombinationHashToColor.Contains(Hash))
					{
						BlockCombinationHashToColor.Add(Hash, BlockCombinationColorIndex++);
						if(BlockCombinationColorIndex >= BlockCombinationColors.Num())
						{
							BlockCombinationColorIndex = 0;
						}
					}
					Color = BlockCombinationColors[BlockCombinationHashToColor.FindChecked(Hash)];
				}
				
				const TOptional<FName> DisplayName = FName(*Label);
				NodeIndex = OutGraph.AddNode(NodeName, DisplayName, Color, Shape);
				OutGraph.AddNodeToSubGraph(NodeIndex, SubGraphIndex);
			}

			TArray<int32> ChildNodeIndices = VisitChildren(InExpr, SubGraphIndex, OutGraph);

			if(NodeIndex != INDEX_NONE)
			{
				for(const int32 ChildNodeIndex : ChildNodeIndices)
				{
					if(ChildNodeIndex != INDEX_NONE)
					{
						OutGraph.AddEdge(ChildNodeIndex, NodeIndex, EVisualGraphEdgeDirection::SourceToTarget);
					}
				}
			}

			return NodeIndex;
		}
	};

	Local LocalStruct;
	for (FRigVMExprAST* Expr : RootExpressions)
	{
		if (Expr == GetObsoleteBlock(false))
		{
			continue;
		}

		LocalStruct.VisitExpr(Expr, INDEX_NONE, VisualGraph);
	}

	return VisualGraph.DumpDot();
}

FRigVMBlockExprAST* FRigVMParserAST::GetObsoleteBlock(bool bCreateIfMissing)
{
	if (ObsoleteBlock == nullptr && bCreateIfMissing)
	{
		ObsoleteBlock = MakeExpr<FRigVMBlockExprAST>(FRigVMExprAST::EType::Block, FRigVMASTProxy());
		ObsoleteBlock->bIsObsolete = true;
		RootExpressions.Add(ObsoleteBlock);
	}
	return ObsoleteBlock;
}

const FRigVMBlockExprAST* FRigVMParserAST::GetObsoleteBlock(bool bCreateIfMissing) const
{
	if (ObsoleteBlock == nullptr && bCreateIfMissing)
	{
		FRigVMParserAST* MutableThis = (FRigVMParserAST*)this;
		MutableThis->ObsoleteBlock = MutableThis->MakeExpr<FRigVMBlockExprAST>(FRigVMExprAST::EType::Block, FRigVMASTProxy());
		MutableThis->ObsoleteBlock->bIsObsolete = true;
		MutableThis->RootExpressions.Add(MutableThis->ObsoleteBlock);
	}
	return ObsoleteBlock;
}

void FRigVMParserAST::RemoveExpressions(TArray<FRigVMExprAST*> InExprs)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if(InExprs.IsEmpty())
	{
		return;
	}
	
	RefreshExprIndices();

	TArray<FRigVMExprAST*> ExpressionsToRemove;
	ExpressionsToRemove.Append(InExprs);

	TArray<int32> NumRemainingParents;
	NumRemainingParents.AddUninitialized(Expressions.Num());
	for(int32 ExpressionIndex = 0; ExpressionIndex < Expressions.Num(); ExpressionIndex++)
	{
		NumRemainingParents[ExpressionIndex] = Expressions[ExpressionIndex]->Parents.Num();
	}

	TArray<bool> bRemoveExpression;
	bRemoveExpression.AddZeroed(Expressions.Num());
	for(int32 ExpressionIndex = 0; ExpressionIndex < ExpressionsToRemove.Num(); ExpressionIndex++)
	{
		FRigVMExprAST* Expr = ExpressionsToRemove[ExpressionIndex];
		bRemoveExpression[Expr->GetIndex()] = true;

		for (FRigVMExprAST* Child : Expr->Children)
		{
			if(--NumRemainingParents[Child->GetIndex()] == 0)
			{
				ExpressionsToRemove.Add(Child);
			}
		}
	}

	TArray<FRigVMExprAST*> RemainingExpressions;
	RemainingExpressions.Reserve(Expressions.Num() - ExpressionsToRemove.Num());
	
	for(int32 ExpressionIndex = 0; ExpressionIndex < Expressions.Num(); ExpressionIndex++)
	{
		if(!bRemoveExpression[ExpressionIndex])
		{
			FRigVMExprAST* Expr = Expressions[ExpressionIndex];
			RemainingExpressions.Add(Expr);

			for(int32 ChildIndex = Expr->Children.Num() - 1; ChildIndex >= 0; ChildIndex--)
			{
				FRigVMExprAST* ChildExpr = Expr->Children[ChildIndex];
				if(bRemoveExpression[ChildExpr->GetIndex()])
				{
					Expr->Children.RemoveAt(ChildIndex);
				}
			}

			for(int32 ParentIndex = Expr->Parents.Num() - 1; ParentIndex >= 0; ParentIndex--)
			{
				FRigVMExprAST* ParentExpr = Expr->Parents[ParentIndex];
				if(bRemoveExpression[ParentExpr->GetIndex()])
				{
					Expr->Parents.RemoveAt(ParentIndex);
				}
			}
		}
	}

	Expressions = RemainingExpressions;

	TArray<FRigVMASTProxy> KeysToRemove;
	for (TPair<FRigVMASTProxy, FRigVMExprAST*> Pair : SubjectToExpression)
	{
		if (bRemoveExpression[Pair.Value->GetIndex()])
		{
			KeysToRemove.Add(Pair.Key);
		}
	}
	
	for (const FRigVMASTProxy& KeyToRemove : KeysToRemove)
	{
		SubjectToExpression.Remove(KeyToRemove);
	}

	for(int32 ExpressionIndex = ExpressionsToRemove.Num() - 1; ExpressionIndex >= 0; ExpressionIndex--)
	{
		ExpressionsToRemove[ExpressionIndex]->Index = INDEX_NONE;
		DeletedExpressions.Add(ExpressionsToRemove[ExpressionIndex]);
	}

	RefreshExprIndices();
	IncrementCacheVersion();
}

void FRigVMParserAST::TraverseParents(const FRigVMExprAST* InExpr, TFunctionRef<bool(const FRigVMExprAST*)> InContinuePredicate)
{
	if (!InContinuePredicate(InExpr))
	{
		return;
	}
	for (const FRigVMExprAST* ParentExpr : InExpr->Parents)
	{
		TraverseParents(ParentExpr, InContinuePredicate);
	}
}

void FRigVMParserAST::TraverseChildren(const FRigVMExprAST* InExpr, TFunctionRef<bool(const FRigVMExprAST*)> InContinuePredicate)
{
	if (!InContinuePredicate(InExpr))
	{
		return;
	}
	for (const FRigVMExprAST* ChildExpr : InExpr->Children)
	{
		TraverseChildren(ChildExpr, InContinuePredicate);
	}
}

TArray<int32> FRigVMParserAST::GetSourceLinkIndices(const FRigVMASTProxy& InPinProxy, bool bRecursive) const
{
	return GetLinkIndices(InPinProxy, true, bRecursive);
}

TArray<int32> FRigVMParserAST::GetTargetLinkIndices(const FRigVMASTProxy& InPinProxy, bool bRecursive) const
{
	return GetLinkIndices(InPinProxy, false, bRecursive);
}

TArray<int32> FRigVMParserAST::GetLinkIndices(const FRigVMASTProxy& InPinProxy, bool bGetSource, bool bRecursive) const
{
	const static TArray<int32> EmptyIntArray;
	TArray<int32> LinkIndices;

	if(const TArray<int32>* LinkIndicesPtr = (bGetSource ? SourceLinkIndices : TargetLinkIndices).Find(InPinProxy))
	{
		LinkIndices = *LinkIndicesPtr;
	}

	if (bRecursive)
	{
		URigVMPin* Pin = InPinProxy.GetSubjectChecked<URigVMPin>();
		for (URigVMPin* SubPin : Pin->GetSubPins())
		{
			// We dont get links to programmatic pins as links are traversed manually in TraversePin rather than
			// using links from parent traits. This ensures we dont double-up on programmatic pin traversal.
			if(!SubPin->IsProgrammaticPin())
			{
				FRigVMASTProxy SubPinProxy = InPinProxy.GetSibling(SubPin);
				LinkIndices.Append(GetLinkIndices(SubPinProxy, bGetSource, true));
			}
		}
	}

	return LinkIndices;
}


const FRigVMASTLinkDescription& FRigVMParserAST::GetLink(int32 InLinkIndex) const
{
	return Links[InLinkIndex];
}

void FRigVMParserAST::Inline(const TArray<URigVMGraph*>& InGraphs)
{
	TArray<FRigVMASTProxy> LocalNodeProxies;
	for(URigVMGraph* Graph : InGraphs)
	{
		for (URigVMNode* LocalNode : Graph->GetNodes())
		{
			if (IsValid(LocalNode))
			{
				LocalNodeProxies.Add(FRigVMASTProxy::MakeFromUObject(LocalNode));
			}
		}
	}
	Inline(InGraphs, LocalNodeProxies);
}

void FRigVMParserAST::Inline(const TArray<URigVMGraph*>& InGraphs, const TArray<FRigVMASTProxy>& InNodeProxies)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	struct LocalPinTraversalInfo
	{
		URigVMPin::FPinOverrideMap* PinOverrides;
		TMap<FRigVMASTProxy, FRigVMASTProxy>* SourcePins;
		TMap<FRigVMASTProxy, TArray<int32>>* TargetLinkIndices;
		TMap<FRigVMASTProxy, TArray<int32>>* SourceLinkIndices;
		TArray<FRigVMASTLinkDescription>* Links;
		TArray<FRigVMASTProxy> LibraryNodeCallstack;
		FRigVMParserASTSettings* Settings;
		URigVMLibraryNode* LibraryNodeBeingCompiled;

		static bool ShouldRecursePin(const FRigVMASTProxy& InPinProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			URigVMPin* Pin = InPinProxy.GetSubjectChecked<URigVMPin>();
			URigVMNode* Node = Pin->GetNode();
			if (URigVMVariableNode* VarNode = Cast<URigVMVariableNode>(Node))
			{
				return VarNode->IsInputArgument();
			}

			// If its an interface node of the library we are compiling, don't recurse
			if (OutTraversalInfo.LibraryNodeBeingCompiled != nullptr)
			{
				if (Node->IsA<URigVMFunctionInterfaceNode>())
				{
					if (Node->GetTypedOuter<URigVMLibraryNode>() == OutTraversalInfo.LibraryNodeBeingCompiled)
					{
						return false;
					}
				}
			}

			// If its a function reference, don't recurse
			if (Node->IsA<URigVMFunctionReferenceNode>())
			{
				return false;
			}
			
			if (Cast<URigVMRerouteNode>(Node))
			{
				return true;
			}

			if (OutTraversalInfo.Settings->bFoldSubPinCopies)
			{
				if (IsBreakStructDispatchNode(Node))
				{
					return true;
				}
			}
				
			return Node->IsA<URigVMLibraryNode>() ||
				Node->IsA<URigVMFunctionInterfaceNode>();
		}

		static bool IsValidPinForAST(const FRigVMASTProxy& InPinProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			return !ShouldRecursePin(InPinProxy, OutTraversalInfo);
		}

		static bool IsValidLinkForAST(const FRigVMASTProxy& InSourcePinProxy, const FRigVMASTProxy& InTargetPinProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			return IsValidPinForAST(InSourcePinProxy, OutTraversalInfo) && IsValidPinForAST(InTargetPinProxy, OutTraversalInfo);
		}

		static FRigVMASTProxy FindSourcePin(const FRigVMASTProxy& InPinProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			return FindSourcePin(InPinProxy, InPinProxy, OutTraversalInfo);
		}

		static FRigVMASTProxy FindSourcePin(const FRigVMASTProxy& InPinProxy, const FRigVMASTProxy& InPinProxyForMap, LocalPinTraversalInfo& OutTraversalInfo)
		{
			URigVMPin* Pin = InPinProxy.GetSubjectChecked<URigVMPin>();
			const bool bStoreSourcePinOnMap = InPinProxy == InPinProxyForMap;

			// if this pin is a root on a library
			if (Pin->GetParentPin() == nullptr)
			{
				if (Pin->GetDirection() == ERigVMPinDirection::Output ||
					Pin->GetDirection() == ERigVMPinDirection::IO)
				{
					URigVMNode* Node = Pin->GetNode();
					if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
					{
						FRigVMASTProxy CollapseNodeProxy = InPinProxy.GetSibling(CollapseNode);
						if (!OutTraversalInfo.LibraryNodeCallstack.Contains(CollapseNodeProxy))
						{
							if (URigVMFunctionReturnNode* ReturnNode = CollapseNode->GetReturnNode())
							{
								if (URigVMPin* ReturnPin = ReturnNode->FindPin(Pin->GetName()))
								{
									OutTraversalInfo.LibraryNodeCallstack.Push(CollapseNodeProxy);
									FRigVMASTProxy ReturnPinProxy = CollapseNodeProxy.GetChild(ReturnPin);
									FRigVMASTProxy SourcePinProxy = FindSourcePin(ReturnPinProxy, OutTraversalInfo);
									SourcePinProxy = SourcePinProxy.IsValid() ? SourcePinProxy : ReturnPinProxy;
									if(bStoreSourcePinOnMap)
									{
										OutTraversalInfo.SourcePins->FindOrAdd(InPinProxyForMap) = SourcePinProxy;
									}
									OutTraversalInfo.LibraryNodeCallstack.Pop();
									return SourcePinProxy;

								}
							}
						}
					}
					else if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
					{
						if (VariableNode->IsInputArgument())
						{
							if (URigVMFunctionEntryNode* EntryNode = VariableNode->GetGraph()->GetEntryNode())
							{
								if (URigVMPin* EntryPin = EntryNode->FindPin(VariableNode->GetVariableName().ToString()))
								{
									FRigVMASTProxy EntryPinProxy = InPinProxy.GetSibling(EntryPin);
									FRigVMASTProxy SourcePinProxy = FindSourcePin(EntryPinProxy, OutTraversalInfo);
									SourcePinProxy = SourcePinProxy.IsValid() ? SourcePinProxy : EntryPinProxy;
									if(bStoreSourcePinOnMap)
									{
										OutTraversalInfo.SourcePins->FindOrAdd(InPinProxyForMap) = SourcePinProxy;
									}
									return SourcePinProxy;
								} 
							}
						}
					}
					else if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(Node))
					{
						if (EntryNode->GetGraph()->GetOuter() == OutTraversalInfo.LibraryNodeBeingCompiled)
						{
							return FRigVMASTProxy(); 
						}
						for (int32 LibraryNodeIndex = OutTraversalInfo.LibraryNodeCallstack.Num() - 1; LibraryNodeIndex >= 0; LibraryNodeIndex--)
						{
							FRigVMASTProxy LibraryNodeProxy = OutTraversalInfo.LibraryNodeCallstack[LibraryNodeIndex];
							URigVMLibraryNode* LastLibraryNode = LibraryNodeProxy.GetSubject<URigVMLibraryNode>();
							if (LastLibraryNode == nullptr)
							{
								continue;
							}

							if(LastLibraryNode->GetEntryNode() == EntryNode)
							{
								if (URigVMPin* LibraryPin = LastLibraryNode->FindPin(Pin->GetName()))
								{
									FRigVMASTProxy LibraryPinProxy = LibraryNodeProxy.GetSibling(LibraryPin);
									FRigVMASTProxy SourcePinProxy = FindSourcePin(LibraryPinProxy, OutTraversalInfo);
									SourcePinProxy = SourcePinProxy.IsValid() ? SourcePinProxy : LibraryPinProxy;
									if(bStoreSourcePinOnMap)
									{
										OutTraversalInfo.SourcePins->FindOrAdd(InPinProxyForMap) = SourcePinProxy;
									}
									return SourcePinProxy;
								}
							}
						}
					}
				}
			}

			if (Pin->GetDirection() != ERigVMPinDirection::Input &&
				Pin->GetDirection() != ERigVMPinDirection::IO &&
				Pin->GetDirection() != ERigVMPinDirection::Output)
			{
				return FRigVMASTProxy();
			}

			bool bIOPinOnLeftOfLibraryNode = false;
			if (Pin->GetDirection() == ERigVMPinDirection::IO)
			{
				if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
				{
					bIOPinOnLeftOfLibraryNode = OutTraversalInfo.LibraryNodeCallstack.Contains(InPinProxy.GetSibling(LibraryNode));
				}
			}

			if (!bIOPinOnLeftOfLibraryNode && bStoreSourcePinOnMap)
			{
				// note: this map isn't going to work for functions which are referenced.
				// (since the pin objects are shared between multiple invocation nodes)
				if (const FRigVMASTProxy* SourcePinProxy = OutTraversalInfo.SourcePins->Find(InPinProxyForMap))
				{
					return *SourcePinProxy;
				}
			}

			TArray<FString> SegmentPath;
			FRigVMASTProxy SourcePinProxy;

			URigVMPin* ChildPin = Pin;
			while (ChildPin != nullptr)
			{
				if (ChildPin->GetDirection() == ERigVMPinDirection::Output && ChildPin->GetParentPin() == nullptr)
				{
					if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(ChildPin->GetNode()))
					{
						if (EntryNode->GetGraph()->GetOuter() == OutTraversalInfo.LibraryNodeBeingCompiled)
						{
							return FRigVMASTProxy(); 
						}
						
						// rather than relying on the other we are going to query what's in the call stack.
						// for collapse nodes that's not a different, but for function ref nodes the outer
						// node is the definition and not the reference node - which sits in the callstack.
						if (URigVMLibraryNode* OuterNode = (InPinProxy.GetParent().GetSubject<URigVMLibraryNode>()))
						{
							if (URigVMPin* OuterPin = OuterNode->FindPin(ChildPin->GetName()))
							{
								FRigVMASTProxy OuterPinProxy = InPinProxy.GetParent().GetSibling(OuterPin);
								SourcePinProxy = FindSourcePin(OuterPinProxy, OutTraversalInfo);
								SourcePinProxy = SourcePinProxy.IsValid() ? SourcePinProxy : OuterPinProxy;
								break;
							}
						}
					}
					else if(URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ChildPin->GetNode()))
					{
						if(ChildPin != InPinProxy.GetSubject<URigVMPin>())
						{
							const FRigVMASTProxy ChildPinProxy = InPinProxy.GetSibling(ChildPin); 
							if (ShouldRecursePin(ChildPinProxy, OutTraversalInfo))
							{
								FRigVMASTProxy SourceSourcePinProxy = FindSourcePin(ChildPinProxy, OutTraversalInfo);
								if(SourceSourcePinProxy.IsValid())
								{
									SourcePinProxy = SourceSourcePinProxy;
								}
							}
						}
					}
					else if (IsBreakStructDispatchNode(ChildPin->GetNode()))
					{
						if (OutTraversalInfo.Settings->bFoldSubPinCopies)
						{
							if (URigVMPin* StructPin = ChildPin->GetNode()->FindRootPinByName(FRigVMDispatch_BreakStruct::StructName.Resolve()))
							{
								SourcePinProxy = InPinProxy.GetSibling(StructPin); 
								if (ShouldRecursePin(SourcePinProxy, OutTraversalInfo))
								{
									FRigVMASTProxy SourceSourcePinProxy = FindSourcePin(SourcePinProxy, OutTraversalInfo);
									if(SourceSourcePinProxy.IsValid())
									{
										SourcePinProxy = SourceSourcePinProxy;
									}
								}
							}
						}
					}
				}

				TArray<URigVMLink*> SourceLinks = ChildPin->GetSourceLinks(false /* recursive */);
				
				URigVMPin* SourcePin = nullptr;
				if (SourceLinks.Num() > 0)
				{
					SourcePin = SourceLinks[0]->GetSourcePin();
				}
				else if(ChildPin->IsBoundToInputArgument())
				{
					if (URigVMGraph* Graph = ChildPin->GetGraph())
					{
						if (URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode())
						{
							SourcePin = EntryNode->FindPin(ChildPin->GetBoundVariableName());
						}
					}
				}
				else if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ChildPin->GetNode()))
				{
					if (VariableNode->IsInputArgument())
					{
						if (URigVMGraph* Graph = ChildPin->GetGraph())
						{
							if (URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode())
							{
								SourcePin = EntryNode->FindPin(VariableNode->GetVariableName().ToString());
								if (ChildPin->GetParentPin())
								{									
									SourcePin = SourcePin->FindSubPin(ChildPin->GetSegmentPath());
								}
							}
						}
					}
				}
				
				if(SourcePin)
				{
					SourcePinProxy = InPinProxy.GetSibling(SourcePin);

					if (ShouldRecursePin(SourcePinProxy, OutTraversalInfo))
					{
						FRigVMASTProxy SourceSourcePinProxy = FindSourcePin(SourcePinProxy, OutTraversalInfo);
						if(SourceSourcePinProxy.IsValid())
						{
							SourcePinProxy = SourceSourcePinProxy;
						}
					}

					break;
				}

				URigVMPin* ParentPin = ChildPin->GetParentPin();
				if (ParentPin)
				{
					FRigVMASTProxy ParentPinProxy = InPinProxy.GetSibling(ParentPin);

					// if we found a parent pin which has a source that is not a reroute
					if (FRigVMASTProxy* ParentSourcePinProxyPtr = OutTraversalInfo.SourcePins->Find(ParentPinProxy))
					{
						FRigVMASTProxy& ParentSourcePinProxy = *ParentSourcePinProxyPtr;
						if (ParentSourcePinProxy.IsValid())
						{
							if (!ShouldRecursePin(ParentSourcePinProxy, OutTraversalInfo))
							{
								// only discard results here if we haven't crossed a collapse node boundary
								if(ParentSourcePinProxy.GetSubjectChecked<URigVMPin>()->GetGraph() == ChildPin->GetGraph())
								{
									SourcePinProxy = FRigVMASTProxy();
									break;
								}
							}
						}
					}

					SegmentPath.Push(ChildPin->GetName());
				}
				ChildPin = ParentPin;
			}
			
			if (SourcePinProxy.IsValid())
			{
				while (!SegmentPath.IsEmpty())
				{
					FString Segment = SegmentPath.Pop();

					URigVMPin* SourcePin = SourcePinProxy.GetSubjectChecked<URigVMPin>();
					if (URigVMPin* SourceSubPin = SourcePin->FindSubPin(Segment))
					{
						SourcePinProxy = SourcePinProxy.GetSibling(SourceSubPin);

						if (ShouldRecursePin(SourcePinProxy, OutTraversalInfo))
						{
							FRigVMASTProxy SourceSourceSubPinProxy = FindSourcePin(SourcePinProxy, OutTraversalInfo);
							if (SourceSourceSubPinProxy.IsValid())
							{
								SourcePinProxy = SourceSourceSubPinProxy;
							}
						}
					}
					else
					{
						SourcePinProxy = FRigVMASTProxy();
						break;
					}
				}
			}

			if (!bIOPinOnLeftOfLibraryNode && bStoreSourcePinOnMap)
			{
				OutTraversalInfo.SourcePins->FindOrAdd(InPinProxyForMap) = SourcePinProxy;
			}
			return SourcePinProxy;
		}

		static void VisitPin(const FRigVMASTProxy& InPinProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			return VisitPin(InPinProxy, InPinProxy, OutTraversalInfo, FString());
		}

		static void VisitPin(const FRigVMASTProxy& InPinProxy, const FRigVMASTProxy& InPinProxyForMap,
			LocalPinTraversalInfo& OutTraversalInfo, const FString& InSegmentPath)
		{
			const FRigVMASTProxy SourcePinProxy = FindSourcePin(InPinProxy, InPinProxyForMap, OutTraversalInfo);
			if (SourcePinProxy.IsValid())
			{
				// The source pin is the final determined source pin, since
				// FindSourcePin is recursive.
				// If the source pin is on a reroute node, this means that
				// we only care about the default value - since it is a 
				// "hanging" reroute without any live input.
				// same goes for library nodes or return nodes - we'll 
				// just use the default pin value in that case.

				URigVMPin* SourcePin = SourcePinProxy.GetSubjectChecked<URigVMPin>();
				URigVMNode* SourceNode = SourcePin->GetNode();

				if (SourceNode->IsA<URigVMRerouteNode>() ||
					SourceNode->IsA<URigVMCollapseNode>() ||
					SourceNode->IsA<URigVMFunctionReturnNode>())
				{
					// for arrays - if there are sub-pins on the determined source, we need to walk those as well
					if(SourcePin->IsArray())
					{
						TArray<URigVMPin*> SourceSubPins = SourcePin->GetSubPins();
						for (int32 SourceSubPinIndex = 0; SourceSubPinIndex < SourceSubPins.Num(); SourceSubPinIndex++)
						{
							URigVMPin* SourceSubPin = SourceSubPins[SourceSubPinIndex];
							const FRigVMASTProxy SubPinProxy = SourcePinProxy.GetSibling(SourceSubPin);
							const FString SegmentPath = SourceSubPin->GetSubPinPath(SourcePin, false);
							VisitPin(SubPinProxy, InPinProxy, OutTraversalInfo, SegmentPath);
							SourceSubPins.Append(SourceSubPin->GetSubPins());
						}
					}

					// when asking for the default value of the array - we may need to get the previously stored override
					const URigVMPin::FPinOverride SourceOverride(SourcePinProxy, *OutTraversalInfo.PinOverrides);
						
					// we query the default value now since the potential array elements have been visited and their
					// potential default value override has been stored to the map.
					OutTraversalInfo.PinOverrides->FindOrAdd(InPinProxy) = URigVMPin::FPinOverrideValue(SourcePin, SourceOverride);
				}
				else
				{
					if (IsValidLinkForAST(SourcePinProxy, InPinProxyForMap, OutTraversalInfo))
					{
						if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(SourceNode))
						{
							if (const FRigVMGraphFunctionData* FunctionData = FunctionReferenceNode->GetReferencedFunctionData())
							{
								const FString PinName = FRigVMPropertyDescription::SanitizeName(SourcePin->GetFName()).ToString();
								if (const FRigVMFunctionCompilationPropertyDescription* Description = FunctionData->CompilationData.WorkPropertyDescriptions.FindByPredicate([PinName](const FRigVMFunctionCompilationPropertyDescription& Description)
								{
									return Description.Name.ToString().EndsWith(PinName);
								}))
								{
									// when asking for the default value of the array - we may need to get the previously stored override
									URigVMPin::FPinOverrideValue OverrideValue;
									OverrideValue.DefaultValue = Description->DefaultValue;
									
									// we query the default value now since the potential array elements have been visited and their
									// potential default value override has been stored to the map.
									OutTraversalInfo.PinOverrides->FindOrAdd(SourcePinProxy) = OverrideValue;
								}
								
							}
						}
						
						FRigVMASTLinkDescription Link(SourcePinProxy, InPinProxyForMap, InSegmentPath);
						Link.LinkIndex = OutTraversalInfo.Links->Num();
						OutTraversalInfo.Links->Add(Link);
						OutTraversalInfo.SourceLinkIndices->FindOrAdd(InPinProxyForMap).Add(Link.LinkIndex);
						OutTraversalInfo.TargetLinkIndices->FindOrAdd(SourcePinProxy).Add(Link.LinkIndex);
					}
				}
			}

			// If the pin is an array, and it has a source pin, the subpins can be safely ignored
			URigVMPin* Pin = InPinProxy.GetSubjectChecked<URigVMPin>();
			if (!Pin->IsArray() || !SourcePinProxy.IsValid())
			{
				for (URigVMPin* SubPin : Pin->GetSubPins())
				{
					FRigVMASTProxy SubPinProxy = InPinProxy.GetSibling(SubPin);
					VisitPin(SubPinProxy, OutTraversalInfo);
				}
			}
		}

		static void VisitNode(const FRigVMASTProxy& InNodeProxy, LocalPinTraversalInfo& OutTraversalInfo)
		{
			if (InNodeProxy.GetSubject()->IsA<URigVMRerouteNode>())
			{
				return;
			}
			
			const bool bIsCompilingFunction = OutTraversalInfo.LibraryNodeBeingCompiled != nullptr;
			if (bIsCompilingFunction)
			{
				if (InNodeProxy.IsA<URigVMFunctionInterfaceNode>())
				{
					URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
					if (Node->GetTypedOuter<URigVMLibraryNode>() != OutTraversalInfo.LibraryNodeBeingCompiled)
					{
						return;
					}
				}
			}
			else
			{
				if (InNodeProxy.IsA<URigVMFunctionInterfaceNode>())
				{
					return;
				}
			}

			if (URigVMCollapseNode* LibraryNode = InNodeProxy.GetSubject<URigVMCollapseNode>())
			{
				if (LibraryNode->GetContainedGraph() == nullptr)
				{
					OutTraversalInfo.Settings->Reportf(
						EMessageSeverity::Error,
						LibraryNode,
						TEXT("Library Node '%s' doesn't contain a subgraph."),
						*LibraryNode->GetName());
					return;
				}

				OutTraversalInfo.LibraryNodeCallstack.Push(InNodeProxy);
				TArray<URigVMNode*> ContainedNodes = LibraryNode->GetContainedNodes();
				for (URigVMNode* ContainedNode : ContainedNodes)
				{
					// create a proxy which uses the previous node as a callstack
					FRigVMASTProxy ContainedNodeProxy = InNodeProxy.GetChild(ContainedNode);
					VisitNode(ContainedNodeProxy, OutTraversalInfo);
				}
				OutTraversalInfo.LibraryNodeCallstack.Pop();
			}
			else if (URigVMFunctionReferenceNode* FuncRefNode = InNodeProxy.GetSubject<URigVMFunctionReferenceNode>())
			{
				if (FuncRefNode->GetReferencedFunctionData() == nullptr)
				{
					FString FunctionPath = FuncRefNode->GetReferencedFunctionHeader().GetHash();

					OutTraversalInfo.Settings->Reportf(
						EMessageSeverity::Error, 
						FuncRefNode, 
						TEXT("Function Reference '%s' references a missing function (%s)."), 
						*FuncRefNode->GetName(),
						*FunctionPath);
				}
					
				for (URigVMPin* Pin : FuncRefNode->GetPins())
				{
					FRigVMASTProxy PinProxy = InNodeProxy.GetSibling(Pin);
					LocalPinTraversalInfo::VisitPin(PinProxy, OutTraversalInfo);
				}
			}
			else
			{
				URigVMNode* Node = InNodeProxy.GetSubjectChecked<URigVMNode>();
				for (URigVMPin* Pin : Node->GetPins())
				{
					FRigVMASTProxy PinProxy = InNodeProxy.GetSibling(Pin);
					LocalPinTraversalInfo::VisitPin(PinProxy, OutTraversalInfo);
				}
			}
		}
	};

	NodeProxies.Reset();
	SourceLinkIndices.Reset();
	TargetLinkIndices.Reset();

	// a) find all of the relevant nodes,
	//    inline and traverse into library nodes
	NodeProxies = InNodeProxies;

	// c) flatten links from an entry node / to a return node
	//    also traverse links along reroutes and flatten them
	LocalPinTraversalInfo TraversalInfo;
	TraversalInfo.PinOverrides = &PinOverrides;
	TraversalInfo.SourcePins = &SharedOperandPins;
	TraversalInfo.TargetLinkIndices = &TargetLinkIndices;
	TraversalInfo.SourceLinkIndices = &SourceLinkIndices;
	TraversalInfo.Links = &Links;
	TraversalInfo.Settings = &Settings;
	TraversalInfo.LibraryNodeBeingCompiled = this->LibraryNodeBeingCompiled;

	for (const FRigVMASTProxy& NodeProxy : NodeProxies)
	{
		LocalPinTraversalInfo::VisitNode(NodeProxy, TraversalInfo);
	}

	// once we are done with the inlining we may need to clean up pin value overrides for pins
	// that also have overrides on sub pins
	TArray<FRigVMASTProxy> PinOverridesToRemove;
	for(const TPair<FRigVMASTProxy, URigVMPin::FPinOverrideValue>& Override : PinOverrides)
	{
		if(URigVMPin* Pin = Override.Key.GetSubject<URigVMPin>())
		{
			for(URigVMPin* SubPin : Pin->GetSubPins())
			{
				const FRigVMASTProxy SubPinProxy = Override.Key.GetSibling(SubPin);
				if(PinOverrides.Contains(SubPinProxy))
				{
					PinOverridesToRemove.Add(Override.Key);
				}
			}
		}
	}
	for(const FRigVMASTProxy& ProxyToRemove : PinOverridesToRemove)
	{
		PinOverrides.Remove(ProxyToRemove);
	}
}

bool FRigVMParserAST::ShouldLinkBeSkipped(const FRigVMASTLinkDescription& InLink) const
{
	const URigVMPin* SourcePin = InLink.SourceProxy.GetSubjectChecked<URigVMPin>();
	const URigVMPin* TargetPin = InLink.TargetProxy.GetSubjectChecked<URigVMPin>();

	for (URigVMLink* LinkToSkip : LinksToSkip)
	{
		if (LinkToSkip->GetSourcePin() == SourcePin &&
			LinkToSkip->GetTargetPin() == TargetPin)
		{
			return true;
		}
	}
	return false;
}

FString FRigVMParserAST::GetLinkAsString(const FRigVMASTLinkDescription& InLink)
{
	const URigVMPin* SourcePin = InLink.SourceProxy.GetSubjectChecked<URigVMPin>();
	const URigVMPin* TargetPin = InLink.TargetProxy.GetSubjectChecked<URigVMPin>();
	static const FString EmptyString;
	static const FString PeriodString = TEXT(".");

	return URigVMLink::GetPinPathRepresentation(SourcePin->GetPinPath(), 
		 FString::Printf(TEXT("%s%s%s"), *TargetPin->GetPinPath(),
			*(InLink.SegmentPath.IsEmpty() ? EmptyString : PeriodString), *InLink.SegmentPath));
}

void FRigVMParserAST::IncrementCacheVersion() const
{
	CacheVersion++;
}

bool FRigVMParserAST::IsNoOpUnitNode(const URigVMNode* InNode)
{
#if WITH_EDITOR
	if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode))
	{
		if (const UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
		{
			if (ScriptStruct->HasMetaDataHierarchical(FRigVMStruct::NoOpMetaName))
			{
				return true;
			}
		}
	}
#endif
	return false;
}

bool FRigVMParserAST::IsConstantDispatchNode(const URigVMNode* InNode)
{
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
		{
			return Factory->GetScriptStruct() == FRigVMDispatch_Constant::StaticStruct();
		}
	}
	return false;
}

bool FRigVMParserAST::IsConstantDispatchValuePin(const URigVMPin* InPin)
{
	if(InPin && InPin->IsRootPin() && IsConstantDispatchNode(InPin->GetNode()))
	{
		return InPin->GetFName() == FRigVMDispatch_Constant::ValueName.Resolve();
	}
	return false;
}

bool FRigVMParserAST::IsMakeStructDispatchNode(const URigVMNode* InNode)
{
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
		{
			return Factory->GetScriptStruct() == FRigVMDispatch_MakeStruct::StaticStruct();
		}
	}
	return false;
}

bool FRigVMParserAST::IsMakeStructDispatchElementsPin(const URigVMPin* InPin)
{
	if(InPin && InPin->IsRootPin() && IsMakeStructDispatchNode(InPin->GetNode()))
	{
		return InPin->GetFName() == FRigVMDispatch_MakeStruct::ElementsName.Resolve();
	}
	return false;
}

bool FRigVMParserAST::IsMakeStructDispatchStructPin(const URigVMPin* InPin)
{
	if(InPin && InPin->IsRootPin() && IsMakeStructDispatchNode(InPin->GetNode()))
	{
		return InPin->GetFName() == FRigVMDispatch_MakeStruct::StructName.Resolve();
	}
	return false;
}

bool FRigVMParserAST::IsBreakStructDispatchNode(const URigVMNode* InNode)
{
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
		{
			return Factory->GetScriptStruct() == FRigVMDispatch_BreakStruct::StaticStruct();
		}
	}
	return false;
}

bool FRigVMParserAST::IsBreakStructDispatchStructPin(const URigVMPin* InPin)
{
	if(InPin && InPin->IsRootPin() && IsBreakStructDispatchNode(InPin->GetNode()))
	{
		return InPin->GetFName() == FRigVMDispatch_MakeStruct::StructName.Resolve();
	}
	return false;
}

bool FRigVMParserAST::IsBreakStructDispatchElementsPin(const URigVMPin* InPin)
{
	if(InPin && InPin->IsRootPin() && IsBreakStructDispatchNode(InPin->GetNode()))
	{
		return InPin->GetFName() == FRigVMDispatch_MakeStruct::ElementsName.Resolve();
	}
	return false;

}

bool FRigVMParserAST::IsFloatOrFloatArrayPinOnVariable(const URigVMPin* InPin)
{
	// To prevent bad assignments in LWC for VMs compiled in non LWC, we do not allow folding of assignments
	// to/from external variables of type float
	if (InPin->GetCPPType() == TEXT("float") || InPin->GetCPPType() == TEXT("TArray<float>"))
	{
		if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InPin->GetNode()))
		{
			if (!VariableNode->IsInputArgument() && !VariableNode->IsLocalVariable())
			{
				return true;
			}
		}
	}
	return false;
}
