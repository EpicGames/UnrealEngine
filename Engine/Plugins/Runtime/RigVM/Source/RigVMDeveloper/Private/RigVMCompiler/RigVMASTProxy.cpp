// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMASTProxy.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"

FString FRigVMCallstack::GetCallPath(bool bIncludeLast) const
{
	TArray<FString> Segments;
	for (TWeakObjectPtr<UObject> Entry : Stack)
	{
		if (Entry.IsValid())
		{
			if (URigVMNode* Node = Cast<URigVMNode>(Entry))
			{
				if (URigVMGraph* Graph = Node->GetGraph())
				{
					if(Graph->IsRootGraph())
					{
						Segments.Add(Node->GetNodePath(true));
					}
					else
					{
						Segments.Add(Node->GetName());
					}
				}
			}
			else if (URigVMPin* Pin = Cast<URigVMPin>(Entry))
			{
				if (URigVMGraph* Graph = Pin->GetGraph())
				{
					const bool bUseNodePath = Stack.Num() == 1 && !Graph->IsA<URigVMFunctionLibrary>();
					Segments.Add(Pin->GetPinPath(bUseNodePath));
				}
			}
		}
	}

	if (!bIncludeLast && !Segments.IsEmpty())
	{
		Segments.Pop();
	}

	if (Segments.Num() == 0)
	{
		return FString();
	}

	FString Result = Segments[0];
	for (int32 PartIndex = 1; PartIndex < Segments.Num(); PartIndex++)
	{
		Result += TEXT("|") + Segments[PartIndex];
	}

	return Result;
}

int32 FRigVMCallstack::Num() const
{
	return Stack.Num();
}

const UObject* FRigVMCallstack::Last() const
{
	if(Stack.IsEmpty() || !Stack.Last().IsValid())
	{
		return nullptr;
	}
	return Stack.Last().Get();
}

const UObject* FRigVMCallstack::operator[](int32 InIndex) const
{
	if (!Stack[InIndex].IsValid())
	{
		return nullptr;
	}
	return Stack[InIndex].Get();
}

bool FRigVMCallstack::Contains(const UObject* InEntry) const
{
	return Stack.ContainsByPredicate([InEntry](const TWeakObjectPtr<UObject>& Object)
	{
		if (Object.IsValid())
		{
			return Object.Get() == InEntry;
		}
		return false;
	});
}

FRigVMCallstack FRigVMCallstack::GetCallStackUpTo(int32 InIndex) const
{
	if(!ensure(Stack.IsValidIndex(InIndex)))
	{
		return FRigVMCallstack();
	}
	
	FRigVMCallstack Partial;
	Partial.Stack = Stack;
	Partial.Stack.SetNum(InIndex + 1);
	return Partial;
}

FRigVMASTProxy FRigVMASTProxy::MakeFromUObject(UObject* InSubject)
{
	UObject* Subject = InSubject;

	FRigVMASTProxy Proxy;
	Proxy.Callstack.Stack.Reset();

	while (Subject != nullptr)
	{
		if (URigVMPin* Pin = Cast<URigVMPin>(Subject))
		{
			Subject = Pin->GetNode();
		}
		else if (URigVMNode* Node = Cast<URigVMNode>(Subject))
		{
			Subject = Node->GetGraph();
		}
		else if (URigVMGraph* Graph = Cast<URigVMGraph>(Subject))
		{
			Subject = Cast<URigVMLibraryNode>(Graph->GetOuter());
			if (Subject)
			{
				Proxy.Callstack.Stack.Insert(Subject, 0);
			}
		}
		else
		{
			break;
		}
	}

	Proxy.Callstack.Stack.Push(InSubject);

#if UE_BUILD_DEBUG
	Proxy.DebugName = Proxy.Callstack.GetCallPath();
#endif
	return Proxy;
}

FRigVMASTProxy FRigVMASTProxy::MakeFromCallPath(const FString& InCallPath, UObject* InRootObject)
{
	if(InCallPath.IsEmpty() || InRootObject == nullptr)
	{
		return FRigVMASTProxy();
	}

	UObject* ParentObject = InRootObject;

	FRigVMASTProxy Proxy;
	Proxy.Callstack.Stack.Reset();

	FString Left;
	FString Right = InCallPath;

	while(!Right.IsEmpty() && ParentObject != nullptr)
	{
		if(!Right.Split(TEXT("|"), &Left, &Right))
		{
			Left = Right;
			Right.Empty();
		}

		if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ParentObject))
		{
			ParentObject = CollapseNode->GetContainedGraph();
		}
		else if(URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(ParentObject))
		{
			if (URigVMLibraryNode* LibraryNode = ReferenceNode->LoadReferencedNode())
			{
				ParentObject = LibraryNode->GetContainedGraph();
			}
		}

		if(URigVMGraph* Graph = Cast<URigVMGraph>(ParentObject))
		{
			if(Graph->IsRootGraph() && Left.EndsWith(TEXT("::")))
			{
				if(Left != Graph->GetNodePath())
				{
					return FRigVMASTProxy();
				}
			}
			else if(URigVMNode* FoundNode = Graph->FindNodeByName(*Left))
			{
				Proxy.Callstack.Stack.Push(FoundNode);
				ParentObject = FoundNode;
			}
			else if(URigVMPin* FoundPin = Graph->FindPin(Left))
			{
				Proxy.Callstack.Stack.Push(FoundPin);
				break;
			}
			else
			{
				return FRigVMASTProxy();
			}
		}
		else
		{
			return FRigVMASTProxy();
		}
	}

#if UE_BUILD_DEBUG
	Proxy.DebugName = Proxy.Callstack.GetCallPath();
#endif

	return Proxy;
}

FRigVMASTProxy FRigVMASTProxy::MakeFromCallstack(const FRigVMCallstack& InCallstack)
{
	FRigVMASTProxy Proxy;
	Proxy.Callstack = InCallstack;
	return Proxy;
}

FRigVMASTProxy FRigVMASTProxy::MakeFromCallstack(const TArray<TWeakObjectPtr<UObject>>* InCallstack)
{
	check(InCallstack);
	FRigVMCallstack Callstack;
	Callstack.Stack = *InCallstack;
	return MakeFromCallstack(Callstack);
}
