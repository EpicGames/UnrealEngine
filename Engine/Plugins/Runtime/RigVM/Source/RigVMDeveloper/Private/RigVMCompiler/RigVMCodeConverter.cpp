// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMCodeConverter.h"

#include "IGameplayProvider.h"
#include "RigVMCompiler/RigVMCodeEnvironment.h"
#include "RigVMDeveloperModule.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/SRigVMExecutionStackView.h"
#include "Algo/Sort.h"

FRigVMCodeConverter::FRigVMCodeConverter(IRigVMEditorAssetInterface* InEditorAsset, const FRigVMCodeConversionSettings& InSettings)
	: EditorAsset(InEditorAsset)
	, Settings(InSettings)
{
	FScopedSlowTask SlowTask(15.f, FText());

	if (!ensureMsgf(EditorAsset, TEXT("FRigVMCodeConverter: EditorAsset is null")))
	{
		bHasError = true;
		return;
	}
	VM = EditorAsset->GetVM(true);
	if (!ensureMsgf(VM, TEXT("FRigVMCodeConverter: VM is null for asset '%s'"), *EditorAsset->GetObject()->GetPathName()))
	{
		bHasError = true;
		return;
	}
	if (!ensureMsgf(!VM->IsNativized(), TEXT("FRigVMCodeConverter: VM '%s' is already nativized"), *VM->GetPathName()))
	{
		bHasError = true;
		return;
	}

	ByteCode = &VM->GetByteCode();
	Instructions = ByteCode->GetInstructions();

	Blocks.Reserve(ByteCode->NumBranches());
	for (int32 BranchIndex = 0; BranchIndex < ByteCode->NumBranches(); ++BranchIndex)
	{
		Blocks.Add(ByteCode->GetBranch(BranchIndex));
	}
	for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if (Instruction.OpCode != ERigVMOpCode::RunInstructions)
		{
			continue;
		}
			
		const FRigVMRunInstructionsOp& RunInstructionsOp = ByteCode->GetOpAt<FRigVMRunInstructionsOp>(Instruction);
		const int32 First = RunInstructionsOp.StartInstruction;
		const int32 Last = RunInstructionsOp.EndInstruction;
		if (Last < First)
		{
			continue;
		}
		if (RunInstructionBranches.Contains({First, Last}))
		{
			continue;
		}

		FRigVMBranchInfo Branch;
		Branch.Label = TEXT("RunInstructions");
		Branch.FirstInstruction = First;
		Branch.LastInstruction = Last;
		Branch.InstructionIndex = Instruction.Index;
		Branch.Index = Blocks.Num();

		Blocks.Add(Branch);
		RunInstructionBranches.Add({First, Last}, Branch.Index);
	}

	if (!ProcessDependency(VM->GetClass()))
	{
		return;
	}

	Json = inja::json::object();

	const FString PackagePath = InEditorAsset->GetObject()->GetOutermost()->GetName();
	PackageName = FPaths::GetCleanFilename(PackagePath); 

	Json["AssetName"] = ToJson(PackageName);
	Json["PackagePath"] = ToJson(PackagePath);
	Json["EditorObjectPathName"] = ToJson(InEditorAsset->GetObject()->GetPathName());
	Json["TargetModule"] = ToJson(InSettings.TargetModule);
	const UScriptStruct* ExecuteContextStruct = EditorAsset->GetRigVMClientHost()->GetRigVMExecuteContextStruct();
	Json["PublicDataContextStruct"] = ToJson(ExecuteContextStruct->GetStructCPPName());
	if (!ProcessDependency(ExecuteContextStruct))
	{
		return;
	}
	Json["LocalizedRegistry"] = ToJson(VM->GetLocalizedRegistryAsString()); 

	// Task 1: Parse External Variables
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(0.f, NSLOCTEXT("RigVMCodeConverter", "ParsingExternalVariables", "Parsing External Variables..."));
	{
		inja::json JsonVariables = inja::json::array();
		const TArray<FRigVMGraphVariableDescription> Variables = EditorAsset->GetAssetVariables();
		const int32 MemoryTypeIndex = (int32)ERigVMMemoryType::External;
		for (int32 VariableIndex = 0; VariableIndex < Variables.Num(); VariableIndex++)
		{
			const FRigVMGraphVariableDescription& Variable = Variables[VariableIndex];
			JsonVariables.push_back(ToJson(Variable));
			PropertyMap.Add({MemoryTypeIndex, VariableIndex}, VariableIndex);
		}

		Json["Variables"] = JsonVariables;
	}

	// Task 2: Setup block and callable index per instruction
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "SetupBlockAndCallableIndices", "Parsing Block / Callable Lookup..."));
	BlockIndexPerInstruction.SetNum(Instructions.Num());
	CallableIndexPerInstruction.SetNum(Instructions.Num());
	for (int32 Index = 0; Index < Instructions.Num(); Index++)
	{
		BlockIndexPerInstruction[Index] = INDEX_NONE;
		CallableIndexPerInstruction[Index] = INDEX_NONE;
	}

	for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); BlockIndex++)
	{
		const FRigVMBranchInfo& Block = Blocks[BlockIndex];
		if (Block.Label == FRigVMStruct::ControlFlowCompletedName ||
			Block.FirstInstruction > Block.LastInstruction)
		{
			continue;
		}
		for (int32 InstructionIndex = Block.FirstInstruction; InstructionIndex <= FMath::Max(Block.FirstInstruction, Block.LastInstruction); InstructionIndex++)
		{
			if(BlockIndexPerInstruction[InstructionIndex] == INDEX_NONE)
			{
				BlockIndexPerInstruction[InstructionIndex] = Block.Index;
			}
			else
			{
				const FRigVMBranchInfo& PreviousBlock = Blocks[BlockIndexPerInstruction[InstructionIndex]];
				int32 Num = FMath::Max(Block.FirstInstruction, Block.LastInstruction) - Block.FirstInstruction + 1; 
				int32 PreviousNum = FMath::Max(PreviousBlock.FirstInstruction, PreviousBlock.LastInstruction) - PreviousBlock.FirstInstruction + 1;
				if (PreviousNum > Num)
				{
					BlockIndexPerInstruction[InstructionIndex] = Block.Index;
				}
			}
		}
	}

	for (int32 CallableIndex = 0; CallableIndex < ByteCode->NumCallables(); CallableIndex++)
	{
		const FRigVMCallableInfo* Callable = ByteCode->GetCallable(CallableIndex);
		for (int32 InstructionIndex = Callable->FirstInstruction; InstructionIndex <= Callable->LastInstruction; InstructionIndex++)
		{
			if (!ensureMsgf(CallableIndexPerInstruction[InstructionIndex] == INDEX_NONE || CallableIndexPerInstruction[InstructionIndex] == Callable->Index,
				TEXT("FRigVMCodeConverter: Instruction %d belongs to multiple callables (%d and %d)"),
				InstructionIndex, CallableIndexPerInstruction[InstructionIndex], Callable->Index))
			{
				bHasError = true;
				return;
			}
			CallableIndexPerInstruction[InstructionIndex] = Callable->Index;
		}

		for (int32 ArgumentIndex = 0; ArgumentIndex < Callable->Arguments.Num(); ArgumentIndex++)
		{
			const FRigVMCallableArgument& Argument = Callable->Arguments[ArgumentIndex];
			const int32 MemoryTypeIndex = (int32)Argument.ForwardedOperand.GetMemoryType();
			CallableArgumentMap.Add(
				{MemoryTypeIndex, Argument.ForwardedOperand.GetRegisterIndex()},
				{Callable->Index, ArgumentIndex, Argument.Name});
		}
	}

	{
		const uint32 VMHash = VM->GetVMHash();
		Json["VMHash"] = VMHash;
		UE_LOGF(LogRigVMDeveloper, Display, "Converting VM '%ls' with hash %lu.", *VM->GetPathName(), VMHash);
	}

	{
		// Task 3: Parse Unique 2D Arrays
		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseUnique2DArrays", "Parsing 2D Arrays..."));
		{
			const TArray<FRigVMMemoryStorageStruct*> MemoryStructsFor2DArrays = {
				&VM->GetDefaultLiteralMemory(),
				&VM->GetDefaultWorkMemory()
			};
			TSet<FString> Unique2DNativeTypes;
			inja::json Json2DNativeTypes = inja::json::array();

			for (FRigVMMemoryStorageStruct* MemoryStruct : MemoryStructsFor2DArrays)
			{
				const UPropertyBag* PropertyBag = MemoryStruct->GetPropertyBagStruct();
				if (!ensureMsgf(PropertyBag, TEXT("FRigVMCodeConverter: Invalid VM - PropertyBag is null. Cannot nativize.")))
				{
					bHasError = true;
					return;
				}
				const TConstArrayView<FPropertyBagPropertyDesc> Descriptions = PropertyBag->GetPropertyDescs();
				for (const FPropertyBagPropertyDesc& Description : Descriptions)
				{
					if (Description.ContainerTypes.Num() != 2)
					{
						continue;
					}
				
					if (!ensure(Description.ContainerTypes[0] == EPropertyBagContainerType::Array) ||
						!ensure(Description.ContainerTypes[1] == EPropertyBagContainerType::Array))
					{
						continue; // Skip malformed 2D array description
					}

					const FRigVMGraphVariableDescription VariableDescription = RigVMVariableUtils::VariableDescriptionFromPropertyDesc(Description);
					const FString NativeType = VariableDescription.CPPType;
					const FString BaseNativeType = SanitizeNativeType(RigVMTypeUtils::BaseTypeFromArrayType(RigVMTypeUtils::BaseTypeFromArrayType(NativeType)));
					if (Unique2DNativeTypes.Contains(BaseNativeType))
					{
						continue;
					}
					Unique2DNativeTypes.Add(BaseNativeType);

					const FString OriginalNativeType = SanitizeNativeType(NativeType);
					const FString ArrayNativeType = SanitizeNativeType(FString::Printf(TEXT("F%s_%s_Array"), *PackageName, *BaseNativeType));
					const FString RemappedNativeType = RigVMTypeUtils::ArrayTypeFromBaseType(ArrayNativeType);
					NativeTypeMap.Add(OriginalNativeType, RemappedNativeType);

					inja::json Json2DNativeType = inja::json::object();
					Json2DNativeType["BaseNativeType"] = ToJson(BaseNativeType);
					Json2DNativeType["OriginalNativeType"] = ToJson(OriginalNativeType);
					Json2DNativeType["ArrayNativeType"] = ToJson(ArrayNativeType);
					Json2DNativeType["RemappedNativeType"] = ToJson(RemappedNativeType);
					Json2DNativeTypes.push_back(Json2DNativeType);
				}
			}

			Json["Native2DArrays"] = Json2DNativeTypes;
		}

		inja::json JsonMemory = inja::json::object();

		// Task 4: Parse Literal Memory
		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseLiteralMemory", "Parsing Literal Memory..."));
		JsonMemory["Literal"] = ToJson(VM->GetDefaultLiteralMemory());

		// Task 5: Parse Work Memory
		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseWorkMemory", "Parsing Work Memory..."));
		JsonMemory["Work"] = ToJson(VM->GetDefaultWorkMemory());
		
		Json["Memory"] = JsonMemory;

		if (VM->ExternalPropertyPaths.Num() != VM->ExternalPropertyPathDescriptions.Num())
		{
			VM->RefreshExternalPropertyPaths();
		}

		// Task 6: Parse Property Paths
		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParsePropertyPaths", "Parsing Property Paths..."));
		
		const TArray<const TArray<FRigVMPropertyPath>*> PropertyPathArrays = {
			&VM->GetDefaultLiteralMemory().GetPropertyPaths(),
			&VM->GetDefaultWorkMemory().GetPropertyPaths(),
			&VM->ExternalPropertyPaths
		};
		
		inja::json UniqueJsonPropertyPaths = inja::json::array();
		for(const TArray<FRigVMPropertyPath>* PropertyPathArray : PropertyPathArrays)
		{
			for (const FRigVMPropertyPath& PropertyPath : (*PropertyPathArray))
			{
				const uint32 Hash = GetPropertyPathHash(PropertyPath);
				if (PropertyPathMap.Contains(Hash))
				{
					continue;
				}
				PropertyPathMap.Add(Hash, UniqueJsonPropertyPaths.size());
				UniqueJsonPropertyPaths.push_back(ToJson(PropertyPath));
			}
		}
		Json["PropertyPaths"] = UniqueJsonPropertyPaths;
	}

	// Task 7: Parse Functions
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	inja::json JsonFunctions = inja::json::array();
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseFunctions", "Parsing Unit Functions..."));
	{
		FRigVMRegistry& Registry = FRigVMRegistry::Get();
		
		const TArray<const FRigVMFunction*>& Functions = VM->GetFunctions();
		for (const FRigVMFunction* Function : Functions)
		{
			JsonFunctions.push_back(ToJson(Function));
		}
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseSingletonFunctions", "Parsing Missing Singleton Unit Functions..."));
	{
		FRigVMRegistry& Registry = FRigVMRegistry::Get();

		for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
		{
			const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
			if (Instruction.OpCode != ERigVMOpCode::Execute)
			{
				continue;
			}
			
			const FRigVMExecuteOp& ExecuteOp = ByteCode->GetOpAt<FRigVMExecuteOp>(Instruction);
			const FRigVMFunction* LocalFunction = VM->GetFunctions()[ExecuteOp.CallableIndex];
			if (LocalFunction->Factory == nullptr)
			{
				continue;
			}
			if (!LocalFunction->Factory->IsSingleton())
			{
				continue;
			}
			const FRigVMFunction* GlobalFunction = Registry.FindFunction(*LocalFunction->Name);
			if (!ensureMsgf(GlobalFunction, TEXT("FRigVMCodeConverter: Could not find global function '%s' in registry"), *LocalFunction->Name))
			{
				bHasError = true;
				return;
			}

			// for singleton functions we may need a specific permutation for the native language counterpart - since arbitrary
			// types and void ptr casting may not be available in the target language.
			FRigVMTemplateTypeMap Types;
			const FRigVMOperandArray Operands = ByteCode->GetOperandsForCallableOp(Instruction);
			for (int32 OperandIndex = 0; OperandIndex < Operands.Num(); OperandIndex++)
			{
				FName ArgumentName;
				{
					FRigVMRegistryReadLock ReadLock(Registry);
					ArgumentName = GlobalFunction->Factory->GetArgumentNameForOperandIndex(OperandIndex, Operands.Num(), ReadLock);
				}
				int32 ArgumentIndex = GlobalFunction->GetArgumentIndex(ArgumentName);
				if (ArgumentIndex == INDEX_NONE)
				{
					const FString ArgumentNameString = ArgumentName.ToString();
					FString ArgumentNamePrefix, ArgumentNameSuffix;
					if (ArgumentNameString.Split(TEXT("_"), &ArgumentNamePrefix, &ArgumentNameSuffix, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
					{
						ArgumentIndex = GlobalFunction->GetArgumentIndex(*ArgumentNamePrefix);
					}
				}
				if (!ensureMsgf(ArgumentIndex != INDEX_NONE, TEXT("FRigVMCodeConverter: Could not find argument '%s' in function '%s'"), *ArgumentName.ToString(), *GlobalFunction->Name))
				{
					bHasError = true;
					return;
				}

				ArgumentName = GlobalFunction->Arguments[ArgumentIndex].Name;
				if (GlobalFunction->IsFixedSizeArray(ArgumentName))
				{
					continue;
				}

				const FRigVMOperand& Operand = Operands[OperandIndex];
				const FString CPPTypeString = GetNativeTypeForOperand(Operand, InstructionIndex);
				Types.Add(ArgumentName, Registry.GetTypeIndexFromCPPType(CPPTypeString));
			}

			TSharedPtr<FRigVMFunction> CustomFunction = MakeShared<FRigVMFunction>();
			CustomFunction->Factory = GlobalFunction->Factory;
			CustomFunction->Struct = GlobalFunction->Struct;
			CustomFunction->TemplateIndex = GlobalFunction->TemplateIndex;
			CustomFunction->Name = GlobalFunction->Name;

			bool bRequiresCustomFunction = false;
			for (const FRigVMFunctionArgument& Argument : GlobalFunction->Arguments)
			{
				FString Type = Argument.Type;
				if (const TRigVMTypeIndex* TypeIndex = Types.Find(Argument.Name))
				{
					const FString CustomType = Registry.GetType(*TypeIndex).CPPType.ToString();
					if (CustomType != Type)
					{
						bRequiresCustomFunction = true;
						Type = CustomType;
					}
				}
				
				FRigVMFunctionArgument& CustomArgument = CustomFunction->Arguments.AddDefaulted_GetRef();
				CustomArgument.NameString = MakeShared<FString>(Argument.Name);
				CustomArgument.TypeString = MakeShared<FString>(Type);
				CustomArgument.Name = CustomArgument.NameString->operator*();
				CustomArgument.Type = CustomArgument.TypeString->operator*();
				CustomArgument.Direction = Argument.Direction;
			}

			if (!bRequiresCustomFunction)
			{
				continue;
			}

			uint32 CustomFunctionHash = GetTypeHash(CustomFunction->Name);
			for (const FRigVMFunctionArgument& Argument : CustomFunction->Arguments)
			{
				CustomFunctionHash = HashCombine(CustomFunctionHash, GetTypeHash(FString(Argument.Name)));
				CustomFunctionHash = HashCombine(CustomFunctionHash, GetTypeHash(FString(Argument.Type)));
			}

			if (HashToCustomFunction.Contains(CustomFunctionHash))
			{
				CustomFunction = HashToCustomFunction[CustomFunctionHash];
			}
			else
			{
				CustomFunction->Index = GlobalFunction->Factory->GetTemplate()->NumPermutations() + HashToCustomFunction.Num(); 
				JsonFunctions.push_back(ToJson(CustomFunction.Get()));
				HashToCustomFunction.Add(CustomFunctionHash, CustomFunction);
			}
			InstructionToCustomFunction.Add(InstructionIndex, CustomFunction);
		}
	}
	Json["Functions"] = JsonFunctions;

	// Task 8: Parse Blocks
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseBlocks", "Parsing Blocks / Branches..."));
	{
		inja::json JsonBlocks = inja::json::array();
		TArray<FName> BlockNames;

		for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); BlockIndex++)
		{
			const FRigVMBranchInfo& Block = Blocks[BlockIndex];
			JsonBlocks.push_back(ToJson(Block));

			if (!ensureMsgf(!Block.Label.IsNone(), TEXT("FRigVMCodeConverter: Block %d has no label"), BlockIndex))
			{
				bHasError = true;
				return;
			}
			BlockNames.AddUnique(Block.Label);
		}
		Json["Blocks"] = JsonBlocks;

		inja::json JsonBlockNames = inja::json::array();
		for (const FName& BlockName : BlockNames)
		{
			JsonBlockNames.push_back(ToJson(BlockName));
		}
		Json["BlockNames"] = JsonBlockNames;
	}

	// Task 9: Parse Callables
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseCallables", "Parsing Callables..."));
	{
		inja::json JsonCallables = inja::json::array();
		for (int32 CallableIndex = 0; CallableIndex < ByteCode->NumCallables(); CallableIndex++)
		{
			const FRigVMCallableInfo* Callable = ByteCode->GetCallable(CallableIndex);
			JsonCallables.push_back(ToJson(Callable));
		}
		Json["Callables"] = JsonCallables;
	}

	// Task 10: Parse Entries
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseEntries", "Parsing Entries..."));
	{
		inja::json JsonEntries = inja::json::array();
		for (int32 EntryIndex = 0; EntryIndex < ByteCode->NumEntries(); EntryIndex++)
		{
			const FRigVMByteCodeEntry& Entry = ByteCode->GetEntry(EntryIndex);
			inja::json JsonEntry = inja::json::object();
			JsonEntry["Name"] = ToJson(Entry.Name.ToString());
			JsonEntry["SanitizedName"] = ToJson(Entry.GetSanitizedName());
			const int32 FirstIndex = Entry.InstructionIndex;
			int32 LastIndex = FirstIndex;
			for (; LastIndex < Instructions.Num(); LastIndex++)
			{
				if (Instructions[LastIndex].OpCode == ERigVMOpCode::Exit)
				{
					break;
				}
			}
			{
				inja::json JsonInstructions = inja::json::array();
				for (int32 InstructionIndex = FirstIndex; InstructionIndex <= LastIndex; InstructionIndex++)
				{
					if (CallableIndexPerInstruction[InstructionIndex] == INDEX_NONE &&
						BlockIndexPerInstruction[InstructionIndex] == INDEX_NONE)
					{
						JsonInstructions.push_back(InstructionIndex);
					}
				}
				JsonEntry["Instructions"] = JsonInstructions;
			}
			JsonEntry["LazyBlocks"] = GetLazyBlocks(FirstIndex, LastIndex, INDEX_NONE, INDEX_NONE);
			JsonEntries.push_back(JsonEntry);
		}
		Json["Entries"] = JsonEntries;
	}

	// Task 11: Parse Instructions
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseInstructions", "Parsing Instructions..."));
	Json["Instructions"] = ToJson(Instructions, 0, Instructions.Num() - 1);

	// Task 12: Parse Lazy Properties
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseLazyProperties", "Parsing Lazy Properties..."));
	{
		inja::json JsonLazyProperties = inja::json::array();
		for (const TTuple<FString,FString>& Pair : LazyProperties)
		{
			inja::json JsonLazyProperty = inja::json::object();
			JsonLazyProperty["Name"] = ToJson(Pair.Get<0>());
			JsonLazyProperty["NativeType"] = ToJson(Pair.Get<1>());
			JsonLazyProperties.push_back(JsonLazyProperty);
		}
		Json["LazyProperties"] = JsonLazyProperties;
	}

	// Task 13: Parse Includes
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseIncludes", "Parsing Includes..."));
	{
		inja::json JsonIncludes = inja::json::array();
		for ( const FString& Include : Includes)
		{
			JsonIncludes.push_back(ToJson(Include));
		}
		Json["CPlusPlusIncludes"] = JsonIncludes;
	}

	// Task 14: Parse Library Dependencies
	if (SlowTask.ShouldCancel())
	{
		return;
	}
	SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeConverter", "ParseLibraries", "Parsing Library Dependencies..."));
	{
		inja::json JsonLibraries = inja::json::array();
		for ( const FString& Library : Libraries)
		{
			JsonLibraries.push_back(ToJson(Library));
		}
		Json["Libraries"] = JsonLibraries;
	}
}

FString FRigVMCodeConverter::GetAssetName() const
{
	check(Json["AssetName"].is_string());
	return FromJson(Json["AssetName"]);
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMGraphVariableDescription& InVariable)
{
	inja::json JsonVariable = inja::json::object();
	
	JsonVariable["Name"] = ToJson(SanitizeName(InVariable.Name.ToString()));
	JsonVariable["OriginalName"] = ToJson(InVariable.Name.ToString());
	if (!InVariable.Category.IsEmpty())
	{
		JsonVariable["Category"] = ToJson(InVariable.Category);
	}
	if (!InVariable.Tooltip.IsEmpty())
	{
		JsonVariable["Tooltip"] = ToJson(InVariable.Tooltip);
	}
	JsonVariable["NativeType"] = ToJson(SanitizeNativeType(InVariable.CPPType));
	if (InVariable.CPPTypeObject)
	{
		JsonVariable["NativePath"] = GetNativePath(InVariable.CPPTypeObject);
		if (!ProcessDependency(InVariable.CPPTypeObject))
		{
			return nullptr;
		}
	}
	JsonVariable["IsPublic"] = InVariable.bPublic;
	JsonVariable["ExposedOnSpawn"] = InVariable.bExposedOnSpawn;
	JsonVariable["ExposeToCinematics"] = InVariable.bExposeToCinematics;
	JsonVariable["DefaultValue"] = ToJson(InVariable.DefaultValue);
	
	return JsonVariable;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMFunction* InFunction)
{
	inja::json JsonFunction = inja::json::object();

	int32 FunctionIndex;
	if (const int32* ExistingFunctionIndex = FunctionIndexMap.Find(InFunction))
	{
		FunctionIndex = *ExistingFunctionIndex;
	}
	else
	{
		FunctionIndex = FunctionIndexMap.Num();
		FunctionIndexMap.Add(InFunction, FunctionIndex);
	}

	JsonFunction["Index"] = FunctionIndex;
	JsonFunction["Name"] = ToJson(InFunction->Name);

	if (!ProcessDependency(InFunction->Struct))
	{
		return nullptr;
	}
	
	if (InFunction->Factory)
	{
		if (!ProcessDependency(InFunction->Factory->GetScriptStruct()))
		{
			return nullptr;
		}
		JsonFunction["NativeType"] = ToJson(SanitizeNativeType(InFunction->Factory->GetScriptStruct()->GetStructCPPName()));
		JsonFunction["NativePath"] = GetNativePath(InFunction->Factory->GetScriptStruct());
		JsonFunction["IsDispatch"] = true;
		FString FactoryName = InFunction->Factory->GetFactoryName().ToString();
		JsonFunction["FactoryName"] = ToJson(FactoryName);
		FactoryName.RemoveFromStart(FRigVMDispatchFactory::DispatchPrefix);
		JsonFunction["DispatchName"] = ToJson(FString::Printf(TEXT("Invoke_%s_%03d"), *FactoryName, InFunction->Index));
	}
	else
	{
		if (!ensureMsgf(InFunction->Struct, TEXT("FRigVMCodeConverter: Function '%s' has no Struct"), *InFunction->Name))
		{
			bHasError = true;
			return nullptr;
		}
		JsonFunction["NativeType"] = ToJson(SanitizeNativeType(InFunction->Struct->GetStructCPPName()));
		JsonFunction["NativePath"] = GetNativePath(InFunction->Struct);
		JsonFunction["Method"] = ToJson(InFunction->GetMethodName());
		JsonFunction["IsDispatch"] = false;
	}

	{
		inja::json JsonArguments = inja::json::array();
		for (int32 ArgumentIndex = 0; ArgumentIndex < InFunction->Arguments.Num(); ++ArgumentIndex)
		{
			const FRigVMFunctionArgument& Argument = InFunction->Arguments[ArgumentIndex];
			
			inja::json JsonArgument = inja::json::object();
			JsonArgument["Name"] = ToJson(SanitizeName(Argument.Name));
			JsonArgument["NativeType"] = ToJson(SanitizeNativeType(Argument.Type));
			if (RigVMTypeUtils::IsArrayType(Argument.Type))
			{
				JsonArgument["BaseNativeType"] = ToJson(SanitizeNativeType(RigVMTypeUtils::BaseTypeFromArrayType(Argument.Type)));
			}
			else
			{
				JsonArgument["BaseNativeType"] = JsonArgument["NativeType"];
			}
			JsonArgument["Direction"] = ToJson<ERigVMPinDirection>(static_cast<int64>(Argument.Direction));

#if WITH_EDITOR
			const bool bIsFixedSizeArray = InFunction->IsFixedSizeArray(Argument.Name);
			JsonArgument["IsFixedSizeArray"] = bIsFixedSizeArray;
			
			const bool bIsLazyValue = InFunction->IsLazyValue(Argument.Name);
			JsonArgument["IsLazyValue"] = bIsLazyValue;
			if (bIsLazyValue)
			{
				LazyFunctionArguments.Add({InFunction->Index, ArgumentIndex});
			}
#endif
			JsonArguments.push_back(JsonArgument);
		}
		JsonFunction["Arguments"] = JsonArguments;
	}

	return JsonFunction;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMBranchInfo& InBlock)
{
	check(ByteCode);

	inja::json JsonBlock = inja::json::object();
	JsonBlock["Index"] = InBlock.Index;
	if (!InBlock.Label.IsNone())
	{
		JsonBlock["Name"] = ToJson(InBlock.Label);
	}
	JsonBlock["BaseInstruction"] = InBlock.InstructionIndex;
	JsonBlock["FirstInstruction"] = InBlock.FirstInstruction;
	JsonBlock["LastInstruction"] = InBlock.LastInstruction;
	JsonBlock["ArgumentIndex"] = InBlock.ArgumentIndex;

	if (Instructions.IsValidIndex(InBlock.InstructionIndex))
	{
		if (Instructions[InBlock.InstructionIndex].OpCode == ERigVMOpCode::RunInstructions)
		{
			JsonBlock["NeedsToCheckExecuteState"] = true; 
		}
	}

	int32 BlockCallableIndex = INDEX_NONE;
	if (InBlock.Label != FRigVMStruct::ControlFlowCompletedName)
	{
		for (int32 CallableIndex = 0; CallableIndex < ByteCode->NumCallables(); ++CallableIndex)
		{
			const FRigVMCallableInfo* Callable = ByteCode->GetCallable(CallableIndex);
			if (Callable->FirstInstruction <= InBlock.FirstInstruction && Callable->LastInstruction >= InBlock.LastInstruction)
			{
				BlockCallableIndex = Callable->Index;
				break;
			}
		}
	}
	JsonBlock["CallableIndex"] = BlockCallableIndex;

	{
		inja::json JsonInstructions = inja::json::array();
		if (InBlock.InstructionIndex != InBlock.FirstInstruction)
		{
			for (int32 InstructionIndex = InBlock.FirstInstruction; InstructionIndex <= FMath::Max(InBlock.FirstInstruction, InBlock.LastInstruction); InstructionIndex++)
			{
				JsonInstructions.push_back(InstructionIndex);
			}
		}
		JsonBlock["Instructions"] = JsonInstructions;
		if (InBlock.Label == FRigVMStruct::ControlFlowCompletedName)
		{
			JsonBlock["LazyBlocks"] = inja::json::array();
		}
		else
		{
			JsonBlock["LazyBlocks"] = GetLazyBlocks(InBlock.FirstInstruction, FMath::Max(InBlock.FirstInstruction, InBlock.LastInstruction), BlockCallableIndex, InBlock.Index);
		}
	}

	return JsonBlock;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMPredicateBranch& InPredicateBranch)
{
	check(ByteCode);

	inja::json JsonBranch = inja::json::object();
	JsonBranch["Block"] = ToJson(InPredicateBranch.BranchInfo);
	return JsonBranch;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMCallableInfo* InCallable)
{
	check(ByteCode);

	inja::json JsonCallable = inja::json::object();
	JsonCallable["Index"] = InCallable->Index;
	JsonCallable["FunctionHash"] = InCallable->FunctionHash;
	JsonCallable["Name"] = ToJson(InCallable->Name);
	FString ShortName = InCallable->Name.ToString();
	FString FunctionName;
	if (ShortName.Split(TEXT(":"), nullptr, &FunctionName, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		ShortName = FunctionName;
	}
	JsonCallable["ShortName"] = ToJson(SanitizeName(ShortName));
	JsonCallable["FirstInstruction"] = InCallable->FirstInstruction;
	JsonCallable["LastInstruction"] = InCallable->LastInstruction;

	inja::json JsonArguments = inja::json::array();
	for (int32 ArgumentIndex = 0; ArgumentIndex < InCallable->Arguments.Num(); ArgumentIndex++)
	{
		const FRigVMCallableArgument& Argument = InCallable->Arguments[ArgumentIndex];
		JsonArguments.push_back(ToJson(Argument));
	}
	JsonCallable["Arguments"] = JsonArguments;

	{
		inja::json JsonInstructions = inja::json::array();
		for (int32 InstructionIndex = InCallable->FirstInstruction; InstructionIndex <= InCallable->LastInstruction; InstructionIndex++)
		{
			if (BlockIndexPerInstruction[InstructionIndex] == -1)
			{
				JsonInstructions.push_back(InstructionIndex);
			}
		}
		JsonCallable["Instructions"] = JsonInstructions;
		JsonCallable["LazyBlocks"] = GetLazyBlocks(InCallable->FirstInstruction, InCallable->LastInstruction, InCallable->Index, INDEX_NONE);
	}
	
	return JsonCallable;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMCallableArgument& InCallableArgument)
{
	inja::json JsonArgument = inja::json::object();
	JsonArgument["Name"] = ToJson(SanitizeName(InCallableArgument.Name.ToString()));
	JsonArgument["NativeType"] = ToJson(SanitizeNativeType(InCallableArgument.TypeString));
	JsonArgument["IsInput"] = !InCallableArgument.IsOutput(); // we have to do this to treat IO as Outputs
	JsonArgument["Direction"] = ToJson<ERigVMPinDirection>(static_cast<int64>(InCallableArgument.Direction));
	JsonArgument["Operand"] = ToJson(InCallableArgument.InterfaceOperand, InCallableArgument.Direction, INDEX_NONE);
	return JsonArgument;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMInstructionArray& InInstructions, int32 InFirstIndex, int32 InLastIndex)
{
	check(ByteCode);
	inja::json JsonInstructions = inja::json::array();
	for (int32 Index = InFirstIndex; Index <= InLastIndex; Index++)
	{
		inja::json JsonInstruction = ToJson(InInstructions[Index]);
		JsonInstruction["BlockIndex"] = BlockIndexPerInstruction[Index];
		const int32& CallableIndex = CallableIndexPerInstruction[Index];
		JsonInstruction["CallableIndex"] = CallableIndex;
		if (CallableIndex != INDEX_NONE)
		{
			if (const FRigVMCallableInfo* Callable = ByteCode->GetCallable(CallableIndex))
			{
				if (Callable->FirstInstruction == Index)
				{
					JsonInstruction["FirstInstructionInCallable"] = true;
				}
			}
		}
		JsonInstructions.push_back(JsonInstruction);
	}
	return JsonInstructions;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMInstruction& InInstruction)
{
	check(ByteCode);
	inja::json JsonInstruction = inja::json::object();
	JsonInstruction["Index"] = InInstruction.Index;
	JsonInstruction["Type"] = ToJson<ERigVMOpCode>((int64)InInstruction.OpCode);

	const FRigVMFunction* Function = nullptr;
	TArray<ERigVMPinDirection> Directions;
	bool bSkipOperands = false;

	switch (InInstruction.OpCode)
	{
		case ERigVMOpCode::Zero:
		case ERigVMOpCode::BoolFalse:
		case ERigVMOpCode::BoolTrue:
		case ERigVMOpCode::Increment:
		case ERigVMOpCode::Decrement:
		case ERigVMOpCode::Equals:
		case ERigVMOpCode::NotEquals:
		case ERigVMOpCode::Exit:
		case ERigVMOpCode::BeginBlock:
		case ERigVMOpCode::EndBlock:
		case ERigVMOpCode::ArrayReset:
		case ERigVMOpCode::ArrayGetNum:
		case ERigVMOpCode::ArraySetNum:
		case ERigVMOpCode::ArrayGetAtIndex:
		case ERigVMOpCode::ArraySetAtIndex:
		case ERigVMOpCode::ArrayAdd:
		case ERigVMOpCode::ArrayInsert:
		case ERigVMOpCode::ArrayRemove:
		case ERigVMOpCode::ArrayFind:
		case ERigVMOpCode::ArrayAppend:
		case ERigVMOpCode::ArrayClone:
		case ERigVMOpCode::ArrayIterator:
		case ERigVMOpCode::ArrayUnion:
		case ERigVMOpCode::ArrayDifference:
		case ERigVMOpCode::ArrayIntersection:
		case ERigVMOpCode::ArrayReverse:
		case ERigVMOpCode::Copy:
		{
			// some instructions don't need any additional info
			break;
		}
		case ERigVMOpCode::Execute:
		{
			const FRigVMExecuteOp& Op = ByteCode->GetOpAt<FRigVMExecuteOp>(InInstruction);
			Function = VM->GetFunctions()[Op.CallableIndex];

			if (const TSharedPtr<FRigVMFunction>* CustomFunctionPtr = InstructionToCustomFunction.Find(InInstruction.Index))
			{
				Function = CustomFunctionPtr->Get();
			}

			const int32 FunctionIndex = FunctionIndexMap.FindChecked(Function);
			JsonInstruction["FunctionIndex"] = FunctionIndex;

#if WITH_EDITOR
			Directions.Reserve(Function->Arguments.Num());
			for (const FRigVMFunctionArgument& Argument : Function->Arguments)
			{
				bool bInput = false, bVisible = false, bOutput = false, bHidden = false, bSingleton = false;
				if (Function->Factory)
				{
					bInput = Function->Factory->HasArgumentMetaData(Argument.Name, FRigVMStruct::InputMetaName);
					bVisible = Function->Factory->HasArgumentMetaData(Argument.Name, FRigVMStruct::VisibleMetaName);
					bOutput = Function->Factory->HasArgumentMetaData(Argument.Name, FRigVMStruct::OutputMetaName);
					bHidden = Function->Factory->HasArgumentMetaData(Argument.Name, FRigVMStruct::HiddenMetaName) && (!bInput && !bVisible && !bOutput);
					bSingleton = Function->Factory->HasArgumentMetaData(Argument.Name, FRigVMStruct::SingletonMetaName);
				}
				else
				{
					if (Function->Struct)
					{
						const FProperty* Property = Function->Struct->FindPropertyByName(Argument.Name);
					 	bInput = Property->HasMetaData(FRigVMStruct::InputMetaName);
					 	bVisible = Property->HasMetaData(FRigVMStruct::VisibleMetaName);
					 	bOutput = Property->HasMetaData(FRigVMStruct::OutputMetaName);
					 	bHidden = Property->HasMetaData(FRigVMStruct::HiddenMetaName) || (!bInput && !bVisible && !bOutput);
					 	bSingleton = Property->HasMetaData(FRigVMStruct::SingletonMetaName);
					}
				}

				if (bVisible)
				{
					Directions.Add(ERigVMPinDirection::Visible);
				}
				else if (bInput && !bOutput)
				{
					Directions.Add(ERigVMPinDirection::Input);
				}
				else if (bInput && bOutput)
				{
					Directions.Add(ERigVMPinDirection::IO);
				}
				else if (!bInput && bOutput)
				{
					Directions.Add(ERigVMPinDirection::Output);
				}
				else if (bHidden && !bSingleton)
				{
					Directions.Add(ERigVMPinDirection::Hidden);
				}
				else
				{
					Directions.Add(ERigVMPinDirection::Invalid);
				}
			}

			if (Function->Factory == nullptr && Function->Struct != nullptr)
			{
				const FStructOnScope StructOnScope(Function->Struct);
				const FRigVMStruct* StructInstance = (const FRigVMStruct*)StructOnScope.GetStructMemory();
				if (StructInstance->IsControlFlowNode())
				{
					JsonInstruction["IsControlFlowNode"] = StructInstance->IsControlFlowNode();
				}
			}
			if (Function->Factory)
			{
				FRigVMDispatchContext DispatchContext;
				FRigVMTemplateTypeMap TypeMap;
				JsonInstruction["IsControlFlowNode"] = Function->Factory->IsControlFlowDispatch(DispatchContext, TypeMap);
			}
#endif
			break;
		}
		case ERigVMOpCode::JumpForward:
		case ERigVMOpCode::JumpBackward:
		{
			const FRigVMJumpOp& Op = ByteCode->GetOpAt<FRigVMJumpOp>(InInstruction);
			JsonInstruction["TargetOffset"] = Op.InstructionIndex;
			JsonInstruction["TargetIndex"] = FRigVMJumpOp::GetTargetInstruction(InInstruction.OpCode, InInstruction.Index, Op.InstructionIndex);
			break;
		}
		case ERigVMOpCode::InvokeEntry:
		{
			const FRigVMInvokeEntryOp& Op = ByteCode->GetOpAt<FRigVMInvokeEntryOp>(InInstruction);
			JsonInstruction["Entry"] = ToJson(Op.EntryName);
			break;
		}
		case ERigVMOpCode::JumpToBranch:
		{
			const FRigVMJumpToBranchOp& Op = ByteCode->GetOpAt<FRigVMJumpToBranchOp>(InInstruction);
			{
				inja::json JsonBlocks = inja::json::array();
				for (int32 BlockIndex = Op.FirstBranchInfoIndex; BlockIndex < Blocks.Num(); BlockIndex++)
				{
					const FRigVMBranchInfo& Block = Blocks[BlockIndex];
					if (Block.InstructionIndex != InInstruction.Index)
					{
						break;
					}
					JsonBlocks.push_back(ToJson(Block));
				}
				JsonInstruction["Blocks"] = JsonBlocks;
			}
			break;
		}
		case ERigVMOpCode::RunInstructions:
		{
			bSkipOperands = true;
				
			const FRigVMRunInstructionsOp& Op = ByteCode->GetOpAt<FRigVMRunInstructionsOp>(InInstruction);
			if (Op.StartInstruction > Op.EndInstruction)
			{
				break;
			}
			const int32 BlockIndex = RunInstructionBranches.FindChecked({Op.StartInstruction, Op.EndInstruction});
			JsonInstruction["Block"] = ToJson(Blocks[BlockIndex]);
			break;
		}
		case ERigVMOpCode::SetupTraits:
		{
			checkNoEntry(); // not yet implemented
			break;
		}
		case ERigVMOpCode::InvokeCallable:
		{
			const FRigVMInvokeCallableOp& Op = ByteCode->GetOpAt<FRigVMInvokeCallableOp>(InInstruction);
			const FRigVMCallableInfo* Callable = ByteCode->GetCallable(Op.CallableIndex);
			JsonInstruction["CallableIndexToInvoke"] = Callable->Index;

			Directions.Reserve(Callable->Arguments.Num());
			for (const FRigVMCallableArgument& Argument : Callable->Arguments)
			{
				Directions.Add(Argument.Direction);
			}
			break;
		}
		case ERigVMOpCode::ChangeType: // not used by the compiler
		case ERigVMOpCode::JumpAbsolute: // not used by the compiler
		case ERigVMOpCode::JumpAbsoluteIf: // not used by the compiler
		case ERigVMOpCode::JumpForwardIf: // not used by the compiler
		case ERigVMOpCode::JumpBackwardIf: // not used by the compiler
		default:
		{
			// unsupported opcodes 
			checkNoEntry();
			break;
		}
	}

	if (bSkipOperands)
	{
		JsonInstruction["Operands"] = inja::json::array();
	}
	else
	{
		const FRigVMOperandArray& Operands = ByteCode->GetOperandsForOp(InInstruction);
		
		if (Directions.IsEmpty())
		{
			Directions.Reserve(Operands.Num());
			for (int32 OperandIndex = 0; OperandIndex < Operands.Num(); OperandIndex++)
			{
				Directions.Add(ERigVMPinDirection::Output);
			}
		}

		TSharedPtr<FRigVMRegistryReadLock> ReadLock;
		FRigVMRegistryHandle* Registry = nullptr;
		if (VM->LocalizedRegistry.IsValid())
		{
			Registry = &VM->LocalizedRegistry->GetHandle_NoLock();
		}
		else
		{
			ReadLock = MakeShared<FRigVMRegistryReadLock>();
			Registry = ReadLock.Get();
		}

		inja::json JsonOperands = inja::json::array();
		for (int32 OperandIndex = 0; OperandIndex < Operands.Num(); OperandIndex++)
		{
			const ERigVMPinDirection Direction = Directions.IsValidIndex(OperandIndex) ? Directions[OperandIndex] : ERigVMPinDirection::IO;
			inja::json JsonOperand = ToJson(Operands[OperandIndex], Direction, InInstruction.Index);
			if (Function)
			{
				int32 ArgumentIndex = OperandIndex;
				if (Function->Factory)
				{
					const FName ArgumentName = Function->Factory->GetArgumentNameForOperandIndex(OperandIndex, Operands.Num(), *Registry);
					ArgumentIndex = Function->GetArgumentIndex(ArgumentName);
					if (ArgumentIndex == INDEX_NONE)
					{
						const FString ArgumentNameString = ArgumentName.ToString();
						FString ArgumentNamePrefix, ArgumentNameSuffix;
						if (ArgumentNameString.Split(TEXT("_"), &ArgumentNamePrefix, &ArgumentNameSuffix, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
						{
							ArgumentIndex = Function->GetArgumentIndex(*ArgumentNamePrefix);
							if (ArgumentIndex != INDEX_NONE)
							{
								if (ArgumentNameSuffix.IsNumeric())
								{
									JsonOperand["FixedArrayIndex"] = FCString::Atoi(*ArgumentNameSuffix);
								}
								else
								{
									JsonOperand["FixedArrayLabel"] = ToJson(ArgumentNameSuffix);
								}
							}
						}
					}
					JsonOperand["ArgumentIndex"] = ArgumentIndex;
				}

				if (LazyFunctionArguments.Contains({Function->Index, ArgumentIndex}))
				{
					if (const FRigVMBranchInfo* LazyBlock = ByteCode->FindBranch(InInstruction.Index, OperandIndex))
					{
						JsonOperand["BlockIndex"] = LazyBlock->Index;
						JsonOperand["LazyValueName"] = ToJson(FString::Printf(TEXT("%s_%d_%d"), *LazyBlock->Label.ToString(), InInstruction.Index, OperandIndex));
					}
				}
			}
			JsonOperands.push_back(JsonOperand);
		}
		JsonInstruction["Operands"] = JsonOperands;
	}
	
	return JsonInstruction;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMMemoryStorageStruct& InMemoryStorageStruct)
{
	inja::json JsonMemory = inja::json::object();
	inja::json JsonProperties = inja::json::array();
	inja::json JsonPropertyPaths = inja::json::array();
	inja::json JsonPropertyOrder = inja::json::array();
	const int32 MemoryTypeIndex = (int32)InMemoryStorageStruct.GetMemoryType();

	const UPropertyBag* PropertyBag = InMemoryStorageStruct.GetPropertyBagStruct();
	if (!ensureMsgf(PropertyBag, TEXT("FRigVMCodeConverter: Invalid VM - PropertyBag is null. Cannot nativize.")))
	{
		bHasError = true;
		return nullptr;
	}

	const TConstArrayView<FPropertyBagPropertyDesc> Descriptions = PropertyBag->GetPropertyDescs();

	TArray<TTuple<int32,const FProperty*>> PropertiesBySize;
	PropertiesBySize.Reserve(Descriptions.Num());
	
	for (int32 DescriptionIndex = 0; DescriptionIndex < Descriptions.Num(); DescriptionIndex++)
	{
		const FPropertyBagPropertyDesc& Description = Descriptions[DescriptionIndex];
		const FProperty* Property = PropertyBag->FindPropertyByName(Description.Name);
		check(Property);

		// skip forwarded memory handles and execute states
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct == FRigVMForwardedMemoryHandle::StaticStruct())
			{
				continue;
			}
			if (StructProperty->Struct == FRigVMInstructionSetExecuteState::StaticStruct())
			{
				continue;
			}
		}
		const FString DefaultValue = InMemoryStorageStruct.GetDataAsStringByNameSafe(Description.Name);
		JsonProperties.push_back(ToJson(ERigVMPinDirection::Invalid, Property, DefaultValue, MemoryTypeIndex));
		const int32 NewPropertyIndex = PropertiesBySize.Num();
		PropertiesBySize.Emplace(NewPropertyIndex, Property);
		PropertyMap.Add({MemoryTypeIndex, DescriptionIndex}, NewPropertyIndex);
	}

	{
		Algo::Sort(PropertiesBySize, [](const TTuple<int32,const FProperty*>& A, const TTuple<int32,const FProperty*>& B) -> bool
		{
			return A.Get<1>()->GetSize() > B.Get<1>()->GetSize();
		});

		for (const TTuple<int32,const FProperty*>& Pair : PropertiesBySize)
		{
			JsonPropertyOrder.push_back(Pair.Get<0>());
		}
	}

	JsonMemory["Type"] = ToJson<ERigVMMemoryType>((int64)InMemoryStorageStruct.GetMemoryType());
	JsonMemory["Hash"] = InMemoryStorageStruct.GetMemoryHash();
	JsonMemory["Properties"] = JsonProperties;
	JsonMemory["PropertyOrder"] = JsonPropertyOrder;

	return JsonMemory;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMPropertyPath& InPropertyPath)
{
	inja::json JsonPropertyPath = inja::json::object();
	JsonPropertyPath["Name"] = ToJson(InPropertyPath.GetName());
	{
		FString ParentNativeType = InPropertyPath.GetHeadCPPType();
		inja::json JsonSegments = inja::json::array();
		for (const FRigVMPropertyPathSegment& Segment : InPropertyPath.GetSegments())
		{
			inja::json JsonSegment = inja::json::object();

			const FProperty* SegmentProperty = Segment.Property;
			
			switch (Segment.Type)
			{
				case ERigVMPropertyPathSegmentType::ArrayElement:
				{
					if (const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(SegmentProperty))
					{
						SegmentProperty = ArrayProperty->Inner;
					}
						
					JsonSegment["Index"] = Segment.Index;
					JsonSegment["Type"] = ToJson(FString(TEXT("Array Element")));
					break;
				}
				case ERigVMPropertyPathSegmentType::StructMember:
				{
					JsonSegment["Name"] = ToJson(Segment.Name);
					JsonSegment["Type"] = ToJson(FString(TEXT("Struct Member")));
					break;
				}
				case ERigVMPropertyPathSegmentType::MapValue:
				{
					JsonSegment["Name"] = ToJson(Segment.Name);
					JsonSegment["Type"] = ToJson(FString(TEXT("Map Value")));
					break;
				}
				default:
				{
					checkNoEntry();
					break;
				}
			}
			FString ExtendedNativeType;
			const FString NativeType = SegmentProperty->GetCPPType(&ExtendedNativeType);

			JsonSegment["NativeType"] = ToJson(SanitizeNativeType(NativeType + ExtendedNativeType));
			JsonSegment["ParentType"] = ToJson(ParentNativeType);
			JsonSegments.push_back(JsonSegment);
			ParentNativeType = NativeType + ExtendedNativeType;
		}
		JsonPropertyPath["Segments"] = JsonSegments;
	}
	return JsonPropertyPath;
}

inja::json FRigVMCodeConverter::ToJson(ERigVMPinDirection InDirection, const FProperty* InProperty, const FString& InDefaultValue, int32 InMemoryType)
{
	check(InProperty);
	
	inja::json JsonDescription = inja::json::object();

	const FString PropertyName = SanitizePropertyName(InProperty->GetName(), InMemoryType);
	JsonDescription["Name"] = ToJson(PropertyName);
	JsonDescription["OriginalName"] = ToJson(InProperty->GetName());

	const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty);
	const FProperty* InnerProperty = ArrayProperty ? ArrayProperty->Inner : InProperty;
	const FStructProperty* StructProperty = CastField<FStructProperty>(InnerProperty);
	const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InnerProperty);

	FString ExtendedType;
	const FString BaseType = InProperty->GetCPPType(&ExtendedType);
	JsonDescription["NativeType"] = ToJson(SanitizeNativeType(BaseType + ExtendedType));
	GetNativePathForProperty(InProperty, JsonDescription);
	JsonDescription["IsArray"] = ArrayProperty != nullptr;
	JsonDescription["IsStruct"] = StructProperty != nullptr;
	JsonDescription["IsObject"] = ObjectProperty != nullptr;

	if (!InDefaultValue.IsEmpty())
	{
		JsonDescription["DefaultValue"] = ToJson(InDefaultValue);
	}

	if (InDirection != ERigVMPinDirection::Invalid)
	{
		JsonDescription["Direction"] = ToJson<ERigVMPinDirection>((int64)InDirection);
	}
	
	return JsonDescription;
}

inja::json FRigVMCodeConverter::ToJson(const FRigVMOperand& InOperand, const ERigVMPinDirection& InDirection, int32 InInstructionIndex)
{
	inja::json JsonOperand = inja::json::object();
	JsonOperand["Type"] = ToJson<ERigVMMemoryType>((int64)InOperand.GetMemoryType());;
	JsonOperand["NativeType"] = ToJson(SanitizeNativeType(GetNativeTypeForOperand(InOperand, InInstructionIndex)));
	const int32 MemoryTypeIndex = (int32)InOperand.GetMemoryType();
	if (const int32* RemappedRegisterIndex = PropertyMap.Find({MemoryTypeIndex, InOperand.GetRegisterIndex()}))
	{
		JsonOperand["Property"] = *RemappedRegisterIndex;
	}
	else if (const TTuple<int32,int32,FName>* CallableArgument = CallableArgumentMap.Find({MemoryTypeIndex, InOperand.GetRegisterIndex()}))
	{
		JsonOperand["CallableArgumentName"] = ToJson(SanitizeName(CallableArgument->Get<2>().ToString()));
	}
	else
	{
		// this is likely a forwarded handle in the original description - which means it relates itself to a function argument.
		checkNoEntry();
	}
	if (InOperand.GetRegisterOffset() != INDEX_NONE)
	{
		const FRigVMPropertyPath* PropertyPath = nullptr;
		switch (InOperand.GetMemoryType())
		{
			case ERigVMMemoryType::Literal:
			{
				PropertyPath = &VM->GetDefaultLiteralMemory().GetPropertyPaths()[InOperand.GetRegisterOffset()];
				break;
			}
			case ERigVMMemoryType::Work:
			{
				PropertyPath = &VM->GetDefaultWorkMemory().GetPropertyPaths()[InOperand.GetRegisterOffset()];
				break;
			}
			case ERigVMMemoryType::External:
			{
				PropertyPath = &VM->ExternalPropertyPaths[InOperand.GetRegisterOffset()];
				break;
			}
			default:
			{
				// debug properties should not have any property paths
				checkNoEntry();
			}
		}

		check(PropertyPath);
		const uint32 Hash = GetPropertyPathHash(*PropertyPath);
		check(PropertyPathMap.Contains(Hash));
		JsonOperand["PropertyPath"] = PropertyPathMap.FindChecked(Hash);
	}
	JsonOperand["Direction"] = ToJson<ERigVMPinDirection>(static_cast<int64>(InDirection));
	return JsonOperand;
}

inja::json FRigVMCodeConverter::ToJson(const UEnum* InEnum, int64 InEnumValue, bool bTrimWhiteSpace)
{
	check(InEnum);
	const FText TextValue = InEnum->GetDisplayNameTextByValue(InEnumValue);
	check(!TextValue.IsEmpty());
	FString StringValue = TextValue.ToString();
	if (bTrimWhiteSpace)
	{
		StringValue.ReplaceInline(TEXT(" "), TEXT(""));
	}
	return ToJson(TextValue);
}

inja::json FRigVMCodeConverter::ToJson(const FString& InString)
{
	return StringCast<ANSICHAR>(*InString).Get();
}

inja::json FRigVMCodeConverter::ToJson(const FName& InName)
{
	return ToJson(InName.ToString());
}

inja::json FRigVMCodeConverter::ToJson(const FText& InText)
{
	return ToJson(InText.ToString());
}

FString FRigVMCodeConverter::FromJson(const inja::json& InJson)
{
	return FromJson(InJson.get<inja::json::string_t>());
}

FString FRigVMCodeConverter::FromJson(const std::string& InJson)
{
	//return (TCHAR*)StringCast<TCHAR>(InJson.c_str()).Get();
	return StringCast<TCHAR>(InJson.c_str()).Get();
}

inja::json FRigVMCodeConverter::GetLazyBlocks(int32 InFirstInstruction, int32 InLastInstruction, int32 CallableIndex, int32 BlockIndex)
{
	TSharedPtr<FRigVMRegistryReadLock> ReadLock;
	FRigVMRegistryHandle* Registry = nullptr;
	if (VM->LocalizedRegistry.IsValid())
	{
		Registry = &VM->LocalizedRegistry->GetHandle_NoLock();
	}
	else
	{
		ReadLock = MakeShared<FRigVMRegistryReadLock>();
		Registry = ReadLock.Get();
	}

	inja::json JsonLazyBlocks = inja::json::array();
	TSet<int32> UniqueBlockIndices;
	for (int32 InstructionIndex = InFirstInstruction; InstructionIndex <= InLastInstruction; InstructionIndex++)
	{
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if (Instruction.OpCode != ERigVMOpCode::Execute)
		{
			continue;
		}

		if (CallableIndexPerInstruction[InstructionIndex] != CallableIndex)
		{
			continue;
		}
		if (BlockIndexPerInstruction[InstructionIndex] != BlockIndex)
		{
			continue;
		}
		
		const FRigVMExecuteOp& ExecuteOp = ByteCode->GetOpAt<FRigVMExecuteOp>(Instruction);
		const FRigVMOperandArray& Operands = ByteCode->GetOperandsForOp(Instruction);
		const FRigVMFunction* Function = VM->GetFunctions()[ExecuteOp.CallableIndex];
		check(Function->Arguments.Num() <= Operands.Num());

		for (int32 ArgumentIndex = 0; ArgumentIndex < Function->Arguments.Num(); ArgumentIndex++)
		{
			if (LazyFunctionArguments.Contains({Function->Index, ArgumentIndex}))
			{
				for (int32 OperandIndex = 0; OperandIndex < Operands.Num(); OperandIndex++)
				{
					if (const FRigVMBranchInfo* LazyBlock = ByteCode->FindBranch(Instruction.Index, OperandIndex))
					{
						if (!UniqueBlockIndices.Contains(LazyBlock->Index))
						{
							FString NativeType = Function->Arguments[ArgumentIndex].Type;
							if (NativeType.RemoveFromStart(TEXT("TRigVMLazyValue<")))
							{
								NativeType.RemoveFromEnd(TEXT(">"));
							}

							const FRigVMOperand& Operand = Operands[OperandIndex];

							FName ArgumentName = NAME_None;
							if (Function->Factory)
							{
								ArgumentName = Function->Factory->GetArgumentNameForOperandIndex(OperandIndex, Operands.Num(), *Registry);
								int32 ArgumentIndexFromOperand = Function->GetArgumentIndex(ArgumentName);
								if (ArgumentIndexFromOperand == INDEX_NONE)
								{
									const FString ArgumentNameString = ArgumentName.ToString();
									FString ArgumentNamePrefix, ArgumentNameSuffix;
									if (ArgumentNameString.Split(TEXT("_"), &ArgumentNamePrefix, &ArgumentNameSuffix, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
									{
										ArgumentName = *ArgumentNamePrefix;
										ArgumentIndexFromOperand = Function->GetArgumentIndex(*ArgumentNamePrefix);
									}
								}

								if (ArgumentIndexFromOperand != ArgumentIndex)
								{
									continue;
								}
							}
							else if (OperandIndex != ArgumentIndex)
							{
								continue;
							}
							else
							{
								ArgumentName = Function->Arguments[ArgumentIndex].Name;
							}
							
							const FString LazyValueName = FString::Printf(TEXT("%s_%d_%d"), *LazyBlock->Label.ToString(), Instruction.Index, OperandIndex);
							FString SanitizedNativeType = SanitizeNativeType(NativeType);
							if (Function->IsFixedSizeArray(ArgumentName))
							{
								SanitizedNativeType = RigVMTypeUtils::BaseTypeFromArrayType(SanitizedNativeType);
							}

							inja::json JsonLazyBlock = inja::json::object();
							JsonLazyBlock["BlockIndex"] = LazyBlock->Index;
							JsonLazyBlock["LazyValueName"] = ToJson(LazyValueName);
							JsonLazyBlock["NativeType"] = ToJson(SanitizedNativeType);
							JsonLazyBlock["Operand"] = ToJson(Operand, ERigVMPinDirection::Input, InstructionIndex);
							JsonLazyBlocks.push_back(JsonLazyBlock);

							LazyProperties.AddUnique({LazyValueName, SanitizedNativeType});
						}
						
						UniqueBlockIndices.Add(LazyBlock->Index);
					}
				}
			}
		}
	}
	return JsonLazyBlocks;
}

bool FRigVMCodeConverter::ProcessDependency(const UObject* InObject)
{
	if (!InObject)
	{
		return true;
	}
	if (!InObject->IsNative())
	{
		UE_LOGF(LogRigVMDeveloper, Error, "Unsupported non-native dependency: '%ls'", *InObject->GetPathName());
		bHasError = true;
		return false;
	}
	if (KnownDependencies.Contains(InObject))
	{
		return true;
	}
	KnownDependencies.Add(InObject);

	if (const UStruct* Struct = Cast<UStruct>(InObject))
	{
		if (Struct->IsChildOf(FRigVMStruct::StaticStruct()))
		{
			FString FunctionModuleName = Struct->GetPackage()->GetName();
			if (FunctionModuleName.Contains(TEXT("/")))
			{
				FunctionModuleName.Split(TEXT("/"), nullptr, &FunctionModuleName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			}
			Libraries.AddUnique(FunctionModuleName);

			static const FString PrivatePrefix = TEXT("Private/");
			static const FString PublicPrefix = TEXT("Public/");

			FString RelativeHeaderPath;
#if WITH_EDITOR
			Struct->GetStringMetaDataHierarchical(TEXT("ModuleRelativePath"), &RelativeHeaderPath);
#endif
			if(RelativeHeaderPath.StartsWith(PrivatePrefix))
			{
				ProcessInclude(RelativeHeaderPath.Mid(PrivatePrefix.Len()));
			}
			else if(RelativeHeaderPath.StartsWith(PublicPrefix))
			{
				ProcessInclude(RelativeHeaderPath.Mid(PublicPrefix.Len()));
			}
			else
			{
				ProcessInclude(FPaths::Combine(FunctionModuleName, RelativeHeaderPath));
			}
			return true;
		}

		if (Struct == URigVM::StaticClass() ||
			Struct == FRigVMByteCode::StaticStruct() || 
			Struct->IsChildOf(FRigVMExecutePin::StaticStruct()) ||
			Struct->IsChildOf(FRigVMBaseOp::StaticStruct()) ||
			Struct->IsChildOf(URigVMMemoryStorage::StaticClass()))
		{
			Libraries.AddUnique(TEXT("RigVM"));
			Includes.AddUnique(TEXT("RigVMCore/RigVMCore.h"));
			Includes.AddUnique(TEXT("RigVMModule.h"));
			return true;
		}		
	}

	return true;
}

void FRigVMCodeConverter::ProcessInclude(const FString& InInclude)
{
	// todo: clean up paths... make relative to this path
	Includes.AddUnique(InInclude);
}

FString FRigVMCodeConverter::SanitizeName(const FString& InString)
{
	FString Result = InString;

	for (int32 i = 0; i < Result.Len(); ++i)
	{
		TCHAR& C = Result[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') ||											 // _ anytime
			(FChar::IsDigit(C) && i > 0);							 // 0-9 anytime after the first char
		if (!bGoodChar)
		{
			C = '_';
		}
	}

	Result.ReplaceInline(TEXT(" "), TEXT("_"));
	while (Result.StartsWith(TEXT("_")))
	{
		Result.RightChopInline(1);
	}
	while (Result.EndsWith(TEXT("_")))
	{
		Result.LeftChopInline(1);
	}
	return Result;
}

FString FRigVMCodeConverter::SanitizeNativeType(const FString& InString) const
{
	FString Result = InString.Replace(TEXT(" "), TEXT(""));
	if (const FString* RemappedNativeType = NativeTypeMap.Find(Result))
	{
		return *RemappedNativeType;
	}
	return Result;
}

FString FRigVMCodeConverter::SanitizePropertyName(const FString& InString, int32 InMemoryType)
{
	FString Result = InString;
	Result.RemoveFromEnd(TEXT("__Const"));
	Result.RemoveFromStart(TEXT("RigVMModel___"));
	if (Result.StartsWith(TEXT("Hash")))
	{
		// for ex: Hash2058268237____AddVectorTwice_Entry_B
		int32 SeparatorIndex = INDEX_NONE;
		if (Result.FindChar(TEXT('_'), SeparatorIndex))
		{
			if (Result.Mid(4, SeparatorIndex - 4).IsNumeric())
			{
				Result.RightChopInline(SeparatorIndex+1);
			}
		}
	}

	const FString SanitizedResult = SanitizeName(Result);
	Result = SanitizedResult;

	int32 SuffixIndex = 0;
	while (UniquePropertyNames.Contains({InMemoryType, Result}))
	{
		Result = SanitizedResult + TEXT("_") + FString::FromInt(++SuffixIndex);
	}
	UniquePropertyNames.Add({InMemoryType, Result});

	return Result; 
}

uint32 FRigVMCodeConverter::GetPropertyPathHash(const FRigVMPropertyPath& InPropertyPath)
{
	uint32 Hash = GetTypeHash(InPropertyPath.GetHeadCPPType());
	for (const FRigVMPropertyPathSegment& Segment : InPropertyPath.GetSegments())
	{
		Hash = HashCombine(Hash, GetTypeHash(Segment.Name), GetTypeHash(Segment.Index));
	}
	return Hash;
}

void FRigVMCodeConverter::GetNativePathForProperty(const FProperty* InProperty, inja::json& OutJsonObject)
{
	check(InProperty);
	const FProperty* Property = InProperty;
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		Property = ArrayProperty->Inner;
	}
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		if (EnumProperty->GetEnum())
		{
			OutJsonObject["NativePath"] = GetNativePath(EnumProperty->GetEnum());
		}
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		if (ByteProperty->Enum)
		{
			OutJsonObject["NativePath"] = GetNativePath(ByteProperty->Enum);
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (StructProperty->Struct)
		{
			OutJsonObject["NativePath"] = GetNativePath(StructProperty->Struct);
		}
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		if (ObjectProperty->PropertyClass)
		{
			OutJsonObject["NativePath"] = GetNativePath(ObjectProperty->PropertyClass);
		}
	}
}

inja::json FRigVMCodeConverter::GetNativePath(const UObject* InObject)
{
	check(InObject);
	const FString PathName = InObject->GetPathName();
	if (!NativePathMap.Contains(PathName))
	{
		NativePathMap.Add(PathName, InObject);
	}
	return ToJson(PathName);
}

FString FRigVMCodeConverter::GetNativeTypeForOperand(const FRigVMOperand& InOperand, int32 InInstructionIndex) const
{
	FString CPPTypeString;
	if (InOperand.GetMemoryType() == ERigVMMemoryType::External)
	{
		if (InOperand.GetRegisterOffset() != INDEX_NONE)
		{
			const FRigVMPropertyPath& PropertyPath = VM->ExternalPropertyPaths[InOperand.GetRegisterOffset()];
			FString ExtendedType;
			FString BaseType = PropertyPath.GetTailProperty()->GetCPPType(&ExtendedType);
			CPPTypeString = BaseType + ExtendedType;
		}
		else
		{
			const TArray<FRigVMGraphVariableDescription> Variables = EditorAsset->GetAssetVariables();
			CPPTypeString = Variables[InOperand.GetRegisterIndex()].CPPType;
		}
	}
	else
	{
		const FProperty* Property;
		if (InOperand.GetRegisterOffset() != INDEX_NONE)
		{
			const FRigVMPropertyPath& PropertyPath = VM->GetDefaultMemoryByType(InOperand.GetMemoryType())->GetPropertyPaths()[InOperand.GetRegisterOffset()];
			Property = PropertyPath.GetTailProperty();
		}
		else
		{
			Property = VM->GetDefaultMemoryByType(InOperand.GetMemoryType())->GetProperty(InOperand.GetRegisterIndex());

			// we are within a callable - look up the callable for this instruction
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct == FRigVMForwardedMemoryHandle::StaticStruct())
				{
					if (InInstructionIndex != INDEX_NONE)
					{
						const FRigVMCallableInfo* CallableForInstruction = ByteCode->FindCallableForInstruction(InInstructionIndex);
						check(CallableForInstruction);

						for (int32 CallableArgumentIndex = 0; CallableArgumentIndex < CallableForInstruction->Arguments.Num(); CallableArgumentIndex++)
						{
							const FRigVMCallableArgument& CallableArgument = CallableForInstruction->Arguments[CallableArgumentIndex];
							if (CallableArgument.ForwardedOperand.GetMemoryType() == InOperand.GetMemoryType() &&
								CallableArgument.ForwardedOperand.GetRegisterIndex() == InOperand.GetRegisterIndex())
							{
								const FRigVMOperand& InterfaceOperand = CallableArgument.InterfaceOperand;
								Property = VM->GetDefaultMemoryByType(InterfaceOperand.GetMemoryType())->GetProperty(InterfaceOperand.GetRegisterIndex());
								break;
							}
						}
					}
				}
			}
		}
		
		FString ExtendedType;
		FString BaseType = Property->GetCPPType(&ExtendedType);
		CPPTypeString = BaseType + ExtendedType;
	}
	return CPPTypeString;
}

const UObject* FRigVMCodeConverter::FindObjectFromNativePath(const FString& InNativePath) const
{
	if(const UObject** ObjectPtr = NativePathMap.Find(InNativePath))
	{
		return *ObjectPtr;
	}
	
	const UObject* Object = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InNativePath);
	NativePathMap.Add(InNativePath, Object);
	return Object;
}

TSharedPtr<FRigVMCodeOutput> FRigVMCodeConverter::Render(const TSharedPtr<FRigVMCodeTemplate>& InTemplate)
{
	if(HasError())
	{
		return nullptr;
	}
	
	TSharedPtr<FRigVMCodeOutput> Output = MakeShared<FRigVMCodeOutput>();
	Output->Name = InTemplate->Name;
	Output->OutputFolder = Settings.OutputFolder;
	Output->FilePath = FPaths::Combine(Settings.OutputFolder, InTemplate->Name.Replace(TEXT("AssetName"), *GetAssetName()));
	InTemplate->LoadIfRequired();

	const FString TemplateFolder = FPaths::GetPath(InTemplate->FilePath);

	const std::string Content = StringCast<ANSICHAR>(*InTemplate->GetContent()).Get();
	const std::string_view ContentView(Content.c_str(), Content.length());

	FRigVMCodeEnvironment Environment(TemplateFolder, this);

#if PLATFORM_EXCEPTIONS_DISABLED
	Output->Content = FromJson(Environment.render(ContentView, Json));
#else
	try
	{
		Output->Content = FromJson(Environment.render(ContentView, Json));
	}
	catch (inja::RenderError& e)
	{
		Output->ErrorMessage = FString(FromJson(e.message));
		Output->Content.Reset();
		UE_LOGF(LogRigVMDeveloper, Error, "Unexpected render error: '%ls', line %d, column %d\n\n%ls", *Output->ErrorMessage, (int32)e.location.line, (int32)e.location.column, *FString(FromJson(e.templatecontent)));
	}
	catch (inja::InjaError& e)
	{
		Output->ErrorMessage = FString(FromJson(e.message));
		Output->Content.Reset();
		UE_LOGF(LogRigVMDeveloper, Error, "Unexpected template error: '%ls'", *Output->ErrorMessage);
	}
#endif

	if (Output->Content.IsSet() && Settings.bWriteFiles)
	{
		if (!Output->Save())
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Cannot write file '%ls'", *Output->FilePath);
		}
		Output->bSaved = true;
	}
	
	return Output;
}
