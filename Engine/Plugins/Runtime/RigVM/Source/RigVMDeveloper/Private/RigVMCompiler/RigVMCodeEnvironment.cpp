// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMCodeEnvironment.h"
#include "RigVMCompiler/RigVMCodeConverter.h"
#include "RigVMCore/RigVMMemoryStorageStruct.h"
#include "Misc/StringOutputDevice.h"
#include "Spatial/FastWinding.h"

FRigVMCodeEnvironment::FRigVMCodeEnvironment(const FString& InTemplateFolder, FRigVMCodeConverter* InConverter)
: inja::Environment(StringCast<ANSICHAR>(*InTemplateFolder).Get())
, Converter(InConverter)
{
	set_lstrip_blocks(false);
	set_trim_blocks(false);
	set_throw_at_missing_includes(true);

	add_callback("sanitize", 1, &FRigVMCodeEnvironment::sanitize);
	add_callback("starts_with", 2, &FRigVMCodeEnvironment::starts_with);
	add_callback("ends_with", 2, &FRigVMCodeEnvironment::ends_with);
	add_callback("remove_whitespace", 1, &FRigVMCodeEnvironment::remove_whitespace);
	add_callback("format_number", 2, &FRigVMCodeEnvironment::format_number);

	add_callback("tabs", 0, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) -> inja::json { return tabs(Renderer, Arguments); });
	add_void_callback("inc_tabs", 0, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) -> void { inc_tabs(Renderer, Arguments); });
	add_void_callback("dec_tabs", 0, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) -> void { dec_tabs(Renderer, Arguments); });

	add_callback("has_key", 2, &FRigVMCodeEnvironment::has_key);
	add_callback("count_key", 2, &FRigVMCodeEnvironment::count_key);

	add_callback("cpp_get_default_for_native_type", 1, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) { return cpp_get_default_for_native_type(Renderer, Arguments); });
	add_callback("cpp_get_initialize_code_for_property", 6, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) { return cpp_get_initialize_code_for_property(Renderer, Arguments); });
	add_callback("cpp_get_block_to_run_operand", 1, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) { return cpp_get_block_to_run_operand(Renderer, Arguments); });
	add_callback("cpp_is_default_value", 2, [this](inja::Renderer* Renderer, inja::Arguments& Arguments) { return cpp_is_default_value(Renderer, Arguments); });
}

inja::json FRigVMCodeEnvironment::sanitize(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("sanitize: Argument 0 is not a string.");
	}
	return ToJson(FRigVMCodeConverter::SanitizeName(FromJson(*InArguments[0])));
}

inja::json FRigVMCodeEnvironment::starts_with(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("starts_with: Argument 0 is not a string.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("starts_with: Argument 1 is not a string.");
	}

	const FString A = FromJson(*InArguments[0]);
	const FString B = FromJson(*InArguments[1]);
	return A.StartsWith(B);
}

inja::json FRigVMCodeEnvironment::ends_with(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("ends_with: Argument 0 is not a string.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("ends_with: Argument 1 is not a string.");
	}

	const FString A = FromJson(*InArguments[0]);
	const FString B = FromJson(*InArguments[1]);
	return A.EndsWith(B);
}

inja::json FRigVMCodeEnvironment::remove_whitespace(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("remove_whitespace: Argument 0 is not a string.");
	}
	FString Content = FromJson(*InArguments[0]);
	Content.ReplaceInline(TEXT(" "), TEXT(""));
	Content.ReplaceInline(TEXT("\t"), TEXT(""));
	return ToJson(Content);
}

inja::json FRigVMCodeEnvironment::format_number(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("format_number: Argument 0 is not a string.");
	}
	if (!InArguments[1]->is_number())
	{
		InRenderer->throw_renderer_error("format_number: Argument 1 is not a number.");
	}
	const FString Format = FromJson(*InArguments[0]);
	const auto FormatUtf8 = StringCast<ANSICHAR>(*Format);

	// Use a local lambda to handle snprintf with RAII via TArray
	auto FormatWithSnprintf = [&FormatUtf8](auto Value) -> FString
	{
		const int32 Size = snprintf(nullptr, 0, FormatUtf8.Get(), Value) + 1;
		TArray<char> Buffer;
		Buffer.SetNumUninitialized(Size);
		snprintf(Buffer.GetData(), Size, FormatUtf8.Get(), Value);
		return UTF8_TO_TCHAR(Buffer.GetData());
	};

	FString Result;
	if (InArguments[1]->is_number_integer())
	{
		Result = FormatWithSnprintf(InArguments[1]->get<int32>());
	}
	else if (InArguments[1]->is_number_unsigned())
	{
		Result = FormatWithSnprintf(InArguments[1]->get<uint32>());
	}
	else
	{
		// float
		Result = FormatWithSnprintf(InArguments[1]->get<float>());
	}
	return ToJson(Result);
}

inja::json FRigVMCodeEnvironment::tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	FString Result;
	for (int32 TabIndex = 0; TabIndex < Tabs; TabIndex++)
	{
		Result += TEXT("\t");
	}
	return ToJson(Result);
}

void FRigVMCodeEnvironment::inc_tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	Tabs++;
}

void FRigVMCodeEnvironment::dec_tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	Tabs--;
}

inja::json FRigVMCodeEnvironment::has_key(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_object())
	{
		InRenderer->throw_renderer_error("has_key: Argument 0 is not an object.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("has_key: Argument 1 is not a string.");
	}

	const inja::json::string_t Key = InArguments[1]->get<inja::json::string_t>();
	const inja::json& Element = *InArguments[0];
	if (!Element.contains(Key))
	{
		return false;
	}
	const inja::json& Value = Element[Key];
	if (Value.is_boolean())
	{
		if (!Value.get<bool>())
		{
			return false;
		}
	}
	return true;
}

inja::json FRigVMCodeEnvironment::count_key(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_array())
	{
		InRenderer->throw_renderer_error("count_if: Argument 0 is not an array.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("count_if: Argument 1 is not a string.");
	}

	const inja::json::string_t Key = InArguments[1]->get<inja::json::string_t>();
	int32 Count = 0;
	for (std::size_t Index = 0; Index < InArguments[0]->size(); Index++)
	{
		const inja::json& Element = InArguments[0]->at(Index);
		if (!Element.is_object())
		{
			continue;
		}
		if (!Element.contains(Key))
		{
			continue;
		}
		const inja::json& Value = Element[Key];
		if (Value.is_boolean())
		{
			if (!Value.get<bool>())
			{
				continue;
			}
		}
		Count++;
	}
	return Count;
}

inja::json FRigVMCodeEnvironment::cpp_get_default_for_native_type(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 1 is not a string.");
	}

	const FString Type = FromJson(*InArguments[0]);
	FString Default;

	// O(1) lookup for known native types
	if (const FString* FoundDefault = NativeTypeDefaults.Find(Type))
	{
		Default = *FoundDefault;
	}

	if (Default.IsEmpty() && Type.StartsWith(TEXT("E"), ESearchCase::CaseSensitive))
	{
		if (const UEnum* Enum = Converter->FindObjectFromNativePath<UEnum>(Type))
		{
			Default = FString::Printf(TEXT("(%s)%d"), *Type, IntCastChecked<int32>(Enum->GetValueByIndex(0)));
		}
	}
	if (Default.IsEmpty() && Type.StartsWith(TEXT("TEnumAsByte<"), ESearchCase::CaseSensitive))
	{
		const FString EnumType = Type.RightChop(12).LeftChop(1);
		if (const UEnum* Enum = Converter->FindObjectFromNativePath<UEnum>(EnumType))
		{
			Default = FString::Printf(TEXT("(%s)%d"), *EnumType, IntCastChecked<int32>(Enum->GetValueByIndex(0)));
		}
	}
	if (Default.IsEmpty() && Type.EndsWith(TEXT("*"), ESearchCase::CaseSensitive))
	{
		Default = TEXT("nullptr");
	}
	if (Default.IsEmpty() && Type.StartsWith(TEXT("TObjectPtr<"), ESearchCase::CaseSensitive))
	{
		Default = TEXT("nullptr");
	}
	return ToJson(Default);
}

inja::json FRigVMCodeEnvironment::cpp_get_initialize_code_for_property(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 1 is not a string.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 2 is not a string.");
	}
	if (!InArguments[2]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 3 is not a string.");
	}
	if (!InArguments[3]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 4 is not a string.");
	}
	if (!InArguments[4]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 5 is not a string.");
	}
	if (!InArguments[5]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_get_default_for_native_type: Argument 6 is not a string.");
	}

	const FString MemoryTypeString = FromJson(*InArguments[0]);
	static const UEnum* MemoryTypeEnum = StaticEnum<ERigVMMemoryType>();
	ERigVMMemoryType MemoryType = (ERigVMMemoryType)MemoryTypeEnum->GetValueByNameString(MemoryTypeString); 
	
	const FString PropertyName = FromJson(*InArguments[1]);
	const FString OriginalPropertyName = FromJson(*InArguments[2]);
	const FString NativeType = FromJson(*InArguments[3]);
	const FString NativePath = FromJson(*InArguments[4]);
	const FString DefaultValue = FromJson(*InArguments[5]);

	inja::json JsonLines = inja::json::array();
	if (DefaultValue.IsEmpty())
	{
		return JsonLines;
	}

	const FProperty* Property = nullptr;
	if (Converter && Converter->VM)
	{
		if (MemoryType == ERigVMMemoryType::Literal || MemoryType == ERigVMMemoryType::Work)
		{
			const FRigVMMemoryStorageStruct* Memory = Converter->VM->GetDefaultMemoryByType(MemoryType);
			Property = Memory->FindPropertyByName(*OriginalPropertyName);
		}
	}

	struct Local
	{
		static bool ProcessSimpleType(const FString& InPath, const FString& InNativeType, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			if (InNativeType == TEXT("FName"))
			{
				if (InDefaultValue.IsEmpty() || InDefaultValue.Equals(TEXT("None"), ESearchCase::IgnoreCase))
				{
					OutLines.Add(FString::Printf(TEXT("%s = NAME_None;"), *InPath));
				}
				else
				{
					FString DefaultValue = InDefaultValue;
					DefaultValue.RemoveFromStart(TEXT("\""));
					DefaultValue.RemoveFromEnd(TEXT("\""));
					OutLines.Add(FString::Printf(TEXT("%s = TEXT(\"%s\");"), *InPath, *DefaultValue));
				}
				return true;
			}
			if (InDefaultValue.IsEmpty())
			{
				return false;
			}
			if (InNativeType == TEXT("FString"))
			{
				FString DefaultValue = InDefaultValue;
				DefaultValue.RemoveFromStart(TEXT("\""));
				DefaultValue.RemoveFromEnd(TEXT("\""));
				OutLines.Add(FString::Printf(TEXT("%s = TEXT(\"%s\");"), *InPath, *DefaultValue));
				return true;
			}
			if (InNativeType == TEXT("bool"))
			{
				const bool bValue = InDefaultValue.Equals(TEXT("true"), ESearchCase::IgnoreCase) || InDefaultValue.Equals(TEXT("1"), ESearchCase::IgnoreCase);
				OutLines.Add(FString::Printf(TEXT("%s = %s;"), *InPath, bValue ? TEXT("true") : TEXT("false")));
				return true;
			}
			if (IntegerTypes.Contains(InNativeType) || InNativeType == TEXT("double"))
			{
				OutLines.Add(FString::Printf(TEXT("%s = %s;"), *InPath, *InDefaultValue));
				return true;
			}
			if (InNativeType == TEXT("float"))
			{
				OutLines.Add(FString::Printf(TEXT("%s = %sf;"), *InPath, *InDefaultValue));
				return true;
			}

			// after this we'll only process structs. nothing to do if the struct is at the default
			if (InDefaultValue.Equals(TEXT("()"), ESearchCase::IgnoreCase))
			{
				return true;
			}
			
			if (InNativeType == TEXT("FVector2D") || InNativeType == TEXT("FVector") || InNativeType == TEXT("FQuat") || InNativeType == TEXT("FRotator"))
			{
				FString DefaultValue = InDefaultValue;
				DefaultValue.ReplaceInline(TEXT("PITCH="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("YAW="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("ROLL="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("X="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("Y="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("Z="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("W="), TEXT(""), ESearchCase::IgnoreCase);
				OutLines.Add(FString::Printf(TEXT("%s = %s%s;"), *InPath, *InNativeType, *DefaultValue));
				return true;
			}
			if (InNativeType == TEXT("FTransform"))
			{
				TArray<FString> DefaultValues = RigVMStringUtils::SplitDefaultValue(InDefaultValue);
				check(DefaultValues.Num() == 3);
				for (FString& DefaultValue : DefaultValues)
				{
					DefaultValue.ReplaceInline(TEXT("X="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("Y="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("Z="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("W="), TEXT(""), ESearchCase::IgnoreCase);
				}
				check(DefaultValues[0].StartsWith(TEXT("Rotation="), ESearchCase::IgnoreCase));
				check(DefaultValues[1].StartsWith(TEXT("Translation="), ESearchCase::IgnoreCase));
				check(DefaultValues[2].StartsWith(TEXT("Scale3D="), ESearchCase::IgnoreCase));
				DefaultValues[0].RightChopInline(9);
				DefaultValues[1].RightChopInline(12);
				DefaultValues[2].RightChopInline(8);
				OutLines.Add(FString::Printf(TEXT("%s = %s(FQuat%s, FVector%s, FVector%s);"), *InPath, *InNativeType, *DefaultValues[0], *DefaultValues[1], *DefaultValues[2]));
				return true;
			}
			if (InNativeType == TEXT("FMatrix"))
			{
				TArray<FString> DefaultValues = RigVMStringUtils::SplitDefaultValue(InDefaultValue);
				check(DefaultValues.Num() == 4);
				for (FString& DefaultValue : DefaultValues)
				{
					if (DefaultValue.Mid(1, 6).Equals(TEXT("Plane=")))
					{
						TArray<FString> PerMatrixDefaultValues = RigVMStringUtils::SplitDefaultValue(DefaultValue.Mid(7));
						if (PerMatrixDefaultValues.Num() == 4)
						{
							if (PerMatrixDefaultValues[0].StartsWith(TEXT("W="), ESearchCase::IgnoreCase))
							{
								const FString WDefaultValue = PerMatrixDefaultValues[0];
								PerMatrixDefaultValues.RemoveAt(0);
								PerMatrixDefaultValues.Add(WDefaultValue);
								DefaultValue = DefaultValue.Left(7) + RigVMStringUtils::JoinDefaultValue(PerMatrixDefaultValues);
							}
						}
					}
					DefaultValue.ReplaceInline(TEXT("X="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("Y="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("Z="), TEXT(""), ESearchCase::IgnoreCase);
					DefaultValue.ReplaceInline(TEXT("W="), TEXT(""), ESearchCase::IgnoreCase);
				}
				check(DefaultValues[0].StartsWith(TEXT("XPlane="), ESearchCase::IgnoreCase));
				check(DefaultValues[1].StartsWith(TEXT("YPlane="), ESearchCase::IgnoreCase));
				check(DefaultValues[2].StartsWith(TEXT("ZPlane="), ESearchCase::IgnoreCase));
				check(DefaultValues[3].StartsWith(TEXT("WPlane="), ESearchCase::IgnoreCase));
				DefaultValues[0].RightChopInline(7);
				DefaultValues[1].RightChopInline(7);
				DefaultValues[2].RightChopInline(7);
				DefaultValues[3].RightChopInline(7);
				OutLines.Add(FString::Printf(TEXT("%s = %s(FPlane%s, FPlane%s, FPlane%s, FPlane%s);"), *InPath, *InNativeType, *DefaultValues[0], *DefaultValues[1], *DefaultValues[2], *DefaultValues[3]));
				return true;
			}
			if (InNativeType == TEXT("FLinearColor"))
			{
				FString DefaultValue = InDefaultValue;
				DefaultValue.ReplaceInline(TEXT("R="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("G="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("B="), TEXT(""), ESearchCase::IgnoreCase);
				DefaultValue.ReplaceInline(TEXT("A="), TEXT(""), ESearchCase::IgnoreCase);
				OutLines.Add(FString::Printf(TEXT("%s = %s%s;"), *InPath, *InNativeType, *DefaultValue));
				return true;
			}

			return false;
		}

		static bool ProcessEnum(const FString& InPath, const FString& InNativeType, const FString& InNativePath, const UEnum* InEnum, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			if(InDefaultValue.IsEmpty())
			{
				return false;
			}
			
			const UEnum* Enum = InEnum;
			if (!Enum)
			{
				
				Enum = InConverter->FindObjectFromNativePath<UEnum>(InNativePath);
				if (!Enum)
				{
					return false;
				}
			}

			if (!Enum->IsNative())
			{
				return false;
			}

			FString FullName = Enum->GenerateFullEnumName(*InDefaultValue);
			OutLines.Add(FString::Printf(TEXT("%s = %s;"), *InPath, *FullName));
			return true;
		}

		static bool ProcessStruct(const FString& InPath, const FString& InNativeType, const FString& InNativePath, const UScriptStruct* InStruct, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			if (InDefaultValue.IsEmpty())
			{
				return false;
			}

			if (ProcessSimpleType(InPath, InNativeType, InDefaultValue, InConverter, OutLines))
			{
				return true;
			}
			
			const UScriptStruct* Struct = InStruct;
			if (!Struct)
			{
				Struct = InConverter->FindObjectFromNativePath<UScriptStruct>(InNativePath);
				if (!Struct)
				{
					return false;
				}
			}

			const TArray<FString> DefaultValues = RigVMStringUtils::SplitDefaultValue(InDefaultValue);
			bool bSuccess = true;
			for (const FString& DefaultValue : DefaultValues)
			{
				FString MemberName, MemberDefaultValue;
				if (!DefaultValue.Split(TEXT("="), &MemberName, &MemberDefaultValue))
				{
					continue;
				}
				const FProperty* MemberProperty = Struct->FindPropertyByName(*MemberName);
				if (!MemberProperty)
				{
					continue;
				}

				FString ExtendedType;
				FString MemberNativeType = MemberProperty->GetCPPType(&ExtendedType);
				
				if (!ProcessProperty(InPath + TEXT(".") + MemberName, MemberNativeType + ExtendedType, FString(), MemberProperty, MemberDefaultValue, InConverter, OutLines))
				{
					bSuccess = false;
				}
			}

			return bSuccess;
		}

		static bool ProcessClass(const FString& InPath, const FString& InNativeType, const FString& InNativePath, const UClass* InClass, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			OutLines.Add(FString::Printf(TEXT("%s = nullptr;"), *InPath));
			return true;
		}

		static bool ProcessArray(const FString& InPath, const FString& InNativeType, const FString& InNativePath, const FArrayProperty* InProperty, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			if (InDefaultValue.IsEmpty())
			{
				OutLines.Add(FString::Printf(TEXT("%s.Reset();"), *InPath));
				return true;
			}
			const TArray<FString> DefaultValues = RigVMStringUtils::SplitDefaultValue(InDefaultValue);
			OutLines.Add(FString::Printf(TEXT("%s.SetNum(%d);"), *InPath, DefaultValues.Num()));

			const FString ElementNativeType = RigVMTypeUtils::BaseTypeFromArrayType(InNativeType);
			bool bSuccess = true;
			for (int32 Index = 0; Index < DefaultValues.Num(); Index++)
			{
				if (!ProcessProperty(InPath + FString::Printf(TEXT("[%d]"), Index), ElementNativeType, InNativePath, InProperty ? InProperty->Inner : nullptr, DefaultValues[Index], InConverter, OutLines))
				{
					bSuccess = false;
				}
			}
			return bSuccess;
		}

		static bool ProcessProperty(const FString& InPath, const FString& InNativeType, const FString& InNativePath, const FProperty* InProperty, const FString& InDefaultValue, const FRigVMCodeConverter* InConverter, TArray<FString>& OutLines)
		{
			if (ProcessSimpleType(InPath, InNativeType, InDefaultValue, InConverter ,OutLines))
			{
				return true;
			}
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				return ProcessStruct(InPath, InNativeType, InNativePath, StructProperty->Struct, InDefaultValue, InConverter, OutLines);
			}
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
			{
				return ProcessArray(InPath, InNativeType, InNativePath, ArrayProperty, InDefaultValue, InConverter, OutLines);
			}
			if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
			{
				return ProcessEnum(InPath, InNativeType, InNativePath, EnumProperty->GetEnum(), InDefaultValue, InConverter, OutLines);
			}
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
			{
				return ProcessEnum(InPath, InNativeType, InNativePath, ByteProperty->Enum, InDefaultValue, InConverter, OutLines);
			}
			if (InNativeType.StartsWith(TEXT("TArray<")))
			{
				return ProcessArray(InPath, InNativeType, InNativePath, nullptr, InDefaultValue, InConverter, OutLines);
			}
			const UObject* CPPTypeObject = InConverter->FindObjectFromNativePath(InNativePath);
			if (const UClass* Class = Cast<UClass>(CPPTypeObject))
			{
				return ProcessClass(InPath, InNativeType, InNativePath, Class, InDefaultValue, InConverter, OutLines);
			}
			if (const UScriptStruct* Struct = Cast<UScriptStruct>(CPPTypeObject))
			{
				return ProcessStruct(InPath, InNativeType, InNativePath, Struct, InDefaultValue, InConverter, OutLines);
			}
			if (const UEnum* Enum = Cast<UEnum>(CPPTypeObject))
			{
				return ProcessEnum(InPath, InNativeType, InNativePath, Enum, InDefaultValue, InConverter, OutLines);
			}
			return false;
		}
	};

	TArray<FString> Lines;
	const FString Path = FString::Printf(TEXT("%s->%s"), *MemoryTypeString, *PropertyName);
	(void)Local::ProcessProperty(Path, NativeType, NativePath, Property, DefaultValue, Converter, Lines);

	for (const FString& Line : Lines)
	{
		JsonLines.push_back(ToJson(Line));
	}
	return JsonLines;
}

inja::json FRigVMCodeEnvironment::cpp_get_block_to_run_operand(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_object())
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand: Argument 1 is not an object.");
	}

	static inja::json IndexKey = ToJson(FString(TEXT("Index")));
	static inja::json OperandsKey = ToJson(FString(TEXT("Operands")));
	
	if (!InArguments[0]->contains(IndexKey))
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand: Argument 1 does not contain key 'Index'.");
	}
	if (!InArguments[0]->contains(OperandsKey))
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand: Argument 1 does not contain key 'Index'.");
	}

	inja::json JsonIndex = InArguments[0]->at(IndexKey);
	inja::json JsonOperands = InArguments[0]->at(OperandsKey);
	if (!JsonIndex.is_number_integer())
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand: Argument 1 contains non-integer 'Index'.");
	}
	if (!JsonOperands.is_array())
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand: Argument 1 contains non-array 'Operands'.");
	}

	const int32 InstructionIndex = (int32)JsonIndex.get<int>();
	if (!Converter->Instructions.IsValidIndex(InstructionIndex))
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand:: Invalid instruction index (out of bounds).");
	}

	const FRigVMInstruction& Instruction = Converter->Instructions[InstructionIndex];
	if (Instruction.OpCode != ERigVMOpCode::Execute)
	{
		InRenderer->throw_renderer_error("cpp_get_block_to_run_operand:: Cannot retrieve block to run for non-execute instructions.");
	}

	const FRigVMExecuteOp& ExecuteOp = Converter->ByteCode->GetOpAt<FRigVMExecuteOp>(Instruction);
	const FRigVMOperandArray& Operands = Converter->ByteCode->GetOperandsForOp(Instruction);
	const FRigVMFunction* Function = Converter->VM->GetFunctions()[ExecuteOp.CallableIndex];
	check(Function);
	check(Operands.Num() == Function->Arguments.Num());
	for (int32 ArgumentIndex = 0; ArgumentIndex < Operands.Num(); ++ArgumentIndex)
	{
		const FRigVMFunctionArgument& Argument = Function->Arguments[ArgumentIndex];
		if (Argument.Name == FRigVMStruct::ControlFlowBlockToRunName)
		{
			return Converter->ToJson(Operands[ArgumentIndex], ERigVMPinDirection::Input, InstructionIndex);
		}
	}

	InRenderer->throw_renderer_error("cpp_get_block_to_run_operand:: Cannot find block to run.");
	return inja::json::object();
}

inja::json FRigVMCodeEnvironment::cpp_is_default_value(inja::Renderer* InRenderer, inja::Arguments& InArguments)
{
	if (!InArguments[0]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_is_default_value: Argument 1 is not a string.");
	}
	if (!InArguments[1]->is_string())
	{
		InRenderer->throw_renderer_error("cpp_is_default_value: Argument 2 is not a string.");
	}

	const FString NativeType = FromJson(*InArguments[0]);
	const FString DefaultValue = FromJson(*InArguments[1]);

	if (NativeType == "bool")
	{
		return DefaultValue.Equals(TEXT("true"), ESearchCase::IgnoreCase);
	}
	if (NativeType.Equals(TEXT("float"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("double"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("uint8"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("uint16"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("uint32"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("uint64"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("int16"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("int32"), ESearchCase::IgnoreCase) || 
		NativeType.Equals(TEXT("int64"), ESearchCase::IgnoreCase))
	{
		FString DefaultValueFormatted = DefaultValue;
		DefaultValueFormatted.ReplaceInline(TEXT("."), TEXT(""));
		DefaultValueFormatted.ReplaceInline(TEXT("f"), TEXT(""));
		while (DefaultValueFormatted.StartsWith(TEXT("00")))
		{
			DefaultValueFormatted.RightChopInline(1);
		}
		return DefaultValueFormatted.Equals(TEXT("0"));
	}
	if (NativeType == "FName")
	{
		return DefaultValue.Equals(TEXT("NAME_None"), ESearchCase::IgnoreCase);
	}
	if (NativeType == "FString")
	{
		return DefaultValue.IsEmpty();
	}

	if (NativeStructTypeDefaults.Contains(NativeType))
	{
		UObject* NativeObject = nullptr;
		if (UObject** NativeObjectPtr = NativeTypeLookup.Find(NativeType))
		{
			NativeObject = *NativeObjectPtr;
		}
		else
		{
			FString UpdatedType = NativeType;
			NativeObject = RigVMTypeUtils::ObjectFromCPPType(UpdatedType);
			NativeTypeLookup.Add(NativeType, NativeObject);
		}
		if (!NativeObject)
		{
			return false;
		}

		if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(NativeObject))
		{
			if (UScriptStruct::ICppStructOps* CppStructOps = ScriptStruct->GetCppStructOps())
			{
				if (CppStructOps->HasIdentical())
				{
					const void* KnownDefaultValue = NativeStructTypeDefaults.FindChecked(NativeType);
					
					FStructOnScope InstanceToCompare(ScriptStruct);
					FStringOutputDevice Errors;
					ScriptStruct->ImportText(*DefaultValue, InstanceToCompare.GetStructMemory(), nullptr, 0, &Errors, ScriptStruct->GetName());

					if (Errors.IsEmpty())
					{
						bool bEqualsKnownDefault = false;
						if (CppStructOps->Identical(KnownDefaultValue, InstanceToCompare.GetStructMemory(), 0, bEqualsKnownDefault))
						{
							return bEqualsKnownDefault;
						}
					}
				}
			}
		}
	}
	
	// Empty default value is not considered a "default" - it means no value was specified
	if (DefaultValue.IsEmpty())
	{
		return false;
	}

	// Check for TArray types - empty arrays "()" are the default
	if (NativeType.StartsWith(TEXT("TArray<")))
	{
		return DefaultValue.Equals(TEXT("()"));
	}

	// For enum types, check if value is 0
	if (NativeType.StartsWith(TEXT("E"), ESearchCase::CaseSensitive) ||
		NativeType.StartsWith(TEXT("TEnumAsByte<"), ESearchCase::CaseSensitive))
	{
		if (DefaultValue.IsNumeric())
		{
			return FCString::Atoi64(*DefaultValue) == 0;
		}
	}

	return false;
}

inja::json FRigVMCodeEnvironment::ToJson(const FString& InString)
{
	return FRigVMCodeConverter::ToJson(InString);
}

inja::json FRigVMCodeEnvironment::ToJson(const FName& InName)
{
	return FRigVMCodeConverter::ToJson(InName);
}

inja::json FRigVMCodeEnvironment::ToJson(const FText& InText)
{
	return FRigVMCodeConverter::ToJson(InText);
}

FString FRigVMCodeEnvironment::FromJson(const inja::json& InJson)
{
	return FRigVMCodeConverter::FromJson(InJson);
}

FString FRigVMCodeEnvironment::FromJson(const std::string& InJson)
{
	return FRigVMCodeConverter::FromJson(InJson);
}
