// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMCodeGenerator.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/IConsoleManager.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "RigVMDeveloperModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Modules/ModuleManager.h"

FAutoConsoleCommand FCmdRigVMGenerateCode
(
	TEXT("RigVM.GenerateNativizedCode"),
	TEXT("Generates code for a given rigvm based asset."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		auto PrintHelp = []()
		{
			UE_LOGF(LogRigVMDeveloper, Warning, "Valid Arguments:\n"
				"asset=pathname - the asset to parse\n"
				"language=CPlusPlus - the language to generate (CPlusPlus or Verse)\n"
				"module=MyModule - the module name to use for the exports\n"
				"output=MyFolder - the [optional] folder to export the files to\n"
				"json=PathToMyJson - the [optional] filepath to save the json file to\n"
				"write=0 - the [optional] settings to enable writing out the files.");
		};
		
		if (Args.Num() < 1)
		{
			PrintHelp();
			return;
		}

		ERigVMCodeLanguage::Type Language = ERigVMCodeLanguage::CPlusPlus;
		
		FString AssetPathName;
		FRigVMCodeConversionSettings Settings;
		Settings.TargetModule = TEXT("MyModule"); 
	 	Settings.OutputFolder = TEXT("/Temp");
		Settings.bWriteFiles = false;

		for (const FString& Arg : Args)
		{
			if(Arg.Len() > 6 && Arg.StartsWith(TEXT("asset="), ESearchCase::IgnoreCase))
			{
				AssetPathName = Arg.Mid(6); // remove asset= from the start. 
			}
			else if(Arg.Len() > 7 && Arg.StartsWith(TEXT("module="), ESearchCase::IgnoreCase))
			{
				Settings.TargetModule = Arg.Mid(7); // remove asset= from the start. 
			}
			else if(Arg.Len() > 9 && Arg.StartsWith(TEXT("language="), ESearchCase::IgnoreCase))
			{
				const FString LanguageString = Arg.Mid(9); // remove language= from the start.
				if (LanguageString.Equals(TEXT("CPlusPlus"), ESearchCase::IgnoreCase))
				{
					Language = ERigVMCodeLanguage::CPlusPlus;
				}
				else if (LanguageString.Equals(TEXT("Verse"), ESearchCase::IgnoreCase))
				{
					Language = ERigVMCodeLanguage::Verse;
				}
				else
				{
					PrintHelp();
					return;
				}
			}
			else if(Arg.Len() > 7 && Arg.StartsWith(TEXT("output="), ESearchCase::IgnoreCase))
			{
				Settings.OutputFolder = Arg.Mid(7); // remove output= from the start.
				if (!FPaths::DirectoryExists(Settings.OutputFolder))
				{
					UE_LOGF(LogRigVMDeveloper, Warning, "Output Folder '%ls' does not exist", *Settings.OutputFolder);
					return;
				}
			}
			else if(Arg.Len() > 5 && Arg.StartsWith(TEXT("json="), ESearchCase::IgnoreCase))
			{
				Settings.JsonFilePath = Arg.Mid(5); // remove json= from the start.
			}
			else if(Arg.Len() > 6 && Arg.StartsWith(TEXT("write="), ESearchCase::IgnoreCase))
			{
				Settings.bWriteFiles =
					Arg.Mid(6).Equals(TEXT("1")) ||
					Arg.Mid(6).Equals(TEXT("true"), ESearchCase::IgnoreCase) ||
					Arg.Mid(6).Equals(TEXT("on"), ESearchCase::IgnoreCase); 
			}
		}

		if (AssetPathName.IsEmpty())
		{
			PrintHelp();
			return;
		}

		FScopedSlowTask SlowTask(3.f /* Number of steps */, FText());
		SlowTask.MakeDialog(true, false);
		SlowTask.EnterProgressFrame(0.f, NSLOCTEXT("RigVMCodeGenerator", "LoadingAsset", "Loading Asset..."));

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPathName));
		if (!AssetData.IsValid())
		{
			UE_LOGF(LogRigVMDeveloper, Error, "The AssetData '%ls' could not be found in the Asset Registry.", *AssetPathName);
			return;
		}

		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeGenerator", "ParsingAsset", "Parsing Asset..."));

		UObject* AssetObject = AssetData.GetAsset();
		IRigVMEditorAssetInterface* EditorAsset = Cast<IRigVMEditorAssetInterface>(AssetObject);
		if (!EditorAsset)
		{
			if (IRigVMRuntimeAssetInterface* RuntimeAsset = Cast<IRigVMRuntimeAssetInterface>(AssetObject))
			{
				EditorAsset = Cast<IRigVMEditorAssetInterface>(RuntimeAsset->GetEditorOnlyData());
			}
		}
		if (!EditorAsset)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "The Asset '%ls' is not a RigVM editor asset.", *AssetPathName);
			return;
		}

		const TSharedPtr<FRigVMCodeConverter> Converter = MakeShared<FRigVMCodeConverter>(EditorAsset, Settings);

		if (SlowTask.ShouldCancel())
		{
			return;
		}
		SlowTask.EnterProgressFrame(1.f, NSLOCTEXT("RigVMCodeGenerator", "GeneratingCode", "Generating Code..."));

		if (!Settings.JsonFilePath.IsEmpty())
		{
			const std::string path(StringCast<ANSICHAR>(*Settings.JsonFilePath).Get());
			std::ofstream json_output_file(path);
			json_output_file << Converter->GetJson().dump(4);
			UE_LOGF(LogRigVMDeveloper, Display, "Json data -> '%ls'", *Settings.JsonFilePath)
		}
			
		TArray<TSharedPtr<FRigVMCodeOutput>> Outputs = FRigVMCodeGenerator(Language).GenerateAll(Converter);
		for (const TSharedPtr<FRigVMCodeOutput>& Output : Outputs)
		{
			if (!Output.IsValid())
			{
				continue;
			}
			
			if (!Output->Content.IsSet())
			{
				UE_LOGF(LogRigVMDeveloper, Error, "Template '%ls' failed to generate (%ls).", *Output->Name, *Output->ErrorMessage)
			}
			else if (Output->bSaved.Get(false))
			{
				UE_LOGF(LogRigVMDeveloper, Display, "Template '%ls' -> '%ls'", *Output->Name, *Output->FilePath)
			}
			else
			{
				UE_LOGF(LogRigVMDeveloper, Display, "Template '%ls' -> '%ls'\n\n%ls", *Output->Name, *Output->FilePath, *Output->Content.GetValue())
			}
		}
	})
);

FRigVMCodeGenerator::FRigVMCodeGenerator(ERigVMCodeLanguage::Type InLanguage)
	: Language(InLanguage)
{
	FString LanguageName;
	switch (InLanguage)
	{
		case ERigVMCodeLanguage::CPlusPlus:
		{
			LanguageName = TEXT("CPlusPlus");
			break;
		}
		default:
		{
			return;
		}
	}

	const FString PluginContentDir = FPaths::EnginePluginsDir() / TEXT("Runtime/RigVM/Content");
	const FString TemplatesDir = FPaths::Combine(PluginContentDir, TEXT("CodeGeneration"), LanguageName);
	const FString TemplateSearchPath = FPaths::Combine(*TemplatesDir, TEXT("*.template"));
	TArray<FString> TemplateFileNames;
	FFileManagerGeneric::Get().FindFiles(TemplateFileNames, *TemplateSearchPath, true, false);

	for (const FString& TemplateFileName : TemplateFileNames)
	{
		TSharedPtr<FRigVMCodeTemplate> Template = MakeShared<FRigVMCodeTemplate>();
		Template->Name = TemplateFileName.LeftChop(9); // remove .template
		Template->FilePath = FPaths::Combine(TemplatesDir, TemplateFileName);
		CodeTemplates.Add(Template);
	}
}

int32 FRigVMCodeGenerator::NumTemplates() const
{
	return CodeTemplates.Num();
}

const TSharedPtr<FRigVMCodeTemplate>& FRigVMCodeGenerator::GetTemplate(const FString& InName) const
{
	for (const TSharedPtr<FRigVMCodeTemplate>& CodeTemplate : CodeTemplates)
	{
		if (CodeTemplate->Name.Equals(InName, ESearchCase::IgnoreCase))
		{
			return CodeTemplate;
		}
	}

	static TSharedPtr<FRigVMCodeTemplate> EmptyTemplate;
	return EmptyTemplate;
}

TSharedPtr<FRigVMCodeOutput> FRigVMCodeGenerator::Generate(const TSharedPtr<FRigVMCodeConverter>& InConverter, const FString& InTemplateName) const
{
	TSharedPtr<FRigVMCodeTemplate> CodeTemplate = GetTemplate(InTemplateName);
	if (!CodeTemplate)
	{
		UE_LOGF(LogRigVMDeveloper, Error, "The Code Template '%ls' cannot be found.", *InTemplateName);
		return nullptr;
	}
	return Generate(InConverter, CodeTemplate);
}

TSharedPtr<FRigVMCodeOutput> FRigVMCodeGenerator::Generate(const TSharedPtr<FRigVMCodeConverter>& InConverter, const TSharedPtr<FRigVMCodeTemplate>& InCodeTemplate)
{
	if (!InConverter.IsValid() || !InCodeTemplate.IsValid())
	{
		return nullptr;
	}
	return InConverter->Render(InCodeTemplate);
}

TArray<TSharedPtr<FRigVMCodeOutput>> FRigVMCodeGenerator::GenerateAll(const TSharedPtr<FRigVMCodeConverter>& InConverter) const
{
	FScopedSlowTask SlowTask(static_cast<float>(CodeTemplates.Num()), FText());

	TArray<TSharedPtr<FRigVMCodeOutput>> Outputs;
	for (const TSharedPtr<FRigVMCodeTemplate>& CodeTemplate : CodeTemplates)
	{
		const FString Message = FString::Printf(TEXT("Generating code for template '%s'..."), *CodeTemplate->Name);
		UE_LOGF(LogRigVMDeveloper, Display, "%ls", *Message);
		SlowTask.EnterProgressFrame(0.f, FText::FromString(Message));
		Outputs.Add(Generate(InConverter, CodeTemplate));
		SlowTask.EnterProgressFrame(1.f);
	}
	return Outputs;
}

#undef TOJSON
#undef FROMJSON
