// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMCodeTemplate.h"
#include "Misc/FileHelper.h"

void FRigVMCodeTemplate::Load()
{
	FString ContentString;
	if (FFileHelper::LoadFileToString(ContentString, *FilePath))
	{
		Content = ContentString;
	}
}

void FRigVMCodeTemplate::LoadIfRequired()
{
	if (Content.IsSet())
	{
		return;
	}
	Load();
}

const FString& FRigVMCodeTemplate::GetContent() const
{
	check(Content.IsSet());
	return Content.GetValue();
}

bool FRigVMCodeOutput::Save() const
{
	if (!Content.IsSet())
	{
		return false;
	}
	return FFileHelper::SaveStringToFile(GetContent(), *FilePath);
}
