// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMModel/Nodes/RigVMBranchNode.h"
#include "RigVMModel/Nodes/RigVMArrayNode.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMNativized.h"
#include "RigVMDeveloperModule.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Interface.h"
#include "Stats/StatsHierarchical.h"
#include "RigVMTypeUtils.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMFunctions/Execution/RigVMFunction_BreakLoop.h" // FRigVMFunction_BreakFromLoop — used for Break node detection in TraverseCallExtern
#include "RigVMFunctions/Execution/RigVMFunction_ContinueLoop.h" // FRigVMFunction_ContinueLoop — used for Continue node detection in TraverseCallExtern
#include "Algo/Sort.h"
#include "String/Join.h"
#include "RigVMDeveloperTypeUtils.h"

#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/AppStyle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMCompiler)

class FRigVMCompilerImportErrorContext : public FOutputDevice
{
public:

	FRigVMCompilerWorkData* WorkData;
	int32 NumErrors;

	FRigVMCompilerImportErrorContext(FRigVMCompilerWorkData* InWorkData)
		: FOutputDevice()
		, WorkData(InWorkData)
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		switch (Verbosity)
		{
		case ELogVerbosity::Error:
		case ELogVerbosity::Fatal:
		{
			WorkData->ReportError(V);
			break;
		}
		case ELogVerbosity::Warning:
		{
			WorkData->ReportWarning(V);
			break;
		}
		default:
		{
			WorkData->ReportInfo(V);
			break;
		}
		}
		NumErrors++;
	}
};

FAutoConsoleVariable CVarRigVMWarnAboutDeprecatedNodes(
	TEXT("RigVM.Compiler.WarnAboutDeprecatedNodes"),
	false,
	TEXT("Enable output of warnings when compiling nodes that are deprecated.")
);

FAutoConsoleVariable CVarRigVMWarnAboutDuplicateEvents(
	TEXT("RigVM.Compiler.WarnAboutDuplicateEvents"),
	false,
	TEXT("Enable output of warnings when compiling event nodes that are duplicated.")
);

FRigVMCompileSettings::FRigVMCompileSettings()
	: SurpressInfoMessages(true)
	, SurpressWarnings(false)
	, SurpressErrors(false)
	, EnablePinWatches(true)
	, IsPreprocessorPhase(false)
	, ASTSettings(FRigVMParserASTSettings::Optimized())
	, SetupNodeInstructionIndex(true)
	, ASTErrorsAsNotifications(false)
	, bWarnAboutDeprecatedNodes(false)
	, bWarnAboutDuplicateEvents(false)
{
}

FRigVMCompileSettings::FRigVMCompileSettings(UScriptStruct* InExecuteContextScriptStruct)
	: FRigVMCompileSettings()
{
	ASTSettings.ExecuteContextStruct = InExecuteContextScriptStruct;
	if(ASTSettings.ExecuteContextStruct == nullptr)
	{
		ASTSettings.ExecuteContextStruct = FRigVMExecuteContext::StaticStruct();
	}
}

void FRigVMCompileSettings::ReportInfo(const FString& InMessage) const
{
	if (SurpressInfoMessages)
	{
		return;
	}
	Report(EMessageSeverity::Info, nullptr, InMessage);
}

void FRigVMCompileSettings::ReportWarning(const FString& InMessage) const
{
	Report(EMessageSeverity::Warning, nullptr, InMessage);
}

void FRigVMCompileSettings::ReportError(const FString& InMessage) const
{
	Report(EMessageSeverity::Error, nullptr, InMessage);
}

FRigVMOperand FRigVMCompilerWorkData::AddProperty(
	ERigVMMemoryType InMemoryType,
	const FName& InName,
	const FString& InCPPType,
	UObject* InCPPTypeObject,
	const FString& InDefaultValue)
{
	check(CompilerPhase == ERigVMCompilerPhase_SetupMemory);

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	FRigVMTemplateArgumentType ArgumentType(*InCPPType, InCPPTypeObject);
	const TRigVMTypeIndex TypeIndex = Registry.GetTypeIndex(ArgumentType);
	if(TypeIndex != INDEX_NONE)
	{
		// for execute pins we should use the graph's default execute context struct
		if(Registry.IsExecuteType(TypeIndex))
		{
			ensure(!ArgumentType.IsArray());
			ArgumentType = FRigVMTemplateArgumentType(ExecuteContextStruct);
		}
	}

	
	FRigVMPropertyDescription Description(InName, ArgumentType.CPPType.ToString(), ArgumentType.CPPTypeObject, InDefaultValue);
	ensure(FindProperty(InMemoryType, Description.Name).IsValid() == false); // Warning : this check can not be done before the SanitizeName inside FRigVMPropertyDescription constructor

	TArray<FRigVMPropertyDescription>& PropertyArray = PropertyDescriptions.FindOrAdd(InMemoryType);
	const int32 PropertyIndex = PropertyArray.Add(Description);

	return FRigVMOperand(InMemoryType, PropertyIndex);
}

FRigVMOperand FRigVMCompilerWorkData::FindProperty(ERigVMMemoryType InMemoryType, const FName& InName) const
{
	const TArray<FRigVMPropertyDescription>* PropertyArray = PropertyDescriptions.Find(InMemoryType);
	if(PropertyArray)
	{
		for(int32 Index=0;Index<PropertyArray->Num();Index++)
		{
			if(PropertyArray->operator[](Index).Name == InName)
			{
				return FRigVMOperand(InMemoryType, Index);
			}
		}
	}
	return FRigVMOperand();
}

FRigVMOperand FRigVMCompilerWorkData::FindOrAddProperty(const ERigVMMemoryType InMemoryType, 
	const FRigVMFunctionCompilationPropertyDescription& InProperty, const FString& InNewName,
	const bool bMakeUniqueProperty)
{
	// Sharing / reusing memory / operands happens as per following contract:
	// 1. properties are only shared if their CPP type matches
	// 2. Literal / constant memory is only shared if the constant values match (and it is not a trait list)
	// 3. Work state is only shared if it is not internal work state private to the instruction referring to it
	// 4. Work state of type FRigVMInstructionSetExecuteState (bMakeUniqueProperty) is never shared either since it is work state private to a lazy branch.

	FString NewName = InNewName;
	int32 Suffix = 0;

	// share literals only when literal folding is globally enabled
	if(Settings.ASTSettings.bFoldLiterals && InMemoryType == ERigVMMemoryType::Literal && !bMakeUniqueProperty && PropertyDescriptions.Contains(ERigVMMemoryType::Literal))
	{
		const TArray<FRigVMPropertyDescription>& LiteralDescriptions = PropertyDescriptions[ERigVMMemoryType::Literal];
		for(int32 Index = 0; Index < LiteralDescriptions.Num(); Index++)
		{
			// skip registers owned by interactive nodes — sharing with them would cause DM
			// writes to bleed into nodes that are not currently being manipulated.
			if(InteractiveLiteralIndices.Contains(Index))
			{
				continue;
			}
			const FRigVMPropertyDescription& ExistingProperty = LiteralDescriptions[Index];
			if(ExistingProperty.CPPType.Equals(InProperty.CPPType, ESearchCase::IgnoreCase))
			{
				// if the value is the same and this is not a trait setup list
				if(ExistingProperty.DefaultValue.Equals(InProperty.DefaultValue, ESearchCase::CaseSensitive))
				{
					return FRigVMOperand(ERigVMMemoryType::Literal, Index);
				}
			}
		}
	}

	FRigVMPropertyDescription::SanitizeName(NewName);
	FRigVMOperand Operand = FindProperty(InMemoryType, *NewName);

	// look for a free / unused operand
	while(Operand.IsValid())
	{
		const FRigVMPropertyDescription& ExistingProperty = PropertyDescriptions[InMemoryType][Operand.GetRegisterIndex()];
		if(!bMakeUniqueProperty &&
			ExistingProperty.CPPType.Equals(InProperty.CPPType, ESearchCase::CaseSensitive))
		{
			if(InMemoryType == ERigVMMemoryType::Literal)
			{
				// if the value is the same and this is not a trait setup list
				if(ExistingProperty.DefaultValue.Equals(InProperty.DefaultValue))
				{
					return Operand;
				}
			}
			else
			{
				if(InMemoryType == ERigVMMemoryType::Work &&
					RigVMTypeUtils::IsArrayType(InProperty.CPPType))
				{
					// for now we don't share operands which have an array type
					// since they may represent internal state. multiple occurrences of
					// the same function cannot share internal work state.
				}
				else
				{
					return Operand;
				}
			}
		}

		NewName = FString::Printf(TEXT("%s_%02d"), *InNewName, ++Suffix);
		FRigVMPropertyDescription::SanitizeName(NewName);
		Operand = FindProperty(InMemoryType, *NewName);
	}
			
	// create a new operand as needed
	return AddProperty(InMemoryType, *NewName, InProperty.CPPType, InProperty.CPPTypeObject.Get(), InProperty.DefaultValue);
}

FRigVMOperand FRigVMCompilerWorkData::FindOrAddProperty(ERigVMMemoryType InMemoryType, const FName& InName, const FString& InCPPType,
	UObject* InCPPTypeObject, const FString& InDefaultValue, bool bMakeUniqueProperty)
{
	FRigVMFunctionCompilationPropertyDescription Description;
	Description.Name = InName;
	Description.CPPType = InCPPType;
	Description.CPPTypeObject = InCPPTypeObject;
	Description.DefaultValue = InDefaultValue;
	return FindOrAddProperty(InMemoryType, Description, InName.ToString(), bMakeUniqueProperty);
}

FRigVMPropertyDescription FRigVMCompilerWorkData::GetProperty(const FRigVMOperand& InOperand)
{
	TArray<FRigVMPropertyDescription>* PropertyArray = PropertyDescriptions.Find(InOperand.GetMemoryType());
	if(PropertyArray)
	{
		if(PropertyArray->IsValidIndex(InOperand.GetRegisterIndex()))
		{
			return PropertyArray->operator[](InOperand.GetRegisterIndex());
		}
	}
	return FRigVMPropertyDescription();
}

int32 FRigVMCompilerWorkData::FindOrAddPropertyPath(const FRigVMOperand& InOperand, const FString& InHeadCPPType, const FString& InSegmentPath)
{
	if(InSegmentPath.IsEmpty())
	{
		checkNoEntry();
		return INDEX_NONE;
	}

	TArray<FRigVMPropertyPathDescription>& Descriptions = PropertyPathDescriptions.FindOrAdd(InOperand.GetMemoryType());
	for(int32 Index = 0; Index < Descriptions.Num(); Index++)
	{
		const FRigVMPropertyPathDescription& Description = Descriptions[Index]; 
		if(Description.HeadCPPType == InHeadCPPType && Description.SegmentPath == InSegmentPath)
		{
			return Index;
		}
	}
	const int32 Index = Descriptions.Add(FRigVMPropertyPathDescription(InOperand.GetRegisterIndex(), InHeadCPPType, InSegmentPath));
	check(Descriptions.Last().IsValid());
	return Index;
}

const FProperty* FRigVMCompilerWorkData::GetPropertyForOperand(const FRigVMOperand& InOperand) const
{
	if(!InOperand.IsValid())
	{
		return nullptr;
	}
	
	check(CompilerPhase == ERigVMCompilerPhase_BuildInstructions);

	auto GetPropertyFromMemory = [this](FRigVMMemoryStorageStruct& InMemory, const FRigVMOperand& InOperand)
	{
		if(InOperand.GetRegisterOffset() == INDEX_NONE)
		{
			return  InMemory.GetProperty(InOperand.GetRegisterIndex());
		}
		if(!InMemory.GetPropertyPaths().IsValidIndex(InOperand.GetRegisterOffset()))
		{
			InMemory.SetPropertyPathDescriptions(PropertyPathDescriptions.FindChecked(InOperand.GetMemoryType()));
			InMemory.RefreshPropertyPaths();
		}
		return InMemory.GetPropertyPaths()[InOperand.GetRegisterOffset()].GetTailProperty();
	};

	const FProperty* Property = nullptr;
	switch(InOperand.GetMemoryType())
	{
	case ERigVMMemoryType::Literal:
		{
			Property = GetPropertyFromMemory(VM->GetDefaultLiteralMemory(), InOperand);
			break;
		}
	case ERigVMMemoryType::Work:
		{
			Property = GetPropertyFromMemory(VM->GetDefaultWorkMemory(), InOperand);
			break;
		}
	case ERigVMMemoryType::Debug:
		{
			// debug memory is not supported.
			break;
		}
	case ERigVMMemoryType::External:
		{
			Property = VM->GetExternalVariableDefs()[InOperand.GetRegisterIndex()].GetProperty();
			if(InOperand.GetRegisterOffset() != INDEX_NONE)
			{
				if(!VM->ExternalPropertyPaths.IsValidIndex(InOperand.GetRegisterOffset()))
				{
					VM->ExternalPropertyPathDescriptions = PropertyPathDescriptions.FindChecked(InOperand.GetMemoryType());
					VM->RefreshExternalPropertyPaths();
				}
				Property = VM->ExternalPropertyPaths[InOperand.GetRegisterOffset()].GetTailProperty();
			}
			break;
		}
	case ERigVMMemoryType::Invalid:
	default:
		{
			break;
		}
	}

	return Property;
}

TRigVMTypeIndex FRigVMCompilerWorkData::GetTypeIndexForOperand(const FRigVMOperand& InOperand) const
{
	const FProperty* Property = GetPropertyForOperand(InOperand);
	if(Property == nullptr)
	{
		if (InOperand.GetMemoryType() == ERigVMMemoryType::External)
		{
			const TArray<FRigVMExternalVariableDef>& ExternalVariables = VM->GetExternalVariableDefs();
			if (ExternalVariables.IsValidIndex(InOperand.GetRegisterIndex()))
			{
				const FRigVMExternalVariableDef& Variable = ExternalVariables[InOperand.GetRegisterIndex()];
				FString CPPType;
				UObject* CPPTypeObject;
				RigVMTypeUtils::CPPTypeFromExternalVariable(Variable, CPPType, &CPPTypeObject);
				return FRigVMRegistry::Get().GetTypeIndex(*CPPType, CPPTypeObject);
			}
		}
		return INDEX_NONE;
	}

	FName CPPTypeName(NAME_None);
	UObject* CPPTypeObject = nullptr;
	RigVMPropertyUtils::GetTypeFromProperty(Property, CPPTypeName, CPPTypeObject);

	return FRigVMRegistry::Get().GetTypeIndex(CPPTypeName, CPPTypeObject);
}

FName FRigVMCompilerWorkData::GetUniquePropertyName(ERigVMMemoryType InMemoryType, const FName& InDesiredName) const
{
	const FString Prefix = InDesiredName.ToString(); 
	FString Name =  Prefix;
	int32 Suffix = 1;
	while(FindProperty(ERigVMMemoryType::Literal, *Name).IsValid())
	{
		Name = FString::Printf(TEXT("%s_%d"), *Prefix, Suffix++);
	}
	return *Name;
}

bool FRigVMCompilerWorkData::AddPropertiesForFunction(
	bool InIsCallable,
	const uint32& InHash,
	const FRigVMGraphFunctionHeader& InHeader,
	const FRigVMFunctionCompilationData* InCompilationData,
	int32 InFirstInstruction, int32 InLastInstruction,
	const TArray<FRigVMOperand>& InInterfaceOperands,
	const TMap<FRigVMOperand, FRigVMOperand>& InForwardedHandleToInterfaceOperand,
	const URigVMFunctionReferenceNode* InFunctionReferenceNode)
{
	TMap<FRigVMOperand, FRigVMOperand>& OperandMap = OperandMapPerFunction.FindOrAdd(InHash, {});
	OperandMap.Reset();

	// Build a one-shot name -> index lookup over the parent VM's external variables, used
	// by the External-operand re-resolution path inside ProcessOperand. The rest of this
	// function (and ProcessOperand) dereferences VM unconditionally, so make the contract
	// explicit here rather than guarding only the map build.
	check(VM);
	TMap<FName, int32> ParentVMExternalNameToIndex;
	{
		const TArray<FRigVMExternalVariableDef>& Defs = VM->GetExternalVariableDefs();
		ParentVMExternalNameToIndex.Reserve(Defs.Num());
		for (int32 ExtIdx = 0; ExtIdx < Defs.Num(); ++ExtIdx)
		{
			ParentVMExternalNameToIndex.Add(Defs[ExtIdx].GetName(), ExtIdx);
		}
	}

	auto ProcessOperand = [this, InHash, InIsCallable, &InHeader, InCompilationData, &OperandMap, &InInterfaceOperands, &InForwardedHandleToInterfaceOperand, &ParentVMExternalNameToIndex, InFunctionReferenceNode]
	(const ERigVMOpCode& InOpCode, const FRigVMOperand& InOldOperand)
	{
		FRigVMOperand OldOperandNoOffset(InOldOperand.GetMemoryType(), InOldOperand.GetRegisterIndex());
		if (const FRigVMOperand* InterfaceOperandFromForwardedHandle = InForwardedHandleToInterfaceOperand.Find(OldOperandNoOffset))
		{
			OldOperandNoOffset = *InterfaceOperandFromForwardedHandle;
		}
		
		ERigVMMemoryType MemoryType = OldOperandNoOffset.GetMemoryType();

		const TArray<FRigVMFunctionCompilationPropertyDescription>* Properties = nullptr;
		switch (MemoryType)
		{
			case ERigVMMemoryType::Work:
			{
				Properties = &InCompilationData->WorkPropertyDescriptions;
				break;
			}
			case ERigVMMemoryType::Literal:
			{
				Properties = &InCompilationData->LiteralPropertyDescriptions;
				break;
			}
			case ERigVMMemoryType::External:
			{
				Properties = &InCompilationData->ExternalPropertyDescriptions;
				break;
			}
			case ERigVMMemoryType::Debug:
			{
				Properties = &InCompilationData->DebugPropertyDescriptions;
				break;
			}
			default:
			{
				checkNoEntry();
			}
		}

		int32 FirstValidIndex = 0;

		// if we don't contain any callables - we should skip over the first couple of operands
		if (!InIsCallable)
		{
			if (MemoryType == ERigVMMemoryType::Work)
			{
				for (const FRigVMGraphFunctionArgument& Argument : InHeader.Arguments)
				{
					if (Argument.IsCPPTypeObjectValid())
					{
						if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Argument.CPPTypeObject.Get()))
						{
							if (ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
							{
								continue;
							}
						}
					}
					if (!Properties->IsValidIndex(FirstValidIndex))
					{
						break;
					}
					const FString PropertyName = FRigVMPropertyDescription::SanitizeName(Argument.Name).ToString();
					if (!Properties->operator[](FirstValidIndex).Name.ToString().EndsWith(PropertyName))
					{
						continue;
					}
					FirstValidIndex++;
				}
			}
		}

		int32 PropertyIndex = OldOperandNoOffset.GetRegisterIndex();
		if ((Properties->IsValidIndex(PropertyIndex) || MemoryType == ERigVMMemoryType::External) && PropertyIndex >= FirstValidIndex)
		{
			static const FString FunctionLibraryPrefix = TEXT("FunctionLibrary");
			static const FString RigVMInstructionSetExecuteStateName = FRigVMInstructionSetExecuteState::StaticStruct()->GetStructCPPName();

			FRigVMOperand NewOperandNoOffset;
			if (const FRigVMOperand* ExistingNewOperandNoOffset = OperandMap.Find(OldOperandNoOffset))
			{
				NewOperandNoOffset = *ExistingNewOperandNoOffset;
				MemoryType = NewOperandNoOffset.GetMemoryType();
			}
			else
			{
				FRigVMFunctionCompilationPropertyDescription CustomDescription;
				const FRigVMFunctionCompilationPropertyDescription* Description = nullptr;

				// Set to the current parent-VM index when an External operand was re-resolved by
				// name; the operand emission below uses it instead of the stale function-side
				// index. INDEX_NONE means no remap was performed.
				int32 ResolvedExternalIndex = INDEX_NONE;

				// Set to true when the External fallback below gives up and synthesizes a
				// placeholder description (variable renamed/removed/retyped, or out-of-range
				// stale index). The operand emission consults this so the bytecode does not
				// end up referencing an out-of-range/wrong External register at runtime.
				bool bExternalPlaceholder = false;

				if (Properties && Properties->IsValidIndex(PropertyIndex))
				{
					Description = &Properties->operator[](PropertyIndex);
				}
				else
				{
					check(MemoryType == ERigVMMemoryType::External);

					// Try to recover the argument name via the function's InterfaceOperands map
					// (operand -> name) and then look it up in the header's argument list. We
					// cannot use VM->GetExternalVariableDefs()[PropertyIndex] directly because
					// the parent VM's external indices differ from the function's register indices.
					FName ArgumentName = NAME_None;
					for (const TPair<FName, FRigVMOperand>& InterfacePair : InCompilationData->InterfaceOperands)
					{
						if (InterfacePair.Value.GetMemoryType() == OldOperandNoOffset.GetMemoryType() &&
							InterfacePair.Value.GetRegisterIndex() == OldOperandNoOffset.GetRegisterIndex())
						{
							ArgumentName = InterfacePair.Key;
							break;
						}
					}

					const FRigVMGraphFunctionArgument* MatchingArgument = nullptr;
					if (!ArgumentName.IsNone())
					{
						for (const FRigVMGraphFunctionArgument& Argument : InHeader.Arguments)
						{
							if (Argument.Name == ArgumentName)
							{
								MatchingArgument = &Argument;
								break;
							}
						}
					}

					// Helper so the description-from-parent-VM-def write lives in one place.
					auto FillDescriptionFromExternalDef = [&CustomDescription](const FRigVMExternalVariableDef& Def)
					{
						CustomDescription.Name = Def.GetName();
						CustomDescription.CPPType = Def.GetExtendedCPPType().ToString();
						CustomDescription.CPPTypeObject = const_cast<UObject*>(Def.GetCPPTypeObject());
						CustomDescription.DefaultValue = FString();
					};

					if (MatchingArgument)
					{
						CustomDescription.Name = MatchingArgument->Name;
						CustomDescription.CPPType = MatchingArgument->CPPType.ToString();
						CustomDescription.CPPTypeObject = MatchingArgument->CPPTypeObject.Get();
						CustomDescription.DefaultValue = FString();
					}
					else
					{
						// Direct external-variable reads inside the function body record the
						// variable's register index relative to the rig layout at function compile
						// time. When the parent VM is later rebuilt with a different layout, look
						// the variable up by name in ExternalRegisterIndexToVariable and remember
						// the resolved parent-VM index for the operand emission below.
						auto FillPlaceholderDescription = [&CustomDescription, &bExternalPlaceholder]()
						{
							CustomDescription.Name = TEXT("InvalidExternal");
							CustomDescription.CPPType = TEXT("uint8");
							CustomDescription.CPPTypeObject = nullptr;
							CustomDescription.DefaultValue = FString();
							bExternalPlaceholder = true;
						};

						if (const FName* MappedName = InCompilationData->ExternalRegisterIndexToVariable.Find(PropertyIndex))
						{
							// When the function is referenced from a different rig (via a
							// URigVMFunctionReferenceNode), the inner variable name recorded by the
							// function may be bound to a different outer variable on the caller.
							// Translate via the function reference node's variable map before looking
							// the name up in the parent VM.
							FName LookupName = *MappedName;
							bool bRemapPathUnsupported = false;
							if (InFunctionReferenceNode)
							{
								const FString RemappedPath = InFunctionReferenceNode->GetOuterVariablePath(*MappedName);
								if (!RemappedPath.IsEmpty())
								{
									if (RemappedPath.Contains(TEXT(".")))
									{
										// Property-path remappings ("Outer.SubField") are resolved by the
										// inline path through PinPathToOperand, but the External fallback
										// here is a name-only TMap lookup against the parent VM's external
										// defs and cannot bind to a sub-field. Fall through to a
										// placeholder rather than silently using the inner name (which
										// could either fail-loudly with the wrong name in the diagnostic
										// or bind to a same-named outer variable and corrupt state).
										UE_LOGF(LogRigVMDeveloper, Error,
											"Function references external variable '%ls' remapped to property-path '%ls' "
											"on the caller. Property-path remaps for direct external-variable reads "
											"are not supported in the External fallback; emit the remap as a regular "
											"outer-variable name or expose the variable as an input-variable pin. "
											"(Func='%ls')",
											*MappedName->ToString(),
											*RemappedPath,
											*InHeader.LibraryPointer.GetLibraryNodePath());
										bRemapPathUnsupported = true;
									}
									else
									{
										LookupName = *RemappedPath;
									}
								}
							}
							const int32* FoundIdx = bRemapPathUnsupported ? nullptr : ParentVMExternalNameToIndex.Find(LookupName);
							if (bRemapPathUnsupported)
							{
								FillPlaceholderDescription();
							}
							else if (FoundIdx)
							{
								const FRigVMExternalVariableDef& ExternalVariableDef = VM->GetExternalVariableDefs()[*FoundIdx];

								// Refuse to bind if the function recorded a CPPType for this register
								// and it doesn't match the parent VM's current type — silently
								// emitting an operand against incompatible memory would corrupt
								// runtime state. Skip the check when no function-side type info is
								// available (the common case for direct rig-variable reads, where
								// ExternalPropertyDescriptions is empty).
								const FString FunctionSideCPPType =
									InCompilationData->ExternalPropertyDescriptions.IsValidIndex(PropertyIndex)
										? InCompilationData->ExternalPropertyDescriptions[PropertyIndex].CPPType
										: FString();
								const FString ParentVMCPPType = ExternalVariableDef.GetExtendedCPPType().ToString();

								if (FunctionSideCPPType.IsEmpty() || FunctionSideCPPType == ParentVMCPPType)
								{
									FillDescriptionFromExternalDef(ExternalVariableDef);
									ResolvedExternalIndex = *FoundIdx;
								}
								else
								{
									UE_LOGF(LogRigVMDeveloper, Error,
										"External variable '%ls' (mapped to '%ls') type changed from '%ls' to '%ls'; function "
										"compiled against the old type cannot be reused. (Func='%ls')",
										*MappedName->ToString(),
										*LookupName.ToString(),
										*FunctionSideCPPType,
										*ParentVMCPPType,
										*InHeader.LibraryPointer.GetLibraryNodePath());
									FillPlaceholderDescription();
								}
							}
							else
							{
								// Function explicitly named a variable that the parent rig no longer
								// has. Don't bind to a different variable at the stale index — that
								// would silently corrupt rig state. Synthesize a placeholder and let
								// the user fix their data.
								UE_LOGF(LogRigVMDeveloper, Error,
									"Function references external variable '%ls' (looked up as '%ls') which is not present on "
									"the parent rig (rig has %d external variables). (Func='%ls')",
									*MappedName->ToString(),
									*LookupName.ToString(),
									VM->GetExternalVariableDefs().Num(),
									*InHeader.LibraryPointer.GetLibraryNodePath());
								FillPlaceholderDescription();
							}
						}
						else
						{
							// No name was recorded for this register — likely stale function bytecode
							// from before ExternalRegisterIndexToVariable was populated. Best-effort
							// compat: bind to whatever the parent VM has at the same index, matching
							// the original pre-fix behaviour. If the index is also out of range we
							// have nothing safe to bind to and synthesize a placeholder.
							UE_LOGF(LogRigVMDeveloper, Warning,
								"Function operand references external register %d with no recorded "
								"variable name; compiled function data is stale. Falling back to the "
								"parent VM's variable at the same index. (Func='%ls')",
								PropertyIndex,
								*InHeader.LibraryPointer.GetLibraryNodePath());

							if (VM->GetExternalVariableDefs().IsValidIndex(PropertyIndex))
							{
								FillDescriptionFromExternalDef(VM->GetExternalVariableDefs()[PropertyIndex]);
							}
							else
							{
								UE_LOGF(LogRigVMDeveloper, Error,
									"...and the index is out of range in the parent VM (%d external "
									"variables). (Func='%ls')",
									VM->GetExternalVariableDefs().Num(),
									*InHeader.LibraryPointer.GetLibraryNodePath());
								FillPlaceholderDescription();
							}
						}
					}
					Description = &CustomDescription;

					// if this is an input argument - make sure to turn it into work state.
					if (InInterfaceOperands.Contains(OldOperandNoOffset))
					{
						MemoryType = ERigVMMemoryType::Work;
					}
				}

				// if this is a trait setup list - let's copy all traits over
				if(InOpCode == ERigVMOpCode::SetupTraits && MemoryType == ERigVMMemoryType::Literal)
				{
					FString OriginalTraitIndicesString = Properties->operator[](PropertyIndex).DefaultValue;
					if(!OriginalTraitIndicesString.IsEmpty() && OriginalTraitIndicesString != TEXT("()"))
					{
						OriginalTraitIndicesString = OriginalTraitIndicesString.TrimChar(TEXT('('));
						OriginalTraitIndicesString = OriginalTraitIndicesString.TrimChar(TEXT(')'));
						FString PinPathRemaining = OriginalTraitIndicesString;
						FString Left, Right;
						TArray<FString> IndexStrings;
						while(PinPathRemaining.Split(TEXT(","), &Left, &Right))
						{
							IndexStrings.Add(Left.TrimStartAndEnd());
							Left.Empty();
							PinPathRemaining = Right;
						}
						if (!Right.IsEmpty())
						{
							IndexStrings.Add(Right.TrimStartAndEnd());
						}

						for(int32 PartIndex = 0; PartIndex < IndexStrings.Num(); PartIndex++)
						{
							const int32 OriginalTraitIndex = FCString::Atoi(*IndexStrings[PartIndex]);
							const FRigVMFunctionCompilationPropertyDescription& TraitPropertyDescription = InCompilationData->WorkPropertyDescriptions[OriginalTraitIndex];
							const FFunctionRegisterData Data = {InHash, ERigVMMemoryType::Work, OriginalTraitIndex};
							const FRigVMOperand NewOperand = FindOrAddProperty(ERigVMMemoryType::Work, TraitPropertyDescription, TraitPropertyDescription.Name.ToString(), true /* unique */);
							FunctionRegisterToOperand.Add(Data, NewOperand);
							OperandMap.Add(FRigVMOperand(ERigVMMemoryType::Work, OriginalTraitIndex), NewOperand);

							IndexStrings[PartIndex] = FString::FromInt(NewOperand.GetRegisterIndex());
						}

						IndexStrings.Remove(FString());
						CustomDescription = *Description;
						CustomDescription.DefaultValue = FString::Printf(TEXT("(%s)"), *FString::Join(IndexStrings, TEXT(",")));
						FString NewName = FString::Printf(TEXT("Hash%lu_%s"), InHash, *CustomDescription.Name.ToString().RightChop(FunctionLibraryPrefix.Len()));
						FRigVMPropertyDescription::SanitizeName(NewName);
						CustomDescription.Name = *NewName;

						Description = &CustomDescription;
					}
				}

				if (MemoryType == ERigVMMemoryType::External)
				{
					if (ResolvedExternalIndex != INDEX_NONE)
					{
						// Description was re-resolved by name above — emit the operand with the
						// current parent-VM index instead of the cached function-side one.
						NewOperandNoOffset = FRigVMOperand(ERigVMMemoryType::External, ResolvedExternalIndex);
					}
					else if (bExternalPlaceholder)
					{
						// The External fallback gave up on this operand (variable
						// renamed/removed/retyped, or out-of-range stale index). Don't emit
						// bytecode against the original index — at runtime that would either
						// access out-of-bounds memory or silently bind to the wrong variable.
						// Emit an Invalid/INDEX_NONE sentinel; the error already logged above
						// is the real diagnostic.
						NewOperandNoOffset = FRigVMOperand(ERigVMMemoryType::Invalid, INDEX_NONE);
					}
					else
					{
						// Legacy compat: no remap needed (function-side and parent-VM indices
						// coincide) or the no-name-recorded best-effort branch handled it.
						NewOperandNoOffset = OldOperandNoOffset;
					}
				}
				else
				{
					FString NewName = Description->Name.ToString();

					const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Description->CPPTypeObject.Get());

					// instantiate function library specific work state as well as
					// instruction set execute state - which is used for lazy blocks.
					const bool bIsExecuteState = Description->CPPType.Equals(RigVMInstructionSetExecuteStateName);
					const bool bIsTrait = ScriptStruct && ScriptStruct->IsChildOf(FRigVMTrait::StaticStruct());
					if (NewName.StartsWith(FunctionLibraryPrefix) || bIsExecuteState || bIsTrait)
					{
						NewName = FString::Printf(TEXT("Hash%lu_%s"), InHash, *NewName.RightChop(FunctionLibraryPrefix.Len()));
						FRigVMPropertyDescription::SanitizeName(NewName);
					}

					check(Description->CPPTypeObject.Get() != FRigVMForwardedMemoryHandle::StaticStruct());

					NewOperandNoOffset = FindOrAddProperty(MemoryType, *Description, NewName, bIsExecuteState || bIsTrait);
				}
				
				FFunctionRegisterData Data = { InHash, OldOperandNoOffset.GetMemoryType(), PropertyIndex};
				FunctionRegisterToOperand.Add(Data, NewOperandNoOffset);
               	OperandMap.Add(OldOperandNoOffset, NewOperandNoOffset);
			}
			
			if (InOldOperand.GetRegisterOffset() != INDEX_NONE)
			{
				const FRigVMOperand OldOperandJustOffset(InOldOperand.GetMemoryType(), INDEX_NONE, InOldOperand.GetRegisterOffset());
				if (!OperandMap.Contains(OldOperandJustOffset))
				{
					// If the no-offset operand resolved to an Invalid placeholder above (External
					// fallback gave up — variable removed/renamed/retyped with no usable remap),
					// the diagnostic has already been logged and the bytecode for this operand is
					// known broken. FindOrAddPropertyPath asserts on FRigVMPropertyPathDescription
					// validity, which requires PropertyIndex != INDEX_NONE — emitting a path against
					// the placeholder fires that assertion. Mirror the no-offset side: record a
					// matching Invalid/INDEX_NONE entry so downstream OperandMap lookups stay
					// consistent and skip the path-description add entirely.
					if (NewOperandNoOffset.GetMemoryType() == ERigVMMemoryType::Invalid)
					{
						OperandMap.Add(OldOperandJustOffset, FRigVMOperand(ERigVMMemoryType::Invalid, INDEX_NONE, INDEX_NONE));
					}
					else
					{
						// copy over the register offset
						const TArray<FRigVMFunctionCompilationPropertyPath>& PathDescriptions =
							InCompilationData->GetPropertyPathDescriptions(InOldOperand.GetMemoryType());

						check(PathDescriptions.IsValidIndex(InOldOperand.GetRegisterOffset()));
						const FRigVMFunctionCompilationPropertyPath& PropertyPath = PathDescriptions[InOldOperand.GetRegisterOffset()];
						const int32 NewRegisterOffset = FindOrAddPropertyPath(NewOperandNoOffset, PropertyPath.HeadCPPType, PropertyPath.SegmentPath);
						const FRigVMOperand NewOperandJustOffset(NewOperandNoOffset.GetMemoryType(), INDEX_NONE, NewRegisterOffset);
						OperandMap.Add(OldOperandJustOffset, NewOperandJustOffset);
					}
				}
			}
		}
	};

	// we should include the interface operands to make sure all memory is there.
	for (const FRigVMOperand& InterfaceOperand : InInterfaceOperands)
	{
		ProcessOperand(ERigVMOpCode::Invalid, InterfaceOperand);
	}

	const FRigVMInstructionArray FunctionInstructions = InCompilationData->ByteCode.GetInstructions();
	for (int32 Index = InFirstInstruction; Index <= FMath::Min(InLastInstruction, FunctionInstructions.Num() - 1); ++Index)
	{
		const ERigVMOpCode OpCode = FunctionInstructions[Index].OpCode;
		const FRigVMOperandArray FunctionOperands = InCompilationData->ByteCode.GetOperandsForOp(FunctionInstructions[Index]);
		for (const FRigVMOperand& FunctionOperand : FunctionOperands)
		{
			ProcessOperand(OpCode, FunctionOperand);
		}
	}

	return true;
}

void FRigVMCompilerWorkData::SetupPropertyPathsForFoldedCopies(const FRigVMNodeExprAST* InExpr)
{
	// setup additional property paths needed for call externs accessing sub pin data directly
	for(const TPair<int32, FString>& SegmentPathPair : InExpr->SegmentPathForChild)
	{
		const FRigVMExprAST* ChildExpr = InExpr->ChildAt(SegmentPathPair.Key);
		check(ChildExpr);

		if(ChildExpr->IsA(FRigVMExprAST::EType::CachedValue))
		{
			ChildExpr = ChildExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr();
		}

		if(!ChildExpr->IsA(FRigVMExprAST::EType::Var))
		{
			continue;
		}

		const FRigVMVarExprAST* SourceVarExpr = Compiler->GetSourceVarExpr(ChildExpr);
		const FRigVMOperand RootOperand = ExprToOperand.FindChecked(SourceVarExpr);
		const FString HeadCPPType = SourceVarExpr->GetPin()->GetRootPin()->GetCPPType();

		FindOrAddPropertyPath(RootOperand, HeadCPPType, SegmentPathPair.Value);
	}
}

bool FRigVMCompilerWorkData::RemapInstructionAt(
	const FRigVMInstruction& InOldInstruction,
	const FString& InFunctionIdentifier,
	const FRigVMFunctionCompilationData* InCompilationData,
	FRigVMByteCode& InNewByteCode,
	const FRigVMInstructionArray& InNewInstructions,
	const URigVMFunctionReferenceNode* InFunctionReferenceNode,
	const TFunction<int32(const int32&)>& InRemapInstructionIndex,
	const TFunction<FRigVMOperand(const FRigVMOperand&)>& InRemapOperand,
	const TFunction<int32(const int32&)>& InRemapBranchIndex,
	const TFunction<int32(const int32&)>& InRemapCallableIndex)
{
	const FRigVMInstruction NewInstruction = InNewInstructions[InRemapInstructionIndex(InOldInstruction.Index)];
	const FRigVMOperandArray NewOperandArray = InNewByteCode.GetOperandsForOp(NewInstruction);
	const FRigVMOperandArray OldOperandArray = InCompilationData->ByteCode.GetOperandsForOp(InOldInstruction);
	checkf(OldOperandArray.Num() == NewOperandArray.Num(), TEXT("Hit a function with a mismatch of function operands, function %s in package %s"), *InFunctionIdentifier, *Compiler->GetPackage()->GetPathName());

	for (int32 j=0; j<NewOperandArray.Num(); ++j)
	{
		FRigVMOperand* NewOperand = const_cast<FRigVMOperand*>(&NewOperandArray[j]);
		const FRigVMOperand OriginalOperand = *NewOperand;
		checkf(OriginalOperand == OldOperandArray[j], TEXT("Hit a function with an invalid / non-matching function operand, function %s in package %s"), *InFunctionIdentifier, *Compiler->GetPackage()->GetPathName());
		*NewOperand = InRemapOperand(OriginalOperand);
	}

	if (NewInstruction.OpCode == ERigVMOpCode::Execute)
	{
		FRigVMExecuteOp& Op = InNewByteCode.GetOpAt<FRigVMExecuteOp>(NewInstruction);
		const FName& FunctionFName = InCompilationData->FunctionNames[Op.CallableIndex];
		const int32 FunctionIndex = VM->AddRigVMFunction(FunctionFName);
		const FRigVMFunction* Function = VM->GetFunctions()[FunctionIndex];
		check(Function)
		if (Function->Factory)
		{
			checkf(Function->Arguments.Num() <= NewOperandArray.Num(), TEXT("%s: invalid number of operands (%d) for dispatch '%s' - expected at least (%d)."), *Compiler->GetPackage()->GetPathName(), NewOperandArray.Num(), *Function->GetName(), Function->Arguments.Num());
		}
		else
		{
			checkf(Function->Arguments.Num() == NewOperandArray.Num(), TEXT("%s: invalid number of operands (%d) for function '%s' - expected (%d)."), *Compiler->GetPackage()->GetPathName(), NewOperandArray.Num(), *Function->GetName(), Function->Arguments.Num());
		}
		Op.CallableIndex = IntCastChecked<uint16>(FunctionIndex);
	}
	else if (NewInstruction.OpCode == ERigVMOpCode::InvokeCallable)
	{
		FRigVMInvokeCallableOp& Op = InNewByteCode.GetOpAt<FRigVMInvokeCallableOp>(NewInstruction);
		const int32 CallableIndex = InRemapCallableIndex(IntCastChecked<int32>(Op.CallableIndex));
		check(InNewByteCode.IsValidCallableIndex(CallableIndex));
		const FRigVMCallableInfo* Callable = InNewByteCode.GetCallable(CallableIndex);
		check(Callable)
		check(Callable->Arguments.Num() == NewOperandArray.Num());
		Op.CallableIndex = IntCastChecked<uint16>(CallableIndex);
	}
	else if (NewInstruction.OpCode == ERigVMOpCode::JumpToBranch)
	{
		FRigVMJumpToBranchOp& Op = InNewByteCode.GetOpAt<FRigVMJumpToBranchOp>(NewInstruction);
		Op.FirstBranchInfoIndex = InRemapBranchIndex(Op.FirstBranchInfoIndex);
		check(InNewByteCode.BranchInfos.IsValidIndex(Op.FirstBranchInfoIndex));
		check(InNewByteCode.BranchInfos[Op.FirstBranchInfoIndex].InstructionIndex == NewInstruction.Index);
	}
	else if (NewInstruction.OpCode == ERigVMOpCode::RunInstructions)
	{
		FRigVMRunInstructionsOp& Op = InNewByteCode.GetOpAt<FRigVMRunInstructionsOp>(NewInstruction);
		Op.StartInstruction = InRemapInstructionIndex(Op.StartInstruction);
		Op.EndInstruction = InRemapInstructionIndex(Op.EndInstruction);
	}

	if (Settings.SetupNodeInstructionIndex)
	{
		if (const TArray<TWeakObjectPtr<UObject>>* Callstack = InCompilationData->ByteCode.GetCallstackForInstruction(InOldInstruction.Index))
		{
			if (Callstack->Num() > 1)
			{
				FRigVMCallstack InstructionCallstack;
				InstructionCallstack.Stack = *Callstack;
				if (InFunctionReferenceNode)
				{
					InstructionCallstack.Stack[0] = TWeakObjectPtr<UObject>(const_cast<URigVMFunctionReferenceNode*>(InFunctionReferenceNode));
				}
				InNewByteCode.SetSubject(NewInstruction.Index, InstructionCallstack.GetCallPath(), InstructionCallstack.GetStack());
			}
		}

		// also store the instruction for the subject so that profiling can determine the cost of functions.
		if (InFunctionReferenceNode)
		{
			InNewByteCode.AddInstructionForSubject(const_cast<URigVMFunctionReferenceNode*>(InFunctionReferenceNode), NewInstruction.Index);
		}
	}

	// remap the input and output operands per instruction
	TArray<FRigVMOperand> InputOperands, OutputOperands;
	if (InCompilationData->ByteCode.InputOperandsPerInstruction.IsValidIndex(InOldInstruction.Index))
	{
		for (const FRigVMOperand& Operand : InCompilationData->ByteCode.InputOperandsPerInstruction[InOldInstruction.Index])
		{
			InputOperands.Add(InRemapOperand(Operand));
		}
	}
	if (InCompilationData->ByteCode.OutputOperandsPerInstruction.IsValidIndex(InOldInstruction.Index))
	{
		for (const FRigVMOperand& Operand : InCompilationData->ByteCode.OutputOperandsPerInstruction[InOldInstruction.Index])
		{
			OutputOperands.Add(InRemapOperand(Operand));
		}
	}
	InNewByteCode.SetOperandsForInstruction(NewInstruction, InputOperands, OutputOperands);
	return true;
}

void FRigVMCompilerWorkData::ReportInfo(const FString& InMessage) const
{
	Settings.ReportInfo(InMessage);
}

void FRigVMCompilerWorkData::ReportWarning(const FString& InMessage) const
{
	Settings.ReportWarning(InMessage);
}

void FRigVMCompilerWorkData::ReportError(const FString& InMessage) const
{
	Settings.ReportError(InMessage);
}

void FRigVMCompilerWorkData::OverrideReportDelegate(bool& bEncounteredASTError, bool& bSurpressedASTError)
{
	check(!OriginalReportDelegate.IsBound());
	OriginalReportDelegate = Settings.ASTSettings.ReportDelegate;
	
	Settings.ASTSettings.ReportDelegate =
		FRigVMReportDelegate::CreateLambda([this, &bEncounteredASTError, &bSurpressedASTError]
			(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
			{
				FString Message = InMessage;
				if(Settings.ASTErrorsAsNotifications &&
					(InSeverity == EMessageSeverity::Error || InSeverity == EMessageSeverity::Warning))
				{
					const bool bIsError = InSeverity == EMessageSeverity::Error;
					static constexpr TCHAR Warning[] = TEXT("Warning");
					static constexpr TCHAR Error[] = TEXT("Error");
					const TCHAR* SeverityLabel = bIsError ? Error : Warning;
					static constexpr TCHAR Format[] = TEXT("%s: The %s '%s' has been surpressed to allow the content to load. Please fix the content since it may become a requirement in future versions.");
					Message = FString::Printf(Format, *Graphs[0]->GetOutermost()->GetPathName(), SeverityLabel, *Message);
#if WITH_EDITOR
					if(InSubject)
					{
						Message.ReplaceInline(TEXT("@@"), *InSubject->GetName());
					}
						
					FNotificationInfo Info(FText::FromString(Message));
					Info.Image = bIsError ?
						FAppStyle::GetBrush("Icons.ErrorWithColor") :
						FAppStyle::GetBrush("Icons.WarningWithColor");
					Info.bFireAndForget = true;
					Info.FadeOutDuration = 1.0f;
					Info.ExpireDuration = 7.0f;
					Info.WidthOverride = 640.f;

					(void)FSlateNotificationManager::Get().AddNotification(Info);
#endif
					InSeverity = EMessageSeverity::Info;
					bSurpressedASTError = true;
				}
				(void)OriginalReportDelegate.ExecuteIfBound(InSeverity, InSubject, Message);
				if(InSeverity == EMessageSeverity::Error)
				{
					bEncounteredASTError = true;
				}
			}
		);
}

void FRigVMCompilerWorkData::RemoveOverrideReportDelegate()
{
	Settings.ASTSettings.ReportDelegate = OriginalReportDelegate;
	OriginalReportDelegate = FRigVMReportDelegate();
}

URigVMCompiler::URigVMCompiler()
	: CurrentlyCompilingFunctionNode(nullptr)
{
}

bool URigVMCompiler::Compile(const FRigVMCompileSettings& InSettings, TArray<URigVMGraph*> InGraphs,
	URigVMController* InController, URigVM* OutVM, FRigVMExtendedExecuteContext& OutVMContext,
	const TArray<FRigVMExternalVariable>& InExternalVariables, TMap<FString, FRigVMOperand>* OutOperands,
	TSharedPtr<FRigVMParserAST> InAST, FRigVMFunctionCompilationData* OutFunctionCompilationData)
{
	FRigVMCompilerWorkData WorkData;
	WorkData.Compiler = this;
	WorkData.Settings = InSettings;
	WorkData.Graphs = InGraphs;
	WorkData.FunctionCompilationData = OutFunctionCompilationData;

	FRigVMCompileSettings& Settings = WorkData.Settings;
	
	double CompilationTime = 0;
	FDurationTimer CompileTimer(CompilationTime);
	
	if (InGraphs.IsEmpty() || InGraphs.Contains(nullptr))
	{
		WorkData.ReportError(TEXT("Provided graph is nullptr."));
		return false;
	}
	
	if (OutVM == nullptr)
	{
		WorkData.ReportError(TEXT("Provided vm is nullptr."));
		return false;
	}

	if (Settings.GetExecuteContextStruct() == nullptr)
	{
		WorkData.ReportError(TEXT("Compiler settings don't provide the ExecuteContext to use. Cannot compile."));
		return false;;
	}

	// also during traverse - find all known execute contexts
	// for functions / dispatches / templates.
	// we only allow compatible execute context structs within a VM
	TArray<UStruct*> ValidExecuteContextStructs = FRigVMTemplate::GetSuperStructs(Settings.GetExecuteContextStruct());
	TArray<FString> ValidExecuteContextStructNames;
	Algo::Transform(ValidExecuteContextStructs, ValidExecuteContextStructNames, [](const UStruct* InStruct)
	{
		return CastChecked<UScriptStruct>(InStruct)->GetStructCPPName();
	});

	for(URigVMGraph* Graph : InGraphs)
	{
		if(Graph->GetExecuteContextStruct())
		{
			if(!ValidExecuteContextStructs.Contains(Graph->GetExecuteContextStruct()))
			{
				WorkData.ReportErrorf(
					TEXT("Compiler settings' ExecuteContext (%s) is not compatible with '%s' graph's ExecuteContext (%s). Cannot compile."),
					*Settings.GetExecuteContextStruct()->GetStructCPPName(),
					*Graph->GetNodePath(),
					*Graph->GetExecuteContextStruct()->GetStructCPPName()
				);
				return false;;
			}
		}
	}

	for(int32 Index = 1; Index < InGraphs.Num(); Index++)
	{
		if(InGraphs[0]->GetOuter() != InGraphs[Index]->GetOuter())
		{
			WorkData.ReportError(TEXT("Provided graphs don't share a common outer / package."));
			return false;
		}
	}

	if(OutVM->GetClass()->IsChildOf(URigVMNativized::StaticClass()))
	{
		WorkData.ReportError(TEXT("Provided vm is nativized."));
		return false;
	}

	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	OutVM->Reset(OutVMContext);
	OutVMContext.VMHash = 0;	// this has to be set when the memory is copied to the context

	TMap<FString, FRigVMOperand> LocalOperands;
	if (OutOperands == nullptr)
	{
		OutOperands = &LocalOperands;
	}
	OutOperands->Reset();

	URigVMFunctionLibrary* FunctionLibrary = InGraphs[0]->GetDefaultFunctionLibrary();
	bool bEncounteredGraphError = false;

	TMap<FString, FCompiledFunctionData> CurrentCompiledFunctions;

#if WITH_EDITOR
	// During cooking, compile all public functions
	if (IsRunningCookCommandlet())
	{
		if (!CurrentlyCompilingFunctionNode)
		{
			if (FunctionLibrary)
			{
				TArray<URigVMLibraryNode*> Functions = FunctionLibrary->GetFunctions();
				for (URigVMLibraryNode* LibraryNode : Functions)
				{
					if (FunctionLibrary->IsFunctionPublic(LibraryNode->GetFName()))
					{
						if (FRigVMGraphFunctionData* FunctionData = LibraryNode->GetFunctionHeader().GetFunctionData())
						{
							TArray<FRigVMExternalVariable> FunctionVariables = LibraryNode->GetExternalVariables();
							CompileFunction(WorkData.Settings, LibraryNode, InController, FunctionVariables, &FunctionData->CompilationData, OutVMContext);
						}
					}
				}
			}
		}
	}
#endif

	// Gather function compilation data
	for(URigVMGraph* Graph : InGraphs)
	{
		TArray<URigVMNode*> Nodes = Graph->GetNodes();
		for (int32 i=0; i<Nodes.Num(); ++i)
		{
			if (URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(Nodes[i]))
			{
				if (!ReferenceNode->GetReferencedFunctionHeader().IsValid())
				{
					static const FString FunctionCompilationErrorMessage = TEXT("Function reference @@ has no function data.");
					Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, FunctionCompilationErrorMessage);
					bEncounteredGraphError = true;
					break;
				}
				
				// Try to find the compiled data
				FString FunctionHash = ReferenceNode->GetReferencedFunctionHeader().GetHash();
				if (!CurrentCompiledFunctions.Contains(FunctionHash))
				{
					if (FRigVMGraphFunctionData* FunctionData = ReferenceNode->GetReferencedFunctionHeader().GetFunctionData())
					{
						// Clear compilation data if compiled with outdated dependency data
						if (FunctionData->CompilationData.IsValid())
						{
							for (const TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : FunctionData->Header.Dependencies)
							{
								bool bDirty = true;
								if (IRigVMGraphFunctionHost* HostObj = Cast<IRigVMGraphFunctionHost>(Pair.Key.HostObject.ResolveObject()))
								{
									if (FRigVMGraphFunctionData* DependencyData = HostObj->GetRigVMGraphFunctionStore()->FindFunction(Pair.Key))
									{
										bDirty = DependencyData->CompilationData.Hash != Pair.Value || Pair.Value == 0;
									}
								}
								if (bDirty)
								{
									FunctionData->ClearCompilationData();
									break;
								}
							}
						}
						
						if (const FRigVMFunctionCompilationData* CompilationData = &FunctionData->CompilationData)
						{
							bool bSuccessfullCompilation = false;
							if (!CompilationData->IsValid() || CompilationData->RequiresRecompilation())
							{
								if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionData->Header.LibraryPointer.GetNodeSoftPath().TryLoad()))
								{
									IRigVMClientHost* ClientHost = LibraryNode->GetImplementingOuter<IRigVMClientHost>();
									URigVMController* FunctionController = ClientHost->GetRigVMClient()->GetOrCreateController(LibraryNode->GetLibrary());
									
									TArray<FRigVMExternalVariable> FunctionVariables = LibraryNode->GetExternalVariables();
									if(ReferenceNode->RequiresVariableRemapping())
									{
										for(FRigVMExternalVariable& FunctionExternalVariable : FunctionVariables)
										{
											const FName OuterVariableName = ReferenceNode->GetOuterVariableName(FunctionExternalVariable.GetName());
											if(OuterVariableName.IsNone())
											{
												const FString VariableRemappingErrorMessage =
													FString::Printf(TEXT("The function's variable '%s' is not remapped on function reference @@."),
													*FunctionExternalVariable.GetName().ToString());
												Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, VariableRemappingErrorMessage);
												bEncounteredGraphError = true;
											}
											else
											{
												const FRigVMExternalVariable* OuterExternalVariable = InExternalVariables.FindByPredicate(
													[OuterVariableName](const FRigVMExternalVariable& ExternalVariable) -> bool
													{
														return ExternalVariable.GetName().IsEqual(OuterVariableName, ENameCase::CaseSensitive);
													}
												);

												if(OuterExternalVariable)
												{
													check(FRigVMRegistry::Get().CanMatchTypes(OuterExternalVariable->GetTypeIndex(), FunctionExternalVariable.GetTypeIndex(), true));
													FunctionExternalVariable.SetProperty(OuterExternalVariable->GetProperty());
													FunctionExternalVariable.SetMemory(const_cast<uint8*>(OuterExternalVariable->GetMemory()));
												}
											}
										}
									}
									else
									{
										// if this function doesn't require remapping it's a function used within
										// the same package. in that case we'll give access to all external variables.
										FunctionVariables = InExternalVariables;
									}

									bSuccessfullCompilation = CompileFunction(WorkData.Settings, LibraryNode, FunctionController, FunctionVariables, &FunctionData->CompilationData, OutVMContext);
								}
								else
								{
									static const FString FunctionCompilationErrorMessage = TEXT("Compilation data for public function @@ has no instructions.");
									Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, FunctionCompilationErrorMessage);
									bEncounteredGraphError = true;
								}
							}
							if (bSuccessfullCompilation || CompilationData->IsValid())
							{
								FunctionData->PatchSharedArgumentOperandsIfRequired();
								CurrentCompiledFunctions.Add(FunctionHash, {ReferenceNode, CompilationData});
							}
							else
							{
								static const FString FunctionCompilationErrorMessage = TEXT("Compilation data for public function @@ has no instructions.");
								Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, FunctionCompilationErrorMessage);
								bEncounteredGraphError = true;
							}
						}
						else
						{
							static const FString FunctionCompilationErrorMessage = TEXT("Could not find compilation data for node @@.");
							Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, FunctionCompilationErrorMessage);
							bEncounteredGraphError = true;
						}
					}
					else
					{
						static const FString FunctionCompilationErrorMessage = TEXT("Could not find graph function data for node @@.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ReferenceNode, FunctionCompilationErrorMessage);
						bEncounteredGraphError = true;
					}
				}
			}
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Nodes[i]))
			{
				if (CollapseNode->GetContainedGraph())
				{
					Nodes.Append(CollapseNode->GetContainedGraph()->GetNodes());
				}
				else
				{
					static const FString FunctionCompilationErrorMessage = TEXT("Could not find contained graph for collapse node @@.");
					Settings.ASTSettings.Report(EMessageSeverity::Error, CollapseNode, FunctionCompilationErrorMessage);
					bEncounteredGraphError = true;
				}
			}
		}
	}

	if (bEncounteredGraphError)
	{
		return false;
	}

	CompiledFunctions = CurrentCompiledFunctions;

#if WITH_EDITOR

	// traverse all graphs and try to clear out orphan pins
	// also check on function references with unmapped variables
	TArray<URigVMGraph*> VisitedGraphs;
	VisitedGraphs.Append(InGraphs);

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	bool bSendMissingPluginNotification = false;

	TSet<FName> FoundEvents;
	for(int32 GraphIndex=0; GraphIndex<VisitedGraphs.Num(); GraphIndex++)
	{
		URigVMGraph* VisitedGraph = VisitedGraphs[GraphIndex];

		if(URigVMController* VisitedGraphController = InController->GetControllerForGraph(VisitedGraph))
		{
			// make sure variables are up to date before validating other things.
			// that is, make sure their cpp type and type object agree with each other
			VisitedGraphController->EnsureLocalVariableValidity();
		}
		
		for(URigVMNode* ModelNode : VisitedGraph->GetNodes())
		{
			URigVMController* VisitedGraphController = InController->GetControllerForGraph(VisitedGraph);
			if(VisitedGraphController == nullptr)
			{
				return false;
			}

			// make sure pins are up to date before validating other things.
			// that is, make sure their cpp type and type object agree with each other
			for(URigVMPin* Pin : ModelNode->Pins)
			{
				if(!URigVMController::EnsurePinValidity(Pin, true))
				{
					static const FString InvalidPinEncountered = TEXT("Pin @@ is not valid - potentially using an invalid type?");
					Settings.ASTSettings.Report(EMessageSeverity::Error, Pin, InvalidPinEncountered);

					if (!bSendMissingPluginNotification)
					{
						bSendMissingPluginNotification = true;

						const FString Message = FString::Printf(TEXT("%s seems to require a missing a plugin."), *Pin->GetPackage()->GetPathName());
						UE_LOGF(LogRigVMDeveloper, Error, "%ls", *Message);

						FNotificationInfo Info(FText::FromString(Message));
						Info.bUseSuccessFailIcons = true;
						Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
						Info.bFireAndForget = true;
						Info.bUseThrobber = true;
						Info.FadeOutDuration = 1.f;
						Info.ExpireDuration = 5.f;
	
						TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
						if (NotificationPtr)
						{
							NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
						}
					}
					return false;
				}
			}

			if((Settings.bWarnAboutDuplicateEvents || CVarRigVMWarnAboutDuplicateEvents->GetBool()) && ModelNode->IsEvent())
			{
				if (FoundEvents.Contains(ModelNode->GetEventName()))
				{
					static const FString LinkedMessage = TEXT("Duplicate event node @@ found.");
					Settings.ASTSettings.Report(EMessageSeverity::Warning, ModelNode, LinkedMessage);
				}

				FoundEvents.Add(ModelNode->GetEventName());
			}

			if(ModelNode->IsA<UDEPRECATED_RigVMBranchNode>())
			{
				static const FString LinkedMessage = TEXT("Node @@ is a deprecated branch node. Cannot compile.");
				Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, LinkedMessage);
				bEncounteredGraphError = true;
			}

			if(ModelNode->IsA<UDEPRECATED_RigVMIfNode>())
			{
				static const FString LinkedMessage = TEXT("Node @@ is a deprecated if node. Cannot compile.");
				Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, LinkedMessage);
				bEncounteredGraphError = true;
			}

			if(ModelNode->IsA<UDEPRECATED_RigVMSelectNode>())
			{
				static const FString LinkedMessage = TEXT("Node @@ is a deprecated select node. Cannot compile.");
				Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, LinkedMessage);
				bEncounteredGraphError = true;
			}

			if(ModelNode->IsA<UDEPRECATED_RigVMArrayNode>())
			{
				static const FString LinkedMessage = TEXT("Node @@ is a deprecated array node. Cannot compile.");
				Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, LinkedMessage);
				bEncounteredGraphError = true;
			}

			if(!VisitedGraphController->RemoveUnusedOrphanedPins(ModelNode))
			{
				static const FString LinkedMessage = TEXT("Node @@ uses [{0}] pins that no longer exist. Please rewire the links and re-compile.");

				FString PinNames;
				for (URigVMPin* Pin : ModelNode->GetOrphanedPins())
				{
					if (PinNames.Len())
					{
						PinNames.Append(TEXT(", "));
					}
					PinNames.Append(Pin->GetDisplayName().ToString());
				}
				
				Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, FString::Format(*LinkedMessage, {PinNames}));
				bEncounteredGraphError = true;
			}

			// avoid function reference related validation for temp assets, a temp asset may get generated during
			// certain content validation process. It is usually just a simple file-level copy of the source asset
			// so these references are usually not fixed-up properly. Thus, it is meaningless to validate them.
			if (!ModelNode->GetPackage()->GetName().StartsWith(TEXT("/Temp/")))
			{
				if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
				{
					if(!FunctionReferenceNode->IsFullyRemapped())
					{
						static const FString UnmappedMessage = TEXT("Node @@ has unmapped variables. Please adjust the node and re-compile.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnmappedMessage);
						bEncounteredGraphError = true;
					}

					FString FunctionHash = FunctionReferenceNode->GetReferencedFunctionHeader().GetHash();
					if (const FCompiledFunctionData* CompiledData = CompiledFunctions.Find(FunctionHash))
					{
						// todo: this should go into a helper function
						TArray<FRigVMExternalVariable> AvailableExternalVariables;
						for (const FRigVMGraphVariableDescription& LocalVariable : VisitedGraph->LocalVariables)
						{
							FRigVMExternalVariable::MergeExternalVariable(AvailableExternalVariables, LocalVariable.ToExternalVariable());
						}
						const TArray<FRigVMGraphVariableDescription> InputVariables = VisitedGraph->GetInputVariables();
						for (const FRigVMGraphVariableDescription& InputVariable : InputVariables)
						{
							FRigVMExternalVariable::MergeExternalVariable(AvailableExternalVariables, InputVariable.ToExternalVariable());
						}
						for (const FRigVMExternalVariable& InExternalVariable : InExternalVariables)
						{
							FRigVMExternalVariable::MergeExternalVariable(AvailableExternalVariables, InExternalVariable);
						}
						if (CurrentlyCompilingFunctionNode)
						{
							if (const FRigVMGraphFunctionData* FunctionData = CurrentlyCompilingFunctionNode->GetFunctionHeader().GetFunctionData())
							{
								// private functions are allowed to access all external variables
								if (!FunctionData->IsPublic())
								{
									if(URigVMController* LibraryController = InController->GetControllerForGraph(VisitedGraph))
									{
										if (LibraryController->GetExternalVariablesDelegate.IsBound())
										{
											const TArray<FRigVMExternalVariable> ExternalVariablesFromDelegate = LibraryController->GetExternalVariablesDelegate.Execute(VisitedGraph);
											for (const FRigVMExternalVariable& ExternalVariableFromDelegate : ExternalVariablesFromDelegate)
											{
												FRigVMExternalVariable::MergeExternalVariable(AvailableExternalVariables, ExternalVariableFromDelegate);
											}
										}
									}
								}
							}
						}
						for (const TPair<int32, FName>& Pair : CompiledData->CompilationData->ExternalRegisterIndexToVariable)
						{
							FName OuterName = FunctionReferenceNode->GetOuterVariableName(Pair.Value);
							if (OuterName.IsNone())
							{
								OuterName = Pair.Value;
							}

							if (!AvailableExternalVariables.ContainsByPredicate([OuterName](const FRigVMExternalVariable& ExternalVariable)
								{
									return ExternalVariable.GetName() == OuterName;								
								}))
							{
								static const FString UnmappedMessage = TEXT("Function referenced in @@ using external variable not found in current rig.");
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnmappedMessage);
								bEncounteredGraphError = true;
							}
						}
					}
					else
					{
						static const FString UnmappedMessage = TEXT("Node @@ referencing function, but could not find compilation data.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnmappedMessage);
						bEncounteredGraphError = true;
					}
				}
			}

			if(ModelNode->IsA<URigVMFunctionInterfaceNode>())
			{
				for(URigVMPin* ExecutePin : ModelNode->Pins)
				{
					if(ExecutePin->IsExecuteContext())
					{
						if(ExecutePin->GetLinks().Num() == 0)
						{
							static const FString UnlinkedExecuteMessage = TEXT("Node @@ has an unconnected Execute pin. The function might cause unexpected behavior.");
							Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnlinkedExecuteMessage);
							bEncounteredGraphError = true;
						}
					}
				}
			}

			if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelNode))
			{
				if(URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph())
				{
					VisitedGraphs.AddUnique(ContainedGraph);
				}
			}

			// Validate that ExternalVariable (InputVariable) pins on unit nodes are bound.
			// These pins are mutable references to caller-owned data — if neither a link
			// nor a variable binding provides the reference, the node cannot execute correctly.
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
			{
				for(URigVMPin* Pin : UnitNode->GetPins())
				{
					if(Pin->IsDefinedAsInputVariable())
					{
						const bool bHasLink = Pin->GetSourceLinks().Num() > 0;
						const bool bHasBinding = Pin->IsBoundToVariable();
						if(!bHasLink && !bHasBinding)
						{
							static const FString UnboundInputVariableMessage = TEXT("Node @@ has an ExternalVariable pin that is not linked or bound to a variable. ExternalVariable pins must be resolved at runtime by the caller.");
							Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnboundInputVariableMessage);
							bEncounteredGraphError = true;
						}
					}
				}
			}

			// for variable let's validate ill formed variable nodes
			if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelNode))
			{
				static const FString IllFormedVariableNodeMessage = TEXT("Variable Node @@ is ill-formed (pin type doesn't match the variable type). Consider recreating the node.");

				const FRigVMGraphVariableDescription VariableDescription = VariableNode->GetVariableDescription();
				const TArray<FRigVMGraphVariableDescription> LocalVariables = VisitedGraph->GetLocalVariables(true);
				const TArray<FRigVMGraphVariableDescription> InputVariables = VisitedGraph->GetInputVariables();

				bool bFoundVariable = false;
				for(const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
				{
					if(LocalVariable.Name == VariableDescription.Name)
					{
						bFoundVariable = true;
						
						if(LocalVariable.CPPType != VariableDescription.CPPType ||
							LocalVariable.CPPTypeObject != VariableDescription.CPPTypeObject)
						{
							Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, IllFormedVariableNodeMessage);
							bEncounteredGraphError = true;
						}
					}
				}

				if (!bFoundVariable)
				{
					for(const FRigVMGraphVariableDescription& InputVariable : InputVariables)
					{
						if(InputVariable.Name == VariableDescription.Name)
						{
							bFoundVariable = true;
						
							if(InputVariable.CPPType != VariableDescription.CPPType ||
								InputVariable.CPPTypeObject != VariableDescription.CPPTypeObject)
							{
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, IllFormedVariableNodeMessage);
								bEncounteredGraphError = true;
							}
						}
					}
				}

				// if the variable is not a local variable, let's test against the external variables.
				if(!bFoundVariable)
				{
					const FRigVMExternalVariable ExternalVariable = VariableDescription.ToExternalVariable();
					for(const FRigVMExternalVariable& InExternalVariable : InExternalVariables)
					{
						if(InExternalVariable.GetName() == ExternalVariable.GetName())
						{
							bFoundVariable = true;
							
							if(!InExternalVariable.IsSameType(ExternalVariable))
							{
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, IllFormedVariableNodeMessage);
								bEncounteredGraphError = true;
							}
						}
					}
				}

				if(!bFoundVariable)
				{
					static const FString MissingVariableNodeMessage = TEXT("Variable Node @@ is using a missing variable. Consider recreating the node.");
					Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, MissingVariableNodeMessage);
					bEncounteredGraphError = true;
				}

				if(VariableDescription.CPPTypeObject && !RigVMCore::SupportsUObjects())
				{
					if(VariableDescription.CPPTypeObject->IsA<UClass>())
					{
						static const FString InvalidObjectTypeMessage = TEXT("Variable Node @@ uses an unsupported UClass type.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, InvalidObjectTypeMessage);
						bEncounteredGraphError = true;
					}
				}


				if (VariableDescription.CPPTypeObject && !RigVMCore::SupportsUInterfaces())
				{
					if (VariableDescription.CPPTypeObject->IsA<UInterface>())
					{
						static const FString InvalidObjectTypeMessage = TEXT("Variable Node @@ uses an unsupported UInterface type.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, InvalidObjectTypeMessage);
						bEncounteredGraphError = true;
					}
				}
			}

			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
			{
				if (!UnitNode->HasWildCardPin())
				{
					UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct(); 
					if(ScriptStruct == nullptr)
					{
						VisitedGraphController->FullyResolveTemplateNode(UnitNode, INDEX_NONE, false);
					}

					if (UnitNode->GetScriptStruct() == nullptr || UnitNode->ResolvedFunctionName.IsEmpty())
					{
						static const FString UnresolvedUnitNodeMessage = TEXT("Node @@ could not be resolved.");
						Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, UnresolvedUnitNodeMessage);
						bEncounteredGraphError = true;
					}

					// Make sure all the pins exist in the node
					ScriptStruct = UnitNode->GetScriptStruct();
					if (ScriptStruct)
					{
						for (TFieldIterator<FProperty> It(ScriptStruct, EFieldIterationFlags::None); It; ++It)
						{
							const FRigVMTemplateArgument ExpectedArgument = FRigVMTemplateArgument::Make(*It);
							if (URigVMPin* Pin = UnitNode->FindPin(ExpectedArgument.Name.ToString()))
							{
								if (Pin->GetTypeIndex() != ExpectedArgument.GetTypeIndex(0))
								{
									FString MissingPinMessage = FString::Printf(TEXT("[%s] Could not find pin %s of type %s in Node @@."), *ModelNode->GetPackage()->GetPathName(), *ExpectedArgument.Name.ToString(), *Registry.GetType(ExpectedArgument.GetTypeIndex(0)).CPPType.ToString());
									Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, MissingPinMessage);
									bEncounteredGraphError = true;
								}
							}
							else
							{
								FString MissingPinMessage = FString::Printf(TEXT("[%s] Could not find pin %s of type %s in Node @@."), *ModelNode->GetPackage()->GetPathName(), *ExpectedArgument.Name.ToString(), *Registry.GetType(ExpectedArgument.GetTypeIndex(0)).CPPType.ToString());
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, MissingPinMessage);
								bEncounteredGraphError = true;
							}
						}
					}
				}

				if((Settings.bWarnAboutDeprecatedNodes || CVarRigVMWarnAboutDeprecatedNodes->GetBool()) && UnitNode->IsOutDated())
				{
					static const FString LinkedMessage = FString::Printf(TEXT("[%s] Node @@ is outdated."), *ModelNode->GetPackage()->GetPathName());
					Settings.ASTSettings.Report(EMessageSeverity::Warning, ModelNode, LinkedMessage);
				}
			}

			// Pure function validation
			if (URigVMCollapseNode* OwningFunction = Cast<URigVMCollapseNode>(ModelNode->FindFunctionForNode()))
			{
				if (OwningFunction->IsPure())
				{
					if (!ModelNode->IsA<URigVMFunctionInterfaceNode>())
					{
						// Reject mutable nodes without Pure metadata
						// Mutable nodes have execute pins and modify the execution context.
						// They need explicit Pure metadata to be allowed in pure functions.
						// Note: Varying/Constant metadata is about whether the output will change on each call,
						// not about side effects, so it is not relevant here.
						if (ModelNode->IsMutable() && !ModelNode->IsPure())
						{
							static const FString PureViolationMessage = TEXT("Mutable node @@ cannot be used in pure function. Mutable nodes must have 'Pure' metadata to be allowed in pure functions.");
							Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, PureViolationMessage);
							bEncounteredGraphError = true;
						}

						// Reject calls to mutable (non-pure) functions
						if (const URigVMFunctionReferenceNode* FuncRefNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
						{
							if (FuncRefNode->GetReferencedFunctionHeader().IsMutable())
							{
								static const FString MutableCallMessage = TEXT("Pure function cannot call mutable function @@.");
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, MutableCallMessage);
								bEncounteredGraphError = true;
							}
						}

						// Reject setting non-local variables (only local variable setters allowed)
						if (const URigVMVariableNode* VarNode = Cast<URigVMVariableNode>(ModelNode))
						{
							if (!VarNode->IsGetter() && !VarNode->IsLocalVariable())
							{
								static const FString NonLocalVarMessage = TEXT("Pure function cannot set non-local variable @@.");
								Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, NonLocalVarMessage);
								bEncounteredGraphError = true;
							}
						}

						// Reject non-local sources connected to non-constant IO pins.
						// Only local variable getters can connect to IO pins in pure functions.
						if (!ModelNode->IsA<URigVMRerouteNode>())
						{
							for (const URigVMPin* Pin : ModelNode->GetPins())
							{
								if (Pin->GetDirection() != ERigVMPinDirection::IO || Pin->IsDefinedAsConstant())
								{
									continue;
								}

								for (URigVMPin* SourcePin : Pin->GetLinkedSourcePins())
								{
									while (SourcePin && SourcePin->GetNode()->IsA<URigVMRerouteNode>())
									{
										TArray<URigVMPin*> SourcePins = Cast<URigVMRerouteNode>(SourcePin->GetNode())->FindPin(URigVMRerouteNode::ValueName)->GetLinkedSourcePins();
										if (SourcePins.Num() == 1)
										{
											SourcePin = SourcePins[0];
										}
										else
										{
											SourcePin = nullptr;
										}
									}
								
									if (!SourcePin)
									{
										continue;
									}
								
									if (SourcePin->IsExecuteContext())
									{
										continue;
									}

									// Local variable getters are allowed, otherwise reject
									bool bIsAllowed = true;
									if (const URigVMVariableNode* SourceVarNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
									{
										if (!SourceVarNode->IsLocalVariable())
										{
											bIsAllowed = false;
										}
									}
								
									// No connections from entry node
									if (SourcePin->GetNode()->IsA<URigVMFunctionEntryNode>())
									{
										bIsAllowed = false;
									}

									if (!bIsAllowed)
									{
										const FString IOPinMessage = FString::Printf(
											TEXT("Only local variables can connect to IO pin '%s' in pure function %s. Non-local data cannot be modified."),
											*Pin->GetPinPath(), *OwningFunction->GetName());
										Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, IOPinMessage);
										bEncounteredGraphError = true;
									}
								}
							}
						}
					}
				}
			}

			auto ReportIncompatibleExecuteContextString = [&] (const FString InExecuteContextName)
			{
				static constexpr TCHAR Format[] = TEXT("ExecuteContext '%s' on node '%s' is not compatible with '%s' provided by the compiler settings."); 
				WorkData.ReportErrorf(
					Format,
					*InExecuteContextName,
					*ModelNode->GetNodePath(),
					*Settings.GetExecuteContextStruct()->GetStructCPPName());
				bEncounteredGraphError = true;
			};

			auto ReportIncompatibleExecuteContext = [&] (const UScriptStruct* InExecuteContext)
			{
				ReportIncompatibleExecuteContextString(InExecuteContext->GetStructCPPName());
			};

			FString ExecuteContextMetaData;
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
			{
				if(UScriptStruct* Struct = UnitNode->GetScriptStruct())
				{
					if(Struct->GetStringMetaDataHierarchical(FRigVMStruct::ExecuteContextName, &ExecuteContextMetaData))
					{
						if(!ValidExecuteContextStructNames.Contains(ExecuteContextMetaData))
						{
							ReportIncompatibleExecuteContextString(ExecuteContextMetaData);
						}
					}
				}
			}

			if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					if(!ValidExecuteContextStructs.Contains(Factory->GetExecuteContextStruct()))
					{
						ReportIncompatibleExecuteContext(Factory->GetExecuteContextStruct());
					}
				}
			}
			else if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelNode))
			{
				if(const FRigVMFunction* ResolvedFunction = TemplateNode->GetResolvedFunction())
				{
					if(UScriptStruct* RigVMStruct = ResolvedFunction->Struct)
					{
						for (TFieldIterator<FProperty> It(RigVMStruct, EFieldIterationFlags::IncludeAll); It; ++It)
						{
							const FProperty* Property = *It;
							if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
							{
								Property = ArrayProperty->Inner;
							}
							
							if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
							{
								if(StructProperty->Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
								{
									if(!ValidExecuteContextStructs.Contains(StructProperty->Struct))
									{
										ReportIncompatibleExecuteContext(StructProperty->Struct);
									}
								}
							}
						}
					}
				}
				
				if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
				{
					const FRigVMDispatchContext DispatchContext;
					for(int32 Index = 0; Index < Template->NumExecuteArguments(DispatchContext); Index++)
					{
						if(const FRigVMExecuteArgument* Argument = Template->GetExecuteArgument(Index, DispatchContext))
						{
							if(Registry.IsExecuteType(Argument->TypeIndex))
							{
								const FRigVMTemplateArgumentType& Type = Registry.GetType(Argument->TypeIndex);
								if(UScriptStruct* ExecuteContextStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
								{
									if(!ValidExecuteContextStructs.Contains(ExecuteContextStruct))
									{
										ReportIncompatibleExecuteContext(ExecuteContextStruct);
									}
								}
							}
						}
					}
				}
			}

			if(ModelNode->IsA<URigVMUnitNode>() || ModelNode->IsA<URigVMDispatchNode>())
			{
				// Maximum number of operands is 65535
				int32 NumOperands = 0;
				for (URigVMPin* Pin : ModelNode->GetPins())
				{
					if (Pin->IsExecuteContext())
					{
						continue;
					}
					
					if (Pin->IsFixedSizeArray())
					{
						NumOperands += Pin->GetSubPins().Num();
					}

					NumOperands++;
				}

				if (NumOperands > 65535)
				{
					FString Format = FString::Printf(TEXT("Maximum number of operands for node @@ is 65535. Number of operands provided %d."), NumOperands);
					Settings.ASTSettings.Report(EMessageSeverity::Error, ModelNode, Format);
					bEncounteredGraphError = true;
				}
			}

			for(URigVMPin* Pin : ModelNode->Pins)
			{
				if(!URigVMController::EnsurePinValidity(Pin, true))
				{
					return false;
				}
			}
		}
	}

	// If compiling a function, check if all mutable paths lead to a return node
	// - Return node path should go through the completed pins of control flow nodes
	// - At least one path on sequence nodes should lead to a return node
	if (CurrentlyCompilingFunctionNode)
	{
		const URigVMFunctionEntryNode* EntryNode = CurrentlyCompilingFunctionNode->GetEntryNode();
		if (EntryNode && EntryNode->IsMutable())
		{
			URigVMFunctionReturnNode* ReturnNode = CurrentlyCompilingFunctionNode->GetReturnNode();
			if (!ReturnNode)
			{
				const FString FunctionCompilationErrorMessage = FString::Printf(TEXT("[%s] Mutable function graph %s does not contain a return node."), *EntryNode->GetPackage()->GetPathName(), *CurrentlyCompilingFunctionNode->GetName());
				Settings.ASTSettings.Report(EMessageSeverity::Error, nullptr, FunctionCompilationErrorMessage);
				bEncounteredGraphError = true;
			}

			bool bReturnNodeFound = false;
			TArray<URigVMPin*> Stack;
			URigVMPin* const* EntryExecutePin = EntryNode->GetPins().FindByPredicate([](URigVMPin* Pin) -> bool
				{
					return Pin->IsExecuteContext();
				}
			);
			Stack.Push(*EntryExecutePin);
			while (!Stack.IsEmpty())
			{
				URigVMPin* Pin = Stack.Pop();
				URigVMNode* Node = Pin->GetNode();
				if (Node == ReturnNode)
				{
					bReturnNodeFound = true;
					break;
				}

				URigVMPin* OutputPin = Pin;
				if (Node->IsControlFlowNode())
				{
					OutputPin = Node->FindPin(FRigVMStruct::ControlFlowCompletedName.ToString());
				}

				// If this is a sequence node, we need to go through all output pins
				if (OutputPin->GetDirection() == ERigVMPinDirection::Input)
				{
					TArray<URigVMPin*> SequenceOutputPins = Node->GetPins().FilterByPredicate([](URigVMPin* Pin) -> bool
					{
						return Pin->IsExecuteContext() && Pin->GetDirection() == ERigVMPinDirection::Output;
					});
					for (URigVMPin* SequenceOutputPin : SequenceOutputPins)
					{
						Stack.Push(SequenceOutputPin);
					}
				}
				else
				{
					check(OutputPin);
					check(OutputPin->GetDirection() == ERigVMPinDirection::Output || OutputPin->GetDirection() == ERigVMPinDirection::IO);
				
					TArray<URigVMPin*> TargetPins = OutputPin->GetLinkedTargetPins();
					if (!TargetPins.IsEmpty())
					{
						Stack.Push(TargetPins[0]);
					}
				}
			}

			if (!bReturnNodeFound)
			{
				const FString FunctionCompilationErrorMessage = FString::Printf(TEXT("[%s] Not all paths of the mutable function graph %s lead to a return node."), *CurrentlyCompilingFunctionNode->GetPackage()->GetPathName(), *CurrentlyCompilingFunctionNode->GetName());
				Settings.ASTSettings.Report(EMessageSeverity::Error, nullptr, FunctionCompilationErrorMessage);
				bEncounteredGraphError = true;
			}
		}
	}

	if(bEncounteredGraphError)
	{
		return false;
	}
#endif

	OutVM->ClearExternalVariables(OutVMContext);
	
	for (const FRigVMExternalVariable& ExternalVariable : InExternalVariables)
	{
		if (CurrentlyCompilingFunctionNode && CurrentlyCompilingFunctionNode->ContainsLocalVariable(ExternalVariable.GetName()))
		{
			continue;
		}
		FRigVMOperand Operand = OutVM->AddExternalVariable(OutVMContext, ExternalVariable, CurrentlyCompilingFunctionNode != nullptr);
		FString Hash = FString::Printf(TEXT("Variable::%s"), *ExternalVariable.GetName().ToString());
		OutOperands->Add(Hash, Operand);
	}

	bool bEncounteredASTError = false;
	bool bSurpressedASTError = false;
	WorkData.AST = InAST;
	if (!WorkData.AST.IsValid())
	{
		WorkData.OverrideReportDelegate(bEncounteredASTError, bSurpressedASTError);
		WorkData.AST = MakeShared<FRigVMParserAST>(InGraphs, InController, Settings.ASTSettings, InExternalVariables);
		WorkData.RemoveOverrideReportDelegate();

		for(URigVMGraph* Graph : InGraphs)
		{
			Graph->RuntimeAST = WorkData.AST;
		}
#if UE_BUILD_DEBUG
		//UE_LOGF(LogRigVMDeveloper, Display, "%ls", *AST->DumpDot());
#endif
	}
	ensure(WorkData.AST.IsValid());

	if(bEncounteredASTError)
	{
		return false;
	}

	WorkData.VM = OutVM;
	WorkData.Context = &OutVMContext;
	WorkData.ExecuteContextStruct = Settings.GetExecuteContextStruct();
	WorkData.PinPathToOperand = OutOperands;
	WorkData.CompilerPhase = ERigVMCompilerPhase_SetupMemory;
	WorkData.ProxySources = &WorkData.AST->SharedOperandPins;

	// tbd: do we need this only when we have no pins?
	//if(!WorkData.WatchedPins.IsEmpty())
	{
		// create the inverse map for the proxies
		WorkData.ProxyTargets.Reserve(WorkData.ProxySources->Num());
		for(const TPair<FRigVMASTProxy,FRigVMASTProxy>& Pair : *WorkData.ProxySources)
		{
			WorkData.ProxyTargets.FindOrAdd(Pair.Value).Add(Pair.Key);
		}
	}

	UE_LOG_RIGVMMEMORY(TEXT("RigVMCompiler: Begin '%s'..."), *InGraph->GetPathName());

	// convert the compiled functions into callables. this inlines the callables at the beginning of the bytecode.
	// the order of callables should be in reverse order of dependency - so the leaves of the dependency tree are first.
	for (const TPair<FString, FCompiledFunctionData>& Pair : CompiledFunctions)
	{
		const FRigVMFunctionCompilationData* FunctionCompilationData = Pair.Value.CompilationData;
		int32 FirstInstructionIndex = 0;
		
		// first look at all dependencies of this function and inline them. 
		for (int32 CallableIndex = 0; CallableIndex < Pair.Value.CompilationData->ByteCode.NumCallables(); CallableIndex++)
		{
			const FRigVMCallableInfo* CallableInfo = Pair.Value.CompilationData->ByteCode.GetCallable(CallableIndex);
			check(CallableInfo);

			const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeaderFromHash(CallableInfo->Name.ToString());
			if (!CompileCallable(WorkData, OutVM, OutVMContext, CallableInfo, Header, Pair.Value.CompilationData, Pair.Value.FunctionReferenceNode))
			{
				return false;
			}

			FirstInstructionIndex = CallableInfo->LastInstruction + 1;
		}

		if (!FunctionCompilationData->SupportsCallable())
		{
			continue;
		}

		if (CVarRigVMCompileFunctionsToCallables->GetBool())
		{
			const FRigVMGraphFunctionHeader& Header = Pair.Value.FunctionReferenceNode->GetReferencedFunctionHeader();

			// convert this function to a callable
			FRigVMCallableInfo Callable;
			Callable.FunctionHash = Pair.Value.CompilationData->Hash;
			Callable.Name = *Header.GetHash(); 
			Callable.FirstInstruction = FirstInstructionIndex;
			Callable.LastInstruction = Pair.Value.CompilationData->ByteCode.GetNumInstructions() - 1;

			// convert all arguments
			bool bAllArgumentsValid = true;
			for (const FRigVMGraphFunctionArgument& FunctionArgument : Header.Arguments)
			{
				if (FunctionArgument.IsExecuteContext())
				{
					continue;
				}
				
				FRigVMCallableArgument CallableArgument;
				CallableArgument.Name = FunctionArgument.Name;
				CallableArgument.TypeString = FunctionArgument.CPPType.ToString();

				// look for the argument in the function's operands
				const FRigVMOperand* InterfaceOperand = FunctionCompilationData->InterfaceOperands.Find(FunctionArgument.Name);
				if (!InterfaceOperand)
				{
					// When a function body reads a rig external variable via a Get-Variable
					// node without the user explicitly exposing it, the controller auto-
					// promotes that read into a library-node exposed pin with
					// bIsInputVariable=true (URigVMController). The auto-promoted pin lands
					// in Header.Arguments but the function compile does not produce an
					// InterfaceOperand for it — the variable continues to be read through
					// External operands inside the function body, not through the pin.
					// Skip the argument here so the callable conversion can proceed; the
					// External operands themselves are resolved in CompileCallable via
					// AddPropertiesForFunction's External fallback (which translates the
					// inner variable name through the reference node's variable map).
					if (FunctionArgument.bIsInputVariable)
					{
						continue;
					}
					bAllArgumentsValid = false;
					UE_LOGF(LogRigVMDeveloper, Warning, "Cannot convert function to callable '%ls', missing interface operand for argument '%ls'.", *Pair.Key, *FunctionArgument.Name.ToString());
					break;
				}

				CallableArgument.InterfaceOperand = *InterfaceOperand;
				CallableArgument.ForwardedOperand = FRigVMOperand();
				if (FunctionArgument.bIsInputVariable)
				{
					CallableArgument.Direction = ERigVMPinDirection::IO;
				}
				else
				{
					CallableArgument.Direction = FunctionArgument.Direction;
				}
				Callable.Arguments.Add(CallableArgument);
			}

			// compile the callable of the top level function
			if (bAllArgumentsValid)
			{
				if (!Pair.Value.FunctionReferenceNode)
				{
					return false;
				}

				if (!CompileCallable(WorkData, OutVM, OutVMContext, &Callable, Header, Pair.Value.CompilationData, Pair.Value.FunctionReferenceNode))
				{
					return false;
				}
			}
		}
	}

	// If we are compiling a function, we want the first registers to represent the interface pins (in the order of the pins)
	// so they can be replaced when inlining the function
	if (CurrentlyCompilingFunctionNode)
	{
		URigVMFunctionEntryNode* EntryNode = CurrentlyCompilingFunctionNode->GetEntryNode();
		URigVMFunctionReturnNode* ReturnNode = CurrentlyCompilingFunctionNode->GetReturnNode();
		for (URigVMPin* Pin : CurrentlyCompilingFunctionNode->GetPins())
		{
			URigVMPin* InterfacePin = nullptr;
			if (Pin->GetDirection() == ERigVMPinDirection::Input ||
				Pin->GetDirection() == ERigVMPinDirection::IO)
			{
				if(EntryNode == nullptr)
				{
					WorkData.ReportErrorf(TEXT("[%s] Corrupt library node '%s' - Missing entry node."), *CurrentlyCompilingFunctionNode->GetPackage()->GetPathName(), *CurrentlyCompilingFunctionNode->GetPathName());
					return false;
				}
				InterfacePin = EntryNode->FindPin(Pin->GetName());
			}
			else
			{
				if(ReturnNode == nullptr)
				{
					WorkData.ReportErrorf(TEXT("[%s] Corrupt library node '%s' - Missing return node."), *CurrentlyCompilingFunctionNode->GetPackage()->GetPathName(), *CurrentlyCompilingFunctionNode->GetPathName());
					return false;
				}
				InterfacePin = ReturnNode->FindPin(Pin->GetName());
			}

			if(InterfacePin == nullptr)
			{
				WorkData.ReportErrorf(TEXT("[%s} Corrupt library node '%s' - Pin '%s' is not part of the entry / return node."), *CurrentlyCompilingFunctionNode->GetPackage()->GetPathName(), *CurrentlyCompilingFunctionNode->GetPathName(), *Pin->GetPathName());
				return false;
			}

			FRigVMASTProxy PinProxy = FRigVMASTProxy::MakeFromUObject(InterfacePin);
			FRigVMVarExprAST* TempVarExpr = WorkData.AST->MakeExpr<FRigVMVarExprAST>(FRigVMExprAST::EType::Var, PinProxy);
			const FRigVMOperand InterfaceOperand = FindOrAddRegister(TempVarExpr, WorkData);
			if (WorkData.FunctionCompilationData)
			{
				if (!InterfacePin->IsExecuteContext())
				{
					const FRigVMOperand RootOperand(InterfaceOperand.GetMemoryType(), InterfaceOperand.GetRegisterIndex(), INDEX_NONE);
					WorkData.FunctionCompilationData->InterfaceOperands.FindOrAdd(InterfacePin->GetFName()) = RootOperand;
				}
			}
		}
	}

	if(Settings.EnablePinWatches)
	{
		for(int32 GraphIndex=0; GraphIndex<VisitedGraphs.Num(); GraphIndex++)
		{
			URigVMGraph* VisitedGraph = VisitedGraphs[GraphIndex];
			for(URigVMNode* ModelNode : VisitedGraph->GetNodes())
			{
				for(URigVMPin* ModelPin : ModelNode->GetPins())
				{
					if(ModelPin->RequiresWatch(true))
					{
						WorkData.WatchedPins.AddUnique(ModelPin);
					}
				}
			}
		}
	}

	// find all blocks
	for (FRigVMExprAST* Expression : WorkData.AST->Expressions)
	{
		const TOptional<uint32> OptionalHash = Expression->GetBlockCombinationHash();
		const uint32 Hash = OptionalHash.Get(0);
		if(!WorkData.LazyBlocks.Contains(Hash))
		{
			WorkData.LazyBlocks.Add(Hash, MakeShared<FRigVMCompilerWorkData::FLazyBlockInfo>());
		}

		if(OptionalHash.IsSet())
		{
			TSharedPtr<FRigVMCompilerWorkData::FLazyBlockInfo> Info = WorkData.LazyBlocks.FindChecked(Hash);
			Info->Hash = OptionalHash;
			if(Info->BlockCombinationName.IsEmpty())
			{
				Info->BlockCombinationName = Expression->GetBlockCombinationName();
			}
			Info->Expressions.Add(Expression);
		}
	}

	WorkData.ExprComplete.Reset();
	for (FRigVMExprAST* RootExpr : *WorkData.AST)
	{
		if (!TraverseExpression(RootExpr, WorkData))
		{
			WorkData.Clear();
			return false;
		}
	}

	// now that we have determined the needed memory, let's
	// setup properties as needed as well as property paths

	WorkData.VM->ClearMemory(*WorkData.Context);

	const TArray<ERigVMMemoryType> MemoryTypes = { ERigVMMemoryType::Literal, ERigVMMemoryType::Work, ERigVMMemoryType::Debug };

	for (ERigVMMemoryType MemoryType : MemoryTypes)
	{
		const TArray<FRigVMPropertyDescription>* Properties = WorkData.PropertyDescriptions.Find(MemoryType);
		WorkData.VM->GenerateDefaultMemoryType(MemoryType, Properties);
	}

	const TMap<const FRigVMExprAST*, bool> ExprSetupMemoryComplete = WorkData.ExprComplete;
	WorkData.CompilerPhase = ERigVMCompilerPhase_BuildInstructions;
	WorkData.ExprComplete.Reset();

	// sort the expressions in each block by depth
	for(auto Pair : WorkData.LazyBlocks)
	{
		auto SortByDepth = [](const FRigVMExprAST* InExpression) -> int32
		{
			return InExpression->GetMaximumDepth();
		};
		Algo::SortBy(Pair.Value->Expressions, SortByDepth);
	}
	
	// traverse the top level blocks
	WorkData.CurrentBlockHash = TOptional<uint32>();
	for (FRigVMExprAST* RootExpr : *WorkData.AST)
	{
		if (!TraverseExpression(RootExpr, WorkData))
		{
			WorkData.Clear();
			return false;
		}
	}

	if (!CurrentlyCompilingFunctionNode)
	{
		if (WorkData.VM->GetByteCode().GetInstructions().Num() == 0)
		{
			WorkData.VM->GetByteCode().AddExitOp();
		}
	}

	// traverse all other blocks - this has to be an index based loop
	if(!WorkData.LazyBlocksToProcess.IsEmpty())
	{
		const int32 JumpToEndOfBlocksExternByte = WorkData.VM->GetByteCode().AddJumpOp(ERigVMOpCode::JumpForward, INDEX_NONE);
		const int32 JumpToEndOfBlocksExternInstruction = WorkData.VM->GetByteCode().GetNumInstructions() - 1;

		FRigVMByteCode& ByteCode = WorkData.VM->GetByteCode();
		for(int32 LazyBlockHashIndex = 0; LazyBlockHashIndex < WorkData.LazyBlocksToProcess.Num(); LazyBlockHashIndex++)
		{
			TSharedPtr<FRigVMCompilerWorkData::FLazyBlockInfo>& BlockInfo =
				WorkData.LazyBlocks.FindChecked(WorkData.LazyBlocksToProcess[LazyBlockHashIndex]);
			if(BlockInfo->bProcessed)
			{
				continue;
			}

			BlockInfo->StartInstruction = ByteCode.GetNumInstructions();

			TGuardValue<TOptional<uint32>> HashGuard(WorkData.CurrentBlockHash, BlockInfo->Hash);
			for(const FRigVMExprAST* Expression : BlockInfo->Expressions)
			{
				if (ExprSetupMemoryComplete.Contains(Expression))
				{
					TraverseExpression(Expression, WorkData);
				}
			}

			BlockInfo->EndInstruction = ByteCode.GetNumInstructions() - 1;
			BlockInfo->bProcessed = true;
		}

		for(int32 LazyBlockHashIndex = 0; LazyBlockHashIndex < WorkData.LazyBlocksToProcess.Num(); LazyBlockHashIndex++)
		{
			TSharedPtr<FRigVMCompilerWorkData::FLazyBlockInfo>& BlockInfo =
				WorkData.LazyBlocks.FindChecked(WorkData.LazyBlocksToProcess[LazyBlockHashIndex]);

			// update all run instructions ops in the bytecode
			for(int32 RunInstructionsByteCode : BlockInfo->RunInstructionsToUpdate)
			{
				FRigVMRunInstructionsOp& RunInstructionsOp = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(RunInstructionsByteCode);
				RunInstructionsOp.StartInstruction = BlockInfo->StartInstruction;
				RunInstructionsOp.EndInstruction = BlockInfo->EndInstruction;
			}
		}
		
		// update the operator with the target instruction 
		const int32 InstructionsToJump = WorkData.VM->GetByteCode().GetNumInstructions() - JumpToEndOfBlocksExternInstruction;
		WorkData.VM->GetByteCode().GetOpAt<FRigVMJumpOp>(JumpToEndOfBlocksExternByte).InstructionIndex = InstructionsToJump;
	}
	
	if (!CurrentlyCompilingFunctionNode)
	{
		WorkData.VM->GetByteCode().AlignByteCode();
		WorkData.VM->GetByteCode().Shrink();
	}

	// now that we have determined the needed memory, let's
	// update the property paths once more
	for(ERigVMMemoryType MemoryType : MemoryTypes)
	{
		const TArray<FRigVMPropertyPathDescription>* Descriptions = WorkData.PropertyPathDescriptions.Find(MemoryType);
		if(FRigVMMemoryStorageStruct* MemoryStorageObject = WorkData.VM->GetDefaultMemoryByType(MemoryType))
		{
			if (Descriptions)
			{
				MemoryStorageObject->SetPropertyPathDescriptions(*Descriptions);
			}
			else
			{
				MemoryStorageObject->ResetPropertyPathDescriptions();
			}
			MemoryStorageObject->RefreshPropertyPaths();
		}
	}

	if(const TArray<FRigVMPropertyPathDescription>* Descriptions = WorkData.PropertyPathDescriptions.Find(ERigVMMemoryType::External))
	{
		WorkData.VM->ExternalPropertyPathDescriptions = *Descriptions;
	}

	// Store function compile data
	if (CurrentlyCompilingFunctionNode && OutFunctionCompilationData)
	{
		OutFunctionCompilationData->ByteCode = WorkData.VM->ByteCodeStorage;
		OutFunctionCompilationData->FunctionNames = WorkData.VM->FunctionNamesStorage;
		if (&OutFunctionCompilationData->Operands != OutOperands)
		{
			OutFunctionCompilationData->Operands = *OutOperands;
		}
		OutFunctionCompilationData->bEncounteredSurpressedErrors = bSurpressedASTError;

		for (uint8 MemoryTypeIndex=0; MemoryTypeIndex<(uint8)ERigVMMemoryType::Invalid; ++MemoryTypeIndex)
		{
			TArray<FRigVMFunctionCompilationPropertyDescription>* PropertyDescriptions = nullptr;
			TArray<FRigVMFunctionCompilationPropertyPath>* PropertyPathDescriptions = nullptr;
			ERigVMMemoryType MemoryType = (ERigVMMemoryType) MemoryTypeIndex;
			switch (MemoryType)
			{
				case ERigVMMemoryType::Work:
				{
					PropertyDescriptions = &OutFunctionCompilationData->WorkPropertyDescriptions;	
					PropertyPathDescriptions = &OutFunctionCompilationData->WorkPropertyPathDescriptions;
					break;
				}
				case ERigVMMemoryType::Literal:
				{
					PropertyDescriptions = &OutFunctionCompilationData->LiteralPropertyDescriptions;	
					PropertyPathDescriptions = &OutFunctionCompilationData->LiteralPropertyPathDescriptions;
					break;
				}
				case ERigVMMemoryType::External:
				{
					PropertyDescriptions = &OutFunctionCompilationData->ExternalPropertyDescriptions;	
					PropertyPathDescriptions = &OutFunctionCompilationData->ExternalPropertyPathDescriptions;
					break;
				}
				case ERigVMMemoryType::Debug:
				{
					PropertyDescriptions = &OutFunctionCompilationData->DebugPropertyDescriptions;	
					PropertyPathDescriptions = &OutFunctionCompilationData->DebugPropertyPathDescriptions;
					break;
				}
				default:
				{
					checkNoEntry();
				}
			}

			PropertyDescriptions->Reset();
			PropertyPathDescriptions->Reset();
			if (const TArray<FRigVMPropertyDescription>* Descriptions = WorkData.PropertyDescriptions.Find(MemoryType))
			{
				PropertyDescriptions->Reserve(Descriptions->Num());
				for (const FRigVMPropertyDescription& Description : (*Descriptions))
				{
					FRigVMFunctionCompilationPropertyDescription NewDescription;
					NewDescription.Name = Description.Name;
					NewDescription.CPPType = Description.CPPType;
					NewDescription.CPPTypeObject = Description.CPPTypeObject;
					NewDescription.DefaultValue = Description.DefaultValue;
					PropertyDescriptions->Add(NewDescription);
				}
			}
			if (const TArray<FRigVMPropertyPathDescription>* PathDescriptions = WorkData.PropertyPathDescriptions.Find(MemoryType))
			{
				PropertyPathDescriptions->Reserve(PathDescriptions->Num());
				for (const FRigVMPropertyPathDescription& Description : (*PathDescriptions))
				{
					FRigVMFunctionCompilationPropertyPath NewDescription;
					NewDescription.PropertyIndex = Description.PropertyIndex;
					NewDescription.SegmentPath = Description.SegmentPath;
					NewDescription.HeadCPPType = Description.HeadCPPType;
					PropertyPathDescriptions->Add(NewDescription);
				}
			}
		}

		// Only add used external registers to the function compilation data
		FRigVMInstructionArray Instructions = OutFunctionCompilationData->ByteCode.GetInstructions();
		TSet<int32> UsedExternalVariableRegisters;
		for (const FRigVMInstruction& Instruction : Instructions)
		{
			const FRigVMOperandArray OperandArray = OutFunctionCompilationData->ByteCode.GetOperandsForOp(Instruction);
			for (const FRigVMOperand& Operand : OperandArray)
			{
				if (Operand.GetMemoryType() == ERigVMMemoryType::External)
				{
					UsedExternalVariableRegisters.Add(Operand.GetRegisterIndex());					
				}
			}			
		}

		for (const TPair<FString, FRigVMOperand>& Pair : (*WorkData.PinPathToOperand))
		{
			if(Pair.Value.GetMemoryType() == ERigVMMemoryType::External)
			{
				static const FString VariablePrefix = TEXT("Variable::");
				if (Pair.Key.StartsWith(VariablePrefix))
				{
					if (UsedExternalVariableRegisters.Contains(Pair.Value.GetRegisterIndex()))
					{
						FString VariableName = Pair.Key.RightChop(VariablePrefix.Len());
						if (CurrentlyCompilingFunctionNode && CurrentlyCompilingFunctionNode->ContainsLocalVariable(*VariableName))
						{
							continue;
						}
						OutFunctionCompilationData->ExternalRegisterIndexToVariable.Add(Pair.Value.GetRegisterIndex(), *VariableName);
					}
				}
			}
		}

		OutFunctionCompilationData->Hash = GetTypeHash(*OutFunctionCompilationData);
	}

	// Localize all of the registry information
	if(CurrentlyCompilingFunctionNode == nullptr)
	{
		WorkData.VM->CreateLocalizedRegistryIfRequired();
	}

	// make sure all functions are known and resolved now.
	WorkData.VM->ResolveFunctionsIfRequired();
	
	if (!CurrentlyCompilingFunctionNode)
	{
		CompileTimer.Stop();

		WorkData.VM->SetVMHash(WorkData.VM->ComputeVMHash());
		WorkData.VM->GetByteCode().SetPublicContextAssetPath(FTopLevelAssetPath(WorkData.Context ? WorkData.Context->GetContextPublicDataStruct() : nullptr));
	}

	return true;
}

bool URigVMCompiler::CompileFunction(const FRigVMCompileSettings& InSettings, const URigVMLibraryNode* InLibraryNode, URigVMController* InController, const TArray<FRigVMExternalVariable>& InExternalVariables, FRigVMFunctionCompilationData* OutFunctionCompilationData, FRigVMExtendedExecuteContext& OutVMContext)
{
	if (FunctionCompilationStack.Contains(InLibraryNode))
	{
		return false;
	}

	FFunctionCompilationScope FunctionCompilationScope(this, InLibraryNode);
	TGuardValue<const URigVMLibraryNode*> CompilationGuard(CurrentlyCompilingFunctionNode, InLibraryNode);

	URigVMController* LibraryController = InController->GetControllerForGraph(InLibraryNode->GetContainedGraph());
	if(LibraryController == nullptr)
	{
		return false;
	}

	double CompilationTime = 0;
	FDurationTimer CompileTimer(CompilationTime);

	if (OutFunctionCompilationData == nullptr)
	{
		return false;
	}

	OutFunctionCompilationData->Hash = 0;
	OutFunctionCompilationData->ByteCode.Reset();

	TArray<FRigVMExternalVariable> FunctionExternalVariables = InExternalVariables;
	TMap<FString, FRigVMOperand> Operands;

	// let's create a property bag for the external variables so they can be backed up with properties and memory
	TArray<FRigVMPropertyDescription> FunctionVariableDescriptions;
	for (int32 FunctionVariableIndex = 0; FunctionVariableIndex < FunctionExternalVariables.Num(); FunctionVariableIndex++)
	{
		const FRigVMExternalVariable& FunctionVariable = FunctionExternalVariables[FunctionVariableIndex];
		static const FName PropertyNamePrefix = TEXT("Property");
		const FName PropertyName(PropertyNamePrefix, FunctionVariableIndex + 1);
		FunctionVariableDescriptions.Emplace(PropertyName, FunctionVariable.GetExtendedCPPType().ToString(), const_cast<UObject*>(FunctionVariable.GetCPPTypeObject()), FString());
	}
	FRigVMMemoryStorageStruct FunctionVariablesPropertyPag;
	FunctionVariablesPropertyPag.AddProperties(FunctionVariableDescriptions, {});
	const UStruct* FunctionVariablesStruct = FunctionVariablesPropertyPag.GetStruct();
	FStructOnScope FunctionVariablesInstance(FunctionVariablesStruct);
	
	for (int32 FunctionVariableIndex = 0; FunctionVariableIndex < FunctionExternalVariables.Num(); FunctionVariableIndex++)
	{
		const FName PropertyName = FunctionVariableDescriptions[FunctionVariableIndex].Name;
		FRigVMExternalVariable& FunctionVariable = FunctionExternalVariables[FunctionVariableIndex];
		FunctionVariable.SetProperty(FunctionVariablesStruct->FindPropertyByName(PropertyName));
		check(FunctionVariable.GetProperty());
		FunctionVariable.SetMemory(FunctionVariable.GetProperty()->ContainerPtrToValuePtr<uint8>(FunctionVariablesInstance.GetStructMemory()));
		check(FunctionVariable.GetMemory());
	}

	URigVM* TempVM = NewObject<URigVM>(InLibraryNode->GetContainedGraph(), TEXT("CompilerTemp_VM"));
	const bool bSuccess = Compile(InSettings, {InLibraryNode->GetContainedGraph()}, LibraryController, TempVM, OutVMContext, FunctionExternalVariables, &Operands, nullptr, OutFunctionCompilationData);
	TempVM->ClearMemory(OutVMContext);
	TempVM->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	TempVM->MarkAsGarbage();

	CompileTimer.Stop();
	InSettings.ReportInfof(TEXT("Compiled Function %s in %fms"), *InLibraryNode->GetName(), CompilationTime*1000);

	// Update the compilation data of this library, and the hashes of the compilation data of its dependencies used for this compilation
	if (IRigVMClientHost* ClientHost = InLibraryNode->GetImplementingOuter<IRigVMClientHost>())
	{
		if (TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost())
		{
			if (FRigVMGraphFunctionStore* Store = FunctionHost->GetRigVMGraphFunctionStore())
			{
				if (FRigVMGraphFunctionData* Data = Store->FindFunction(InLibraryNode->GetFunctionIdentifier()))
				{
					for(TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : Data->Header.Dependencies)
					{
						if (IRigVMGraphFunctionHost* ReferencedFunctionHost = Cast<IRigVMGraphFunctionHost>(Pair.Key.HostObject.ResolveObject()))
						{
							if (FRigVMGraphFunctionData* ReferencedData = ReferencedFunctionHost->GetRigVMGraphFunctionStore()->FindFunction(Pair.Key))
							{
								Pair.Value = ReferencedData->CompilationData.Hash;
							}
						}
					}
				}
		
				Store->UpdateFunctionCompilationData(InLibraryNode->GetFunctionIdentifier(), *OutFunctionCompilationData);
			}
		}
	}

	return bSuccess;
}

bool URigVMCompiler::CompileCallable(FRigVMCompilerWorkData& WorkData, URigVM* OutVM, FRigVMExtendedExecuteContext& OutVMContext,
	const FRigVMCallableInfo* InCallable, const FRigVMGraphFunctionHeader& InHeader, const FRigVMFunctionCompilationData* InFunctionCompilationData, const URigVMFunctionReferenceNode* InFunctionReferenceNode)
{
	check(OutVM);
	check(InCallable);
	check(InFunctionCompilationData);

	if (!InHeader.IsValid())
	{
		return false;
	}
	
	// if we already compiled this callable - carry on
	FRigVMByteCode& ByteCode = OutVM->GetByteCode();
	if (ByteCode.FindCallable(InCallable->FunctionHash))
	{
		return true;
	}

	const FRigVMByteCode& FunctionByteCode = InFunctionCompilationData->ByteCode;
	const FRigVMInstructionArray FunctionInstructions = FunctionByteCode.GetInstructions();

	// Add all required properties, property paths and set up WorkData.OperandMapPerFunction
	TArray<FRigVMOperand> ArgumentOperands;
	TMap<FRigVMOperand,FRigVMOperand> OldForwardedHandleToInterfaceOperand;
	ArgumentOperands.Reserve(InCallable->Arguments.Num());
	for (const FRigVMCallableArgument& Argument : InCallable->Arguments)
	{
		ArgumentOperands.Add(Argument.InterfaceOperand);
		if (Argument.ForwardedOperand.IsValid())
		{
			OldForwardedHandleToInterfaceOperand.Add(Argument.ForwardedOperand, Argument.InterfaceOperand);
		}
	}

	const uint32 HashForProperties = GetTypeHash(*InCallable);
	(void)WorkData.AddPropertiesForFunction(true, HashForProperties, InHeader, InFunctionCompilationData, InCallable->FirstInstruction, InCallable->LastInstruction, ArgumentOperands, OldForwardedHandleToInterfaceOperand, InFunctionReferenceNode);

	FRigVMCallableInfo NewCallable;
	NewCallable.Index = ByteCode.NumCallables();
	NewCallable.FunctionHash = InCallable->FunctionHash;
	NewCallable.Name = InCallable->Name;
	NewCallable.FirstInstruction = ByteCode.GetNumInstructions();
	NewCallable.LastInstruction = NewCallable.FirstInstruction + InCallable->LastInstruction - InCallable->FirstInstruction;

	TMap<FRigVMOperand, FRigVMOperand> NewInterfaceOperandToForwardedHandle;

	const TMap<FRigVMOperand, FRigVMOperand>& OperandMap = WorkData.OperandMapPerFunction.FindChecked(HashForProperties);
	auto RemapOperand = [&OperandMap, &NewInterfaceOperandToForwardedHandle, &OldForwardedHandleToInterfaceOperand, InHeader](const FRigVMOperand& InOldOperand, bool bRemapToForwardedHandle)
	{
		FRigVMOperand OperandNoOffset(InOldOperand.GetMemoryType(), InOldOperand.GetRegisterIndex(), INDEX_NONE);
		if (const FRigVMOperand* InterfaceOperand = OldForwardedHandleToInterfaceOperand.Find(OperandNoOffset))
		{
			OperandNoOffset = *InterfaceOperand;
		}
		
		const FRigVMOperand* ExistingOperandNoOffset = OperandMap.Find(OperandNoOffset);
		if (!ExistingOperandNoOffset)
		{
			checkf(false, TEXT("Missing remapped operand for callable '%s'"), *InHeader.LibraryPointer.GetLibraryNodePath());
			return FRigVMOperand();
		}

		FRigVMOperand NewOperand = *ExistingOperandNoOffset;
		if (bRemapToForwardedHandle)
		{
			if (const FRigVMOperand* ForwardedOperand = NewInterfaceOperandToForwardedHandle.Find(NewOperand))
			{
				NewOperand = *ForwardedOperand;
			}
		}

		if (InOldOperand.GetRegisterOffset() != INDEX_NONE)
		{
			const FRigVMOperand OperandJustOffset(InOldOperand.GetMemoryType(), INDEX_NONE, InOldOperand.GetRegisterOffset());
			const FRigVMOperand* ExistingOperandJustOffset = OperandMap.Find(OperandJustOffset);
			if (!ExistingOperandJustOffset)
			{
				checkf(false, TEXT("Missing remapped register offset for callable '%s'"), *InHeader.LibraryPointer.GetLibraryNodePath());
				return FRigVMOperand();
			}
			NewOperand.RegisterOffset = ExistingOperandJustOffset->RegisterOffset;
		}

		return NewOperand;
	};
	
	// now add the forwarded handles for all arguments.
    // we are only doing this if we are compiling a non-function bytecode
    NewCallable.Arguments = InCallable->Arguments;

    // if this is using a forwarded memory handle - we should look up the original property from it
    for (int32 ArgumentIndex = 0; ArgumentIndex < InCallable->Arguments.Num(); ArgumentIndex++)
    {
    	FRigVMCallableArgument& Argument = NewCallable.Arguments[ArgumentIndex]; 
    	Argument.InterfaceOperand = RemapOperand(Argument.InterfaceOperand, false);
    	check(Argument.InterfaceOperand.GetMemoryType() == ERigVMMemoryType::Work);
    	check(Argument.InterfaceOperand.GetRegisterOffset() == INDEX_NONE);

    	const FRigVMPropertyDescription& InterfaceDescription = WorkData.PropertyDescriptions[ERigVMMemoryType::Work][Argument.InterfaceOperand.GetRegisterIndex()];
    	check(InterfaceDescription.CPPTypeObject != FRigVMForwardedMemoryHandle::StaticStruct());
    	check(InterfaceDescription.CPPType == Argument.TypeString);

    	static const FString ForwardedHandleSuffix = TEXT("_ForwardedHandle");
    	const FString ForwardedNewName = InterfaceDescription.Name.ToString() + ForwardedHandleSuffix;

    	FRigVMFunctionCompilationPropertyDescription ForwardedDescription;
    	ForwardedDescription.Name = *ForwardedNewName;
    	ForwardedDescription.CPPType = FRigVMForwardedMemoryHandle::StaticStruct()->GetStructCPPName();
    	ForwardedDescription.CPPTypeObject = FRigVMForwardedMemoryHandle::StaticStruct();
    	ForwardedDescription.DefaultValue = TEXT("()");
    	Argument.ForwardedOperand = WorkData.FindOrAddProperty(ERigVMMemoryType::Work, ForwardedDescription, ForwardedNewName, true);
    	NewInterfaceOperandToForwardedHandle.Add(Argument.InterfaceOperand, Argument.ForwardedOperand);
    }

	ByteCode.CallableInfos.Add(NewCallable);


	if (NewCallable.GetNumInstructions() > 0)
	{
		check(FunctionInstructions.IsValidIndex(InCallable->FirstInstruction));
		check(FunctionInstructions.IsValidIndex(InCallable->LastInstruction));
		ByteCode.InlineFunction(&InFunctionCompilationData->ByteCode, InCallable->FirstInstruction, InCallable->LastInstruction);

		auto RemapInstructionIndex = [InCallable, &NewCallable](const int32& InOldInstructionIndex)
		{
			check(FMath::IsWithinInclusive(InOldInstructionIndex, InCallable->FirstInstruction, InCallable->LastInstruction + 1))
			const int32 RemappedIndex = InOldInstructionIndex - InCallable->FirstInstruction + NewCallable.FirstInstruction;
			check(FMath::IsWithinInclusive(RemappedIndex, NewCallable.FirstInstruction, NewCallable.LastInstruction + 1))
			return RemappedIndex;
		};
	
		auto RemapOperandForByteCode = [&RemapOperand](const FRigVMOperand& InOldOperand)
		{
			return RemapOperand(InOldOperand, true);
		};

		// Import branch infos, with the proper instruction indices
		TMap<int32,int32> BranchIndexMap;
		for(const FRigVMBranchInfo& BranchInfo : FunctionByteCode.BranchInfos)
		{
			if (!FMath::IsWithinInclusive(BranchInfo.InstructionIndex, InCallable->FirstInstruction, InCallable->LastInstruction) ||
				!FMath::IsWithinInclusive(BranchInfo.LastInstruction, InCallable->FirstInstruction, InCallable->LastInstruction))
			{
				continue;
			}
			if (!FMath::IsWithinInclusive(BranchInfo.FirstInstruction, InCallable->FirstInstruction, InCallable->LastInstruction))
			{
				if (BranchInfo.FirstInstruction > InCallable->LastInstruction + 1)
				{
					continue;
				}
				if (BranchInfo.Label != FRigVMStruct::ControlFlowCompletedName)
				{
					continue;
				}
			}

			FRigVMBranchInfo NewBranchInfo = BranchInfo;
			NewBranchInfo.InstructionIndex = RemapInstructionIndex(BranchInfo.InstructionIndex);
			NewBranchInfo.FirstInstruction = RemapInstructionIndex(BranchInfo.FirstInstruction);
			NewBranchInfo.LastInstruction = RemapInstructionIndex(BranchInfo.LastInstruction);
			
			const int32 NewBranchIndex = ByteCode.AddBranchInfo(NewBranchInfo);
			BranchIndexMap.Add(BranchInfo.Index, NewBranchIndex);
		}

		auto RemapBranchInfoIndex = [&BranchIndexMap](const int32& InBranchIndex)
		{
			return BranchIndexMap.FindChecked(InBranchIndex);
		};

		auto RemapCallableIndex = [&ByteCode, &FunctionByteCode](const int32& InCallableIndex)
		{
			const FRigVMCallableInfo* OldCallable = FunctionByteCode.GetCallable(InCallableIndex);
			check(OldCallable);
			const FRigVMCallableInfo* NewCallable = ByteCode.FindCallable(OldCallable->FunctionHash);
			check(NewCallable);
			return NewCallable->Index;
		};
		
		// remap all operand, branches and callables
		const FRigVMInstructionArray Instructions = ByteCode.GetInstructions();
		for (int32 OldInstructionIndex = InCallable->FirstInstruction; OldInstructionIndex <= InCallable->LastInstruction; ++OldInstructionIndex)
		{
			if (!WorkData.RemapInstructionAt(
				FunctionInstructions[OldInstructionIndex],
				InHeader.LibraryPointer.GetLibraryNodePath(),
				InFunctionCompilationData,
				ByteCode,
				Instructions,
				nullptr,
				RemapInstructionIndex,
				RemapOperandForByteCode,
				RemapBranchInfoIndex,
				RemapCallableIndex))
			{
				return false;
			}
		}
	}
	return true;
}

bool URigVMCompiler::TraverseExpression(const FRigVMExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (WorkData.ExprToSkip.Contains(InExpr))
	{
		return true;
	}

	// if we hit an expression which is on a different block
	// take care of redirecting this to the RunInstructions op
	if(!WorkData.TraversalExpressions.IsEmpty())
	{
		const TOptional<uint32> PreviousBlockHash = WorkData.TraversalExpressions.Last()->GetBlockCombinationHash();
		const TOptional<uint32> NextBlockHash = InExpr->GetBlockCombinationHash();

		if((PreviousBlockHash != NextBlockHash) && NextBlockHash.IsSet())
		{
			TSharedPtr<FRigVMCompilerWorkData::FLazyBlockInfo>& BlockInfo = WorkData.LazyBlocks.FindChecked(NextBlockHash.GetValue());

			// if we are hitting the block for the first time - let's register the execution state operand
			if(WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
			{
				if(!BlockInfo->ExecuteStateOperand.IsValid())
				{
					static constexpr TCHAR Format[] = TEXT("BlockExecuteState_%u");
					const FString BlockStateName = FString::Printf(Format, NextBlockHash.GetValue());
					BlockInfo->ExecuteStateOperand = WorkData.AddProperty(
						ERigVMMemoryType::Work,
						*BlockStateName,
						FRigVMInstructionSetExecuteState::StaticStruct()->GetStructCPPName(),
						FRigVMInstructionSetExecuteState::StaticStruct()
					);
				}
			}
			else
			{
				check(BlockInfo->ExecuteStateOperand.IsValid());

				FRigVMByteCode& ByteCode = WorkData.VM->GetByteCode();

				// add an operator to invoke the block lazily
				const int32 RunInstructionsOpIndex = ByteCode.GetNumInstructions();
				BlockInfo->RunInstructionsToUpdate.Add(
					ByteCode.AddRunInstructionsOp(BlockInfo->ExecuteStateOperand, INDEX_NONE, INDEX_NONE)
				);
				if (WorkData.Settings.SetupNodeInstructionIndex)
				{
					const FRigVMCallstack Callstack = InExpr->GetProxy().GetCallstack();
					if(Callstack.Num() > 0)
					{
						WorkData.VM->GetByteCode().SetSubject(RunInstructionsOpIndex, Callstack.GetCallPath(), Callstack.GetStack());
					}
				}

				// mark the block to be processed
				if(!BlockInfo->bProcessed)
				{
					WorkData.LazyBlocksToProcess.AddUnique(NextBlockHash.GetValue());
				}

				// return here and stop traversal
				return true;
			}
		}
	}

	if(WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
	{
		// skip any expression which is not part of this block
		if(WorkData.CurrentBlockHash.IsSet())
		{
			const TOptional<uint32> NextBlockHash = InExpr->GetBlockCombinationHash();
			if(WorkData.CurrentBlockHash != NextBlockHash)
			{
				return true;
			}
		}
	}

	if (WorkData.ExprComplete.Contains(InExpr))
	{
		return true;
	}
	WorkData.ExprComplete.Add(InExpr, true);

	struct FTraversalGuard
	{
		FTraversalGuard(const FRigVMExprAST* InExpr, FRigVMCompilerWorkData& InWorkData)
			: WorkData(InWorkData)
		{
			WorkData.TraversalExpressions.Push(InExpr);
		}

		~FTraversalGuard()
		{
			WorkData.TraversalExpressions.Pop();
		}
		
		FRigVMCompilerWorkData& WorkData;
	};

	const FTraversalGuard TraversalGuard(InExpr, WorkData);
	
	switch (InExpr->GetType())
	{
		case FRigVMExprAST::EType::Block:
		{
			return TraverseBlock(InExpr->To<FRigVMBlockExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Entry:
		{
			return TraverseEntry(InExpr->To<FRigVMEntryExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::CallExtern:
		{
			const FRigVMCallExternExprAST* CallExternExpr = InExpr->To<FRigVMCallExternExprAST>();
			return TraverseCallExtern(CallExternExpr, WorkData);
		}
		case FRigVMExprAST::EType::InlineFunction:
		{
			const FRigVMInlineFunctionExprAST* InlineExpr = InExpr->To<FRigVMInlineFunctionExprAST>();
			return TraverseInlineFunction(InlineExpr, WorkData);
		}
		case FRigVMExprAST::EType::NoOp:
		{
			return TraverseNoOp(InExpr->To<FRigVMNoOpExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Var:
		{
			return TraverseVar(InExpr->To<FRigVMVarExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Literal:
		{
			return TraverseLiteral(InExpr->To<FRigVMLiteralExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Assign:
		{
			return TraverseAssign(InExpr->To<FRigVMAssignExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Copy:
		{
			return TraverseCopy(InExpr->To<FRigVMCopyExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::CachedValue:
		{
			return TraverseCachedValue(InExpr->To<FRigVMCachedValueExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::Exit:
		{
			return TraverseExit(InExpr->To<FRigVMExitExprAST>(), WorkData);
		}
		case FRigVMExprAST::EType::InvokeEntry:
		{
			return TraverseInvokeEntry(InExpr->To<FRigVMInvokeEntryExprAST>(), WorkData);
		}
		default:
		{
			ensure(false);
			break;
		}
	}
	return false;
}

bool URigVMCompiler::TraverseChildren(const FRigVMExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	for (FRigVMExprAST* ChildExpr : *InExpr)
	{
		if (!TraverseExpression(ChildExpr, WorkData))
		{
			return false;
		}
	}
	return true;
}

bool URigVMCompiler::TraverseBlock(const FRigVMBlockExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (InExpr->IsObsolete())
	{
		return true;
	}

	if (InExpr->NumChildren() == 0)
	{
		return true;
	}

	// check if the block is under a lazy pin, in which case we need to set up a branch info
	URigVMNode* CallExternNode = nullptr;
	FRigVMBranchInfo BranchInfo;
	if(WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
	{
		if(const FRigVMExprAST* ParentExpr = InExpr->GetParent())
		{
			if(const FRigVMExprAST* GrandParentExpr = ParentExpr->GetParent())
			{
				if(GrandParentExpr->IsA(FRigVMExprAST::CallExtern))
				{
					const URigVMPin* Pin = nullptr;
					if(ParentExpr->IsA(FRigVMExprAST::Var))
					{
						Pin = ParentExpr->To<FRigVMVarExprAST>()->GetPin();
					}
					else if(ParentExpr->IsA(FRigVMExprAST::CachedValue))
					{
						Pin = ParentExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr()->GetPin();
					}

					if(Pin)
					{
						URigVMPin* RootPin = Pin->GetRootPin();
						if(RootPin->IsLazy())
						{
							CallExternNode = RootPin->GetNode();
							
							if(RootPin->IsFixedSizeArray() && Pin->GetParentPin() == RootPin)
							{
								BranchInfo.Label = FRigVMBranchInfo::GetFixedArrayLabel(RootPin->GetFName(), Pin->GetFName());
							}
							else
							{
								BranchInfo.Label = RootPin->GetFName();
							}
							BranchInfo.InstructionIndex = INDEX_NONE; // we'll fill in the instruction info later
							BranchInfo.FirstInstruction = WorkData.VM->GetByteCode().GetNumInstructions();

							// find the argument index for the given pin
							if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(CallExternNode))
							{
								if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
								{
									int32 FlatArgumentIndex = 0;
									for(int32 ArgumentIndex = 0; ArgumentIndex != Template->NumArguments(); ArgumentIndex++)
									{
										const FRigVMTemplateArgument* Argument = Template->GetArgument(ArgumentIndex);
										if(Template->GetArgument(ArgumentIndex)->GetName() == RootPin->GetFName())
										{
											BranchInfo.ArgumentIndex = FlatArgumentIndex;

											if(RootPin->IsFixedSizeArray() && Pin->GetParentPin() == RootPin)
											{
												BranchInfo.ArgumentIndex += Pin->GetPinIndex();
											}
											break;
										}

										if(const URigVMPin* PinForArgument = RootPin->GetNode()->FindPin(Argument->Name.ToString()))
										{
											if(PinForArgument->IsFixedSizeArray())
											{
												FlatArgumentIndex += RootPin->GetSubPins().Num();
												continue;
											}
										}
										
										FlatArgumentIndex++;
									}
								}
								// we also need to deal with unit nodes separately here. if a unit node does
								// not offer a valid backing template - we need to visit its properties. since
								// templates don't contain executecontext type arguments anymore - we need
								// to step over them as well here.
								else if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(CallExternNode))
								{
									if(const FRigVMFunction* Function = UnitNode->GetResolvedFunction())
									{
										for(int32 ArgumentIndex = 0; ArgumentIndex != Function->Arguments.Num(); ArgumentIndex++)
										{
											const FRigVMFunctionArgument& Argument = Function->Arguments[ArgumentIndex];
											if(Argument.Name == RootPin->GetFName())
											{
												BranchInfo.ArgumentIndex = ArgumentIndex;
												break;
											}
										}
									}
								}	
							}

							check(BranchInfo.ArgumentIndex != INDEX_NONE);
						}
					}
				}
			}
		}
	}

	if(WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions && !BranchInfo.Label.IsNone())
	{
		// We need to make sure lazy blocks are properly populated, with all the operations that need to run
		// during the evaluation of the lazy branch, even if some of the expressions
		// have already been visited in other parts of the traversal. See RigVM.Compiler.IfFromSameNode unit test. 
		TGuardValue<TMap<const FRigVMExprAST*, bool>> ExprCompletedGuard(WorkData.ExprComplete, {});
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}
	}
	else
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}
	}

	if(!BranchInfo.Label.IsNone())
	{
		BranchInfo.LastInstruction = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
		WorkData.BranchInfos.FindOrAdd(CallExternNode).Add(BranchInfo);
	}

	return true;
}

bool URigVMCompiler::TraverseEntry(const FRigVMEntryExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InExpr->GetNode()))
	{
		if(!ValidateNode(WorkData.Settings, UnitNode))
		{
			return false;
		}

		if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
		{
			TSharedPtr<FStructOnScope> DefaultStruct = UnitNode->ConstructStructInstance();
			if (!TraverseChildren(InExpr, WorkData))
			{
				return false;
			}
		}
		else
		{
			TArray<FRigVMOperand> Operands;
			for (FRigVMExprAST* ChildExpr : *InExpr)
			{
				if (ChildExpr->IsA(FRigVMExprAST::EType::Var))
				{
					const FRigVMVarExprAST* SourceVarExpr = GetSourceVarExpr(ChildExpr);
					if(!SourceVarExpr->IsExecuteContext())
					{
						Operands.Add(WorkData.ExprToOperand.FindChecked(SourceVarExpr));
					}
				}
				else
				{
					break;
				}
			}

			// setup the instruction
			int32 FunctionIndex = WorkData.VM->AddRigVMFunction(UnitNode->GetScriptStruct(), UnitNode->GetMethodName());
#if WITH_EDITOR
			const FRigVMFunction* Function = WorkData.VM->GetFunctions()[FunctionIndex];
			check(Function);
			if (Function->Factory)
			{
				checkf(Function->Arguments.Num() <= Operands.Num(), TEXT("%s: invalid number of operands (%d) for dispatch '%s' - expected at least (%d)."), *GetPackage()->GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
			}
			else
			{
				checkf(Function->Arguments.Num() == Operands.Num(), TEXT("%s: invalid number of operands (%d) for function '%s' - expected (%d)."), *GetPackage()->GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
			}
#endif
			WorkData.VM->GetByteCode().AddExecuteOp(FunctionIndex, Operands, 0, 0);
		
			int32 EntryInstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
			FName Entryname = UnitNode->GetEventName();

			if (WorkData.VM->GetByteCode().FindEntryIndex(Entryname) == INDEX_NONE)
			{
				FRigVMByteCodeEntry Entry;
				Entry.Name = Entryname;
				Entry.InstructionIndex = EntryInstructionIndex;
				WorkData.VM->GetByteCode().Entries.Add(Entry);
			}

			if (WorkData.Settings.SetupNodeInstructionIndex)
			{
				const FRigVMCallstack Callstack = InExpr->GetProxy().GetCallstack();
				WorkData.VM->GetByteCode().SetSubject(EntryInstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
			}
		}
	}
	else if (CurrentlyCompilingFunctionNode && InExpr->NumParents() == 0)
	{
		// Initialize local variables
		if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
		{
			TArray<FRigVMGraphVariableDescription> LocalVariables = CurrentlyCompilingFunctionNode->GetContainedGraph()->GetLocalVariables();
			for (const FRigVMGraphVariableDescription& Variable : LocalVariables)
			{
				FString Path = FString::Printf(TEXT("LocalVariableDefault::%s|%s::Const"), *CurrentlyCompilingFunctionNode->GetContainedGraph()->GetGraphName(), *Variable.Name.ToString());
				FRigVMOperand Operand = WorkData.AddProperty(ERigVMMemoryType::Literal, *Path, Variable.CPPType, Variable.CPPTypeObject, Variable.DefaultValue);
				WorkData.PinPathToOperand->Add(Path, Operand);
			}
		}
		else
		{
			TArray<FRigVMGraphVariableDescription> LocalVariables = CurrentlyCompilingFunctionNode->GetContainedGraph()->GetLocalVariables();
			for (const FRigVMGraphVariableDescription& Variable : LocalVariables)
			{
				FString TargetPath = FString::Printf(TEXT("LocalVariable::%s|%s"), *CurrentlyCompilingFunctionNode->GetContainedGraph()->GetGraphName(), *Variable.Name.ToString());
				FString SourcePath = FString::Printf(TEXT("LocalVariableDefault::%s|%s::Const"), *CurrentlyCompilingFunctionNode->GetContainedGraph()->GetGraphName(), *Variable.Name.ToString());
				FRigVMOperand* TargetPtr = WorkData.PinPathToOperand->Find(TargetPath);
				FRigVMOperand* SourcePtr = WorkData.PinPathToOperand->Find(SourcePath);
				if (SourcePtr && TargetPtr)
				{
					const FRigVMOperand& Source = *SourcePtr;
					const FRigVMOperand& Target = *TargetPtr;
	
					WorkData.VM->GetByteCode().AddCopyOp(WorkData.VM->GetCopyOpForOperands(Source, Target));
				}
			}
		}
	}

	return TraverseChildren(InExpr, WorkData);
}

/**
 * Walks the AST parent chain from InExpr to find the nearest enclosing loop
 * whose BODY contains InExpr (and whose body block is sliced). Loops reached
 * through their Completed branch are skipped, because by the time InExpr
 * fires those loops have already finished iterating and their per-iteration
 * BlockToRun register is no longer being read.
 * Stops (returns nullptr) if an InlineFunction boundary is encountered
 * before a matching loop is found.
 *
 * This is shared by Break (SetupMemory + BuildInstructions phases) and Continue
 * (SetupMemory + BuildInstructions phases) to avoid duplicating the walk logic.
 *
 * @param InExpr  The expression whose parents are walked (typically a Break/Continue CallExternExprAST).
 * @return        The nearest enclosing loop that contains InExpr in its body and has a sliced block; nullptr otherwise.
 */
static const FRigVMCallExternExprAST* FindEnclosingSlicedLoopExpr(const FRigVMExprAST* InExpr)
{
	for (const FRigVMExprAST* ParentExpr = InExpr->GetParent();
		 ParentExpr != nullptr;
		 ParentExpr = ParentExpr->GetParent())
	{
		// Stop at inline-function boundaries — Break/Continue must not jump across function scope.
		// In the current AST topology this is unreachable (inlined function bodies are pre-compiled
		// separately), but guards against future AST changes.
		if (ParentExpr->IsA(FRigVMExprAST::EType::InlineFunction))
		{
			break;
		}

		if (!ParentExpr->IsA(FRigVMExprAST::EType::CallExtern))
		{
			continue;
		}

		const FRigVMCallExternExprAST* ParentCallExtern = ParentExpr->To<FRigVMCallExternExprAST>();
		URigVMNode* ParentNode = ParentCallExtern->GetNode();

		if (!ParentNode->IsLoopNode())
		{
			continue;
		}

		// Bind only when InExpr is inside this loop's BODY subtree. Asking the question
		// positively (rather than "is InExpr under Completed?") fails safe in the user's
		// favor when the AST is a DAG — a Break that has multiple ancestry paths and is
		// reachable through the body block is still bound to this loop, even if some
		// other path also passes through the Completed branch.
		// Example: ForLoop.Completed → Branch → Break is meant to break an outer loop —
		// the body-parented test fails for the inner ForLoop, the walk continues outward
		// and finds the outer loop.
		// URigVMNode::IsLoopNode() guarantees GetControlFlowBlocks()[0] is the body pin
		// name (ExecuteContextName or ExecutePinName).
		const TArray<FName>& Blocks = ParentNode->GetControlFlowBlocks();
		const FName BodyPinName = Blocks[0];
		const FRigVMExprAST* BodyPinExpr = ParentCallExtern->FindExprWithPinName(BodyPinName);
		if (BodyPinExpr == nullptr || !InExpr->IsParentedTo(BodyPinExpr))
		{
			continue;
		}

		// IsLoopNode() does not imply slicing — verify that at least one block is sliced.
		// A non-sliced loop (if one existed) would have no per-iteration BlockToRun memory.
		for (const FName& BlockName : Blocks)
		{
			if (ParentNode->IsControlFlowBlockSliced(BlockName))
			{
				return ParentCallExtern;
			}
		}
	}
	return nullptr;
}

bool URigVMCompiler::TraverseCallExtern(const FRigVMCallExternExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	URigVMNode* Node = InExpr->GetNode();
	URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node);
	URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node);
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	if(!ValidateNode(WorkData.Settings, UnitNode, false) && !ValidateNode(WorkData.Settings, DispatchNode, false))
	{
		return false;
	}

	auto CheckExecuteStruct = [this, &WorkData](URigVMNode* Subject, const UScriptStruct* ExecuteStruct) -> bool
	{
		if(ExecuteStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
		{
			// top level expected execute struct is provided by the graph
			const UScriptStruct* SpecializedExecuteStruct = WorkData.ExecuteContextStruct;
			if(!SpecializedExecuteStruct->IsChildOf(ExecuteStruct))
			{
				static constexpr TCHAR UnknownExecuteContextMessage[] = TEXT("[%s] Node @@ uses an unexpected execute type '%s'. This graph uses '%s'.");
				WorkData.Settings.Report(EMessageSeverity::Error, Subject, FString::Printf(
					UnknownExecuteContextMessage, *Subject->GetPackage()->GetPathName(), *ExecuteStruct->GetStructCPPName(), *SpecializedExecuteStruct->GetStructCPPName()));
				return false;
			}
		}
		return true;
	};
	
	if(UnitNode)
	{
		const UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct(); 
		if(ScriptStruct == nullptr)
		{
			static const FString UnresolvedMessage = FString::Printf(TEXT("[%s] Node @@ is unresolved."), *UnitNode->GetPackage()->GetPathName());
			WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, UnresolvedMessage);
			return false;
		}

		// check execute pins for compatibility
		for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
		{
			if(const FStructProperty* StructProperty = CastField<FStructProperty>(*It))
			{
				if(!CheckExecuteStruct(UnitNode, StructProperty->Struct))
				{
					return false;
				}
			}
		}
	}

	if(DispatchNode)
	{
		if(DispatchNode->GetFactory() == nullptr)
		{
			static const FString UnresolvedDispatchMessage = FString::Printf(TEXT("[%s] Dispatch node @@ has no factory."), *DispatchNode->GetPackage()->GetPathName());
			WorkData.Settings.Report(EMessageSeverity::Error, DispatchNode, UnresolvedDispatchMessage);
			return false;
		}

		// check execute pins for compatibility
		if(!CheckExecuteStruct(DispatchNode, DispatchNode->GetFactory()->GetExecuteContextStruct()))
		{
			return false;
		}
	}

	const FRigVMCallExternExprAST* CallExternExpr = InExpr->To<FRigVMCallExternExprAST>();
	int32 CallExternInstructionIndex = INDEX_NONE;
	const FRigVMCallstack Callstack = InExpr->GetProxy().GetCallstack();

	// When a Break or Continue node is detected during the BuildInstructions phase,
	// this is set to the enclosing loop's expression so a JumpForward can be emitted
	// after the CallExtern op for the Break/Continue node.
	const FRigVMCallExternExprAST* EnclosingLoopExprForJump = nullptr;

	// ── Break-from-loop special case ──────────────────────────────────
	// When the current node is FRigVMFunction_BreakFromLoop, its BlockToRun
	// pin must alias the enclosing loop's BlockToRun memory slot instead of
	// allocating a new register. We walk the AST parent chain to locate the
	// nearest enclosing sliced loop, retrieve its already-registered
	// BlockToRun operand, and pre-insert it into ExprToOperand for the
	// Break's own BlockToRun var expr. FindOrAddRegister will then find the
	// entry and reuse the operand — no new memory is allocated.
	if (UnitNode && UnitNode->GetScriptStruct() == FRigVMFunction_BreakFromLoop::StaticStruct())
	{
		// Only alias during the SetupMemory phase — that is when
		// FindOrAddRegister is called and would otherwise allocate.
		if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
		{
			// Walk the AST parent chain to find the nearest enclosing sliced loop.
			// See FindEnclosingSlicedLoopExpr (above TraverseCallExtern) for the walk
			// logic and cross-scope safety rationale.
			const FRigVMCallExternExprAST* EnclosingLoopExpr = FindEnclosingSlicedLoopExpr(InExpr);

			// Error: Break used outside any loop scope.
			if (EnclosingLoopExpr == nullptr)
			{
				static const FString BreakOutsideLoopMessage = TEXT("Break From Loop node @@ is not inside a loop.");
				WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, BreakOutsideLoopMessage);
				return false;
			}

			// Retrieve the loop's BlockToRun var expr.
			const FRigVMVarExprAST* LoopBlockToRunVarExpr =
				EnclosingLoopExpr->FindVarWithPinName(FRigVMStruct::ControlFlowBlockToRunName);
			if (!LoopBlockToRunVarExpr)
			{
				static const FString MissingBlockToRunMessage = TEXT("Break From Loop node @@ could not find BlockToRun on enclosing loop.");
				WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, MissingBlockToRunMessage);
				return false;
			}

			// The Break node lives inside the loop's body block, which may be
			// traversed before the loop's BlockToRun var expr has been registered
			// in ExprToOperand (child ordering is not guaranteed). Force-traverse
			// the loop's BlockToRun now so FindOrAddRegister allocates its operand.
			const FRigVMVarExprAST* LoopBlockToRunSourceExpr = GetSourceVarExpr(LoopBlockToRunVarExpr);
			if (LoopBlockToRunSourceExpr == nullptr)
			{
				LoopBlockToRunSourceExpr = LoopBlockToRunVarExpr;
			}

			if (!WorkData.ExprToOperand.Contains(LoopBlockToRunSourceExpr))
			{
				if (!TraverseExpression(const_cast<FRigVMExprAST*>(static_cast<const FRigVMExprAST*>(LoopBlockToRunVarExpr)), WorkData))
				{
					return false;
				}
			}

			const FRigVMOperand* LoopOperandPtr = WorkData.ExprToOperand.Find(LoopBlockToRunSourceExpr);
			if (!LoopOperandPtr)
			{
				static const FString UnregisteredOperandMessage = TEXT("Break From Loop node @@ found enclosing loop but its BlockToRun operand is not yet registered.");
				WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, UnregisteredOperandMessage);
				return false;
			}

			// Pre-insert the Break's BlockToRun var expr with the loop's operand so
			// FindOrAddRegister reuses it during TraverseChildren below.
			const FRigVMVarExprAST* BreakBlockToRunVarExpr =
				InExpr->FindVarWithPinName(FRigVMStruct::ControlFlowBlockToRunName);
			if (!BreakBlockToRunVarExpr)
			{
				static const FString MissingBreakBlockToRunMessage = TEXT("Break From Loop node @@ has no BlockToRun pin expression.");
				WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, MissingBreakBlockToRunMessage);
				return false;
			}

			const FRigVMVarExprAST* BreakBlockToRunSourceExpr = GetSourceVarExpr(BreakBlockToRunVarExpr);
			if (!WorkData.ExprToOperand.Contains(BreakBlockToRunSourceExpr))
			{
				WorkData.ExprToOperand.Add(BreakBlockToRunSourceExpr, *LoopOperandPtr);
			}
		}
		// ── Break: BuildInstructions phase — locate enclosing loop for JumpForward ──
		// The CallExtern op for the Break node is emitted by the standard path below.
		// We find the enclosing loop here and store it in EnclosingLoopExprForJump so
		// that a JumpForward placeholder can be emitted immediately after the CallExtern.
		// If nullptr, SetupMemory already reported an error and returned false, so we
		// will not reach this point for an out-of-loop Break.
		else if (WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
		{
			EnclosingLoopExprForJump = FindEnclosingSlicedLoopExpr(InExpr);
		}
		// In other phases, fall through to the normal CallExtern path —
		// the aliased operand is already in ExprToOperand from the SetupMemory pass.
	}

	// ── Continue-loop special case ──────────────────────────────────
	// When the current node is FRigVMFunction_ContinueLoop, the compiler emits a
	// JumpForward after its CallExtern op to skip the remaining loop body nodes.
	// Unlike Break, Continue has no BlockToRun pin and needs no operand aliasing.
	if (UnitNode && UnitNode->GetScriptStruct() == FRigVMFunction_ContinueLoop::StaticStruct())
	{
		// ── Continue: SetupMemory phase — validate that we're inside a loop ──
		if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
		{
			// Error: Continue used outside any loop scope.
			if (FindEnclosingSlicedLoopExpr(InExpr) == nullptr)
			{
				static const FString ContinueOutsideLoopMessage = TEXT("Continue Loop node @@ is not inside a loop.");
				WorkData.Settings.Report(EMessageSeverity::Error, UnitNode, ContinueOutsideLoopMessage);
				return false;
			}
			// No operand aliasing needed — Continue has no BlockToRun pin.
		}
		// ── Continue: BuildInstructions phase — locate enclosing loop for JumpForward ──
		// If nullptr, SetupMemory already reported an error for an out-of-loop Continue.
		else if (WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
		{
			EnclosingLoopExprForJump = FindEnclosingSlicedLoopExpr(InExpr);
		}
	}

	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}

		WorkData.SetupPropertyPathsForFoldedCopies(InExpr);

		if(WorkData.Settings.ASTSettings.bSetupTraits)
		{
			const TArray<URigVMPin*> TraitPins = Node->GetTraitPins(); 
			if(!TraitPins.IsEmpty())
			{
				// also take care of the empty trait list
				if(!WorkData.TraitListLiterals.Contains(nullptr))
				{
					const FName Name = WorkData.GetUniquePropertyName(ERigVMMemoryType::Literal, TEXT("EmptyTraitList"));
					const FRigVMOperand& ListOperand = WorkData.AddProperty(ERigVMMemoryType::Literal, Name, RigVMTypeUtils::ArrayTypeFromBaseType(RigVMTypeUtils::Int32Type), nullptr, TEXT("()"));
					WorkData.TraitListLiterals.Add(nullptr, ListOperand);
				}

				if(!WorkData.TraitListLiterals.Contains(Node))
				{
					TArray<FString> DefaultValues;
					for(const URigVMPin* TraitPin : TraitPins)
					{
						auto AddPinRegister = [this, &DefaultValues, &WorkData](const FRigVMVarExprAST* InPinExpr)
						{
							const FRigVMOperand TraitOperand = FindOrAddRegister(InPinExpr, WorkData);
							check(TraitOperand.GetMemoryType() == ERigVMMemoryType::Work);
							DefaultValues.Add(FString::FromInt(TraitOperand.GetRegisterIndex()));
						};

						if(const FRigVMVarExprAST* TraitPinExpr = InExpr->FindVarWithPinName(TraitPin->GetFName()))
						{
							// Add programmatic pin expressions first. This is done to allow memory handles to be built at runtime and passed to their
							// respective 'owning' traits
							const TArray<URigVMPin*> ProgrammaticPins = TraitPin->GetProgrammaticSubPins(); 
							for(URigVMPin* ProgrammaticPin : ProgrammaticPins)
							{
								if(const FRigVMVarExprAST* ProgrammaticPinExpr = InExpr->FindVarWithPinName(*ProgrammaticPin->GetPinPath()))
								{
									AddPinRegister(ProgrammaticPinExpr);
								}
							}

							AddPinRegister(TraitPinExpr);
						}
					}

					if(!DefaultValues.IsEmpty())
					{
						const FName Name = WorkData.GetUniquePropertyName(ERigVMMemoryType::Literal, TEXT("TraitList"));
						const FString DefaultValue = FString::Printf(TEXT("(%s)"), *FString::Join(DefaultValues, TEXT(",")));
						const FRigVMOperand& ListOperand = WorkData.AddProperty(ERigVMMemoryType::Literal, Name, RigVMTypeUtils::ArrayTypeFromBaseType(RigVMTypeUtils::Int32Type), nullptr, DefaultValue);
						WorkData.TraitListLiterals.Add(Node, ListOperand);
					}
				}
			}
		}
	}
	else
	{
		TArray<FRigVMOperand> Operands;
		TArray<const FRigVMVarExprAST*> VarExpressionsPerOperand;

		// iterate over the child expressions in the order of the arguments on the function
		const FRigVMFunction* Function = nullptr;

		if(UnitNode)
		{
			Function = Registry.FindFunction(UnitNode->GetScriptStruct(), *UnitNode->GetMethodName().ToString());
		}
		else if(DispatchNode)
		{
			Function = DispatchNode->GetResolvedFunction();
		}

		checkf(Function, TEXT("Could not find function for node %s in package %s"), *Node->GetPathName(), *GetPackage()->GetPathName());

		FRigVMOperand CountOperand;
		FRigVMOperand IndexOperand;
		FRigVMOperand BlockToRunOperand;
		for(const FRigVMFunctionArgument& Argument : Function->GetArguments())
		{
			auto ProcessArgument = [
				&WorkData,
				Argument,
				&Operands,
				&VarExpressionsPerOperand,
				&BlockToRunOperand,
				&CountOperand,
				&IndexOperand
			](const FRigVMExprAST* InExpr)
			{
				const FRigVMVarExprAST* SourceVarExpr = nullptr;
				if (InExpr->GetType() == FRigVMExprAST::EType::CachedValue)
				{
					SourceVarExpr = GetSourceVarExpr(InExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr());
				}
				else if (InExpr->IsA(FRigVMExprAST::EType::Var))
				{
					SourceVarExpr = GetSourceVarExpr(InExpr->To<FRigVMVarExprAST>());
				}
				else
				{
					return;
				}

				Operands.Add(WorkData.ExprToOperand.FindChecked(SourceVarExpr));
				VarExpressionsPerOperand.Add(SourceVarExpr);

				if(Argument.Name == FRigVMStruct::ControlFlowBlockToRunName)
				{
					BlockToRunOperand = Operands.Last();
				}
				else if(Argument.Name == FRigVMStruct::ControlFlowCountName)
				{
					CountOperand = Operands.Last();
				}
				else if(Argument.Name == FRigVMStruct::ControlFlowIndexName)
				{
					IndexOperand = Operands.Last();
				}
				
			};

			if(URigVMPin* Pin = InExpr->GetNode()->FindPin(Argument.Name))
			{
				if(Pin->IsFixedSizeArray())
				{
					for(URigVMPin* SubPin : Pin->GetSubPins())
					{
						const FString PinName = FRigVMBranchInfo::GetFixedArrayLabel(Pin->GetName(), SubPin->GetName());
						const FRigVMExprAST* SubPinExpr = InExpr->FindExprWithPinName(*PinName);
						check(SubPinExpr);
						ProcessArgument(SubPinExpr);
					}
					continue;
				}
			}
			
			const FRigVMExprAST* ChildExpr = InExpr->FindExprWithPinName(Argument.Name);
			check(ChildExpr);
			ProcessArgument(ChildExpr);
		}

		// for call externs we can only apply the segment path after all operands are known
		for (const TPair<int32, FString>& Pair : CallExternExpr->SegmentPathForChild)
		{
			FRigVMRegistryReadLock LockedRegistry;
			
			const FString& SegmentPath = Pair.Value;
			check(!SegmentPath.IsEmpty());

			int32 bNumUpdatedOperands = 0;
			const FName& PinName = CallExternExpr->GetPinNameForChildIndex(Pair.Key);
			for (int32 OperandIndex = 0; OperandIndex < Operands.Num(); OperandIndex++)
			{
				const FName& ArgumentName = Function->GetArgumentNameForOperandIndex(OperandIndex, Operands.Num(), LockedRegistry);
				if (ArgumentName != PinName)
				{
					continue;
				}
				
				const FRigVMVarExprAST* SourceVarExpr = VarExpressionsPerOperand[OperandIndex];
				check(SourceVarExpr);

				const FString HeadCPPType = SourceVarExpr->GetPin()->GetCPPType();
				const int32 RegisterOffsetIndex = WorkData.FindOrAddPropertyPath(Operands[OperandIndex], HeadCPPType, SegmentPath);
				check(RegisterOffsetIndex != INDEX_NONE);
				Operands[OperandIndex].RegisterOffset = static_cast<uint16>(RegisterOffsetIndex);
				bNumUpdatedOperands++;
			}

			check(bNumUpdatedOperands == 1);
		}

		// make sure to skip the output blocks while we are traversing this call extern
		TArray<const FRigVMExprAST*> ExpressionsToSkip;
		TArray<int32> BranchIndices;
		if(Node->IsControlFlowNode())
		{
			const TArray<FName>& BlockNames = Node->GetControlFlowBlocks();
			BranchIndices.Reserve(BlockNames.Num());
			
			for(const FName& BlockName : BlockNames)
			{
				const FRigVMVarExprAST* BlockExpr = InExpr->FindVarWithPinName(BlockName);
				check(BlockExpr);
				WorkData.ExprToSkip.AddUnique(BlockExpr);
				BranchIndices.Add(WorkData.VM->GetByteCode().AddBranchInfo(FRigVMBranchInfo()));
			}
		}

		// traverse all non-lazy children
		TArray<const FRigVMExprAST*> LazyChildExprs;
		for (const FRigVMExprAST* ChildExpr : *InExpr)
		{
			// if there's a direct child block under this - the pin may be lazy
			if(ChildExpr->IsA(FRigVMExprAST::Var) || ChildExpr->IsA(FRigVMExprAST::CachedValue))
			{
				if(const FRigVMExprAST* BlockExpr = ChildExpr->GetFirstChildOfType(FRigVMExprAST::Block))
				{
					if(BlockExpr->GetParent() == ChildExpr)
					{
						URigVMPin* Pin = nullptr;
						if(ChildExpr->IsA(FRigVMExprAST::Var))
						{
							Pin = ChildExpr->To<FRigVMVarExprAST>()->GetPin();
						}
						else
						{
							Pin = ChildExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr()->GetPin();
						}
						check(Pin);
						
						if(Pin->IsLazy())
						{
							LazyChildExprs.Add(ChildExpr);
							continue;
						}
					}
				}
			}
			if (!TraverseExpression(ChildExpr, WorkData))
			{
				return false;
			}
		}

		if(!LazyChildExprs.IsEmpty())
		{
			// set up an operator to skip the lazy branches 
			const int32 JumpToCallExternByte = WorkData.VM->GetByteCode().AddJumpOp(ERigVMOpCode::JumpForward, INDEX_NONE);
			const int32 JumpToCallExternInstruction = WorkData.VM->GetByteCode().GetNumInstructions() - 1;

			// traverse the lazy children 
			for (const FRigVMExprAST* ChildExpr : LazyChildExprs)
			{
				if (!TraverseExpression(ChildExpr, WorkData))
				{
					return false;
				}
			}

			// update the operator with the target instruction 
			const int32 InstructionsToJump = WorkData.VM->GetByteCode().GetNumInstructions() - JumpToCallExternInstruction;
			WorkData.VM->GetByteCode().GetOpAt<FRigVMJumpOp>(JumpToCallExternByte).InstructionIndex = InstructionsToJump;
		}

		int32 StartPredicateIndex = INDEX_NONE;
		int32 PredicateCount = 0;
		const TArray<FRigVMFunction>* Predicates = nullptr;
		if (UnitNode)
		{
			Predicates = Registry.GetPredicatesForStruct(Function->Struct->GetFName());
		}
		else if(DispatchNode)
		{
			FRigVMTemplateTypeMap TypeMap = DispatchNode->GetTemplatePinTypeMap(true);
			const FString PermutationName = DispatchNode->GetFactory()->GetPermutationName(TypeMap);
			Predicates = Registry.GetPredicatesForStruct(*PermutationName);
		}
		if (Predicates)
		{
			StartPredicateIndex = WorkData.VM->GetByteCode().PredicateBranches.Num();
			for (const FRigVMFunction& Predicate : *Predicates)
			{
				WorkData.VM->GetByteCode().AddPredicateBranch(FRigVMPredicateBranch());
				PredicateCount++;
			}
		}
		
		if(Node->IsControlFlowNode())
		{
			check(BlockToRunOperand.IsValid());
			WorkData.VM->GetByteCode().AddZeroOp(BlockToRunOperand);
		}

		if (Operands.Num() > 65535)
		{
			return false;
		}

		// setup the trait list for the context
		bool bSetupTraits = false;
		if(const FRigVMOperand* TraitListOperand = WorkData.TraitListLiterals.Find(Node))
		{
			if (WorkData.Settings.SetupNodeInstructionIndex)
			{
				WorkData.VM->GetByteCode().SetSubject(WorkData.VM->GetByteCode().GetNumInstructions(), Callstack.GetCallPath(), Callstack.GetStack());
			}
			WorkData.VM->GetByteCode().AddSetupTraitsOp(*TraitListOperand);
			bSetupTraits = true;
		}

		/*
		// sanity check the operand types
		check(Function->Arguments.Num() == Operands.Num() || Function->Factory != nullptr);
		if (Function->Arguments.Num() == Operands.Num() && Function->Factory == nullptr)
		{
			for (int32 ArgumentIndex = 0; ArgumentIndex < Function->Arguments.Num(); ArgumentIndex++)
			{
				const FRigVMFunctionArgument& Argument = Function->Arguments[ArgumentIndex];
				FString ExpectedType = Argument.Type;
				if (ExpectedType.Contains(TEXT("TSubclassOf<")))
				{
					continue;
				}

				if (ExpectedType.Contains(TEXT("TEnumAsByte<")))
				{
					ExpectedType.ReplaceInline(TEXT("TEnumAsByte<"), TEXT(""));
					ExpectedType.RemoveFromEnd(TEXT(">"));
				}

				const FRigVMOperand& Operand = Operands[ArgumentIndex];
				const FProperty* Property = WorkData.GetPropertyForOperand(Operand);
				check(Property != nullptr);

				FString CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(Property);

				// the argument may have an element type of an array, while the operand is a sliced array.
				if (ExpectedType != CPPType)
				{
					const FString SlicedArrayCPPType = RigVMTypeUtils::ArrayTypeFromBaseType(ExpectedType);
					if (SlicedArrayCPPType == CPPType)
					{
						// make sure to check that the pin is internal
						const FRigVMVarExprAST* SourceVarExpr = InExpr->FindVarWithPinName(Argument.Name);
						if (SourceVarExpr->GetType() == FRigVMExprAST::EType::CachedValue)
						{
							SourceVarExpr = GetSourceVarExpr(SourceVarExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr());
						}
						else if (SourceVarExpr->IsA(FRigVMExprAST::EType::Var))
						{
							SourceVarExpr = GetSourceVarExpr(SourceVarExpr->To<FRigVMVarExprAST>());
						}
						check(SourceVarExpr);
						check(SourceVarExpr->GetPinDirection() == ERigVMPinDirection::Hidden);
						ExpectedType = SlicedArrayCPPType;;
					}
				}
				
				check (ExpectedType == CPPType);
			}
		}
		*/
		
		// setup the instruction
		const int32 FunctionIndex = WorkData.VM->AddRigVMFunction(Function->GetName());
		check(FunctionIndex != INDEX_NONE);
#if WITH_EDITOR
		if (Function->Factory)
		{
			checkf(Function->Arguments.Num() <= Operands.Num(), TEXT("%s: invalid number of operands (%d) for dispatch '%s' - expected at least (%d)."), *GetPackage()->GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
		}
		else
		{
			checkf(Function->Arguments.Num() == Operands.Num(), TEXT("%s: invalid number of operands (%d) for function '%s' - expected (%d)."), *GetPackage()->GetPathName(), Operands.Num(), *Function->GetName(), Function->Arguments.Num());
		}
#endif
		const int32 CallExternByteIndex = WorkData.VM->GetByteCode().AddExecuteOp(FunctionIndex, Operands, StartPredicateIndex, PredicateCount);
		CallExternInstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
		const FRigVMInstruction CallExternInstruction(ERigVMOpCode::Execute, CallExternInstructionIndex, CallExternByteIndex);

		// setup the branch infos for this call extern instruction
		if(const TArray<FRigVMBranchInfo>* BranchInfosPtr = WorkData.BranchInfos.Find(Node))
		{
			const TArray<FRigVMBranchInfo>& BranchInfos = *BranchInfosPtr;
			for(FRigVMBranchInfo BranchInfo : BranchInfos)
			{
				BranchInfo.InstructionIndex = CallExternInstructionIndex;
				(void)WorkData.VM->GetByteCode().AddBranchInfo(BranchInfo);
			}
		}

#if WITH_EDITORONLY_DATA
		TArray<FRigVMOperand> InputOperands, OutputOperands;

		for(const URigVMPin* InputPin : Node->GetPins())
		{
			if(InputPin->IsExecuteContext())
			{
				continue;
			}

			int32 OperandIndex = Function->Arguments.IndexOfByPredicate([InputPin](const FRigVMFunctionArgument& FunctionArgument) -> bool
			{
				return FunctionArgument.Name == InputPin->GetName();
			});
			if(!Operands.IsValidIndex(OperandIndex))
			{
				continue;
			}
			if (InputPin->IsFixedSizeArray())
			{
				for (int32 SubPinIndex=0; SubPinIndex<InputPin->GetSubPins().Num(); ++SubPinIndex)
				{
					const URigVMPin* SubPin = InputPin->GetSubPins()[SubPinIndex];
					const FRigVMOperand& Operand = Operands[OperandIndex+SubPinIndex];
					if(SubPin->GetDirection() == ERigVMPinDirection::Output || SubPin->GetDirection() == ERigVMPinDirection::IO)
					{
						OutputOperands.Add(Operand);
					}

					if(SubPin->GetDirection() != ERigVMPinDirection::Input && SubPin->GetDirection() != ERigVMPinDirection::IO)
					{
						continue;
					}

					InputOperands.Add(Operand);
				}
			}
			else
			{
				const FRigVMOperand& Operand = Operands[OperandIndex];
				
				if(InputPin->GetDirection() == ERigVMPinDirection::Output || InputPin->GetDirection() == ERigVMPinDirection::IO)
				{
					OutputOperands.Add(Operand);
				}

				if(InputPin->GetDirection() != ERigVMPinDirection::Input && InputPin->GetDirection() != ERigVMPinDirection::IO)
				{
					continue;
				}

				InputOperands.Add(Operand);
			}
		}

		WorkData.VM->GetByteCode().SetOperandsForInstruction(
			CallExternInstruction,
			FRigVMOperandArray(InputOperands.GetData(), InputOperands.Num()),
			FRigVMOperandArray(OutputOperands.GetData(), OutputOperands.Num()));

#endif
		
		if (WorkData.Settings.SetupNodeInstructionIndex)
		{
			WorkData.VM->GetByteCode().SetSubject(CallExternInstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
		}

		// ── Emit JumpForward for Break / Continue nodes ──────────────────
		// If EnclosingLoopExprForJump was set during the Break or Continue
		// detection above, the CallExtern for the Break/Continue has now been
		// emitted. Emit a JumpForward placeholder (with INDEX_NONE) that will
		// be patched to jump to the loop's EndBlock instruction once it is emitted.
		if (EnclosingLoopExprForJump != nullptr)
		{
			const int32 JumpByteOffset = WorkData.VM->GetByteCode().AddJumpOp(ERigVMOpCode::JumpForward, INDEX_NONE);
			const int32 JumpInstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions() - 1;

			// Register the pending jump so it can be patched when EndBlock is emitted for the loop.
			FRigVMCompilerWorkData::FPendingLoopJump PendingJump;
			PendingJump.ByteOffset = JumpByteOffset;
			PendingJump.InstructionIndex = JumpInstructionIndex;
			WorkData.PendingLoopJumps.FindOrAdd(EnclosingLoopExprForJump).Add(PendingJump);
		}

		if(Node->IsControlFlowNode())
		{
			// add an operator to jump to the right branch
			const int32 JumpToBranchInstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions();

			// use the index of the first branch info relating to this control flow node.
			// branches are stored on the bytecode in order for each control flow node - so the
			// VM needs to know which branch to start to look at then evaluating the JumpToBranchOp.
			// Branches are stored in order - similar to this example representing two JumpBranchOps
			// with BranchIndices [0, 1] and [2, 3]
			// [
			//    0 = {ExecuteContext, InstructionIndex 2, First 3, Last 5},
			//    1 = {Completed, InstructionIndex 2, First 6, Last 12},
			//    2 = {ExecuteContext, InstructionIndex 17, First 18, Last 21},
			//    3 = {Completed, InstructionIndex 17, First 22, Last 28},
			// ]
			// The first index of the branch in the overall list of branches is stored in the operator (BranchIndices[0])
			WorkData.VM->GetByteCode().AddJumpToBranchOp(BlockToRunOperand, BranchIndices[0]);

			// create a copy here for ensure memory validity
			TArray<FName> BlockNames = Node->GetControlFlowBlocks();

			// traverse all of the blocks now
			for(int32 BlockIndex = 0; BlockIndex < BlockNames.Num(); BlockIndex++)
			{
				const FName BlockName = BlockNames[BlockIndex];
				int32 BranchIndex = BranchIndices[BlockIndex];
				{
					FRigVMBranchInfo& BranchInfo = WorkData.VM->GetByteCode().BranchInfos[BranchIndex];
					BranchInfo.Label = BlockName;
					BranchInfo.InstructionIndex = JumpToBranchInstructionIndex;
					BranchInfo.FirstInstruction = WorkData.VM->GetByteCode().GetNumInstructions();
					// BranchInfo can be invalidated by ByteCode array reallocs in the code below, so do not keep a reference to it
				}

				// check if the block requires slicing or not.
				// (do we want the private state of the nodes to be unique per run of the block)
				if(Node->IsControlFlowBlockSliced(BlockName))
				{
					check(BlockName != FRigVMStruct::ControlFlowCompletedName);
					check(CountOperand.IsValid());
					check(IndexOperand.IsValid());
					
					WorkData.VM->GetByteCode().AddBeginBlockOp(CountOperand, IndexOperand);
				}

				// traverse the body of the block
				const FRigVMVarExprAST* BlockExpr = InExpr->FindVarWithPinName(BlockName);
				check(BlockExpr);
				WorkData.ExprToSkip.Remove(BlockExpr);
				if (!TraverseExpression(BlockExpr, WorkData))
				{
					return false;
				}

				// end the block if necessary
				if(Node->IsControlFlowBlockSliced(BlockName))
				{
					WorkData.VM->GetByteCode().AddEndBlockOp();

					// ── Patch pending Break/Continue JumpForward ops for this loop ──
					// Any Break or Continue nodes inside this loop body will have emitted
					// JumpForward placeholders that need their relative offsets set to land
					// on the EndBlock instruction (so EndBlock still executes and unwinds
					// BeginLoopSlice/EndLoopSlice state). Now that EndBlock has been emitted,
					// we know its instruction index and can patch all pending jumps.
					if (TArray<FRigVMCompilerWorkData::FPendingLoopJump>* Jumps = WorkData.PendingLoopJumps.Find(InExpr))
					{
						// EndBlock was just emitted — its instruction index is the last one.
						const int32 EndBlockInstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
						for (const FRigVMCompilerWorkData::FPendingLoopJump& PendingJump : *Jumps)
						{
							// Relative forward distance from the JumpForward to the EndBlock.
							WorkData.VM->GetByteCode().GetOpAt<FRigVMJumpOp>(PendingJump.ByteOffset).InstructionIndex =
								EndBlockInstructionIndex - PendingJump.InstructionIndex;
						}
						WorkData.PendingLoopJumps.Remove(InExpr);
					}
				}

				// if this is not the completed block - we need to jump back to the control flow instruction
				if(BlockName != FRigVMStruct::ControlFlowCompletedName)
				{
					const int32 JumpToCallExternInstruction = WorkData.VM->GetByteCode().GetNumInstructions();
					WorkData.VM->GetByteCode().AddJumpOp(ERigVMOpCode::JumpBackward, JumpToCallExternInstruction - CallExternInstructionIndex);
				}

				WorkData.VM->GetByteCode().BranchInfos[BranchIndex].LastInstruction = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
			}
		}

		if(bSetupTraits)
		{
			// passing nullptr retrieves the empty trait list
			if(const FRigVMOperand* EmptyTraitListOperand = WorkData.TraitListLiterals.Find(nullptr))
			{
				if (WorkData.Settings.SetupNodeInstructionIndex)
				{
					WorkData.VM->GetByteCode().SetSubject(WorkData.VM->GetByteCode().GetNumInstructions(), Callstack.GetCallPath(), Callstack.GetStack());
				}
				WorkData.VM->GetByteCode().AddSetupTraitsOp(*EmptyTraitListOperand);
			}
		}
	}

	return true;
}

bool URigVMCompiler::TraverseInlineFunction(const FRigVMInlineFunctionExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	URigVMNode* Node = InExpr->GetNode();
	URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node);
	if(!ValidateNode(WorkData.Settings, FunctionReferenceNode, false))
	{
		return false;
	}

	const FRigVMGraphFunctionHeader& Header = FunctionReferenceNode->GetReferencedFunctionHeader(); 
	
	FString FunctionHash = Header.GetHash();
	if (!CompiledFunctions.Contains(FunctionHash))
	{
		return true;
	}

	// if we are considering callables, and we've seen this function before, let's take a shortcut.
	if(CVarRigVMCompileFunctionsToCallables->GetBool())
	{
		const uint32 CallableHash = CompiledFunctions.FindChecked(FunctionHash).CompilationData->Hash;
		if (TraverseCallableFunction(InExpr, CallableHash, WorkData))
		{
			return true;
		}
		// Callable lowering is requested but TraverseCallableFunction couldn't take over.
		// By default we silently fall back to function-reference inlining; tests that need
		// to verify the callable path is actually exercised disable that fallback by
		// setting RigVM.Compiler.AllowCallableFallbackToInlining=0, which causes the
		// compile to fail here instead of producing a misleading pass via the inline path.
		if (!CVarRigVMAllowCallableFallbackToInlining->GetBool())
		{
			UE_LOGF(LogRigVMDeveloper, Error,
				"Callable lowering could not be performed for function '%ls' and fallback to function-reference inlining is disabled (RigVM.Compiler.AllowCallableFallbackToInlining=0).",
				*FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer.GetLibraryNodePath());
			return false;
		}
	}

	const FRigVMFunctionCompilationData* FunctionCompilationData = CompiledFunctions.FindChecked(FunctionHash).CompilationData;
	const FRigVMByteCode& FunctionByteCode = FunctionCompilationData->ByteCode;
	const uint32 UniqueHashForReferenceNode = GetTypeHash(FunctionReferenceNode->GetPathName());

	// Bytecode to be inlined should never be aligned
	checkf(!FunctionByteCode.bByteCodeIsAligned, TEXT("Trying to inline aligned function bytecode %s in package %s"), *FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(), *GetPackage()->GetPathName());

	FRigVMByteCode& ByteCode = WorkData.VM->GetByteCode();
	
	// let's make sure we already have all required callables
	int32 FirstFunctionInstructionIndex = 0;
	for (int32 CallableIndex = 0; CallableIndex < FunctionByteCode.NumCallables(); CallableIndex++)
	{
		const FRigVMCallableInfo* OldCallable = FunctionByteCode.GetCallable(CallableIndex);
		check(OldCallable);
		const FRigVMCallableInfo* NewCallable = ByteCode.FindCallable(OldCallable->FunctionHash);
		if (!ensure(NewCallable != nullptr))
		{
			UE_LOGF(LogRigVMDeveloper, Error, "'%ls': Unexpected missing callable '%ls' when inlining function '%ls'.",
				*GetPackage()->GetPathName(), *OldCallable->Name.ToString(), *Header.LibraryPointer.GetLibraryNodePath());
			return false;
		}
		FirstFunctionInstructionIndex = OldCallable->LastInstruction + 1;
	}

	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}

		// Sort by pin name for deterministic operand ordering across sessions
		TArray<FRigVMOperand> InterfaceOperands;
		TArray<TPair<FName, FRigVMOperand>> SortedPairs(FunctionCompilationData->InterfaceOperands.Array());
		SortedPairs.Sort([](const TPair<FName, FRigVMOperand>& A, const TPair<FName, FRigVMOperand>& B)
		{
			return A.Key.ToString() < B.Key.ToString();
		});
		
		for (const TPair<FName, FRigVMOperand>& Pair : SortedPairs)
		{
			InterfaceOperands.Add(Pair.Value);
		}
		(void)WorkData.AddPropertiesForFunction(false, UniqueHashForReferenceNode, Header, FunctionCompilationData, FirstFunctionInstructionIndex, FunctionByteCode.NumInstructions - 1, InterfaceOperands, {}, FunctionReferenceNode);
		WorkData.SetupPropertyPathsForFoldedCopies(InExpr);
	}
	else
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}
		
		// Inline the bytecode from the function
		const int32 InstructionIndexStart = ByteCode.GetNumInstructions();
		FRigVMInstructionArray OldInstructions = ByteCode.GetInstructions();
		ByteCode.InlineFunction(&FunctionByteCode, FirstFunctionInstructionIndex, FunctionByteCode.GetNumInstructions() - 1);
		FRigVMCallstack FuncRefCallstack = InExpr->GetProxy().GetCallstack();

		auto RemapInstructionIndex = [InstructionIndexStart, FirstFunctionInstructionIndex](const int32& InOldInstructionIndex) -> int32
		{
			return InOldInstructionIndex - FirstFunctionInstructionIndex + InstructionIndexStart;				
		};

		// Import branch infos, with the proper instruction indices
		TMap<int32,int32> BranchIndexMap;
		for(const FRigVMBranchInfo& BranchInfo : FunctionByteCode.BranchInfos)
		{
			// skip the branch infos relating to callables
			if (BranchInfo.InstructionIndex < FirstFunctionInstructionIndex ||
				BranchInfo.FirstInstruction < FirstFunctionInstructionIndex ||
				BranchInfo.LastInstruction < FirstFunctionInstructionIndex)
			{
				continue;
			}

			const int32 BranchInfoIndex = ByteCode.AddBranchInfo(BranchInfo);
			ByteCode.BranchInfos[BranchInfoIndex].Index = BranchInfoIndex;
			ByteCode.BranchInfos[BranchInfoIndex].InstructionIndex = RemapInstructionIndex(BranchInfo.InstructionIndex);
			ByteCode.BranchInfos[BranchInfoIndex].FirstInstruction = RemapInstructionIndex(BranchInfo.FirstInstruction);
			ByteCode.BranchInfos[BranchInfoIndex].LastInstruction = RemapInstructionIndex(BranchInfo.LastInstruction);
			BranchIndexMap.Add(BranchInfo.Index, BranchInfoIndex);
		}

		auto RemapBranchIndex = [&BranchIndexMap](const int32& InOldBranchIndex) -> int32
		{
			return BranchIndexMap.FindChecked(InOldBranchIndex);
		};
		
		auto RemapCallableIndex = [&FunctionByteCode, &ByteCode](const int32& InOldCallableIndex) -> int32
		{
			const FRigVMCallableInfo* OldCallable = FunctionByteCode.GetCallable(InOldCallableIndex);
			check(OldCallable);
			const FRigVMCallableInfo* NewCallable = ByteCode.FindCallable(OldCallable->FunctionHash);
			check(NewCallable);
			return NewCallable->Index;
		};

		TArray<FRigVMOperand> InterfaceOperands;
		TArray<FString> OperandsPinNames;
		// Find operands related to the function's interface
		for(const URigVMPin* Pin : FunctionReferenceNode->GetPins())
		{
			const FRigVMExprAST* ChildExpr = InExpr->FindExprWithPinName(Pin->GetFName());
			checkf(ChildExpr, TEXT("Found unexpected opaque argument for %s while inlining function %s in package %s"), *InExpr->Name.ToString(), *FunctionReferenceNode->GetPathName(), *GetPackage()->GetPathName());

			const FRigVMVarExprAST* SourceVarExpr = nullptr;
			if (ChildExpr->GetType() == FRigVMExprAST::EType::CachedValue)
			{
				SourceVarExpr = GetSourceVarExpr(ChildExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr());
			}
			else if (ChildExpr->IsA(FRigVMExprAST::EType::Var))
			{
				SourceVarExpr = GetSourceVarExpr(ChildExpr->To<FRigVMVarExprAST>());
			}
			else
			{
				break;
			}
			checkf(SourceVarExpr, TEXT("Hit nullptr SourceVarExpr, function %s in package %s"), *FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(), *GetPackage()->GetPathName());
			
			if(SourceVarExpr->IsExecuteContext())
			{
				continue;
			}

			InterfaceOperands.Add(WorkData.ExprToOperand.FindChecked(SourceVarExpr));
			OperandsPinNames.Add(GetPinNameWithDirectionPrefix(Pin));

			if ( const FString* SegmentPath = InExpr->SegmentPathForChild.Find(Pin->GetPinIndex()))
			{
				checkf(!SegmentPath->IsEmpty(), TEXT("Hit empty segment path, function %s in package %s"), *FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(), *GetPackage()->GetPathName());
				
				const FString HeadCPPType = SourceVarExpr->GetPin()->GetCPPType();
				const int32 RegisterOffsetIndex = WorkData.FindOrAddPropertyPath(InterfaceOperands.Last(), HeadCPPType, *SegmentPath);
				checkf(RegisterOffsetIndex != INDEX_NONE, TEXT("Hit invalid register offset, function %s in package %s"), *FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(), *GetPackage()->GetPathName());
				InterfaceOperands.Last().RegisterOffset = static_cast<uint16>(RegisterOffsetIndex);
			}
		}
		
		auto RemapOperand = [this, &WorkData, UniqueHashForReferenceNode, &FunctionReferenceNode, FunctionCompilationData, &InterfaceOperands, &OperandsPinNames](const FRigVMOperand& InOldOperand) -> FRigVMOperand
		{
			FRigVMOperand InterfaceOperand;
			FRigVMOperand NewOperand = InOldOperand;

			// Remap the variable: find the operand index of the outer variable
			if (NewOperand.GetMemoryType() == ERigVMMemoryType::External)
			{
				const FName& InnerVariableName = FunctionCompilationData->ExternalRegisterIndexToVariable[NewOperand.GetRegisterIndex()];
				FString OuterVariablePath = InnerVariableName.ToString();
				const FString VariableRemapped = FunctionReferenceNode->GetOuterVariablePath(*OuterVariablePath);
				if (!VariableRemapped.IsEmpty())
				{
					OuterVariablePath = VariableRemapped;
				}
				else
				{
					FRigVMGraphFunctionIdentifier LibraryPointer = FunctionReferenceNode->GetFunctionIdentifier();
					const FString& LibraryPackagePath = LibraryPointer.GetNodeSoftPath().GetLongPackageName();
					ensureMsgf(!FunctionReferenceNode->RequiresVariableRemapping(),
						TEXT("Could not find variable %s in function reference %s variable map, in package %s\n. Library package is %s\n"),
						*InnerVariableName.ToString(),
						*FunctionReferenceNode->GetNodePath(),
						*GetPackage()->GetPathName(),
						*LibraryPackagePath);
				}

				// for now variable paths are not supported for mapping.
				if (OuterVariablePath.Contains(TEXT(".")))
				{
					UE_LOGF(
						LogRigVMDeveloper,
						Error,
						"Variable '%ls' on function '%ls' is mapped to path '%ls'. Property paths on variable mappings are not yet supported.",
						*InnerVariableName.ToString(),
						*FunctionReferenceNode->GetReferencedFunctionHeader().Name.ToString(),
						*OuterVariablePath);
				}
				else
				{
					const FString OuterOperandKey = FString::Printf(TEXT("Variable::%s"), *OuterVariablePath);
					checkf(WorkData.PinPathToOperand->Contains(OuterOperandKey), TEXT("hit a non-existing operand '%s', function %s in package %s"), *OuterOperandKey, *FunctionReferenceNode->GetReferencedFunctionHeader().Name.ToString(), *GetPackage()->GetPathName());

					const FRigVMOperand& OuterOperand = WorkData.PinPathToOperand->FindChecked(OuterOperandKey);
					NewOperand.MemoryType = OuterOperand.MemoryType;
					NewOperand.RegisterIndex = OuterOperand.RegisterIndex;
				}
			}
			else
			{
				// If Operand is an interface pin: replace the index and memory type
				if (const int32 FunctionInterfaceParameterIndex = GetOperandFunctionInterfaceParameterIndex(OperandsPinNames, FunctionCompilationData, NewOperand); FunctionInterfaceParameterIndex != INDEX_NONE)
				{
					InterfaceOperand = InterfaceOperands[FunctionInterfaceParameterIndex];
					NewOperand.MemoryType = InterfaceOperand.MemoryType;
					NewOperand.RegisterIndex = InterfaceOperand.RegisterIndex;
				}
				else
				{
					// Operand is internal
					// Replace with added Operand
					const FRigVMCompilerWorkData::FFunctionRegisterData Data = {UniqueHashForReferenceNode, NewOperand.GetMemoryType(), NewOperand.GetRegisterIndex()};
					checkf(WorkData.FunctionRegisterToOperand.Contains(Data), TEXT("hit a missing function operand, function %s in package %s"), *FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(), *GetPackage()->GetPathName());
					FRigVMOperand RemappedOperand = WorkData.FunctionRegisterToOperand.FindChecked(Data);
					NewOperand.MemoryType = RemappedOperand.MemoryType;
					NewOperand.RegisterIndex = RemappedOperand.RegisterIndex;
				}
			}

			// For all operands, check to see if we need to add a property path
			if (NewOperand.GetRegisterOffset() != INDEX_NONE || InterfaceOperand.GetRegisterOffset() != INDEX_NONE)
			{
				if (InterfaceOperand.GetRegisterOffset() != INDEX_NONE && NewOperand.GetRegisterOffset() == INDEX_NONE)
				{
					NewOperand.RegisterOffset = IntCastChecked<uint16>(InterfaceOperand.GetRegisterOffset());
				}
				else
				{
					FRigVMFunctionCompilationPropertyPath DescriptionFromFunction;
					switch (InOldOperand.MemoryType)
					{
						case ERigVMMemoryType::Work:
						{
							DescriptionFromFunction = FunctionCompilationData->WorkPropertyPathDescriptions[NewOperand.GetRegisterOffset()];
							break;
						}
						case ERigVMMemoryType::Literal:
						{
							DescriptionFromFunction = FunctionCompilationData->LiteralPropertyPathDescriptions[NewOperand.GetRegisterOffset()];
							break;
						}
						case ERigVMMemoryType::External:
						{
							DescriptionFromFunction = FunctionCompilationData->ExternalPropertyPathDescriptions[NewOperand.GetRegisterOffset()];
							break;
						}
						case ERigVMMemoryType::Debug:
						{
							DescriptionFromFunction = FunctionCompilationData->DebugPropertyPathDescriptions[NewOperand.GetRegisterOffset()];
							break;
						}
						default:
						{
							checkNoEntry();
						}
					}

					if (InterfaceOperand.GetRegisterOffset() == INDEX_NONE)
					{
						NewOperand.RegisterOffset = IntCastChecked<uint16>(WorkData.FindOrAddPropertyPath(NewOperand, DescriptionFromFunction.HeadCPPType, DescriptionFromFunction.SegmentPath));
					}
					else
					{
						// in this case both the external operand and the operand within the function have a register offset.
						// they need to be combined.
						const FRigVMPropertyPathDescription& Description = WorkData.PropertyPathDescriptions.FindOrAdd(InterfaceOperand.GetMemoryType())[InterfaceOperand.GetRegisterOffset()];
						const FString CombinedSegmentPath = URigVMPin::JoinPinPath(Description.SegmentPath, DescriptionFromFunction.SegmentPath);
						NewOperand.RegisterOffset = IntCastChecked<uint16>(WorkData.FindOrAddPropertyPath(NewOperand, Description.HeadCPPType, CombinedSegmentPath));
					}
				}
			}
		
			return NewOperand;
		};

		// For each instruction, substitute the operand for the one used in the current bytecode
		const FRigVMInstructionArray FunctionInstructions = FunctionByteCode.GetInstructions();
		FRigVMInstructionArray Instructions = ByteCode.GetInstructions();
		for (int32 FunctionInstructionIndex = FirstFunctionInstructionIndex; FunctionInstructionIndex < FunctionInstructions.Num(); FunctionInstructionIndex++)
		{
			WorkData.RemapInstructionAt(
				FunctionInstructions[FunctionInstructionIndex],
				FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath(),
				FunctionCompilationData,
				ByteCode,
				Instructions,
				FunctionReferenceNode,
				RemapInstructionIndex,
				RemapOperand,
				RemapBranchIndex,
				RemapCallableIndex);
		}

		const TMap<FRigVMOperand, FRigVMOperand>& FunctionOperandToNewOperand = WorkData.OperandMapPerFunction.FindChecked(UniqueHashForReferenceNode);
		
		// Add all pin paths to operand from the function
		TMap<FRigVMOperand, TArray<FString>> OperandToPinPath;
		for (const TPair<FString, FRigVMOperand>& PinPathToOperand : FunctionCompilationData->Operands)
		{
			if (const FRigVMOperand* NewOperand = FunctionOperandToNewOperand.Find(PinPathToOperand.Value))
			{
				// avoid copying the variable pin paths over to the outer VM since
				// the variables from within the function asset no longer exist and
				// will have been remapped to the outer VM's external variables.1
				static const FString VariablePrefix = TEXT("Variable::");
				if(PinPathToOperand.Key.StartsWith(VariablePrefix, ESearchCase::CaseSensitive))
				{
					continue;
				}
				
				WorkData.PinPathToOperand->Add(PinPathToOperand.Key, *NewOperand);
				TArray<FString>& Paths = OperandToPinPath.Emplace(*NewOperand);
				Paths.Add(PinPathToOperand.Key);
			}
		}
	}

	return true;
}

bool URigVMCompiler::TraverseCallableFunction(const FRigVMInlineFunctionExprAST* InExpr, const uint32 InCallableHash, FRigVMCompilerWorkData& WorkData)
{
	FRigVMByteCode& ByteCode = WorkData.VM->GetByteCode();
	const FRigVMCallableInfo* Callable = ByteCode.FindCallable(InCallableHash);
	if (!Callable)
	{
		return false;
	}

	URigVMNode* Node = InExpr->GetNode();
	URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node);
	if(!ValidateNode(WorkData.Settings, FunctionReferenceNode, false))
	{
		return false;
	}

	const FRigVMGraphFunctionHeader& Header = FunctionReferenceNode->GetReferencedFunctionHeader(); 
	const FString FunctionHash = Header.GetHash();
	if (!CompiledFunctions.Contains(FunctionHash))
	{
		return true;
	}

	const FRigVMFunctionCompilationData* FunctionCompilationData = CompiledFunctions.FindChecked(FunctionHash).CompilationData;

	// not all function support callable instructions
	if (!FunctionCompilationData->SupportsCallable())
	{
		return false;
	}

	const FRigVMCallstack Callstack = InExpr->GetProxy().GetCallstack();
	
	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}
	}
	else
	{
		if (!TraverseChildren(InExpr, WorkData))
		{
			return false;
		}

		TArray<FRigVMOperand> CallableOperands;
		CallableOperands.SetNum(Callable->Arguments.Num());
#if WITH_EDITOR
		TArray<FRigVMOperand> InputOperands, OutputOperands;
#endif

		for (int32 ChildIndex = 0; ChildIndex < InExpr->NumChildren(); ChildIndex++)
		{
			const FRigVMExprAST* ChildExpr = InExpr->ChildAt(ChildIndex);
			const FRigVMVarExprAST* SourceVarExpr = nullptr;
			if (ChildExpr->IsVar())
			{
				SourceVarExpr = ChildExpr->To<FRigVMVarExprAST>();
			}
			else if (ChildExpr->GetType() == FRigVMExprAST::EType::CachedValue)
			{
				SourceVarExpr = ChildExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr();
			}
			else
			{
				// the AST is corrupt. we only expect cached values or var expressions as children here.
				checkNoEntry();
			}

			if (SourceVarExpr->IsExecuteContext())
			{
				continue;
			}
			
			SourceVarExpr = GetSourceVarExpr(SourceVarExpr);
			
			check(WorkData.ExprToOperand.Contains(SourceVarExpr));
			const FRigVMOperand SourceOperand = WorkData.ExprToOperand.FindChecked(SourceVarExpr);
			
			const FName PinName = InExpr->GetPinNameForChildIndex(ChildIndex);
			check(!PinName.IsNone());

			const int32 ArgumentIndex = Callable->GetArgumentIndex(PinName);
			if (!Callable->Arguments.IsValidIndex(ArgumentIndex))
			{
				// Mirror of the callable-conversion skip above: an auto-promoted input-
				// variable argument has no slot in Callable->Arguments. The matching
				// child pin on the reference node is still walked here, so skip it.
				continue;
			}
			const FRigVMCallableArgument& Argument = Callable->Arguments[ArgumentIndex];
			check(Argument.InterfaceOperand.IsValid());
			check(Argument.ForwardedOperand.IsValid());
			check(SourceOperand.IsValid());

			CallableOperands[ArgumentIndex] = SourceOperand;

#if WITH_EDITOR
			if (Argument.IsInput())
			{
				InputOperands.Add(SourceOperand);
			}
			if (Argument.IsOutput())
			{
				OutputOperands.Add(SourceOperand);
			}
#endif
		}
		
		const int32 InvokeCallableByteIndex = ByteCode.AddInvokeCallableOp(Callable->FunctionHash, CallableOperands);
		const int32 InvokeCallableInstructionIndex = ByteCode.GetNumInstructions() - 1;
		const FRigVMInstruction InvokeCallableInstruction(ERigVMOpCode::InvokeCallable, InvokeCallableInstructionIndex, InvokeCallableByteIndex);

#if WITH_EDITOR
		WorkData.VM->GetByteCode().SetOperandsForInstruction(
			InvokeCallableInstruction,
			FRigVMOperandArray(InputOperands.GetData(), InputOperands.Num()),
			FRigVMOperandArray(OutputOperands.GetData(), OutputOperands.Num()));
#endif
		if (WorkData.Settings.SetupNodeInstructionIndex)
		{
			WorkData.VM->GetByteCode().SetSubject(InvokeCallableInstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
		}
	}
	return true;
}

bool URigVMCompiler::TraverseNoOp(const FRigVMNoOpExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InExpr->GetNode()))
	{
		if (!VariableNode->IsGetter())
		{
			const URigVMPin* ValuePin = VariableNode->GetValuePin();

			// if the value pin has sub pin links
			if (ValuePin->GetSourceLinks(false).IsEmpty() &&
				!ValuePin->GetSourceLinks(true).IsEmpty())
			{
				// find all the paths that require a copy from the default
				TArray<FString> PropertyPathsRequiringCopy;
				TArray<URigVMPin*> SubPins = ValuePin->GetSubPins();
				for (int32 SubPinIndex = 0; SubPinIndex < SubPins.Num(); ++SubPinIndex)
				{
					const URigVMPin* SubPin = SubPins[SubPinIndex];
					if (!SubPin->GetSourceLinks(false).IsEmpty())
					{
						continue;
					}
					if (SubPin->GetSourceLinks(true).IsEmpty())
					{
						PropertyPathsRequiringCopy.Add(SubPin->GetSubPinPath(ValuePin, false));
					}
					else
					{
						SubPins.Append(SubPin->GetSubPins());
					}
				}

				if (!PropertyPathsRequiringCopy.IsEmpty())
				{
					// run the recursion now so that all the registers exist.
					if (!TraverseChildren(InExpr, WorkData))
					{
						return false;
					}
					
					FName VariableName = VariableNode->GetVariableName();
					FRigVMASTProxy ParentProxy = InExpr->GetProxy();
					while(ParentProxy.GetCallstack().Num() > 1)
					{
						ParentProxy = ParentProxy.GetParent();

						if(URigVMFunctionReferenceNode* FunctionReferenceNode = ParentProxy.GetSubject<URigVMFunctionReferenceNode>())
						{
							const FName RemappedVariableName = FunctionReferenceNode->GetOuterVariableName(VariableName);
							if(!RemappedVariableName.IsNone())
							{
								VariableName = RemappedVariableName;
							}
						}
					}

					// Default value literals will be shared with variable sharing the same default
					const FString DefaultValue = ValuePin->GetDefaultValue(); 
					const uint32 DefaultValueHash = GetTypeHash(DefaultValue);
					FString RegisterNameString;
					if (VariableNode->IsLocalVariable())
					{
						RegisterNameString = FString::Printf(TEXT("LocalVariableDefault::%s|%s|%lu::Const"), *VariableNode->GetGraph()->GetGraphName(), *VariableName.ToString(), (unsigned long)DefaultValueHash);
					}
					else
					{
						RegisterNameString = FString::Printf(TEXT("VariableDefault::%s|%lu::Const"), *VariableName.ToString(), (unsigned long)DefaultValueHash);
					}
					const FName RegisterName = FRigVMPropertyDescription::SanitizeName(*RegisterNameString, false);

					// set up a default value literal based on the value pin configuration
					const FRigVMOperand RootDefaultValueOperand = WorkData.FindProperty(ERigVMMemoryType::Literal, RegisterName);

					FRigVMOperand VariableOperand;
					FString VariableRegisterNameString;
					if (VariableNode->IsLocalVariable())
					{
						VariableRegisterNameString = FString::Printf(TEXT("LocalVariable::%s|%s"), *VariableNode->GetGraph()->GetGraphName(), *VariableName.ToString());
						const FName VariableRegisterName = FRigVMPropertyDescription::SanitizeName(*VariableRegisterNameString, false);
						VariableOperand = WorkData.FindProperty(ERigVMMemoryType::Work, VariableRegisterName);
					}
					else
					{
						VariableRegisterNameString = FString::Printf(TEXT("Variable::%s"), *VariableName.ToString());
						if (const FRigVMOperand* VariableOperandPtr = WorkData.PinPathToOperand->Find(VariableRegisterNameString))
						{
							VariableOperand = *VariableOperandPtr;
						}
					}

					if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
					{
						if (!RootDefaultValueOperand.IsValid() && VariableOperand.IsValid())
						{
							const FRigVMOperand DefaultOperand = WorkData.AddProperty(ERigVMMemoryType::Literal, RegisterName, ValuePin->GetCPPType(), ValuePin->GetCPPTypeObject(), DefaultValue);
							for (const FString& PropertyPath : PropertyPathsRequiringCopy)
							{
								// add the property path both to the external variable as well as the default operand
								WorkData.FindOrAddPropertyPath(DefaultOperand, ValuePin->GetCPPType(), PropertyPath);
								WorkData.FindOrAddPropertyPath(VariableOperand, ValuePin->GetCPPType(), PropertyPath);
							}
						}
					}
					else if (VariableOperand.IsValid() && ensure(RootDefaultValueOperand.IsValid()))
					{
						for (const FString& PropertyPath : PropertyPathsRequiringCopy)
						{
							// set up an operand both for the default literal and the variable
							// given the same property path.
							const int32 DefaultRegisterOffset = WorkData.FindOrAddPropertyPath(RootDefaultValueOperand, ValuePin->GetCPPType(), PropertyPath);
							const int32 VariableRegisterOffset = WorkData.FindOrAddPropertyPath(VariableOperand, ValuePin->GetCPPType(), PropertyPath);
							const FRigVMOperand DefaultOperand(RootDefaultValueOperand.GetMemoryType(), RootDefaultValueOperand.GetRegisterIndex(), DefaultRegisterOffset);
							const FRigVMOperand SubPinVariableOperand(VariableOperand.GetMemoryType(), VariableOperand.GetRegisterIndex(), VariableRegisterOffset);

							const FRigVMCopyOp CopyOp = WorkData.VM->GetCopyOpForOperands(DefaultOperand, SubPinVariableOperand);
							WorkData.VM->GetByteCode().AddCopyOp(CopyOp);
						}
					}

					// return here since we've already run traverse children
					return true;
				}
			}
		}
	}
	return TraverseChildren(InExpr, WorkData);
}

bool URigVMCompiler::TraverseVar(const FRigVMVarExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (!TraverseChildren(InExpr, WorkData))
	{
		return false;
	}

	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		FindOrAddRegister(InExpr, WorkData);
	}

	return true;
}

bool URigVMCompiler::TraverseLiteral(const FRigVMVarExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	return TraverseVar(InExpr, WorkData);
}

bool URigVMCompiler::TraverseAssign(const FRigVMAssignExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	if (!TraverseChildren(InExpr, WorkData))
	{
		return false;
	}

	ensure(InExpr->NumChildren() > 0);

	const FRigVMVarExprAST* SourceExpr = nullptr;

	const FRigVMExprAST* ChildExpr = InExpr->ChildAt(0);
	if (ChildExpr->IsA(FRigVMExprAST::EType::Var))
	{
		SourceExpr = ChildExpr->To<FRigVMVarExprAST>();
	}
	else if (ChildExpr->GetType() == FRigVMExprAST::EType::CachedValue)
	{
		SourceExpr = ChildExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr();
	}
	else if (ChildExpr->GetType() == FRigVMExprAST::EType::NoOp ||
		ChildExpr->GetType() == FRigVMExprAST::EType::Entry)
	{
		ensure(ChildExpr->NumChildren() > 0);

		for (FRigVMExprAST* GrandChild : *ChildExpr)
		{
			if (GrandChild->IsA(FRigVMExprAST::EType::Var))
			{
				const FRigVMVarExprAST* VarExpr = GrandChild->To<FRigVMVarExprAST>();
				const URigVMPin* Pin = VarExpr->GetPin();
				if (!Pin)
				{
					continue;
				}
				const URigVMNode* Node = Pin->GetNode();
				if (!Node)
				{
					continue;
				}
				if (Node->IsA<URigVMVariableNode>() && Pin->GetFName() == URigVMVariableNode::ValueName)
				{
					SourceExpr = VarExpr;
					break;
				}
				if (Node->IsA<URigVMEnumNode>() && Pin->GetFName() == URigVMEnumNode::EnumIndexName)
				{
					SourceExpr = VarExpr;
					break;
				}
				if (Node->IsA<URigVMFunctionEntryNode>() && Pin->GetRootPin()->IsDefinedAsInputVariable())
				{
					// the expression on the link equals the pin on the entry node or is a sub-pin thereof
					if (InExpr->GetSourcePin() == VarExpr->GetPin() || InExpr->GetSourcePin()->IsInOuter(VarExpr->GetPin()))
					{
						SourceExpr = VarExpr;
						break;
					}
				}
			}
		}

		check(SourceExpr);
	}
	else
	{
		checkNoEntry();
	}

	FRigVMOperand Source = WorkData.ExprToOperand.FindChecked(GetSourceVarExpr(SourceExpr));

	const FRigVMVarExprAST* TargetExpr = InExpr->GetFirstParentOfType(FRigVMVarExprAST::EType::Var)->To<FRigVMVarExprAST>();
	TargetExpr = GetSourceVarExpr(TargetExpr);

	// We cannot wait for the target var expr to be created, we need it now to setup the register offset
	const FRigVMVarExprAST* VarExpr = GetSourceVarExpr(TargetExpr);
	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		if (!WorkData.ExprToOperand.Contains(VarExpr))
		{
			FindOrAddRegister(VarExpr, WorkData);
		}
	}
	
	FRigVMOperand Target = WorkData.ExprToOperand.FindChecked(VarExpr);
	if(Target == Source)
	{
		return true;
	}
	
	// if this is a copy - we should check if operands need offsets
	if (InExpr->GetType() == FRigVMExprAST::EType::Copy)
	{
		struct Local
		{
			static void SetupRegisterOffset(URigVM* VM, const FRigVMASTLinkDescription& InLink, URigVMPin* Pin,
				FRigVMOperand& Operand, const FRigVMVarExprAST* VarExpr, bool bSource, FRigVMCompilerWorkData& WorkData)
			{
				const bool bHasTargetSegmentPath = !bSource && !InLink.SegmentPath.IsEmpty();
				
				URigVMPin* RootPin = Pin->GetRootPin();
				if (Pin == RootPin && !bHasTargetSegmentPath)
				{
					return;
				}

				if(Pin->IsProgrammaticPin())
				{
					return;
				}

				FString SegmentPath = Pin->GetSegmentPath(false);
				if(bHasTargetSegmentPath)
				{
					if(SegmentPath.IsEmpty())
					{
						SegmentPath = InLink.SegmentPath;
					}
					else
					{
						SegmentPath = URigVMPin::JoinPinPath(SegmentPath, InLink.SegmentPath);
					}
				}

				// for fixed array pins we create a register for each array element
				// thus we do not need to setup a registeroffset for the array element.
				if (RootPin->IsFixedSizeArray())
				{
					if (Pin->GetParentPin() == RootPin)
					{
						return;
					}

					// if the pin is a sub pin of a case of a fixed array
					// we'll need to re-adjust the root pin to the case pin (for example: Values.0)
					TArray<FString> SegmentPathPaths;
					if(ensure(URigVMPin::SplitPinPath(SegmentPath, SegmentPathPaths)))
					{
						RootPin = RootPin->FindSubPin(SegmentPathPaths[0]);

						SegmentPathPaths.RemoveAt(0);
						ensure(SegmentPathPaths.Num() > 0);
						SegmentPath = URigVMPin::JoinPinPath(SegmentPathPaths);
					}
					else
					{
						return;
					}
				}

				const int32 PropertyPathIndex = WorkData.FindOrAddPropertyPath(Operand, RootPin->GetCPPType(), SegmentPath);
				Operand = FRigVMOperand(Operand.GetMemoryType(), Operand.GetRegisterIndex(), PropertyPathIndex);
			}
		};

		const FRigVMASTLinkDescription& Link = InExpr->GetLink();
		Local::SetupRegisterOffset(WorkData.VM, Link, InExpr->GetSourcePin(), Source, SourceExpr, true, WorkData);
		Local::SetupRegisterOffset(WorkData.VM, Link, InExpr->GetTargetPin(), Target, TargetExpr, false, WorkData);
	}
	
	if (WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
	{
		FRigVMCopyOp CopyOp = WorkData.VM->GetCopyOpForOperands(Source, Target);
		if(CopyOp.IsValid())
		{
			AddCopyOperator(CopyOp, InExpr, SourceExpr, TargetExpr, WorkData);
		}
	}

	return true;
}

bool URigVMCompiler::TraverseCopy(const FRigVMCopyExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	return TraverseAssign(InExpr->To<FRigVMAssignExprAST>(), WorkData);
}

bool URigVMCompiler::TraverseCachedValue(const FRigVMCachedValueExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	return TraverseChildren(InExpr, WorkData);
}

bool URigVMCompiler::TraverseExit(const FRigVMExitExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	ensure(InExpr->NumChildren() == 0);
	if (WorkData.CompilerPhase == ERigVMCompilerPhase_BuildInstructions)
	{
		WorkData.VM->GetByteCode().AddExitOp();
	}
	return true;
}

bool URigVMCompiler::TraverseInvokeEntry(const FRigVMInvokeEntryExprAST* InExpr, FRigVMCompilerWorkData& WorkData)
{
	URigVMInvokeEntryNode* InvokeEntryNode = Cast<URigVMInvokeEntryNode>(InExpr->GetNode());
	if(!ValidateNode(WorkData.Settings, InvokeEntryNode))
	{
		return false;
	}

	if (WorkData.CompilerPhase == ERigVMCompilerPhase_SetupMemory)
	{
		return true;
	}
	else
	{
		const int32 InstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions();
		WorkData.VM->GetByteCode().AddInvokeEntryOp(InvokeEntryNode->GetEntryName());

		if (WorkData.Settings.SetupNodeInstructionIndex)
		{
			const FRigVMCallstack Callstack = InExpr->GetProxy().GetSibling(InvokeEntryNode).GetCallstack();
			WorkData.VM->GetByteCode().SetSubject(InstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
		}
	}
	return true;
}

void URigVMCompiler::AddCopyOperator(const FRigVMCopyOp& InOp, const FRigVMAssignExprAST* InAssignExpr,
                                     const FRigVMVarExprAST* InSourceExpr, const FRigVMVarExprAST* InTargetExpr,  FRigVMCompilerWorkData& WorkData,
                                     bool bDelayCopyOperations)
{
	if(bDelayCopyOperations)
	{
		// if this is a full literal copy, let's delay it.
		// to maintain the execution order we want nodes which compose a value
		// to delay their reset to the default value, which happens prior to
		// computing dependencies.
		// so for example an external variable of FVector may need to be reset
		// to a literal value prior to the rest of the composition, for example
		// if there's a float link only on the Y component. the execution order
		// desired is this:
		//
		// * Run all dependent branches
		// * Copy the literal value into the variable
		// * Copy the parts into the variable (like the Y component).
		// 
		// By delaying the copy operator until right before the very first composition
		// copy operator we ensure the desired execution order
		if(InOp.Target.GetRegisterOffset() == INDEX_NONE && 
			InOp.Source.GetMemoryType() == ERigVMMemoryType::Literal &&
			InOp.Source.GetRegisterOffset() == INDEX_NONE)
		{
			if(URigVMPin* Pin = InTargetExpr->GetPin())
			{
				if(URigVMPin* RootPin = Pin->GetRootPin())
				{
					const FRigVMASTProxy RootPinProxy = InTargetExpr->GetProxy().GetSibling(RootPin);

					// if the root pin has only links on its subpins
					if(WorkData.AST->GetSourceLinkIndices(RootPinProxy, false).Num() == 0)
					{
						if(WorkData.AST->GetSourceLinkIndices(RootPinProxy, true).Num() > 0)
						{					
							FRigVMCompilerWorkData::FCopyOpInfo DeferredCopyOp;
							DeferredCopyOp.Op = InOp;
							DeferredCopyOp.AssignExpr = InAssignExpr;
							DeferredCopyOp.SourceExpr = InSourceExpr;
							DeferredCopyOp.TargetExpr = InTargetExpr;
				
							const FRigVMOperand Key(InOp.Target.GetMemoryType(), InOp.Target.GetRegisterIndex());
							WorkData.DeferredCopyOps.FindOrAdd(Key) = DeferredCopyOp;
							return;
						}
					}
				}
			}
		}
		
		bDelayCopyOperations = false;
	}

	// look up a potentially delayed copy operation which needs to happen
	// just prior to this one and inject it as well.
	if(!bDelayCopyOperations)
	{
		const FRigVMOperand DeferredKey(InOp.Target.GetMemoryType(), InOp.Target.GetRegisterIndex());
		const FRigVMCompilerWorkData::FCopyOpInfo* DeferredCopyOpPtr = WorkData.DeferredCopyOps.Find(DeferredKey);
		if(DeferredCopyOpPtr != nullptr)
		{
			FRigVMCompilerWorkData::FCopyOpInfo CopyOpInfo = *DeferredCopyOpPtr;
			WorkData.DeferredCopyOps.Remove(DeferredKey);
			AddCopyOperator(CopyOpInfo, WorkData, false);
		}
	}

	bool bAddCopyOp = true;

	// check if we need to inject a cast instead of a copy operator
	const TRigVMTypeIndex SourceTypeIndex = WorkData.GetTypeIndexForOperand(InOp.Source);
	const TRigVMTypeIndex TargetTypeIndex = WorkData.GetTypeIndexForOperand(InOp.Target);
	if(SourceTypeIndex != TargetTypeIndex)
	{
		// if the type system can't auto cast these types (like float vs double)
		if(!FRigVMRegistry::Get().CanMatchTypes(SourceTypeIndex, TargetTypeIndex, true))
		{
			const FRigVMFunction* CastFunction = RigVMTypeUtils::GetCastForTypeIndices(SourceTypeIndex, TargetTypeIndex);
			if(CastFunction == nullptr)
			{
				const FRigVMRegistry& Registry = FRigVMRegistry::Get();
				static constexpr TCHAR MissingCastMessage[] = TEXT("Cast (%s to %s) for Node @@ not found.");
				const FString& SourceCPPType = Registry.GetType(SourceTypeIndex).CPPType.ToString();
				const FString& TargetCPPType = Registry.GetType(TargetTypeIndex).CPPType.ToString();
				WorkData.Settings.Report(EMessageSeverity::Error, InAssignExpr->GetTargetPin()->GetNode(),
					FString::Printf(MissingCastMessage, *SourceCPPType, *TargetCPPType));
				return;
			}

			check(CastFunction->Arguments.Num() >= 2);

			const FRigVMOperand Source = InOp.Source;
			const FRigVMOperand Target = InOp.Target;

			const int32 FunctionIndex = WorkData.VM->AddRigVMFunction(CastFunction->Name);
			WorkData.VM->GetByteCode().AddExecuteOp(FunctionIndex, {Source, Target}, 0, 0);

			bAddCopyOp = false;
		}
	}


	// if we are copying into an array variable
	if(bAddCopyOp)
	{
		if(const URigVMPin* Pin = InTargetExpr->GetPin())
		{
			if(Pin->IsArray() && Pin->GetNode()->IsA<URigVMVariableNode>())
			{
				if(InOp.Source.GetRegisterOffset() == INDEX_NONE &&
					InOp.Target.GetRegisterOffset() == INDEX_NONE)
				{
					static const FString ArrayCloneName =
						FRigVMRegistry::Get().FindOrAddSingletonDispatchFunction<FRigVMDispatch_ArrayClone>();
					const int32 FunctionIndex = WorkData.VM->AddRigVMFunction(ArrayCloneName);
					WorkData.VM->GetByteCode().AddExecuteOp(FunctionIndex, {InOp.Source, InOp.Target}, 0, 0);
					bAddCopyOp = false;
				}
			}
		}
	}
	
	if(bAddCopyOp)
	{
		WorkData.VM->GetByteCode().AddCopyOp(InOp);
	}

	int32 InstructionIndex = WorkData.VM->GetByteCode().GetNumInstructions() - 1;
	if (WorkData.Settings.SetupNodeInstructionIndex)
	{
		bool bSetSubject = false;
		if (URigVMPin* SourcePin = InAssignExpr->GetSourcePin())
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
			{
				const FRigVMCallstack Callstack = InSourceExpr->GetProxy().GetSibling(VariableNode).GetCallstack();
				WorkData.VM->GetByteCode().SetSubject(InstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
				bSetSubject = true;
			}
		}

		if (!bSetSubject)
		{
			if (URigVMPin* TargetPin = InAssignExpr->GetTargetPin())
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(TargetPin->GetNode()))
				{
					const FRigVMCallstack Callstack = InTargetExpr->GetProxy().GetSibling(VariableNode).GetCallstack();
					WorkData.VM->GetByteCode().SetSubject(InstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
					bSetSubject = true;
				}
				else
				{
					const FRigVMCallstack Callstack = InTargetExpr->GetProxy().GetSibling(TargetPin->GetNode()).GetCallstack();
					WorkData.VM->GetByteCode().SetSubject(InstructionIndex, Callstack.GetCallPath(), Callstack.GetStack());
					bSetSubject = true;
				}
			}
		}
	}
}

void URigVMCompiler::AddCopyOperator(
	const FRigVMCompilerWorkData::FCopyOpInfo& CopyOpInfo,
	FRigVMCompilerWorkData& WorkData,
	bool bDelayCopyOperations)
{
	AddCopyOperator(CopyOpInfo.Op, CopyOpInfo.AssignExpr, CopyOpInfo.SourceExpr, CopyOpInfo.TargetExpr, WorkData, bDelayCopyOperations);
}

FString URigVMCompiler::GetPinHashImpl(const URigVMPin* InPin, const FRigVMVarExprAST* InVarExpr, const URigVMLibraryNode* FunctionCompiling, const FRigVMASTProxy& InPinProxy)
{
	if (!InPin)
	{
		return FString();
	}

	FString Prefix;
	FString Suffix;

	if (InPin->IsExecuteContext())
	{
		return TEXT("ExecuteContext!");
	}

	URigVMNode* Node = InPin->GetNode();

	bool bIsExecutePin = false;
	bool bIsLiteral = false;
	bool bIsVariable = false;
	bool bIsFunctionInterfacePin = false;

	if (InVarExpr != nullptr)
	{
		// for IO array pins we'll walk left and use that pin hash instead
		if(const FRigVMVarExprAST* SourceVarExpr = GetSourceVarExpr(InVarExpr))
		{
			if(SourceVarExpr != InVarExpr)
			{
				return GetPinHash(SourceVarExpr->GetPin(), SourceVarExpr, FunctionCompiling);
			}
		}

		bIsExecutePin = InPin->IsExecuteContext();
		bIsLiteral = InVarExpr->GetType() == FRigVMExprAST::EType::Literal;

		bIsVariable = Cast<URigVMVariableNode>(Node) != nullptr;
		bIsFunctionInterfacePin = (Cast<URigVMFunctionEntryNode>(Node) || Cast<URigVMFunctionReturnNode>(Node)) &&
			Node->GetTypedOuter<URigVMLibraryNode>() == FunctionCompiling;

		// determine if this is an initialization for an IO pin
		if (!bIsLiteral &&
			!bIsVariable &&
			!bIsFunctionInterfacePin &&
			!bIsExecutePin && (InPin->GetDirection() == ERigVMPinDirection::IO ||
			(InPin->GetDirection() == ERigVMPinDirection::Input && InPin->GetSourceLinks().Num() == 0)))
		{
			Suffix = TEXT("::IO");
		}
		else if (bIsLiteral)
		{
			Suffix = TEXT("::Const");
		}
	}

	bool bUseFullNodePath = true;
	URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node);
	const bool bIsVariableClone = InVarExpr && InVarExpr->Name.ToString().EndsWith(TEXT("::Clone"));
	if (VariableNode && !bIsVariableClone)
	{
		if (InPin->GetName() == TEXT("Value"))
		{
			FName VariableName = VariableNode->GetVariableName();

			if(VariableNode->IsLocalVariable())
			{
				if (bIsLiteral)
				{
					if (InVarExpr && InVarExpr->NumParents() == 0 && InVarExpr->NumChildren() == 0)
					{
						// Default literal values will be reused for all instance of local variables
						return FString::Printf(TEXT("%sLocalVariableDefault::%s|%s%s"), *Prefix, *Node->GetGraph()->GetGraphName(), *VariableName.ToString(), *Suffix);
					}
					else if (InVarExpr)
					{
						const FString PinPath = InVarExpr->GetProxy().GetCallstack().GetCallPath(true);
						return FString::Printf(TEXT("%sLocalVariable::%s%s"), *Prefix, *PinPath, *Suffix);					
					}
					else
					{
						return FString::Printf(TEXT("%sLocalVariable::%s|%s%s"), *Prefix, *Node->GetGraph()->GetGraphName(), *VariableName.ToString(), *Suffix);					
					}			
				}
				else
				{
					if(InVarExpr)
					{
						FRigVMASTProxy ParentProxy = InVarExpr->GetProxy();
						while(ParentProxy.GetCallstack().Num() > 1)
						{
							ParentProxy = ParentProxy.GetParent();

							if(ParentProxy.GetSubject<URigVMLibraryNode>())
							{
								break;
							}
						}

						// Local variables for root / non-root graphs are in the format "LocalVariable::PathToGraph|VariableName"
						return FString::Printf(TEXT("%sLocalVariable::%s|%s%s"), *Prefix, *Node->GetGraph()->GetGraphName(), *VariableName.ToString(), *Suffix);
					}
				}
			}
			else if(VariableNode->IsInputArgument())
			{
				FString FullPath;
				if (InPinProxy.IsValid())
				{
					FullPath = InPinProxy.GetCallstack().GetCallPath(true);
				}
				else if(InVarExpr)
				{						
					const FRigVMASTProxy NodeProxy = InVarExpr->GetProxy().GetSibling(Node);
					FullPath = InPinProxy.GetCallstack().GetCallPath(true);
				}
				return FString::Printf(TEXT("%s%s%s"), *Prefix, *FullPath, *Suffix);
			}

			if (!bIsLiteral)
			{		
				// determine if this variable needs to be remapped
				if(InVarExpr)
				{
					FRigVMASTProxy ParentProxy = InVarExpr->GetProxy();
					while(ParentProxy.GetCallstack().Num() > 1)
					{
						ParentProxy = ParentProxy.GetParent();

						if(URigVMFunctionReferenceNode* FunctionReferenceNode = ParentProxy.GetSubject<URigVMFunctionReferenceNode>())
						{
							const FName RemappedVariableName = FunctionReferenceNode->GetOuterVariableName(VariableName);
							if(!RemappedVariableName.IsNone())
							{
								VariableName = RemappedVariableName;
							}
						}
					}
				}
			
				return FString::Printf(TEXT("%sVariable::%s%s"), *Prefix, *VariableName.ToString(), *Suffix);
			}
		}		
	}
	else
	{
		if (InVarExpr && InPin->IsDefinedAsInputVariable() && !bIsVariableClone)
		{
			const FString VariableName = InPin->IsBoundToVariable() ? InPin->GetBoundVariableName() : InPin->GetName();
			return FString::Printf(TEXT("%sVariable::%s%s"), *Prefix, *VariableName, *Suffix);
		}
		if (InVarExpr)
		{
			const FRigVMASTProxy NodeProxy = InVarExpr->GetProxy().GetSibling(Node);
			if (InVarExpr->GetParser()->GetExprForSubject(NodeProxy))
			{
				// rely on the proxy callstack to differentiate registers
				const FString CallStackPath = NodeProxy.GetCallstack().GetCallPath(false /* include last */);
				if (!CallStackPath.IsEmpty() && !InPinProxy.IsValid())
				{
					Prefix += CallStackPath + TEXT("|");
					bUseFullNodePath = false;
				}
			}
			else if(Node->IsA<URigVMFunctionInterfaceNode>())
			{
				FString FullPath = InPinProxy.GetCallstack().GetCallPath(true);

				// if this is a pin on an entry / return node within a collapse node
				if (const URigVMGraph* Graph = Node->GetGraph())
				{
					if (const URigVMCollapseNode* CollapseNode = Graph->GetTypedOuter<URigVMCollapseNode>())
					{
						// if this collapse node is not on a function library
						if (Cast<URigVMFunctionLibrary>(CollapseNode->GetGraph()) == nullptr)
						{
							static const FString EntryPrefix = FString::Printf(TEXT("|%s."), FRigVMGraphFunctionData::EntryString); 
							static const FString ReturnPrefix = FString::Printf(TEXT("|%s."), FRigVMGraphFunctionData::ReturnString);
							static const FString CollapseEntryPrefix = FString::Printf(TEXT("|Collapse%s."), FRigVMGraphFunctionData::EntryString); 
							static const FString CollapseReturnPrefix = FString::Printf(TEXT("|Collapse%s."), FRigVMGraphFunctionData::ReturnString);

							// adapt the path to use CollapseEntry instead of Entry
							if (FullPath.ReplaceInline(*EntryPrefix, *CollapseEntryPrefix) == 0)
							{
								// adapt the path to use CollapseReturn instead of Return
								(void)FullPath.ReplaceInline(*ReturnPrefix, *CollapseReturnPrefix);
							}
						}
					}
				}
				return FString::Printf(TEXT("%s%s%s"), *Prefix, *FullPath, *Suffix);
			}
		}

		if (bIsVariableClone)
		{
			Suffix = TEXT("::Clone");
			bUseFullNodePath = true;
		}
	}

	if (InPinProxy.IsValid())
	{
		const FString FullPath = InPinProxy.GetCallstack().GetCallPath(true);
		return FString::Printf(TEXT("%s%s%s"), *Prefix, *FullPath, *Suffix);
	}

	if (InVarExpr)
	{
		if (bUseFullNodePath)
		{
			FString FullPath = InVarExpr->GetProxy().GetCallstack().GetCallPath(true);
			return FString::Printf(TEXT("%s%s%s"), *Prefix, *FullPath, *Suffix);
		}
		else
		{
			return FString::Printf(TEXT("%s%s%s"), *Prefix, *InPin->GetPinPath(), *Suffix);
		}
	}

	FString PinPath = InPin->GetPinPath(bUseFullNodePath);
	return FString::Printf(TEXT("%s%s%s"), *Prefix, *PinPath, *Suffix);
}

FString URigVMCompiler::GetPinHash(const URigVMPin* InPin, const FRigVMVarExprAST* InVarExpr, const URigVMLibraryNode* FunctionCompiling, const FRigVMASTProxy& InPinProxy)
{
	const FString Hash = GetPinHashImpl(InPin, InVarExpr, FunctionCompiling, InPinProxy);
	if(FunctionCompiling == nullptr)
	{
		ensureMsgf(!Hash.Contains(TEXT("FunctionLibrary::")), TEXT("A library path should never be part of a pin hash %s."), *Hash);
	}
	return Hash;
}

const FRigVMVarExprAST* URigVMCompiler::GetSourceVarExpr(const FRigVMExprAST* InExpr)
{
	if(InExpr)
	{
		if(InExpr->IsA(FRigVMExprAST::EType::CachedValue))
		{
			return GetSourceVarExpr(InExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr());
		}

		if(InExpr->IsA(FRigVMExprAST::EType::Var))
		{
			const FRigVMVarExprAST* VarExpr = InExpr->To<FRigVMVarExprAST>();
			
			if(VarExpr->GetPin()->IsReferenceCountedContainer() &&
				((VarExpr->GetPin()->GetDirection() == ERigVMPinDirection::Input) || (VarExpr->GetPin()->GetDirection() == ERigVMPinDirection::IO)))
			{
				// if this is a variable setter we cannot follow the source var
				if(VarExpr->GetPin()->GetDirection() == ERigVMPinDirection::Input)
				{
					if(VarExpr->GetPin()->GetNode()->IsA<URigVMVariableNode>())
					{
						return VarExpr;
					}
				}
				
				if(const FRigVMExprAST* AssignExpr = VarExpr->GetFirstChildOfType(FRigVMExprAST::EType::Assign))
				{
					// don't follow a copy assignment
					if(AssignExpr->IsA(FRigVMExprAST::EType::Copy))
					{
						return VarExpr;
					}
					
					if(const FRigVMExprAST* CachedValueExpr = VarExpr->GetFirstChildOfType(FRigVMExprAST::EType::CachedValue))
					{
						return GetSourceVarExpr(CachedValueExpr->To<FRigVMCachedValueExprAST>()->GetVarExpr());
					}
					else if(const FRigVMExprAST* ChildExpr = VarExpr->GetFirstChildOfType(FRigVMExprAST::EType::Var))
					{
						return GetSourceVarExpr(ChildExpr->To<FRigVMVarExprAST>());
					}
				}
			}
			return VarExpr;
		}
	}

	return nullptr;
}

void URigVMCompiler::MarkDebugWatch(FRigVMExtendedExecuteContext& Context, bool bRequired,
	URigVMPin* InPin, URigVM* OutVM, TMap<FString, FRigVMOperand>* OutOperands,
	TSharedPtr<FRigVMParserAST> InRuntimeAST)
{
	check(InPin);
	check(OutVM);
	check(OutOperands);
	check(InRuntimeAST.IsValid());
	
	URigVMPin* Pin = InPin->GetRootPin();
	TArray<const FRigVMExprAST*> Expressions = InRuntimeAST->GetExpressionsForSubject(Pin);

	if (Expressions.IsEmpty())
	{
		FString PinHash = GetPinHashImpl(Pin, nullptr, nullptr);
		if(const FRigVMOperand* Operand = OutOperands->Find(PinHash))
		{
			if (OutVM->GetDebugMemory(Context)->Num() == 0)
			{
				OutVM->CreateDebugMemory(Context);
			}
			Context.MarkOperandForDebugging(*Operand, bRequired);
		}
		return;
	}
	
	TArray<FRigVMOperand> VisitedKeys;
	for(int32 ExpressionIndex=0;ExpressionIndex<Expressions.Num();ExpressionIndex++)
	{
		const FRigVMExprAST* Expression = Expressions[ExpressionIndex];
		
		check(Expression->IsA(FRigVMExprAST::EType::Var));
		const FRigVMVarExprAST* VarExpression = Expression->To<FRigVMVarExprAST>();

		if(VarExpression->GetPin() == Pin)
		{
			// literals don't need to be stored on the debug memory
			if(VarExpression->IsA(FRigVMExprAST::Literal))
			{
				// check if there's also an IO expression for this pin
				for(int32 ParentIndex=0;ParentIndex<VarExpression->NumParents();ParentIndex++)
				{
					const FRigVMExprAST* ParentExpression = VarExpression->ParentAt(ParentIndex);
					if(ParentExpression->IsA(FRigVMExprAST::EType::Assign))
					{
						if(const FRigVMExprAST* GrandParentExpression = ParentExpression->GetParent())
						{
							if(GrandParentExpression->IsA(FRigVMExprAST::EType::Var))
							{
								if(GrandParentExpression->To<FRigVMVarExprAST>()->GetPin() == Pin)
								{
									Expressions.Add(GrandParentExpression);
								}
							}
						}
					}
				}
				continue;
			}
		}

		FString PinHash = GetPinHashImpl(Pin, VarExpression, CurrentlyCompilingFunctionNode);
		if(const FRigVMOperand* Operand = OutOperands->Find(PinHash))
		{
			if (OutVM->GetDebugMemory(Context)->Num() == 0)
			{
				OutVM->CreateDebugMemory(Context);
			}
			Context.MarkOperandForDebugging(*Operand, bRequired);
		}
	}
}

UScriptStruct* URigVMCompiler::GetScriptStructForCPPType(const FString& InCPPType)
{
	if (InCPPType == TEXT("FRotator"))
	{
		return TBaseStructure<FRotator>::Get();
	}
	if (InCPPType == TEXT("FQuat"))
	{
		return TBaseStructure<FQuat>::Get();
	}
	if (InCPPType == TEXT("FTransform"))
	{
		return TBaseStructure<FTransform>::Get();
	}
	if (InCPPType == TEXT("FLinearColor"))
	{
		return TBaseStructure<FLinearColor>::Get();
	}
	if (InCPPType == TEXT("FColor"))
	{
		return TBaseStructure<FColor>::Get();
	}
	if (InCPPType == TEXT("FPlane"))
	{
		return TBaseStructure<FPlane>::Get();
	}
	if (InCPPType == TEXT("FVector"))
	{
		return TBaseStructure<FVector>::Get();
	}
	if (InCPPType == TEXT("FVector2D"))
	{
		return TBaseStructure<FVector2D>::Get();
	}
	if (InCPPType == TEXT("FVector4"))
	{
		return TBaseStructure<FVector4>::Get();
	}
	return nullptr;
}

TArray<URigVMPin*> URigVMCompiler::GetLinkedPins(URigVMPin* InPin, bool bInputs, bool bOutputs, bool bRecursive)
{
	TArray<URigVMPin*> LinkedPins;
	for (URigVMLink* Link : InPin->GetLinks())
	{
		if (bInputs && Link->GetTargetPin() == InPin)
		{
			LinkedPins.Add(Link->GetSourcePin());
		}
		else if (bOutputs && Link->GetSourcePin() == InPin)
		{
			LinkedPins.Add(Link->GetTargetPin());
		}
	}

	if (bRecursive)
	{
		for (URigVMPin* SubPin : InPin->GetSubPins())
		{
			LinkedPins.Append(GetLinkedPins(SubPin, bInputs, bOutputs, bRecursive));
		}
	}

	return LinkedPins;
}

int32 URigVMCompiler::GetElementSizeFromCPPType(const FString& InCPPType, UScriptStruct* InScriptStruct)
{
	if (InScriptStruct == nullptr)
	{
		InScriptStruct = GetScriptStructForCPPType(InCPPType);
	}
	if (InScriptStruct != nullptr)
	{
		return InScriptStruct->GetStructureSize();
	}
	if (InCPPType == TEXT("bool"))
	{
		return sizeof(bool);
	}
	if (InCPPType == TEXT("int32"))
	{
		return sizeof(int32);
	}
	if (InCPPType == TEXT("float"))
	{
		return sizeof(float);
	}
	if (InCPPType == TEXT("double"))
	{
		return sizeof(double);
	}
	if (InCPPType == TEXT("FName"))
	{
		return sizeof(FName);
	}
	if (InCPPType == TEXT("FString"))
	{
		return sizeof(FString);
	}

	ensure(false);
	return 0;
}

FRigVMOperand URigVMCompiler::FindOrAddRegister(const FRigVMVarExprAST* InVarExpr, FRigVMCompilerWorkData& WorkData)
{
	InVarExpr = GetSourceVarExpr(InVarExpr);

	FRigVMOperand const* ExistingOperandPtr = WorkData.ExprToOperand.Find(InVarExpr);
	if (ExistingOperandPtr)
	{
		return *ExistingOperandPtr;
	}

	const URigVMPin::FPinOverrideMap& PinOverrides = InVarExpr->GetParser()->GetPinOverrides();
	URigVMPin::FPinOverride PinOverride(InVarExpr->GetProxy(), PinOverrides);

	URigVMPin* Pin = InVarExpr->GetPin();

	if(Pin->IsExecuteContext())
	{
		return FRigVMOperand();
	}
	
	FString CPPType = Pin->GetCPPType();
	FString BaseCPPType = Pin->IsArray() ? Pin->GetArrayElementCppType() : CPPType;
	FString Hash = GetPinHash(Pin, InVarExpr, CurrentlyCompilingFunctionNode);
	FRigVMOperand Operand;
	FString RegisterKey = Hash;

	bool bIsLiteral = InVarExpr->GetType() == FRigVMExprAST::EType::Literal;
	bool bIsVariable = Pin->IsRootPin() && (Pin->GetName() == URigVMVariableNode::ValueName) &&
		InVarExpr->GetPin()->GetNode()->IsA<URigVMVariableNode>();

	// external variables don't require to add any register.
	if((bIsVariable || Pin->IsDefinedAsInputVariable()))
	{
		const TArray<FRigVMExternalVariableDef>& VMExternalVariableDefs = WorkData.VM->GetExternalVariableDefs();
		for(int32 ExternalVariableIndex = 0; ExternalVariableIndex < VMExternalVariableDefs.Num(); ExternalVariableIndex++)
		{
			const FName& ExternalVariableName = VMExternalVariableDefs[ExternalVariableIndex].GetName();
			const FString ExternalVariableHash = FString::Printf(TEXT("Variable::%s"), *ExternalVariableName.ToString());
			if(ExternalVariableHash == Hash)
			{
				Operand = FRigVMOperand(ERigVMMemoryType::External, ExternalVariableIndex, INDEX_NONE);
				WorkData.ExprToOperand.Add(InVarExpr, Operand);
				WorkData.PinPathToOperand->FindOrAdd(Hash) = Operand;
				return Operand;
			}
		}
	}

	const ERigVMMemoryType MemoryType =
		bIsLiteral ? ERigVMMemoryType::Literal : ERigVMMemoryType::Work;

	TArray<FString> HashesWithSharedOperand;
	
	ExistingOperandPtr = WorkData.PinPathToOperand->Find(Hash);
	if (!ExistingOperandPtr)
	{
		if(WorkData.Settings.ASTSettings.bFoldAssignments) 		
		{
			// Get all possible pins that lead to the same operand		
			const FRigVMCompilerWorkData::FRigVMASTProxyArray PinProxies = FindProxiesWithSharedOperand(InVarExpr, WorkData);
			ensure(!PinProxies.IsEmpty());

			// Look for an existing operand from a different pin with shared operand
			for (const FRigVMASTProxy& Proxy : PinProxies)
			{
				if (const URigVMPin* VirtualPin = Cast<URigVMPin>(Proxy.GetSubject()))
				{
					const FString VirtualPinHash = GetPinHash(VirtualPin, InVarExpr, CurrentlyCompilingFunctionNode, Proxy);
					HashesWithSharedOperand.Add(VirtualPinHash);
					if (Pin != VirtualPin)
					{
						ExistingOperandPtr = WorkData.PinPathToOperand->Find(VirtualPinHash);
						if (ExistingOperandPtr)
						{
							break;
						}
					}
				}	
			}
		}
	}
	
	if (ExistingOperandPtr)
	{
		// Dereference the operand pointer here since modifying the PinPathToOperand map will invalidate the pointer.
		FRigVMOperand ExistingOperand = *ExistingOperandPtr;
		
		// Add any missing hash that shares this existing operand
		for (const FString& VirtualPinHash : HashesWithSharedOperand)
		{
			WorkData.PinPathToOperand->Add(VirtualPinHash, ExistingOperand);
		}
		
		check(!WorkData.ExprToOperand.Contains(InVarExpr));
		WorkData.ExprToOperand.Add(InVarExpr, ExistingOperand);
		
		return ExistingOperand;
	}

	// create remaining operands / registers
	if (!Operand.IsValid())
	{
		FName RegisterName = *RegisterKey;

		FString JoinedDefaultValue;
		TArray<FString> DefaultValues;
		if (Pin->IsArray())
		{
			if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				JoinedDefaultValue = Pin->GetDefaultValue(PinOverride);
				DefaultValues = URigVMPin::SplitDefaultValue(JoinedDefaultValue);
			}
			else
			{
				JoinedDefaultValue = Pin->GetDefaultValue(PinOverride);
				if(!JoinedDefaultValue.IsEmpty())
				{
					if(JoinedDefaultValue[0] == TCHAR('('))
					{
						DefaultValues = URigVMPin::SplitDefaultValue(JoinedDefaultValue);
					}
					else
					{
						DefaultValues.Add(JoinedDefaultValue);
					}
				}
			}

			while (DefaultValues.Num() < Pin->GetSubPins().Num())
			{
				DefaultValues.Add(FString());
			}
		}
		else if (URigVMEnumNode* EnumNode = Cast<URigVMEnumNode>(Pin->GetNode()))
		{
			FString EnumValueStr = EnumNode->GetDefaultValue(PinOverride);
			if (UEnum* Enum = EnumNode->GetEnum())
			{
				JoinedDefaultValue = FString::FromInt((int32)Enum->GetValueByNameString(EnumValueStr));
				DefaultValues.Add(JoinedDefaultValue);
			}
			else
			{
				JoinedDefaultValue = FString::FromInt(0);
				DefaultValues.Add(JoinedDefaultValue);
			}
		}
		else
		{
			JoinedDefaultValue = Pin->GetDefaultValue(PinOverride);
			DefaultValues.Add(JoinedDefaultValue);
		}

		UScriptStruct* ScriptStruct = Pin->GetScriptStruct();
		if (ScriptStruct == nullptr)
		{
			ScriptStruct = GetScriptStructForCPPType(BaseCPPType);
		}

		if(Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			bool bValidHiddenPin = false;
			if(Pin->GetNode()->IsA<URigVMUnitNode>())
			{
				UScriptStruct* UnitStruct = Cast<URigVMUnitNode>(Pin->GetNode())->GetScriptStruct();
				const FProperty* Property = UnitStruct->FindPropertyByName(Pin->GetFName());
				check(Property);

				JoinedDefaultValue.Reset();
					
				FStructOnScope StructOnScope(UnitStruct);
				const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope.GetStructMemory();
				const uint8* PropertyMemory = Property->ContainerPtrToValuePtr<uint8>(StructMemory);
				
				Property->ExportText_Direct(
					JoinedDefaultValue,
					PropertyMemory,
					PropertyMemory,
					nullptr,
					PPF_None,
					nullptr);

				if (!Property->HasMetaData(FRigVMStruct::SingletonMetaName))
				{
					bValidHiddenPin = true;
				}
			}
			else if(URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Pin->GetNode()))
			{
				bValidHiddenPin = true;
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					bValidHiddenPin = !Factory->HasArgumentMetaData(Pin->GetFName(), FRigVMStruct::SingletonMetaName);
					JoinedDefaultValue = Factory->GetArgumentDefaultValue(Pin->GetFName(), Pin->GetTypeIndex());
				}
			}

			if(bValidHiddenPin)
			{
				CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(CPPType);
				JoinedDefaultValue = URigVMPin::GetDefaultValueForArray({ JoinedDefaultValue });
			}
		}

		// Force a unique literal register for interactive nodes (e.g. nodes with DM active) so
		// that writing to the register during manipulation does not affect other nodes that
		// would otherwise share the same literal register due to having the same default value.
		const bool bIsInteractiveLiteral = bIsLiteral
			&& !WorkData.Settings.ASTSettings.InteractiveNodes.IsEmpty()
			&& WorkData.Settings.ASTSettings.InteractiveNodes.Contains(FObjectKey(Pin->GetNode()));
		const bool bMakeUniqueProperty = (MemoryType == ERigVMMemoryType::Work) || bIsInteractiveLiteral;
		Operand = WorkData.FindOrAddProperty(MemoryType, RegisterName, CPPType, Pin->GetCPPTypeObject(), JoinedDefaultValue, bMakeUniqueProperty);

		// Record the index so FindOrAddProperty's sharing scan skips it for non-interactive
		// callers — preventing other nodes from being mapped to this node's register.
		if(bIsInteractiveLiteral && Operand.IsValid())
		{
			WorkData.InteractiveLiteralIndices.Add(Operand.GetRegisterIndex());
		}
	}
	ensure(Operand.IsValid());

	// Get all possible pins that lead to the same operand
	if(WorkData.Settings.ASTSettings.bFoldAssignments)
	{
		// tbd: this functionality is only needed when there is a watch anywhere?
		//if(!WorkData.WatchedPins.IsEmpty())
		{
			for (const FString& VirtualPinHash : HashesWithSharedOperand)
			{
				WorkData.PinPathToOperand->Add(VirtualPinHash, Operand);
			}
		}
	}
	else
	{
		if(ExistingOperandPtr == nullptr)
		{
			WorkData.PinPathToOperand->Add(Hash, Operand);
		}
	}
	
	check(!WorkData.ExprToOperand.Contains(InVarExpr));
	WorkData.ExprToOperand.Add(InVarExpr, Operand);

	return Operand;
}

const FRigVMCompilerWorkData::FRigVMASTProxyArray& URigVMCompiler::FindProxiesWithSharedOperand(const FRigVMVarExprAST* InVarExpr, FRigVMCompilerWorkData& WorkData)
{
	const FRigVMASTProxy& InProxy = InVarExpr->GetProxy();
	if(const FRigVMCompilerWorkData::FRigVMASTProxyArray* ExistingArray = WorkData.CachedProxiesWithSharedOperand.Find(InProxy))
	{
		return *ExistingArray;
	}
	
	FRigVMCompilerWorkData::FRigVMASTProxyArray PinProxies, PinProxiesToProcess;
	const FRigVMCompilerWorkData::FRigVMASTProxySourceMap& ProxySources = *WorkData.ProxySources;

	PinProxiesToProcess.Add(InProxy);

	const FString CPPType = InProxy.GetSubjectChecked<URigVMPin>()->GetCPPType();
	const bool bEnableCallables = CVarRigVMCompileFunctionsToCallables->GetBool(); 

	for(int32 ProxyIndex = 0; ProxyIndex < PinProxiesToProcess.Num(); ProxyIndex++)
	{
		if (PinProxiesToProcess[ProxyIndex].IsValid())
		{
			if (URigVMPin* Pin = Cast<URigVMPin>(PinProxiesToProcess[ProxyIndex].GetSubject()))
			{
				if (Pin->GetNode()->IsA<URigVMVariableNode>())
				{
					if (Pin->GetDirection() == ERigVMPinDirection::Input)
					{
						continue;
					}
				}

				// due to LWC we may have two pins that don't
				// actually share the same CPP type (float vs double)
				if(Pin->GetCPPType() != CPPType)
				{
					continue;
				}

				// Non-lazy pins in node with lazy pins cannot share operands
				if (Pin->GetDirection() == ERigVMPinDirection::Input && !Pin->IsLazy() && Pin->GetNode()->HasLazyPin())
				{
					continue;
				}

				if(Pin->IsProgrammaticPin())
				{
					continue;
				}
			}
			PinProxies.Add(PinProxiesToProcess[ProxyIndex]);
		}

		if(const FRigVMASTProxy* SourceProxy = ProxySources.Find(PinProxiesToProcess[ProxyIndex]))
		{
			if(SourceProxy->IsValid())
			{
				if (!PinProxies.Contains(*SourceProxy) && !PinProxiesToProcess.Contains(*SourceProxy))
				{
					bool bValidSourceProxy = true;
					if (bEnableCallables && PinProxiesToProcess[ProxyIndex].IsValid())
					{
						if (URigVMPin* TargetPin = Cast<URigVMPin>(PinProxiesToProcess[ProxyIndex].GetSubject()))
						{
							if (TargetPin->GetNode()->IsA<URigVMFunctionReturnNode>())
							{
								if (URigVMPin* SourcePin = Cast<URigVMPin>(SourceProxy->GetSubject()))
								{
									if (SourcePin->GetNode()->IsA<URigVMFunctionEntryNode>())
									{
										bValidSourceProxy = false;
									}
								}
							}
						}
					}

					if (bValidSourceProxy)
					{
						PinProxiesToProcess.Add(*SourceProxy);
					}
				}
			}
		}

		if(const FRigVMCompilerWorkData::FRigVMASTProxyArray* TargetProxies = WorkData.ProxyTargets.Find(PinProxiesToProcess[ProxyIndex]))
		{
			for(const FRigVMASTProxy& TargetProxy : *TargetProxies)
			{
				if(TargetProxy.IsValid())
				{
					if (!PinProxies.Contains(TargetProxy) && !PinProxiesToProcess.Contains(TargetProxy))
					{
						bool bValidTargetProxy = true;
						if (bEnableCallables && PinProxiesToProcess[ProxyIndex].IsValid())
						{
							if (URigVMPin* SourcePin = Cast<URigVMPin>(PinProxiesToProcess[ProxyIndex].GetSubject()))
							{
								if (SourcePin->GetNode()->IsA<URigVMFunctionEntryNode>())
								{
									if (URigVMPin* TargetPin = Cast<URigVMPin>(TargetProxy.GetSubject()))
									{
										if (TargetPin->GetNode()->IsA<URigVMFunctionReturnNode>())
										{
											bValidTargetProxy = false;
										}
									}
								}
							}
						}

						if (bValidTargetProxy)
						{
							PinProxiesToProcess.Add(TargetProxy);
						}
					}
				}
			}
		}
	}

	if (PinProxies.IsEmpty())
	{
		PinProxies.Add(InVarExpr->GetProxy());
	}

	// store the cache for all other proxies within this group
	for(const FRigVMASTProxy& CurrentProxy : PinProxies)
	{
		if(CurrentProxy != InProxy)
		{
			WorkData.CachedProxiesWithSharedOperand.Add(CurrentProxy, PinProxies);
		}
	}

	// finally store and return the cache the the input proxy
	return WorkData.CachedProxiesWithSharedOperand.Add(InProxy, PinProxies);
}

FString URigVMCompiler::GetPinNameWithDirectionPrefix(const URigVMPin* Pin)
{
	static const FString EntryString = FRigVMGraphFunctionData::EntryString;
	static const FString ReturnString = FRigVMGraphFunctionData::ReturnString;
	const FString & Prefix = (Pin->GetDirection() == ERigVMPinDirection::Input) ? EntryString : (Pin->GetDirection() == ERigVMPinDirection::Output) ? ReturnString : "";
	const FString NameWithDirectionPrefix = Prefix + "_" + FRigVMPropertyDescription::SanitizeName(FName(Pin->GetName())).ToString();
	return NameWithDirectionPrefix;
}

int32 URigVMCompiler::GetOperandFunctionInterfaceParameterIndex(const TArray<FString>& OperandsPinNames, const FRigVMFunctionCompilationData* FunctionCompilationData, const FRigVMOperand& Operand)
{
	const FRigVMFunctionCompilationPropertyDescription& CompilationPropertyDescription = (Operand.GetMemoryType() == ERigVMMemoryType::Work)
		? FunctionCompilationData->WorkPropertyDescriptions[Operand.GetRegisterIndex()]
		: FunctionCompilationData->LiteralPropertyDescriptions[Operand.GetRegisterIndex()];

	const FString PropertyName = FRigVMPropertyDescription::SanitizeName(CompilationPropertyDescription.Name).ToString();
	const int32 NumParams = OperandsPinNames.Num();
	for (int32 ParamIndex = 0; ParamIndex < NumParams; ParamIndex++)
	{
		const FString& InterfacePinName = OperandsPinNames[ParamIndex];
		if (const int32 SubStrStart = PropertyName.Find(InterfacePinName, ESearchCase::CaseSensitive, ESearchDir::FromEnd); SubStrStart != -1)
		{
			const FString Name = PropertyName.RightChop(SubStrStart);  // with this, we make sure that we don't return Argument_1 when searching for Argument
			if (Name.Equals(InterfacePinName, ESearchCase::CaseSensitive))
			{
				return ParamIndex;
			}
		}
	}

	return INDEX_NONE;
}

bool URigVMCompiler::ValidateNode(const FRigVMCompileSettings& InSettings, URigVMNode* InNode, bool bCheck)
{
	if(bCheck)
	{
		check(InNode)
	}
	if(InNode)
	{
		if(InNode->HasWildCardPin())
		{
			InSettings.Reportf(EMessageSeverity::Error, InNode, TEXT("[%s] Node @@ has unresolved pins of wildcard type."), *InNode->GetPackage()->GetPathName());
			return false;
		}
		return true;
	}
	return false;
}

void URigVMCompiler::ReportInfo(const FRigVMCompileSettings& InSettings, const FString& InMessage)
{
	if (InSettings.SurpressInfoMessages)
	{
		return;
	}
	InSettings.Report(EMessageSeverity::Info, nullptr, InMessage);
}

void URigVMCompiler::ReportWarning(const FRigVMCompileSettings& InSettings, const FString& InMessage)
{
	InSettings.Report(EMessageSeverity::Warning, nullptr, InMessage);
}

void URigVMCompiler::ReportError(const FRigVMCompileSettings& InSettings, const FString& InMessage)
{
	InSettings.Report(EMessageSeverity::Error, nullptr, InMessage);
}
