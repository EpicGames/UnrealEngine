// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMDeveloper.h: Module implementation.
=============================================================================*/

#include "RigVMDeveloperModule.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintCompiler.h"
#include "RigVMModel/RigVMBuildData.h"

DEFINE_LOG_CATEGORY(LogRigVMDeveloper);

class FRigVMDeveloperModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FRigVMBlueprintCompiler RigVMBlueprintCompiler;
	static TSharedPtr<FKismetCompilerContext> GetRigVMCompiler(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions);
};

IMPLEMENT_MODULE(FRigVMDeveloperModule, RigVMDeveloper);

void FRigVMDeveloperModule::StartupModule()
{
	// Register blueprint compiler
	FKismetCompilerContext::RegisterCompilerForBP(URigVMBlueprint::StaticClass(), &FRigVMDeveloperModule::GetRigVMCompiler);
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetCompilers().Add(&RigVMBlueprintCompiler);
	URigVMBuildData::Get()->SetupRigVMGraphFunctionPointers();
}

void FRigVMDeveloperModule::ShutdownModule()
{
	IKismetCompilerInterface* KismetCompilerModule = FModuleManager::GetModulePtr<IKismetCompilerInterface>("KismetCompiler");
	if (KismetCompilerModule)
	{
		KismetCompilerModule->GetCompilers().Remove(&RigVMBlueprintCompiler);
	}
	URigVMBuildData::TearDownRigVMGraphFunctionPointers();
}

TSharedPtr<FKismetCompilerContext> FRigVMDeveloperModule::GetRigVMCompiler(UBlueprint* BP, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompileOptions)
{
	return TSharedPtr<FKismetCompilerContext>(new FRigVMBlueprintCompilerContext(BP, InMessageLog, InCompileOptions));
}
