// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMDeveloperTypeUtils.h"

#include "Internationalization/StringTableCore.h"
#include "RigVMFunctions/RigVMDispatch_CastEnum.h"
#include "RigVMFunctions/RigVMDispatch_CastObject.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMCore/RigVMVariableDescription.h"

#define LOCTEXT_NAMESPACE "RigVMTypeUtils"

#if WITH_EDITOR

#include "EdGraphSchema_K2.h"


FEdGraphPinType RigVMTypeUtils::PinTypeFromTypeIndex(const TRigVMTypeIndex& InTypeIndex)
{
	const FRigVMTemplateArgumentType& Type = FRigVMRegistry::Get().GetType(InTypeIndex);
	if(Type.CPPType.IsValid())
	{
		return PinTypeFromCPPType(Type.CPPType, Type.CPPTypeObject);
	}
	return FEdGraphPinType();
}

FEdGraphPinType RigVMTypeUtils::PinTypeFromCPPType(const FName& InCPPType, UObject* InCPPTypeObject)
{
	FEdGraphPinType PinType;
	PinType.ResetToDefaults();
	PinType.PinCategory = NAME_None;

	FName BaseCPPType = InCPPType;
	PinType.ContainerType = EPinContainerType::None;
	if (RigVMTypeUtils::IsArrayType(InCPPType.ToString()))
	{
		BaseCPPType = *RigVMTypeUtils::BaseTypeFromArrayType(InCPPType.ToString());
		PinType.ContainerType = EPinContainerType::Array;
	}

	if (BaseCPPType == BoolTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	}
	else if (BaseCPPType == Int32TypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
	}
	else if (BaseCPPType == FloatTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
	}
	else if (BaseCPPType == DoubleTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	}
	else if (BaseCPPType == FNameTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
	}
	else if (BaseCPPType == FStringTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	}
	else if (Cast<UScriptStruct>(InCPPTypeObject))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = InCPPTypeObject;
	}
	else if (Cast<UEnum>(InCPPTypeObject))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
		PinType.PinSubCategoryObject = InCPPTypeObject;
	}
	else if (Cast<UClass>(InCPPTypeObject))
	{
		const bool bIsClass = RigVMTypeUtils::IsUClassType(InCPPType.ToString());
		PinType.PinCategory = bIsClass ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = InCPPTypeObject;
	}

	return PinType;
}

FEdGraphPinType RigVMTypeUtils::PinTypeFromExternalVariable(const FRigVMExternalVariable& InExternalVariable)
{
	FEdGraphPinType PinType;
	PinType.ResetToDefaults();
	PinType.PinCategory = NAME_None;

	if (InExternalVariable.GetBaseCPPType() == BoolTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	}
	else if (InExternalVariable.GetBaseCPPType() == Int32TypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
	}
	else if (InExternalVariable.GetBaseCPPType() == FloatTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
	}
	else if (InExternalVariable.GetBaseCPPType() == DoubleTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	}
	else if (InExternalVariable.GetBaseCPPType() == FNameTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
	}
	else if (InExternalVariable.GetBaseCPPType() == FStringTypeName)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	}
	else if (Cast<UScriptStruct>(InExternalVariable.GetCPPTypeObject()))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InExternalVariable.GetCPPTypeObject());
	}
	else if (Cast<UEnum>(InExternalVariable.GetCPPTypeObject()))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InExternalVariable.GetCPPTypeObject());
	}
	else if (Cast<UClass>(InExternalVariable.GetCPPTypeObject()))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = const_cast<UObject*>(InExternalVariable.GetCPPTypeObject());
	}

	if (InExternalVariable.IsArray())
	{
		PinType.ContainerType = EPinContainerType::Array;
	}
	else
	{
		PinType.ContainerType = EPinContainerType::None;
	}

	return PinType;
}

FEdGraphPinType RigVMTypeUtils::PinTypeFromRigVMVariableDescription(
	const FRigVMGraphVariableDescription& InVariableDescription)
{
	FEdGraphPinType PinType;
	PinType.ResetToDefaults();
	PinType.PinCategory = NAME_None;

	FString CurrentCPPType = InVariableDescription.CPPType;
	if (IsArrayType(CurrentCPPType))
	{
		PinType.ContainerType = EPinContainerType::Array;
		CurrentCPPType = BaseTypeFromArrayType(CurrentCPPType);
	}
	else
	{
		PinType.ContainerType = EPinContainerType::None;
	}

	if (CurrentCPPType == BoolType)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	}
	else if (CurrentCPPType == Int32Type)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
	}
	else if (CurrentCPPType == FloatType)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Float;
	}
	else if (CurrentCPPType == DoubleType)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Real;
		PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	}
	else if (CurrentCPPType == FNameType)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
	}
	else if (CurrentCPPType == FStringType)
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	}
	else if (Cast<UScriptStruct>(InVariableDescription.CPPTypeObject))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = InVariableDescription.CPPTypeObject;
	}
	else if (Cast<UEnum>(InVariableDescription.CPPTypeObject))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
		PinType.PinSubCategoryObject = InVariableDescription.CPPTypeObject;
	}
	else if (Cast<UClass>(InVariableDescription.CPPTypeObject))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		PinType.PinSubCategoryObject = InVariableDescription.CPPTypeObject;
	}

	return PinType;
}

FEdGraphPinType RigVMTypeUtils::SubPinType(const FEdGraphPinType& InPinType, const FString& InSegmentPath)
{
	FEdGraphPinType Result;
	if (InSegmentPath.IsEmpty())
	{
		return InPinType;
	}

	if (InPinType.PinSubCategoryObject.IsValid())
	{
		if (UStruct* Struct = Cast<UStruct>(InPinType.PinSubCategoryObject))
		{
			int32 PartIndex = 0; 
			TArray<FString> Parts;
			if (!URigVMPin::SplitPinPath(InSegmentPath, Parts))
			{
				Parts.Add(InSegmentPath);
			}
			
			FProperty* Property = Struct->FindPropertyByName(*Parts[PartIndex++]);
			while (PartIndex < Parts.Num() && Property != nullptr)
			{
				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					Property = ArrayProperty->Inner;
					PartIndex++;
					continue;
				}

				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					Struct = StructProperty->Struct;
					Property = Struct->FindPropertyByName(*Parts[PartIndex++]);
					continue;
				}

				break;
			}

			if (PartIndex == Parts.Num() && Property)
			{
				UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>()->ConvertPropertyToPinType(Property, Result);
			}
		}
	}

	return Result;
}

bool RigVMTypeUtils::CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, UObject** OutCPPTypeObject, bool bGetBaseCPPType)
{
	check(InPin);
	OutCPPType = InPin->GetCPPType();
	if(bGetBaseCPPType && IsArrayType(OutCPPType))
	{
		OutCPPType = BaseTypeFromArrayType(OutCPPType);
	}
	if(OutCPPTypeObject)
	{
		*OutCPPTypeObject = InPin->GetCPPTypeObject();
	}
	return true;
}

bool RigVMTypeUtils::CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, FName& OutCPPTypeObjectPath, bool bGetBaseCPPType)
{
	OutCPPType = FString();
	OutCPPTypeObjectPath = NAME_None;
	UObject* CPPTypeObject = nullptr;
	if (RigVMTypeUtils::CPPTypeFromPin(InPin, OutCPPType, &CPPTypeObject, bGetBaseCPPType))
	{
		if (CPPTypeObject)
		{
			OutCPPTypeObjectPath = *CPPTypeObject->GetPathName();
		}
		return true;
	}
	return false;
}

bool RigVMTypeUtils::CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, FString& OutCPPTypeObjectPath, bool bGetBaseCPPType)
{
	OutCPPType = FString();
	OutCPPTypeObjectPath.Reset();
	FName CPPTypeObjectPath(NAME_None);
	if (RigVMTypeUtils::CPPTypeFromPin(InPin, OutCPPType, CPPTypeObjectPath, bGetBaseCPPType))
	{
		if (!CPPTypeObjectPath.IsNone())
		{
			OutCPPTypeObjectPath = CPPTypeObjectPath.ToString();
		}
		return true;
	}
	return false;
}

bool RigVMTypeUtils::CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, UObject** OutCPPTypeObject)
{
	FString Prefix = "";
	FString Suffix = "";
	if (InPinType.ContainerType == EPinContainerType::Array)
	{
		Prefix = TEXT("TArray<");
		Suffix = TEXT(">");
	}

	OutCPPType = FString();
	*OutCPPTypeObject = nullptr;
	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		OutCPPType = Prefix + BoolType + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		OutCPPType = Prefix + Int32Type + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
		{
			OutCPPType = Prefix + FloatType + Suffix;
		}
		else if (InPinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
		{
			OutCPPType = Prefix + DoubleType + Suffix;
		}
		else
		{
			checkf(false, TEXT("Unexpected subcategory for PC_Real pin type."));
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Float)
	{
		OutCPPType = Prefix + FloatType + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Double)
	{
		OutCPPType = Prefix + DoubleType + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		OutCPPType = Prefix + FNameType + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		OutCPPType = Prefix + FStringType + Suffix;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (UScriptStruct* Struct = Cast<UScriptStruct>(InPinType.PinSubCategoryObject))
		{
			OutCPPType = Prefix + GetUniqueStructTypeName(Struct) + Suffix;
			*OutCPPTypeObject = Struct;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
		InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject ||
		InPinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes)
	{
		if (UClass* Class = Cast<UClass>(InPinType.PinSubCategoryObject))
		{
			OutCPPType = Prefix + FString::Printf(TEXT("TObjectPtr<%s%s>"), Class->GetPrefixCPP(), *Class->GetName()) + Suffix;
			*OutCPPTypeObject = Class;
		}
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Byte ||
		InPinType.PinCategory == UEdGraphSchema_K2::PC_Enum)
	{
		if (UEnum* Enum = Cast<UEnum>(InPinType.PinSubCategoryObject))
		{
			OutCPPType = Prefix + Enum->GetFName().ToString() + Suffix;
			*OutCPPTypeObject = Enum;
		}
		else
		{
			OutCPPType = Prefix + UInt8Type + Suffix;
		}
	}
	else
	{
		return false;
	}
		
	return true;
}

bool RigVMTypeUtils::CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, FName& OutCPPTypeObjectPath)
{
	OutCPPType = FString();
	OutCPPTypeObjectPath = NAME_None;
	UObject* CPPTypeObject = nullptr;
	if (RigVMTypeUtils::CPPTypeFromPinType(InPinType, OutCPPType, &CPPTypeObject))
	{
		if (CPPTypeObject)
		{
			OutCPPTypeObjectPath = *CPPTypeObject->GetPathName();
			if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
			{
				FString StructName = GetUniqueStructTypeName(ScriptStruct);
				while (IsArrayType(OutCPPType))
				{
					OutCPPType = BaseTypeFromArrayType(OutCPPType);
					StructName = ArrayTypeFromBaseType(StructName);
				}
				OutCPPType = StructName;
			}
		}

		return true;
	}

	return false;
}

bool RigVMTypeUtils::CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, FString& OutCPPTypeObjectPath)
{
	OutCPPType = FString();
	OutCPPTypeObjectPath.Reset();
	FName CPPTypeObjectPath(NAME_None);
	if (RigVMTypeUtils::CPPTypeFromPinType(InPinType, OutCPPType, CPPTypeObjectPath))
	{
		if(!CPPTypeObjectPath.IsNone())
		{
			OutCPPTypeObjectPath = CPPTypeObjectPath.ToString();
		}
		return true;
	}

	return false;
}

TRigVMTypeIndex RigVMTypeUtils::TypeIndexFromPinType(const FEdGraphPinType& InPinType)
{
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	if(CPPTypeFromPinType(InPinType, CPPType, &CPPTypeObject))
	{
		const FRigVMTemplateArgumentType Type(*CPPType, CPPTypeObject);
		return FRigVMRegistry::Get().GetTypeIndex(Type);
	}
	return INDEX_NONE;
}

bool RigVMTypeUtils::AreCompatible(const FName& InCPPTypeA, UObject* InCPPTypeObjectA, const FName& InCPPTypeB,	UObject* InCPPTypeObjectB)
{
	return AreCompatible(PinTypeFromCPPType(InCPPTypeA, InCPPTypeObjectA), PinTypeFromCPPType(InCPPTypeB, InCPPTypeObjectB));
}

bool RigVMTypeUtils::AreCompatible(const FRigVMExternalVariable& InTypeA, const FRigVMExternalVariable& InTypeB, const FString& InSegmentPathA, const FString& InSegmentPathB)
{
	return AreCompatible(PinTypeFromExternalVariable(InTypeA), PinTypeFromExternalVariable(InTypeB), InSegmentPathA, InSegmentPathB);
}

bool RigVMTypeUtils::AreCompatible(const FEdGraphPinType& InTypeA, const FEdGraphPinType& InTypeB, const FString& InSegmentPathA, const FString& InSegmentPathB)
{
	FEdGraphPinType SubPinTypeA = SubPinType(InTypeA, InSegmentPathA);
	FEdGraphPinType SubPinTypeB = SubPinType(InTypeB, InSegmentPathB);

	// We allow connections between floats and doubles, while EdGraphSchema_K2 does not
	// Every other case is evaluated by UEdGraphSchema_K2::ArePinTypesCompatible
	if (SubPinTypeA.ContainerType == SubPinTypeB.ContainerType)
	{
		if ((SubPinTypeA.PinCategory == FloatTypeName && SubPinTypeB.PinCategory == DoubleTypeName) ||
			(SubPinTypeA.PinCategory == DoubleTypeName && SubPinTypeB.PinCategory == FloatTypeName))
		{
			return true;
		}
	}

	if(UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>()->ArePinTypesCompatible(SubPinTypeA, SubPinTypeB))
	{
		return true;
	}

	// also check if there's a cast available for the type
	const TRigVMTypeIndex TypeIndexA = TypeIndexFromPinType(SubPinTypeA);
	const TRigVMTypeIndex TypeIndexB = TypeIndexFromPinType(SubPinTypeB);
	if (TypeIndexA == TypeIndexB)
	{
		return true;
	}
	
	const TArray<TRigVMTypeIndex>& AvailableCasts = GetAvailableCasts(TypeIndexA, true);
	return AvailableCasts.Contains(TypeIndexB);
 }

namespace RigVMTypeUtils
{
	static const FLazyName CastTemplateValueName = TEXT("Value");
	static const FLazyName CastTemplateResultName = TEXT("Result");
	static const FLazyName CastTemplateNotation = TEXT("Cast::Execute(in Value,out Result)");
}

const TArray<TRigVMTypeIndex>& RigVMTypeUtils::GetAvailableCasts(const TRigVMTypeIndex& InTypeIndex, bool bAsInput)
{
	static TMap<int32, TArray<TRigVMTypeIndex>> AvailableInputCastMap;
	static TMap<int32, TArray<TRigVMTypeIndex>> AvailableOutputCastMap;

	TMap<int32, TArray<TRigVMTypeIndex>>& AvailableCastMap = bAsInput ? AvailableInputCastMap : AvailableOutputCastMap;

	if(const TArray<TRigVMTypeIndex>* AvailableCasts = AvailableCastMap.Find(InTypeIndex))
	{
		return *AvailableCasts;
	}

	// find a specific template called "Cast"
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	if(const FRigVMTemplate* CastTemplate = Registry.FindTemplate(CastTemplateNotation))
	{
		const FRigVMTemplateArgument* ArgumentA = CastTemplate->FindArgument(bAsInput ? CastTemplateValueName : CastTemplateResultName);
		const FRigVMTemplateArgument* ArgumentB = CastTemplate->FindArgument(bAsInput ? CastTemplateResultName : CastTemplateValueName);

		if(ArgumentA && ArgumentB)
		{
			TArray<TRigVMTypeIndex>& AvailableCasts = AvailableCastMap.Add(InTypeIndex);
			
			int32 Index = 0;
			ArgumentA->ForEachType([&](const TRigVMTypeIndex TypeIndexA)
			{
				if(TypeIndexA == InTypeIndex)
				{
					AvailableCasts.Add(ArgumentB->GetTypeIndex(Index));
				}
				Index++;
				return true;
			}, Registry.GetHandle_NoLock());

			return AvailableCasts;
		}
	}

	static const TArray<TRigVMTypeIndex> EmptyCasts;
	return EmptyCasts;
}

bool RigVMTypeUtils::CanCastTypes(const TRigVMTypeIndex& InSourceTypeIndex, const TRigVMTypeIndex& InTargetTypeIndex)
{
	return GetCastForTypeIndices(InSourceTypeIndex, InTargetTypeIndex) != nullptr;
}

const FRigVMFunction* RigVMTypeUtils::GetCastForTypeIndices(const TRigVMTypeIndex& InSourceTypeIndex, const TRigVMTypeIndex& InTargetTypeIndex)
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();

	const FRigVMTemplate* CastTemplates[] =
	{
		Registry.FindTemplate(CastTemplateNotation),
		Registry.FindTemplate(FRigVMDispatch_CastObject().GetTemplateNotation()),
		Registry.FindTemplate(FRigVMDispatch_CastIntToEnum().GetTemplateNotation()),
		Registry.FindTemplate(FRigVMDispatch_CastEnumToInt().GetTemplateNotation())
	};

	for(const FRigVMTemplate* CastTemplate : CastTemplates)
	{
		if(CastTemplate)
		{
			const FRigVMTemplateArgument* SourceArgument = CastTemplate->FindArgument(CastTemplateValueName);
			const FRigVMTemplateArgument* TargetArgument = CastTemplate->FindArgument(CastTemplateResultName);

			if(SourceArgument && TargetArgument)
			{
				FRigVMTemplateTypeMap Types;
				Types.Add(SourceArgument->GetName(), InSourceTypeIndex);
				Types.Add(TargetArgument->GetName(), InTargetTypeIndex);

				int32 PermutationIndex = INDEX_NONE;
				if(CastTemplate->FullyResolve(Types, PermutationIndex))
				{
					return const_cast<FRigVMTemplate*>(CastTemplate)->GetOrCreatePermutation(PermutationIndex);
				}
			}
		}
	}
	return nullptr;
}

const FLazyName& RigVMTypeUtils::GetCastTemplateValueName()
{
	return CastTemplateValueName;
}

const FLazyName& RigVMTypeUtils::GetCastTemplateResultName()
{
	return CastTemplateResultName;
}

const FLazyName& RigVMTypeUtils::GetCastTemplateNotation()
{
	return CastTemplateNotation;
}

FText RigVMTypeUtils::GetDisplayTextForArgumentType(const FRigVMTemplateArgumentType& InType)
{
	if(UObject* CPPTypeObject = InType.CPPTypeObject)
	{
		if (const UClass* Class = Cast<UClass>(CPPTypeObject))
		{
			return Class->GetDisplayNameText();
		}
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			return ScriptStruct->GetDisplayNameText();
		}
		if(const UEnum* Enum = Cast<UEnum>(CPPTypeObject))
		{
			return Enum->GetDisplayNameText();
		}
	}
	else
	{
		FString CPPType = InType.CPPType.ToString();
		if(IsArrayType(CPPType))
		{
			CPPType = BaseTypeFromArrayType(CPPType);
		}

		static const FText BoolLabel = LOCTEXT("BoolLabel", "Boolean");
		static const FText FloatLabel = LOCTEXT("FloatLabel", "Float");
		static const FText Int32Label = LOCTEXT("Int32Label", "Integer");
		static const FText FNameLabel = LOCTEXT("FNameLabel", "Name");
		static const FText FStringLabel = LOCTEXT("FStringLabel", "String");

		if(CPPType == BoolType)
		{
			return BoolLabel;
		}
		if(CPPType == FloatType || CPPType == DoubleType)
		{
			return FloatLabel;
		}
		if(CPPType == Int32Type)
		{
			return Int32Label;
		}
		if(CPPType == FNameType)
		{
			return FNameLabel;
		}
		if(CPPType == FStringType)
		{
			return FStringLabel;
		}

		return FText::FromString(CPPType);
	}
	return FText();
}

#endif

#undef LOCTEXT_NAMESPACE