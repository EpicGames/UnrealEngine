// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorAsset.h"

#include "RigVMBlueprintLegacy.h"

#include "RigVMBlueprintGeneratedClass.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphNode_Comment.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"
#include "RigVMObjectVersion.h"
#include "BlueprintCompilationManager.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "RigVMPythonUtils.h"
#include "RigVMTypeUtils.h"
#include "Algo/Count.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "RigVMModel/RigVMExternalDependency.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Stats/StatsHierarchical.h"
#include "String/ParseTokens.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectHash.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/StringOutputDevice.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEditorAsset)

#if WITH_EDITOR
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMBlueprintUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "Editor/Transactor.h"
#include "CookOnTheSide/CookOnTheFlyServer.h"
#include "RigVMEditorModule.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h"
#if !WITH_RIGVMLEGACYEDITOR
#include "RigVMEditor/Private/Editor/Kismet/RigVMBlueprintCompilationManager.h"
#endif
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Styling/AppStyle.h"
#endif//WITH_EDITOR

#define LOCTEXT_NAMESPACE "RigVMAsset"

TAutoConsoleVariable<bool> CVarRigVMEnablePreLoadFiltering(
	TEXT("RigVM.EnablePreLoadFiltering"),
	true,
	TEXT("When true the RigVMGraphs will be skipped during preload to speed up load times."));

TAutoConsoleVariable<bool> CVarRigVMEnablePostLoadHashing(
	TEXT("RigVM.EnablePostLoadHashing"),
	true,
	TEXT("When true refreshing the RigVMGraphs will be skipped if the hash matches the serialized hash."));

static TArray<UClass*> GetClassObjectsInPackage(UPackage* InPackage)
{
	TArray<UObject*> Objects;
	GetObjectsWithOuter(InPackage, Objects, EGetObjectsFlags::None);

	TArray<UClass*> ClassObjects;
	for (UObject* Object : Objects)
	{
		if (UClass* Class = Cast<UClass>(Object))
		{
			ClassObjects.Add(Class);
		}
	}

	return ClassObjects;
}

FRigVMEdGraphDisplaySettings::FRigVMEdGraphDisplaySettings(): bShowNodeInstructionIndex(false)
                                                            , bShowNodeRunCounts(false)
                                                            , NodeRunLowerBound(0)
                                                            , NodeRunLimit(256)
                                                            , MinMicroSeconds(0.0)
                                                            , MaxMicroSeconds(1.0)
                                                            , TotalMicroSeconds(0.0)
                                                            , AverageFrames(64)
                                                            , bAutoDetermineRange(true)
                                                            , LastMinMicroSeconds(0.0)
                                                            , LastMaxMicroSeconds(1.0)
                                                            , MinDurationColor(FLinearColor::Green)
                                                            , MaxDurationColor(FLinearColor::Red)
                                                            , TagDisplayMode(ERigVMTagDisplayMode::All)
{
}

FRigVMEdGraphDisplaySettings::~FRigVMEdGraphDisplaySettings() = default;

void FRigVMEdGraphDisplaySettings::SetTotalMicroSeconds(double InTotalMicroSeconds)
{
	TotalMicroSeconds = AggregateAverage(TotalMicroSecondsFrames, TotalMicroSeconds, InTotalMicroSeconds);
}

void FRigVMEdGraphDisplaySettings::SetLastMinMicroSeconds(double InMinMicroSeconds)
{
	LastMinMicroSeconds = AggregateAverage(MinMicroSecondsFrames, LastMinMicroSeconds, InMinMicroSeconds);
}

void FRigVMEdGraphDisplaySettings::SetLastMaxMicroSeconds(double InMaxMicroSeconds)
{
	LastMaxMicroSeconds = AggregateAverage(MaxMicroSecondsFrames, LastMaxMicroSeconds, InMaxMicroSeconds);
}

double FRigVMEdGraphDisplaySettings::AggregateAverage(TArray<double>& InFrames, double InPrevious, double InNext) const
{
	const int32 NbFrames = FMath::Min(AverageFrames, 256);
	if(NbFrames < 2)
	{
		InFrames.Reset();
		return InNext;
	}
	
	InFrames.Add(InNext);
	if(InFrames.Num() >= NbFrames)
	{
		double Average = 0;
		for(const double Value : InFrames)
		{
			Average += Value;
		}
		Average /= double(NbFrames);
		InFrames.Reset();
		return Average;
	}

	if(InPrevious == DBL_MAX || InPrevious < -SMALL_NUMBER)
	{
		return InNext;
	}
	return InPrevious;
}

FRigVMEditorAssetInterfacePtr IRigVMEditorAssetInterface::GetInterfaceOuter(const UObject* InObject)
{
	if (!InObject)
	{
		return nullptr;
	}
	 
	UObject* Outer = InObject->GetOuter();
	while (Outer)
	{
		if (Outer->Implements<URigVMEditorAssetInterface>())
		{
			return FRigVMEditorAssetInterfacePtr(Outer);
		}
		Outer = Outer->GetOuter();
	}
	return nullptr;
}

TScriptInterface<IRigVMRuntimeAssetInterface> IRigVMEditorAssetInterface::GetRuntimeAssetInterface() const
{
	if (const UObject* Object = GetObject())
	{
		if (UObject* Outer = Object->GetOuter())
		{
			if (Outer->Implements<URigVMRuntimeAssetInterface>())
			{
				return Outer;
			}
		}
		
		return Object->GetClass();
	}
	return nullptr;
}

FSoftObjectPath IRigVMEditorAssetInterface::PreDuplicateHostPath;
FSoftObjectPath IRigVMEditorAssetInterface::PreDuplicateFunctionLibraryPath;
TArray<FRigVMEditorAssetInterfacePtr> IRigVMEditorAssetInterface::sCurrentlyOpenedRigVMBlueprints;
#if WITH_EDITOR
FCriticalSection IRigVMEditorAssetInterface::QueuedCompilerMessageDelegatesMutex;
TArray<FOnRigVMReportCompilerMessage::FDelegate> IRigVMEditorAssetInterface::QueuedCompilerMessageDelegates;
#endif

IRigVMEditorAssetInterface::IRigVMEditorAssetInterface(const FObjectInitializer& ObjectInitializer)
{
	bSuspendModelNotificationsForSelf = false;
	bSuspendAllNotifications = false;
	bSuspendPythonMessagesForRigVMClient = true;
	bMarkBlueprintAsStructurallyModifiedPending = false;

	bAutoRecompileVM = true;
	bVMRecompilationRequired = false;
	bIsCompiling = false;
	VMRecompilationBracket = 0;
	bSkipDirtyBlueprintStatus = false;

	bDirtyDuringLoad = false;
	bErrorsDuringCompilation = false;

#if WITH_EDITOR
	TArray<FOnRigVMReportCompilerMessage::FDelegate> DelegatesForReportFromCompiler;
	{
		FScopeLock Lock(&QueuedCompilerMessageDelegatesMutex);
		Swap(QueuedCompilerMessageDelegates, DelegatesForReportFromCompiler);
	}

	for(const FOnRigVMReportCompilerMessage::FDelegate& Delegate : DelegatesForReportFromCompiler)
	{
		ReportCompilerMessageEvent.Add(Delegate);
	}
#endif

}

bool IRigVMEditorAssetInterface::IsBlueprintAsset() const
{
	return GetObject()->IsA<UBlueprint>();
}

void IRigVMEditorAssetInterface::BeginDestroy()
{
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
	FRigVMRegistry::Get().OnRigVMRegistryChanged().RemoveAll(this);
}

TScriptInterface<IRigVMClientHost> IRigVMEditorAssetInterface::GetRigVMClientHost()
{
	return GetObject();
}

void IRigVMEditorAssetInterface::RequestClientHostRigVMInit()
{
	if (TScriptInterface<IRigVMClientHost> ClientHost = GetRigVMClientHost())
	{
		return ClientHost->RequestRigVMInit();
	}
}

FRigVMGraphVariableDescription IRigVMEditorAssetInterface::FindAssetVariableByGuid(const FGuid& InGuid) const
{
	const TArray<FRigVMGraphVariableDescription> Variables = GetAssetVariables();
	for (const FRigVMGraphVariableDescription& Variable : Variables)
	{
		if (Variable.Guid == InGuid)
		{
			return Variable;
		}
	}
	return FRigVMGraphVariableDescription();
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMClient* InClient) const
{
	check(InClient);

	const TArray<URigVMGraph*> Graphs = InClient->GetAllModels(true, true);
	for(const URigVMGraph* Graph : Graphs)
	{
		CollectExternalDependencies(OutDependencies, InCategory, Graph);
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionStore* InFunctionStore) const
{
	check(InFunctionStore);
	for(const FRigVMGraphFunctionData& Function : InFunctionStore->PublicFunctions)
	{
		CollectExternalDependencies(OutDependencies, InCategory, &Function);
	}
	for(const FRigVMGraphFunctionData& Function : InFunctionStore->PrivateFunctions)
	{
		CollectExternalDependencies(OutDependencies, InCategory, &Function);
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionData* InFunction) const
{
	check(InFunction);
	CollectExternalDependencies(OutDependencies, InCategory, &InFunction->Header);
	CollectExternalDependencies(OutDependencies, InCategory, &InFunction->CompilationData);
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionHeader* InHeader) const
{
	check(InHeader);
	if(InCategory == IRigVMExternalDependencyManager::RigVMGraphFunctionCategory)
	{
		OutDependencies.AddUnique({InHeader->LibraryPointer.GetLibraryNodePath(), InCategory});
	}
	for(const FRigVMGraphFunctionArgument& Argument : InHeader->Arguments)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Argument.CPPTypeObject.Get());
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMFunctionCompilationData* InFunction) const
{
	check(InFunction);

	for(const FRigVMFunctionCompilationPropertyDescription& Property : InFunction->LiteralPropertyDescriptions)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Property.CPPTypeObject.Get());
	}
	for(const FRigVMFunctionCompilationPropertyDescription& Property : InFunction->WorkPropertyDescriptions)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Property.CPPTypeObject.Get());
	}

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	for(const FName& FunctionName : InFunction->FunctionNames)
	{
		if(const FRigVMFunction* Function = Registry.FindFunction(*FunctionName.ToString()))
		{
			for(const FRigVMFunctionArgument& Argument :  Function->Arguments)
			{
				const FRigVMTemplateArgumentType& ArgumentType = Registry.FindTypeFromCPPType(Argument.Type);
				if(ArgumentType.IsValid())
				{
					CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, ArgumentType.CPPTypeObject.Get());
				}
			}
		}
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMGraph* InGraph) const
{
	for(const URigVMNode* Node : InGraph->GetNodes())
	{
		CollectExternalDependencies(OutDependencies, InCategory, Node);
	}

	const TArray<FRigVMGraphVariableDescription> LocalVariables = InGraph->GetLocalVariables();
	for(const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, LocalVariable.CPPTypeObject.Get());
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMNode* InNode) const
{
	for(const URigVMPin* Pin : InNode->GetPins())
	{
		CollectExternalDependencies(OutDependencies, InCategory, Pin);
	}
	if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		if(const FRigVMGraphFunctionData* Function = FunctionReferenceNode->GetReferencedFunctionData(true))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Function);
		}
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMPin* InPin) const
{
	CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, InPin->GetCPPTypeObject());
	for(const URigVMPin* SubPin : InPin->GetSubPins())
	{
		CollectExternalDependencies(OutDependencies, InCategory, SubPin);
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UStruct* InStruct) const
{
	check(InStruct);
	if(InCategory == IRigVMExternalDependencyManager::UserDefinedStructCategory)
	{
		if(const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStruct))
		{
			OutDependencies.AddUnique({UserDefinedStruct->GetPathName(), InCategory});
		}
	}
	for (TFieldIterator<FProperty> PropertyIt(InStruct); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		while(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			Property = ArrayProperty->Inner;
		}
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			CollectExternalDependencies(OutDependencies, InCategory, StructProperty->Struct);
		}
		else if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			if(const UEnum* Enum = EnumProperty->GetEnum())
			{
				CollectExternalDependencies(OutDependencies, InCategory, Enum);
			}
		}
		else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if(const UEnum* Enum = ByteProperty->Enum)
			{
				CollectExternalDependencies(OutDependencies, InCategory, Enum);
			}
		}
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UEnum* InEnum) const
{
	check(InEnum);
	if(InCategory == IRigVMExternalDependencyManager::UserDefinedEnumCategory)
	{
		if(const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(InEnum))
		{
			OutDependencies.AddUnique({UserDefinedEnum->GetPathName(), InCategory});
		}
	}
}

void IRigVMEditorAssetInterface::CollectExternalDependenciesForCPPTypeObject(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UObject* InObject) const
{
	if(InObject)
	{
		if(const UEnum* Enum = Cast<UEnum>(InObject))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Enum);
		}
		else if(const UStruct* Struct = Cast<UStruct>(InObject))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Struct);
		}
	}
}

void IRigVMEditorAssetInterface::CommonInitialization(const FObjectInitializer& ObjectInitializer)
{
	// guard against this running multiple times
	TScriptInterface<IRigVMClientHost> Host(ObjectInitializer.GetObj());
	FRigVMClient* RigVMClient = Host->GetRigVMClient();
	check(RigVMClient->GetDefaultSchemaClass() == nullptr);

	RigVMClient->SetDefaultSchemaClass(Host->GetRigVMSchemaClass());
	RigVMClient->SetDefaultExecuteContextStruct(Host->GetRigVMExecuteContextStruct());

	for(UEdGraph* UberGraph : GetUberGraphs())
	{
		if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(UberGraph))
		{
			EdGraph->Schema = Host->GetRigVMEdGraphSchemaClass();
		}
	}

	RigVMClient->SetOuterClientHost(GetObject(), TEXT("RigVMClient"));
	{
		TGuardValue<bool> DisableClientNotifs(RigVMClient->bSuspendNotifications, true);
		RigVMClient->GetOrCreateFunctionLibrary(false, &ObjectInitializer, false);
		URigVMGraph* Graph = RigVMClient->AddModel(FRigVMClient::RigVMModelPrefix, false, &ObjectInitializer, false);
		RigVMClient->GetOrCreateController(Graph);
	}

	TObjectPtr<URigVMEdGraph>& FunctionLibraryEdGraph = GetFunctionLibraryEdGraph();
 	FunctionLibraryEdGraph = Cast<URigVMEdGraph>(ObjectInitializer.CreateDefaultSubobject(ObjectInitializer.GetObj(), TEXT("RigVMFunctionLibraryEdGraph"), Host->GetRigVMEdGraphClass(), Host->GetRigVMEdGraphClass(), true, true));
	FunctionLibraryEdGraph->Schema = Host->GetRigVMEdGraphSchemaClass();
	FunctionLibraryEdGraph->bAllowRenaming = 0;
	FunctionLibraryEdGraph->bEditable = 0;
	FunctionLibraryEdGraph->bAllowDeletion = 0;
	FunctionLibraryEdGraph->bIsFunctionDefinition = false;
	FunctionLibraryEdGraph->ModelNodePath = RigVMClient->GetFunctionLibrary()->GetNodePath();
	FunctionLibraryEdGraph->InitializeFromAsset(ObjectInitializer.GetObj());
}

void IRigVMEditorAssetInterface::InitializeModelIfRequired(bool bRecompileVM)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMClient* RigVMClient = GetRigVMClient();
	if (RigVMClient->GetController(0) == nullptr || GetUberGraphs().IsEmpty())
	{
		const TArray<URigVMGraph*> Models = RigVMClient->GetAllModels(true, false);
		for(const URigVMGraph* Model : Models)
		{
			RigVMClient->GetOrCreateController(Model);
		}

		TSet<URigVMGraph*> InitializedGraphs;
		bool bRecompileRequired = false;
		TArray<TObjectPtr<UEdGraph>> UbergraphPages = GetUberGraphs();
		for (int32 i = 0; i < UbergraphPages.Num(); ++i)
		{
			if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(UbergraphPages[i]))
			{
				if (bRecompileVM)
				{
					bRecompileRequired = true;
				}

				Graph->InitializeFromAsset(GetObject());
				InitializedGraphs.Add(Graph->GetModel());
			}
		}

		// In case there are missing EdGraphs
		for (URigVMGraph* Model : Models)
		{
			if (!InitializedGraphs.Contains(Model))
			{
				CreateEdGraph(Model);
			}
		}

		if(bRecompileRequired)
		{
			RecompileVM();
		}

		GetFunctionLibraryEdGraph()->InitializeFromAsset(GetObject());
	}

	// Rebuild the function-library EdGraphs from the model when FunctionGraphs is empty.
	// FunctionGraphs is transient (so it's wiped every load) but HandleModifiedEvent's
	// NodeAdded/NodeRemoved cases keep it consistent at runtime — meaning a non-empty
	// FunctionGraphs is already in sync and a rebuild would just be O(N²) wasted work.
	// Sits outside the gate above because the gate's signals (Controller(0), RootEdGraphs)
	// don't predict transient state.
	if (GetFunctionGraphs().IsEmpty())
	{
		if (URigVMFunctionLibrary* FunctionLibrary = RigVMClient->GetFunctionLibrary())
		{
			TGuardValue<bool> SelfGuard(bSuspendModelNotificationsForSelf, true);
			TGuardValue<bool> ClientIgnoreModificationsGuard(RigVMClient->bIgnoreModelNotifications, true);

			TArray<URigVMGraph*> FunctionLibraryGraphs;
			FunctionLibraryGraphs.Add(FunctionLibrary);
			FunctionLibraryGraphs.Append(FunctionLibrary->GetContainedGraphs(true /* recursive */));

			for (URigVMGraph* Graph : FunctionLibraryGraphs)
			{
				for (URigVMNode* Node : Graph->GetNodes())
				{
					if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
					{
						CreateEdGraphForCollapseNodeIfNeeded(CollapseNode, /*bForce=*/false);
					}
				}
			}
		}
	}
}

bool IRigVMEditorAssetInterface::ExportGraphToText(UEdGraph* InEdGraph, FString& OutText)
{
	OutText.Empty();

	if (URigVMGraph* RigGraph = GetModel(InEdGraph))
	{
		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(RigGraph->GetOuter()))
		{
			if (URigVMController* Controller = GetOrCreateController(CollapseNode->GetGraph()))
			{
				TArray<FName> NodeNamesToExport;
				NodeNamesToExport.Add(CollapseNode->GetFName());
				OutText = Controller->ExportNodesToText(NodeNamesToExport);
			}
		}
	}

	// always return true so that the default mechanism doesn't take over
	return true;
}

bool IRigVMEditorAssetInterface::CanImportGraphFromText(const FString& InClipboardText)
{
	return GetTemplateController(true)->CanImportNodesFromText(InClipboardText);
}

bool IRigVMEditorAssetInterface::RequiresForceLoadMembers(UObject* InObject) const
{
	// only filter if the console variable is enabled
	if(!CVarRigVMEnablePreLoadFiltering->GetBool())
	{
		return RequiresForceLoadMembersSuper(InObject);
	}

	// we can stop traversing when hitting a URigVMNode
	// except for collapse nodes - since they contain a graphs again
	// and variable  nodes - since they are needed during preload by the BP compiler
	if(InObject->IsA<URigVMNode>())
	{
		if(!InObject->IsA<URigVMCollapseNode>() &&
			!InObject->IsA<URigVMVariableNode>())
		{
			return false;
		}
	}
	return RequiresForceLoadMembersSuper(InObject);
}

void IRigVMEditorAssetInterface::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	PostEditChangeChainPropertyEvent.Broadcast(PropertyChangedEvent);
}

void IRigVMEditorAssetInterface::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	TArray<UClass*> ClassObjects = GetClassObjectsInPackage(GetObject()->GetPackage());

	for (UClass* ClassObject : ClassObjects)
	{
		if (URigVMMemoryStorageGeneratorClass* MemoryClass = Cast<URigVMMemoryStorageGeneratorClass>(ClassObject))
		{
			OutDeps.Add(MemoryClass);
		}
	}
}

UObject* IRigVMEditorAssetInterface::GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const
{
	if(InVMGraph)
	{
		if(InVMGraph->GetOutermost() != GetObject()->GetOutermost())
		{
			return nullptr;
		}

		if(InVMGraph->IsA<URigVMFunctionLibrary>())
		{
			return GetFunctionLibraryEdGraph();
		}

		TArray<UEdGraph*> EdGraphs;
		GetAllEdGraphs(EdGraphs);

		bool bIsFunctionDefinition = false;
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InVMGraph->GetOuter()))
		{
			bIsFunctionDefinition = LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>();
		}

		for (UEdGraph* EdGraph : EdGraphs)
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(EdGraph))
			{
				if (RigGraph->bIsFunctionDefinition != bIsFunctionDefinition)
				{
					continue;
				}

				if ((RigGraph->ModelNodePath == InVMGraph->GetNodePath()) ||
					(RigGraph->ModelNodePath.IsEmpty() && (GetRigVMClient()->GetDefaultModel() == InVMGraph)))
				{
					return RigGraph;
				}
			}
		}
	}
	
	return nullptr;
}

URigVMGraph* IRigVMEditorAssetInterface::GetRigVMGraphForEditorObject(UObject* InObject) const
{
	if(URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(InObject))
	{
		const FRigVMClient* RigVMClient = GetRigVMClient();
		if (RigVMEdGraph->bIsFunctionDefinition)
		{
			if (URigVMLibraryNode* LibraryNode = RigVMClient->GetFunctionLibrary()->FindFunction(*RigVMEdGraph->ModelNodePath))
			{
				return LibraryNode->GetContainedGraph();
			}
		}
		else
		{
			return RigVMClient->GetModel(RigVMEdGraph->ModelNodePath);
		}
	}

	return nullptr;
}

void IRigVMEditorAssetInterface::HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath)
{
	if(URigVMGraph* Model = InClient->GetModel(InNodePath))
	{
		if(!GetObject()->HasAnyFlags(RF_ClassDefaultObject | RF_NeedInitialization | RF_NeedLoad | RF_NeedPostLoad) &&
			GetObject()->GetOuter() != GetTransientPackage() &&
			!GIsTransacting)
		{
			CreateEdGraph(Model, true);
			RecompileVM();
		}

#if WITH_EDITOR
		if(!bSuspendPythonMessagesForRigVMClient)
		{
			const FString BlueprintName = InClient->GetDefaultSchema()->GetSanitizedName(GetObject()->GetName(), true, false);
			RigVMPythonUtils::Print(BlueprintName, 
				FString::Printf(TEXT("asset.add_model('%s')"),
					*Model->GetName()));
		}
#endif
	}
}

void IRigVMEditorAssetInterface::HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath)
{
	if(URigVMGraph* Model = InClient->GetModel(InNodePath))
	{
		RemoveEdGraph(Model);
		RecompileVM();

#if WITH_EDITOR
		if(!bSuspendPythonMessagesForRigVMClient)
		{
			const FString BlueprintName = InClient->GetDefaultSchema()->GetSanitizedName(GetObject()->GetName(), true, false);
			RigVMPythonUtils::Print(BlueprintName, 
				FString::Printf(TEXT("asset.remove_model('%s')"),
					*Model->GetName()));
		}
#endif
	}
}

void IRigVMEditorAssetInterface::HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath)
{
	if(InClient->GetModel(InNewNodePath))
	{
		TArray<UEdGraph*> EdGraphs;
		GetAllEdGraphs(EdGraphs);

		for (UEdGraph* EdGraph : EdGraphs)
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(EdGraph))
			{
				RigGraph->HandleRigVMGraphRenamed(InOldNodePath, InNewNodePath);
			}
		}

		MarkAssetAsModified();
	}
}

void IRigVMEditorAssetInterface::HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure)
{
	InControllerToConfigure->OnModified().AddRaw(this, &IRigVMEditorAssetInterface::HandleModifiedEvent);

	TWeakObjectPtr<UObject> WeakThis(GetObject());

	// this delegate is used by the controller to determine variable validity
	// during a bind process. the controller itself doesn't own the variables,
	// so we need a delegate to request them from the owning blueprint
	InControllerToConfigure->GetExternalVariablesDelegate.BindLambda([](URigVMGraph* InGraph) -> TArray<FRigVMExternalVariable> {

		if (InGraph)
		{
			if(IRigVMEditorAssetInterface* Asset = InGraph->GetImplementingOuter<IRigVMEditorAssetInterface>())
			{
				return Asset->GetExternalVariables(true /* rely on variables within blueprint */);
			}
		}
		return TArray<FRigVMExternalVariable>();
	});


	// this delegate is used by the controller to retrieve the current bytecode of the VM
	InControllerToConfigure->GetCurrentByteCodeDelegate.BindLambda([WeakThis]() -> const FRigVMByteCode* {

		if (UObject* Object = WeakThis.Get())
		{
			if (FRigVMEditorAssetInterfacePtr Asset = Object)
			{
				if (URigVM* VM = Asset->GetVM(false))
				{
					return &VM->GetByteCode();
				}
			}
		}
		return nullptr;
	});

#if WITH_EDITOR

	// this sets up three delegates:
	// a) get external variables (mapped to Controller->GetExternalVariables)
	// b) bind pin to variable (mapped to Controller->BindPinToVariable)
	// c) create external variable (mapped to the passed in tfunction)
	// the last one is defined within the blueprint since the controller
	// doesn't own the variables and can't create one itself.
	InControllerToConfigure->SetupDefaultUnitNodeDelegates(TDelegate<FName(FRigVMExternalVariable, FString)>::CreateLambda(
		[WeakThis](FRigVMExternalVariable InVariableToCreate, FString InDefaultValue) -> FName {
			if (UObject* Object = WeakThis.Get())
			{
				if (Object->Implements<URigVMEditorAssetInterface>())
				{
					FRigVMEditorAssetInterfacePtr Asset = Object;
					return Asset->AddHostMemberVariableFromExternal(InVariableToCreate, InDefaultValue);
				}
			}
			return NAME_None;
		}
	));

	TWeakObjectPtr<URigVMController> WeakController = InControllerToConfigure;
	InControllerToConfigure->RequestBulkEditDialogDelegate.BindLambda([WeakThis, WeakController](URigVMLibraryNode* InFunction, ERigVMControllerBulkEditType InEditType) -> FRigVMController_BulkEditResult 
	{
		if (UObject* Object = WeakThis.Get())
		{
			if (FRigVMEditorAssetInterfacePtr Asset = Object)
			{
				if (URigVMController* Controller = WeakController.Get())
				{
					if(Asset->OnRequestBulkEditDialog().IsBound())
					{
						return Asset->OnRequestBulkEditDialog().Execute(Asset, Controller, InFunction, InEditType);
					}
				}
			}
		}
		return FRigVMController_BulkEditResult();
	});

	InControllerToConfigure->RequestBreakLinksDialogDelegate.BindLambda([WeakThis, WeakController](TArray<URigVMLink*> InLinks) -> bool 
	{
		if (UObject* Object = WeakThis.Get())
		{
			if (Object->Implements<URigVMEditorAssetInterface>())
			{
				FRigVMEditorAssetInterfacePtr BaseAsset = Object;
				if (URigVMController* Controller = WeakController.Get())
				{
					if(BaseAsset->OnRequestBreakLinksDialog().IsBound())
					{
						return BaseAsset->OnRequestBreakLinksDialog().Execute(InLinks);
					}
				}
			}
		}
		return false;
	});

	InControllerToConfigure->RequestPinTypeSelectionDelegate.BindLambda([WeakThis](const TArray<TRigVMTypeIndex>& InTypes) -> TRigVMTypeIndex 
	{
		if (UObject* Object = WeakThis.Get())
		{
			if (Object->Implements<URigVMEditorAssetInterface>())
			{
				FRigVMEditorAssetInterfacePtr BaseAsset = Object;
				if(BaseAsset->OnRequestPinTypeSelectionDialog().IsBound())
				{
					return BaseAsset->OnRequestPinTypeSelectionDialog().Execute(InTypes);
				}
			}
		}
		return INDEX_NONE;
	});

	InControllerToConfigure->RequestNewExternalVariableDelegate.BindLambda([WeakThis](FRigVMGraphVariableDescription InVariable, bool bInIsPublic, bool bInIsReadOnly) -> FName
	{
		if (UObject* Object = WeakThis.Get())
		{
			if (Object->Implements<URigVMEditorAssetInterface>())
			{
				FRigVMEditorAssetInterfacePtr BaseAsset = Object;
				TArray<FRigVMExternalVariable> ExternalVariables = BaseAsset->GetExternalVariables(true);
				for (FRigVMExternalVariable& ExistingVariable : ExternalVariables)
				{
					if (ExistingVariable.GetName() == InVariable.Name)
					{
						return FName();
					}
				}

				FRigVMExternalVariable ExternalVariable = InVariable.ToExternalVariable();
				return BaseAsset->AddMemberVariable(InVariable.Name,
					ExternalVariable.GetCPPTypeObject() ? ExternalVariable.GetCPPTypeObject()->GetPathName() : ExternalVariable.GetExtendedCPPType().ToString(),
					bInIsPublic,
					bInIsReadOnly,
					InVariable.DefaultValue);
			}
		}
		
		return FName();
	});


	InControllerToConfigure->RequestJumpToHyperlinkDelegate.BindLambda([WeakThis](const UObject* InSubject)
	{
		if (UObject* Object = WeakThis.Get())
		{
			if (Object->Implements<URigVMEditorAssetInterface>())
			{
				FRigVMEditorAssetInterfacePtr BaseAsset = Object;
				if(BaseAsset->OnRequestJumpToHyperlink().IsBound())
				{
					BaseAsset->OnRequestJumpToHyperlink().Execute(InSubject);
				}
			}
		}
	});

#endif
}

bool IRigVMEditorAssetInterface::TryImportGraphFromText(const FString& InClipboardText, UEdGraph** OutGraphPtr)
{
	if (OutGraphPtr)
	{
		*OutGraphPtr = nullptr;
	}

	if (URigVMController* FunctionLibraryController = GetOrCreateController(GetLocalFunctionLibrary()))
	{
		TGuardValue<FRigVMController_RequestLocalizeFunctionDelegate> RequestLocalizeDelegateGuard(
            FunctionLibraryController->RequestLocalizeFunctionDelegate,
            FRigVMController_RequestLocalizeFunctionDelegate::CreateLambda([this](FRigVMGraphFunctionIdentifier& InFunctionToLocalize)
            {
            	BroadcastRequestLocalizeFunctionDialog(InFunctionToLocalize);
				const URigVMLibraryNode* LocalizedFunctionNode = GetLocalFunctionLibrary()->FindPreviouslyLocalizedFunction(InFunctionToLocalize);
				return LocalizedFunctionNode != nullptr;
            })
        );
		
		TArray<FName> ImportedNodeNames = FunctionLibraryController->ImportNodesFromText(InClipboardText, true, true);
		if (ImportedNodeNames.Num() == 0)
		{
			return false;
		}

		URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetLocalFunctionLibrary()->FindFunction(ImportedNodeNames[0]));
		if (ImportedNodeNames.Num() > 1 || CollapseNode == nullptr || CollapseNode->GetContainedGraph() == nullptr)
		{
			FunctionLibraryController->Undo();
			return false;
		}

		UEdGraph* EdGraph = GetEdGraph(CollapseNode->GetContainedGraph());
		if (OutGraphPtr)
		{
			*OutGraphPtr = EdGraph;
		}

		BroadcastGraphImported(EdGraph);
	}

	// always return true so that the default mechanism doesn't take over
	return true;
}

URigVMEditorSettings* IRigVMEditorAssetInterface::GetRigVMEditorSettings() const
{
	return GetMutableDefault<URigVMEditorSettings>(GetRigVMClientHost()->GetRigVMEditorSettingsClass());
}

#if WITH_EDITOR
const FLazyName& IRigVMEditorAssetInterface::GetPanelNodeFactoryName() const
{
	return RigVMPanelNodeFactoryName;
}

const FLazyName& IRigVMEditorAssetInterface::GetPanelPinFactoryName() const
{
	return RigVMPanelPinFactoryName;
}

bool IRigVMEditorAssetInterface::SetEarlyExitInstruction(URigVMNode* InNodeToExitEarlyAfter, int32 InInstruction, bool bRequestHyperLink)
{
	if (InNodeToExitEarlyAfter == nullptr)
	{
		return ResetEarlyExitInstruction(true);
	}

	const FRigVMByteCode* ByteCode = GetController()->GetCurrentByteCode();
	if (ByteCode == nullptr)
	{
		return false;
	}

	int32 InstructionIndex = InInstruction;
	
	const TArray<int32>& InstructionIndices = ByteCode->GetAllInstructionIndicesForSubject(InNodeToExitEarlyAfter);
	if (!InstructionIndices.IsEmpty())
	{
		// find the last consecutive instruction for a node in the first block
		int32 IndexInArray = 0;
		if (InInstruction != INDEX_NONE)
		{
			IndexInArray = InstructionIndices.Find(InInstruction);
			if (IndexInArray == INDEX_NONE)
			{
				IndexInArray = 0;
			}
		}
		while (IndexInArray < InstructionIndices.Num() - 1 && InstructionIndices[IndexInArray] == InstructionIndices[IndexInArray + 1] - 1)
		{
			IndexInArray++;
		}

		InstructionIndex = InstructionIndices[IndexInArray];
	}

	const bool bResetResult = ResetEarlyExitInstruction(InstructionIndex == INDEX_NONE);
	if (InstructionIndex == INDEX_NONE)
	{
		return bResetResult;
	}

	if (InstructionIndex < 0 || InstructionIndex >= ByteCode->GetNumInstructions())
	{
		return false;
	}

	GetDebugInfo().RunToInstruction(InstructionIndex);

	if (URigVMHost* DebuggedHost = GetDebuggedRigVMHost())
	{
		DebuggedHost->GetDebugInfo().SetStepCondition(GetDebugInfo().GetStepCondition());
		DebuggedHost->SetIsInDebugMode(true);
	}

	MarkNodeForPreviewHere(InNodeToExitEarlyAfter, InstructionIndex, bRequestHyperLink);

	return true;
}

bool IRigVMEditorAssetInterface::ResetEarlyExitInstruction(bool bResetCallstack)
{
	GetDebugInfo().Reset();

	if (URigVMHost* DebuggedHost = GetDebuggedRigVMHost())
	{
		DebuggedHost->GetDebugInfo().SetStepCondition(GetDebugInfo().GetStepCondition());
	}

	TArray<URigVMGraph*> AllGraphs = GetAllModels();
	for (URigVMGraph* Graph : AllGraphs)
	{
		for (URigVMNode* Node : Graph->GetNodes())
		{
			Node->SetHasEarlyExitMarker(false);
			Node->SetIsExcludedByEarlyExit(false);
		}
	}

	if (bResetCallstack)
	{
		LastPreviewHereNodes.Reset();
	}
	return true;
}

void IRigVMEditorAssetInterface::TogglePreviewHere(const URigVMGraph* InGraph)
{
	if (!InGraph)
	{
		return;
	}

	URigVMHost* DebuggedHost = Cast<URigVMHost>(GetObjectBeingDebugged());
	if (!DebuggedHost)
	{
		return;
	}

	const int32 CurrentEarlyExistInstruction = DebuggedHost->GetDebugInfo().GetStepCondition().OriginInstruction;
	if (CurrentEarlyExistInstruction != INDEX_NONE)
	{
		if (CurrentEarlyExistInstruction == GetPreviewNodeInstructionIndexFromSelection(InGraph))
		{
			ResetEarlyExitInstruction(true);
			return;
		}
	}

	const TArray<FName> SelectedNodeNames = InGraph->GetSelectNodes();
	if (SelectedNodeNames.Num() == 0)
	{
		ResetEarlyExitInstruction(true);
		return;
	}

	URigVMNode* Node = InGraph->FindNodeByName(SelectedNodeNames[0]);
	if (!Node)
	{
		return;
	}

	if (!Node->IsMutable())
	{
		return;
	}

	SetEarlyExitInstruction(Node);
}

void IRigVMEditorAssetInterface::PreviewHereStepForward()
{
	URigVMHost* DebuggedHost = Cast<URigVMHost>(GetObjectBeingDebugged());
	if (!DebuggedHost)
	{
		return;
	}
	if (!DebuggedHost->GetRigVMExtendedExecuteContext().StepInto())
	{
		return;
	}

	const int32 InstructionIndex = DebuggedHost->GetDebugInfo().GetStepCondition().OriginInstruction;
	if (InstructionIndex == INDEX_NONE)
	{
		return;
	}

	URigVMNode* Node = Cast<URigVMNode>(DebuggedHost->GetVM()->GetByteCode().GetSubjectForInstruction(InstructionIndex));
	if (!Node)
	{
		return;
	}
	MarkNodeForPreviewHere(Node, InstructionIndex, true);
}

bool IRigVMEditorAssetInterface::CanPreviewHereStepForward() const
{
	URigVMHost* DebuggedHost = Cast<URigVMHost>(GetObjectBeingDebugged());
	if (!DebuggedHost)
	{
		return false;
	}

	return DebuggedHost->GetDebugInfo().GetStepCondition().OriginInstruction != INDEX_NONE;

}

bool IRigVMEditorAssetInterface::MarkNodeForPreviewHere(URigVMNode* InNode, int32 InInstructionIndex, bool bRequestHyperLink)
{
	check(InNode);
	
	TArray<URigVMNode*> IncludedNodes;
	TSet<URigVMNode*> VisitedNodes;

	IncludedNodes.Add(InNode);

	for (int32 Index = 0; Index < IncludedNodes.Num(); Index++)
	{
		VisitedNodes.Add(IncludedNodes[Index]);
		
		const TArray<URigVMNode*> SourceNodes = IncludedNodes[Index]->GetLinkedSourceNodes();
		for (URigVMNode* SourceNode : SourceNodes)
		{
			if (VisitedNodes.Contains(SourceNode))
			{
				continue;
			}
			IncludedNodes.AddUnique(SourceNode);
		}
	}

	InNode->SetHasEarlyExitMarker(true);

	TArray<URigVMNode*> AllNodes = InNode->GetGraph()->GetNodes();
	for (URigVMNode* Node : AllNodes)
	{
		Node->SetHasEarlyExitMarker(Node == InNode);
		Node->SetIsExcludedByEarlyExit(!VisitedNodes.Contains(Node));
	}

	GetRigGraphDisplaySettings().bShowNodeRunCounts = true;

	LastPreviewHereNodes.Emplace(InNode, InInstructionIndex);

	if (bRequestHyperLink)
	{
		if (FRigVMClient* Client = GetRigVMClient())
		{
			if (URigVMController* Controller = Client->GetOrCreateController(InNode->GetGraph()))
			{
				Controller->RequestJumpToHyperLink(InNode);
			}
		}
	}

	return true;
}

int32 IRigVMEditorAssetInterface::GetPreviewNodeInstructionIndexFromSelection(const URigVMGraph* InGraph) const
{
	if (!InGraph)
	{
		return INDEX_NONE;
	}
	
	URigVMHost* DebuggedHost = Cast<URigVMHost>(GetObjectBeingDebugged());
	if (!DebuggedHost)
	{
		return INDEX_NONE;
	}

	const TArray<FName> SelectedNodeNames = InGraph->GetSelectNodes();
	if (SelectedNodeNames.Num() == 0)
	{
		return INDEX_NONE;
	}

	URigVMNode* Node = InGraph->FindNodeByName(SelectedNodeNames[0]);
	if (!Node || !Node->IsMutable())
	{
		return INDEX_NONE;
	}

	const URigVM* VM = DebuggedHost->GetVM();
	if (!VM)
	{
		return INDEX_NONE;
	}

	const TArray<int32>& InstructionIndices = VM->GetByteCode().GetAllInstructionIndicesForSubject(Node);
	if (InstructionIndices.IsEmpty())
	{
		return INDEX_NONE;
	}

	// find the last consecutive instruction for a node in the first block
	int32 IndexInArray = 0;
	while (IndexInArray < InstructionIndices.Num() - 1 && InstructionIndices[IndexInArray] == InstructionIndices[IndexInArray + 1] - 1)
	{
		IndexInArray++;
	}

	return InstructionIndices[IndexInArray];
}

IRigVMEditorModule* IRigVMEditorAssetInterface::GetEditorModule() const
{
	return &IRigVMEditorModule::Get();
}
#endif

void IRigVMEditorAssetInterface::SerializeImpl(FArchive& Ar)
{
	UE_RIGVM_ARCHIVETRACE_SCOPE(Ar, FString::Printf(TEXT("IRigVMEditorAssetInterface(%s)"), *GetObject()->GetName()));
	
	Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);
	
	if(IsValidChecked(GetObject()))
	{
		TScriptInterface<IRigVMClientHost> Host = GetRigVMClientHost();
		Host->GetRigVMClient()->SetOuterClientHost(GetObject(), TEXT("RigVMClient"));
	}
	
	SerializeSuper(Ar);
	UE_RIGVM_ARCHIVETRACE_ENTRY(Ar, TEXT("Super::Serialize"));

	if(Ar.IsObjectReferenceCollector())
	{
		Ar.UsingCustomVersion(FRigVMObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
		if (Ar.IsCooking() && IsReferencedObjectPathsStored())
		{
			for (FSoftObjectPath ObjectPath : GetReferencedObjectPaths())
			{
				ObjectPath.Serialize(Ar);
			}
		}
		else
#endif
		{
			TArray<TScriptInterface<IRigVMGraphFunctionHost>> ReferencedFunctionHosts = GetReferencedFunctionHosts(false);

			for(TScriptInterface<IRigVMGraphFunctionHost> ReferencedFunctionHost : ReferencedFunctionHosts)
			{
				if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = ReferencedFunctionHost.GetObject())
				{
					Ar << RuntimeAsset;
				}
			}
		}
	}

	if(Ar.IsLoading())
	{
		TArray<UEdGraph*> EdGraphs;
		GetAllEdGraphs(EdGraphs);
		for (UEdGraph* EdGraph : EdGraphs)
		{
			EdGraph->Schema = GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
			if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(EdGraph))
			{
				RigVMEdGraph->CachedModelGraph.Reset();
			}
		}

		if (Ar.CustomVer(FRigVMObjectVersion::GUID) < FRigVMObjectVersion::AddVariantToRigVMAssets)
		{
			GetAssetVariant().Guid = FRigVMVariant::GenerateGUID(GetObject()->GetPackage()->GetPathName());
		}
	}

	PostSerialize(Ar);

#if WITH_EDITOR
	if (GUnrealEd && GUnrealEd->IsAutosaving())
	{
		TWeakObjectPtr<UObject> WeakThis(GetObject());

		ExecuteOnGameThread(UE_SOURCE_LOCATION,
			[WeakThis]()
			{
				if (UObject* Object = WeakThis.Get())
				{
					if (FRigVMAssetInterfacePtr Asset = Object)
					{
						Asset->RecompileVM();
					}
				}
			}
		);
	}
#endif
}

void IRigVMEditorAssetInterface::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	GetRigVMClient()->PreSave(ObjectSaveContext);

	UpdateSupportedEventNames();
	if (TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = GetRigVMClientHost()->GetRigVMGraphFunctionHost())
	{
		FRigVMGraphFunctionStore* Store = FunctionHost->GetRigVMGraphFunctionStore();

		TArray<FRigVMGraphFunctionHeader> PublicFunctions;
		PublicFunctions.Reset();
		PublicFunctions.SetNum(Store->PublicFunctions.Num());
		for (int32 i=0; i<Store->PublicFunctions.Num(); ++i)
		{
			PublicFunctions[i] = Store->PublicFunctions[i].Header;
		}
		SetPublicGraphFunctions(PublicFunctions);

		URigVMFunctionLibrary* FunctionLibrary = GetLocalFunctionLibrary();
		FunctionLibrary->FunctionToVariant.Reset();
		for (int32 Pass=0; Pass<2; ++Pass)
		{
			const TArray<FRigVMGraphFunctionData>& Functions = (Pass == 0) ?
				Store->PrivateFunctions
				: Store->PublicFunctions;
			for (const FRigVMGraphFunctionData& Function : Functions)
			{
				FunctionLibrary->FunctionToVariant.Add(Function.Header.Name, Function.Header.Variant);
			}
		}
	}

#if WITH_EDITORONLY_DATA
	GetReferencedObjectPaths().Reset();

	TArray<TScriptInterface<IRigVMGraphFunctionHost>> ReferencedFunctionHosts = GetReferencedFunctionHosts(false);
	for(TScriptInterface<IRigVMGraphFunctionHost> ReferencedFunctionHost : ReferencedFunctionHosts)
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = ReferencedFunctionHost.GetObject())
		{
			GetReferencedObjectPaths().AddUnique(RuntimeAsset.GetObject());
		}
	}

	SetReferencedObjectPathsStored(true);
#endif

	GetFunctionReferenceNodeData() = GetReferenceNodeData();
	IAssetRegistry::GetChecked().AssetTagsFinalized(*GetObject());

	CachedAssetTags.Reset();

	// also store the user defined struct guid to path name on the blueprint itself
	// to aid the controller when recovering from user defined struct name changes or
	// guid changes.
	GetUserDefinedStructGuidToPathName(false).Reset();
	GetUserDefinedEnumToPathName(false).Reset();
	GetUserDefinedTypesInUse(false).Reset();
	TArray<URigVMGraph*> AllModels = GetAllModels();
	for(const URigVMGraph* Graph : AllModels)
	{
		for(const URigVMNode* Node : Graph->GetNodes())
		{
			const TArray<URigVMPin*> AllPins = Node->GetAllPinsRecursively();
			for(const URigVMPin* Pin : AllPins)
			{
				if(const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(Pin->GetCPPTypeObject()))
				{
					const FString GuidBasedName = RigVMTypeUtils::GetUniqueStructTypeName(UserDefinedStruct);
					GetUserDefinedStructGuidToPathName(false).FindOrAdd(GuidBasedName) = FSoftObjectPath(UserDefinedStruct);
				}
				else if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(Pin->GetCPPTypeObject()))
				{
					const FString EnumName = RigVMTypeUtils::CPPTypeFromEnum(UserDefinedEnum);
					GetUserDefinedEnumToPathName(false).FindOrAdd(EnumName, UserDefinedEnum);
				}
			}
		}
	}

#if WITH_EDITORONLY_DATA
	OldMemoryStorageGeneratorClasses.Reset();
#endif
}

void IRigVMEditorAssetInterface::PostLoad()
{
	const TArray<FString>& RequiredPlugins = GetRequiredPlugins(false);
	for (const FString& RequiredPlugin : RequiredPlugins)
	{
		if (IPluginManager::Get().FindEnabledPlugin(*RequiredPlugin) == nullptr)
		{
			const FString Message = FString::Printf(TEXT("%s requires the '%s' plugin to be enabled."), *GetObject()->GetPackage()->GetPathName(), *RequiredPlugin);
			UE_LOGF(LogRigVMDeveloper, Error, "%ls", *Message);

			FNotificationInfo Info(FText::FromString(Message));
			Info.bUseSuccessFailIcons = true;
			Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Error"));
			Info.bFireAndForget = true;
			Info.bUseThrobber = true;
			Info.FadeOutDuration = 1.f;
			Info.ExpireDuration = 10.f;
	
			TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
			if (NotificationPtr)
			{
				NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
			}
		}
	}
	
	FRigVMRegistry::Get().RefreshEngineTypesIfRequired();
	
	FRigVMClient* RigVMClient = GetRigVMClient();
	TScriptInterface<IRigVMClientHost> ClientHost = GetRigVMClientHost();
	TScriptInterface<IRigVMGraphFunctionHost> FunctionHost;
	if (ClientHost)
	{
		FunctionHost = ClientHost->GetRigVMGraphFunctionHost();
	}

	bVMRecompilationRequired = true;
	{
		TGuardValue<bool> IsCompilingGuard(bIsCompiling, true);
		
		TArray<TScriptInterface<IRigVMGraphFunctionHost>> ReferencedFunctionHosts = GetReferencedFunctionHosts(true);

		// PostLoad all referenced function hosts so that their function data are fully loaded 
		// and ready to be inlined into this BP during compilation
		for (TScriptInterface<IRigVMGraphFunctionHost> ReferencedFunctionHost : ReferencedFunctionHosts)
		{
			if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = ReferencedFunctionHost.GetObject())
			{
				if (RuntimeAsset.GetObject()->HasAllFlags(RF_NeedPostLoad))
				{
					RuntimeAsset.GetObject()->ConditionalPostLoad();
				}
			}
		}
		
		// temporarily disable default value validation during load time, serialized values should always be accepted
		TGuardValue<bool> DisablePinDefaultValueValidation(GetOrCreateController()->bValidatePinDefaults, false);

		// remove all non-controlrig-graphs
		TArray<TObjectPtr<UEdGraph>> NewUberGraphPages;
		for (UEdGraph* Graph : GetUberGraphs())
		{
			URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph);
			if (RigGraph && ClientHost && RigGraph->GetClass() == ClientHost->GetRigVMEdGraphClass())
			{
				NewUberGraphPages.Add(RigGraph);
			}
			else
			{
                // We are renaming an object to a new outer while we may still be loading. Since we
                // are destroying the object, pass REN_AllowPackageLinkerMismatch to avoid forcing
                // the load to complete since that is wasteful.
				Graph->MarkAsGarbage();
				Graph->Rename(nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch);
			}
		}
		SetUberGraphs(NewUberGraphPages);
		
		TArray<TGuardValue<bool>> EditableGraphGuards;
		{
			for (URigVMGraph* Graph : GetAllModels())
			{
				EditableGraphGuards.Emplace(Graph->bEditable, true);
			}
		}
		
		InitializeModelIfRequired(false /* recompile vm */);
		if (RigVMClient)
		{
			TGuardValue<bool> GuardNotifications(bSuspendModelNotificationsForSelf, true);
			
			const FRigVMClientPatchResult PatchResult = RigVMClient->PatchModelsOnLoad();
			if(PatchResult.RequiresToMarkPackageDirty())
			{
				(void)GetObject()->MarkPackageDirty();
				bDirtyDuringLoad = true;
			}
			
			RigVMClient->PatchFunctionReferencesOnLoad();
			GetFunctionReferenceNodeData() = GetReferenceNodeData();

			PatchVariableNodesOnLoad();
			PatchVariableNodesWithIncorrectType();
			RepairDuplicateVariableGuidsOnLoad();
			PathDomainSpecificContentOnLoad();
			PatchBoundVariables();
			PatchParameterNodesOnLoad();
			PatchLinksWithCast();
			
			TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader> OldHeaders;
			// Backwards compatibility. Store public access in the model
			TArray<FName> BackwardsCompatiblePublicFunctions;
			GetBackwardsCompatibilityPublicFunctions(BackwardsCompatiblePublicFunctions, OldHeaders);

			if (FunctionHost)
			{
				RigVMClient->PatchFunctionsOnLoad(FunctionHost.GetInterface(), BackwardsCompatiblePublicFunctions, OldHeaders);
			}

			const FRigVMClientPatchResult PinDefaultValuePatchResult = RigVMClient->PatchPinDefaultValues();
			if(PinDefaultValuePatchResult.RequiresToMarkPackageDirty())
			{
				(void)GetObject()->MarkPackageDirty();
				bDirtyDuringLoad = true;
			}
		}

#if WITH_EDITOR

		{
			TGuardValue<bool> GuardNotifications(bSuspendModelNotificationsForSelf, true);

			// refresh the graph such that the pin hierarchies matches their CPPTypeObject
			// this step is needed everytime we open a BP in the editor, b/c even after load
			// model data can change while the Control Rig BP is not opened
			// for example, if a user defined struct changed after BP load,
			// any pin that references the struct needs to be regenerated
			RefreshAllModels();
		}
		
		// at this point we may still have links which are detached. we may or may not be able to 
		// reattach them.
		if (RigVMClient)
		{
			RigVMClient->ProcessDetachedLinks();
		}

		if (FunctionHost)
		{
			if (FRigVMGraphFunctionStore* FunctionStore = FunctionHost->GetRigVMGraphFunctionStore())
			{
				FunctionStore->RemoveAllCompilationData();
			}
		}

		// perform backwards compat value upgrades
		TArray<URigVMGraph*> GraphsToValidate = GetAllModels();
		for (int32 GraphIndex = 0; GraphIndex < GraphsToValidate.Num(); GraphIndex++)
		{
			URigVMGraph* GraphToValidate = GraphsToValidate[GraphIndex];
			if(GraphToValidate == nullptr)
			{
				continue;
			}

			for(URigVMNode* Node : GraphToValidate->GetNodes())
			{
				URigVMController* Controller = GetOrCreateController(GraphToValidate);
				FRigVMControllerNotifGuard NotifGuard(Controller, true);
				Controller->RemoveUnusedOrphanedPins(Node, true);
			}
				
			for(URigVMNode* Node : GraphToValidate->GetNodes())
			{
				// avoid function reference related validation for temp assets, a temp asset may get generated during
				// certain content validation process. It is usually just a simple file-level copy of the source asset
				// so these references are usually not fixed-up properly. Thus, it is meaningless to validate them.
				// They should not be allowed to dirty the source asset either.
				if (!GetObject()->GetPackage()->GetName().StartsWith("/Temp/"))
				{
					if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
					{
						if(URigVMBuildData* BuildData = URigVMBuildData::Get())
						{
							BuildData->RegisterFunctionReference(FunctionReferenceNode->GetReferenceNodeData());
						}
					}
				}
			}
		}

		CompileLog.Messages.Reset();
		CompileLog.NumErrors = CompileLog.NumWarnings = 0;
#endif
	}

	// remove invalid class objects that were parented to the rigvmbp object
	RemoveDeprecatedVMMemoryClass();
	
#if WITH_EDITOR
	if(GIsEditor)
	{
		// delay compilation until the package has been loaded
		FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &IRigVMEditorAssetInterface::HandlePackageDone);
	}
#else
	RecompileVMIfRequired();
#endif
	if (ClientHost)
	{
		ClientHost->RequestRigVMInit();
	}

	
	if (!GetAssetVariant().Guid.IsValid())
	{
		GetAssetVariant().Guid = FRigVMVariant::GenerateGUID();
	}

	if (UPackage* Package = GetObject()->GetPackage())
	{
		Package->SetDirtyFlag(bDirtyDuringLoad);
	}

#if WITH_EDITOR
	// if we are running with -game we are in editor code,
	// but GIsEditor is turned off
	if(!GIsEditor)
	{
		HandlePackageDone();
	}
#endif

	// RigVMRegistry changes can be triggered when new user defined types(structs/enums) are added/removed
	// in which case we have to refresh the model
	FRigVMRegistry::Get().OnRigVMRegistryChanged().RemoveAll(this);
	FRigVMRegistry::Get().OnRigVMRegistryChanged().AddRaw(this, &IRigVMEditorAssetInterface::OnRigVMRegistryChanged);
}

#if WITH_EDITORONLY_DATA
void IRigVMEditorAssetInterface::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMController::StaticClass()));
}
#endif

#if WITH_EDITOR
void IRigVMEditorAssetInterface::HandlePackageDone(const FEndLoadPackageContext& Context)
{
	if (!Context.LoadedPackages.Contains(GetObject()->GetPackage()))
	{
		return;
	}
	HandlePackageDone();
}

void IRigVMEditorAssetInterface::HandlePackageDone()
{
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);

	if(URigVMBuildData* BuildData = URigVMBuildData::Get())
	{
		if(URigVMFunctionLibrary* FunctionLibrary = GetRigVMClient()->GetFunctionLibrary())
		{
			// for backwards compatibility load the function references from the
			// model's storage over to the centralized build data
			if(!FunctionLibrary->FunctionReferences_DEPRECATED.IsEmpty())
			{
				// let's also update the asset data of the dependents
				IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
				
				for(const TTuple< TObjectPtr<URigVMLibraryNode>, FRigVMFunctionReferenceArray >& Pair :
					FunctionLibrary->FunctionReferences_DEPRECATED)
				{
					TSoftObjectPtr<URigVMLibraryNode> FunctionKey(Pair.Key);
						
					for(int32 ReferenceIndex = 0; ReferenceIndex < Pair.Value.Num(); ReferenceIndex++)
					{
						// update the build data
						BuildData->RegisterFunctionReference(FunctionKey->GetFunctionIdentifier(), Pair.Value[ReferenceIndex]);

						// find all control rigs matching the reference node
						FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(
							Pair.Value[ReferenceIndex].ToSoftObjectPath().GetWithoutSubPath());

						// if the asset has never been loaded - make sure to load it once and mark as dirty
						if(AssetData.IsValid() && !AssetData.IsAssetLoaded())
						{
							if(FRigVMEditorAssetInterfacePtr Dependent = AssetData.GetAsset())
							{
								if(Dependent != this)
								{
									(void)Dependent->GetObject()->MarkPackageDirty();
								}
							}
						}
					}
				}
				
				FunctionLibrary->FunctionReferences_DEPRECATED.Reset();
				(void)GetObject()->MarkPackageDirty();
			}
		}

		// update the build data from the current function references
		const TArray<FRigVMReferenceNodeData> ReferenceNodeDatas = GetReferenceNodeData();
		for(const FRigVMReferenceNodeData& ReferenceNodeData : ReferenceNodeDatas)
		{
			BuildData->RegisterFunctionReference(ReferenceNodeData);
		}

		BuildData->ClearInvalidReferences();
	}
	
	{
		const FRigVMCompileSettingsDuringLoadGuard Guard(GetVMCompileSettings());
		RecompileVM();
	}
	GetRigVMClientHost()->RequestRigVMInit();
	BroadcastRigVMPackageDone();
}

void IRigVMEditorAssetInterface::BroadcastRigVMPackageDone()
{
	TArray<UObject*> ArchetypeInstances = GetArchetypeInstances(true, false);
	for (UObject* Instance : ArchetypeInstances)
	{
		URigVMHost* InstanceHost = Cast<URigVMHost>(Instance);
		if (!URigVMHost::IsGarbageOrDestroyed(InstanceHost))
		{
			InstanceHost->BroadCastEndLoadPackage();
		}
	}
}

void IRigVMEditorAssetInterface::RemoveDeprecatedVMMemoryClass() 
{
	TArray<UObject*> Objects;
	GetObjectsWithOuter(GetObject(), Objects, EGetObjectsFlags::None);

#if WITH_EDITORONLY_DATA
	OldMemoryStorageGeneratorClasses.Reserve(Objects.Num());
	for (UObject* Object : Objects)
	{
		if (URigVMMemoryStorageGeneratorClass* DeprecatedClass = Cast<URigVMMemoryStorageGeneratorClass>(Object))
		{
			// Making sure it is fully loaded before removing it to avoid ambiguity regarding load order
			DeprecatedClass->ConditionalPostLoad();
			
			DeprecatedClass->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			OldMemoryStorageGeneratorClasses.Add(DeprecatedClass);
		}
	}
#endif
}
#endif

void IRigVMEditorAssetInterface::GenerateUserDefinedDependenciesData(FRigVMExtendedExecuteContext& Context)
{
	if (URigVM* VM = GetVM(true))
	{
		const TArray<const UObject*> UserDefinedDependencies = VM->GetUserDefinedDependencies({ VM->GetDefaultMemoryByType(ERigVMMemoryType::Literal), VM->GetDefaultMemoryByType(ERigVMMemoryType::Work) });
		GetUserDefinedStructGuidToPathName(true).Reset();
		GetUserDefinedEnumToPathName(true).Reset();
		GetUserDefinedTypesInUse(true).Reset();

		for (const UObject* UserDefinedDependency : UserDefinedDependencies)
		{
			if (const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(UserDefinedDependency))
			{
				const FString GuidBasedName = RigVMTypeUtils::GetUniqueStructTypeName(UserDefinedStruct);
				GetUserDefinedStructGuidToPathName(true).Add(GuidBasedName, UserDefinedStruct);
			}
			else if (const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(UserDefinedDependency))
			{
				const FString EnumName = RigVMTypeUtils::CPPTypeFromEnum(UserDefinedEnum);
				GetUserDefinedEnumToPathName(true).Add(EnumName, UserDefinedEnum);
			}
		}
	}
}

void IRigVMEditorAssetInterface::RecompileVM()
{
	if(bIsCompiling)
	{
		return;
	}

	TGuardValue<bool> CompilingGuard(bIsCompiling, true);
	
	bErrorsDuringCompilation = false;

	FRigVMEdGraphDisplaySettings& RigGraphDisplaySettings = GetRigGraphDisplaySettings();
	if(RigGraphDisplaySettings.bAutoDetermineRange)
	{
		RigGraphDisplaySettings.MinMicroSeconds = RigGraphDisplaySettings.LastMinMicroSeconds = DBL_MAX;
		RigGraphDisplaySettings.MaxMicroSeconds = RigGraphDisplaySettings.LastMaxMicroSeconds = (double)INDEX_NONE;
	}
	else if(RigGraphDisplaySettings.MaxMicroSeconds < RigGraphDisplaySettings.MinMicroSeconds)
	{
		RigGraphDisplaySettings.MinMicroSeconds = 0;
		RigGraphDisplaySettings.MaxMicroSeconds = 5;
	}
	
	RigGraphDisplaySettings.TotalMicroSeconds = 0.0;
	RigGraphDisplaySettings.MinMicroSecondsFrames.Reset();
	RigGraphDisplaySettings.MaxMicroSecondsFrames.Reset();
	RigGraphDisplaySettings.TotalMicroSecondsFrames.Reset();

	WeakWatchedPins.Reset();

	URigVM* VM = GetVM(true);
	FRigVMExtendedExecuteContext* Context = GetRigVMExtendedExecuteContext();
	if (VM && Context)
	{
		FRigVMClient* RigVMClient = GetRigVMClient();
		TGuardValue<bool> ReentrantGuardSelf(bSuspendModelNotificationsForSelf, true);
		TGuardValue<bool> ReentrantGuardOthers(RigVMClient->bSuspendModelNotificationsForOthers, true);

		ResetEarlyExitInstruction(true);
		
		PreCompile();

		GetObject()->Modify(false);
		VM->Reset(*Context);

		// Clear all Errors
		CompileLog.Messages.Reset();
		CompileLog.NumErrors = CompileLog.NumWarnings = 0;
		
		TArray<UEdGraph*> EdGraphs;
		GetAllEdGraphs(EdGraphs);

		for (UEdGraph* Graph : EdGraphs)
		{
			URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph);
			if (RigGraph == nullptr)
			{
				continue;
			}

			for (UEdGraphNode* GraphNode : Graph->Nodes)
			{
				if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
				{
					RigVMEdGraphNode->ClearErrorInfo();
				}
			}
		}

		URigVMCompiler* Compiler = URigVMCompiler::StaticClass()->GetDefaultObject<URigVMCompiler>();
		GetVMCompileSettings().SetExecuteContextStruct(RigVMClient->GetDefaultExecuteContextStruct());

		const FRigVMCompileSettings Settings = GetVMCompileSettings();
		Compiler->Compile(Settings, RigVMClient->GetAllModels(false, false), GetOrCreateController(), VM, *Context, GetExternalVariables(false), &PinToOperandMap);

		VM->Initialize(*Context);
		GenerateUserDefinedDependenciesData(*Context);

		if (bErrorsDuringCompilation)
		{
			if(Settings.SurpressErrors)
			{
				Settings.Reportf(EMessageSeverity::Info, GetObject(),
					TEXT("Compilation Errors may be suppressed for ControlRigBlueprint: %s. See VM Compile Setting in Class Settings for more Details"), *GetObject()->GetName());
			}
			bVMRecompilationRequired = false;
			if(VM)
			{
				VMCompiledEvent.Broadcast(GetObject(), VM, *Context);
			}
			SetAssetStatus(ERigVMAssetStatus::RVMA_Error);
			return;
		}
		else
		{
			SetAssetStatus(ERigVMAssetStatus::RVMA_UpToDate);
		}

		// Update external variables on VM
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
		{
			URigVM* RuntimeVM = RuntimeAsset->GetVM();
			FRigVMExtendedExecuteContext* ExtendedExecuteContext = RuntimeAsset->GetRigVMExtendedExecuteContext();
			ensure(VM == RuntimeVM);

			RuntimeVM->ClearExternalVariables(*ExtendedExecuteContext);
			RuntimeVM->SetExternalVariableDefs(RuntimeAsset->GetExternalVariables());
		}

		// Settings.Reportf(EMessageSeverity::Info, GetObject(),
		// 			TEXT("Compilation successful: %s."), *GetObject()->GetName());
		
		InitializeArchetypeInstances();
		UpdateSupportedEventNames();

		bVMRecompilationRequired = false;
		VMCompiledEvent.Broadcast(GetObject(), VM, *Context);
	}
}

void IRigVMEditorAssetInterface::RecompileVMIfRequired()
{
	if (bVMRecompilationRequired)
	{
		RecompileVM();
	}
}

void IRigVMEditorAssetInterface::RequestAutoVMRecompilation()
{
	bVMRecompilationRequired = true;
	if (bAutoRecompileVM && VMRecompilationBracket == 0)
	{
		RecompileVMIfRequired();
	}
}

void IRigVMEditorAssetInterface::IncrementVMRecompileBracket()
{
	VMRecompilationBracket++;
}

void IRigVMEditorAssetInterface::DecrementVMRecompileBracket()
{
	if (VMRecompilationBracket == 1)
	{
		if (bAutoRecompileVM)
		{
			RecompileVMIfRequired();
		}
		VMRecompilationBracket = 0;
	}
	else if (VMRecompilationBracket > 0)
	{
		VMRecompilationBracket--;
	}
}

void IRigVMEditorAssetInterface::SetEditingLocked(bool InEditingLocked)
{
	if (!CanSetEditingLocked())
	{
		return;
	}
	
	if (IsEditingLocked() == InEditingLocked)
	{
		return;
	}

	URigVMHost* Host = GetEditorHost();
	if (!Host)
	{
		return;
	}

#if WITH_EDITOR
	Host->EnableNativizedVM(InEditingLocked);
#endif
	Host->Initialize(true);
}

bool IRigVMEditorAssetInterface::IsEditingLocked() const
{
	if (!CanSetEditingLocked())
	{
		return false;
	}
	
	URigVMHost* Host = const_cast<IRigVMEditorAssetInterface*>(this)->GetEditorHost();
	if (!Host)
	{
		return false;
	}

#if RIGVM_TRACE_ENABLED
	if (Host->IsPlayingRewindDebugTrace())
	{
		return true;
	}
#endif
	
	if (const URigVM* VM = Host->GetVM())
	{
		return VM->IsNativized();
	}
	
	return false;
}

bool IRigVMEditorAssetInterface::CanSetEditingLocked() const
{
	URigVMHost* Host = const_cast<IRigVMEditorAssetInterface*>(this)->GetEditorHost();
	if (!Host)
	{
		return false;
	}

#if RIGVM_TRACE_ENABLED
	if (Host->IsPlayingRewindDebugTrace())
	{
		return false;
	}
#endif
	
	return Host->CanSwapVMToNativized();
}

bool IRigVMEditorAssetInterface::IsEditingLockedVisible() const
{
	return IsEditingLocked() || CanSetEditingLocked();
}

FText IRigVMEditorAssetInterface::GetEditingLockedLabel() const
{
	URigVMHost* Host = const_cast<IRigVMEditorAssetInterface*>(this)->GetEditorHost();
	if (!Host)
	{
		return FText();
	}

#if RIGVM_TRACE_ENABLED
	if (Host->IsPlayingRewindDebugTrace())
	{
		return LOCTEXT("EditingLockedRewindDebuggerActive", "Rewind Debugger");
	}
#endif
	
	return LOCTEXT("EditingLockedNativized", "Nativized");
}

FText IRigVMEditorAssetInterface::GetEditingLockedTooltip() const
{
	URigVMHost* Host = const_cast<IRigVMEditorAssetInterface*>(this)->GetEditorHost();
	if (!Host)
	{
		return FText();
	}

#if RIGVM_TRACE_ENABLED
	if (Host->IsPlayingRewindDebugTrace())
	{
		return LOCTEXT("EditingLockedRewindDebuggerActiveTooltip", "This asset is being driven by Rewind Debugger.");
	}
#endif
	
	return LOCTEXT("EditingLockedNativizedTooltip", "This asset is using a nativized VM if on.");
}

void IRigVMEditorAssetInterface::RefreshAllModels(ERigVMLoadType InLoadType)
{
	const bool bEnablePostLoadHashing = CVarRigVMEnablePostLoadHashing->GetBool();

	GetRigVMClient()->RefreshAllModels(InLoadType, bEnablePostLoadHashing, bIsCompiling);
}

void IRigVMEditorAssetInterface::OnRigVMRegistryChanged()
{
	RefreshAllModels();
	RebuildGraphFromModel();
	// avoids slate crash
	RefreshAllNodes();
}

void IRigVMEditorAssetInterface::HandleReportFromCompiler(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
{
	UObject* SubjectForMessage = InSubject;
	if(URigVMNode* ModelNode = Cast<URigVMNode>(SubjectForMessage))
	{
		if(IRigVMEditorAssetInterface* RigBlueprint = ModelNode->GetImplementingOuter<IRigVMEditorAssetInterface>())
		{
			if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(RigBlueprint->GetEdGraph(ModelNode->GetGraph())))
			{
				if(UEdGraphNode* EdNode = EdGraph->FindNodeForModelNodeName(ModelNode->GetFName()))
				{
					SubjectForMessage = EdNode;
				}
			}
		}
	}

	FCompilerResultsLog* CurrentMessageLog = GetCurrentMessageLog();
	FCompilerResultsLog* Log = CurrentMessageLog ? CurrentMessageLog : &CompileLog;
	if (InSeverity == EMessageSeverity::Error)
	{
		SetAssetStatus(ERigVMAssetStatus::RVMA_Error);

		// see UnitTest "ControlRig.Basics.OrphanedPins" to learn why errors are suppressed this way
		if (GetVMCompileSettings().SurpressErrors)
		{
			Log->bSilentMode = true;
		}

		if(InMessage.Contains(TEXT("@@")))
		{
			Log->Error(*InMessage, SubjectForMessage);
		}
		else
		{
			Log->Error(*InMessage);
		}

		BroadCastReportCompilerMessage(InSeverity, InSubject, InMessage);

		// see UnitTest "ControlRig.Basics.OrphanedPins" to learn why errors are suppressed this way
		if (!GetVMCompileSettings().SurpressErrors)
		{ 
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *InMessage, *FString());
		}
		
		bErrorsDuringCompilation = true;
	}
	else if (InSeverity == EMessageSeverity::Warning)
	{
		if(InMessage.Contains(TEXT("@@")))
		{
			Log->Warning(*InMessage, SubjectForMessage);
		}
		else
		{
			Log->Warning(*InMessage);
		}

		BroadCastReportCompilerMessage(InSeverity, InSubject, InMessage);
		FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *InMessage, *FString());
	}
	else
	{
		if(InMessage.Contains(TEXT("@@")))
		{
			Log->Note(*InMessage, SubjectForMessage);
		}
		else
		{
			Log->Note(*InMessage);
		}

		static const FString Error = TEXT("Error");
		static const FString Warning = TEXT("Warning");
		if(InMessage.Contains(Error, ESearchCase::IgnoreCase) ||
			InMessage.Contains(Warning, ESearchCase::IgnoreCase))
		{
			BroadCastReportCompilerMessage(InSeverity, InSubject, InMessage);
		}
		UE_LOGF(LogRigVMDeveloper, Display, "%ls", *InMessage);
	}

	if (URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(SubjectForMessage))
	{
		EdGraphNode->SetErrorInfo(InSeverity, InMessage);
		EdGraphNode->bHasCompilerMessage = EdGraphNode->ErrorType <= int32(EMessageSeverity::Info);
	}
}

TArray<TScriptInterface<IRigVMGraphFunctionHost>> IRigVMEditorAssetInterface::GetReferencedFunctionHosts(bool bForceLoad) const
{
	TArray<TScriptInterface<IRigVMGraphFunctionHost>> ReferencedBlueprints;
	
	TArray<UEdGraph*> EdGraphs;
	GetAllEdGraphs(EdGraphs);
	for (UEdGraph* EdGraph : EdGraphs)
	{
		for(UEdGraphNode* Node : EdGraph->Nodes)
		{
			if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
			{
				if(URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(RigNode->GetModelNode()))
				{
					TScriptInterface<IRigVMGraphFunctionHost> Host = nullptr;
					if (bForceLoad || FunctionRefNode->IsReferencedFunctionHostLoaded())
					{
						// Load the function host
						Host = FunctionRefNode->GetReferencedFunctionHeader().GetFunctionHostObject();
					}
					else if (bForceLoad || FunctionRefNode->IsReferencedNodeLoaded())
					{
						// Load the reference library node
						if(const URigVMLibraryNode* ReferencedNode = FunctionRefNode->LoadReferencedNode())
						{
							if(URigVMFunctionLibrary* ReferencedFunctionLibrary = ReferencedNode->GetLibrary())
							{
								FSoftObjectPath FunctionHostPath = ReferencedFunctionLibrary->GetFunctionHostObjectPath();
								if (UObject* FunctionHostObj = FunctionHostPath.TryLoad())
								{
									Host = FunctionHostObj;
								}
							}
						}
					}

					if (Host != nullptr && Host != GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetInterface())
					{
						ReferencedBlueprints.Add(Host);
					}
				}
			}
		}
	}
	
	return ReferencedBlueprints;
}

#if WITH_EDITOR

TArray<FRigVMReferenceNodeData> IRigVMEditorAssetInterface::GetReferenceNodeData() const
{
	TArray<FRigVMReferenceNodeData> Data;
	
	const TArray<URigVMGraph*> AllModels = GetAllModels();
	for (URigVMGraph* ModelToVisit : AllModels)
	{
		for(URigVMNode* Node : ModelToVisit->GetNodes())
		{
			if(URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
			{
				Data.Add(ReferenceNode->GetReferenceNodeData());
			}
		}
	}
	return Data;
}

#endif

void IRigVMEditorAssetInterface::SetupDefaultObjectDuringCompilation(URigVMHost* InCDO)
{
	InCDO->PostInitInstanceIfRequired();
	InCDO->VMRuntimeSettings = GetVMRuntimeSettings();
}

void IRigVMEditorAssetInterface::MarkDirtyDuringLoad()
{
	bDirtyDuringLoad = true;
}

bool IRigVMEditorAssetInterface::IsMarkedDirtyDuringLoad() const
{
	return bDirtyDuringLoad;
}

void IRigVMEditorAssetInterface::UpdateDebugMemoryOnHost(URigVMHost* InHost)
{
	// potentially set up the debug information based on the watched pins
	TArray<URigVMGraph*> Graphs = GetAllModels();
	if (!WeakWatchedPins.IsSet())
	{
		WeakWatchedPins = TArray<TWeakObjectPtr<URigVMPin>>();
			
		for (int32 GraphIndex = 0; GraphIndex < Graphs.Num(); GraphIndex++)
		{
			for (URigVMNode* Node : Graphs[GraphIndex]->GetNodes())
			{
				for (URigVMPin* Pin : Node->GetPins())
				{
					if (Pin->RequiresWatch())
					{
						WeakWatchedPins.GetValue().Add(Pin);
					}

					if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
					{
						if (URigVMLibraryNode* LibraryNode = FunctionReferenceNode->LoadReferencedNode())
						{
							if (URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph())
							{
								Graphs.AddUnique(ContainedGraph);
							}
						}
					}
					else if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Node))
					{
						if (URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph())
						{
							Graphs.AddUnique(ContainedGraph);
						}
					}
				}
			}
		}
	}

	if (Graphs[0]->GetRuntimeAST().IsValid())
	{
		URigVMCompiler* Compiler = URigVMCompiler::StaticClass()->GetDefaultObject<URigVMCompiler>();
			
		for (const TWeakObjectPtr<URigVMPin>& WeakPin : WeakWatchedPins.GetValue())
		{
			if (!WeakPin.IsValid())
			{
				continue;
			}

			Compiler->MarkDebugWatch(InHost->GetRigVMExtendedExecuteContext(), true, WeakPin.Get(), InHost->GetVM(), &PinToOperandMap, Graphs[0]->GetRuntimeAST());
		}
	}
}

void IRigVMEditorAssetInterface::RequestRigVMInit() const
{
	TArray<UObject*> Instances = GetArchetypeInstances(true, true);
	for (UObject* Instance : Instances)
	{
		URigVMHost* RigVMHost = Cast<URigVMHost>(Instance);
		if (!URigVMHost::IsGarbageOrDestroyed(RigVMHost))
		{
			RigVMHost->RequestInit();
		}
	}
}

URigVMGraph* IRigVMEditorAssetInterface::GetModel(const UEdGraph* InEdGraph) const
{
	const FRigVMClient* RigVMClient = GetRigVMClient();
#if WITH_EDITORONLY_DATA
	if (InEdGraph != nullptr && InEdGraph == GetFunctionLibraryEdGraph())
	{
		return RigVMClient->GetFunctionLibrary();
	}
#endif

	return RigVMClient->GetModel(InEdGraph);
}

URigVMGraph* IRigVMEditorAssetInterface::GetModel(const FString& InNodePath) const
{
	return GetRigVMClient()->GetModel(InNodePath);
}

URigVMGraph* IRigVMEditorAssetInterface::GetDefaultModel() const
{
	return GetRigVMClient()->GetDefaultModel();
}

TArray<URigVMGraph*> IRigVMEditorAssetInterface::GetAllModels() const
{
	return GetRigVMClient()->GetAllModels(true, true);
}

URigVMFunctionLibrary* IRigVMEditorAssetInterface::GetLocalFunctionLibrary() const
{
	return GetRigVMClient()->GetFunctionLibrary();
}

URigVMFunctionLibrary* IRigVMEditorAssetInterface::GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo)
{
	return GetRigVMClient()->GetOrCreateFunctionLibrary(bSetupUndoRedo);
}

URigVMGraph* IRigVMEditorAssetInterface::AddModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
	return GetRigVMClient()->AddModel(InName, bSetupUndoRedo, bPrintPythonCommand);
}

bool IRigVMEditorAssetInterface::RemoveModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	TGuardValue<bool> EnablePythonPrint(bSuspendPythonMessagesForRigVMClient, !bPrintPythonCommand);
	return GetRigVMClient()->RemoveModel(InName, bSetupUndoRedo, bPrintPythonCommand);
}

FRigVMGetFocusedGraph& IRigVMEditorAssetInterface::OnGetFocusedGraph()
{
	return GetRigVMClient()->OnGetFocusedGraph();
}

const FRigVMGetFocusedGraph& IRigVMEditorAssetInterface::OnGetFocusedGraph() const
{
	return GetRigVMClient()->OnGetFocusedGraph();
}

URigVMHost* IRigVMEditorAssetInterface::GetDebuggedRigVMHost()
{
	return Cast<URigVMHost>(GetObjectBeingDebugged());
}

URigVMGraph* IRigVMEditorAssetInterface::GetFocusedModel() const
{
	return GetRigVMClient()->GetFocusedModel();
}

URigVMController* IRigVMEditorAssetInterface::GetController(const URigVMGraph* InGraph) const
{
	return GetRigVMClient()->GetController(InGraph);
}

URigVMController* IRigVMEditorAssetInterface::GetControllerByName(const FString InGraphName) const
{
	return GetRigVMClient()->GetControllerByName(InGraphName);
}

URigVMController* IRigVMEditorAssetInterface::GetOrCreateController(URigVMGraph* InGraph)
{
	return GetRigVMClient()->GetOrCreateController(InGraph);
}

URigVMController* IRigVMEditorAssetInterface::GetController(const UEdGraph* InEdGraph) const
{
	return GetRigVMClient()->GetController(InEdGraph);
}

URigVMController* IRigVMEditorAssetInterface::GetOrCreateController(const UEdGraph* InEdGraph)
{
	return GetRigVMClient()->GetOrCreateController(InEdGraph);
}

TArray<FString> IRigVMEditorAssetInterface::GeneratePythonCommands()
{
	TArray<FString> InternalCommands;
	
	// Add variables
	for (const FRigVMExternalVariable& ExternalVariable : GetExternalVariables(true))
	{
		FString CPPType;
		UObject* CPPTypeObject = nullptr;
		RigVMTypeUtils::CPPTypeFromExternalVariable(ExternalVariable, CPPType, &CPPTypeObject);
		if (CPPTypeObject)
		{
			if (ExternalVariable.IsArray())
			{
				CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(CPPTypeObject->GetPathName());
			}
			else
			{
				CPPType = CPPTypeObject->GetPathName();
			}
		}
		// FName AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT(""));
		InternalCommands.Add(FString::Printf(TEXT("asset.add_member_variable('%s', '%s', %s, %s)"),
					*ExternalVariable.GetName().ToString(),
					*CPPType,
					ExternalVariable.IsPublic() ? TEXT("True") : TEXT("False"),
					ExternalVariable.IsReadOnly() ? TEXT("True") : TEXT("False")));	
	}
	
	// Create graphs
	{
		TArray<URigVMGraph*> AllModels = GetAllModels();
		AllModels.RemoveAll([](const URigVMGraph* GraphToRemove) -> bool
		{
			return GraphToRemove->GetTypedOuter<URigVMAggregateNode>() != nullptr;
		});
		
		// Find all graphs to process and sort them by dependencies
		TArray<URigVMGraph*> ProcessedGraphs;
		while (ProcessedGraphs.Num() < AllModels.Num())
		{
			for (URigVMGraph* Graph : AllModels)
			{
				if (ProcessedGraphs.Contains(Graph))
				{
					continue;
				}

				bool bFoundUnprocessedReference = false;
				for (auto Node : Graph->GetNodes())
				{
					if (URigVMFunctionReferenceNode* Reference = Cast<URigVMFunctionReferenceNode>(Node))
					{
						if (Reference->GetReferencedFunctionHeader().LibraryPointer.HostObject != GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetObject())
						{
							continue;
						}

						URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Reference->GetReferencedFunctionHeader().LibraryPointer.GetNodeSoftPath().ResolveObject());
						if (!ProcessedGraphs.Contains(LibraryNode->GetContainedGraph()))
						{
							bFoundUnprocessedReference = true;
							break;
						}
					}
					else if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
					{
						if(!CollapseNode->IsA<URigVMAggregateNode>())
						{
							if (!ProcessedGraphs.Contains(CollapseNode->GetContainedGraph()))
							{
								bFoundUnprocessedReference = true;
								break;
							}
						}
					}
				}

				if (!bFoundUnprocessedReference)
				{
					ProcessedGraphs.Add(Graph);
				}
			}
		}	

		// Dump python commands for each graph
		for (URigVMGraph* Graph : ProcessedGraphs)
		{
			if (Graph->IsA<URigVMFunctionLibrary>())
			{
				continue;
			}

			URigVMController* Controller = GetController(Graph);
			if (Graph->GetParentGraph()) 
			{
				// Add them all as functions (even collapsed graphs)
				// The controller will deal with deleting collapsed graph function when it creates the collapse node
				{						
					// Add Function
					InternalCommands.Add(FString::Printf(TEXT("function_%s = library_controller.add_function_to_library('%s', mutable=%s)\ngraph = function_%s.get_contained_graph()"),
							*RigVMPythonUtils::PythonizeName(Graph->GetGraphName()),
							*Graph->GetGraphName(),
							Graph->GetEntryNode()->IsMutable() ? TEXT("True") : TEXT("False"),
							*RigVMPythonUtils::PythonizeName(Graph->GetGraphName())));

					URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());

					InternalCommands.Add(FString::Printf(TEXT("library_controller.set_node_category_by_name('%s', '%s')"),
						*Graph->GetGraphName(),
						*LibraryNode->GetNodeCategory()));

					InternalCommands.Add(FString::Printf(TEXT("library_controller.set_node_keywords_by_name('%s', '%s')"),
						*Graph->GetGraphName(),
						*LibraryNode->GetNodeKeywords() ));

					InternalCommands.Add(FString::Printf(TEXT("library_controller.set_node_description_by_name('%s', '%s')"),
						*Graph->GetGraphName(),
						*LibraryNode->GetNodeDescription()));

					InternalCommands.Add(FString::Printf(TEXT("library_controller.set_node_color_by_name('%s', %s)"),
						*Graph->GetGraphName(),
						*RigVMPythonUtils::LinearColorToPythonString(LibraryNode->GetNodeColor()) ));
					
					URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode();
					URigVMFunctionReturnNode* ReturnNode = Graph->GetReturnNode();
					
					// Set Entry and Return nodes in the correct position
					{
						//bool SetNodePositionByName(const FName& InNodeName, const FVector2D& InPosition, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);
						InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_position_by_name('Entry', unreal.Vector2D(%f, %f))"),
								*Graph->GetGraphName(),
								EntryNode->GetPosition().X, 
								EntryNode->GetPosition().Y));

						InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_position_by_name('Return', unreal.Vector2D(%f, %f))"),
								*Graph->GetGraphName(),
								ReturnNode->GetPosition().X, 
								ReturnNode->GetPosition().Y));
					}
					
					// Add Exposed Pins
					{

						bool bHitFirstExecute = false;
						bool bRenamedExecute = false;
						for (auto Pin : EntryNode->GetPins())
						{
							if (Pin->GetDirection() != ERigVMPinDirection::Output)
							{
								continue;
							}

							if(Pin->IsExecuteContext())
							{
								if(!bHitFirstExecute)
								{
									bHitFirstExecute = true;
									if (Pin->GetName() != FRigVMStruct::ExecuteContextName.ToString())
									{
										bRenamedExecute = true;
										InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').rename_exposed_pin('%s', '%s')"),
											*Graph->GetGraphName(),
											*FRigVMStruct::ExecuteContextName.ToString(),
											*Pin->GetName()));
									}
									continue;
								}
							}

							// FName AddExposedPin(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo = true);
							InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_exposed_pin('%s', %s, '%s', '%s', '%s')"),
									*Graph->GetGraphName(),
									*Pin->GetName(),
									*RigVMPythonUtils::EnumValueToPythonString<ERigVMPinDirection>((int64)ERigVMPinDirection::Input),
									*Pin->GetCPPType(),
									Pin->GetCPPTypeObject() ? *Pin->GetCPPTypeObject()->GetPathName() : TEXT(""),
									*Pin->GetDefaultValue()));
						}

						bHitFirstExecute = false;
						for (auto Pin : ReturnNode->GetPins())
						{
							if (Pin->GetDirection() != ERigVMPinDirection::Input)
							{
								continue;
							}

							if(Pin->IsExecuteContext())
							{
								if(!bHitFirstExecute)
								{
									bHitFirstExecute = true;
									if (!bRenamedExecute && Pin->GetName() != FRigVMStruct::ExecuteContextName.ToString())
									{
										bRenamedExecute = true;
										InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').rename_exposed_pin('%s', '%s')"),
											*Graph->GetGraphName(),
											*FRigVMStruct::ExecuteContextName.ToString(),
											*Pin->GetName()));
									}
									continue;
								}
							}

							// FName AddExposedPin(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo = true);
							InternalCommands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_exposed_pin('%s', %s, '%s', '%s', '%s')"),
									*Graph->GetGraphName(),
									*Pin->GetName(),
									*RigVMPythonUtils::EnumValueToPythonString<ERigVMPinDirection>((int64)ERigVMPinDirection::Output),
									*Pin->GetCPPType(),
									Pin->GetCPPTypeObject() ? *Pin->GetCPPTypeObject()->GetPathName() : TEXT(""),
									*Pin->GetDefaultValue()));
						}
					}
				}
			}
			else if(Graph != GetDefaultModel())
			{
				InternalCommands.Add(FString::Printf(TEXT("asset.add_model('%s')"),
						*Graph->GetName()));
			}
			
			InternalCommands.Append(Controller->GeneratePythonCommands());
		}
	}

	InternalCommands.Add(TEXT("asset.set_auto_vm_recompile(True)"));

	// Split multiple commands into different array elements
	TArray<FString> InnerFunctionCmds;
	for (FString Cmd : InternalCommands)
	{
		FString Left, Right=Cmd;
		while (Right.Split(TEXT("\n"), &Left, &Right))
		{
			InnerFunctionCmds.Add(Left);
		}
		InnerFunctionCmds.Add(Right);
	}

	return InnerFunctionCmds;
}

TArray<FRigVMExternalDependency> IRigVMEditorAssetInterface::GetExternalDependenciesForCategory(const FName& InCategory) const
{
	TArray<FRigVMExternalDependency> Dependencies;
	if(const FRigVMClient* Client = GetRigVMClient())
	{
		CollectExternalDependencies(Dependencies, InCategory, Client);
	}
	if(const TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = GetRigVMClientHost()->GetRigVMGraphFunctionHost())
	{
		if(const FRigVMGraphFunctionStore* FunctionStore = FunctionHost->GetRigVMGraphFunctionStore())
		{
			CollectExternalDependencies(Dependencies, InCategory, FunctionStore);
		}
	}

#if WITH_EDITOR
	const TArray<FRigVMGraphVariableDescription> MemberVariables = GetAssetVariables();
	for(const FRigVMGraphVariableDescription& MemberVariable : MemberVariables)
	{
		CollectExternalDependenciesForCPPTypeObject(Dependencies, InCategory, MemberVariable.CPPTypeObject.Get());
	}
#endif
	return Dependencies;
}

#if WITH_EDITOR
void IRigVMEditorAssetInterface::AddVariableSearchMetaDataInfo(const FName InVariableName, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const
{
	for (const FRigVMExternalVariable& Variable : GetExternalVariables(true))
	{
		if (Variable.GetName().IsEqual(InVariableName))
		{
			OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_Name, FText::FromName(InVariableName));

			const FEdGraphPinType& Type = RigVMTypeUtils::PinTypeFromExternalVariable(Variable);
			OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinCategory, FText::FromName(Type.PinCategory));
			OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_PinSubCategory, FText::FromName(Type.PinSubCategory));
			if (Type.PinSubCategoryObject.IsValid())
			{
				OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_ObjectClass, FText::FromString(Type.PinSubCategoryObject->GetPathName()));
			}
			OutTaggedMetaData.Emplace(FRigVMSearchTags::FiB_IsArray, FText::Format(LOCTEXT("RigVMNodePinIsArray", "{0}"), Type.IsArray() ? 1 : 0));
		}
	}
}
#endif

URigVMGraph* IRigVMEditorAssetInterface::GetTemplateModel(bool bIsFunctionLibrary)
{
#if WITH_EDITORONLY_DATA
	URigVMGraph* Model = nullptr;
	if (bIsFunctionLibrary)
	{
		Model = NewObject<URigVMFunctionLibrary>(GetObject(), TEXT("TemplateFunctionLibrary"));
	}
	else
	{
		Model = NewObject<URigVMGraph>(GetObject(), TEXT("TemplateModel"));
	}
	Model->SetFlags(RF_Transient);
	Model->SetExecuteContextStruct(GetRigVMClient()->GetDefaultExecuteContextStruct());
	return Model;
#else
	return nullptr;
#endif
}

URigVMController* IRigVMEditorAssetInterface::GetTemplateController(bool bIsFunctionLibrary)
{
#if WITH_EDITORONLY_DATA
	URigVMController* Controller = nullptr; 
	if (bIsFunctionLibrary)
	{
		Controller = NewObject<URigVMController>(GetObject(), TEXT("FunctionLibraryTemplateController"));
	}
	else
	{
		Controller = NewObject<URigVMController>(GetObject(), TEXT("TemplateController"));
	}
	Controller->EnableReporting(false);
	Controller->SetFlags(RF_Transient);
	Controller->SetSchemaClass(GetRigVMClient()->GetDefaultSchemaClass());
	Controller->SetGraph(GetTemplateModel(bIsFunctionLibrary));
	return Controller;
#else
	return nullptr;
#endif
}

UEdGraph* IRigVMEditorAssetInterface::GetEdGraph(const URigVMGraph* InModel) const
{
	return Cast<UEdGraph>(GetEditorObjectForRigVMGraph(InModel));
}

UEdGraph* IRigVMEditorAssetInterface::GetEdGraph(const FString& InNodePath) const
{
	if (URigVMGraph* ModelForNodePath = GetModel(InNodePath))
	{
		return GetEdGraph(ModelForNodePath);
	}
	return nullptr;
}

bool IRigVMEditorAssetInterface::IsFunctionPublic(const FName& InFunctionName) const
{
	return GetLocalFunctionLibrary()->IsFunctionPublic(InFunctionName);	
}

void IRigVMEditorAssetInterface::MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic)
{
	if(IsFunctionPublic(InFunctionName) == bIsPublic)
	{
		return;
	}
	
	URigVMController* Controller = GetRigVMClient()->GetOrCreateController(GetLocalFunctionLibrary());
	Controller->MarkFunctionAsPublic(InFunctionName, bIsPublic);
}

TArray<IRigVMEditorAssetInterface*> IRigVMEditorAssetInterface::GetDependencies(bool bRecursive) const
{
	TArray<IRigVMEditorAssetInterface*> Dependencies;

	TArray<URigVMGraph*> Graphs = GetAllModels();
	for(URigVMGraph* Graph : Graphs)
	{
		for(URigVMNode* Node : Graph->GetNodes())
		{
			if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
			{
				if(const URigVMLibraryNode* LibraryNode = FunctionReferenceNode->LoadReferencedNode())
				{
					if(IRigVMEditorAssetInterface* DependencyBlueprint = LibraryNode->GetImplementingOuter<IRigVMEditorAssetInterface>())
					{
						if(DependencyBlueprint != this)
						{
							if(!Dependencies.Contains(DependencyBlueprint))
							{
								Dependencies.Add(DependencyBlueprint);

								if(bRecursive)
								{
									TArray<IRigVMEditorAssetInterface*> ChildDependencies = DependencyBlueprint->GetDependencies(true);
									for(IRigVMEditorAssetInterface* ChildDependency : ChildDependencies)
									{
										Dependencies.AddUnique(ChildDependency);
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return Dependencies;
}

TArray<FAssetData> IRigVMEditorAssetInterface::GetDependentAssets() const
{
	TArray<FAssetData> Dependents;
	TArray<FSoftObjectPath> AssetPaths;

	if(URigVMFunctionLibrary* FunctionLibrary = GetRigVMClient()->GetFunctionLibrary())
	{
		IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		TArray<URigVMLibraryNode*> Functions = FunctionLibrary->GetFunctions();
		for(URigVMLibraryNode* Function : Functions)
		{
			const FName FunctionName = Function->GetFName();
			if(IsFunctionPublic(FunctionName))
			{
				TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> References = FunctionLibrary->GetReferencesForFunction(FunctionName);
				for(const TSoftObjectPtr<URigVMFunctionReferenceNode>& Reference : References)
				{
					if (const URigVMFunctionReferenceNode* ReferencePtr = Reference.Get())
					{
						if (const FRigVMEditorAssetInterfacePtr ControlRigBlueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(ReferencePtr))
						{
							const TSoftObjectPtr<UObject> Blueprint = ControlRigBlueprint.GetObject();
							const FSoftObjectPath AssetPath = Blueprint.ToSoftObjectPath();
							if(AssetPath.GetLongPackageName().StartsWith(TEXT("/Engine/Transient")))
							{
								continue;
							}
				
							if(!AssetPaths.Contains(AssetPath))
							{
								AssetPaths.Add(AssetPath);

								const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(AssetPath);
								if(AssetData.IsValid())
								{
									Dependents.Add(AssetData);
								}
							}
						}
					}
				}
			}
		}
	}

	return Dependents;
}

TArray<IRigVMEditorAssetInterface*> IRigVMEditorAssetInterface::GetDependentResolvedAssets(bool bRecursive, bool bOnlyLoaded) const
{
	TArray<FAssetData> Assets = GetDependentAssets();
	TArray<IRigVMEditorAssetInterface*> Dependents;

	for(const FAssetData& Asset : Assets)
	{
		if (!bOnlyLoaded || Asset.IsAssetLoaded())
		{
			if(FRigVMEditorAssetInterfacePtr Dependent = Asset.GetAsset())
			{
				if(!Dependents.Contains(Dependent.GetInterface()))
				{
					Dependents.Add(Dependent.GetInterface());

					if(bRecursive && Dependent != this)
					{
						TArray<IRigVMEditorAssetInterface*> ParentDependents = Dependent->GetDependentResolvedAssets(true);
						for(IRigVMEditorAssetInterface* ParentDependent : ParentDependents)
						{
							Dependents.AddUnique(ParentDependent);
						}
					}
				}
			}
		}
	}

	return Dependents;
}

void IRigVMEditorAssetInterface::BroadcastRefreshEditor()
{
	return RefreshEditorEvent.Broadcast(GetObject());
}

FOnRigVMRequestInspectMemoryStorage& IRigVMEditorAssetInterface::OnRequestInspectMemoryStorage()
{
	return OnRequestInspectMemoryStorageEvent;
}

void IRigVMEditorAssetInterface::RequestInspectMemoryStorage(const TArray<FRigVMMemoryStorageStruct*>& InMemoryStorageStructs) const
{
	OnRequestInspectMemoryStorageEvent.Broadcast(InMemoryStorageStructs);
}

void IRigVMEditorAssetInterface::SetObjectBeingDebugged(UObject* NewObject)
{
	URigVMHost* PreviousRigBeingDebugged = Cast<URigVMHost>(GetObjectBeingDebugged());
	if (PreviousRigBeingDebugged && PreviousRigBeingDebugged != NewObject)
	{
		PreviousRigBeingDebugged->DrawInterface.Reset();
		PreviousRigBeingDebugged->RigVMLog = nullptr;
#if WITH_EDITOR
		PreviousRigBeingDebugged->bIsBeingDebugged = false;
		PreviousRigBeingDebugged->GetDebugInfo().Reset();
#endif
	}

	SetObjectBeingDebuggedSuper(NewObject);

#if WITH_EDITOR
	if(URigVMHost* NewRigBeingDebugged = Cast<URigVMHost>(NewObject))
	{
		NewRigBeingDebugged->bIsBeingDebugged = true;

		UpdateDebugMemoryOnHost(NewRigBeingDebugged);
	}
	
	ResetEarlyExitInstruction(true);
#endif
}

void IRigVMEditorAssetInterface::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		// The action stack undo/redo transaction should always execute first
		// It already knows whether or not it has already executed or not
		GetRigVMClient()->GetOrCreateActionStack()->PostTransacted(TransactionEvent);
	}
	
	PostTransactedSuper(TransactionEvent);

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		TArray<FName> PropertiesChanged = TransactionEvent.GetChangedProperties();

		if (PropertiesChanged.Contains(TEXT("VMRuntimeSettings")))
		{
			PropagateRuntimeSettingsFromBPToInstances();
		}

		if (PropertiesChanged.Contains(TEXT("NewVariables")))
		{
			if (RefreshEditorEvent.IsBound())
			{
				RefreshEditorEvent.Broadcast(GetObject());
			}
			(void)GetObject()->MarkPackageDirty();
		}

		if (PropertiesChanged.Contains(TEXT("RigVMClient")) ||
			PropertiesChanged.Contains(TEXT("UbergraphPages")))
		{
			GetUberGraphs().RemoveAll([](const UEdGraph* UberGraph) -> bool
			{
 				return UberGraph == nullptr || !IsValid(UberGraph);
			});
			GetRigVMClient()->PostTransacted(TransactionEvent);
			
			RecompileVM();
			(void)GetObject()->MarkPackageDirty();
		}
	}
}

void IRigVMEditorAssetInterface::ReplaceDeprecatedNodes()
{
	TArray<UEdGraph*> EdGraphs;
	GetAllEdGraphs(EdGraphs);

	for (UEdGraph* EdGraph : EdGraphs)
	{
		EdGraph->Schema = GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
	}

	ReplaceDeprecatedNodesSuper();
}

void IRigVMEditorAssetInterface::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	PreDuplicateSuper(DupParams);
	PreDuplicateHostPath = GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetObject();
	PreDuplicateFunctionLibraryPath = GetRigVMClient()->GetFunctionLibrary();

	// look for graphs which are no longer reachable and remove them. this can happen in case
	// one of the default sub objects got renamed.
	FRigVMClient* RigVMClient = GetRigVMClient();
	TArray<URigVMGraph*> AllModels = RigVMClient->GetAllModels(true, true);
	for (TObjectIterator<URigVMGraph> RigVMIt; RigVMIt; ++RigVMIt)
	{
		URigVMGraph* Graph = *RigVMIt;
		if (Graph->IsInOuter(GetObject()))
		{
			if (!AllModels.Contains(Graph))
			{
				if (URigVMController* Controller = RigVMClient->GetOrCreateController(RigVMClient->GetDefaultModel()))
				{
					Controller->DestroyObject(Graph);
				}
			}
		}
	}
}

void IRigVMEditorAssetInterface::ReplaceFunctionIdentifiers(const FString& OldFunctionHostPath, const FString& NewFunctionHostPath, const FString& OldFunctionLibraryPath, const FString& NewFunctionLibraryPath)
{
	if (!OldFunctionHostPath.Equals(NewFunctionHostPath))
	{
		const FString OldLibraryPath = OldFunctionLibraryPath + TEXT(".");
		const FString NewLibraryPath = NewFunctionLibraryPath + TEXT(".");
		
		auto ReplaceIdentifier = [OldLibraryPath, NewLibraryPath, OldFunctionHostPath, NewFunctionHostPath](FRigVMGraphFunctionIdentifier& Identifier)
		{
			FString& LibraryNodePath = Identifier.GetLibraryNodePath();
			FSoftObjectPath& HostPath = Identifier.HostObject;
			FString HostPathStr = HostPath.ToString();
			if(LibraryNodePath.StartsWith(OldLibraryPath, ESearchCase::CaseSensitive))
			{
				LibraryNodePath = NewLibraryPath + LibraryNodePath.Mid(OldLibraryPath.Len());
			}
			if(HostPathStr.StartsWith(OldFunctionHostPath, ESearchCase::CaseSensitive))
			{
				HostPathStr = NewFunctionHostPath + HostPathStr.Mid(OldFunctionHostPath.Len());
				HostPath = HostPathStr;
			}
		};

		// Replace identifiers in store
		if(TScriptInterface<IRigVMGraphFunctionHost> CRGeneratedClass = GetRigVMClientHost()->GetRigVMGraphFunctionHost())
		{
			FRigVMGraphFunctionStore* Store = CRGeneratedClass->GetRigVMGraphFunctionStore();
			for (int32 i=0; i<2; ++i)
			{
				TArray<FRigVMGraphFunctionData>& Functions = (i == 0) ? Store->PublicFunctions : Store->PrivateFunctions;
				for (FRigVMGraphFunctionData& Data : Functions)
				{
					ReplaceIdentifier(Data.Header.LibraryPointer);
					for (TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : Data.Header.Dependencies)
					{
						ReplaceIdentifier(Pair.Key);
					}
				}
			}
		}

		// Replace identifiers in function references
		TArray<URigVMGraph*> AllModels = GetRigVMClient()->GetAllModels(true, true);
		for(URigVMGraph* Model : AllModels)
		{
			for (URigVMNode* Node : Model->GetNodes())
			{
				if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
				{
					ReplaceIdentifier(FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer);
				}
			}
		}
	}
}

void IRigVMEditorAssetInterface::PostDuplicate(bool bDuplicateForPIE)
{
	// assuming PostDuplicate is always followed by a PostLoad:
	// so theoretically, PostDuplicate just makes corrections to the serialized data and does nothing more,
	// while PostLoad looks at whatever is serialized and load it into memory according to the version of the editor used
	// note: how to know if we have corrected everything?
	// ans: check the reference viewer for the duplicated BP and make sure that the original BP does not appear in there
	
	{
		// pause compilation because we need to patch some stuff first
		TGuardValue<bool> CompilingGuard(bIsCompiling, true);
		// this will create the new EMPTY generated class to be used as the function store for this BP
		// it will be filled during PostLoad based on the graph model
		PostDuplicateSuper(bDuplicateForPIE);
	}

	const FString OldFunctionHostPath = PreDuplicateHostPath.ToString();
	const FString NewFunctionHostPath = FSoftObjectPath(GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetObject()).ToString();
	const FString OldFunctionLibraryPath = PreDuplicateFunctionLibraryPath.ToString();
	const FString NewFunctionLibraryPath = FSoftObjectPath(GetRigVMClient()->GetFunctionLibrary()).ToString();
	
	ReplaceFunctionIdentifiers(OldFunctionHostPath, NewFunctionHostPath, OldFunctionLibraryPath, NewFunctionLibraryPath);

	PreDuplicateHostPath.Reset();
	PreDuplicateFunctionLibraryPath.Reset();

	SplitAssetVariant();

	TArray<UEdGraph*> EdGraphs;
	GetAllEdGraphs(EdGraphs);
	for (UEdGraph* EdGraph : EdGraphs)
	{
		if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(EdGraph))
		{
			RigVMEdGraph->CachedModelGraph.Reset();
		}
	}

	MarkAssetAsStructurallyModified();
}

void IRigVMEditorAssetInterface::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (CachedAssetTags.IsEmpty())
	{
		GetAssetRegistryTagsSuper(Context);
		CachedAssetTags.Reset(Context.GetNumTags());
		Context.EnumerateTags([this](const UObject::FAssetRegistryTag& Tag)
			{
				CachedAssetTags.Add(Tag);
			});
	}
	else
	{
		for (const UObject::FAssetRegistryTag& Tag : CachedAssetTags)
		{
			Context.AddTag(Tag);
		}
	}
}

FRigVMGraphModifiedEvent& IRigVMEditorAssetInterface::OnModified()
{
	return ModifiedEvent;
}

FOnRigVMCompiledEvent& IRigVMEditorAssetInterface::OnVMCompiled()
{
	return VMCompiledEvent;
}

URigVMHost* IRigVMEditorAssetInterface::CreateRigVMHostImpl()
{
	RecompileVMIfRequired();

	URigVMHost* Host = CreateRigVMHostSuper(GetObject());
	Host->Initialize(true);
	return Host;
}

TArray<UStruct*> IRigVMEditorAssetInterface::GetAvailableRigVMStructs() const
{
	TArray<UStruct*> Structs;
	UStruct* BaseStruct = FRigVMStruct::StaticStruct();

	for (const FRigVMFunction& Function : FRigVMRegistry::Get().GetFunctions())
	{
		if (Function.Struct)
		{
			if (Function.Struct->IsChildOf(BaseStruct))
			{
				Structs.Add(Function.Struct);
				// todo: filter by available types
				// todo: filter by execute context
			}
		}
	}

	return Structs;
}

#if WITH_EDITOR


FRigVMVariantRef IRigVMEditorAssetInterface::GetAssetVariantRefImpl() const
{
	return FRigVMVariantRef(FSoftObjectPath(GetObject()), GetAssetVariant());
}

bool IRigVMEditorAssetInterface::SplitAssetVariantImpl()
{
	if(GetMatchingVariants().IsEmpty())
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("SplitAssetVariant", "Split Asset Variant"));
	GetObject()->Modify();

	// prefer the path based (deterministic) guid - and fall back on random.
	const FGuid PathBasedGuid = FRigVMVariant::GenerateGUID(GetObject()->GetPathName());
	if(PathBasedGuid != GetAssetVariant().Guid)
	{
		GetAssetVariant().Guid = PathBasedGuid;
	}
	else
	{
		GetAssetVariant().Guid = FRigVMVariant::GenerateGUID();
	}
	
	return true;
}

bool IRigVMEditorAssetInterface::JoinAssetVariantImpl(const FGuid& InGuid)
{
	if(GetAssetVariant().Guid != InGuid)
	{
		FScopedTransaction Transaction(LOCTEXT("JoinAssetVariant", "Join Asset Variant"));
		GetObject()->Modify();
		
		GetAssetVariant().Guid = InGuid;
		return true;
	}

	return false;
}

TArray<FRigVMVariantRef> IRigVMEditorAssetInterface::GetMatchingVariantsImpl() const
{
	if(URigVMBuildData* BuildData = URigVMBuildData::Get())
	{
		TArray<FRigVMVariantRef> Variants = BuildData->FindAssetVariantRefs(GetAssetVariant().Guid);
		const FRigVMVariantRef MyVariantRef = FRigVMVariantRef(GetObject()->GetPathName(), GetAssetVariant());
		Variants.RemoveAll([MyVariantRef](const FRigVMVariantRef& VariantRef) -> bool
		{
			return VariantRef == MyVariantRef;
		});
		return Variants;
	}
	return TArray<FRigVMVariantRef>();
}

#endif

void IRigVMEditorAssetInterface::RebuildGraphFromModel()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TGuardValue<bool> SelfGuard(bSuspendModelNotificationsForSelf, true);
	TGuardValue<bool> ClientIgnoreModificationsGuard(GetRigVMClient()->bIgnoreModelNotifications, true);
	
	verify(GetOrCreateController());

	TArray<UEdGraph*> EdGraphs;
	GetAllEdGraphs(EdGraphs);

	for (UEdGraph* Graph : EdGraphs)
	{
		TArray<UEdGraphNode*> Nodes = Graph->Nodes;
		for (UEdGraphNode* Node : Nodes)
		{
			Graph->RemoveNode(Node);
		}

		if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph))
		{
			if (RigGraph->bIsFunctionDefinition)
			{
				GetFunctionGraphs().Remove(RigGraph);
			}
		}
	}

	if(GetFunctionLibraryEdGraph() && GetRigVMClient()->GetFunctionLibrary())
	{
		GetFunctionLibraryEdGraph()->ModelNodePath = GetRigVMClient()->GetFunctionLibrary()->GetNodePath();
	}

	TArray<URigVMGraph*> RigGraphs = GetRigVMClient()->GetAllModels(true, true);

	for (int32 RigGraphIndex = 0; RigGraphIndex < RigGraphs.Num(); RigGraphIndex++)
	{
		GetOrCreateController(RigGraphs[RigGraphIndex])->ResendAllNotifications();
	}

	for (int32 RigGraphIndex = 0; RigGraphIndex < RigGraphs.Num(); RigGraphIndex++)
	{
		URigVMGraph* RigGraph = RigGraphs[RigGraphIndex];

		for (URigVMNode* RigNode : RigGraph->GetNodes())
		{
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(RigNode))
			{
				CreateEdGraphForCollapseNodeIfNeeded(CollapseNode, true);
			}
		}
	}
}

void IRigVMEditorAssetInterface::Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject)
{
	GetOrCreateController()->Notify(InNotifType, InSubject);
}

void IRigVMEditorAssetInterface::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

#if WITH_EDITOR

	if (bSuspendAllNotifications)
	{
		return;
	}

	// since it's possible that a notification will be already sent / forwarded to the
	// listening objects within the switch statement below - we keep a flag to mark
	// the notify for still pending (or already sent)
	bool bNotifForOthersPending = true;

	auto MarkBlueprintAsStructurallyModified = [this]()
	{
		if(VMRecompilationBracket == 0)
		{
			if(bMarkBlueprintAsStructurallyModifiedPending)
			{
				bMarkBlueprintAsStructurallyModifiedPending = false;
				MarkAssetAsStructurallyModified(bSkipDirtyBlueprintStatus);
			}
		}
		else
		{
			bMarkBlueprintAsStructurallyModifiedPending = true;
		}
	};

	if (!bSuspendModelNotificationsForSelf)
	{
		switch (InNotifType)
		{
			case ERigVMGraphNotifType::InteractionBracketOpened:
			{
				IncrementVMRecompileBracket();
				break;
			}
			case ERigVMGraphNotifType::InteractionBracketClosed:
			case ERigVMGraphNotifType::InteractionBracketCanceled:
			{
				DecrementVMRecompileBracket();
				
				// RecompileVM already updates the status, no need to dirty it again
				const bool bSkipDirty = !bVMRecompilationRequired;
				FGuardSkipDirtyBlueprintStatus DoNotDirtyStatus(GetObject(), bSkipDirty);
				MarkBlueprintAsStructurallyModified();
				break;
			}
			case ERigVMGraphNotifType::PinDefaultValueChanged:
			{
				if (URigVMPin* Pin = Cast<URigVMPin>(InSubject))
				{
					bool bRequiresRecompile = false;

					URigVMPin* RootPin = Pin->GetRootPin();
					static const FString ConstSuffix = TEXT(":Const");
					const FString PinHash = RootPin->GetPinPath(true) + ConstSuffix;
					
					if (const FRigVMOperand* Operand = PinToOperandMap.Find(PinHash))
					{
						FRigVMASTProxy RootPinProxy = FRigVMASTProxy::MakeFromUObject(RootPin);
						if(const FRigVMExprAST* Expression = InGraph->GetRuntimeAST()->GetExprForSubject(RootPinProxy))
						{
							bRequiresRecompile = Expression->NumParents() > 1;
						}
						else
						{
							bRequiresRecompile = true;
						}

						// If we are only changing a pin's default value, we need to
						// check if there is a connection to a sub-pin of the root pin
						// that has its value is directly stored in the root pin due to optimization, if so,
						// we want to recompile to make sure the pin's new default value and values from other connections
						// are both applied to the root pin because GetDefaultValue() alone cannot account for values
						// from other connections.
						if(!bRequiresRecompile)
						{
							TArray<URigVMPin*> SourcePins = RootPin->GetLinkedSourcePins(true);
							for (const URigVMPin* SourcePin : SourcePins)
							{
								// check if the source node is optimized out, if so, only a recompile will allows us
								// to re-query its value.
								FRigVMASTProxy SourceNodeProxy = FRigVMASTProxy::MakeFromUObject(SourcePin->GetNode());
								if (InGraph->GetRuntimeAST()->GetExprForSubject(SourceNodeProxy) == nullptr)
								{
									bRequiresRecompile = true;
									break;
								}
							}
						} 
						
						if(!bRequiresRecompile)
						{
							const FString DefaultValue = RootPin->GetDefaultValue();

							URigVM* VM = GetVM(true);
							FRigVMExtendedExecuteContext* Context = GetRigVMExtendedExecuteContext();
							if (VM != nullptr && Context != nullptr)
							{
								VM->SetPropertyValueFromString(*Context, *Operand, DefaultValue);
							}

							TArray<UObject*> ArchetypeInstances = GetArchetypeInstances(false, false);
							for (UObject* ArchetypeInstance : ArchetypeInstances)
							{
								URigVMHost* InstancedHost = Cast<URigVMHost>(ArchetypeInstance);
								if (!URigVMHost::IsGarbageOrDestroyed(InstancedHost))
								{
									if (!InstancedHost->HasAllFlags(RF_ClassDefaultObject))
									{
										if (InstancedHost->GetVM())
										{
											InstancedHost->VM->SetPropertyValueFromString(InstancedHost->GetRigVMExtendedExecuteContext(), *Operand, DefaultValue);
										}
									}
								}
							}

							if (Pin->IsDefinedAsConstant() || Pin->GetRootPin()->IsDefinedAsConstant())
							{
								// re-init the rigs
								GetRigVMClientHost()->RequestRigVMInit();
								bRequiresRecompile = true;
							}
						}
					}
					else
					{
						bRequiresRecompile = true;
					}
				
					if(bRequiresRecompile)
					{
						RequestAutoVMRecompilation();
					}
				}
				(void)GetObject()->MarkPackageDirty();
				break;
			}
			case ERigVMGraphNotifType::LocalVariableDefaultValueChanged:
			{
				RequestAutoVMRecompilation();
					break;
			}
			case ERigVMGraphNotifType::NodeAdded:
			case ERigVMGraphNotifType::NodeRemoved:
			{
				const bool bAdded = InNotifType == ERigVMGraphNotifType::NodeAdded;
				if (URigVMHost::IsGarbageOrDestroyed(InSubject))
				{
					break;
				}

				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					if (bAdded)
					{
						// If the controller for this graph already exist, make sure it is referencing the correct graph
						if (URigVMController* Controller = GetRigVMClient()->GetController(CollapseNode->GetContainedGraph()))
						{
							Controller->SetGraph(CollapseNode->GetContainedGraph());
						}
						
						CreateEdGraphForCollapseNodeIfNeeded(CollapseNode);
					}
					else
					{
						bNotifForOthersPending = !RemoveEdGraphForCollapseNode(CollapseNode, true);

						// Cannot remove from the Controllers array because we would lose the action stack on that graph
						// Controllers.Remove(CollapseNode->GetContainedGraph();
					}

					RequestAutoVMRecompilation();
					ResetEarlyExitInstruction(true);

					(void)GetObject()->MarkPackageDirty();
					MarkBlueprintAsStructurallyModified();
					break;
				}

				if (URigVMNode* RigVMNode = Cast<URigVMNode>(InSubject))
				{
					if(RigVMNode->IsEvent() && RigVMNode->GetGraph()->IsRootGraph())
					{
						// let the UI know the title for the graph may have changed.
						GetRigVMClient()->NotifyOuterOfPropertyChange();

						if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(GetEdGraph(RigVMNode->GetGraph())))
						{
							// decide if this graph should be renameable
							const int32 NumberOfEvents = static_cast<int32>(Algo::CountIf(RigVMNode->GetGraph()->GetNodes(), [](const URigVMNode* NodeToCount) -> bool
							{
								return NodeToCount->IsEvent() && NodeToCount->CanOnlyExistOnce();
							}));
							EdGraph->bAllowRenaming = NumberOfEvents != 1;
						}
					}
				}
				// fall through to the next case
			}
			case ERigVMGraphNotifType::LinkAdded:
			case ERigVMGraphNotifType::LinkRemoved:
			case ERigVMGraphNotifType::PinArraySizeChanged:
			case ERigVMGraphNotifType::PinDirectionChanged:
			{
				RequestAutoVMRecompilation();
				ResetEarlyExitInstruction(true);
				(void)GetObject()->MarkPackageDirty();

				// we don't need to mark the blueprint as modified since we only
				// need to recompile the VM here - unless we don't auto recompile.
				if(!bAutoRecompileVM)
				{
					MarkBlueprintAsStructurallyModified();
				}
				break;
			}
			case ERigVMGraphNotifType::PinWatchedChanged:
			{
				if (URigVMHost* DebuggedHost = Cast<URigVMHost>(GetObjectBeingDebugged()))
				{
					URigVMPin* Pin = CastChecked<URigVMPin>(InSubject)->GetRootPin(); 
					URigVMCompiler* Compiler = URigVMCompiler::StaticClass()->GetDefaultObject<URigVMCompiler>();

					TSharedPtr<FRigVMParserAST> RuntimeAST = GetDefaultModel()->GetRuntimeAST();
					
					if(Pin->RequiresWatch())
					{
						// check if the node is optimized out - in that case we need to recompile
						if(DebuggedHost->GetVM()->GetByteCode().GetFirstInstructionIndexForSubject(Pin->GetNode()) == INDEX_NONE)
						{
							RequestAutoVMRecompilation();
							(void)GetObject()->MarkPackageDirty();
						}
						else
						{
							(void)DebuggedHost->GetDebugMemory(true /* create if required */);
							Compiler->MarkDebugWatch(DebuggedHost->GetRigVMExtendedExecuteContext(), true, Pin, DebuggedHost->GetVM(), &PinToOperandMap, RuntimeAST);
						}
					}
					else
					{
						Compiler->MarkDebugWatch(DebuggedHost->GetRigVMExtendedExecuteContext(), false, Pin, DebuggedHost->GetVM(), &PinToOperandMap, RuntimeAST);
					}
				}
				// break; fall through
			}
			case ERigVMGraphNotifType::PinTypeChanged:
			case ERigVMGraphNotifType::PinIndexChanged:
			{
				if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
				{
					if (UEdGraph* EdGraph = GetEdGraph(InGraph))
					{							
						if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(EdGraph))
						{
							if (UEdGraphNode* EdNode = Graph->FindNodeForModelNodeName(ModelPin->GetNode()->GetFName()))
							{
								if (UEdGraphPin* EdPin = EdNode->FindPin(*ModelPin->GetPinPath()))
								{
									if (ModelPin->RequiresWatch())
									{
										AddPinWatch(EdPin);
									}
									else
									{
										RemovePinWatch(EdPin);
									}

									if(InNotifType == ERigVMGraphNotifType::PinWatchedChanged)
									{
										return;
									}
									RequestAutoVMRecompilation();
									(void)GetObject()->MarkPackageDirty();
								}
							}
						}
					}
				}
				// fall through another time
			}
			case ERigVMGraphNotifType::PinAdded:
			case ERigVMGraphNotifType::PinRemoved:
			case ERigVMGraphNotifType::PinRenamed:
			{
				if (URigVMHost::IsGarbageOrDestroyed(InSubject))
				{
					break;
				}

				// exposed pin changes like this (as well as type change etc)
				// require to mark the blueprint as structurally modified,
				// so that the instance actions work out.
				if (URigVMPin* ModelPin = Cast<URigVMPin>(InSubject))
				{
					if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelPin->GetNode()))
					{
						if(Cast<URigVMFunctionLibrary>(CollapseNode->GetOuter()))
						{
							MarkBlueprintAsStructurallyModified();
						}
					}
				}
				break;
			}
			case ERigVMGraphNotifType::PinBoundVariableChanged:
			{
				RequestAutoVMRecompilation();
				(void)GetObject()->MarkPackageDirty();
				break;
			}
			case ERigVMGraphNotifType::NodeRenamed:
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					FString NewNodePath = CollapseNode->GetNodePath(true /* recursive */);
					FString Left, Right = NewNodePath;
					URigVMNode::SplitNodePathAtEnd(NewNodePath, Left, Right);
					FString OldNodePath = CollapseNode->GetPreviousFName().ToString();
					if (!Left.IsEmpty())
					{
						OldNodePath = URigVMNode::JoinNodePath(Left, OldNodePath);
					}

					HandleRigVMGraphRenamed(GetRigVMClient(), OldNodePath, NewNodePath);

					if (UEdGraph* ContainedEdGraph = GetEdGraph(CollapseNode->GetContainedGraph()))
					{
						ContainedEdGraph->Rename(*CollapseNode->GetEditorSubGraphName(), nullptr);
					}

					MarkBlueprintAsStructurallyModified();
				}
				break;
			}
			case ERigVMGraphNotifType::NodeCategoryChanged:
			case ERigVMGraphNotifType::NodeKeywordsChanged:
			case ERigVMGraphNotifType::NodeDescriptionChanged:
			{
				MarkBlueprintAsStructurallyModified();
				break;
			}
			default:
			{
				break;
			}
		}
	}

	// if the notification still has to be sent...
	if (bNotifForOthersPending && !GetRigVMClient()->bSuspendModelNotificationsForOthers)
	{
		if (OnModified().IsBound())
		{
			OnModified().Broadcast(InNotifType, InGraph, InSubject);
		}
	}
#endif
}

void IRigVMEditorAssetInterface::SuspendNotifications(bool bSuspendNotifs)
{
	if (bSuspendAllNotifications == bSuspendNotifs)
	{
		return;
	}

	bSuspendAllNotifications = bSuspendNotifs;
	if (!bSuspendNotifs)
	{
		RebuildGraphFromModel();
		RefreshEditorEvent.Broadcast(GetObject());
		RequestAutoVMRecompilation();
	}
}

void IRigVMEditorAssetInterface::CreateMemberVariablesOnLoad()
{
#if WITH_EDITOR

	AddedMemberVariableMap.Reset();
	TArray<FRigVMExternalVariable> Variables = GetExternalVariables(true);
	for (int32 VariableIndex = 0; VariableIndex < Variables.Num(); VariableIndex++)
	{
		AddedMemberVariableMap.Add(Variables[VariableIndex].GetName(), VariableIndex);
	}

	if (GetRigVMClient()->Num() == 0)
	{
		return;
	}

#endif
}

void IRigVMEditorAssetInterface::PatchVariableNodesOnLoad()
{
#if WITH_EDITOR
	AddedMemberVariableMap.Reset();
#endif
}

void IRigVMEditorAssetInterface::PatchBoundVariables()
{
}

void IRigVMEditorAssetInterface::PatchVariableNodesWithIncorrectType()
{
	TGuardValue<bool> GuardNotifsSelf(bSuspendModelNotificationsForSelf, true);

	struct Local
	{
		static bool RefreshIfNeeded(URigVMController* Controller, URigVMVariableNode* VariableNode, const FString& CPPType, UObject* CPPTypeObject)
		{
			if (URigVMPin* ValuePin = VariableNode->GetValuePin())
			{
				if (ValuePin->GetCPPType() != CPPType || ValuePin->GetCPPTypeObject() != CPPTypeObject)
				{
					Controller->RefreshVariableNode(VariableNode->GetFName(), VariableNode->GetVariableName(), CPPType, CPPTypeObject, false);
					if (RigVMTypeUtils::AreCompatible(*ValuePin->GetCPPType(), ValuePin->GetCPPTypeObject(), *CPPType, CPPTypeObject))
					{
						return false;
					}
					return true;
				}
			}
			return false;
		}
	};

	for (URigVMGraph* Graph : GetAllModels())
	{
		URigVMController* Controller = GetOrCreateController(Graph);
		TArray<URigVMNode*> Nodes = Graph->GetNodes();
		for (URigVMNode* Node : Nodes)
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (VariableNode->IsInputArgument())
				{
					continue;
				}
				
				FRigVMGraphVariableDescription Description = VariableNode->GetVariableDescription();

				// Check for local variables
				if (VariableNode->IsLocalVariable())
				{
					TArray<FRigVMGraphVariableDescription> LocalVariables = Graph->GetLocalVariables(false);
					for (FRigVMGraphVariableDescription Variable : LocalVariables)
					{
						if (Variable.Name == Description.Name)
						{
							if (Local::RefreshIfNeeded(Controller, VariableNode, Variable.CPPType, Variable.CPPTypeObject))
							{
								bDirtyDuringLoad = true;
							}
							break;
						}
					}
				}
				else
				{
					for (FRigVMGraphVariableDescription& Variable : GetAssetVariables())
					{
						if (Variable.Name == Description.Name)
						{
							if (Local::RefreshIfNeeded(Controller, VariableNode, Variable.CPPType, Variable.CPPTypeObject))
							{
								bDirtyDuringLoad = true;
							}
						}
					}
				}
			}
		}
	}
}

void IRigVMEditorAssetInterface::RepairDuplicateVariableGuidsOnLoad()
{
#if WITH_EDITOR
	// [GuidDedupe] One-shot recovery for the duplicate-Guid corruption. See header comment for
	// the full rationale. Short version: the Guid is the ground-truth identity of a variable;
	// the corruption is "two variables share an identity"; recovery is "give later duplicates a
	// fresh identity and rewrite each variable node that previously pointed at the now-old Guid
	// to point at the matching variable's new Guid".
	//
	// Two independent passes: Pass A is asset-variables-bag (only runs if the bag exists and has
	// duplicates), Pass B is per-graph local variables (per-graph detect, runs unconditionally —
	// local variables live on the graph, not the bag, so they can have duplicates even when the
	// bag is healthy or absent). Detection scans always run (a TSet<FGuid> per scope); no writes
	// occur on a healthy asset and bDirtyDuringLoad is left untouched.

	// Each FRepair captures everything the per-node patch step needs at scan time, so neither
	// step has to re-look anything up after the bag/array has been mutated.
	struct FRepair
	{
		FName VarName;
		FGuid OldGuid;                     // the duplicate Guid that was originally on this variable
		FGuid NewGuid;                     // freshly-generated replacement
		int32 OriginalIndex;               // index in the bag's desc array (Pass A only); INDEX_NONE for Pass B
		FString CPPType;                   // captured at scan time so RefreshVariableNode can preserve type info
		TObjectPtr<UObject> CPPTypeObject; // captured at scan time
	};

	int32 BagRepairCount = 0;
	int32 ReconciledCount = 0;
	int32 LocalRepairCount = 0;
	int32 LocalNodeReconciledCount = 0;

	TGuardValue<bool> GuardNotifsSelf(bSuspendModelNotificationsForSelf, true);

	const TArray<URigVMGraph*> AllModels = GetAllModels();

	// Pass A: bag-level (asset variables) duplicate detection + repair.
	if (FRigVMPropertyBag* Bag = GetVariablesPropertyBag())
	{
		if (const UPropertyBag* BagStruct = Bag->GetPropertyBagStruct())
		{
			TArray<FRepair> Repairs;
			{
				const TConstArrayView<FPropertyBagPropertyDesc> Descs = BagStruct->GetPropertyDescs();
				TSet<FGuid> SeenGuids;
				const TArray<FRigVMGraphVariableDescription> AssetVariables = GetAssetVariables();
				for (int32 DescIndex = 0; DescIndex < Descs.Num(); ++DescIndex)
				{
					const FPropertyBagPropertyDesc& Desc = Descs[DescIndex];
					if (!Desc.ID.IsValid())
					{
						continue;
					}
					bool bAlreadyIn = false;
					SeenGuids.Add(Desc.ID, &bAlreadyIn);
					if (bAlreadyIn)
					{
						// Capture type info now — at scan time the bag is still consistent with
						// what GetAssetVariables() reports, so we don't need to re-resolve later.
						FString CapturedCPPType;
						TObjectPtr<UObject> CapturedCPPTypeObject = nullptr;
						for (const FRigVMGraphVariableDescription& Variable : AssetVariables)
						{
							if (Variable.Name == Desc.Name)
							{
								CapturedCPPType = Variable.CPPType;
								CapturedCPPTypeObject = Variable.CPPTypeObject;
								break;
							}
						}
						Repairs.Add({Desc.Name, Desc.ID, FGuid::NewGuid(), DescIndex,
							MoveTemp(CapturedCPPType), CapturedCPPTypeObject});
					}
				}
			}

			if (Repairs.Num() > 0)
			{
				UE_LOGF(LogRigVMDeveloper, Warning,
					"RepairDuplicateVariableGuidsOnLoad: asset '%ls' has %d variable(s) sharing a Guid with another variable. Repairing in place.",
					*GetObject()->GetPathName(), Repairs.Num());

				// Step 1: regenerate Guids in the bag for each duplicated variable. Verify the
				// re-add succeeded — this is recovery code on a load path, and AddProperties is
				// fallible (the existing AddHostMemberVariableFromExternal path also checks the
				// bag's count to detect failure).
				for (const FRepair& R : Repairs)
				{
					// Look up the variable in the asset list to capture its current state. We can't
					// use FindAssetVariableByGuid here — the corruption being fixed makes Guid
					// lookup ambiguous (returns the first match, which may be the wrong variable).
					FRigVMGraphVariableDescription Snapshot;
					{
						for (const FRigVMGraphVariableDescription& Variable : GetAssetVariables())
						{
							if (Variable.Name == R.VarName)
							{
								Snapshot = Variable;
								break;
							}
						}
					}
					if (Snapshot.Name.IsNone())
					{
						continue;
					}
					FRigVMGraphVariableDescription Replacement = Snapshot;
					Replacement.Guid = R.NewGuid;
					FRigVMPropertyDescription Description = RigVMVariableUtils::PropertyDescriptionFromVariableDescription(Replacement, true);

					if (Bag->RemovePropertyByName(R.VarName) != EPropertyBagAlterationResult::Success)
					{
						UE_LOGF(LogRigVMDeveloper, Error,
							"  Bag: failed to remove '%ls' for repair; skipping.",
							*R.VarName.ToString());
						continue;
					}

					const int32 NumBeforeAdd = Bag->GetPropertyBagStruct() ? Bag->GetPropertyBagStruct()->GetPropertyDescs().Num() : 0;
					Bag->AddProperties({Description});
					const int32 NumAfterAdd = Bag->GetPropertyBagStruct() ? Bag->GetPropertyBagStruct()->GetPropertyDescs().Num() : 0;
					if (NumAfterAdd <= NumBeforeAdd)
					{
						// Re-add failed; the variable was lost. Loud error and skip the node patch.
						UE_LOGF(LogRigVMDeveloper, Error,
							"  Bag: re-add of '%ls' failed after Guid regen; variable was dropped. Skipping node patch for this entry.",
							*R.VarName.ToString());
						continue;
					}

					if (R.OriginalIndex != INDEX_NONE)
					{
						SetVariableIndex(R.VarName, R.OriginalIndex);
					}
					++BagRepairCount;
					UE_LOGF(LogRigVMDeveloper, Verbose, "  Bag: regenerated Guid for variable '%ls': %ls -> %ls",
						*R.VarName.ToString(), *R.OldGuid.ToString(), *R.NewGuid.ToString());
				}

				// Build a Guid->Repair lookup so the per-node walk is O(1) per node instead of
				// O(repairs) per node. The lookup is also self-documenting about the invariant
				// that each (VarName, OldGuid) tuple is added at most once.
				TMap<FGuid, const FRepair*> RepairByOldGuid;
				RepairByOldGuid.Reserve(Repairs.Num());
				for (const FRepair& R : Repairs)
				{
					RepairByOldGuid.Add(R.OldGuid, &R);
				}

				// Step 2: walk variable nodes that pointed at one of the duplicate Guids; update
				// only the ones whose Name pin matches a regenerated variable. The (NodeGuid ==
				// OldGuid) check is the primary signal; the Name match disambiguates which of
				// the (formerly identical-Guid) variables this node belonged to.
				for (URigVMGraph* Graph : AllModels)
				{
					if (!Graph)
					{
						continue;
					}
					URigVMController* Controller = GetOrCreateController(Graph);
					if (!ensureMsgf(Controller, TEXT("RepairDuplicateVariableGuidsOnLoad: failed to obtain controller for graph '%s'; node patch skipped, asset will be left half-repaired."), *Graph->GetName()))
					{
						continue;
					}

					const TArray<URigVMNode*> Nodes = Graph->GetNodes();
					for (URigVMNode* Node : Nodes)
					{
						URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node);
						if (!VariableNode)
						{
							continue;
						}
						if (VariableNode->IsInputArgument() || VariableNode->IsLocalVariable())
						{
							continue;
						}

						const FGuid NodeGuid = VariableNode->GetVariableGuid();
						const FRepair* const* Found = RepairByOldGuid.Find(NodeGuid);
						if (!Found)
						{
							continue;
						}
						const FRepair& R = **Found;
						if (VariableNode->GetVariableName() != R.VarName)
						{
							continue;
						}

						UE_LOGF(LogRigVMDeveloper, Verbose,
							"  Node: patching '%ls' in graph '%ls' (variable '%ls'): %ls -> %ls",
							*Node->GetName(), *Graph->GetName(), *R.VarName.ToString(),
							*R.OldGuid.ToString(), *R.NewGuid.ToString());
						Controller->RefreshVariableNode(Node->GetFName(), R.NewGuid, R.VarName,
							R.CPPType, R.CPPTypeObject.Get(),
							/*bSetupUndoRedo=*/false, /*bSetupOrphanPins=*/false);
						++ReconciledCount;
					}
				}
			}
		}
	}

	// Pass B: per-graph local-variable duplicate detection + repair. Local variables live on each
	// URigVMGraph and are graph-scoped, so this pass is independent of Pass A and runs even when
	// the bag is absent or healthy. Through the normal URigVMController::AddLocalVariable path
	// local vars always get a fresh Guid, so this is defense in depth (corrupted assets from
	// older engine versions, scripting bypassing the controller, etc.).
	for (URigVMGraph* Graph : AllModels)
	{
		if (!Graph)
		{
			continue;
		}

		TArray<FRepair> LocalRepairs;
		{
			TSet<FGuid> SeenLocalGuids;
			for (const FRigVMGraphVariableDescription& LocalVar : Graph->LocalVariables)
			{
				if (!LocalVar.Guid.IsValid())
				{
					continue;
				}
				bool bAlreadyIn = false;
				SeenLocalGuids.Add(LocalVar.Guid, &bAlreadyIn);
				if (bAlreadyIn)
				{
					LocalRepairs.Add({LocalVar.Name, LocalVar.Guid, FGuid::NewGuid(), INDEX_NONE,
						LocalVar.CPPType, LocalVar.CPPTypeObject});
				}
			}
		}
		if (LocalRepairs.Num() == 0)
		{
			continue;
		}

		URigVMController* Controller = GetOrCreateController(Graph);
		if (!ensureMsgf(Controller, TEXT("RepairDuplicateVariableGuidsOnLoad: failed to obtain controller for graph '%s'; local-var node patch skipped, graph will be left half-repaired."), *Graph->GetName()))
		{
			continue;
		}

		UE_LOGF(LogRigVMDeveloper, Warning,
			"RepairDuplicateVariableGuidsOnLoad: graph '%ls' has %d local variable(s) sharing a Guid. Repairing in place.",
			*Graph->GetName(), LocalRepairs.Num());

		// Step 1: regenerate Guids in-place. Direct array access works here via the
		// IRigVMEditorAssetInterface friendship on URigVMGraph; no remove+add round-trip needed
		// since this isn't a property bag.
		for (const FRepair& R : LocalRepairs)
		{
			for (FRigVMGraphVariableDescription& LocalVar : Graph->LocalVariables)
			{
				if (LocalVar.Name == R.VarName && LocalVar.Guid == R.OldGuid)
				{
					LocalVar.Guid = R.NewGuid;
					++LocalRepairCount;
					UE_LOGF(LogRigVMDeveloper, Verbose,
						"  LocalVar: regenerated Guid for '%ls' in graph '%ls': %ls -> %ls",
						*R.VarName.ToString(), *Graph->GetName(),
						*R.OldGuid.ToString(), *R.NewGuid.ToString());
					break;
				}
			}
		}

		// Build a Guid->Repair lookup for the per-node walk (same shape as Pass A Step 2).
		TMap<FGuid, const FRepair*> LocalRepairByOldGuid;
		LocalRepairByOldGuid.Reserve(LocalRepairs.Num());
		for (const FRepair& R : LocalRepairs)
		{
			LocalRepairByOldGuid.Add(R.OldGuid, &R);
		}

		// Step 2: patch local variable nodes within this graph that pointed at the old (duplicate)
		// Guid for the regenerated variable.
		const TArray<URigVMNode*> Nodes = Graph->GetNodes();
		for (URigVMNode* Node : Nodes)
		{
			URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node);
			if (!VariableNode)
			{
				continue;
			}
			if (!VariableNode->IsLocalVariable())
			{
				continue;
			}

			const FGuid NodeGuid = VariableNode->GetVariableGuid();
			const FRepair* const* Found = LocalRepairByOldGuid.Find(NodeGuid);
			if (!Found)
			{
				continue;
			}
			const FRepair& R = **Found;
			if (VariableNode->GetVariableName() != R.VarName)
			{
				continue;
			}

			UE_LOGF(LogRigVMDeveloper, Verbose,
				"  Local Node: patching '%ls' in graph '%ls' (variable '%ls'): %ls -> %ls",
				*Node->GetName(), *Graph->GetName(), *R.VarName.ToString(),
				*R.OldGuid.ToString(), *R.NewGuid.ToString());
			Controller->RefreshVariableNode(Node->GetFName(), R.NewGuid, R.VarName,
				R.CPPType, R.CPPTypeObject.Get(),
				/*bSetupUndoRedo=*/false, /*bSetupOrphanPins=*/false);
			++LocalNodeReconciledCount;
		}
	}

	if (BagRepairCount > 0 || ReconciledCount > 0 || LocalRepairCount > 0 || LocalNodeReconciledCount > 0)
	{
		bDirtyDuringLoad = true;
		UE_LOGF(LogRigVMDeveloper, Warning,
			"RepairDuplicateVariableGuidsOnLoad: asset-variables { regenerated %d Guid(s), patched %d node(s) }, local-variables { regenerated %d Guid(s), patched %d node(s) }.",
			BagRepairCount, ReconciledCount, LocalRepairCount, LocalNodeReconciledCount);
	}
#endif
}

void IRigVMEditorAssetInterface::PatchLinksWithCast()
{
#if WITH_EDITOR

	{
		TGuardValue<bool> DisableAutoCompile(bAutoRecompileVM, false);

		// find all links containing a cast
		TArray<TTuple<URigVMGraph*,TWeakObjectPtr<URigVMLink>,FString,FString>> LinksWithCast;
		for (URigVMGraph* Graph : GetAllModels())
		{
			for(URigVMLink* Link : Graph->GetLinks())
			{
				const URigVMPin* SourcePin = Link->GetSourcePin();
				const URigVMPin* TargetPin = Link->GetTargetPin();
				if (SourcePin && TargetPin)
				{
					const TRigVMTypeIndex SourceTypeIndex = SourcePin->GetTypeIndex();
					const TRigVMTypeIndex TargetTypeIndex = TargetPin->GetTypeIndex();
					
					if(SourceTypeIndex != TargetTypeIndex)
					{
						if(!FRigVMRegistry::Get().CanMatchTypes(SourceTypeIndex, TargetTypeIndex, true))
						{
							LinksWithCast.Emplace(Graph, TWeakObjectPtr<URigVMLink>(Link), SourcePin->GetPinPath(), TargetPin->GetPinPath());
						}
					}
				}
			}
		}

		// remove all of those links
		for(const auto& Tuple : LinksWithCast)
		{
			URigVMController* Controller = GetController(Tuple.Get<0>());

			if(URigVMLink* Link = Tuple.Get<1>().Get())
			{
				// the link may be detached, attach it first so that removal works.
				const URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();
				if(!SourcePin->IsLinkedTo(TargetPin))
				{
					const TArray<URigVMController::FLinkedPath> LinkedPaths = Controller->GetLinkedPaths({Link});
					Controller->RestoreLinkedPaths(LinkedPaths);
				}
			}

			Controller->BreakLink(Tuple.Get<2>(), Tuple.Get<3>(), false);

			// notify the user that the link has been broken.
			UE_LOGF(LogRigVMDeveloper, Warning,
				"A link was removed in %ls (%ls) - it contained different types on source and target pin (former cast link?).",
				*Controller->GetGraph()->GetNodePath(),
				*URigVMLink::GetPinPathRepresentation(Tuple.Get<2>(), Tuple.Get<3>())
			);
		}
	}
#endif
}

void IRigVMEditorAssetInterface::GetBackwardsCompatibilityPublicFunctions(TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders)
{
	TScriptInterface<IRigVMGraphFunctionHost> CRGeneratedClass = GetRigVMClientHost()->GetRigVMGraphFunctionHost();
	FRigVMGraphFunctionStore* Store = CRGeneratedClass->GetRigVMGraphFunctionStore();
	if (GetObject()->GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RigVMSaveFunctionAccessInModel)
	{
		for (const FRigVMGraphFunctionData& FunctionData : Store->PublicFunctions)
		{
			BackwardsCompatiblePublicFunctions.Add(FunctionData.Header.Name);
			URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionData.Header.LibraryPointer.GetNodeSoftPath().ResolveObject());
			OldHeaders.Add(LibraryNode, FunctionData.Header);
		}
	}

	// Addressing issue where PublicGraphFunctions is populated, but the model PublicFunctionNames is not
	URigVMFunctionLibrary* FunctionLibrary = GetLocalFunctionLibrary();
	if (FunctionLibrary)
	{
		if (GetPublicGraphFunctions().Num() > FunctionLibrary->PublicFunctionNames.Num())
		{
			for (const FRigVMGraphFunctionHeader& PublicHeader : GetPublicGraphFunctions())
			{
				BackwardsCompatiblePublicFunctions.Add(PublicHeader.Name);
			}
		}
	}
}

void IRigVMEditorAssetInterface::PropagateRuntimeSettingsFromBPToInstances()
{
	TArray<UObject*> ArchetypeInstances = GetArchetypeInstances(true, false);
	for (UObject* ArchetypeInstance : ArchetypeInstances)
	{
		URigVMHost* InstanceHost = Cast<URigVMHost>(ArchetypeInstance);
		if (!URigVMHost::IsGarbageOrDestroyed(InstanceHost))
		{
			InstanceHost->VMRuntimeSettings = GetVMRuntimeSettings();
		}
	}

	TArray<UEdGraph*> EdGraphs;
	GetAllEdGraphs(EdGraphs);

	for (UEdGraph* Graph : EdGraphs)
	{
		TArray<UEdGraphNode*> Nodes = Graph->Nodes;
		for (UEdGraphNode* Node : Nodes)
		{
			if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
			{
				RigNode->ReconstructNode_Internal(true);
			}
		}
	}
}

void IRigVMEditorAssetInterface::InitializeArchetypeInstances()
{
	if (URigVM* VM = GetVM(true))
	{
		TArray<UObject*> ArchetypeInstances = GetArchetypeInstances(false, false);
		for (UObject* Instance : ArchetypeInstances)
		{
			URigVMHost* InstanceHost = Cast<URigVMHost>(Instance);
			if (URigVMHost::IsGarbageOrDestroyed(InstanceHost))
			{
				continue;
			}
			if (InstanceHost->HasAllFlags(RF_ClassDefaultObject))
			{
				continue;
			}

			// No objects should be created during load, so PostInitInstanceIfRequired, which creates a new VM and
			// DynamicHierarchy, should not be called during load
			if (!InstanceHost->HasAllFlags(RF_NeedPostLoad))
			{
				InstanceHost->PostInitInstanceIfRequired();
			}
			
			InstanceHost->ResetEvaluationsLeft();
			InstanceHost->InstantiateVMFromCDO();
			InstanceHost->CopyExternalVariableDefaultValuesFromCDO();
		}
	}

	if (URigVMHost* DebuggedHost = GetDebuggedRigVMHost())
	{
		UpdateDebugMemoryOnHost(DebuggedHost);
	}
}

#if WITH_EDITOR

void IRigVMEditorAssetInterface::OnVariableAdded(const FName& InVarName)
{
	FRigVMGraphVariableDescription Variable;
	for (FRigVMGraphVariableDescription& NewVariable : GetAssetVariables())
	{
		if (NewVariable.Name == InVarName)
		{
			Variable = NewVariable;
			break;
		}
	}

	const FRigVMExternalVariable ExternalVariable = RigVMVariableUtils::ExternalVariableFromRigVMVariableDescription(Variable);
    FString CPPType;
    UObject* CPPTypeObject = nullptr;
    RigVMTypeUtils::CPPTypeFromExternalVariable(ExternalVariable, CPPType, &CPPTypeObject);
	if (CPPTypeObject)
	{
		if (ExternalVariable.IsArray())
		{
			CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(CPPTypeObject->GetPathName());
		}
		else
		{
			CPPType = CPPTypeObject->GetPathName();
		}
	}

	// register the type in the registry
	FRigVMRegistry::Get().FindOrAddType(FRigVMTemplateArgumentType(*CPPType, CPPTypeObject));
	
    RigVMPythonUtils::Print(GetObject()->GetFName().ToString(),
		FString::Printf(TEXT("asset.add_member_variable('%s', '%s', %s, %s, '%s')"),
			*InVarName.ToString(),
			*CPPType,
			(ExternalVariable.IsPublic()) ? TEXT("False") : TEXT("True"), 
			(ExternalVariable.IsReadOnly()) ? TEXT("True") : TEXT("False"), 
			*Variable.DefaultValue)); 
	
	BroadcastExternalVariablesChangedEvent();
}

void IRigVMEditorAssetInterface::OnVariableRemoved(const FGuid& InVarGuid)
{
	TArray<URigVMGraph*> AllGraphs = GetAllModels();
	for (URigVMGraph* Graph : AllGraphs)
	{
		if (URigVMController* Controller = GetOrCreateController(Graph))
		{
#if WITH_EDITOR
			const bool bSetupUndoRedo = !GIsTransacting;
#else
			const bool bSetupUndoRedo = false;
#endif
			Controller->OnExternalVariableRemoved(InVarGuid, bSetupUndoRedo);
		}
	}
	
	const FRigVMGraphVariableDescription Variable = FindAssetVariableByGuid(InVarGuid);

	RigVMPythonUtils::Print(GetObject()->GetFName().ToString(),
		FString::Printf(TEXT("asset.remove_member_variable('%s')"),
			*Variable.Name.ToString()));
	
	BroadcastExternalVariablesChangedEvent();
}

void IRigVMEditorAssetInterface::OnVariableRenamed(const FGuid& InVarGuid, const FName& InNewVarName)
{
	TArray<URigVMGraph*> AllGraphs = GetAllModels();
	for (URigVMGraph* Graph : AllGraphs)
	{
		if (URigVMController* Controller = GetOrCreateController(Graph))
		{
#if WITH_EDITOR
			const bool bSetupUndoRedo = !GIsTransacting;
#else
			const bool bSetupUndoRedo = false;
#endif
			Controller->OnExternalVariableRenamed(InVarGuid, InNewVarName, bSetupUndoRedo);
		}
	}

	const FRigVMGraphVariableDescription Variable = FindAssetVariableByGuid(InVarGuid);
	RigVMPythonUtils::Print(GetObject()->GetFName().ToString(),
		FString::Printf(TEXT("asset.rename_member_variable('%s', '%s')"),
			*Variable.Name.ToString(),
			*InNewVarName.ToString()));
	
	BroadcastExternalVariablesChangedEvent();
}

void IRigVMEditorAssetInterface::OnVariableTypeChanged(const FGuid& InVarGuid, FEdGraphPinType InOldPinType, FEdGraphPinType InNewPinType)
{
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromPinType(InNewPinType, CPPType, &CPPTypeObject);
	
	TArray<URigVMGraph*> AllGraphs = GetAllModels();
	for (URigVMGraph* Graph : AllGraphs)
	{
		if (URigVMController* Controller = GetOrCreateController(Graph))
		{
#if WITH_EDITOR
			const bool bSetupUndoRedo = !GIsTransacting;
#else
			const bool bSetupUndoRedo = false;
#endif

			if (!CPPType.IsEmpty())
			{
				Controller->OnExternalVariableTypeChanged(InVarGuid, CPPType, CPPTypeObject, bSetupUndoRedo);
			}
			else
			{
				Controller->OnExternalVariableRemoved(InVarGuid, bSetupUndoRedo);
			}
		}
	}

	if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
	{
		for (auto Var : GetAssetVariables())
		{
			if (Var.Guid == InVarGuid)
			{
				CPPType = ScriptStruct->GetName();
			}
		}
	}
	else if (UEnum* Enum = Cast<UEnum>(CPPTypeObject))
	{
		for (auto Var : GetAssetVariables())
		{
			if (Var.Guid == InVarGuid)
			{
				CPPType = Enum->GetName();
			}
		}
	}

	// register the type in the registry
	FRigVMRegistry::Get().FindOrAddType(FRigVMTemplateArgumentType(*CPPType, CPPTypeObject));
	
	const FRigVMGraphVariableDescription Variable = FindAssetVariableByGuid(InVarGuid);

	RigVMPythonUtils::Print(GetObject()->GetFName().ToString(),
		FString::Printf(TEXT("asset.change_member_variable_type('%s', '%s')"),
		*Variable.Name.ToString(),
		*CPPType));

	BroadcastExternalVariablesChangedEvent();
}

FName IRigVMEditorAssetInterface::AddAssetVariableFromPinType(const FName& InName, const FEdGraphPinType& InType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromPinType(InType, CPPType, &CPPTypeObject);
	return AddMemberVariable(InName, CPPTypeObject ? CPPTypeObject->GetPathName() : CPPType, bIsPublic, bIsReadOnly, InDefaultValue);
}

void IRigVMEditorAssetInterface::BroadcastExternalVariablesChangedEvent()
{
	ExternalVariablesChangedEvent.Broadcast(GetExternalVariables(false));
}

void IRigVMEditorAssetInterface::BroadcastNodeDoubleClicked(URigVMNode* InNode)
{
	NodeDoubleClickedEvent.Broadcast(GetObject(), InNode);
}

void IRigVMEditorAssetInterface::BroadcastGraphImported(UEdGraph* InGraph)
{
	GraphImportedEvent.Broadcast(InGraph);
}

void IRigVMEditorAssetInterface::BroadcastPostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	PostEditChangeChainPropertyEvent.Broadcast(PropertyChangedChainEvent);
}

void IRigVMEditorAssetInterface::SetProfilingEnabled(const bool bEnabled)
{
	GetVMRuntimeSettings().bEnableProfiling = bEnabled;
	PropagateRuntimeSettingsFromBPToInstances();
	RequestAutoVMRecompilation();
}

void IRigVMEditorAssetInterface::RefreshAllNodes()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
#if WITH_EDITORONLY_DATA
	// Avoid refreshing EdGraph nodes during cook
	if (GIsCookerLoadingPackage)
	{
		return;
	}
	
	// The only time we need to refresh EdGraph is when this asset is being opened in an editor,
	// which can only happen on Game thread. We need a check here because this function can be called from
	// AsyncLoadingThread while this asset itself is going through PostLoad() on GameThread, 
	// if this asset is a BP asset and a dependency of another BP that is getting async loaded, such as an AnimBP, 
	if (!IsInGameThread())
	{
		return;
	}
	
	// Avoid refreshing EdGraph if PostLoad() hasn't been called, since reconstruct node later
	// can access model data that hasn't been fully loaded. And it is ok to skip here because
	// the EdGraph will be reconstructed later when the CR editor
	// initializes, as that is when the EdGraph is actually used.
	if (GetObject()->HasAnyFlags(RF_NeedPostLoad))
	{
		return;
	}
	
	// Same goes for any referenced function host
	for (TScriptInterface<IRigVMGraphFunctionHost> ReferencedFunctionHost : GetReferencedFunctionHosts(false))
	{
		if (ReferencedFunctionHost.GetObject()->HasAnyFlags(RF_NeedPostLoad))
		{
			return;
		}
	}
	
	if (GetDefaultModel() == nullptr)
	{
		return;
	}

	TArray<URigVMEdGraphNode*> AllNodes;
	TArray<UEdGraph*> AllGraphs;
	GetAllEdGraphs(AllGraphs);
	for(int32 i=0; i<AllGraphs.Num(); i++)
	{
		check(AllGraphs[i] != NULL);
		TArray<URigVMEdGraphNode*> GraphNodes;
		AllGraphs[i]->GetNodesOfClass<URigVMEdGraphNode>(GraphNodes);
		AllNodes.Append(GraphNodes);
	}

	for (URigVMEdGraphNode* Node : AllNodes)
	{
		Node->SetFlags(RF_Transient);
	}

	for(URigVMEdGraphNode* Node : AllNodes)
	{
		Node->ReconstructNode();
	}

	for (URigVMEdGraphNode* Node : AllNodes)
	{
		Node->ClearFlags(RF_Transient);
	}
	
#endif
}

void IRigVMEditorAssetInterface::BroadcastRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier InFunction, bool bForce)
{
	RequestLocalizeFunctionDialog.Broadcast(InFunction, GetController(GetDefaultModel()), GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetInterface(), bForce);
}

const FCompilerResultsLog& IRigVMEditorAssetInterface::GetCompileLog() const
{
	return CompileLog;
}

FCompilerResultsLog& IRigVMEditorAssetInterface::GetCompileLog()
{
	return CompileLog;
}

void IRigVMEditorAssetInterface::BroadCastReportCompilerMessage(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
{
	ReportCompilerMessageEvent.Broadcast(InSeverity, InSubject, InMessage);
}

#endif

UEdGraph* IRigVMEditorAssetInterface::CreateEdGraph(URigVMGraph* InModel, bool bForce)
{
	check(InModel);

#if WITH_EDITORONLY_DATA
	if(InModel->IsA<URigVMFunctionLibrary>())
	{
		return GetFunctionLibraryEdGraph();
	}
#endif
	
	if(bForce)
	{
		RemoveEdGraph(InModel);
	}

	FString GraphName = InModel->GetName();
	GraphName.RemoveFromStart(FRigVMClient::RigVMModelPrefix);
	GraphName.TrimStartAndEndInline();

	if(GraphName.IsEmpty())
	{
		GraphName = URigVMEdGraphSchema::GraphName_RigVM.ToString();
	}

	GraphName = GetRigVMClient()->GetUniqueName(*GraphName).ToString();

	URigVMEdGraph* RigVMEdGraph = NewObject<URigVMEdGraph>(GetObject(), GetRigVMClientHost()->GetRigVMEdGraphClass(), *GraphName, RF_Transactional);
	RigVMEdGraph->Schema = GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
	RigVMEdGraph->bAllowDeletion = true;
	RigVMEdGraph->ModelNodePath = InModel->GetNodePath();
	RigVMEdGraph->InitializeFromAsset(GetObject());
	
	AddUbergraphPage(RigVMEdGraph);
	AddLastEditedDocument(RigVMEdGraph);

	return RigVMEdGraph;
}

bool IRigVMEditorAssetInterface::RemoveEdGraph(URigVMGraph* InModel)
{
	if(URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetEdGraph(InModel)))
	{
		TArray<TObjectPtr<UEdGraph>>& UbergraphPages = GetUberGraphs();
		if(UbergraphPages.Contains(RigGraph))
		{
			GetObject()->Modify();
			UbergraphPages.Remove(RigGraph);
		}
		DestroyObject(RigGraph);
		return true;
	}
	return false;
}

void IRigVMEditorAssetInterface::DestroyObject(UObject* InObject)
{
	GetRigVMClient()->DestroyObject(InObject);
}

void IRigVMEditorAssetInterface::RenameGraph(const FString& InNodePath, const FName& InNewName)
{
	FName OldName = NAME_None;
	UEdGraph* EdGraph = GetEdGraph(InNodePath);
	if(EdGraph)
	{
		OldName = EdGraph->GetFName();
	}
	
	GetRigVMClient()->RenameModel(InNodePath, InNewName, true);

	if(EdGraph)
	{
		NotifyGraphRenamedSuper(EdGraph, OldName, EdGraph->GetFName());
	}
}

void IRigVMEditorAssetInterface::CreateEdGraphForCollapseNodeIfNeeded(URigVMCollapseNode* InNode, bool bForce)
{
	check(InNode);

	if (bForce)
	{
		RemoveEdGraphForCollapseNode(InNode, false);
	}

	if (InNode->GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			bool bFunctionGraphExists = false;
			for (UEdGraph* FunctionGraph : GetFunctionGraphs())
			{
				if (URigVMEdGraph* RigFunctionGraph = Cast<URigVMEdGraph>(FunctionGraph))
				{
					if (RigFunctionGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						bFunctionGraphExists = true;
						break;
					}
				}
			}

			// If not tracked in the transient FunctionGraphs array (which is wiped on
			// load and cleared during rebuilds), reuse an orphaned-but-valid instance already parented
			// to this asset instead of spawning a duplicate. Duplicate function EdGraph instances
			// desynced the editor's graph tab / SGraphPanel subscription from the instance that live
			// edits actually mutate, so a node added after reopen never got a widget. Persistent
			// UBlueprint-backed rigs don't hit this (their FunctionGraphs are serialized); this restores
			// the same single-stable-instance behaviour for Blueprint-independent rigs.
			if (!bFunctionGraphExists)
			{
				URigVMEdGraph* ReusedGraph = nullptr;
				ForEachObjectWithOuter(GetObject(),
					[&ReusedGraph, &ContainedGraph](UObject* InObject)
					{
						if (ReusedGraph != nullptr)
						{
							return;
						}
						if (URigVMEdGraph* RigFunctionGraph = Cast<URigVMEdGraph>(InObject))
						{
							if (RigFunctionGraph->bIsFunctionDefinition
								&& IsValid(RigFunctionGraph)
								&& RigFunctionGraph->ModelNodePath == ContainedGraph->GetNodePath())
							{
								ReusedGraph = RigFunctionGraph;
							}
						}
					});

				if (ReusedGraph != nullptr)
				{
					bFunctionGraphExists = true;
					GetFunctionGraphs().AddUnique(ReusedGraph);
					ReusedGraph->InitializeFromAsset(GetObject());
					GetOrCreateController(ContainedGraph)->ResendAllNotifications();
				}
			}

			if (!bFunctionGraphExists)
			{
				// create a sub graph
				URigVMEdGraph* RigFunctionGraph = NewObject<URigVMEdGraph>(GetObject(), GetRigVMClientHost()->GetRigVMEdGraphClass(), *InNode->GetName(), RF_Transactional);
				RigFunctionGraph->Schema = GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
				RigFunctionGraph->bAllowRenaming = 1;
				RigFunctionGraph->bEditable = 1;
				RigFunctionGraph->bAllowDeletion = 1;
				RigFunctionGraph->ModelNodePath = ContainedGraph->GetNodePath();
				RigFunctionGraph->bIsFunctionDefinition = true;

				GetFunctionGraphs().Add(RigFunctionGraph);

				RigFunctionGraph->InitializeFromAsset(GetObject());

				GetOrCreateController(ContainedGraph)->ResendAllNotifications();
			}

		}
	}
	else if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetEdGraph(InNode->GetGraph())))
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			bool bSubGraphExists = false;
			for (UEdGraph* SubGraph : RigGraph->SubGraphs)
			{
				if (URigVMEdGraph* SubRigGraph = Cast<URigVMEdGraph>(SubGraph))
				{
					if (SubRigGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						bSubGraphExists = true;
						break;
					}
				}
			}

			if (!bSubGraphExists)
			{
				bool bEditable = true;
				if (InNode->IsA<URigVMAggregateNode>())
				{
					bEditable = false;
				}
				
				// create a sub graph
				URigVMEdGraph* SubRigGraph = NewObject<URigVMEdGraph>(RigGraph, GetRigVMClientHost()->GetRigVMEdGraphClass(), *InNode->GetEditorSubGraphName(), RF_Transactional);
				SubRigGraph->Schema = GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
				SubRigGraph->bAllowRenaming = 1;
				SubRigGraph->bEditable = bEditable;
				SubRigGraph->bAllowDeletion = 1;
				SubRigGraph->ModelNodePath = ContainedGraph->GetNodePath();
				SubRigGraph->bIsFunctionDefinition = false;

				RigGraph->SubGraphs.Add(SubRigGraph);

				SubRigGraph->InitializeFromAsset(GetObject());

				GetOrCreateController(ContainedGraph)->ResendAllNotifications();
			}
		}
	}
}

bool IRigVMEditorAssetInterface::RemoveEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bNotify)
{
	check(InNode);

	if (InNode->GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			for (UEdGraph* FunctionGraph : GetFunctionGraphs())
			{
				if (URigVMEdGraph* RigFunctionGraph = Cast<URigVMEdGraph>(FunctionGraph))
				{
					if (RigFunctionGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						if (URigVMController* SubController = GetController(ContainedGraph))
						{
							SubController->OnModified().RemoveAll(RigFunctionGraph);
						}

						if (OnModified().IsBound() && bNotify)
						{
							OnModified().Broadcast(ERigVMGraphNotifType::NodeRemoved, InNode->GetGraph(), InNode);
						}

						GetFunctionGraphs().Remove(RigFunctionGraph);
						RigFunctionGraph->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
						if(RigFunctionGraph->IsRooted())
						{
							RigFunctionGraph->RemoveFromRoot();
						}
						RigFunctionGraph->MarkAsGarbage();
						return bNotify;
					}
				}
			}
		}
	}
	else if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GetEdGraph(InNode->GetGraph())))
	{
		if (URigVMGraph* ContainedGraph = InNode->GetContainedGraph())
		{
			for (UEdGraph* SubGraph : RigGraph->SubGraphs)
			{
				if (URigVMEdGraph* SubRigGraph = Cast<URigVMEdGraph>(SubGraph))
				{
					if (SubRigGraph->ModelNodePath == ContainedGraph->GetNodePath())
					{
						if (URigVMController* SubController = GetController(ContainedGraph))
						{
							SubController->OnModified().RemoveAll(SubRigGraph);
						}

						if (OnModified().IsBound() && bNotify)
						{
							OnModified().Broadcast(ERigVMGraphNotifType::NodeRemoved, InNode->GetGraph(), InNode);
						}

						RigGraph->SubGraphs.Remove(SubRigGraph);
						SubRigGraph->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
						if(SubRigGraph->IsRooted())
						{
							SubRigGraph->RemoveFromRoot();
						}
						SubRigGraph->MarkAsGarbage();
						return bNotify;
					}
				}
			}
		}
	}

	return false;
}

#if WITH_EDITOR

void IRigVMEditorAssetInterface::QueueCompilerMessageDelegate(const FOnRigVMReportCompilerMessage::FDelegate& InDelegate)
{
	FScopeLock Lock(&QueuedCompilerMessageDelegatesMutex);
	QueuedCompilerMessageDelegates.Add(InDelegate);
}

void IRigVMEditorAssetInterface::ClearQueuedCompilerMessageDelegates()
{
	FScopeLock Lock(&QueuedCompilerMessageDelegatesMutex);
	QueuedCompilerMessageDelegates.Reset();
}

#endif

URigVMEditorAsset::URigVMEditorAsset()
	: IRigVMEditorAssetInterface()
{
	CurrentMessageLog = MakeUnique<FCompilerResultsLog>();
	CurrentMessageLog->SetSourcePath(GetPathName());
}

URigVMRuntimeAsset* URigVMEditorAsset::GetRuntimeAsset() const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeInterface = GetRuntimeAssetInterface())
	{
		return Cast<URigVMRuntimeAsset>(RuntimeInterface.GetObject());
	}
	return nullptr;
}

void URigVMEditorAsset::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	IRigVMEditorAssetInterface::PreSave(SaveContext);
}

void URigVMEditorAsset::PostLoad()
{
	Super::PostLoad();
	return IRigVMEditorAssetInterface::PostLoad();
}

void URigVMEditorAsset::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	Super::PostTransacted(TransactionEvent);
	return IRigVMEditorAssetInterface::PostTransacted(TransactionEvent);
}

void URigVMEditorAsset::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);
	return IRigVMEditorAssetInterface::PreDuplicate(DupParams);
}

void URigVMEditorAsset::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	return IRigVMEditorAssetInterface::PostDuplicate(bDuplicateForPIE);
}

void URigVMEditorAsset::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// For BP-independent assets, the RuntimeAsset stores the correct old/new paths
	// before calling EditorAsset->Rename(). We use those paths here because by this point
	// OldOuter (the RuntimeAsset) has already been moved to its new location, so we can't
	// reconstruct the old package path from it.
	URigVMRuntimeAsset* RuntimeAsset = Cast<URigVMRuntimeAsset>(GetOuter());
	check(RuntimeAsset);

	const FString& OldAssetPath = RuntimeAsset->PreRenameOldAssetPath;
	const FString& NewAssetPath = RuntimeAsset->PreRenameNewAssetPath;

	if (OldAssetPath.IsEmpty() || NewAssetPath.IsEmpty())
	{
		UE_LOG(LogRigVMDeveloper, Warning,
			TEXT("URigVMEditorAsset::PostRename called without pre-rename paths set. "
				 "Function identifiers will not be updated. This is expected only if "
				 "the rename was not initiated through URigVMRuntimeAsset."));
		return;
	}

	// Function identifier replacement using full object paths
	// - Function host path: replace RuntimeAsset path
	// - Function library path: replace RuntimeAsset path AND EditorAsset name (subobject after the colon)
	const FString NewFunctionHostPath = FSoftObjectPath(GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetObject()).ToString();
	const FString NewFunctionLibraryPath = FSoftObjectPath(GetRigVMClient()->GetFunctionLibrary()).ToString();
	const FString OldFunctionHostPath = NewFunctionHostPath.Replace(*NewAssetPath, *OldAssetPath);
	const FString OldFunctionLibraryPath = NewFunctionLibraryPath
		.Replace(*NewAssetPath, *OldAssetPath)
		.Replace(*GetFName().ToString(), *OldName.ToString());
	ReplaceFunctionIdentifiers(OldFunctionHostPath, NewFunctionHostPath, OldFunctionLibraryPath, NewFunctionLibraryPath);
}

void URigVMEditorAsset::BeginDestroy()
{
	Super::BeginDestroy();
	IRigVMEditorAssetInterface::BeginDestroy();
}

URigVMEditorAsset::URigVMEditorAsset(const FObjectInitializer& ObjectInitializer)
	: IRigVMEditorAssetInterface(ObjectInitializer)
{
	if(GetClass() == URigVMEditorAsset::StaticClass())
	{
		CommonInitialization(ObjectInitializer);
	}
}

void URigVMEditorAsset::GetAllEdGraphs(TArray<UEdGraph*>& Graphs) const
{
	Graphs.Reset();
	Graphs.Reserve(RootEdGraphs.Num() + FunctionGraphs.Num());
	for (int32 i = 0; i < RootEdGraphs.Num(); ++i)
	{
		UEdGraph* Graph = RootEdGraphs[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}

	for (int32 i = 0; i < FunctionGraphs.Num(); ++i)
	{
		UEdGraph* Graph = FunctionGraphs[i];
		if(Graph)
		{
			Graphs.Add(Graph);
			Graph->GetAllChildrenGraphs(Graphs);
		}
	}
}

TArray<FRigVMGraphVariableDescription> URigVMEditorAsset::GetAssetVariables() const
{
	TArray<FRigVMGraphVariableDescription> Variables;
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetAssetVariables();
	}
	return Variables;
}

FOnRigVMPreVariablesChanged& URigVMEditorAsset::OnPreVariablesChanged()
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->OnPreVariablesChanged();
	}
	return DummyOnPreVariableChanged;
}

FOnRigVMPostVariablesChanged& URigVMEditorAsset::OnPostVariablesChanged()
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->OnPostVariablesChanged();
	}
	return DummyOnPostVariableChanged;
}

FProperty* URigVMEditorAsset::FindGeneratedPropertyByName(const FName& InName)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->FindGeneratedPropertyByName(InName);
	}
	return nullptr;
}

FName URigVMEditorAsset::AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue)
{
	FName Result = NAME_None;
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		Result = RuntimeAsset->AddHostMemberVariableFromExternal(InVariableToCreate, InDefaultValue);
		if (!Result.IsNone())
		{
			OnVariableAdded(Result);
		}
	}
	return Result;
}

FName URigVMEditorAsset::AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	FRigVMExternalVariable Variable = RigVMTypeUtils::ExternalVariableFromCPPTypePath(FGuid::NewGuid(), InName, InCPPType, bIsPublic, bIsReadOnly);
	FName Result = AddHostMemberVariableFromExternal(Variable, InDefaultValue);
	return Result;
}

bool URigVMEditorAsset::RemoveMemberVariable(const FName& InName)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		const FRigVMGraphVariableDescription Variable = RuntimeAsset->FindAssetVariable(InName);
		if (RuntimeAsset->RemoveMemberVariable(InName))
		{
			OnVariableRemoved(Variable.Guid);
			return true;
		}
	}
	return false;
}

bool URigVMEditorAsset::BulkRemoveMemberVariables(const TArray<FName>& InNames)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		TMap<FName, FGuid> NameToGuid;
		for (const FName& Name : InNames)
		{
			const FRigVMGraphVariableDescription Variable = RuntimeAsset->FindAssetVariable(Name);
			NameToGuid.Add(Name, Variable.Guid);
		}
		if (RuntimeAsset->BulkRemoveMemberVariables(InNames))
		{
			for (const FName& Name : InNames)
			{
				OnVariableRemoved(NameToGuid.FindChecked(Name));
			}
			return true;
		}
	}
	return false;
}

bool URigVMEditorAsset::RenameMemberVariable(const FName& InOldName, const FName& InNewName)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		if (RuntimeAsset->RenameMemberVariable(InOldName, InNewName))
		{
			const FRigVMGraphVariableDescription Variable = RuntimeAsset->FindAssetVariable(InNewName);
			OnVariableRenamed(Variable.Guid, InNewName);
			return true;
		}
	}
	return false;
}

bool URigVMEditorAsset::ChangeMemberVariableType(const FName& InName, const FEdGraphPinType& InType)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		FString CPPType;
		UObject* CPPTypeObject = nullptr;
		RigVMTypeUtils::CPPTypeFromPinType(InType, CPPType, &CPPTypeObject);
		FRigVMGraphVariableDescription Variable = RuntimeAsset->FindAssetVariable(InName);
		return ChangeMemberVariableType(InName, CPPType, Variable.bPublic, Variable.bPrivate, Variable.DefaultValue);
	}
	return false;
}

bool URigVMEditorAsset::ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		FRigVMGraphVariableDescription OldVariable = RuntimeAsset->FindAssetVariable(InName);
		FEdGraphPinType OldPinType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(OldVariable);
		if (RuntimeAsset->ChangeMemberVariableType(InName, InCPPType, bIsPublic, bIsReadOnly, InDefaultValue))
		{
			FRigVMGraphVariableDescription NewVariable = RuntimeAsset->FindAssetVariable(InName);
			FEdGraphPinType NewPinType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(NewVariable);
			OnVariableTypeChanged(OldVariable.Guid, OldPinType, NewPinType);
			return true;
		}
	}
	return false;
}

bool URigVMEditorAsset::SetVariableIndex(const FName& InName, int32 NewIndex)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableIndex(InName, NewIndex);
	}
	return false;
}

bool URigVMEditorAsset::SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableIndex(InVariableGuid, NewIndex);
	}
	return false;
}

FText URigVMEditorAsset::GetVariableTooltip(const FName& InName) const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetVariableTooltip(InName);
	}
	return FText();
}

FString URigVMEditorAsset::GetVariableCategory(const FName& InName)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetVariableCategory(InName);
	}
	return FString();
}

FString URigVMEditorAsset::GetVariableMetadataValue(const FName& InName, const FName& InKey)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetVariableMetadataValue(InName, InKey);
	}
	return FString();
}

bool URigVMEditorAsset::SetVariableCategory(const FName& InName, const FString& InCategory)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableCategory(InName, InCategory);
	}
	return false;
}

bool URigVMEditorAsset::SetVariableTooltip(const FName& InName, const FText& InTooltip)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableTooltip(InName, InTooltip);
	}
	return false;
}

bool URigVMEditorAsset::SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableExposeOnSpawn(InName, bInExposeOnSpawn);
	}
	return false;
}

bool URigVMEditorAsset::SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariableExposeToCinematics(InName, bInExposeToCinematics);
	}
	return false;
}

bool URigVMEditorAsset::SetVariablePrivate(const FName& InName, const bool bInPrivate)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariablePrivate(InName, bInPrivate);
	}
	return false;
}

bool URigVMEditorAsset::SetVariablePublic(const FName& InName, const bool bIsPublic)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetVariablePublic(InName, bIsPublic);
	}
	return false;
}

FString URigVMEditorAsset::OnCopyVariable(const FName& InName) const
{
	FString OutputString;
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		FRigVMGraphVariableDescription Variable = RuntimeAsset->FindAssetVariable(InName);
		if (!Variable.Name.IsNone())
		{
			FRigVMGraphVariableDescription::StaticStruct()->ExportText(OutputString, &Variable, &Variable, nullptr, 0, nullptr, false);
			OutputString = TEXT("RigVMVar") + OutputString;
		}
	}

	return OutputString;
}

bool URigVMEditorAsset::OnPasteVariable(const FString& InText)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		bool bIsRigVMVar = InText.StartsWith(TEXT("RigVMVar"), ESearchCase::CaseSensitive);
		bool bIsBPVar = !bIsRigVMVar && InText.StartsWith(TEXT("BPVar"), ESearchCase::CaseSensitive);

		if (!ensure(bIsRigVMVar || bIsBPVar))
		{
			return false;
		}

		if (bIsRigVMVar)
		{
			FRigVMGraphVariableDescription NewVar;
			FStringOutputDevice Errors;
			const TCHAR* Import = InText.GetCharArray().GetData() + FCString::Strlen(TEXT("RigVMVar"));
			FRigVMGraphVariableDescription::StaticStruct()->ImportText(Import, &NewVar, nullptr, PPF_None, &Errors, FBPVariableDescription::StaticStruct()->GetName());
			if (Errors.IsEmpty())
			{
				FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteVariable", "Paste Variable: {0}"), FText::FromName(NewVar.Name)));

				NewVar.Name = FRigVMBlueprintUtils::FindUniqueVariableName(this, NewVar.Name.ToString());
				NewVar.Guid = FGuid::NewGuid();
				FRigVMExternalVariable ExternalVariable = RigVMVariableUtils::ExternalVariableFromRigVMVariableDescription(NewVar);
				FName Result = RuntimeAsset->AddHostMemberVariableFromExternal(ExternalVariable, NewVar.DefaultValue);

				return !Result.IsNone();
			}
		}
		
		if (bIsBPVar)
		{
			FBPVariableDescription Description;
			FStringOutputDevice Errors;
			const TCHAR* Import = InText.GetCharArray().GetData() + FCString::Strlen(TEXT("BPVar"));
			FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, PPF_None, &Errors, FBPVariableDescription::StaticStruct()->GetName());
			if (Errors.IsEmpty())
			{
				FScopedTransaction Transaction(FText::Format(LOCTEXT("PasteVariable", "Paste Variable: {0}"), FText::FromName(Description.VarName)));

				Description.VarName = FRigVMBlueprintUtils::FindUniqueVariableName(this, Description.VarName.ToString());
				Description.VarGuid = FGuid::NewGuid();
				FRigVMExternalVariable ExternalVariable = RigVMTypeUtils::ExternalVariableFromBPVariableDescription(Description);
				ExternalVariable.SetName(FRigVMBlueprintUtils::FindUniqueVariableName(this, ExternalVariable.GetName().ToString()));
				FName Result = RuntimeAsset->AddHostMemberVariableFromExternal(ExternalVariable, Description.DefaultValue);

				return !Result.IsNone();
			}
		}
	}
	return false;
}

UObject* URigVMEditorAsset::GetObjectBeingDebugged(bool bEvenIfPendingKill)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetObjectBeingDebugged(bEvenIfPendingKill);
	}
	return nullptr;
}

const FString& URigVMEditorAsset::GetObjectBeingDebuggedPath() const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetObjectBeingDebuggedPath();
	}
	static FString EmptyPath;
	return EmptyPath;
}

UWorld* URigVMEditorAsset::GetWorldBeingDebugged() const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetWorldBeingDebugged();
	}
	return nullptr;
}

void URigVMEditorAsset::SetWorldBeingDebugged(UWorld* NewWorld)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetWorldBeingDebugged(NewWorld);
	}
}

TArray<UObject*> URigVMEditorAsset::GetArchetypeInstances(bool bIncludeCDO, bool bIncludeDerivedClass) const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetArchetypeInstances(bIncludeDerivedClass);
	}
	static TArray<UObject*> EmptyArray = TArray<UObject*>();
    return EmptyArray;
}

ERigVMAssetStatus URigVMEditorAsset::GetAssetStatus() const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetAssetStatus();
	}
	return ERigVMAssetStatus::RVMA_Unknown;
}

void URigVMEditorAsset::SetAssetStatus(const ERigVMAssetStatus& InStatus)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetAssetStatus(InStatus);
	}
}

bool URigVMEditorAsset::IsUpToDate() const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->IsUpToDate();
	}
	return false;
}

URigVM* URigVMEditorAsset::GetVM(bool bCreateIfNeeded) const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetVM();
	}
	return nullptr;
}

FRigVMRuntimeSettings& URigVMEditorAsset::GetVMRuntimeSettings()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetVMRuntimeSettings();
	}
	static FRigVMRuntimeSettings EmptyVMRuntimeSettings = FRigVMRuntimeSettings();
	return EmptyVMRuntimeSettings;
}

void URigVMEditorAsset::AddPinWatch(UEdGraphPin* InPin)
{
	WatchedPin.AddUnique(InPin);
}

void URigVMEditorAsset::RemovePinWatch(UEdGraphPin* InPin)
{
	WatchedPin.Remove(InPin);
}

bool URigVMEditorAsset::IsPinBeingWatched(const UEdGraphPin* InPin) const
{
	return WatchedPin.Contains(InPin);
}

bool URigVMEditorAsset::NodeContainsWatchedPins(const UEdGraphNode* InNode) const
{
	for (UEdGraphPin* Pin : WatchedPin)
	{
		if (Pin->GetOuter() == InNode)
		{
			return true;
		}
	}
	return false;
}

void URigVMEditorAsset::ClearPinWatches()
{
	WatchedPin.Empty();
}

void URigVMEditorAsset::ForeachPinWatch(TFunctionRef<void(UEdGraphPin*)> Task)
{
	for (UEdGraphPin* Pin : WatchedPin)
	{
		Task(Pin);
	}
}

void URigVMEditorAsset::MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus)
{
	//RecompileVM(); // Not sure if we need to recompile, or increment compilation bracket, or just MarkAssetAsModified
	
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->MarkAssetAsStructurallyModified(bSkipDirtyAssetStatus);
	}
}

void URigVMEditorAsset::MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->MarkAssetAsModified(PropertyChangedEvent);
	}
}

FRigVMVariant& URigVMEditorAsset::GetAssetVariant()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetAssetVariant();
	}
	static FRigVMVariant EmptyVariant = FRigVMVariant();
	return EmptyVariant;
}

FRigVMVariantRef URigVMEditorAsset::GetAssetVariantRef() const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return FRigVMVariantRef(FSoftObjectPath(GetRuntimeAsset()), RuntimeAsset->GetAssetVariant());
	}
	return IRigVMEditorAssetInterface::GetAssetVariantRefImpl();
}

const TArray<FRigVMGraphFunctionHeader>& URigVMEditorAsset::GetPublicGraphFunctions() const
{
	if (const URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetPublicGraphFunctions();
	}
	static TArray<FRigVMGraphFunctionHeader> EmptyArray = TArray<FRigVMGraphFunctionHeader>();
	return EmptyArray;
}

void URigVMEditorAsset::SetPublicGraphFunctions(const TArray<FRigVMGraphFunctionHeader>& InHeaders) 
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->SetPublicGraphFunctions(InHeaders);
	}
}

void URigVMEditorAsset::DebuggingWorldRegistrationHelper(UObject* ObjectProvidingWorld, UObject* ValueToRegister)
{
	if (ObjectProvidingWorld != NULL)
	{
		// Fix up the registration with the world
		UWorld* ObjWorld = NULL;
		UObject* ObjOuter = ObjectProvidingWorld->GetOuter();
		while (ObjOuter != NULL)
		{
			ObjWorld = Cast<UWorld>(ObjOuter);
			if (ObjWorld != NULL)
			{
				break;
			}

			ObjOuter = ObjOuter->GetOuter();
		}

		// if we can't find the world on the outer chain, fallback to the GetWorld method
		if (ObjWorld == NULL)
		{
			ObjWorld = ObjectProvidingWorld->GetWorld();
		}

		//if (ObjWorld != NULL)
		{
			// if( !ObjWorld->HasAnyFlags(RF_BeginDestroyed))
			// {
			// 	ObjWorld->NotifyOfBlueprintDebuggingAssociation(this, ValueToRegister);
			// }
			OnSetObjectBeingDebugged().Broadcast(ValueToRegister);
		}
	}
}

void URigVMEditorAsset::SetObjectBeingDebuggedSuper(UObject* NewObject)
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		// Unregister the old object (even if PendingKill)
		if (UObject* OldObject = RuntimeAsset->GetObjectBeingDebugged(true))
		{
			if (OldObject == NewObject)
			{
				// Nothing changed
				return;
			}

			DebuggingWorldRegistrationHelper(OldObject, nullptr);
		}

		// Update the current object being debugged
		RuntimeAsset->SetObjectBeingDebugged(NewObject);

		// Register the new object
		if (NewObject != nullptr)
		{
			RuntimeAsset->SetObjectBeingDebuggedPath(NewObject->GetPathName());
			DebuggingWorldRegistrationHelper(NewObject, NewObject);
		}
		else
		{
			RuntimeAsset->SetObjectBeingDebuggedPath(FString());
		}
	}
}

void URigVMEditorAsset::PostDuplicateSuper(bool bDuplicateForPIE)
{
	UObject::PostDuplicate(bDuplicateForPIE);
	RecompileVM();
}

TArray<FRigVMExternalVariable> URigVMEditorAsset::GetExternalVariables(bool bFallbackToBlueprint) const
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetExternalVariables();
	}
	return TArray<FRigVMExternalVariable>();
}

const UStruct* URigVMEditorAsset::GetVariablesStruct()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetVariablesStruct();
	}
	return nullptr;
}

uint8* URigVMEditorAsset::GetVariablesMemory()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetVariablesMemory();
	}
	return nullptr;
}

FString URigVMEditorAsset::GetVariableDefaultValue(const FName& InName, bool bFromDebuggedObject) const
{
	if (bFromDebuggedObject)
	{
		if (UObject* Debugged = GetObjectBeingDebugged())
		{
			if (URigVMHost* DebuggedHost = Cast<URigVMHost>(Debugged))
			{
				return DebuggedHost->Variables.GetDataAsStringByName(InName);
			}
		}
	}
	else if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetVariableDefaultValue(InName);
	}
	return FString();
}

FRigVMExtendedExecuteContext* URigVMEditorAsset::GetRigVMExtendedExecuteContext()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetRigVMExtendedExecuteContext();
	}
	return nullptr;
}

TMap<FString, FSoftObjectPath>& URigVMEditorAsset::GetUserDefinedStructGuidToPathName(bool bFromCDO)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetUserDefinedStructGuidToPathName();
	}
	static TMap<FString, FSoftObjectPath> EmptyArray = TMap<FString, FSoftObjectPath>();
	return EmptyArray;
}

TMap<FString, FSoftObjectPath>& URigVMEditorAsset::GetUserDefinedEnumToPathName(bool bFromCDO)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetUserDefinedEnumToPathName();
	}
	static TMap<FString, FSoftObjectPath> EmptyArray = TMap<FString, FSoftObjectPath>();
	return EmptyArray;
}

TSet<TObjectPtr<UObject>>& URigVMEditorAsset::GetUserDefinedTypesInUse(bool bFromCDO)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetUserDefinedTypesInUse();
	}
	static TSet<TObjectPtr<UObject>> EmptyArray = TSet<TObjectPtr<UObject>>();
	return EmptyArray;
}

FRigVMDebugInfo& URigVMEditorAsset::GetDebugInfo()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetDebugInfo();
	}
	static FRigVMDebugInfo Empty = FRigVMDebugInfo();
	return Empty;
}

TArray<FName>& URigVMEditorAsset::GetSupportedEventNames()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetSupportedEventNames();
	}
	static TArray<FName> EmptyArray = TArray<FName>();
	return EmptyArray;
}

void URigVMEditorAsset::UpdateSupportedEventNames()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->UpdateSupportedEventNames();
	}
}

TArray<FRigVMReferenceNodeData>& URigVMEditorAsset::GetFunctionReferenceNodeData()
{
	if (URigVMRuntimeAsset* RuntimeAsset = Cast<URigVMRuntimeAsset>(GetRuntimeAssetInterface().GetObject()))
	{
		return RuntimeAsset->FunctionReferenceNodeData;
	}
	
	static TArray<FRigVMReferenceNodeData> EmptyArray = TArray<FRigVMReferenceNodeData>();
	return EmptyArray;
}

URigVMHost* URigVMEditorAsset::CreateRigVMHostSuper(UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->InstantiateObject(InOuter, InName, InFlags);
	}
	return nullptr;
}

void URigVMEditorAsset::AddUbergraphPage(URigVMEdGraph* RigVMEdGraph)
{
	RootEdGraphs.Add(RigVMEdGraph);
}

void URigVMEditorAsset::AddLastEditedDocument(URigVMEdGraph* RigVMEdGraph)
{
	LastEditedDocuments.AddUnique(FEditedDocumentInfo(RigVMEdGraph));
}

FCompilerResultsLog URigVMEditorAsset::CompileBlueprint()
{
	RecompileVM();
	return CompileLog;
}

void URigVMEditorAsset::PostSerialize(FArchive& Ar)
{
	if (FRigVMClient* Client = GetRigVMClient())
	{
		Client->SetGetFunctionHostObjectPathDelegate(Client->GetFunctionLibrary());
	}
}

UClass* URigVMEditorAsset::GetRigVMEdGraphClass() const
{
	return URigVMEdGraph::StaticClass();
}

UClass* URigVMEditorAsset::GetRigVMEdGraphNodeClass() const
{
	return URigVMEdGraphNode::StaticClass();
}

UClass* URigVMEditorAsset::GetRigVMEdGraphSchemaClass() const
{
	return URigVMEdGraphSchema::StaticClass();
}

UClass* URigVMEditorAsset::GetRigVMEditorSettingsClass() const
{
	return URigVMEditorSettings::StaticClass();
}

void URigVMEditorAsset::RecompileVM()
{
	IRigVMEditorAssetInterface::RecompileVM();
}

const TArray<FString>& URigVMEditorAsset::GetRequiredPlugins(bool bRefresh) const
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetRequiredPlugins(bRefresh);
	}
	static TArray<FString> EmptyArray = TArray<FString>();
	return EmptyArray;
}

FRigVMPropertyBag* URigVMEditorAsset::GetVariablesPropertyBag()
{
	if (URigVMRuntimeAsset* RuntimeAsset = GetRuntimeAsset())
	{
		return RuntimeAsset->GetVariablesPropertyBag();
	}
	return nullptr;
}

FRigVMDrawContainer& URigVMEditorAsset::GetDrawContainer()
{
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = GetRuntimeAssetInterface())
	{
		return RuntimeAsset->GetDrawContainer();
	}
	static FRigVMDrawContainer EmptyDrawContainer;
	return EmptyDrawContainer;
}

TArray<FString> URigVMEditorAsset::GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset)
{
	TArray<FString> InternalCommands;
	if(GetObject()->GetClass() == GetObject()->StaticClass())
	{
		InternalCommands.Add(TEXT("import unreal"));
		InternalCommands.Add(TEXT("unreal.load_module('RigVMDeveloper')"));
	
		if (bCreateAsset)
		{
			InternalCommands.Add(TEXT("factory = unreal.ControlRigAssetFactory"));
			InternalCommands.Add(FString::Printf(TEXT("runtime_asset = factory.create_new_control_rig_asset(desired_package_path = '%s')"), *InNewBlueprintName));
		}
		else
		{
			InternalCommands.Add(FString::Printf(TEXT("runtime_asset = unreal.load_object(name = '%s', outer = None)"), *InNewBlueprintName));
		}
		InternalCommands.Add(TEXT("asset = runtime_asset.get_editor_asset()"));
	}

	InternalCommands.Add(TEXT("library = asset.get_local_function_library()"));
	InternalCommands.Add(TEXT("library_controller = asset.get_controller(library)"));
	InternalCommands.Add(TEXT("asset.set_auto_vm_recompile(False)"));
	return InternalCommands;
}

FRigVMBlueprintCompileScope::FRigVMBlueprintCompileScope(FRigVMEditorAssetInterfacePtr InBlueprint): Blueprint(InBlueprint)
{
	check(Blueprint);
	Blueprint->IncrementVMRecompileBracket();
}

FRigVMBlueprintCompileScope::~FRigVMBlueprintCompileScope()
{
	Blueprint->DecrementVMRecompileBracket();
}

#undef LOCTEXT_NAMESPACE


