// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMJsonUtils.h"
#include "HAL/IConsoleManager.h"
#include "RigVMDeveloperModule.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMEditorAsset.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMSchema.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMCommentNode.h"
#include "RigVMCore/RigVMVariableDescription.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/UObjectIterator.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/Engine.h"

#define LOCTEXT_NAMESPACE "RigVMJsonUtils"

#if WITH_EDITOR
#include "HAL/PlatformApplicationMisc.h"
#endif

FAutoConsoleCommand FCmdRigVMJsonPrintAvailableNodes
(
	TEXT("RigVM.JSON.PrintAvailableNodes"),
	TEXT("Prints out a json string containing information about all available nodes."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		TArray<FString> ConsecutiveCommands;
		{
			FRigVMRegistryReadLock Registry;
			RigVMJsonUtils::FIntrospectionSettings Settings(Registry);

			for (const FString& Arg : Args)
			{
				if (Arg.Equals(TEXT("exclude-deprecated"), ESearchCase::IgnoreCase))
				{
					Settings.bExcludeDeprecated = true;
				}
				else if (Arg.Equals(TEXT("exclude-functions"), ESearchCase::IgnoreCase))
				{
					Settings.bExcludeFunctions = true;
				}
				else if (Arg.Equals(TEXT("exclude-dispatches"), ESearchCase::IgnoreCase))
				{
					Settings.bExcludeDispatches = true;
				}
				else if (Arg.Equals(TEXT("create-images"), ESearchCase::IgnoreCase))
				{
					Settings.bCreateImages = true;
				}
				else if (Arg.Equals(TEXT("merge-templates"), ESearchCase::IgnoreCase))
				{
					Settings.bMergeTemplates = true;
				}
				else if(Arg.Len() > 8 && Arg.StartsWith(TEXT("context="), ESearchCase::IgnoreCase))
				{
					const FString ArgContextName = Arg.Mid(8);
					UClass* ArgSchemaClass = nullptr;
					for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
					{
						if (*ClassIterator == URigVMSchema::StaticClass())
						{
							continue;
						}
						if (ClassIterator->IsChildOf(URigVMSchema::StaticClass()))
						{
							const FString ClassName = ClassIterator->GetName();
							const FString ContextName = ClassName.Replace(TEXT("Schema"), TEXT(""));
							if (ArgContextName.Equals(ContextName, ESearchCase::IgnoreCase))
							{
								ArgSchemaClass = *ClassIterator;
								break;
							}
						}
					}

					if (ArgSchemaClass == nullptr)
					{
						UE_LOGF(LogRigVMDeveloper, Error, "Cannot find RigVMSchema class for context '%ls'", *ArgContextName);
						return;
					}

					Settings.Schema = ArgSchemaClass->GetDefaultObject<URigVMSchema>();
					check(Settings.Schema->GetEdGraphSchemaClass());
					Settings.EdGraphSchema = Settings.Schema->GetEdGraphSchemaClass()->GetDefaultObject<URigVMEdGraphSchema>();
				}
				else if(Arg.Len() > 8 && Arg.StartsWith(TEXT("filepath="), ESearchCase::IgnoreCase))
				{
					Settings.FilePath = Arg.Mid(9);
				}
				else
				{
					FString ContextNamesJoined;
					TArray<FString> ContextNames;
				
					for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
					{
						if (*ClassIterator == URigVMSchema::StaticClass())
						{
							continue;
						}
						if (!ClassIterator->IsChildOf(URigVMSchema::StaticClass()))
						{
							continue;
						}
						const FString ClassName = ClassIterator->GetName();
						const FString ContextName = ClassName.Replace(TEXT("Schema"), TEXT(""));

						static const TArray<FString> StringsToFilter = {
							TEXT("EngineTest"),
							TEXT("TestPlugin"),
							TEXT("RigVMExample"),
						};

						bool bValidContextName = true;
						for (const FString& StringToFilter : StringsToFilter)
						{
							if (ContextName.Contains(StringToFilter, ESearchCase::IgnoreCase))
							{
								bValidContextName = false;
								break;
							}
						}
						if (!bValidContextName)
						{
							continue;
						}
						ContextNames.Add(ContextName);
					}

					if (!ContextNames.IsEmpty())
					{
						ContextNamesJoined = RigVMStringUtils::JoinStrings(ContextNames, TEXT(", "));
					}

					UE_LOGF(LogRigVMDeveloper, Warning, "Valid Arguments:\n"
						"context=MyContextName - filter the nodes by context in question (%ls)\n"
						"exclude-deprecated - only includes the current / non-deprecated nodes\n"
						"exclude-functions - excludes RigVM struct based functions\n"
						"exclude-dispatches - excludes RigVM dispatch factories\n"
						"filepath=MyFilePath - the [optional] file path to export the content to", *ContextNamesJoined);
					return;
				}
			}

			if (Settings.bCreateImages && Settings.FilePath.IsEmpty())
			{
				UE_LOGF(LogRigVMDeveloper, Display, "Cannot create node images - filepath argument not specified.");
				Settings.bCreateImages = false;
			}
			
			TSharedPtr<FJsonObject> JsonObject = RigVMJsonUtils::AvailableNodesToJson(Settings, [](const UScriptStruct* InStruct) -> bool
			{
				check(InStruct);
				const UPackage* Package = InStruct->GetPackage();

				static const TArray<FString> StringsToFilter = {
					TEXT("EngineTest"),
					TEXT("TestPlugin"),
					TEXT("RigVMExample"),
				};
				for (const FString& StringToFilter : StringsToFilter)
				{
					if (Package->GetName().Contains(StringToFilter))
					{
						return false;
					}
				}
				return true;
			});
			
			if (JsonObject)
			{
				FString Text;
				RigVMJsonUtils::FJsonWriterRef Writer = RigVMJsonUtils::MakeJsonWriter(Text);
				if (FJsonSerializer::Serialize(JsonObject, Writer))
				{
	#if WITH_EDITOR
					if (Settings.FilePath.IsEmpty())
					{
						UE_LOGF(LogRigVMDeveloper, Display, "The content has been copied to the clipboard.");
						FPlatformApplicationMisc::ClipboardCopy(*Text);
						return;
					}
	#endif
					if (!Settings.FilePath.IsEmpty())
					{
						if (FFileHelper::SaveStringToFile(Text, *Settings.FilePath))
						{
							UE_LOGF(LogRigVMDeveloper, Display, "The content has been save to '%ls'.", *Settings.FilePath);
						}
						else
						{
							UE_LOGF(LogRigVMDeveloper, Error, "The content could not be saved to '%ls'.", *Settings.FilePath);
						}
					}
				}

				ConsecutiveCommands = Settings.CommandsToRun;
			}
		}

		if (!ConsecutiveCommands.IsEmpty())
		{
			FScopedSlowTask Progress((float)ConsecutiveCommands.Num(), LOCTEXT("ExecutingConsecutiveCommands", "Executing consecutive commands"));
			Progress.MakeDialog(true);
			for (const FString& Command : ConsecutiveCommands)
			{
				if (Progress.ShouldCancel())
				{
					break;
				}
				Progress.EnterProgressFrame(1.f, FText::FromString(Command));
				GEngine->Exec(nullptr, *Command);
			}
		}
})
);

FAutoConsoleCommand FCmdRigVMJsonPrintGraph
(
	TEXT("RigVM.JSON.PrintGraph"),
	TEXT("Prints out a json string representing a rigvm graph."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() != 1)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Missing graph pathname argument for RigVM.JSON.PrintGraph");
			return;
		}

		FString GraphPathName = Args[0];
		UObject* GraphObject = StaticLoadObject(URigVMGraph::StaticClass(), nullptr, GraphPathName);
		if (!GraphObject)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Cannot find graph '%ls' for RigVM.JSON.PrintGraph", *GraphPathName);
			return;
		}
			
			FRigVMRegistryReadLock Registry;
			RigVMJsonUtils::FIntrospectionSettings Settings(Registry);
			if (TSharedPtr<FJsonObject> JsonObject = RigVMJsonUtils::ToJson(Settings, CastChecked<URigVMGraph>(GraphObject)))
		{
			FString Text;
			RigVMJsonUtils::FJsonWriterRef Writer = RigVMJsonUtils::MakeJsonWriter(Text);
			if (FJsonSerializer::Serialize(JsonObject, Writer))
			{
#if WITH_EDITOR
				UE_LOGF(LogRigVMDeveloper, Display, "The content has been copied to the clipboard.");
				FPlatformApplicationMisc::ClipboardCopy(*Text);
#endif
			}
		}
	})
);

FAutoConsoleCommand FCmdRigVMJsonPrintAsset
(
	TEXT("RigVM.JSON.PrintAsset"),
	TEXT("Prints out a json string representing a rigvm asset."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() != 1)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Missing asset pathname argument for RigVM.JSON.PrintAsset");
			return;
		}

		FString AssetPathName = Args[0];
		UObject* AssetObject = StaticLoadObject(UObject::StaticClass(), nullptr, AssetPathName);
		if (!AssetObject)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Cannot find asset '%ls' for RigVM.JSON.PrintAsset", *AssetPathName);
			return;
		}

		const IRigVMEditorAssetInterface* Asset = Cast<IRigVMEditorAssetInterface>(AssetObject);
		if (!Asset)
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Object '%ls' exists, but it's not a RigVMAsset (when running RigVM.JSON.PrintAsset)", *AssetPathName);
			return;
		}

		FRigVMRegistryReadLock Registry;
		RigVMJsonUtils::FIntrospectionSettings Settings(Registry);
		if (TSharedPtr<FJsonObject> JsonObject = RigVMJsonUtils::ToJson(Settings, Asset))
		{
			FString Text;
			RigVMJsonUtils::FJsonWriterRef Writer = RigVMJsonUtils::MakeJsonWriter(Text);
			if (FJsonSerializer::Serialize(JsonObject, Writer))
			{
#if WITH_EDITOR
				UE_LOGF(LogRigVMDeveloper, Display, "The content has been copied to the clipboard.");
				FPlatformApplicationMisc::ClipboardCopy(*Text);
#endif
			}
		}
	})
);

namespace RigVMJsonUtils
{
	static FString PostProcessToolTip(const FString& InToolTip)
	{
		if (InToolTip.StartsWith(TEXT("#define")))
		{
			return FString();
		}
		FString Result = InToolTip;
		Result.ReplaceInline(TEXT("\\n* "), TEXT("\\n"));
		Result.RemoveFromStart(TEXT("* "));
		Result.RemoveFromStart(TEXT("*"));
		return Result;
	}
}

RigVMJsonUtils::FIntrospectionSettings::FIntrospectionSettings(FRigVMRegistryHandle& InRegistry)
: Registry(InRegistry)
, Schema(URigVMSchema::StaticClass()->GetDefaultObject<URigVMSchema>())
, EdGraphSchema(URigVMEdGraphSchema::StaticClass()->GetDefaultObject<URigVMEdGraphSchema>())
{
}

void RigVMJsonUtils::FIntrospectionSettings::AddDiscoveredType(const TRigVMTypeIndex& InTypeIndex)
{
	if (InTypeIndex == INDEX_NONE)
	{
		return;
	}
	if (DiscoveredTypes.Contains(InTypeIndex))
	{
		return;
	}
	if (Registry->IsArrayType_NoLock(InTypeIndex))
	{
		AddDiscoveredType(Registry->GetBaseTypeFromArrayTypeIndex_NoLock(InTypeIndex));
		return;
	}
	if (Schema)
	{
		if (!Schema->SupportsType_NoLock(nullptr, InTypeIndex, Registry))
		{
			return;
		}
	}
	DiscoveredTypes.Add(InTypeIndex);
}

RigVMJsonUtils::FJsonWriterRef RigVMJsonUtils::MakeJsonWriter(FString& OutJsonText)
{
	return FJsonFactory::Create(&OutJsonText);
}

TSharedPtr<FJsonObject> RigVMJsonUtils::AvailableNodesToJson(FIntrospectionSettings& InSettings, TFunction<bool(const UScriptStruct*)> InFilterCallback)
{
	if (!InSettings.Registry.IsValid())
	{
		return nullptr;
	}

	InSettings.DiscoveredTypes.Reset();
	TArray<TSharedPtr<FJsonValue>> TypeValues, FunctionJsonValues, FactoryJsonValues;
	const TChunkedArray<FRigVMFunction>& Functions = InSettings.Registry->GetFunctions_NoLock();
	const TArray<FRigVMDispatchFactory*>& Factories = InSettings.Registry->GetFactories_NoLock();

	for (const FRigVMFunction& Function : Functions)
	{
		if (Function.Factory)
		{
			continue;
		}
		check(Function.Struct);
		if (InFilterCallback && !InFilterCallback(Function.Struct))
		{
			continue;
		}
		if (const TSharedPtr<FJsonObject> JsonObject = ToJson(InSettings, &Function))
		{
			FunctionJsonValues.Add(MakeShared<FJsonValueObject>(JsonObject));
		}
	}

	for (const FRigVMDispatchFactory* Factory : Factories)
	{
		check(Factory);
		if (InFilterCallback && !InFilterCallback(Factory->GetScriptStruct()))
		{
			continue;
		}
		if (const TSharedPtr<FJsonObject> JsonObject = ToJson(InSettings, Factory))
		{
			FactoryJsonValues.Add(MakeShared<FJsonValueObject>(JsonObject));
		}
	}

	TArray<TRigVMTypeIndex> DiscoveredTypes = InSettings.DiscoveredTypes.Array();
	DiscoveredTypes.Sort();
	for (const TRigVMTypeIndex& Type : DiscoveredTypes)
	{
		if (const TSharedPtr<FJsonObject> JsonObject = ToJson(InSettings, Type))
		{
			TypeValues.Add(MakeShared<FJsonValueObject>(JsonObject));
		}
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetArrayField(TEXT("Types"), TypeValues);
	JsonObject->SetArrayField(TEXT("Functions"), FunctionJsonValues);
	JsonObject->SetArrayField(TEXT("Dispatches"), FactoryJsonValues);
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const FRigVMFunction* InFunction)
{
	if (InSettings.bExcludeFunctions)
	{
		return nullptr;
	}

	check(InFunction);
	check(InFunction->Struct);

	TArray<const FRigVMFunction*> Permutations = {InFunction};
	
	const FRigVMTemplate* Template = InFunction->GetTemplate_NoLock(InSettings.Registry);

#if WITH_EDITOR
	// this may be a deprecated template
	if (!Template && InSettings.bMergeTemplates && InFunction->Struct->HasMetaDataHierarchical(FRigVMStruct::DeprecatedMetaName))
	{
		FString TemplateMetadata;
		if (InFunction->Struct->GetStringMetaDataHierarchical(FRigVMRegistry::TemplateNameMetaName, &TemplateMetadata))
		{
			const FString TemplateName = RigVMStringUtils::JoinStrings(TemplateMetadata, InFunction->GetMethodName().ToString(), TEXT("::"));
			const FName Notation = FRigVMTemplate(InFunction->Struct, TemplateName, InFunction->Index, InSettings.Registry).GetNotation();
			if (!Notation.IsNone())
			{
				Template = InSettings.Registry->FindTemplate_NoLock(Notation, true /* include deprecated */, false /* remap */);
			}
		}
	}
#endif
	
	if (Template && InSettings.bMergeTemplates)
	{
		if (Template->GetPermutation_NoLock(0, InSettings.Registry) != InFunction)
		{
			return nullptr;
		}

		Permutations.Reset();
		const int32 NumPermutations = Template->NumPermutations_NoLock(InSettings.Registry);
		for (int32 PermutationIndex = 0; PermutationIndex < NumPermutations; PermutationIndex++)
		{
			if (const FRigVMFunction* Permutation = Template->GetPermutation_NoLock(PermutationIndex, InSettings.Registry))
			{
				if (!InSettings.Schema->SupportsUnitFunction_NoLock(nullptr, Permutation, InSettings.Registry))
				{
					continue;
				}
				Permutations.Add(Permutation);
			}
		}
		if (Permutations.IsEmpty())
		{
			return nullptr;
		}
	}
	else if (InSettings.Schema)
	{
		if (!InSettings.Schema->SupportsUnitFunction_NoLock(nullptr, InFunction, InSettings.Registry))
		{
			return nullptr;
		}
	}

	// rely on the first valid permutation
	// (which by default is the passed in function)
	InFunction = Permutations[0];

	check(InFunction);
	const UScriptStruct* Struct = InFunction->Struct;
	check(Struct);

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	if (!GenericStructAttributesToJson(InSettings, Struct, JsonObject.Get()))
	{
		return nullptr;
	}

	FString TemplateName;
	if (Template)
	{
		TemplateName = Template->GetName().ToString();
		if (Permutations.Num() > 1)
		{
			JsonObject->RemoveField(TEXT("Type"));
		}

#if WITH_EDITOR
		JsonObject->SetStringField(TEXT("DisplayName"), Template->GetNodeName().ToString());
		JsonObject->SetStringField(TEXT("Category"), Template->GetCategory());
		FString KeywordsJoined = Template->GetKeywords();
		TArray<FString> Keywords;
		if (RigVMStringUtils::SplitString(KeywordsJoined, TEXT(","), Keywords))
		{
			TSet<FString> UniqueKeywords(Keywords);
			Keywords = UniqueKeywords.Array();
			KeywordsJoined = RigVMStringUtils::JoinStrings(Keywords, TEXT(","));
		}
		JsonObject->SetStringField(TEXT("Keywords"), KeywordsJoined);
		FString ToolTip = Template->GetTooltipText().ToString();
		if (ToolTip == Template->GetName().ToString())
		{
			TArray<FString> ToolTips;
			for (const FRigVMFunction* Permutation : Permutations)
			{
				if (!Permutation->Struct)
				{
					continue;
				}
				FString PermutationToolTip = PostProcessToolTip(Permutation->Struct->GetToolTipText().ToString());
				if (PermutationToolTip.IsEmpty())
				{
					continue;
				}
				if (PermutationToolTip == Permutation->Struct->GetDisplayNameText().ToString())
				{
					continue;
				}
				ToolTips.Add(PermutationToolTip);
			}
			TSet<FString> UniqueToolTips(ToolTips);
			ToolTips = UniqueToolTips.Array();
			ToolTip = RigVMStringUtils::JoinStrings(ToolTips, TEXT("\n"));
		}
		JsonObject->SetStringField(TEXT("ToolTip"), ToolTip);
#endif
	}
	JsonObject->SetStringField(TEXT("TemplateName"), TemplateName);

	FStructOnScope StructOnScope(Struct);
	FRigVMStruct* StructInstance = reinterpret_cast<FRigVMStruct*>(StructOnScope.GetStructMemory());

	TArray<TSharedPtr<FJsonValue>> IO, Inputs, Outputs;

	const FRigVMStructUpgradeInfo UpgradeInfo = StructInstance->GetUpgradeInfo();
	if (UpgradeInfo.IsValid())
	{
		if (const UScriptStruct* NewStruct = UpgradeInfo.GetNewStruct())
		{
			JsonObject->SetStringField(TEXT("UpgradedVersion"), NewStruct->GetStructCPPName());
		}
	}

	if (InSettings.bCreateImages)
	{
		FString FileName;
		FString Subject;
		if (Template && InSettings.bMergeTemplates)
		{
			Subject = TemplateName;
			Subject.ReplaceInline(TEXT(" "), TEXT("_"));
			FileName = FString::Printf(TEXT("Template_%s"), *Subject);
			FileName.ReplaceInline(TEXT(" "), TEXT("_"));
		}
		else
		{
			FileName = Struct->GetStructCPPName();
			Subject = Struct->GetPathName();
		}
		
		const FString OutputBaseFolder = FPaths::GetPath(InSettings.FilePath);
		const FString RelativeImageFilePath = FString::Printf(TEXT("Images/%s.png"), *FileName);
		const FString AbsoluteImageFilePath = FString::Printf(TEXT("%s/%s"), *OutputBaseFolder, *RelativeImageFilePath);
		const FString Command = FString::Printf(TEXT("RigVM.JSON.RenderNodeToPNG subject=%s filepath=%s"), *Subject, *AbsoluteImageFilePath);
		InSettings.CommandsToRun.AddUnique(Command);
		JsonObject->SetStringField(TEXT("Image"), RelativeImageFilePath);
	}

	TMap<FName,int32> ArgumentLookup;
	for (int32 Index = 0; Index < InFunction->Arguments.Num(); ++Index)
	{
		ArgumentLookup.Add(InFunction->Arguments[Index].Name, Index);
	}
	
	for (TFieldIterator<FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		bool bIsExecutePin = false;
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			bIsExecutePin = StructProperty->Struct->IsChildOf(FRigVMExecutePin::StaticStruct());
		}

		static const FString ExecutePinType = FRigVMExecuteContext::StaticStruct()->GetStructCPPName();
		if (bIsExecutePin)
		{
			InSettings.AddDiscoveredType(InSettings.Registry->GetTypeIndex_NoLock<FRigVMExecuteContext>());
		}

		int32 ArgumentIndex = INDEX_NONE;
		const FRigVMFunctionArgument* Argument = nullptr;
		if (const int32* ArgumentIndexPtr = ArgumentLookup.Find(Property->GetFName()))
		{
			ArgumentIndex = *ArgumentIndexPtr;
			Argument = &InFunction->Arguments[ArgumentIndex];
		}

		if (!Argument && !bIsExecutePin)
		{
			continue;
		}

		bool bIsInput = Argument ? Argument->Direction == ERigVMFunctionArgumentDirection::Input : true;
		bool bIsOutput = Argument ? Argument->Direction == ERigVMFunctionArgumentDirection::Output : true;
		bool bIsConstant = false;
		
#if WITH_EDITOR
		bIsInput =
			Property->HasMetaData(FRigVMStruct::InputMetaName) ||
			Property->HasMetaData(FRigVMStruct::VisibleMetaName);
		bIsOutput = Property->HasMetaData(FRigVMStruct::OutputMetaName);
		bIsConstant = Property->HasMetaData(FRigVMStruct::ConstantMetaName);
#endif

		if (!bIsInput && !bIsOutput)
		{
			continue;
		}

		FString DefaultValue;
		if (!bIsExecutePin)
		{
			const uint8* Memory = Property->ContainerPtrToValuePtr<uint8>(StructInstance);
			Property->ExportTextItem_Direct(DefaultValue, Memory, Memory, nullptr, PPF_None, nullptr);
			if (DefaultValue.IsEmpty())
			{
				if (Property->IsA<FArrayProperty>() || Property->IsA<FStructProperty>())
				{
					DefaultValue = TEXT("()");
				}
				else if (Property->IsA<FNameProperty>())
				{
					DefaultValue = TEXT("None");
				}
			}
			if (Argument)
			{
				InSettings.AddDiscoveredType(InSettings.Registry->GetTypeIndexFromCPPType_NoLock(Argument->Type));
			}
		}

		TSharedPtr<FJsonObject> ArgumentJsonObject = MakeShared<FJsonObject>();
		ArgumentJsonObject->SetStringField(TEXT("Name"), Property->GetName());

		if (Argument)
		{
			if (Permutations.Num() == 1)
			{
				ArgumentJsonObject->SetStringField(TEXT("Type"), Argument->Type);
			}
			else
			{
				TArray<FString> Types;
				for (const FRigVMFunction* Permutation : Permutations)
				{
					Types.Add(Permutation->Arguments[ArgumentIndex].Type);
				}
				Types.Sort([&InSettings](const FString& A, const FString& B) -> bool
				{
					const TRigVMTypeIndex TypeIndexA = InSettings.Registry->GetTypeIndexFromCPPType_NoLock(A);
					const TRigVMTypeIndex TypeIndexB = InSettings.Registry->GetTypeIndexFromCPPType_NoLock(B);
					if (TypeIndexA != INDEX_NONE && TypeIndexB != INDEX_NONE)
					{
						return TypeIndexA < TypeIndexB;
					}
					return A.Compare(B, ESearchCase::IgnoreCase) < 0;
				});
				TSet<FString> UniqueTypes(Types);
				Types = UniqueTypes.Array();
				ArgumentJsonObject->SetStringField(TEXT("Type"), RigVMStringUtils::JoinStrings(Types, TEXT(",")));
			}
		}
		else
		{
			ArgumentJsonObject->SetStringField(TEXT("Type"), ExecutePinType);
		}

		FString PropertyToolTip = PostProcessToolTip(Property->GetToolTipText().ToString());
		if (PropertyToolTip.IsEmpty() && bIsExecutePin)
		{
			PropertyToolTip = TEXT("This pin is used to chain multiple mutable units together");
		}
		ArgumentJsonObject->SetStringField(TEXT("PropertyToolTip"), PropertyToolTip);

		if ((bIsInput || bIsConstant) && !bIsExecutePin)
		{
			ArgumentJsonObject->SetStringField(TEXT("DefaultValue"), DefaultValue);
		}

		if (bIsInput && bIsOutput)
		{
			IO.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else if (bIsInput)
		{
			Inputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else // if bIsOutput
		{
			Outputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
	}
	
	JsonObject->SetArrayField(TEXT("IO"), IO);
	JsonObject->SetArrayField(TEXT("Inputs"), Inputs);
	JsonObject->SetArrayField(TEXT("Outputs"), Outputs);

	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const FRigVMDispatchFactory* InFactory)
{
	if (InSettings.bExcludeDispatches)
	{
		return nullptr;
	}
	
	check(InFactory);
	const UScriptStruct* Struct = InFactory->GetScriptStruct();
	check(Struct);

	if (InSettings.Schema)
	{
		if (!InSettings.Schema->SupportsDispatchFactory_NoLock(nullptr, InFactory, InSettings.Registry))
		{
			return nullptr;
		}
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	if (!GenericStructAttributesToJson(InSettings, Struct, JsonObject.Get()))
	{
		return nullptr;
	}

	const FRigVMTemplate* Template = InFactory->GetTemplate_NoLock(InSettings.Registry);
	JsonObject->SetStringField(TEXT("TemplateName"), Template->GetName().ToString());
	const FText ToolTip = InFactory->GetNodeTooltip({});
	if (!ToolTip.IsEmpty())
	{
		JsonObject->SetStringField(TEXT("ToolTip"), PostProcessToolTip(ToolTip.ToString()));
	}

	FRigVMDispatchContext DispatchContext;
	TArray<TSharedPtr<FJsonValue>> IO, Inputs, Outputs;

	const FRigVMStructUpgradeInfo UpgradeInfo = InFactory->GetUpgradeInfo({}, DispatchContext);
	if (UpgradeInfo.IsValid())
	{
		if (const UScriptStruct* NewStruct = UpgradeInfo.GetNewStruct())
		{
			JsonObject->SetStringField(TEXT("UpgradedVersion"), NewStruct->GetStructCPPName());
		}
	}

	if (InSettings.bCreateImages)
	{
		const FString FileName = Template->GetName().ToString();
		const FString Subject = FileName;
		const FString OutputBaseFolder = FPaths::GetPath(InSettings.FilePath);
		const FString RelativeImageFilePath = FString::Printf(TEXT("Images/%s.png"), *FileName);
		const FString AbsoluteImageFilePath = FString::Printf(TEXT("%s/%s"), *OutputBaseFolder, *RelativeImageFilePath);
		const FString Command = FString::Printf(TEXT("RigVM.JSON.RenderNodeToPNG subject=%s filepath=%s"), *Subject, *AbsoluteImageFilePath);
		InSettings.CommandsToRun.AddUnique(Command);
		JsonObject->SetStringField(TEXT("Image"), RelativeImageFilePath);
	}

	TArray<FRigVMExecuteArgument> ExecuteArguments = InFactory->GetExecuteArguments_NoLock(DispatchContext, InSettings.Registry);
	for (const FRigVMExecuteArgument& ExecuteArgument : ExecuteArguments)
	{
		static const FString ExecutePinType = FRigVMExecuteContext::StaticStruct()->GetStructCPPName();

		TSharedPtr<FJsonObject> ArgumentJsonObject = MakeShared<FJsonObject>();
		ArgumentJsonObject->SetStringField(TEXT("Name"), ExecuteArgument.Name.ToString());
		ArgumentJsonObject->SetStringField(TEXT("Type"), ExecutePinType);
		if (ExecuteArgument.Name == FRigVMStruct::ExecuteName || ExecuteArgument.Name == FRigVMStruct::ExecuteContextName)
		{
			ArgumentJsonObject->SetStringField(TEXT("PropertyToolTip"), TEXT("This pin is used to chain multiple mutable units together"));
		}
		else
		{
			ArgumentJsonObject->SetStringField(TEXT("PropertyToolTip"), FString::Printf(TEXT("The %s execute argument."), *ExecuteArgument.Name.ToString()));
		}

		if (ExecuteArgument.Direction == ERigVMPinDirection::IO)
		{
			IO.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else if (ExecuteArgument.Direction == ERigVMPinDirection::Input)
		{
			Inputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else // if bIsOutput
		{
			Outputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
	}

	const TArray<FRigVMTemplateArgumentInfo>& ArgumentInfos = InFactory->GetArgumentInfos(InSettings.Registry);
	for (const FRigVMTemplateArgumentInfo& ArgumentInfo : ArgumentInfos)
	{
		if (ArgumentInfo.Direction != ERigVMPinDirection::Input &&
			ArgumentInfo.Direction != ERigVMPinDirection::Visible &&
			ArgumentInfo.Direction != ERigVMPinDirection::IO &&
			ArgumentInfo.Direction != ERigVMPinDirection::Output)
		{
			continue;
		}
		
		const FRigVMTemplateArgument& Argument = ArgumentInfo.GetArgument_NoLock(InSettings.Registry);
		TArray<TRigVMTypeIndex> TypeIndices = Argument.GetSupportedTypeIndices_NoLock({}, InSettings.Registry);
		check(!TypeIndices.IsEmpty());

		bool bIsWildCard = false;

		FString Type;
		if (TypeIndices.Num() == 1)
		{
			if (!InSettings.Schema->SupportsType_NoLock(nullptr, TypeIndices[0], InSettings.Registry))
			{
				continue;
			}
			Type = InSettings.Registry->GetType_NoLock(TypeIndices[0]).CPPType.ToString();
			InSettings.AddDiscoveredType(TypeIndices[0]);
		}
		else if (TypeIndices.Num() < 16)
		{
			TArray<FString> Types;
			Types.Reserve(TypeIndices.Num());
			TypeIndices.Sort();
			for (const TRigVMTypeIndex& TypeIndex : TypeIndices)
			{
				if (InSettings.Schema)
				{
					if (!InSettings.Schema->SupportsType_NoLock(nullptr, TypeIndex, InSettings.Registry))
					{
						continue;
					}
				}
				const FRigVMTemplateArgumentType& RigVMType = InSettings.Registry->GetType_NoLock(TypeIndex);
				if (RigVMType.CPPTypeObject)
				{
					if (RigVMType.CPPTypeObject->GetPathName().Contains(TEXT("EngineTest")))
					{
						continue;
					}
				}
				if (Cast<UUserDefinedStruct>(RigVMType.CPPTypeObject))
				{
					Types.AddUnique(TEXT("User Defined Structs"));
					continue;
				}
				if (Cast<UUserDefinedEnum>(RigVMType.CPPTypeObject))
				{
					Types.AddUnique(TEXT("User Defined Enums"));
					continue;
				}
				Types.Add(InSettings.Registry->GetType_NoLock(TypeIndex).CPPType.ToString());
				InSettings.AddDiscoveredType(TypeIndex);
			}
			TSet<FString> UniqueTypes(Types);
			Types = UniqueTypes.Array();
			Type = RigVMStringUtils::JoinStrings(Types, TEXT(","));
		}
		else
		{
			static const FString WildcardType = FRigVMUnknownType::StaticStruct()->GetStructCPPName();
			static const FString WildcardArrayType = RigVMTypeUtils::ArrayTypeFromBaseType(WildcardType);

			int32 NumArrayTypes = 0;
			for (const TRigVMTypeIndex& TypeIndex : TypeIndices)
			{
				if (InSettings.Registry->IsArrayType_NoLock(TypeIndex))
				{
					NumArrayTypes++;
				}
			}
			if (NumArrayTypes == TypeIndices.Num())
			{
				Type = WildcardArrayType;
				InSettings.AddDiscoveredType(RigVMTypeUtils::TypeIndex::WildCardArray);
			}
			else
			{
				Type = WildcardType;
				InSettings.AddDiscoveredType(RigVMTypeUtils::TypeIndex::WildCard);
			}

			bIsWildCard = true;
		}
		
		TSharedPtr<FJsonObject> ArgumentJsonObject = MakeShared<FJsonObject>();
		ArgumentJsonObject->SetStringField(TEXT("Name"), ArgumentInfo.Name.ToString());
		ArgumentJsonObject->SetStringField(TEXT("Type"), Type);
		ArgumentJsonObject->SetStringField(TEXT("PropertyToolTip"), PostProcessToolTip(InFactory->GetArgumentTooltip(ArgumentInfo.Name, TypeIndices[0]).ToString()));

		if ((Argument.GetDirection() != ERigVMPinDirection::Output) && !bIsWildCard)
		{
			if (Argument.GetNumTypes_NoLock(InSettings.Registry) == 1)
			{
				const TRigVMTypeIndex FirstTypeIndex = Argument.GetTypeIndex_NoLock(0, InSettings.Registry);
				FString DefaultValue = InFactory->GetArgumentDefaultValue(Argument.GetName(), FirstTypeIndex);
				if (DefaultValue.IsEmpty())
				{
					if (FirstTypeIndex == RigVMTypeUtils::TypeIndex::FName)
					{
						DefaultValue = TEXT("None");
					}
					else if(InSettings.Registry->IsArrayType_NoLock(FirstTypeIndex))
					{
						DefaultValue = TEXT("()");
					}
					else if(Cast<UStruct>(InSettings.Registry->GetType_NoLock(FirstTypeIndex).CPPTypeObject))
					{
						DefaultValue = TEXT("()");
					}
				}
				ArgumentJsonObject->SetStringField(TEXT("DefaultValue"), DefaultValue);
			}
		}

		if (Argument.GetDirection() == ERigVMPinDirection::IO)
		{
			IO.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else if (Argument.GetDirection() == ERigVMPinDirection::Input)
		{
			Inputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
		else // if bIsOutput
		{
			Outputs.Add(MakeShared<FJsonValueObject>(ArgumentJsonObject));
		}
	}

	JsonObject->SetArrayField(TEXT("IO"), IO);
	JsonObject->SetArrayField(TEXT("Inputs"), Inputs);
	JsonObject->SetArrayField(TEXT("Outputs"), Outputs);

	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const TRigVMTypeIndex& InTypeIndex)
{
	if (InTypeIndex == INDEX_NONE)
	{
		return nullptr;
	}
	if (InSettings.Schema)
	{
		if (!InSettings.Schema->SupportsType_NoLock(nullptr, InTypeIndex, InSettings.Registry))
		{
			return nullptr;
		}
	}

	const FRigVMTemplateArgumentType& Type = InSettings.Registry->GetType_NoLock(InTypeIndex);
	if (!Type.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("Type"), Type.CPPType.ToString());
	if (Type.CPPTypeObject)
	{
		if (const UField* Field = Cast<UField>(Type.CPPTypeObject))
		{
			FString Label = Field->GetDisplayNameText().ToString();
			FString ToolTip = PostProcessToolTip(Field->GetToolTipText().ToString());
			if (ToolTip.Equals(Label))
			{
				ToolTip.Reset();
			}
			if (Type.CPPTypeObject->IsA<UEnum>())
			{
				if (Label.StartsWith(TEXT("E")) &&
					Label.StartsWith(Type.CPPType.ToString().Left(2), ESearchCase::CaseSensitive))
				{
					Label.RightChopInline(1);
				}
			}
			JsonObject->SetStringField(TEXT("Label"), Label);
			if (!ToolTip.IsEmpty())
			{
				JsonObject->SetStringField(TEXT("ToolTip"), ToolTip);
			}
		}
		JsonObject->SetStringField(TEXT("PackageName"), Type.CPPTypeObject->GetOutermost()->GetPathName());
	}
	if (InSettings.EdGraphSchema)
	{
		const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(Type.CPPType, Type.CPPTypeObject);
		const FLinearColor PinColor = InSettings.EdGraphSchema->GetPinTypeColor(PinType);
		JsonObject->SetStringField(TEXT("Color"), PinColor.ToString());
	}
	return JsonObject;
}

bool RigVMJsonUtils::GenericStructAttributesToJson(FIntrospectionSettings& InSettings, const UScriptStruct* InStruct, FJsonObject* OutJsonObject)
{
	check(InStruct);
	check(OutJsonObject);
	
	FString CategoryMetadata, KeywordsMetadata, DisplayNameMetadata, DeprecatedMetadata;
#if WITH_EDITOR
	InStruct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &CategoryMetadata);
	InStruct->GetStringMetaDataHierarchical(FRigVMStruct::KeywordsMetaName, &KeywordsMetadata);
	InStruct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
	InStruct->GetStringMetaDataHierarchical(FRigVMStruct::DeprecatedMetaName, &DeprecatedMetadata);
#endif
	if(DisplayNameMetadata.IsEmpty())
	{
		DisplayNameMetadata = InStruct->GetDisplayNameText().ToString();
	}

	OutJsonObject->SetStringField(TEXT("DisplayName"), DisplayNameMetadata);
	OutJsonObject->SetStringField(TEXT("Type"), InStruct->GetStructCPPName());
	if (!DeprecatedMetadata.IsEmpty())
	{
		if (InSettings.bExcludeDeprecated && !DeprecatedMetadata.IsEmpty())
		{
			return false;
		}
		OutJsonObject->SetStringField(TEXT("Deprecated"), DeprecatedMetadata);
	}
	OutJsonObject->SetStringField(TEXT("Category"), CategoryMetadata);
	OutJsonObject->SetStringField(TEXT("Keywords"), KeywordsMetadata);
	OutJsonObject->SetStringField(TEXT("ToolTip"), PostProcessToolTip(InStruct->GetToolTipText().ToString()));
	OutJsonObject->SetStringField(TEXT("PackageName"), InStruct->GetOutermost()->GetPathName());

	return true;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const IRigVMEditorAssetInterface* InAsset)
{
	check(InAsset);
	
	TArray<TSharedPtr<FJsonValue>> Variables, GraphValues;

	TArray<FRigVMGraphVariableDescription> VariableDescriptions = InAsset->GetAssetVariables();
	for (const FRigVMGraphVariableDescription& VariableDescription : VariableDescriptions)
	{
		if (TSharedPtr<FJsonObject> VariableJsonObject = ToJson(InSettings, VariableDescription))
		{
			Variables.Add(MakeShared<FJsonValueObject>(VariableJsonObject));
		}
	}

	if (const FRigVMClient* Client = InAsset->GetRigVMClient())
	{
		const TArray<URigVMGraph*> Graphs =Client->GetAllModels(true, false);
		for (const URigVMGraph* Graph : Graphs)
		{
			if (TSharedPtr<FJsonObject> GraphJsonObject = ToJson(InSettings, Graph))
			{
				GraphValues.Add(MakeShared<FJsonValueObject>(GraphJsonObject));
			}
		}
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

	if (const UObject* AssetObject = InAsset->GetObject())
	{
		JsonObject->SetStringField(TEXT("Name"), AssetObject->GetName());
		JsonObject->SetStringField(TEXT("PackageName"), AssetObject->GetOutermost()->GetPathName());
	}
	JsonObject->SetArrayField(TEXT("Variables"), Variables);
	JsonObject->SetArrayField(TEXT("Graphs"), GraphValues);
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const URigVMGraph* InGraph, bool bOnlySelection)
{
	check(InGraph);

	TSet<FName> ExportedNodes;
	TArray<TSharedPtr<FJsonValue>> Variables, Nodes, Links;

	if(!InGraph->IsTopLevelGraph())
	{
		TArray<FRigVMGraphVariableDescription> VariableDescriptions = InGraph->GetLocalVariables(false);
		for (const FRigVMGraphVariableDescription& VariableDescription : VariableDescriptions)
		{
			if (TSharedPtr<FJsonObject> VariableJsonObject = ToJson(InSettings, VariableDescription))
			{
				Variables.Add(MakeShared<FJsonValueObject>(VariableJsonObject));
			}
		}
	}

	const TArray<FName> SelectedNodes = InGraph->GetSelectNodes(); 
	for (const URigVMNode* Node : InGraph->GetNodes())
	{
		if (bOnlySelection)
		{
			if (!SelectedNodes.Contains(Node->GetName()))
			{
				continue;
			}
		}
		
		if (TSharedPtr<FJsonObject> NodeJsonObject = ToJson(InSettings, Node))
		{
			Nodes.Add(MakeShared<FJsonValueObject>(NodeJsonObject));
			ExportedNodes.Add(Node->GetFName());
		}
	}
	
	for (const URigVMLink* Link : InGraph->GetLinks())
	{
		const URigVMNode* SourceNode = Link->GetSourceNode();
		const URigVMNode* TargetNode = Link->GetTargetNode();
		if (!SourceNode || !TargetNode)
		{
			continue;
		}
		if (!ExportedNodes.Contains(SourceNode->GetFName()) || !ExportedNodes.Contains(TargetNode->GetFName()))
		{
			continue;
		}
		if (TSharedPtr<FJsonObject> LinkJsonObject = ToJson(InSettings, Link))
		{
			Links.Add(MakeShared<FJsonValueObject>(LinkJsonObject));
		}
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();

	if (InGraph->IsA<URigVMFunctionLibrary>())
	{
		JsonObject->SetStringField(TEXT("Name"), TEXT("FunctionLibrary"));
		JsonObject->SetArrayField(TEXT("GraphFunctions"), Nodes);
	}
	else
	{
		JsonObject->SetStringField(TEXT("Name"), InGraph->GetName());
		JsonObject->SetArrayField(TEXT("LocalVariables"), Variables);
		JsonObject->SetArrayField(TEXT("Nodes"), Nodes);
		JsonObject->SetArrayField(TEXT("Links"), Links);
	}
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const URigVMNode* InNode)
{
	check(InNode);

	TArray<TSharedPtr<FJsonValue>> Pins;
	for (const URigVMPin* Pin : InNode->GetPins())
	{
		if (Pin->GetDirection() != ERigVMPinDirection::Input &&
			Pin->GetDirection() != ERigVMPinDirection::Visible &&
			Pin->GetDirection() != ERigVMPinDirection::IO &&
			Pin->GetDirection() != ERigVMPinDirection::Output)
		{
			continue;
		}

		if (TSharedPtr<FJsonObject> PinJsonObject = ToJson(InSettings, Pin))
		{
			Pins.Add(MakeShared<FJsonValueObject>(PinJsonObject));
		}
	}
	
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("Name"), InNode->GetName());
	JsonObject->SetStringField(TEXT("Title"), InNode->GetNodeTitle());
	JsonObject->SetStringField(TEXT("Type"), InNode->GetClass()->GetName());

	if (!InNode->IsInjected())
	{
		const FVector2D Position = InNode->GetPosition();
		JsonObject->SetStringField(TEXT("Position"), StructValueToJson(TBaseStructure<FVector2D>::Get(), &Position));
	}

	if (const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode))
	{
		JsonObject->SetStringField(TEXT("Function"), UnitNode->GetScriptStruct()->GetStructCPPName());
	}
	else if (const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		JsonObject->SetStringField(TEXT("Factory"), DispatchNode->GetFactory()->GetScriptStruct()->GetStructCPPName());
	}
	else if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode))
	{
		JsonObject->SetStringField(TEXT("Variable"), VariableNode->GetVariableName().ToString());
	}
	else if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		FString LibraryNodePath = FunctionReferenceNode->GetFunctionIdentifier().GetLibraryNodePath();
		if (const IRigVMEditorAssetInterface* Asset = FunctionReferenceNode->GetImplementingOuter<IRigVMEditorAssetInterface>())
		{
			if (const FRigVMClient* Client = Asset->GetRigVMClient())
			{
				if (const URigVMFunctionLibrary* FunctionLibrary = Client->GetFunctionLibrary())
				{
					const FString FunctionLibraryPrefix = FunctionLibrary->GetPathName() + TEXT(".");
					if (LibraryNodePath.StartsWith(FunctionLibraryPrefix))
					{
						LibraryNodePath = LibraryNodePath.Mid(FunctionLibraryPrefix.Len());
					}
				}
			}
		}
		
		JsonObject->SetStringField(TEXT("GraphFunction"), LibraryNodePath);
	}
	else if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode))
	{
		if (TSharedPtr<FJsonObject> ContainedGraphJsonObject = ToJson(InSettings, CollapseNode->GetContainedGraph()))
		{
			if (CollapseNode->GetGraph()->IsA<URigVMFunctionLibrary>())
			{
				JsonObject->SetStringField(TEXT("Category"), CollapseNode->GetNodeCategory());
				JsonObject->SetStringField(TEXT("Keywords"), CollapseNode->GetNodeKeywords());
				JsonObject->SetStringField(TEXT("Description"), CollapseNode->GetNodeDescription());
			}
			JsonObject->SetObjectField(TEXT("Graph"), ContainedGraphJsonObject);
		}
	}
	else if (const URigVMCommentNode* CommentNode = Cast<URigVMCommentNode>(InNode))
	{
		const FVector2D Size = InNode->GetSize();
		const FLinearColor Color = InNode->GetNodeColor();
		JsonObject->SetStringField(TEXT("Size"), StructValueToJson(TBaseStructure<FVector2D>::Get(), &Size));
		JsonObject->SetStringField(TEXT("Color"), StructValueToJson(TBaseStructure<FLinearColor>::Get(), &Color));
		JsonObject->SetStringField(TEXT("CommentText"), CommentNode->GetCommentText());
	}

	JsonObject->SetArrayField(TEXT("Pins"), Pins);
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const URigVMPin* InPin)
{
	check(InPin);
	
	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("Name"), InPin->GetName());
	JsonObject->SetStringField(TEXT("Type"), InPin->GetCPPType());
	if (InPin->GetDirection() != ERigVMPinDirection::Output)
	{
		JsonObject->SetStringField(TEXT("DefaultValue"), InPin->GetDefaultValue());
	}

	const TArray<URigVMInjectionInfo*> InjectedNodes = InPin->GetInjectedNodes();
	check(InjectedNodes.Num() < 2);
	if (!InjectedNodes.IsEmpty())
	{
		JsonObject->SetStringField(TEXT("InjectedNode"), InjectedNodes[0]->Node->GetName());
	}
	
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const URigVMLink* InLink)
{
	check(InLink);

	const URigVMPin* SourcePin = InLink->GetSourcePin();
	const URigVMPin* TargetPin = InLink->GetTargetPin();
	if (!SourcePin || !TargetPin)
	{
		return nullptr;
	}

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("Source"), SourcePin->GetPinPath());
	JsonObject->SetStringField(TEXT("Target"), TargetPin->GetPinPath());
	return JsonObject;
}

TSharedPtr<FJsonObject> RigVMJsonUtils::ToJson(FIntrospectionSettings& InSettings, const FRigVMGraphVariableDescription& InVariable)
{
	if (InVariable.Name.IsNone())
	{
		return nullptr;
	}

	static const TCHAR* True = TEXT("True");
	static const TCHAR* False = TEXT("False");

	TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
	JsonObject->SetStringField(TEXT("Name"), InVariable.Name.ToString());
	JsonObject->SetStringField(TEXT("TypeName"), InVariable.CPPType);
	JsonObject->SetStringField(TEXT("TypePath"), InVariable.CPPTypeObjectPath.IsNone() ? FString() : InVariable.CPPTypeObjectPath.ToString());
	JsonObject->SetStringField(TEXT("DefaultValue"), InVariable.DefaultValue);
	JsonObject->SetStringField(TEXT("Category"), InVariable.Category.ToString());
	JsonObject->SetStringField(TEXT("ToolTip"), PostProcessToolTip(InVariable.Tooltip.ToString()));
	JsonObject->SetStringField(TEXT("ExposedOnSpawn"), InVariable.bExposedOnSpawn ? True : False);
	JsonObject->SetStringField(TEXT("ExposeToCinematics"), InVariable.bExposeToCinematics ? True : False);
	JsonObject->SetStringField(TEXT("Public"), InVariable.bPublic ? True : False);
	JsonObject->SetStringField(TEXT("Private"), InVariable.bPrivate ? True : False);
	return JsonObject;
}

FString RigVMJsonUtils::StructValueToJson(const UScriptStruct* InScriptStruct, const void* InMemory)
{
	check(InScriptStruct);
	FString Value;
	InScriptStruct->ExportText(Value, InMemory, InMemory, nullptr, PPF_ExternalEditor, nullptr);
	return Value;
}

#undef LOCTEXT_NAMESPACE
