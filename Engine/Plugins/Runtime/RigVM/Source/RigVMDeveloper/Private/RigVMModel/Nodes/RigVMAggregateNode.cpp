// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMAggregateNode.h"

#include "RigVMModel/RigVMController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMAggregateNode)

URigVMAggregateNode::URigVMAggregateNode()
	: Super()
	, FirstInnerNodeCache(nullptr)
	, LastInnerNodeCache(nullptr)
{
}

bool URigVMAggregateNode::IsInputAggregate() const
{
	for (const URigVMNode* Node : GetContainedNodes())
	{
		if (!Node->IsA<URigVMFunctionEntryNode>() && !Node->IsA<URigVMFunctionReturnNode>())
		{
			if (const URigVMPin* FirstAggregatePin = Node->GetFirstAggregatePin())
			{
				return FirstAggregatePin->GetDirection() == ERigVMPinDirection::Input;
			}
		}
	}
	return false;
}

URigVMNode* URigVMAggregateNode::GetFirstInnerNode() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED

	if (FirstInnerNodeCache == nullptr)
	{
		if (GetContainedNodes().Num() < 3)
		{
			return nullptr;
		}
		
		if (IsInputAggregate())
		{
			const URigVMPin* FirstAggregatePin = GetFirstAggregatePin();
			const URigVMPin* SecondAggregatePin = GetSecondAggregatePin();
			if(FirstAggregatePin && SecondAggregatePin)
			{
				// Find node connected twice to the entry (through aggregate arguments)
				const FString Arg1Name = FirstAggregatePin->GetName();
				const FString Arg2Name = SecondAggregatePin->GetName();
				const URigVMFunctionEntryNode* EntryNode = GetEntryNode();
				TArray<URigVMNode*> ConnectedNodes;
				for (const URigVMPin* EntryPin : EntryNode->GetPins())
				{
					TArray<URigVMPin*> TargetPins = EntryPin->GetLinkedTargetPins();
					if (TargetPins.IsEmpty())
					{
						// If there are no target pins, they might not be reattached yet
						const TArray<URigVMLink*>& Links = GetContainedLinks();
						for (const URigVMLink* Link : Links)
						{
							if (Link->GetSourcePin() == EntryPin)
							{
								TargetPins.Add(Link->GetTargetPin());
								break;
							}
						}
					}
					if (TargetPins.Num() > 0)
					{
						if (TargetPins[0]->GetName() == Arg1Name || TargetPins[0]->GetName() == Arg2Name)
						{
							URigVMNode* TargetNode = TargetPins[0]->GetNode();
							if (ConnectedNodes.Contains(TargetNode))
							{
								FirstInnerNodeCache = TargetNode;
								return FirstInnerNodeCache;
							}
	
							ConnectedNodes.Add(TargetNode);
						}
					}
				}
			}
		}
		else
		{
			// Find node connected to entry throught the opposite aggregate argument
			const URigVMPin* OppositeAggregatePin = GetOppositeAggregatePin();
			if(OppositeAggregatePin)
			{
				const FString ArgOppositeName = OppositeAggregatePin->GetName();
				const URigVMFunctionEntryNode* EntryNode = GetEntryNode();
				if (const URigVMPin* EntryPin = EntryNode->FindPin(ArgOppositeName))
				{
					TArray<URigVMPin*> TargetPins = EntryPin->GetLinkedTargetPins();
					if (TargetPins.IsEmpty())
					{
						// If there are no target pins, they might not be reattached yet
						const TArray<URigVMLink*>& Links = GetContainedLinks();
						for (const URigVMLink* Link : Links)
						{
							if (Link->GetSourcePin() == EntryPin)
							{
								TargetPins.Add(Link->GetTargetPin());
								break;
							}
						}
					}
					if (TargetPins.Num() > 0)
					{
						FirstInnerNodeCache = TargetPins[0]->GetNode();
						return FirstInnerNodeCache;
					}
				}
			}
		}
	}
#endif
	return FirstInnerNodeCache;
}

URigVMNode* URigVMAggregateNode::GetLastInnerNode() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED

	if (LastInnerNodeCache == nullptr)
	{
		if(const URigVMPin* OppositeAggregatePin = GetOppositeAggregatePin())
		{
			const FString ArgOppositeName = OppositeAggregatePin->GetName();
			if (IsInputAggregate())
			{
				LastInnerNodeCache = GetAggregateInputs().Last()->GetLinkedTargetPins()[0]->GetNode();
			}
			else
			{
				LastInnerNodeCache = GetAggregateOutputs().Last()->GetLinkedSourcePins()[0]->GetNode();
			}
		}
	}
#endif
	return LastInnerNodeCache;
}

URigVMPin* URigVMAggregateNode::GetFirstAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	for (const URigVMNode* Node : GetContainedNodes())
	{
		if (!Node->IsA<URigVMFunctionEntryNode>() && !Node->IsA<URigVMFunctionReturnNode>())
		{
			return Node->GetFirstAggregatePin();
		}
	}
#endif
	return nullptr;
}

URigVMPin* URigVMAggregateNode::GetSecondAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	for (const URigVMNode* Node : GetContainedNodes())
	{
		if (!Node->IsA<URigVMFunctionEntryNode>() && !Node->IsA<URigVMFunctionReturnNode>())
		{
			return Node->GetSecondAggregatePin();
		}
	}
#endif
	return nullptr;
}

URigVMPin* URigVMAggregateNode::GetOppositeAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	for (const URigVMNode* Node : GetContainedNodes())
	{
		if (!Node->IsA<URigVMFunctionEntryNode>() && !Node->IsA<URigVMFunctionReturnNode>())
		{
			return Node->GetOppositeAggregatePin();
		}
	}
#endif
	return nullptr;
}

TArray<URigVMPin*> URigVMAggregateNode::GetAggregateInputs() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	
	TArray<URigVMPin*> Result;
	const FString Arg1Name = GetFirstAggregatePin()->GetName();
	const FString Arg2Name = GetSecondAggregatePin()->GetName();

	// Find pins connected to an aggregate pin
	const URigVMFunctionEntryNode* EntryNode = GetEntryNode();
	for(URigVMPin* EntryPin : EntryNode->GetPins())
	{
		const TArray<URigVMPin*> SourcePins = EntryPin->GetLinkedTargetPins();
		if(SourcePins.Num() > 0)
		{
			const URigVMPin* SourcePin = SourcePins[0];
			if(SourcePin->GetName() == Arg1Name || SourcePin->GetName() == Arg2Name)
			{
				Result.Add(EntryPin);
			}
		}
	}
	return Result;
	
#else
	return Super::GetAggregateInputs();
#endif
}

TArray<URigVMPin*> URigVMAggregateNode::GetAggregateOutputs() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED

	TArray<URigVMPin*> Result;
	const FString Arg1Name = GetFirstAggregatePin()->GetName();
	const FString Arg2Name = GetSecondAggregatePin()->GetName();

	// Find pins connected to an aggregate pin
	const URigVMFunctionReturnNode* ReturnNode = GetReturnNode();
	for(URigVMPin* ReturnPin : ReturnNode->GetPins())
	{
		const TArray<URigVMPin*> SourcePins = ReturnPin->GetLinkedSourcePins();
		if(SourcePins.Num() > 0)
		{
			const URigVMPin* SourcePin = SourcePins[0];
			if(SourcePin->GetName() == Arg1Name || SourcePin->GetName() == Arg2Name)
			{
				Result.Add(ReturnPin);
			}
		}
	}
	return Result;
	
#else
	return Super::GetAggregateOutputs();
#endif
}

FString URigVMAggregateNode::GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const
{
	if(InRootPin->CanProvideDefaultValue())
	{
		if(const URigVMFunctionEntryNode* EntryNode = GetEntryNode())
		{
			if(const URigVMPin* EntryPin = EntryNode->FindPin(InRootPin->GetName()))
			{
				if(!EntryPin->GetLinks().IsEmpty())
				{
					// linked targets only since only input pins on the outer node are valid for default values
					if(const URigVMPin* InnerRootPin = EntryPin->GetLinks()[0]->GetTargetPin())
					{
						return InnerRootPin->GetOriginalDefaultValue();
					}
				}
			}
		}
	}
	return Super::GetOriginalDefaultValueForRootPin(InRootPin);
}

void URigVMAggregateNode::InvalidateCache()
{
	Super::InvalidateCache();
	FirstInnerNodeCache = nullptr;
	LastInnerNodeCache = nullptr;
}

FString URigVMAggregateNode::GetNodeTitle() const
{
	if (const URigVMNode* InnerNode = GetFirstInnerNode())
	{
		return InnerNode->GetNodeTitle();
	}
	
	return Super::GetNodeTitle();
}

FLinearColor URigVMAggregateNode::GetNodeColor() const
{
	if (const URigVMNode* InnerNode = GetFirstInnerNode())
	{
		return InnerNode->GetNodeColor();
	}
	
	return Super::GetNodeColor();
}

FName URigVMAggregateNode::GetMethodName() const
{
	if (const URigVMUnitNode* InnerNode = Cast<URigVMUnitNode>(GetFirstInnerNode()))
	{
		return InnerNode->GetMethodName();
	}
	
	return NAME_None;
}

FText URigVMAggregateNode::GetToolTipText() const
{
	if (const URigVMNode* InnerNode = GetFirstInnerNode())
	{
		return InnerNode->GetToolTipText();
	}
	
	return Super::GetToolTipText();
}

FText URigVMAggregateNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	if (const URigVMPin* EntryPin = GetContainedGraph()->GetEntryNode()->FindPin(InPin->GetName()))
	{
		const TArray<URigVMPin*> Targets = EntryPin->GetLinkedTargetPins();
		if (Targets.Num() > 0)
		{
			return Targets[0]->GetToolTipText();
		}
	}
	else if(const URigVMPin* ReturnPin = GetContainedGraph()->GetReturnNode()->FindPin(InPin->GetName()))
	{
		const TArray<URigVMPin*> Sources = ReturnPin->GetLinkedSourcePins();
		if (Sources.Num() > 0)
		{
			return Sources[0]->GetToolTipText();
		}
	}
	
	return Super::GetToolTipTextForPin(InPin);
}

