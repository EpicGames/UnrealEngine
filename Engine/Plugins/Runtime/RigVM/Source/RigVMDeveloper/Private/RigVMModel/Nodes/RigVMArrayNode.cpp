// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMArrayNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCore/RigVMStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMArrayNode)

UDEPRECATED_RigVMArrayNode::UDEPRECATED_RigVMArrayNode()
: OpCode(ERigVMOpCode::Invalid)
{
}

ERigVMOpCode UDEPRECATED_RigVMArrayNode::GetOpCode() const
{
	return OpCode;
}

FString UDEPRECATED_RigVMArrayNode::GetCPPType() const
{
	const URigVMPin* ArrayPin = FindPin(TEXT("Array"));
	if (ArrayPin == nullptr)
	{
		return FString();
	}
	return RigVMTypeUtils::BaseTypeFromArrayType(ArrayPin->GetCPPType());
}

UObject* UDEPRECATED_RigVMArrayNode::GetCPPTypeObject() const
{
	const URigVMPin* ArrayPin = FindPin(TEXT("Array"));
	if (ArrayPin == nullptr)
	{
		return nullptr;
	}
	return ArrayPin->GetCPPTypeObject();
}
