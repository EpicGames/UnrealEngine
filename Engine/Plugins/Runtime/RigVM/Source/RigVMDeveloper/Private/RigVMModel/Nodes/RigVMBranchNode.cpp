// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMBranchNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMBranchNode)

