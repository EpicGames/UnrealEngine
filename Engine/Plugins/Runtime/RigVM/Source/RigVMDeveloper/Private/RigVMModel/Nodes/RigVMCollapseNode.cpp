// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMCollapseNode)

URigVMCollapseNode::URigVMCollapseNode()
	: URigVMLibraryNode()
{
	NodeColor = FLinearColor::White;
}

FText URigVMCollapseNode::GetToolTipText() const
{
	const FString ToolTipString = GetNodeDescription();
	if(!ToolTipString.IsEmpty())
	{
		return FText::FromString(ToolTipString);
	}
	return Super::GetToolTipText();
}

URigVMFunctionLibrary* URigVMCollapseNode::GetLibrary() const
{
	return Cast<URigVMFunctionLibrary>(GetOuter());
}

bool URigVMCollapseNode::IsPure() const
{
	if (IsGraphFunctionDefinition())
	{
		return FindExecutePin() == nullptr;
	}
	return false;
}

TArray<FRigVMExternalVariable> URigVMCollapseNode::GetExternalVariables() const
{
	TArray<FRigVMExternalVariable> ExternalVariables;
	if (IsGraphFunctionDefinition())
	{
		for (const URigVMPin* Pin : GetPins())
		{
			if (Pin->IsDefinedAsInputVariable())
			{
				FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, Pin->GetInputVariableDescription().ToExternalVariable());
			}
		}
	}
	const TArray<FRigVMExternalVariable> SuperExternalVariables = Super::GetExternalVariables();
	for (const FRigVMExternalVariable& SuperExternalVariable : SuperExternalVariables)
	{
		FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, SuperExternalVariable);
	}
	return ExternalVariables;
}

FString URigVMCollapseNode::GetEditorSubGraphName() const
{
	return FString::Printf(TEXT("%s_SubGraph"), *GetName());
}

FGuid URigVMCollapseNode::FindPinGuid(const URigVMPin* InPin) const
{
	if (URigVMPin* RootPin = InPin->GetRootPin())
	{
		if (const FGuid* Guid = PinNameToGuid.Find(RootPin->GetFName()))
		{
			return *Guid;
		}
	}
	return FGuid();
}
