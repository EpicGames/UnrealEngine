// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMCommentNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMCommentNode)

URigVMCommentNode::URigVMCommentNode()
{
	Size = FVector2D(400.f, 300.f);
	FontSize = 18;
	bBubbleVisible = false;
	bColorBubble = false;
}

FString URigVMCommentNode::GetCommentText() const
{
	return CommentText;
}

int32 URigVMCommentNode::GetCommentFontSize() const
{
	return FontSize;
}

bool URigVMCommentNode::GetCommentBubbleVisible() const
{
	return bBubbleVisible;
}

bool URigVMCommentNode::GetCommentColorBubble() const
{
	return bColorBubble;
}

