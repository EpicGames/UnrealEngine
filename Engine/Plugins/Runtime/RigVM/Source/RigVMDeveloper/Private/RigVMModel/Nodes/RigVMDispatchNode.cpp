// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMUserWorkflowRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDispatchNode)

FString URigVMDispatchNode::GetNodeTitle() const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetNodeTitle(GetTemplatePinTypeMap());
	}
	return Super::GetNodeTitle();
}

bool URigVMDispatchNode::SupportsRenaming() const
{
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->SupportsRenaming();
	}
	return Super::SupportsRenaming();
}

FText URigVMDispatchNode::GetToolTipText() const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		FText ToolTipText = Factory->GetNodeTooltip(GetTemplatePinTypeMap());
		if(!ToolTipText.IsEmpty())
		{
			return ToolTipText;
		}
	}
	return URigVMNode::GetToolTipText();
}

FLinearColor URigVMDispatchNode::GetNodeColor() const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetNodeColor();
	}
	return URigVMNode::GetNodeColor();
}

bool URigVMDispatchNode::IsDefinedAsConstant() const
{
	if (const UScriptStruct* Struct = GetFactoryStruct())
	{
		return Struct->HasMetaData(FRigVMStruct::ConstantMetaName);
	}
	return false;
}

bool URigVMDispatchNode::IsDefinedAsVarying() const
{
	if (const UScriptStruct* Struct = GetFactoryStruct())
	{
		return Struct->HasMetaData(FRigVMStruct::VaryingMetaName);
	}
	return false;
}

bool URigVMDispatchNode::IsPure() const
{
	if (const UScriptStruct* Struct = GetFactoryStruct())
	{
		if (FRigVMStruct::HasMetaData(Struct, FRigVMStruct::PureMetaName))
		{
			return true;
		}
	}
	return Super::IsPure();
}

const TArray<FName>& URigVMDispatchNode::GetControlFlowBlocks() const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetControlFlowBlocks(GetDispatchContext(), GetTemplatePinTypeMap());
	}
	return Super::GetControlFlowBlocks();
}

const bool URigVMDispatchNode::IsControlFlowBlockSliced(const FName& InBlockName) const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->IsControlFlowBlockSliced(InBlockName);
	}
	return Super::IsControlFlowBlockSliced(InBlockName);
}

FText URigVMDispatchNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	if (const FRigVMDispatchFactory* Factory = GetFactory())
	{
		const URigVMPin* RootPin = InPin->GetRootPin();
		FText ToolTipText = Factory->GetArgumentTooltip(RootPin->GetFName(), RootPin->GetTypeIndex());
		if(!ToolTipText.IsEmpty())
		{
			return ToolTipText;
		}
	}
	return URigVMNode::GetToolTipTextForPin(InPin);
}

bool URigVMDispatchNode::IsOutDated() const
{
	return !GetDeprecatedMetadata().IsEmpty();
}

FString URigVMDispatchNode::GetDeprecatedMetadata() const
{
	if (const UScriptStruct* Struct = GetFactoryStruct())
	{
		FString DeprecatedMetadata;
		if(Struct->GetStringMetaDataHierarchical(FRigVMStruct::DeprecatedMetaName, &DeprecatedMetadata))
		{
			return DeprecatedMetadata;
		}
	}
	return FString();
}

FRigVMDispatchContext URigVMDispatchNode::GetDispatchContext() const
{
	FRigVMDispatchContext Context;
	Context.Instance = ConstructFactoryInstance(false, &Context.StringRepresentation);
	Context.Subject = this;
	return Context;
}

TSharedPtr<FStructOnScope> URigVMDispatchNode::ConstructFactoryInstance(bool bUseDefault, FString* OutFactoryDefault) const
{
	if (UScriptStruct* Struct = (UScriptStruct*)GetFactoryStruct())
	{
		TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Struct));
		FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		if (!bUseDefault)
		{
			const FString FactoryDefaultValue = GetFactoryDefaultValue();
			if(OutFactoryDefault)
			{
				*OutFactoryDefault = FactoryDefaultValue;
			}
			Struct->ImportText(*FactoryDefaultValue, StructMemory, nullptr, PPF_IncludeTransient, GLog, Struct->GetName());
		}
		return StructOnScope;
	}
	return nullptr;
}

TArray<URigVMPin*> URigVMDispatchNode::GetAggregateInputs() const
{
	TArray<URigVMPin*> AggregateInputs;
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		const TArray<FName> ArgumentNames = Factory->GetAggregateInputArguments();
		for(const FName& ArgumentName : ArgumentNames)
		{
			AggregateInputs.Add(FindPin(ArgumentName.ToString()));
		}
	}
#endif
	return AggregateInputs;
}

TArray<URigVMPin*> URigVMDispatchNode::GetAggregateOutputs() const
{
	TArray<URigVMPin*> AggregateOutputs;
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		const TArray<FName> ArgumentNames = Factory->GetAggregateOutputArguments();
		for(const FName& ArgumentName : ArgumentNames)
		{
			AggregateOutputs.Add(FindPin(ArgumentName.ToString()));
		}
	}
#endif
	return AggregateOutputs;
}

FName URigVMDispatchNode::GetNextAggregateName(const FName& InLastAggregatePinName) const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetNextAggregateName(InLastAggregatePinName);
	}
#endif
	return FName();
}

const FRigVMDispatchFactory* URigVMDispatchNode::GetFactory() const
{
	if(CachedFactory)
	{
		return CachedFactory;
	}
	
	if(const FRigVMTemplate* Template = GetTemplate())
	{
		CachedFactory = Template->GetDispatchFactory(); 
		return CachedFactory;
	}

	// Try to find factories that have not been registered yet
	FRigVMRegistry::Get().RefreshEngineTypes();
	if(const FRigVMTemplate* Template = GetTemplate())
	{
		CachedFactory = Template->GetDispatchFactory(); 
		return CachedFactory;
	}
	
	return nullptr;
}

const UScriptStruct* URigVMDispatchNode::GetFactoryStruct() const
{
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetScriptStruct();
	}
	return nullptr;
}

void URigVMDispatchNode::InvalidateCache()
{
	Super::InvalidateCache();
	CachedFactory = nullptr;
}

bool URigVMDispatchNode::ShouldInputPinComputeLazily(const URigVMPin* InPin) const
{
	const URigVMPin* RootPin = InPin->GetRootPin();
	check(RootPin->GetNode() == this);

	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->HasArgumentMetaData(RootPin->GetFName(), FRigVMStruct::ComputeLazilyMetaName); 
	}
	return Super::ShouldInputPinComputeLazily(InPin);
}

bool URigVMDispatchNode::IsPinConstant(const URigVMPin* InPin) const
{
	const URigVMPin* RootPin = InPin->GetRootPin();
	check(RootPin->GetNode() == this);

	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->HasArgumentMetaData(RootPin->GetFName(), FRigVMStruct::ConstantMetaName); 
	}
	return Super::IsPinConstant(InPin);
}

FString URigVMDispatchNode::GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const
{
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		if(InRootPin->CanProvideDefaultValue())
		{
			return Factory->GetArgumentDefaultValue(InRootPin->GetFName(), InRootPin->GetTypeIndex());
		}
	}
	return Super::GetOriginalDefaultValueForRootPin(InRootPin);
}

FString URigVMDispatchNode::GetFactoryDefaultValue() const
{
	TArray<FString> PinDefaultValues;
	for (URigVMPin* Pin : GetPins())
	{
		if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			continue;
		}
		FString PinDefaultValue = Pin->GetDefaultValue();
		if (Pin->IsStringType())
		{
			PinDefaultValue = TEXT("\"") + PinDefaultValue + TEXT("\"");
		}
		else if (PinDefaultValue.IsEmpty() || PinDefaultValue == TEXT("()"))
		{
			continue;
		}
		PinDefaultValues.Add(FString::Printf(TEXT("%s=%s"), *Pin->GetName(), *PinDefaultValue));
	}
	if (PinDefaultValues.Num() == 0)
	{
		return TEXT("()");
	}
	return FString::Printf(TEXT("(%s)"), *FString::Join(PinDefaultValues, TEXT(",")));
}

FRigVMStructUpgradeInfo URigVMDispatchNode::GetUpgradeInfo() const
{
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetUpgradeInfo(GetTemplatePinTypeMap(), GetDispatchContext());
	}
	return FRigVMStructUpgradeInfo();
}

FName URigVMDispatchNode::GetDisplayNameForPin(const URigVMPin* InPin) const
{
	if(const FRigVMDispatchFactory* Factory = GetFactory())
	{
		return Factory->GetDisplayNameForArgument(*InPin->GetSubPinPath(InPin->GetRootPin(), true), GetTemplatePinTypeMap());
	}
	return Super::GetDisplayNameForPin(InPin);
}

