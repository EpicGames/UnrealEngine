// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMEnumNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEnumNode)

const TCHAR* URigVMEnumNode::EnumName = TEXT("Enum");

URigVMEnumNode::URigVMEnumNode()
{
}

FString URigVMEnumNode::GetNodeTitle() const
{
	if(UEnum* Enum = GetEnum())
	{
		return FString::Printf(TEXT("Enum %s"), *Enum->GetName());
	}
	return TEXT("Enum");
}

UEnum* URigVMEnumNode::GetEnum() const
{
	return Cast<UEnum>(GetCPPTypeObject());
}

FString URigVMEnumNode::GetCPPType() const
{
	URigVMPin* EnumValuePin = FindPin(EnumValueName);
	if (EnumValuePin == nullptr)
	{
		return FString();
	}
	return EnumValuePin->GetCPPType();
}

UObject* URigVMEnumNode::GetCPPTypeObject() const
{
	URigVMPin* EnumValuePin = FindPin(EnumValueName);
	if (EnumValuePin == nullptr)
	{
		return nullptr;
	}
	return EnumValuePin->GetCPPTypeObject();
}

FString URigVMEnumNode::GetDefaultValue(const URigVMPin::FPinOverride& InOverride) const
{
	URigVMPin* EnumValuePin = FindPin(EnumValueName);
	if (EnumValuePin == nullptr)
	{
		return FString();
	}
	return EnumValuePin->GetDefaultValue(InOverride);
}

