// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunctionEntryNode)

bool URigVMFunctionEntryNode::IsWithinLoop() const
{
	if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetGraph()->GetOuter()))
	{
		return CollapseNode->IsWithinLoop();
	}
	return Super::IsWithinLoop();
}

FString URigVMFunctionEntryNode::GetNodeTitle() const
{
	return TEXT("Entry");
}
