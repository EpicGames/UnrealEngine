// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMFunctionInterfaceNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunctionInterfaceNode)

uint32 URigVMFunctionInterfaceNode::GetStructureHash() const
{
	// Avoid hashing the template for library nodes
	return URigVMNode::GetStructureHash();
}

FLinearColor URigVMFunctionInterfaceNode::GetNodeColor() const
{
	if(URigVMGraph* RootGraph = GetRootGraph())
	{
		if(RootGraph->IsA<URigVMFunctionLibrary>())
		{
			return FLinearColor(FColor::FromHex("CB00FFFF"));
		}
	}
	return FLinearColor(FColor::FromHex("005DFFFF"));
}

bool URigVMFunctionInterfaceNode::IsDefinedAsVarying() const
{ 
	return true; 
}

FText URigVMFunctionInterfaceNode::GetToolTipText() const
{
	return FText::FromName(GetGraph()->GetOuter()->GetFName());
}

FText URigVMFunctionInterfaceNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	FText ToolTip;
	if(const URigVMPin* OuterPin = FindReferencedPin(InPin))
	{
		ToolTip = OuterPin->GetToolTipText();
	}
	else
	{
		ToolTip = Super::GetToolTipTextForPin(InPin);
	}

	if (!IsInterfacePinUsed(InPin->GetRootPin()->GetFName()))
	{
		static const FText Format = NSLOCTEXT("RigVMFunctionInterfaceNode", "PinIsUnused", "{0}\n\nThis pin is not used in this function (grayed out).");
		ToolTip = FText::Format(Format, ToolTip);
	}
	
	return ToolTip;
}

bool URigVMFunctionInterfaceNode::IsInterfacePinUsed(const FName& InInterfacePinName) const
{
	const URigVMPin* RootPin = FindRootPinByName(InInterfacePinName);
	if (RootPin == nullptr)
	{
		return false;
	}
	
	// if we have links - early exit
	if (RootPin->IsLinked(false))
	{
		return true;
	}

	// if we potentially have links anywhere - stick with normal color
	if (RootPin->IsLinked(true))
	{
		return true;
	}

	// only input arguments need to check bound variables
	// (input arguments are output pins in the contained graph)
	if (RootPin->GetDirection() != ERigVMPinDirection::Output)
	{
		return false;
	}

	// if there are any pins bound to this one...
	if (const URigVMGraph* Graph = RootPin->GetGraph())
	{
		for (const URigVMNode* Node : Graph->GetNodes())
		{
			const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node);
			if (!VariableNode)
			{
				continue;
			}
			if (VariableNode->GetVariableName() != RootPin->GetName())
			{
				continue;
			}

			// we found a variable node that sets the interface pin value
			if (!VariableNode->IsGetter())
			{
				return true;
			}

			// we found a variable node that's bound to our input pin
			if (VariableNode->GetValuePin()->IsLinked(true))
			{
				return true;
			}
		}
	}

	return false;
}

const URigVMCollapseNode* URigVMFunctionInterfaceNode::FindReferencedNode() const
{
	return Cast<URigVMCollapseNode>(GetGraph()->GetOuter());
}

const URigVMPin* URigVMFunctionInterfaceNode::FindReferencedPin(const URigVMPin* InPin) const
{
	return FindReferencedPin(InPin->GetSegmentPath(true));
}

const URigVMPin* URigVMFunctionInterfaceNode::FindReferencedPin(const FString& InPinPath) const
{
	if(const URigVMCollapseNode* OuterNode = FindReferencedNode())
	{
		return OuterNode->FindPin(InPinPath);
	}
	return nullptr;
}
