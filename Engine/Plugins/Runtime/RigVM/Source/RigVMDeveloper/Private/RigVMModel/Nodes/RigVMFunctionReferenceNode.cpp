// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMEditorAsset.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "Widgets/SRigVMEditorGraphExplorerTreeView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunctionReferenceNode)

FString URigVMFunctionReferenceNode::GetNodeTitle() const
{
	return ReferencedFunctionHeader.NodeTitle;
}

FLinearColor URigVMFunctionReferenceNode::GetNodeColor() const
{
	return ReferencedFunctionHeader.NodeColor;
}

FText URigVMFunctionReferenceNode::GetToolTipText() const
{
	return ReferencedFunctionHeader.GetTooltip();
}

FName URigVMFunctionReferenceNode::GetDisplayNameForPin(const URigVMPin* InPin) const
{
	check(InPin);
	
	const FString PinPath = InPin->GetPinPath();
	if(const FString* DisplayName = ReferencedFunctionHeader.Layout.FindDisplayName(PinPath))
	{
		if(!DisplayName->IsEmpty())
		{
			return *(*DisplayName);
		}
	}

	if(InPin->IsRootPin())
	{
		const FRigVMGraphFunctionArgument* Argument = ReferencedFunctionHeader.Arguments.FindByPredicate([InPin](const FRigVMGraphFunctionArgument& Argument)
		{
			return Argument.Name == InPin->GetFName();
		});

		if (Argument && !Argument->DisplayName.IsNone())
		{
			return Argument->DisplayName;
		}
	}

	if (IsReferencedNodeLoaded())
	{
		if(const URigVMPin* ReferencedPin = FindReferencedPin(InPin))
		{
			return ReferencedPin->GetDisplayName();
		}
	}

	return Super::GetDisplayNameForPin(InPin);
}

FString URigVMFunctionReferenceNode::GetCategoryForPin(const FString& InPinPath) const
{
	if(const FString* Category = ReferencedFunctionHeader.Layout.FindCategory(InPinPath))
	{
		if(!Category->IsEmpty())
		{
			return *Category;
		}
	}

	if (IsReferencedNodeLoaded())
	{
		if(const URigVMPin* ReferencedPin = FindReferencedPin(InPinPath))
		{
			return ReferencedPin->GetCategory();
		}
	}

	return Super::GetCategoryForPin(InPinPath);
}

int32 URigVMFunctionReferenceNode::GetIndexInCategoryForPin(const FString& InPinPath) const
{
	if(const int32* Index = ReferencedFunctionHeader.Layout.PinIndexInCategory.Find(InPinPath))
	{
		return *Index;
	}

	if (IsReferencedNodeLoaded())
	{
		if(const URigVMPin* ReferencedPin = FindReferencedPin(InPinPath))
		{
			return ReferencedPin->GetIndexInCategory();
		}
	}

	return INDEX_NONE;
}

FString URigVMFunctionReferenceNode::GetNodeCategory() const
{
	return ReferencedFunctionHeader.Category;
}

FString URigVMFunctionReferenceNode::GetNodeKeywords() const
{
	return ReferencedFunctionHeader.Keywords;
}

bool URigVMFunctionReferenceNode::ReferencesFunctionInSamePackage() const
{
	const FString ThisPackagePath = GetPackage()->GetPathName();
	
	// avoid function reference related validation for temp assets, a temp asset may get generated during
	// certain content validation process. It is usually just a simple file-level copy of the source asset
	// so these references are usually not fixed-up properly. Thus, it is meaningless to validate them.
	if (ThisPackagePath.StartsWith(TEXT("/Temp/")))
	{
		return true;
	}
	
	const FRigVMGraphFunctionIdentifier& LibraryPointer = ReferencedFunctionHeader.LibraryPointer;
	const FString& LibraryPackagePath = LibraryPointer.GetNodeSoftPath().GetLongPackageName();
	return LibraryPackagePath == ThisPackagePath;
}

bool URigVMFunctionReferenceNode::RequiresVariableRemapping() const
{
	if (GetPins().ContainsByPredicate([](const URigVMPin* Pin) -> bool
	{
		return Pin->IsDefinedAsInputVariable();
	}))
	{
		return true;
	}
	if (ReferencesFunctionInSamePackage())
	{
		return false;
	}
	return !GetExternalVariables().IsEmpty();
}

bool URigVMFunctionReferenceNode::IsFullyRemapped() const
{
	for (const URigVMPin* Pin : GetPins())
	{
		if (Pin->IsDefinedAsInputVariable())
		{
			if (GetOuterVariableName(Pin->GetFName()).IsNone())
			{
				return false;
			}
		}
	}

	if (!ReferencesFunctionInSamePackage())
	{
		TArray<FRigVMExternalVariable> ExternalVariables = GetExternalVariables();
		for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
		{
			if (GetOuterVariableName(ExternalVariable.GetName()).IsNone())
			{
				return false;
			}
		}
	}
	return true;
}

TArray<FRigVMExternalVariable> URigVMFunctionReferenceNode::GetExternalVariables() const
{
	if (const FRigVMGraphFunctionData* FunctionData = GetReferencedFunctionData())
	{
		return FunctionData->Header.ExternalVariables;
	}
	return Super::GetExternalVariables();
}

const TMap<FName, FName> URigVMFunctionReferenceNode::GetVariableMap() const
{
	TMap<FName, FName> Result =  VariableMap_DEPRECATED;
	for (const URigVMPin* InterfacePin : GetPins())
	{
		if (!InterfacePin->IsDefinedAsInputVariable())
		{
			continue;
		}
		const FString BoundVariable = InterfacePin->GetBoundVariableName();
		if (BoundVariable.IsEmpty())
		{
			Result.FindOrAdd(InterfacePin->GetFName(), NAME_None);
		}
		else
		{
			Result.FindOrAdd(InterfacePin->GetFName(), *BoundVariable);
		}
	}
	return Result;
}

FName URigVMFunctionReferenceNode::GetOuterVariableName(const FName& InInnerVariableName) const
{
	FString VariablePath = GetOuterVariablePath(InInnerVariableName);
	if (VariablePath.IsEmpty())
	{
		return NAME_None;
	}
	FString VariableName = VariablePath, VariableSubPath;
	(void)VariablePath.Split(TEXT("."), &VariableName, &VariableSubPath);
	return *VariableName;
}

FString URigVMFunctionReferenceNode::GetOuterVariablePath(const FName& InInnerVariableName) const
{
	if (const URigVMPin* InterfacePin = FindRootPinByName(InInnerVariableName))
	{
		if (InterfacePin->IsDefinedAsInputVariable())
		{
			const FString BoundVariablePath = InterfacePin->GetBoundVariablePath();
			if (!BoundVariablePath.IsEmpty())
			{
				return BoundVariablePath;
			}
		}
	}
	if(const FName* OuterVariableName = VariableMap_DEPRECATED.Find(InInnerVariableName))
	{
		return OuterVariableName->ToString();
	}
	
	// we may be on a function reference within the same asset.
	// in that case we allow to reference asset wide variables.
	if (ReferencesFunctionInSamePackage())
	{
		const TArray<FRigVMExternalVariable> AssetVariables = GetAssetExternalVariables();
		if (AssetVariables.ContainsByPredicate(
		[InInnerVariableName](const FRigVMExternalVariable& Variable) -> bool
			{
				return Variable.GetName() == InInnerVariableName;
			}))
		{
			return InInnerVariableName.ToString();
		}
	}
	return FString();
}

uint32 URigVMFunctionReferenceNode::GetStructureHash() const
{
	uint32 Hash = Super::GetStructureHash();

	const FRigVMRegistryWriteLock Registry;
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.Name.ToString()));
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.NodeTitle));
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.LibraryPointer.GetLibraryNodePath()));
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.Keywords));
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.Description));
	Hash = HashCombine(Hash, GetTypeHash(ReferencedFunctionHeader.NodeColor));

	for(const FRigVMGraphFunctionArgument& Argument : ReferencedFunctionHeader.Arguments)
	{
		Hash = HashCombine(Hash, GetTypeHash(Argument.Name.ToString()));
		Hash = HashCombine(Hash, GetTypeHash((int32)Argument.Direction));
		const TRigVMTypeIndex TypeIndex = Registry->GetTypeIndexFromCPPType_NoLock(Argument.CPPType.ToString());
		Hash = HashCombine(Hash, Registry->GetHashForType_NoLock(TypeIndex));

		for(const TPair<FString, FText>& Pair : Argument.PathToTooltip)
		{
			Hash = HashCombine(Hash, GetTypeHash(Pair.Key));
			Hash = HashCombine(Hash, GetTypeHash(Pair.Value.ToString()));
		}
	}

	const TArray<FRigVMExternalVariable> ExternalVariables = ReferencedFunctionHeader.GetExternalVariables();
	for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
	{
		Hash = HashCombine(Hash, GetTypeHash(ExternalVariable.GetName().ToString()));
		const TRigVMTypeIndex TypeIndex = Registry->GetTypeIndexFromCPPType_NoLock(ExternalVariable.GetExtendedCPPType().ToString());
		Hash = HashCombine(Hash, Registry->GetHashForType_NoLock(TypeIndex));
	}

	return Hash;
}

void URigVMFunctionReferenceNode::UpdateFunctionHeaderFromHost()
{
	if (const FRigVMGraphFunctionData* Data = GetReferencedFunctionData())
	{
		const_cast<FRigVMGraphFunctionData*>(Data)->PatchExternalVariablesToArguments();
		
		ReferencedFunctionHeader = Data->Header;
		InvalidateCache();
	}
}

const FRigVMGraphFunctionData* URigVMFunctionReferenceNode::GetReferencedFunctionData(bool bLoadIfNecessary) const
{
	if (TScriptInterface<IRigVMGraphFunctionHost> Host = ReferencedFunctionHeader.GetFunctionHostObject(bLoadIfNecessary))
	{
		return Host->GetRigVMGraphFunctionStore()->FindFunction(ReferencedFunctionHeader.LibraryPointer);
	}
	return nullptr;
}

bool URigVMFunctionReferenceNode::SupportsCallable() const
{
	if (const FRigVMGraphFunctionData* Data = GetReferencedFunctionData())
	{
		return Data->CompilationData.SupportsCallable();
	}
	return false;
}

TArray<FRigVMTag> URigVMFunctionReferenceNode::GetVariantTags() const
{
	if (const FRigVMGraphFunctionData* Data = GetReferencedFunctionData(false))
	{
		return Data->Header.Variant.Tags;
	}
	return GetReferencedFunctionHeader().Variant.Tags;
}

FRigVMReferenceNodeData URigVMFunctionReferenceNode::GetReferenceNodeData() const
{
	FRigVMReferenceNodeData NodeData;
	NodeData.ReferenceNodePath = TSoftObjectPtr<const URigVMFunctionReferenceNode>(this).ToString();
	NodeData.ReferencedFunctionIdentifier = GetReferencedFunctionHeader().LibraryPointer;
	return NodeData;
}

FString URigVMFunctionReferenceNode::GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const
{
	if(InRootPin->CanProvideDefaultValue())
	{
		const FRigVMGraphFunctionHeader& Header = GetReferencedFunctionHeader();
		if(!Header.IsValid())
		{
			// we don't have a valid header yet
			// maybe still waiting for the function reference to resolve?
			return FString();
		}
		const FRigVMGraphFunctionArgument* Argument = Header.Arguments.FindByPredicate([InRootPin](const FRigVMGraphFunctionArgument& InArgument) -> bool
		{
			return InArgument.Name == InRootPin->GetFName();
		});
		if(Argument)
		{
			return Argument->DefaultValue;
		}
	}
	return Super::GetOriginalDefaultValueForRootPin(InRootPin);
}

TArray<FRigVMExternalVariable> URigVMFunctionReferenceNode::GetAssetExternalVariables() const
{
	if (const IRigVMAssetInterface* Asset = GetImplementingOuter<IRigVMAssetInterface>())
	{
		const TArray<FRigVMGraphVariableDescription> VariableDescriptions = Asset->GetAssetVariables();
		TArray<FRigVMExternalVariable> ExternalVariables;
		for (const FRigVMGraphVariableDescription& VariableDescription : VariableDescriptions)
		{
			FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, VariableDescription.ToExternalVariable());
		}
		return ExternalVariables;
	}
	if (IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if (FRigVMClient* Client = ClientHost->GetRigVMClient())
		{
			if (const URigVMController* Controller = Client->GetOrCreateController(GetGraph()))
			{
				if (Controller->GetExternalVariablesDelegate.IsBound())
				{
					return Controller->GetExternalVariablesDelegate.Execute(GetGraph());
				}
			}
		}
	}
	return {};
}

FText URigVMFunctionReferenceNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	check(InPin);

	URigVMPin* RootPin = InPin->GetRootPin();
	const FRigVMGraphFunctionArgument* Argument = ReferencedFunctionHeader.Arguments.FindByPredicate([RootPin](const FRigVMGraphFunctionArgument& Argument)
	{
		return Argument.Name == RootPin->GetFName();
	});

	if (Argument)
	{
		if (const FText* Tooltip = Argument->PathToTooltip.Find(InPin->GetSegmentPath(false)))
		{
			return *Tooltip;
		}
	}

	if (IsReferencedNodeLoaded())
	{
		if(const URigVMPin* ReferencedPin = FindReferencedPin(InPin))
		{
			return ReferencedPin->GetToolTipText();
		}
	}

	return Super::GetToolTipTextForPin(InPin);
}

TArray<FString> URigVMFunctionReferenceNode::GetPinCategories() const
{
	FRigVMNodeLayout ReferenceLayout;
	const FRigVMNodeLayout* Layout = &ReferencedFunctionHeader.Layout;
	if(IsReferencedNodeLoaded())
	{
		URigVMLibraryNode* ReferencedNode = LoadReferencedNode();
		if (ensure(ReferencedNode))
		{
			ReferenceLayout = LoadReferencedNode()->GetNodeLayout();
			Layout = &ReferenceLayout;
		}
	}
	TArray<FString> TransientPinCategories;
	for(const FRigVMPinCategory& Category : Layout->Categories)
	{
		TransientPinCategories.Add(Category.Path);
	}
	return TransientPinCategories;
}

FRigVMNodeLayout URigVMFunctionReferenceNode::GetNodeLayout(bool bIncludeEmptyCategories) const
{
	if(IsReferencedNodeLoaded())
	{	
		URigVMLibraryNode* ReferencedNode = LoadReferencedNode();
		if (ensure(ReferencedNode))
		{ 
			return ReferencedNode->GetNodeLayout(bIncludeEmptyCategories); 
		}
	}
	return ReferencedFunctionHeader.Layout;
}

FRigVMGraphFunctionIdentifier URigVMFunctionReferenceNode::GetFunctionIdentifier() const
{
	return GetReferencedFunctionHeader().LibraryPointer;
}

bool URigVMFunctionReferenceNode::IsReferencedFunctionHostLoaded() const
{
	return ReferencedFunctionHeader.LibraryPointer.HostObject.ResolveObject() != nullptr;
}

bool URigVMFunctionReferenceNode::IsReferencedNodeLoaded() const
{
	return ReferencedFunctionHeader.LibraryPointer.GetNodeSoftPath().ResolveObject() != nullptr;
}

URigVMLibraryNode* URigVMFunctionReferenceNode::LoadReferencedNode() const
{
	FSoftObjectPath SoftObjectPath = ReferencedFunctionHeader.LibraryPointer.GetNodeSoftPath();
	UObject* LibraryNode = SoftObjectPath.ResolveObject();
	if (!LibraryNode)
	{
		LibraryNode = SoftObjectPath.TryLoad();
	}
	return Cast<URigVMLibraryNode>(LibraryNode);
	
}

TArray<int32> URigVMFunctionReferenceNode::GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
	TArray<int32> Instructions = URigVMNode::GetInstructionsForVMImpl(Context, InVM, InProxy);

	// if the base cannot find any matching instructions, fall back to the library node's implementation
	if(Instructions.IsEmpty())
	{
		Instructions = Super::GetInstructionsForVMImpl(Context, InVM, InProxy);
	}
	
	return Instructions;
}

const URigVMPin* URigVMFunctionReferenceNode::FindReferencedPin(const URigVMPin* InPin) const
{
	return FindReferencedPin(InPin->GetSegmentPath(true));
}

const URigVMPin* URigVMFunctionReferenceNode::FindReferencedPin(const FString& InPinPath) const
{
	if(const URigVMLibraryNode* LibraryNode = LoadReferencedNode())
	{
		return LibraryNode->FindPin(InPinPath);
	}
	return nullptr;
}
