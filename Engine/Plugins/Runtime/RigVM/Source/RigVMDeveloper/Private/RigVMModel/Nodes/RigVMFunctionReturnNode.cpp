// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunctionReturnNode)

FString URigVMFunctionReturnNode::GetNodeTitle() const
{
	return TEXT("Return");
}
