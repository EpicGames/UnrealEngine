// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMIfNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMIfNode)

FName UDEPRECATED_RigVMIfNode::GetNotation() const
{
	static constexpr TCHAR Format[] = TEXT("%s(in %s,in %s,in %s,out %s)");
	static const FLazyName Notation(*FString::Printf(Format, IfName, ConditionName, TrueName, FalseName, ResultName));
	return Notation;
}

const FRigVMTemplate* UDEPRECATED_RigVMIfNode::GetTemplate() const
{
	if(const FRigVMTemplate* SuperTemplate = Super::GetTemplate())
	{
		return SuperTemplate;
	}
	
	if(CachedTemplate == nullptr)
	{
		static const FRigVMTemplate* IfNodeTemplate = nullptr;
		if(IfNodeTemplate)
		{
			return IfNodeTemplate;
		}

		static const FLazyName ConditionFName(ConditionName);
		static const FLazyName TrueFName(TrueName);
		static const FLazyName FalseFName(FalseName);
		static const FLazyName ResultFName(ResultName);

		static const TArray<FRigVMTemplateArgument::ETypeCategory> Categories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};

		TArray<FRigVMTemplateArgumentInfo> ArgumentInfos;
		ArgumentInfos.Reserve(4);
		ArgumentInfos.Emplace(ConditionFName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Bool);
		ArgumentInfos.Emplace(TrueFName, ERigVMPinDirection::Input, Categories);
		ArgumentInfos.Emplace(FalseFName, ERigVMPinDirection::Input, Categories);
		ArgumentInfos.Emplace(ResultFName, ERigVMPinDirection::Output, Categories);

		FRigVMTemplateDelegates Delegates;
		Delegates.NewArgumentTypeDelegate = 
			FRigVMTemplate_NewArgumentTypeDelegate::CreateLambda([](const FName& InArgumentName, int32 InTypeIndex, const FRigVMRegistryHandle& InRegistry)
			{
				FRigVMTemplateTypeMap Types;

				if(InArgumentName == TrueFName || InArgumentName == FalseFName || InArgumentName == ResultFName)
				{
					Types.Add(ConditionFName, RigVMTypeUtils::TypeIndex::Bool);
					Types.Add(TrueFName, InTypeIndex);
					Types.Add(FalseFName, InTypeIndex);
					Types.Add(ResultFName, InTypeIndex);
				}

				return Types;
			});

		IfNodeTemplate = CachedTemplate = FRigVMRegistry::Get().GetOrAddTemplateFromArguments(IfName, ArgumentInfos, Delegates);
	}
	return CachedTemplate;
}


