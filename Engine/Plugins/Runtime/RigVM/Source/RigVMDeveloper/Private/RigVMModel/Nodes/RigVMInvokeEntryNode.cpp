// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"

#include "RigVMModel/RigVMGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMInvokeEntryNode)

URigVMInvokeEntryNode::URigVMInvokeEntryNode()
{
}

FString URigVMInvokeEntryNode::GetNodeTitle() const
{
	return FString::Printf(TEXT("Run %s"), *GetEntryName().ToString());
}

FName URigVMInvokeEntryNode::GetEntryName() const
{
	URigVMPin* EntryPin = GetEntryNamePin();
	if (EntryPin == nullptr)
	{
		return NAME_None;
	}
	return *EntryPin->GetDefaultValue();
}

URigVMPin* URigVMInvokeEntryNode::GetEntryNamePin() const
{
	return FindPin(EntryName);
}

