// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMLibraryNode)

const TArray<URigVMNode*> URigVMLibraryNode::EmptyNodes;
const TArray<URigVMLink*> URigVMLibraryNode::EmptyLinks;


bool URigVMLibraryNode::IsDefinedAsConstant() const
{
	return !IsDefinedAsVarying();
}

bool URigVMLibraryNode::IsDefinedAsVarying() const
{
	if (URigVMGraph* Graph = GetContainedGraph())
	{
		const TArray<URigVMNode*>& Nodes = Graph->GetNodes();
		for(URigVMNode* Node : Nodes)
		{
			if (Node->IsDefinedAsVarying())
			{
				return true;
			}
		}
	}
	return false;
}

uint32 URigVMLibraryNode::GetStructureHash() const
{
	// Avoid hashing the template for library nodes
	return URigVMNode::GetStructureHash();
}

FText URigVMLibraryNode::GetToolTipText() const
{
	return FText::FromName(GetFunctionIdentifier().HostObject.GetLongPackageFName());
}

TArray<int32> URigVMLibraryNode::GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
	TArray<int32> Instructions;

#if WITH_EDITOR

	if(InVM == nullptr)
	{
		return Instructions;
	}

	const FRigVMASTProxy Proxy = InProxy.GetChild((UObject*)this);
	for(URigVMNode* ContainedNode : GetContainedNodes())
	{
		Instructions.Append(ContainedNode->GetInstructionsForVM(Context, InVM, Proxy));
	}

#endif

	
	return Instructions;
}

/*
int32 URigVMLibraryNode::GetInstructionVisitedCount(URigVM* InVM, const FRigVMASTProxy& InProxy, bool bConsolidatePerNode) const
{
	int32 Count = 0;

#if WITH_EDITOR

	if(InVM == nullptr)
	{
		return Count;
	}

	const FRigVMASTProxy Proxy = InProxy.GetChild((UObject*)this);
	for(URigVMNode* ContainedNode : GetContainedNodes())
	{
		const int32 CountPerNode = ContainedNode->GetInstructionVisitedCount(InVM, Proxy, bConsolidatePerNode);
		Count += CountPerNode;
	}

#endif

	return Count;
}
*/

const TArray<URigVMNode*>& URigVMLibraryNode::GetContainedNodes() const
{
	if(URigVMGraph* Graph = GetContainedGraph())
	{
		return Graph->GetNodes();
	}
	return EmptyNodes;
}

const TArray<URigVMLink*>& URigVMLibraryNode::GetContainedLinks() const
{
	if (URigVMGraph* Graph = GetContainedGraph())
	{
		return Graph->GetLinks();
	}
	return EmptyLinks;

}

URigVMFunctionEntryNode* URigVMLibraryNode::GetEntryNode() const
{
	if (URigVMGraph* Graph = GetContainedGraph())
	{
		return Graph->GetEntryNode();
	}
	return nullptr;
}

URigVMFunctionReturnNode* URigVMLibraryNode::GetReturnNode() const
{
	if (URigVMGraph* Graph = GetContainedGraph())
	{
		return Graph->GetReturnNode();
	}
	return nullptr;
}

bool URigVMLibraryNode::Contains(URigVMLibraryNode* InContainedNode, bool bRecursive) const
{
	if(InContainedNode == nullptr)
	{
		return false;
	}
	
	for(URigVMNode* ContainedNode : GetContainedNodes())
	{
		if(ContainedNode == InContainedNode)
		{
			return true;
		}

		if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ContainedNode))
		{
			FRigVMGraphFunctionIdentifier ReferencedNode = FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer;
			if(ReferencedNode.GetNodeSoftPath() == InContainedNode)
			{
				return true;
			}
		}
		if(URigVMLibraryNode* ContainedLibraryNode = Cast<URigVMLibraryNode>(ContainedNode))
		{
			if(ContainedLibraryNode->Contains(InContainedNode))
			{
				return true;
			}
		}
	}

	return false;
}

TArray<FRigVMExternalVariable> URigVMLibraryNode::GetExternalVariables() const
{
	TArray<FRigVMExternalVariable> Variables;

	if(URigVMGraph* ContainedGraph = GetContainedGraph())
	{
		TArray<FRigVMExternalVariable> ContainedVariables = ContainedGraph->GetExternalVariables();
		for(const FRigVMExternalVariable& ContainedVariable : ContainedVariables)
		{
			FRigVMExternalVariable::MergeExternalVariable(Variables, ContainedVariable);
		}
	}

	return Variables;
}

TMap<FRigVMGraphFunctionIdentifier, uint32> URigVMLibraryNode::GetDependencies() const
{
	TMap<FRigVMGraphFunctionIdentifier, uint32> Dependencies;
	
    TArray<URigVMNode*> NodesToProcess = GetContainedNodes();
    for (int32 i=0; i<NodesToProcess.Num(); ++i)
    {
    	const URigVMNode* Node = NodesToProcess[i];
    	if (const URigVMFunctionReferenceNode* RefNode = Cast<URigVMFunctionReferenceNode>(Node))
    	{
    		uint32 Hash = 0;
    		const FRigVMGraphFunctionData* FunctionData = RefNode->GetReferencedFunctionData();
    		if (FunctionData)
    		{
    			Hash = FunctionData->CompilationData.Hash;
    			for (const TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : FunctionData->Header.Dependencies)
    			{
    				Dependencies.Add(Pair);
    			}
    		}
    		Dependencies.Add(RefNode->GetReferencedFunctionHeader().LibraryPointer, Hash);
    	}
    	if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
    	{
    		NodesToProcess.Append(CollapseNode->GetContainedNodes());
    	}
    }
	
	return Dependencies;
}

FRigVMGraphFunctionIdentifier URigVMLibraryNode::GetFunctionIdentifier() const
{
	FRigVMGraphFunctionIdentifier Identifier;
	if (URigVMFunctionLibrary* Library = GetLibrary())
	{
		Identifier.HostObject = Library->GetFunctionHostObjectPath();
	}
	Identifier.SetLibraryNodePath(this->GetPathName());
	return Identifier;
}

FRigVMGraphFunctionHeader URigVMLibraryNode::GetFunctionHeader(IRigVMGraphFunctionHost* InHostObject) const
{
	FRigVMGraphFunctionHeader Header;
	
	if (InHostObject == nullptr)
	{
		Header.LibraryPointer = GetFunctionIdentifier();
	}
	else
	{
	    Header.LibraryPointer.SetLibraryNodePath(this->GetPathName());
	    Header.LibraryPointer.HostObject = Cast<UObject>(InHostObject);
	}

	if (URigVMFunctionLibrary* Library = GetLibrary())
	{
		if (const FRigVMVariant* Variant = Library->GetFunctionVariant(GetFName()))
		{
			Header.Variant = *Variant;
		}
	}

	UObject* HostPtr = Cast<UObject>(Header.LibraryPointer.HostObject.ResolveObject());
	if (IRigVMGraphFunctionHost* Host = Cast<IRigVMGraphFunctionHost>(HostPtr))
	{
		if (FRigVMGraphFunctionData* Data = Host->GetRigVMGraphFunctionStore()->FindFunction(Header.LibraryPointer))
		{
			if (!Header.Variant.Guid.IsValid())
			{
				if (!Data->Header.Variant.Guid.IsValid())
				{
					Data->Header.Variant.Guid = FRigVMVariant::GenerateGUID();
				}
				Header.Variant = Data->Header.Variant;

				if (URigVMFunctionLibrary* Library = GetLibrary())
				{
					if (FRigVMVariant* Variant = Library->GetFunctionVariant(GetFName()))
					{
						Variant->Guid = Header.Variant.Guid;
					}
				}
			}
		}
	}

	if (!Header.Variant.Guid.IsValid())
	{
		Header.Variant.Guid = FRigVMVariant::GenerateGUID();
	}

    Header.Name = GetFName();
	Header.Description = GetNodeDescription();
    Header.Category = GetNodeCategory();
    Header.Keywords = GetNodeKeywords();
	
	// Avoid "initialized from FString" warning while cooking. The graph function tooltip does not have useful information anyway.
	//Header.Tooltip = GetToolTipText();
	
    Header.NodeColor = GetNodeColor();
	Header.NodeTitle = GetNodeTitle();
    for(URigVMPin* Pin : GetPins())
    {
    	FRigVMGraphFunctionArgument Arg;
    	Arg.Name = Pin->GetFName();
    	Arg.DisplayName = Pin->GetDisplayName();
    	Arg.bIsArray = Pin->IsArray();
    	Arg.Direction = Pin->GetDirection();
    	Arg.CPPType = *Pin->GetCPPType();
    	Arg.CPPTypeObject = Pin->GetCPPTypeObject();
    	Arg.DefaultValue = Pin->GetDefaultValue();
    	Arg.bIsConst = Pin->IsDefinedAsConstant();
    	Arg.bIsInputVariable = Pin->IsDefinedAsInputVariable();

    	TArray<URigVMPin*> PinsToProcess;
    	PinsToProcess.Add(Pin);
    	for (int32 i=0; i<PinsToProcess.Num(); ++i)
    	{
    		const FString SubPinPath = PinsToProcess[i]->GetSubPinPath(Pin, false);

    		// Avoid "initialized from FString" warning while cooking. The graph function tooltip does not have useful information anyway.
    		//Arg.PathToTooltip.Add(SubPinPath, GetToolTipTextForPin(PinsToProcess[i]));
    		Arg.PathToTooltip.Add(SubPinPath, FText());
    		
    		PinsToProcess.Append(PinsToProcess[i]->GetSubPins());
    	}

    	Header.Arguments.Add(Arg);
    }

	// If the header already exists, try to find it and get its external variables, dependencies and layout
	TArray<FString> Dependencies;
	TArray<FRigVMExternalVariable> ExternalVariables;
	if (IRigVMGraphFunctionHost* Host = Cast<IRigVMGraphFunctionHost>(HostPtr))
	{
		if (FRigVMGraphFunctionData* Data = Host->GetRigVMGraphFunctionStore()->FindFunction(Header.LibraryPointer))
		{
			Header.Dependencies = Data->Header.Dependencies;
			Header.ExternalVariables = Data->Header.ExternalVariables;
		}
		else
		{
			Header.ExternalVariables.Reset();
		}
	}

	Header.Layout = GetNodeLayout();
	
	return Header;
}

FRigVMVariant URigVMLibraryNode::GetFunctionVariant() const
{
	return GetFunctionHeader().Variant;
}

FRigVMVariantRef URigVMLibraryNode::GetFunctionVariantRef() const
{
	return FRigVMVariantRef(FSoftObjectPath(this), GetFunctionVariant());
}

TArray<FRigVMVariantRef> URigVMLibraryNode::GetMatchingVariants() const
{
	const FRigVMGraphFunctionHeader Header = GetFunctionHeader();
	if(Header.IsValid())
	{
		return Header.LibraryPointer.GetVariants(false);
	}
	return TArray<FRigVMVariantRef>();
}

bool URigVMLibraryNode::ContainsLocalVariable(const FName& InVariableName) const
{
	const URigVMGraph* ContainedGraph = GetContainedGraph();
	if (!ContainedGraph)
	{
		return false;
	}

	return ContainedGraph->LocalVariables.ContainsByPredicate(
		[InVariableName](const FRigVMGraphVariableDescription& LocalVariable) -> bool
		{
			return InVariableName == LocalVariable.Name;
		}
	);
}


