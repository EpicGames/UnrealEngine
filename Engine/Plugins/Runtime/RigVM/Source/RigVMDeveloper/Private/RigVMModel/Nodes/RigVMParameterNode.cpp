// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMParameterNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMParameterNode)

URigVMParameterNode::URigVMParameterNode()
{
}

FString URigVMParameterNode::GetNodeTitle() const
{
	return GetParameterName().ToString();
}

FName URigVMParameterNode::GetParameterName() const
{
	URigVMPin* ParameterPin = FindPin(ParameterName);
	if (ParameterPin == nullptr)
	{
		return NAME_None;
	}
	return *ParameterPin->GetDefaultValue();
}

bool URigVMParameterNode::IsInput() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return false;
	}
	return ValuePin->GetDirection() == ERigVMPinDirection::Output;
}

FString URigVMParameterNode::GetCPPType() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return FString();
	}
	return ValuePin->GetCPPType();
}

UObject* URigVMParameterNode::GetCPPTypeObject() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return nullptr;
	}
	return ValuePin->GetCPPTypeObject();
}

FString URigVMParameterNode::GetDefaultValue() const
{

	URigVMPin* ValuePin = FindPin(DefaultName);
	if (ValuePin == nullptr)
	{
		return FString();
	}
	return ValuePin->GetDefaultValue();
}

FRigVMGraphParameterDescription URigVMParameterNode::GetParameterDescription() const
{
	FRigVMGraphParameterDescription Parameter;
	Parameter.Name = GetParameterName();
	Parameter.bIsInput = IsInput();
	Parameter.CPPType = GetCPPType();
	Parameter.CPPTypeObject = GetCPPTypeObject();
	Parameter.DefaultValue = GetDefaultValue();
	return Parameter;
}

