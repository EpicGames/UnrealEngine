// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "RigVMModel/RigVMGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMRerouteNode)

URigVMRerouteNode::URigVMRerouteNode()
{
}

FString URigVMRerouteNode::GetNodeTitle() const
{
	if(const URigVMPin* ValuePin = FindPin(ValueName))
	{
		FString TypeDisplayName;
		if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ValuePin->GetCPPTypeObject()))
		{
			TypeDisplayName = ScriptStruct->GetDisplayNameText().ToString();
		}
		else if(const UEnum* Enum = Cast<UEnum>(ValuePin->GetCPPTypeObject()))
		{
			TypeDisplayName = Enum->GetName();
		}
		else if(const UClass* Class = Cast<UClass>(ValuePin->GetCPPTypeObject()))
		{
			TypeDisplayName = Class->GetDisplayNameText().ToString();
		}
		else if(ValuePin->IsArray())
		{
			TypeDisplayName = ValuePin->GetArrayElementCppType();
		}
		else
		{
			TypeDisplayName = ValuePin->GetCPPType();
		}

		if(TypeDisplayName.IsEmpty())
		{
			return RerouteName;
		}

		TypeDisplayName = TypeDisplayName.Left(1).ToUpper() + TypeDisplayName.Mid(1);

		if(ValuePin->IsArray())
		{
			TypeDisplayName += TEXT(" Array");
		}

		return TypeDisplayName;
	}
	return RerouteName;
}

FLinearColor URigVMRerouteNode::GetNodeColor() const
{
	return FLinearColor::White;
}

FText URigVMRerouteNode::GetToolTipText() const
{
	for(const URigVMLink* Link : GetPins()[0]->GetLinks())
	{
		if(const URigVMPin* SourcePin = Link->GetSourcePin())
		{
			if(SourcePin->GetNode() != this)
			{
				if(const URigVMRerouteNode* SourceRerouteNode = Cast<URigVMRerouteNode>(SourcePin->GetNode()))
				{
					return SourceRerouteNode->GetToolTipText();
				}
				const FText Format = NSLOCTEXT("URigVMRerouteNode", "ToolTipFormat", "Points to: {0}\n\n{1}");
				return FText::Format(Format, FText::FromString(SourcePin->GetPinPath()), SourcePin->GetNode()->GetToolTipText());
			}
		}
	}
	return Super::GetToolTipText();
}

bool URigVMRerouteNode::IsLiteral() const
{
	return GetPins()[0]->GetSourceLinks(true).IsEmpty();
}

