// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMSelectNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMSelectNode)

bool UDEPRECATED_RigVMSelectNode::AllowsLinksOn(const URigVMPin* InPin) const
{
	if(InPin->GetRootPin() == InPin)
	{
		if(InPin->GetName() == ValueName)
		{
			return false;
		}
	}

	return true;
}

FName UDEPRECATED_RigVMSelectNode::GetNotation() const
{
	static constexpr TCHAR Format[] = TEXT("%s(in %s,in %s,out %s)");
	static const FLazyName Notation(*FString::Printf(Format, SelectName, IndexName, ValueName, ResultName));
	return Notation;
}

const FRigVMTemplate* UDEPRECATED_RigVMSelectNode::GetTemplate() const
{
	if(const FRigVMTemplate* SuperTemplate = Super::GetTemplate())
	{
		return SuperTemplate;
	}
	
	if(CachedTemplate == nullptr)
	{
		static const FRigVMTemplate* SelectNodeTemplate = nullptr;
		if(SelectNodeTemplate)
		{
			return SelectNodeTemplate;
		}

		static const FLazyName IndexFName(IndexName);
		static const FLazyName ValueFName(ValueName);
		static const FLazyName ResultFName(ResultName);

		static const TArray<FRigVMTemplateArgument::ETypeCategory> ValueTypeCategories = {
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayArrayAnyValue
		};
		static const TArray<FRigVMTemplateArgument::ETypeCategory> ResultTypeCategories = {
			FRigVMTemplateArgument::ETypeCategory_SingleAnyValue,
			FRigVMTemplateArgument::ETypeCategory_ArrayAnyValue
		};
		
		TArray<FRigVMTemplateArgumentInfo> ArgumentInfos;
		ArgumentInfos.Reserve(3);
		ArgumentInfos.Emplace(IndexFName, ERigVMPinDirection::Input, RigVMTypeUtils::TypeIndex::Int32);
		ArgumentInfos.Emplace(ValueFName, ERigVMPinDirection::Input, ValueTypeCategories);
		ArgumentInfos.Emplace(ResultFName, ERigVMPinDirection::Output, ResultTypeCategories);
		
		FRigVMTemplateDelegates Delegates;
		Delegates.NewArgumentTypeDelegate = 
			FRigVMTemplate_NewArgumentTypeDelegate::CreateLambda([](const FName& InArgumentName, int32 InTypeIndex, const FRigVMRegistryHandle& InRegistry)
			{
				FRigVMTemplateTypeMap Types;

				int32 ValueTypeIndex = INDEX_NONE;
				int32 ResultTypeIndex = INDEX_NONE;

				if(InArgumentName == ValueFName)
				{
					ValueTypeIndex = InTypeIndex;
					ResultTypeIndex = FRigVMRegistry::Get().GetBaseTypeFromArrayTypeIndex(InTypeIndex);
				}
				else if(InArgumentName == ResultFName)
				{
					ValueTypeIndex = FRigVMRegistry::Get().GetArrayTypeFromBaseTypeIndex(InTypeIndex);;
					ResultTypeIndex = InTypeIndex;
				}
				
				if(ValueTypeIndex != INDEX_NONE && ResultTypeIndex != INDEX_NONE)
				{
					Types.Add(IndexFName, RigVMTypeUtils::TypeIndex::Int32);
					Types.Add(ValueFName, ValueTypeIndex);
					Types.Add(ResultFName, ResultTypeIndex);
				}

				return Types;
			});

		SelectNodeTemplate = CachedTemplate = FRigVMRegistry::Get().GetOrAddTemplateFromArguments(SelectName, ArgumentInfos, Delegates);
	}
	return CachedTemplate;
}
