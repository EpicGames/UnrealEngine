// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMStringUtils.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMModel/RigVMController.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTemplateNode)

////////////////////////////////////////////////////////////////////////////////////////////////

void FRigVMTemplatePreferredType::UpdateStringFromIndex()
{
	static constexpr TCHAR Format[] = TEXT("%s,%s");
	const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(TypeIndex);
	TypeString = FString::Printf(Format, *Type.CPPType.ToString(), *Type.GetCPPTypeObjectPath().ToString());
}

void FRigVMTemplatePreferredType::UpdateIndexFromString()
{
	// during duplicate we may not have type strings here
	if(TypeString.IsEmpty())
	{
		return;
	}

	FString OriginalCPPType, CPPTypeObjectPath;
	verify(TypeString.Split(TEXT(","), &OriginalCPPType, &CPPTypeObjectPath));
	FString CPPType = OriginalCPPType;

	UObject* CPPTypeObject = nullptr;

	static const FString NoneString = FName(NAME_None).ToString(); 
	if(CPPTypeObjectPath != NoneString)
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(CPPTypeObjectPath);
	}

	// If we still haven't found the object, try to find it with the CPPType
	if (!CPPTypeObject && RigVMTypeUtils::RequiresCPPTypeObject(CPPType))
	{
		CPPTypeObject = RigVMTypeUtils::ObjectFromCPPType(CPPType, true);
	}

	const FRigVMTemplateArgumentType Type(*CPPType, CPPTypeObject);
	TypeIndex = FRigVMRegistry::Get().FindOrAddType(Type);

	// We might have used redirectors. Update the TypeString with the new information.
	if (TypeIndex != INDEX_NONE && OriginalCPPType != Type.CPPType.ToString())
	{
		UpdateStringFromIndex();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

URigVMTemplateNode::URigVMTemplateNode()
	: Super()
	, TemplateNotation(NAME_None)
	, ResolvedFunctionName()
	, CachedTemplate(nullptr)
	, CachedFunction(nullptr)
	, ResolvedPermutation(INDEX_NONE)
{
}

void URigVMTemplateNode::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
}

void URigVMTemplateNode::PostLoad()
{
	Super::PostLoad();

	for (URigVMPin* Pin : GetPins())
	{
		// GetTypeIndex ensures that the pin type is registered
		// Only need to register for visible pins
		if (Pin->GetDirection() == ERigVMPinDirection::IO ||
			Pin->GetDirection() == ERigVMPinDirection::Input ||
			Pin->GetDirection() == ERigVMPinDirection::Output)
		{
			Pin->GetTypeIndex();
		}
	}
	
	// if there are brackets in the notation remove them
	const FString OriginalNotation = TemplateNotation.ToString();
	const FString SanitizedNotation = OriginalNotation.Replace(TEXT("[]"), TEXT(""));
	if(OriginalNotation != SanitizedNotation)
	{
		TemplateNotation = *SanitizedNotation;
	}

	// upgrade from a previous version where we stored the preferred types as strings
	if(!PreferredPermutationTypes_DEPRECATED.IsEmpty())
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();

		PreferredPermutationPairs_DEPRECATED.Reserve(PreferredPermutationTypes_DEPRECATED.Num());

		for(const FString& PreferredPermutation : PreferredPermutationTypes_DEPRECATED)
		{
			FString ArgName, CPPType;
			PreferredPermutation.Split(TEXT(":"), &ArgName, &CPPType);

			CPPType = RigVMTypeUtils::PostProcessCPPType(CPPType);
			const FRigVMTemplateArgumentType Type = Registry.FindTypeFromCPPType(CPPType);
			PreferredPermutationPairs_DEPRECATED.Emplace(*ArgName, Registry.GetTypeIndex(Type));
		}

		PreferredPermutationTypes_DEPRECATED.Reset();
	}

	InvalidateCache();

	// the template notation may have changed
	if(const FRigVMTemplate* Template = GetTemplate())
	{
		TemplateNotation = Template->GetNotation();
	}
}

uint32 URigVMTemplateNode::GetStructureHash() const
{
	uint32 Hash = Super::GetStructureHash();

	if(const FRigVMTemplate* Template = GetTemplate())
	{
		Hash = HashCombine(Hash, GetTypeHash(*Template));
	}

	return Hash;
}

UScriptStruct* URigVMTemplateNode::GetScriptStruct() const
{
	if(const FRigVMFunction* Function = GetResolvedFunction())
	{
		return Function->Struct;
	}
	return nullptr;
}

FString URigVMTemplateNode::GetNodeTitle() const
{
	if(!IsResolved())
	{
		if(const FRigVMTemplate* Template = GetTemplate())
		{
			return Template->GetNodeName().ToString();
		}
	}
	
	FString ResolvedNodeTitle = Super::GetNodeTitle();

	const int32 BracePos = ResolvedNodeTitle.Find(TEXT(" ("));
	if(BracePos != INDEX_NONE)
	{
		ResolvedNodeTitle = ResolvedNodeTitle.Left(BracePos);
	}

	return ResolvedNodeTitle;
}

FName URigVMTemplateNode::GetMethodName() const
{
	if(const FRigVMFunction* Function = GetResolvedFunction())
	{
		return Function->GetMethodName();
	}
	return NAME_None;
}

FText URigVMTemplateNode::GetToolTipText() const
{
	if(const FRigVMTemplate* Template = GetTemplate())
	{
		const TArray<int32> PermutationIndices = GetResolvedPermutationIndices(false);
		return Template->GetTooltipText(PermutationIndices);
	}
	return Super::GetToolTipText();
}

FText URigVMTemplateNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	const FText SuperToolTip = Super::GetToolTipTextForPin(InPin);
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		const URigVMPin* RootPin = InPin->GetRootPin();
		if (RootPin->IsWildCard())
		{
			if (const FRigVMTemplateArgument* Arg = Template->FindArgument(RootPin->GetFName()))
			{
				FString Tooltip;
				
				if(Arg->GetNumTypes(FRigVMRegistry::Get().GetHandle_NoLock()) > UE_RIGVM_MANYTYPES_LIMIT)
				{
					Tooltip = TEXT("Supports any type.");
				}
				
				if(Tooltip.IsEmpty())
				{
					FString SupportedTypesJoined;
					TArray<TRigVMTypeIndex> TypesPrinted;
					for (int32 Index=0; Index<Template->NumPermutations(); ++Index)
					{
						const TRigVMTypeIndex TypeIndex = Arg->GetTypeIndex(Index);
						if (TypesPrinted.Contains(TypeIndex))
						{
							continue;
						}
						TypesPrinted.Add(TypeIndex);
						FString Type = FRigVMRegistry::Get().GetType(TypeIndex).CPPType.ToString();
						SupportedTypesJoined += Type + TEXT("\n");
					}
					Tooltip = TEXT("Supported Types:\n\n") + SupportedTypesJoined;
				}

				if(!SuperToolTip.IsEmpty())
				{
					Tooltip += TEXT("\n\n") + SuperToolTip.ToString();
				}
				
				return FText::FromString(Tooltip);
			}
		}
	}
	return SuperToolTip;
}

TArray<URigVMPin*> URigVMTemplateNode::GetAggregateInputs() const
{
	return GetAggregatePins(ERigVMPinDirection::Input);
}

TArray<URigVMPin*> URigVMTemplateNode::GetAggregateOutputs() const
{
	return GetAggregatePins(ERigVMPinDirection::Output);
}

TArray<URigVMPin*> URigVMTemplateNode::GetAggregatePins(const ERigVMPinDirection& InDirection) const
{
	TArray<URigVMPin*> AggregatePins;
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		TArray<int32> Permutations;
		Permutations.Reserve(Template->NumPermutations());
		for (int32 i=0; i<Template->NumPermutations(); ++i)
		{
			Permutations.Add(i);
		}
		
		if (Permutations.IsEmpty())
		{
			return AggregatePins;
		}

		auto FindAggregatePins = [&](const int32& PermutationIndex)
		{
			TArray<URigVMPin*> Inputs;
			if (const FRigVMFunction* Permutation = Template->GetPermutation(PermutationIndex))
			{
				const TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Permutation->Struct));
				if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(StructOnScope->GetStruct()))
				{
					for (URigVMPin* Pin : GetPins())
					{
						if (Pin->GetDirection() == InDirection)
						{
							if (const FProperty* Property = ScriptStruct->FindPropertyByName(Pin->GetFName()))
							{
								if (Property->HasMetaData(FRigVMStruct::AggregateMetaName))
								{
									Inputs.Add(Pin);
								}
							}			
						}
					}
				}
			}
			return Inputs;
		};
		
		AggregatePins = FindAggregatePins(Permutations[0]);
		for (int32 i=1; i<Permutations.Num(); ++i)
		{
			if(Permutations[i] >= Template->NumPermutations())
			{
				continue;
			}
			
			TArray<URigVMPin*> OtherAggregateInputs = FindAggregatePins(Permutations[i]);
			if (OtherAggregateInputs.Num() != AggregatePins.Num() ||
				OtherAggregateInputs.FilterByPredicate([&](const URigVMPin* OtherPin)
				{
					return !AggregatePins.Contains(OtherPin);
				}).Num() > 0)
			{
				return {};
			}
		}
	}
#endif
	return AggregatePins;
}

FName URigVMTemplateNode::GetNextAggregateName(const FName& InLastAggregatePinName) const
{
	FName NextName;	

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		if (const FRigVMFunction* Permutation = Template->GetPermutation(0))
		{
			const TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Permutation->Struct));
			if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(StructOnScope->GetStruct()))
			{
				check(ScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()));

				const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope->GetStructMemory();
				return StructMemory->GetNextAggregateName(InLastAggregatePinName);
			}
		}
	}
#endif
	return FName();
}

FName URigVMTemplateNode::GetNotation() const
{
	return TemplateNotation;
}

bool URigVMTemplateNode::IsSingleton() const
{
	return GetTemplate() == nullptr;
}

bool URigVMTemplateNode::SupportsType(const URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex)
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	static const FString WildCardCPPType = RigVMTypeUtils::GetWildCardCPPType();
	static const FString WildCardArrayCPPType = RigVMTypeUtils::ArrayTypeFromBaseType(WildCardCPPType);

	const URigVMPin* RootPin = InPin->GetRootPin();

	TRigVMTypeIndex TypeIndex = InTypeIndex;

	if(InPin->GetParentPin() == RootPin && RootPin->IsArray())
	{
		TypeIndex = Registry.GetArrayTypeFromBaseTypeIndex(TypeIndex);
	}

	// we always support the unknown type
	if(Registry.IsWildCardType(TypeIndex))
	{
		if(const FRigVMTemplate* Template = GetTemplate())
		{
			if(const FRigVMTemplateArgument* Argument = Template->FindArgument(RootPin->GetFName()))
			{
				// support this only on non-singleton arguments
				if(Argument->IsSingleton())
				{
					return false;
				}

				if(Registry.IsArrayType(TypeIndex))
				{
					if(Argument->GetArrayType() == FRigVMTemplateArgument::EArrayType_SingleValue)
					{
						return false;
					}
				}
				else
				{
					if(Argument->GetArrayType() == FRigVMTemplateArgument::EArrayType_ArrayValue)
					{
						return false;
					}
				}
				
				if(OutTypeIndex)
				{
					*OutTypeIndex = InTypeIndex;
				}
				return true;
			}
			else if (IsA<URigVMFunctionInterfaceNode>())
			{
				if(OutTypeIndex)
				{
					*OutTypeIndex = InTypeIndex;
				}
				return true;
			}
		}
		return false;
	}
	
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		const TPair<FName,TRigVMTypeIndex> CacheKey(RootPin->GetFName(), TypeIndex);
		TRigVMTypeIndex SupportedTypeIndex = INDEX_NONE;
		if (Template->ArgumentSupportsTypeIndex(RootPin->GetFName(), TypeIndex, &SupportedTypeIndex))
		{
			if (OutTypeIndex)
			{
				(*OutTypeIndex) = SupportedTypeIndex;
			}
			return true;
		}

		// an entry/return node that does not contain an argument for the pin will always support the connections
		if (IsA<URigVMFunctionInterfaceNode>())
		{
			if (Template->FindArgument(RootPin->GetFName()) == nullptr)
			{
				return true;
			}
		}
		return false;
	}
	
	if(RootPin->GetTypeIndex() == TypeIndex)
	{
		if(OutTypeIndex)
		{
			*OutTypeIndex = TypeIndex;
		}
		return true;
	}
	return false;
}

TArray<const FRigVMFunction*> URigVMTemplateNode::GetResolvedPermutations() const
{
	TArray<const FRigVMFunction*> Functions;
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		TArray<int32> ResolvedPermutations = GetResolvedPermutationIndices(true);
		for(const int32 Index : ResolvedPermutations)
		{
			Functions.Add(Template->GetPermutation(Index));
		}
	}
	return Functions;
}

TArray<int32> URigVMTemplateNode::GetResolvedPermutationIndices(bool bAllowFloatingPointCasts) const
{
	TArray<int32> ResolvedPermutations;
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		FRigVMTemplateTypeMap PinTypeMap = GetTemplatePinTypeMap();
		Template->Resolve(PinTypeMap, ResolvedPermutations, bAllowFloatingPointCasts);
	}
	return ResolvedPermutations;
}

FRigVMTemplateTypeMap URigVMTemplateNode::GetTemplatePinTypeMap(bool bIncludeHiddenPins, bool bIncludeExecutePins) const
{
	FRigVMTemplateTypeMap PinTypeMap;
	for (const URigVMPin* Pin : GetPins())
	{
		if (!bIncludeHiddenPins && Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			continue;
		}
		if (!bIncludeExecutePins && Pin->IsExecuteContext())
		{
			continue;
		}
		if (!Pin->IsWildCard())
		{
			PinTypeMap.Add(Pin->GetFName(), Pin->GetTypeIndex());
		}
	}
	return PinTypeMap;
}

const FRigVMTemplate* URigVMTemplateNode::GetTemplate() const
{
	if(CachedTemplate == nullptr)
	{
		CachedTemplate = FRigVMRegistry::Get().FindTemplate(GetNotation());
	}
	return CachedTemplate;
}

const FRigVMFunction* URigVMTemplateNode::GetResolvedFunction() const
{
	if(CachedFunction == nullptr)
	{
		if(!ResolvedFunctionName.IsEmpty())
		{
			CachedFunction = FRigVMRegistry::Get().FindFunction(*ResolvedFunctionName);
		}

		if(CachedFunction == nullptr)
		{
			if(ResolvedPermutation != INDEX_NONE)
			{
				CachedFunction = ((FRigVMTemplate*)GetTemplate())->GetOrCreatePermutation(ResolvedPermutation);
			}
		}
	}
	return CachedFunction;
}

bool URigVMTemplateNode::IsResolved() const
{
	return GetScriptStruct() != nullptr;
}

bool URigVMTemplateNode::IsFullyUnresolved() const
{
	check(GetTemplate());

	if (const FRigVMTemplate* Template = GetTemplate())
	{
		for (int32 i=0; i<Template->NumArguments(); ++i)
		{
			const FRigVMTemplateArgument* Argument = Template->GetArgument(i);
			if (!Argument->IsSingleton())
			{
				if (const URigVMPin* Pin = FindPin(Argument->GetName().ToString()))
				{
					if (!Pin->IsWildCard())
					{
						return false;
					}
				}
			}
		}
	}

	// all permutations are available means we haven't resolved any wildcard pin
	return true;
}

FString URigVMTemplateNode::GetInitialDefaultValueForPin(const FName& InRootPinName, const TArray<int32>& InPermutationIndices) const
{
	const FRigVMTemplate* Template = GetTemplate();
	if(Template == nullptr)
	{
		return FString();
	}

	URigVMPin* Pin = FindPin(InRootPinName.ToString());
	if (!Pin)
	{
		return FString();
	}

	TArray<int32> PermutationIndices = InPermutationIndices;
	if(PermutationIndices.IsEmpty())
	{
		PermutationIndices = GetResolvedPermutationIndices(false);
	}

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	FRigVMRegistryReadLock RegistryReadLock(Registry);
	
	FString DefaultValue;
	const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory(); 
	const FRigVMTemplateArgument* Argument = Template->FindArgument(InRootPinName);

	if (Argument)
	{
		for(const int32 PermutationIndex : PermutationIndices)
		{
			FString NewDefaultValue;

			const TRigVMTypeIndex TypeIndex = Argument->GetTypeIndex_NoLock(PermutationIndex, RegistryReadLock);

			// INDEX_NONE indicates deleted permutation
			if (TypeIndex == INDEX_NONE)
			{
				continue;
			}
			
			if(Factory)
			{
				NewDefaultValue = Factory->GetArgumentDefaultValue(Argument->GetName(), TypeIndex);
			}
			else if(const FRigVMFunction* Permutation = Template->GetPermutation_NoLock(PermutationIndex, RegistryReadLock))
			{
				const TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Permutation->Struct));
				const FRigVMStruct* DefaultStruct = (const FRigVMStruct*)StructOnScope->GetStructMemory();

				const bool bUseQuotes = TypeIndex != RigVMTypeUtils::TypeIndex::FString && TypeIndex != RigVMTypeUtils::TypeIndex::FName;
				NewDefaultValue = DefaultStruct->ExportToFullyQualifiedText(
					Cast<UScriptStruct>(StructOnScope->GetStruct()), InRootPinName, nullptr, bUseQuotes);
			}
			else
			{
				if (RegistryReadLock->IsArrayType_NoLock(TypeIndex))
				{
					NewDefaultValue = TEXT("()");
				}
				else
				{
					const FRigVMTemplateArgumentType& Type = RegistryReadLock->GetType_NoLock(TypeIndex);
					if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
					{
						TArray<uint8, TAlignedHeapAllocator<16>> TempBuffer;
						TempBuffer.AddUninitialized(ScriptStruct->GetStructureSize());

						// call the struct constructor to initialize the struct
						ScriptStruct->InitializeDefaultValue(TempBuffer.GetData());

						ScriptStruct->ExportText(NewDefaultValue, TempBuffer.GetData(), nullptr, nullptr, PPF_None, nullptr);
						ScriptStruct->DestroyStruct(TempBuffer.GetData());				
					}
					else if (const UEnum* Enum = Cast<UEnum>(Type.CPPTypeObject))
					{
						NewDefaultValue = Enum->GetNameStringByValue(0);
					}
					else if(const UClass* Class = Cast<UClass>(Type.CPPTypeObject))
					{
						// not supporting objects yet
						ensure(false);
					}
					else if (Type.CPPType == RigVMTypeUtils::FloatTypeName)
					{
						NewDefaultValue = TEXT("0.000000");				
					}
					else if (Type.CPPType == RigVMTypeUtils::DoubleTypeName)
					{
						NewDefaultValue = TEXT("0.000000");
					}
					else if (Type.CPPType == RigVMTypeUtils::Int32TypeName)
					{
						NewDefaultValue = TEXT("0");
					}
					else if (Type.CPPType == RigVMTypeUtils::BoolTypeName)
					{
						NewDefaultValue = TEXT("False");
					}
					else if (Type.CPPType == RigVMTypeUtils::FStringTypeName)
					{
						NewDefaultValue = TEXT("");
					}
					else if (Type.CPPType == RigVMTypeUtils::FNameTypeName)
					{
						NewDefaultValue = TEXT("");
					}
				}			
			}

			if(!NewDefaultValue.IsEmpty())
			{
				if(DefaultValue.IsEmpty())
				{
					DefaultValue = NewDefaultValue;
				}
				else if(!NewDefaultValue.Equals(DefaultValue))
				{
					return FString();
				}
			}
		}
	}
	
	return DefaultValue;
}

FName URigVMTemplateNode::GetDisplayNameForPin(const URigVMPin* InPin, const TArray<int32>& InPermutationIndices) const
{
#if WITH_EDITOR
	if(const FRigVMTemplate* Template = GetTemplate())
	{
		check(InPin);
		
		if(InPin->IsRootPin())
		{
			if(InPin->IsExecuteContext())
			{
				if(InPin->GetDirection() == ERigVMPinDirection::IO)
				{
					return FRigVMStruct::ExecuteName;
				}

				const FRigVMDispatchContext DispatchContext;
				if(Template->NumExecuteArguments(DispatchContext) == 1)
				{
					return FRigVMStruct::ExecuteName;
				}
			}
		
			const FName RootPinName = InPin->GetFName();
			const FText DisplayNameText = Template->GetDisplayNameForArgument(RootPinName, InPermutationIndices.Num() > 0 ? InPermutationIndices : GetResolvedPermutationIndices(false));
			if(DisplayNameText.IsEmpty())
			{
				return NAME_None;
			}

			FString DefaultDisplayName = RootPinName.ToString();
			FString Left, Right;
			if(RigVMStringUtils::SplitPinPathAtEnd(DefaultDisplayName, Left, Right))
			{
				DefaultDisplayName = Right;
			}

			const FName DisplayName = *DisplayNameText.ToString();
			if(DisplayName.IsEqual(*DefaultDisplayName))
			{
				return NAME_None;
			}
			return DisplayName;
		}
	}
#endif
	return Super::GetDisplayNameForPin(InPin);
}

FName URigVMTemplateNode::GetDisplayNameForPin(const URigVMPin* InPin) const
{
	return GetDisplayNameForPin(InPin, GetResolvedPermutationIndices(true));
}

TRigVMTypeIndex URigVMTemplateNode::TryReduceTypesToSingle(const TArray<TRigVMTypeIndex>& InTypes, const TRigVMTypeIndex PreferredType) const
{
	return TryReduceTypesToSingle_NoLock(InTypes, FRigVMRegistryReadLock(), PreferredType);
}

TRigVMTypeIndex URigVMTemplateNode::TryReduceTypesToSingle_NoLock(const TArray<TRigVMTypeIndex>& InTypes, const FRigVMRegistryHandle& InRegistry, const TRigVMTypeIndex PreferredType) const
{
	if (InTypes.IsEmpty())
	{
		return INDEX_NONE;
	}

	if (InTypes.Num() == 1)
	{
		return InTypes[0];
	}
	
	for (int32 i=1; i<InTypes.Num(); ++i)
	{
		if (!InRegistry->CanMatchTypes_NoLock(InTypes[0], InTypes[i], true))
		{
			return INDEX_NONE;
		}
	}

	if (InTypes.Contains(PreferredType))
	{
		return PreferredType;
	}

	return InTypes[0];
}

TArray<int32> URigVMTemplateNode::FindPermutationsForTypes(const TArray<FRigVMTemplatePreferredType>& ArgumentTypes, bool bAllowCasting) const
{
	TArray<int32> Permutations;
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		
		TArray<const FRigVMTemplateArgument*> Args;
		TArray<TRigVMTypeIndex> TypeIndices;
		for (const FRigVMTemplatePreferredType& ArgumentType : ArgumentTypes)
		{
			if (const FRigVMTemplateArgument* Argument = Template->FindArgument(ArgumentType.GetArgument()))
			{
				Args.Add(Argument);
				TypeIndices.Add(ArgumentType.GetTypeIndex());
			}
			else
			{
				return {};
			}
		}
		
		for (int32 i=0; i<Template->NumPermutations(); ++i)
		{
			bool bAllArgsMatched = true;
			for (int32 ArgIndex = 0; ArgIndex < Args.Num(); ++ArgIndex)
			{
				const FRigVMTemplateArgument* Argument = Args[ArgIndex];
				if (Argument)
				{
					if ((bAllowCasting && !Registry.CanMatchTypes(Argument->GetTypeIndex(i), TypeIndices[ArgIndex], true)) ||
						(!bAllowCasting && Argument->GetTypeIndex(i) != TypeIndices[ArgIndex]))
					{
						bAllArgsMatched = false;
						break;
					}
				}
				else
				{
					// The preferred type doesn't own an argument yet. Supports all types.
				}
			}

			if (bAllArgsMatched)
			{
				Permutations.Add(i);
			}
		}
	}
	return Permutations;
}

FRigVMTemplateTypeMap URigVMTemplateNode::GetTypesForPermutation(const int32 InPermutationIndex) const
{
	if (const FRigVMTemplate* Template = GetTemplate())
	{
		return Template->GetTypesForPermutation(InPermutationIndex);
	}

	return FRigVMTemplateTypeMap();
}

void URigVMTemplateNode::InvalidateCache()
{
	Super::InvalidateCache();
	
	CachedFunction = nullptr;
	CachedTemplate = nullptr;
	ResolvedPermutation = INDEX_NONE;

	if (HasWildCardPin())
	{
		ResolvedFunctionName.Reset();
	}
}
