// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMUnitNode.h"

#include "RigVMCore/RigVMStruct.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "RigVMUserWorkflowRegistry.h"
#include "RigVMHost.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMDeveloperModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMUnitNode)

void URigVMUnitNode::PostLoad()
{
	Super::PostLoad();

	// if we have a script struct but no notation let's figure out the template
	if(GetScriptStruct() != nullptr)
	{
		if (IsOutDated())
		{
			TemplateNotation = NAME_None;
			if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(GetScriptStruct(), *GetMethodName().ToString()))
			{
				ResolvedFunctionName = Function->GetName();
			}
		}
		else if(GetTemplate() == nullptr)
		{
			if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(GetScriptStruct(), *GetMethodName().ToString()))
			{
				if(Function->TemplateIndex != INDEX_NONE)
				{
					const FRigVMTemplate& Template = FRigVMRegistry::Get().GetTemplates()[Function->TemplateIndex];
					TemplateNotation = Template.GetNotation();				
				}
				ResolvedFunctionName = Function->GetName();
			}			
		}
	}
}

FString URigVMUnitNode::GetNodeTitle() const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		return Struct->GetDisplayNameText().ToString();
	}
	return Super::GetNodeTitle();
}

FString URigVMUnitNode::GetNodeSubTitle() const
{
	TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(false);
	if (StructOnScope.IsValid())
	{
		const FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->GetUnitSubTitle();
	}
	return Super::GetNodeSubTitle();
}

FText URigVMUnitNode::GetToolTipText() const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		return Struct->GetToolTipText();
	}
	return URigVMNode::GetToolTipText();
}

bool URigVMUnitNode::IsDefinedAsConstant() const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		return Struct->HasMetaData(FRigVMStruct::ConstantMetaName);
	}
	return false;
}

bool URigVMUnitNode::IsDefinedAsVarying() const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		return Struct->HasMetaData(FRigVMStruct::VaryingMetaName);
	}
	return false;
}

bool URigVMUnitNode::IsPure() const
{
	// A node with an input variable pin holds a mutable reference to caller-owned data.
	// Treating it as pure would allow mutation in a pure context, so it must be non-pure.
	for (const URigVMPin* Pin : GetPins())
	{
		if (Pin->IsDefinedAsInputVariable())
		{
			return false;
		}
	}

	if (UScriptStruct* Struct = GetScriptStruct())
	{
		if (FRigVMStruct::HasMetaData(Struct, FRigVMStruct::PureMetaName))
		{
			return true;
		}
	}
	return Super::IsPure();
}

FName URigVMUnitNode::GetEventName() const
{
	if(!CachedEventName.IsBound())
	{
		CachedEventName.Bind(this);
	}
	if(CachedEventName.IsValid())
	{
		return CachedEventName.GetValue();
	}
	TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(false);
	if (StructOnScope.IsValid())
	{
		const FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		CachedEventName = StructMemory->GetEventName();
	}
	return CachedEventName.Get(Super::GetEventName());
}

bool URigVMUnitNode::CanOnlyExistOnce() const
{
	TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(false);
	if (StructOnScope.IsValid())
	{
		const FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->CanOnlyExistOnce();
	}
	return Super::CanOnlyExistOnce();
}

const TArray<FName>& URigVMUnitNode::GetControlFlowBlocks() const
{
	const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(true);
	if (StructOnScope.IsValid())
	{
		const FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->GetControlFlowBlocks();
	}
	return Super::GetControlFlowBlocks();
}

const bool URigVMUnitNode::IsControlFlowBlockSliced(const FName& InBlockName) const
{
	const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(true);
	if (StructOnScope.IsValid())
	{
		const FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->IsControlFlowBlockSliced(InBlockName);
	}
	return Super::IsControlFlowBlockSliced(InBlockName);
}

FText URigVMUnitNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	if(UScriptStruct* Struct = GetScriptStruct())
	{
		TArray<FString> Parts;
		URigVMPin::SplitPinPath(InPin->GetPinPath(), Parts);

		auto GetTooltipTextFromStruct = [this, InPin, &Parts](UScriptStruct* InStruct, int32 InStartIndex)
		{
			for (int32 PartIndex = InStartIndex; PartIndex < Parts.Num(); PartIndex++)
			{
				FProperty* Property = InStruct->FindPropertyByName(*Parts[PartIndex]);
				if (!Property)
				{
					return FText::GetEmpty();
				}

				if (PartIndex == Parts.Num() - 1)
				{
					return Property->GetToolTipText();
				}

				if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					if (PartIndex < Parts.Num() - 1)
					{
						Property = ArrayProperty->Inner;
						PartIndex++;
					}
				}

				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					InStruct = StructProperty->Struct;
				}
			}

			return FText::GetEmpty();
		};

		FText TooltipText = GetTooltipTextFromStruct(Struct, 1);
		if (!TooltipText.IsEmpty())
		{
			return TooltipText; 
		}

		// Could be a trait pin
		Struct = InPin->GetTraitScriptStruct();
		if (Struct)
		{
			TooltipText = GetTooltipTextFromStruct(Struct, 2);
			if (!TooltipText.IsEmpty())
			{
				return TooltipText; 
			}
		}

		// Possible the pin was programmatically generated from the trait's shared struct
		TSharedPtr<FStructOnScope> TraitScope = InPin->GetTraitInstance();
		if(TraitScope.IsValid())
		{
			const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
			Struct = VMTrait->GetTraitSharedDataStruct();
			if (Struct)
			{
				TooltipText = GetTooltipTextFromStruct(Struct, 2);
			}

			if (!TooltipText.IsEmpty())
			{
				return TooltipText; 
			}
		}
	}
	return URigVMNode::GetToolTipTextForPin(InPin);
}

bool URigVMUnitNode::ShouldInputPinComputeLazily(const URigVMPin* InPin) const
{
	const URigVMPin* RootPin = InPin->GetRootPin();
	check(RootPin->GetNode() == this);

	if(const UScriptStruct* Struct = GetScriptStruct())
	{
		if(const FProperty* Property = Struct->FindPropertyByName(RootPin->GetFName()))
		{
			return Property->HasMetaData(FRigVMStruct::ComputeLazilyMetaName);
		}
	}

	return Super::ShouldInputPinComputeLazily(InPin);
}

bool URigVMUnitNode::IsPinConstant(const URigVMPin* InPin) const
{
	const URigVMPin* RootPin = InPin->GetRootPin();
	check(RootPin->GetNode() == this);

	if(const UScriptStruct* Struct = GetScriptStruct())
	{
		if(const FProperty* Property = Struct->FindPropertyByName(RootPin->GetFName()))
		{
			return Property->HasMetaData(FRigVMStruct::ConstantMetaName);
		}
	}
	return Super::IsPinConstant(InPin);
}

FString URigVMUnitNode::GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const
{
	if(InRootPin->CanProvideDefaultValue())
	{
		const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance(true);
		if(StructOnScope.IsValid())
		{
			if(const UStruct* Struct = StructOnScope->GetStruct())
			{
				if (const FProperty* Property = Struct->FindPropertyByName(InRootPin->GetFName()))
				{
					const uint8* PropertyMemory = Property->ContainerPtrToValuePtr<uint8>(StructOnScope->GetStructMemory());

					FString DefaultValue;
					Property->ExportText_Direct(
						DefaultValue,
						PropertyMemory,
						PropertyMemory,
						nullptr,
						PPF_None,
						nullptr
					);

					return DefaultValue;
				}
			}
		}
	}
	return Super::GetOriginalDefaultValueForRootPin(InRootPin);
}

bool URigVMUnitNode::IsOutDated() const
{
	return !GetDeprecatedMetadata().IsEmpty();
}

FString URigVMUnitNode::GetDeprecatedMetadata() const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		FString DeprecatedMetadata;
		if(Struct->GetStringMetaDataHierarchical(FRigVMStruct::DeprecatedMetaName, &DeprecatedMetadata))
		{
			return DeprecatedMetadata;
		}
	}
	return FString();
}

TArray<FRigVMUserWorkflow> URigVMUnitNode::GetSupportedWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const
{
	TArray<FRigVMUserWorkflow> Workflows = Super::GetSupportedWorkflows(InType, InSubject);

	if(InSubject == nullptr)
	{
		InSubject = this;
	}

	if(UScriptStruct* Struct = GetScriptStruct())
	{
		check(Struct->IsChildOf(FRigVMStruct::StaticStruct()));

		const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance();
		const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope->GetStructMemory();
		TArray<FRigVMUserWorkflow> StructWorkflows = StructMemory->GetWorkflows(InType, InSubject);
		StructWorkflows.Append(URigVMUserWorkflowRegistry::Get()->GetWorkflows(InType, Struct, InSubject));
		Swap(Workflows, StructWorkflows);
		Workflows.Append(StructWorkflows);
	}

	return Workflows;
}

TArray<URigVMPin*> URigVMUnitNode::GetAggregateInputs() const
{
	TArray<URigVMPin*> AggregateInputs;
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if (const UScriptStruct* Struct = GetScriptStruct())
	{
		for (URigVMPin* Pin : GetPins())
		{
			if (Pin->GetDirection() == ERigVMPinDirection::Input)
			{
				if (const FProperty* Property = Struct->FindPropertyByName(Pin->GetFName()))
				{
					if (Property->HasMetaData(FRigVMStruct::AggregateMetaName))
					{
						AggregateInputs.Add(Pin);
					}
				}			
			}
		}
	}
	else
	{
		return Super::GetAggregateInputs();
	}
#endif
	return AggregateInputs;
}

TArray<URigVMPin*> URigVMUnitNode::GetAggregateOutputs() const
{
	TArray<URigVMPin*> AggregateOutputs;
#if UE_RIGVM_AGGREGATE_NODES_ENABLED	
	if (const UScriptStruct* Struct = GetScriptStruct())
	{
		for (URigVMPin* Pin : GetPins())
		{
			if (Pin->GetDirection() == ERigVMPinDirection::Output)
			{
				if (const FProperty* Property = Struct->FindPropertyByName(Pin->GetFName()))
				{
					if (Property->HasMetaData(FRigVMStruct::AggregateMetaName))
					{
						AggregateOutputs.Add(Pin);
					}
				}			
			}
		}
	}
	else
	{
		return Super::GetAggregateOutputs();
	}
#endif
	return AggregateOutputs;
}

FName URigVMUnitNode::GetNextAggregateName(const FName& InLastAggregatePinName) const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED	
	if(const UScriptStruct* Struct = GetScriptStruct())
	{
		check(Struct->IsChildOf(FRigVMStruct::StaticStruct()));

		const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance();
		const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->GetNextAggregateName(InLastAggregatePinName);
	}

	return Super::GetNextAggregateName(InLastAggregatePinName);
#else
	return FName();
#endif
}

FName URigVMUnitNode::GetDisplayNameForPin(const URigVMPin* InPin) const
{
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		check(InPin);
		
		if(InPin->IsExecuteContext())
		{
			if(InPin->GetDirection() == ERigVMPinDirection::IO)
			{
				return FRigVMStruct::ExecuteName;
			}

			int32 NumExecuteContextPins = 0;
			for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
			{
				if(const FStructProperty* StructProperty = CastField<FStructProperty>(*It))
				{
					if(StructProperty->Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
					{
						NumExecuteContextPins++;
					}
				}
			}
			if(NumExecuteContextPins == 1)
			{
				return FRigVMStruct::ExecuteName;
			}
		}
		
		const FName StructMemberDisplayName = GetDisplayNameForStructMember(ScriptStruct, InPin->GetSegmentPath(true));
		if((StructMemberDisplayName == FRigVMStruct::ExecuteContextName) ||
			(InPin->GetFName() == FRigVMStruct::ExecuteContextName && StructMemberDisplayName.IsNone()))
		{
			return FRigVMStruct::ExecuteName;
		}
		return StructMemberDisplayName;
	}
	return Super::GetDisplayNameForPin(InPin);
}

UScriptStruct* URigVMUnitNode::GetScriptStruct() const
{
	if(UScriptStruct* ResolvedStruct = Super::GetScriptStruct())
	{
		return ResolvedStruct;
	}
	return ScriptStruct_DEPRECATED;
}

FName URigVMUnitNode::GetMethodName() const
{
	const FName ResolvedMethodName = Super::GetMethodName();
	if(!ResolvedMethodName.IsNone())
	{
		return ResolvedMethodName;
	}
	return MethodName_DEPRECATED;
}

FString URigVMUnitNode::GetStructDefaultValue() const
{
	TArray<FString> PinDefaultValues;
	for (URigVMPin* Pin : GetPins())
	{
		FString PinDefaultValue = Pin->GetDefaultValue();
		if (Pin->IsStringType())
		{
			PinDefaultValue = TEXT("\"") + PinDefaultValue + TEXT("\"");
		}
		else if (PinDefaultValue.IsEmpty() || PinDefaultValue == TEXT("()"))
		{
			continue;
		}
		PinDefaultValues.Add(FString::Printf(TEXT("%s=%s"), *Pin->GetName(), *PinDefaultValue));
	}
	if (PinDefaultValues.Num() == 0)
	{
		return TEXT("()");
	}
	return FString::Printf(TEXT("(%s)"), *FString::Join(PinDefaultValues, TEXT(",")));

}

TSharedPtr<FStructOnScope> URigVMUnitNode::ConstructStructInstance(bool bUseDefault) const
{
	if (UScriptStruct* Struct = GetScriptStruct())
	{
		TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(Struct));
		FRigVMStruct* StructMemory = (FRigVMStruct*)StructOnScope->GetStructMemory();
		if (!bUseDefault)
		{
			FString StructDefaultValue = GetStructDefaultValue();
			Struct->ImportText(*StructDefaultValue, StructMemory, nullptr, PPF_IncludeTransient, GLog, Struct->GetName());
		}
		return StructOnScope;
	}
	return nullptr;
}

void URigVMUnitNode::EnumeratePropertiesOnHostAndStructInstance(URigVMHost* InHost, TSharedPtr<FStructOnScope> InInstance, bool bPreferLiterals, TFunction<void(const URigVMPin*,const FProperty*,uint8*,const FProperty*,uint8*)> InEnumerationFunction, int32 InSliceIndex) const
{
	check(InHost);
	check(InInstance.IsValid() && InInstance->IsValid());

	const IRigVMEditorAssetInterface* Blueprint = GetImplementingOuter<IRigVMEditorAssetInterface>();
	if(Blueprint == nullptr)
	{
		return;
	}

	URigVM* VM = InHost->GetVM();
	if(VM == nullptr)
	{
		return;
	}

	const FRigVMByteCode& ByteCode = VM->GetByteCode();
	const TArray<int32>& InstructionIndices = ByteCode.GetAllInstructionIndicesForSubject(const_cast<URigVMUnitNode*>(this));
	if(InstructionIndices.IsEmpty())
	{
		return;
	}

	const FRigVMInstructionArray& Instructions = ByteCode.GetInstructions();
	for(int32 Index = 0; Index < InstructionIndices.Num(); Index++)
	{
		const int32& InstructionIndex = InstructionIndices[Index];
		if (!Instructions.IsValidIndex(InstructionIndex))
		{
			continue;
		}
		
		const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
		if(Instruction.OpCode != ERigVMOpCode::Execute)
		{
			continue;
		}

		const FRigVMOperandArray Operands = ByteCode.GetOperandsForOp(Instruction);
		const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InInstance->GetStruct());

		const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(const_cast<UScriptStruct*>(ScriptStruct), *GetMethodName().ToString());
		if(Function == nullptr)
		{
			return;
		}

		const TArray<FRigVMFunctionArgument>& FunctionArguments = Function->GetArguments();
		for(int32 ArgumentIndex = 0; ArgumentIndex < FunctionArguments.Num(); ArgumentIndex++)
		{
			const FRigVMFunctionArgument& Argument = FunctionArguments[ArgumentIndex];
			const URigVMPin* Pin = FindPin(Argument.Name);
			if(Pin == nullptr)
			{
				continue;
			}

			if (!Operands.IsValidIndex(ArgumentIndex))
			{
				continue;
			}
			
			FRigVMOperand Operand = Operands[ArgumentIndex];

			if(bPreferLiterals && Operand.GetMemoryType() != ERigVMMemoryType::Literal)
			{
				// walk backwards on the bytecode to find the copy expression copying
				// the default value literal into the target operand during initialization
				for(int32 CopyInstructionIndex = InstructionIndex - 1; CopyInstructionIndex >= 0; CopyInstructionIndex--)
				{
					const FRigVMInstruction& CopyInstruction = Instructions[CopyInstructionIndex];
					if(CopyInstruction.OpCode == ERigVMOpCode::Copy)
					{
						const FRigVMCopyOp& CopyOp = ByteCode.GetOpAt<FRigVMCopyOp>(CopyInstruction);
						if(CopyOp.Target == Operand &&
							CopyOp.Source.GetMemoryType() == ERigVMMemoryType::Literal &&
							CopyOp.Source.GetRegisterOffset() == INDEX_NONE)
						{
							Operand = CopyOp.Source;
							break;
						}
					}
				}
			}

			const FProperty* InstanceProperty = ScriptStruct->FindPropertyByName(Argument.Name);
			if(InstanceProperty == nullptr)
			{
				continue;
			}

			uint8* InstanceMemory = InstanceProperty->ContainerPtrToValuePtr<uint8>(InInstance->GetStructMemory());
			uint8* HostMemory = nullptr;
			const FProperty* HostProperty = nullptr;
			
			switch(Operand.GetMemoryType())
			{
				case ERigVMMemoryType::External:
				{
					TArray<FRigVMExternalVariable> ExternalVariables = InHost->GetExternalVariables();
					if(ExternalVariables.IsValidIndex(Operand.GetRegisterIndex()))
					{
						FRigVMExternalVariable& ExternalVariable = ExternalVariables[Operand.GetRegisterIndex()];
						HostMemory = ExternalVariable.GetMemory(); 
						HostProperty = ExternalVariable.GetProperty();
					}
					break;
				}
				case ERigVMMemoryType::Literal:
				{
					if(FRigVMMemoryStorageStruct* LiteralMemory = InHost->GetLiteralMemory())
					{
						if(const FProperty* Property = LiteralMemory->GetProperty(Operand.GetRegisterIndex()))
						{
							HostMemory = Property->ContainerPtrToValuePtr<uint8>(LiteralMemory->GetContainerPtr());
							HostProperty = Property;
						}
					}
					break;
				}
				case ERigVMMemoryType::Work:
				{
					if(FRigVMMemoryStorageStruct* WorkMemory = InHost->GetWorkMemory())
					{
						if(const FProperty* Property = WorkMemory->GetProperty(Operand.GetRegisterIndex()))
						{
							HostMemory = Property->ContainerPtrToValuePtr<uint8>(WorkMemory->GetContainerPtr());
							HostProperty = Property;
						}
					}
					break;
				}
				default:
				{
					break;
				}
			}

			// hidden properties are backed up by sliced memory
			if(Pin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				if(InSliceIndex < 0)
				{
					InSliceIndex = 0;
				}
				
				if(const FArrayProperty* HostArrayProperty = CastField<FArrayProperty>(HostProperty))
				{
					FScriptArrayHelper ArrayHelper(HostArrayProperty, HostMemory);
					if(ArrayHelper.Num() <= InSliceIndex)
					{
						UE_LOGF(LogRigVMDeveloper, Warning,
							"Provided slice index %d is invalid when enumerating properties on node '%ls' (total number of slices is %d).",
							InSliceIndex, *GetPathName(), ArrayHelper.Num());
						continue;
					}

					HostProperty = HostArrayProperty->Inner;
					HostMemory = ArrayHelper.GetRawPtr(InSliceIndex);
				}
			}

			if(HostMemory && HostProperty && InstanceProperty && InstanceMemory)
			{
				InEnumerationFunction(Pin, HostProperty, HostMemory, InstanceProperty, InstanceMemory);
			}
		}
	}
}

TSharedPtr<FStructOnScope> URigVMUnitNode::ConstructLiveStructInstance(URigVMHost* InHost, int32 InSliceIndex) const
{
	check(InHost);
	
	TSharedPtr<FStructOnScope> NodeInstance = ConstructStructInstance(false);
	if(!NodeInstance.IsValid() || !NodeInstance->IsValid())
	{
		return NodeInstance;
	}

	EnumeratePropertiesOnHostAndStructInstance(InHost, NodeInstance, false,
	[](const URigVMPin* Pin, const FProperty* HostProperty, uint8* HostMemory, const FProperty* InstanceProperty, uint8* InstanceMemory) {

		// copy the live memory from the host into the node instance
		URigVMMemoryStorage::CopyProperty(InstanceProperty, InstanceMemory, HostProperty, HostMemory);
	}, InSliceIndex);
	
	return NodeInstance;
}

bool URigVMUnitNode::UpdateHostFromStructInstance(URigVMHost* InHost, TSharedPtr<FStructOnScope> InInstance, int32 InSliceIndex) const
{
	check(InHost);
	check(InInstance.IsValid() && InInstance->IsValid());

	bool bChangedSomething = false;
	
	EnumeratePropertiesOnHostAndStructInstance(InHost, InInstance, true,
	[&bChangedSomething]
	(const URigVMPin* Pin, const FProperty* HostProperty, uint8* HostMemory, const FProperty* InstanceProperty, uint8* InstanceMemory) {

		// only update registers for inputs
		if(Pin->GetDirection() != ERigVMPinDirection::Input &&
			Pin->GetDirection() != ERigVMPinDirection::IO &&
			Pin->GetDirection() != ERigVMPinDirection::Visible)
		{
			return;
		}

		// only update registers for pins which don't have a top level link
		if(!Pin->GetRootPin()->GetLinkedSourcePins().IsEmpty())
		{
			return;
		}

		// ignore properties with matching data
		if(HostProperty->SameType(InstanceProperty))
		{
			if(HostProperty->Identical(HostMemory, InstanceMemory))
			{
				return;
			}
		}

		// copy the memory back to the host
		FRigVMMemoryStorageStruct::CopyProperty(HostProperty, HostMemory, InstanceProperty, InstanceMemory);
		bChangedSomething = true;

	}, InSliceIndex);

	return bChangedSomething;
}

void URigVMUnitNode::ComputePinValueDifferences(TSharedPtr<FStructOnScope> InCurrentInstance, TSharedPtr<FStructOnScope> InDesiredInstance, TMap<FString, FString>& OutNewPinDefaultValues) const
{
	check(InCurrentInstance.IsValid() && InCurrentInstance->IsValid());
	check(InDesiredInstance.IsValid() && InDesiredInstance->IsValid());
	
	const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InCurrentInstance->GetStruct());
	check(ScriptStruct == Cast<UScriptStruct>(InDesiredInstance->GetStruct()));
	
	for(URigVMPin* Pin : GetPins())
	{
		if(Pin->GetDirection() != ERigVMPinDirection::Input &&
			Pin->GetDirection() != ERigVMPinDirection::IO &&
			Pin->GetDirection() != ERigVMPinDirection::Visible)
		{
			continue;
		}

		const FProperty* Property = ScriptStruct->FindPropertyByName(Pin->GetFName());
		if(Property == nullptr)
		{
			continue;
		}

		const uint8* CurrentMemory = Property->ContainerPtrToValuePtr<uint8>(InCurrentInstance->GetStructMemory());
		const uint8* DesiredMemory = Property->ContainerPtrToValuePtr<uint8>(InDesiredInstance->GetStructMemory());
		if(Property->Identical(CurrentMemory, DesiredMemory))
		{
			continue;
		}

		FString DefaultValue;
		Property->ExportText_Direct(
			DefaultValue,
			DesiredMemory,
			DesiredMemory,
			nullptr,
			PPF_None,
			nullptr
		);

		if(!DefaultValue.IsEmpty())
		{
			OutNewPinDefaultValues.Add(Property->GetName(), DefaultValue);
		}
	}
}

void URigVMUnitNode::ComputePinValueDifferences(TSharedPtr<FStructOnScope> InDesiredInstance, TMap<FString, FString>& OutNewPinDefaultValues) const
{
	const TSharedPtr<FStructOnScope> DefaultInstance = ConstructStructInstance(false);
	ComputePinValueDifferences(DefaultInstance, InDesiredInstance, OutNewPinDefaultValues);
}

bool URigVMUnitNode::IsPartOfRuntime() const
{
	if(const IRigVMEditorAssetInterface* Blueprint = GetImplementingOuter<IRigVMEditorAssetInterface>())
	{
		if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
		{
			return IsPartOfRuntime(DebuggedHost);
		}
	}
	return false;
}

bool URigVMUnitNode::IsPartOfRuntime(URigVMHost* InHost) const
{
	check(InHost);
	if(URigVM* VM = InHost->GetVM())
	{
		const FRigVMByteCode& ByteCode = VM->GetByteCode();
		return ByteCode.GetFirstInstructionIndexForSubject(const_cast<URigVMUnitNode*>(this)) != INDEX_NONE;
	}
	return false;
}

FRigVMStructUpgradeInfo URigVMUnitNode::GetUpgradeInfo() const
{
	if(UScriptStruct* Struct = GetScriptStruct())
	{
		check(Struct->IsChildOf(FRigVMStruct::StaticStruct()));

		const TSharedPtr<FStructOnScope> StructOnScope = ConstructStructInstance();
		const FRigVMStruct* StructMemory = (const FRigVMStruct*)StructOnScope->GetStructMemory();
		return StructMemory->GetUpgradeInfo();
	}
	return FRigVMStructUpgradeInfo();
}

uint32 URigVMUnitNode::GetStructureHash() const
{
	uint32 Hash = Super::GetStructureHash();
	if(const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		Hash = HashCombine(Hash, FRigVMRegistry::Get().GetHashForScriptStruct(ScriptStruct)); 
	}
	return Hash;
}

