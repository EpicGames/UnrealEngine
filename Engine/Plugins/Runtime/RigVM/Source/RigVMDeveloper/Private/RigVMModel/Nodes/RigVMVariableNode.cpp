// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/Nodes/RigVMVariableNode.h"

#include "RigVMModel/RigVMGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMVariableNode)

URigVMVariableNode::URigVMVariableNode()
{
}

FString URigVMVariableNode::GetNodeTitle() const
{
	if (IsGetter())
	{
		return FString::Printf(TEXT("Get %s"), *GetVariableName().ToString());
	}
	return FString::Printf(TEXT("Set %s"), *GetVariableName().ToString());
}

FName URigVMVariableNode::GetVariableName() const
{
	URigVMPin* VariablePin = FindPin(VariableName);
	if (VariablePin == nullptr)
	{
		return NAME_None;
	}
	return *VariablePin->GetDefaultValue();
}

bool URigVMVariableNode::IsGetter() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return false;
	}
	return ValuePin->GetDirection() == ERigVMPinDirection::Output;
}

bool URigVMVariableNode::IsExternalVariable() const
{
	return !IsLocalVariable() && !IsInputArgument() && !IsInputVariable();
}

bool URigVMVariableNode::IsLocalVariable() const
{
	const FName CurrentVariableName = GetVariableName();
	
	if(URigVMGraph* Graph = GetGraph())
	{
		const TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->GetLocalVariables();
		for (const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
		{
			if(LocalVariable.Name == CurrentVariableName)
			{
				return true;
			}
		}
	}
	return false;
}

bool URigVMVariableNode::IsInputArgument() const
{
	const FName CurrentVariableName = GetVariableName();
	
	if(URigVMGraph* Graph = GetGraph())
	{
		if (URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode())
		{
			if (const URigVMPin* Pin = EntryNode->FindRootPinByName(CurrentVariableName))
			{
				return !Pin->IsDefinedAsInputVariable();
			}
		}
	}
	return false;
}

bool URigVMVariableNode::IsInputVariable() const
{
	const FName CurrentVariableName = GetVariableName();
	
	if(URigVMGraph* Graph = GetGraph())
	{
		if (URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode())
		{
			if (const URigVMPin* Pin = EntryNode->FindRootPinByName(CurrentVariableName))
			{
				return Pin->IsDefinedAsInputVariable();
			}
		}
	}
	return false;
}

FString URigVMVariableNode::GetCPPType() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return FString();
	}
	return ValuePin->GetCPPType();
}

UObject* URigVMVariableNode::GetCPPTypeObject() const
{
	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return nullptr;
	}
	return ValuePin->GetCPPTypeObject();
}

FString URigVMVariableNode::GetDefaultValue() const
{

	URigVMPin* ValuePin = FindPin(ValueName);
	if (ValuePin == nullptr)
	{
		return FString();
	}
	return ValuePin->GetDefaultValue();
}

FRigVMGraphVariableDescription URigVMVariableNode::GetVariableDescription() const
{
	FRigVMGraphVariableDescription Variable;
	Variable.Name = GetVariableName();
	Variable.CPPType = GetCPPType();
	Variable.CPPTypeObject = GetCPPTypeObject();
	if(Variable.CPPTypeObject)
	{
		Variable.CPPTypeObjectPath = *Variable.CPPTypeObject->GetPathName();
	}
	Variable.DefaultValue = GetDefaultValue();
	return Variable;
}

bool URigVMVariableNode::IsPure() const
{
	if (IsGetter())
	{
		return true;
	}
	
	// If its a local variable return true
	if (IsLocalVariable())
	{
		return true;
	}
	
	return false;
}

URigVMPin* URigVMVariableNode::GetVariableNamePin() const
{
	return FindPin(VariableName);
}

URigVMPin* URigVMVariableNode::GetValuePin() const
{
	return FindPin(ValueName);
}

FString URigVMVariableNode::GetNodeSubTitle() const
{
	FString SubTitle;
	
	if(IsLocalVariable())
	{
		static const FString LocalVariableString = TEXT("Local Variable");
		const FString DefaultValue = GetVariableDescription().DefaultValue;
		if(DefaultValue.IsEmpty())
		{
			SubTitle = LocalVariableString;
		}
		else
		{
			SubTitle = FString::Printf(TEXT("%s\nDefault %s"), *LocalVariableString, *DefaultValue);
		}
	}
	else if (IsInputArgument())
	{
		SubTitle = TEXT("Input parameter");
	}
	else if (IsInputVariable())
	{
		SubTitle = TEXT("Input Variable");
	}

	return SubTitle;
}
