// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMClient.h"

#include "RigVMDeveloperModule.h"
#include "RigVMEditorAsset.h"
#include "Misc/TransactionObjectEvent.h"
#include "Misc/ScopedSlowTask.h"
#include "Exporters/Exporter.h"
#include "UObject/ObjectSaveContext.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "EdGraph/RigVMEdGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMClient)

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

UObject* IRigVMClientHost::ResolveUserDefinedTypeById(const FString& InTypeName) const
{
	return nullptr;
}

void FRigVMClient::SetDefaultSchemaClass(TSubclassOf<URigVMSchema> InSchemaClass)
{
	check(InSchemaClass);

	if(InSchemaClass == DefaultSchemaClass)
	{
		return;
	}

	DefaultSchemaClass = InSchemaClass;

	UE::TScopeLock Lock(ControllersLock);
	for(TPair<FSoftObjectPath, TObjectPtr<URigVMController>>& ObjectControllerPair : Controllers)
	{
		ObjectControllerPair.Value->SetSchemaClass(InSchemaClass);
	}
}

void FRigVMClient::SetControllerClass(TSubclassOf<URigVMController> InControllerClass)
{
	check(InControllerClass);

	if (InControllerClass == ControllerClass)
	{
		return;
	}

	for (URigVMGraph* Model : GetModels())
	{
		RemoveController(Model);
	}

	ControllerClass = InControllerClass;
}

void FRigVMClient::SetOuterClientHost(UObject* InOuterClientHost, const FName& InOuterClientHostPropertyName)
{
	{
		UE::TScopeLock Lock(OuterClientHostLock);
	
		OuterClientHost = InOuterClientHost;
		OuterClientPropertyName = InOuterClientHostPropertyName;
	
		check(OuterClientHost->Implements<URigVMClientHost>());
		check(GetOuterClientProperty() != nullptr);
	}

	// create the null graph / default controller.
	// we need this to react to notifs without a valid graph
	// such as interaction brackets.
	static const URigVMGraph* NullGraph = nullptr;
	GetOrCreateController(NullGraph);
}

void FRigVMClient::SetFromDeprecatedData(URigVMGraph* InDefaultGraph, URigVMFunctionLibrary* InFunctionLibrary)
{
	if(GetDefaultModel() != InDefaultGraph ||
		GetFunctionLibrary() != InFunctionLibrary)
	{
		if(GetDefaultModel() == InDefaultGraph)
		{
			Models.Reset();
		}
		
		if(InFunctionLibrary == nullptr)
		{
			Swap(FunctionLibrary, InFunctionLibrary);
		}

		Reset();
		if(InDefaultGraph)
		{
			AddModel(InDefaultGraph, false);
		}
		if(InFunctionLibrary)
		{
			AddModel(InFunctionLibrary, false);
		}
	}
}

void FRigVMClient::SetExternalModelHost(IRigVMClientExternalModelHost* InExternalModelHost)
{
	// We dont allow this to be set dynamically, assume it is set once at creation time
	check(ExternalModelHost == nullptr);

	ExternalModelHost = InExternalModelHost;
}

FRigVMClient& FRigVMClient::operator=(const FRigVMClient& InOther)
{
	OnGetFocusedGraphDelegate = InOther.OnGetFocusedGraphDelegate;
	DefaultSchemaClass = InOther.DefaultSchemaClass;
	ControllerClass = InOther.ControllerClass;
	Models = InOther.Models;
	FunctionLibrary = InOther.FunctionLibrary;

	{
		UE::TScopeLock Lock(ControllersLock);
		Controllers = InOther.Controllers;
	}
	ActionStack = InOther.ActionStack;

	UndoRedoIndex = 0;
	UndoStack.Reset();
	RedoStack.Reset();

	bSuspendNotifications = InOther.bSuspendNotifications;
	bIgnoreModelNotifications = InOther.bIgnoreModelNotifications;
	bDefaultModelCanBeRemoved = InOther.bDefaultModelCanBeRemoved;
	bSuspendModelNotificationsForOthers = InOther.bSuspendModelNotificationsForOthers;

	{
		UE::TScopeLock Lock(OuterClientHostLock);
		OuterClientHost = InOther.OuterClientHost;
		OuterClientPropertyName = InOther.OuterClientPropertyName;
	}
	ExternalModelHost = InOther.ExternalModelHost;

#if WITH_EDITOR
	PostGraphModifiedDelegate = InOther.PostGraphModifiedDelegate;
#endif // WITH_EDITOR
	
	return *this;
}

void FRigVMClient::Reset()
{
	for(URigVMGraph* Model : GetModels())
	{
		DestroyObject(Model);
	}

	{
		UE::TScopeLock Lock(ControllersLock);
		for(auto Pair : Controllers)
		{
			DestroyObject(Pair.Value);
		}
		Controllers.Reset();
	}
	
	DestroyObject(FunctionLibrary);

	Models.Reset();
	FunctionLibrary = nullptr;

	ResetActionStack();
}

URigVMSchema* FRigVMClient::GetDefaultSchema() const
{
	check(DefaultSchemaClass);
	return DefaultSchemaClass->GetDefaultObject<URigVMSchema>();
}

URigVMGraph* FRigVMClient::GetDefaultModel() const
{
	if(GetModels().IsEmpty())
	{
		return nullptr;
	}
	return GetModel(0);
}

URigVMGraph* FRigVMClient::GetModel(int32 InIndex) const
{
	const TArray<TObjectPtr<URigVMGraph>>& LocalModels = GetModels();
	if(LocalModels.IsValidIndex(InIndex))
	{
		return LocalModels[InIndex];
	}
	return nullptr;
}


URigVMGraph* FRigVMClient::GetModel(const UEdGraph* InEdGraph) const
{
	if (InEdGraph == nullptr)
	{
		return GetDefaultModel();
	}

//#if WITH_EDITORONLY_DATA
//	if (InEdGraph == FunctionLibraryEdGraph)
//	{
//		return RigVMClient.GetFunctionLibrary();
//	}
//#endif

	const URigVMEdGraph* RigGraph = Cast< URigVMEdGraph>(InEdGraph);
	check(RigGraph);
	return GetModel(RigGraph->ModelNodePath);

}

URigVMGraph* FRigVMClient::GetModel(const FString& InNodePathOrName) const
{
	if(InNodePathOrName.IsEmpty())
	{
		return GetDefaultModel();
	}

	TArray<URigVMGraph*> ModelsAndFunctionLibrary = GetAllModels(true, false);
	for(URigVMGraph* Model : ModelsAndFunctionLibrary)
	{
		if(Model->GetNodePath() == InNodePathOrName || Model->GetName() == InNodePathOrName)
		{
			return Model;
		}

		static constexpr TCHAR NodePathPrefixFormat[] = TEXT("%s|");
		const FString NodePathPrefix = FString::Printf(NodePathPrefixFormat, *Model->GetNodePath());
		if(InNodePathOrName.StartsWith(NodePathPrefix))
		{
			const FString RemainingNodePath = InNodePathOrName.Mid(NodePathPrefix.Len());
			if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Model->FindNode(RemainingNodePath)))
			{
				return CollapseNode->GetContainedGraph();
			}
		}
	}
	return nullptr;
}

URigVMGraph* FRigVMClient::GetModel(const UObject* InEditorSideObject) const
{
	check(InEditorSideObject);

	if (InEditorSideObject->Implements<URigVMEditorSideObject>())
	{
		const IRigVMEditorSideObject* UserInterfaceGraph = Cast<IRigVMEditorSideObject>(InEditorSideObject);
		return GetModel(UserInterfaceGraph->GetRigVMNodePath());
	}

	return nullptr;
}

void FRigVMClient::RefreshAllModels(ERigVMLoadType InLoadType, bool bEnablePostLoadHashing, bool& bIsCompiling)
{
	const bool bIsPostLoad = InLoadType == ERigVMLoadType::PostLoad;

	// avoid any compute if the current structure hashes match with the serialized ones
	if (bEnablePostLoadHashing && GetStructureHash() == GetSerializedStructureHash())
	{
		if (bIsPostLoad)
		{
			TArray<URigVMGraph*> ModelGraphs = GetAllModels(true, true);
			Algo::Reverse(ModelGraphs);
			for (URigVMGraph* ModelGraph : ModelGraphs)
			{
				URigVMController* Controller = GetOrCreateController(ModelGraph);
				URigVMController::FRestoreLinkedPathSettings Settings;
				Settings.bFollowCoreRedirectors = true;
				Settings.bRelayToOrphanPins = true;
				Controller->ProcessDetachedLinks(Settings);
			}
		}
		return;
	}

	TGuardValue<bool> IsCompilingGuard(bIsCompiling, true);
	TGuardValue<bool> ClientIgnoreModificationsGuard(bIgnoreModelNotifications, true);

	TArray<URigVMGraph*> AllModelsLeavesFirst = GetAllModelsLeavesFirst(true);
	TMap<const URigVMGraph*, TArray<URigVMController::FLinkedPath>> LinkedPaths;

	if (ensure(IsInGameThread()))
	{
		TArray<URigVMController::FRepopulatePinsNodeData> RepopulatePinsNodesData;
		constexpr int32 REPOPULATE_NODES_NUM_RESERVED = 800;
		RepopulatePinsNodesData.Reserve(REPOPULATE_NODES_NUM_RESERVED);

		for (URigVMGraph* Graph : AllModelsLeavesFirst)
		{
			URigVMController* Controller = GetOrCreateController(Graph);
			// temporarily disable default value validation during load time, serialized values should always be accepted
			TGuardValue<bool> PerGraphDisablePinDefaultValueValidation(Controller->bValidatePinDefaults, false);
			TGuardValue<bool> GuardEditGraph(Graph->bEditable, true);
			FRigVMControllerNotifGuard NotifGuard(Controller, true);
			LinkedPaths.Add(Graph, Controller->GetLinkedPaths());

			const TArray<URigVMNode*> Nodes = Graph->GetNodes();
			if (Nodes.Num() > 0)
			{
				RepopulatePinsNodesData.Reset();

				for (URigVMNode* Node : Nodes)
				{
					Controller->GenerateRepopulatePinsNodeData(RepopulatePinsNodesData, Node, true, true);
				}

#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
				UE_LOGF(LogRigVMDeveloper, Display, "--- Graph: [%ls/%ls]  - NumNodes : [%d]", *Graph->GetOuter()->GetName(), *Graph->GetName(), RepopulatePinsNodesData.Num());
#endif

				Controller->OrphanPins(RepopulatePinsNodesData);
				Controller->FastBreakLinkedPaths(LinkedPaths.FindChecked(Graph));
				Controller->RepopulatePins(RepopulatePinsNodesData);
			}
		}

		if (IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter()))
		{
			ClientHost->SetupPinRedirectorsForBackwardsCompatibility();
		}
	}

	for (URigVMGraph* Graph : AllModelsLeavesFirst)
	{
		URigVMController* Controller = GetOrCreateController(Graph);
		TGuardValue<bool> GuardEditGraph(Graph->bEditable, true);
		FRigVMControllerNotifGuard NotifGuard(Controller, true);
		{
			URigVMController::FRestoreLinkedPathSettings Settings;
			Settings.bFollowCoreRedirectors = true;
			Settings.bRelayToOrphanPins = true;
			Controller->RestoreLinkedPaths(LinkedPaths.FindChecked(Graph), Settings);
		}

		for (URigVMNode* ModelNode : Graph->GetNodes())
		{
			Controller->RemoveUnusedOrphanedPins(ModelNode);
		}

		if (bIsPostLoad)
		{
			for (URigVMNode* ModelNode : Graph->GetNodes())
			{
				if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelNode))
				{
					TemplateNode->InvalidateCache();
					TemplateNode->PostLoad();
				}
				
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelNode))
				{
					if (!VariableNode->GetVariableGuid().IsValid())
					{
						FName VarName = VariableNode->GetVariableName();
						Controller->RefreshVariableNode(VariableNode->GetFName(), VarName, VariableNode->GetCPPType(), VariableNode->GetCPPTypeObject(), false, false);
					}
				}
			}
		}

#if WITH_EDITOR

		if (bIsPostLoad)
		{
			for (URigVMNode* ModelNode : Graph->GetNodes())
			{
				if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode))
				{
					if (!UnitNode->HasWildCardPin())
					{
						UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct();
						if (ScriptStruct == nullptr)
						{
							Controller->FullyResolveTemplateNode(UnitNode, INDEX_NONE, false);
						}

						// Try to find a deprecated template
						if (UnitNode->GetScriptStruct() == nullptr && !UnitNode->TemplateNotation.IsNone())
						{
							const FRigVMTemplate* Template = FRigVMRegistry::Get().FindTemplate(UnitNode->TemplateNotation, true);
							FRigVMTemplate::FTypeMap TypeMap = UnitNode->GetTemplatePinTypeMap();

							int32 Permutation;
							if (Template->FullyResolve(TypeMap, Permutation))
							{
								const FRigVMFunction* Function = Template->GetPermutation(Permutation);
								UnitNode->ResolvedFunctionName = Function->GetName();
							}
						}

						if (UnitNode->GetScriptStruct() == nullptr)
						{
							static constexpr TCHAR UnresolvedUnitNodeMessage[] = TEXT("Node %s could not be resolved.");
							Controller->ReportErrorf(UnresolvedUnitNodeMessage, *ModelNode->GetNodePath(true));
						}
					}
				}
				if (URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
				{
					if (DispatchNode->GetFactory() == nullptr)
					{
						static constexpr TCHAR UnresolvedDispatchNodeMessage[] = TEXT("Dispatch node %s has no factory..");
						Controller->ReportErrorf(UnresolvedDispatchNodeMessage, *ModelNode->GetNodePath(true));
					}
					else if (!DispatchNode->HasWildCardPin())
					{
						if (DispatchNode->GetResolvedFunction() == nullptr)
						{
							Controller->FullyResolveTemplateNode(DispatchNode, INDEX_NONE, false);
						}
						if (DispatchNode->GetResolvedFunction() == nullptr)
						{
							static constexpr TCHAR UnresolvedDispatchNodeMessage[] = TEXT("Node %s could not be resolved.");
							Controller->ReportErrorf(UnresolvedDispatchNodeMessage, *ModelNode->GetNodePath(true));
						}
					}
				}
			}
		}
#endif

	}
}

void FRigVMClient::RefreshVariableNodes(const TArray<FRigVMExternalVariable> InPublicVariables)
{
	TArray<URigVMGraph*> AllModels = GetModels();
	for(URigVMGraph* Model : AllModels)
	{
		URigVMController* Controller = GetOrCreateController(Model);
		TGuardValue<bool> DirtyGuard(Controller->bSuspendMarkingPackageDirty, true);
		for (URigVMNode* ModelNode : Model->GetNodes())
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(ModelNode))
			{
				for (const FRigVMExternalVariable& PublicVariable : InPublicVariables)
				{
					if (VariableNode->GetVariableGuid() == PublicVariable.GetGuid())
					{
						FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(PublicVariable.GetTypeIndex());
						Controller->RefreshVariableNode(VariableNode->GetFName(), PublicVariable.GetName(), Type.CPPType.ToString(), Type.CPPTypeObject, false, true);
					}
				}
			}
		}
	}
}

const TArray<TObjectPtr<URigVMGraph>>& FRigVMClient::GetModels() const
{
	return ExternalModelHost != nullptr ? ExternalModelHost->GetExternalModels() : Models;
}

TArray<URigVMGraph*> FRigVMClient::GetAllModels(bool bIncludeFunctionLibrary, bool bRecursive) const
{
	TArray<URigVMGraph*> AllModels = GetModels();
	if(bRecursive)
	{
		for(URigVMGraph* Model : GetModels())
		{
			AllModels.Append(Model->GetContainedGraphs(true /* recursive */));
		}
	}
	if(bIncludeFunctionLibrary && FunctionLibrary)
	{
		AllModels.Add(FunctionLibrary);
		if(bRecursive)
		{
			AllModels.Append(FunctionLibrary->GetContainedGraphs(true /* recursive */));
		}
	}
	return AllModels;
}

TArray<URigVMGraph*> FRigVMClient::GetAllModelsLeavesFirst(bool bIncludeFunctionLibrary) const
{
	TArray<URigVMGraph*> SortedModels = GetAllModels(bIncludeFunctionLibrary, true);
	URigVMController::SortGraphElementsByGraphDepth(SortedModels, true);
	return SortedModels;
}

URigVMController* FRigVMClient::GetController(int32 InIndex) const
{
	return GetController(GetModel(InIndex));
}

URigVMController* FRigVMClient::GetController(const FString& InNodePathOrName) const
{
	return GetController(GetModel(InNodePathOrName));
}

URigVMController* FRigVMClient::GetController(const URigVMGraph* InModel) const
{
	if(InModel == nullptr)
	{
		InModel = GetDefaultModel();
	}
	
	if(InModel)
	{
		const FSoftObjectPath Key(InModel);
		UE::TScopeLock Lock(ControllersLock);

		if(const TObjectPtr<URigVMController>* ExistingController = Controllers.Find(Key))
		{
			if  (!URigVMHost::IsGarbageOrDestroyed(ExistingController->Get()))
			{
				URigVMController* Existing = ExistingController->Get();
				// Identity compare, not path: the stale slot has the same FSoftObjectPath key but its
				// controller's graph is a different UObject. This happens when undo of a creation rolls
				// the original graph into the transient package and a fresh URigVMGraph is then allocated
				// at the same asset path. Defensive cleanup at the read site — see PurgeStaleController.
				if(Existing->GetGraph() == InModel)
				{
					return Existing;
				}
				PurgeStaleController(Key, Existing);
			}
		}
	}
	return nullptr;
}

URigVMController* FRigVMClient::GetController(const UObject* InEditorSideObject) const
{
	check(InEditorSideObject);

	if (InEditorSideObject->Implements<URigVMEditorSideObject>())
	{
		const IRigVMEditorSideObject* UserInterfaceGraph = Cast<IRigVMEditorSideObject>(InEditorSideObject);
		return GetController(UserInterfaceGraph->GetRigVMNodePath());
	}

	return nullptr;
}

URigVMController* FRigVMClient::GetOrCreateController(int32 InIndex)
{
	return GetOrCreateController(GetModel(InIndex));
}

URigVMController* FRigVMClient::GetOrCreateController(const FString& InNodePathOrName)
{
	return GetOrCreateController(GetModel(InNodePathOrName));
}

URigVMController* FRigVMClient::GetOrCreateController(const URigVMGraph* InModel)
{
	if(InModel == nullptr)
	{
		InModel = GetDefaultModel();
	}

	if(InModel)
	{
		if(URigVMController* Controller = GetController(InModel))
		{
			if (IsValid(Controller))
			{
				// We associate controllers to graphs via soft path, so they can match newly created graphs
				// If this happens make sure the graph is correctly bound to the controller
				if (!InModel->ModifiedEvent.IsBound())
				{
					const_cast<URigVMGraph*>(InModel)->OnModified().AddUObject(Controller, &URigVMController::HandleModifiedEvent);
				}
				return Controller;
			}
		}
		return CreateController(InModel);
	}
	return nullptr; 
}

URigVMController* FRigVMClient::GetOrCreateController(const UObject* InEditorSideObject)
{
	check(InEditorSideObject);

	if (InEditorSideObject->Implements<URigVMEditorSideObject>())
	{
		const IRigVMEditorSideObject* UserInterfaceGraph = Cast<IRigVMEditorSideObject>(InEditorSideObject);
		return GetOrCreateController(UserInterfaceGraph->GetRigVMNodePath());
	}

	return nullptr;
}

URigVMController* FRigVMClient::GetControllerByName(const FString InGraphName) const
{
	if (InGraphName.IsEmpty())
	{
		if (const URigVMGraph* DefaultModel = GetDefaultModel())
		{
			return GetController(DefaultModel);
		}
	}

	for (const URigVMGraph* Graph : GetAllModels(true, true))
	{
		if (Graph->GetName() == InGraphName || Graph->GetGraphName() == InGraphName)
		{
			return GetController(Graph);
		}
	}

	return nullptr;
}

bool FRigVMClient::RemoveController(const URigVMGraph* InModel)
{
	UE::TScopeLock Lock(ControllersLock);

	if(InModel == nullptr)
	{
		InModel = GetDefaultModel();
	}
	
	const FSoftObjectPath Key(InModel);
	
	URigVMController* Controller = nullptr;
	bool bSuccess = false;
	if(const TObjectPtr<URigVMController>* ExistingController = Controllers.Find(Key))
	{
		bSuccess = true;
		Controller = ExistingController->Get();
	}

	PurgeStaleController(Key, Controller);
	return bSuccess;
}

void FRigVMClient::PurgeStaleController(const FSoftObjectPath& Key, URigVMController* StaleController) const
{
	// Caller owns ControllersLock. Evicts the slot from the map first so any reentrant lookup that fires
	// from the teardown ops below misses cleanly (rather than seeing a controller in mid-destruction), then
	// tears down the orphan if one was supplied. The garbage/null checks live here so callers can pass any
	// controller pointer without thinking about it — this helper is the single source of truth for the
	// "drop a slot and tear its controller down" sequence.
	Controllers.Remove(Key);
	if (StaleController && !URigVMHost::IsGarbageOrDestroyed(StaleController))
	{
		StaleController->OnModified().Clear();
		StaleController->SetActionStack(nullptr);
		StaleController->Rename(nullptr, GetTransientPackage(),
			REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		StaleController->RemoveFromRoot();
		StaleController->MarkAsGarbage();
	}
}

URigVMGraph* FRigVMClient::AddModel(const FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	const FString DesiredName = FString::Printf(TEXT("%s %s"), FRigVMClient::RigVMModelPrefix, *InName);
	return AddModel(*DesiredName, bSetupUndoRedo);
}

URigVMGraph* FRigVMClient::AddModel(const FName& InName, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer, bool bCreateController)
{
	check(DefaultSchemaClass);

	URigVMGraph* NewModel = CreateModel(InName, DefaultSchemaClass, bSetupUndoRedo, GetOuter(), ObjectInitializer, bCreateController);
	AddModel(NewModel, bCreateController);
	return NewModel;
}

URigVMGraph* FRigVMClient::AddModel(const FName& InName, TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer, bool bCreateController)
{
	URigVMGraph* NewModel = CreateModel(InName, InSchemaClass, bSetupUndoRedo, GetOuter(), ObjectInitializer, bCreateController);
	AddModel(NewModel, bCreateController);
	return NewModel;
}

URigVMGraph* FRigVMClient::CreateModel(const FName& InName, TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, UObject* InOuter, const FObjectInitializer* ObjectInitializer, bool bCreateController)
{
	check(InSchemaClass);

#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> Transaction;
	if(bSetupUndoRedo)
	{
		Transaction = MakeShared<FScopedTransaction>(NSLOCTEXT("RigVMClient", "AddModel", "Add new root graph"));
		InOuter->Modify();
	}
#endif

	const FName SafeGraphName = GetUniqueName(InName);
	URigVMGraph* Model = nullptr;
	if(ObjectInitializer)
	{
		Model = ObjectInitializer->CreateDefaultSubobject<URigVMGraph>(InOuter, SafeGraphName);
	}
	else
	{
		Model = NewObject<URigVMGraph>(InOuter, SafeGraphName);
	}

	Model->SetSchemaClass(InSchemaClass);

	if(bSetupUndoRedo)
	{
		InOuter->Modify();
		UndoRedoIndex++;

		FRigVMClientAction Action;
		Action.Type = ERigVMClientAction::ERigVMClientAction_AddModel;
		Action.NodePath = Model->GetNodePath();
		UndoStack.Push(Action);
		RedoStack.Reset();
	}
	return Model;
}

TObjectPtr<URigVMGraph> FRigVMClient::CreateContainedGraphModel(URigVMCollapseNode* CollapseNode, const FName& Name)
{
	TObjectPtr<URigVMGraph> Model = nullptr;

	check(CollapseNode);

	if (ExternalModelHost != nullptr)
	{
		Model = ExternalModelHost->CreateContainedGraphModel(CollapseNode, Name);
	}
	else
	{
		Model = NewObject<URigVMGraph>(CollapseNode, Name);

		// keep schema from collapse node graph, if exists
		if (CollapseNode->GetGraph() != nullptr && CollapseNode->GetGraph()->GetSchema() != nullptr)
		{
			Model->SetSchemaClass(CollapseNode->GetGraph()->GetSchema()->GetClass());
		}
		else
		{
			Model->SetSchemaClass(GetDefaultSchemaClass());
		}
	}

	return Model;
}

void FRigVMClient::AddModel(URigVMGraph* InModel, bool bCreateController)
{
	check(InModel);

	if(InModel->IsA<URigVMFunctionLibrary>())
	{
		check(FunctionLibrary == nullptr);
		FunctionLibrary = Cast<URigVMFunctionLibrary>(InModel);
	}
	else
	{
		if(ExternalModelHost == nullptr)
		{
			Models.Add(InModel);
		}
	}

	if(InModel->GetSchemaClass() == nullptr)
	{
		InModel->SetSchemaClass(GetDefaultSchemaClass());
	}

	InModel->SetExecuteContextStruct(InModel->GetSchema()->GetExecuteContextStruct());

	if(bCreateController)
	{
		CreateController(InModel);
	}

	if(InModel->IsA<URigVMFunctionLibrary>())
	{
		for(URigVMGraph* Model : GetModels())
		{
			Model->SetDefaultFunctionLibrary(FunctionLibrary);
		}
		InModel->SetDefaultFunctionLibrary(FunctionLibrary);
	}
	else if(FunctionLibrary)
	{
		InModel->SetDefaultFunctionLibrary(FunctionLibrary);
	}

	if (GetOuter()->Implements<URigVMClientHost>())
	{
		IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter());
		ClientHost->HandleRigVMGraphAdded(this, InModel->GetNodePath());
	}
		
	NotifyOuterOfPropertyChange();

}

URigVMFunctionLibrary* FRigVMClient::GetOrCreateFunctionLibrary(bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer, bool bCreateController)
{
	check(DefaultSchemaClass);
	return GetOrCreateFunctionLibrary(DefaultSchemaClass, bSetupUndoRedo, ObjectInitializer, bCreateController);
}

void FRigVMClient::SetGetFunctionHostObjectPathDelegate(URigVMFunctionLibrary* InFunctionLibrary)
{
	if (GetOuter()->Implements<URigVMClientHost>())
	{
		if (IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter()))
		{
			TWeakObjectPtr<UObject> WeakClientHost = Cast<UObject>(ClientHost);
			InFunctionLibrary->GetFunctionHostObjectPathDelegate.BindLambda([WeakClientHost]() -> const FSoftObjectPath
				{
					if (WeakClientHost.IsValid())
					{
						if (IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(WeakClientHost.Get()))
						{
							return ClientHost->GetRigVMGraphFunctionHost().GetObject();
						}
					}
					return nullptr;
				});
		}
	}
};

URigVMFunctionLibrary* FRigVMClient::GetOrCreateFunctionLibrary(TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer, bool bCreateController)
{
	if(FunctionLibrary)
	{
		if (!FunctionLibrary->GetFunctionHostObjectPathDelegate.IsBound())
		{
			SetGetFunctionHostObjectPathDelegate(FunctionLibrary);
		}
		return FunctionLibrary;
	}
	
#if WITH_EDITOR
	TSharedPtr<FScopedTransaction> Transaction;
	if(bSetupUndoRedo)
	{
		Transaction = MakeShared<FScopedTransaction>(NSLOCTEXT("RigVMClient", "AddModel", "Add new root graph"));
		GetOuter()->Modify();
	}
#endif

	static constexpr TCHAR FunctionLibraryName[] = TEXT("RigVMFunctionLibrary");
	const FName SafeGraphName = GetUniqueName(FunctionLibraryName);
	URigVMFunctionLibrary* NewFunctionLibrary = nullptr;
	if(ObjectInitializer)
	{
		NewFunctionLibrary = ObjectInitializer->CreateDefaultSubobject<URigVMFunctionLibrary>(GetOuter(), SafeGraphName);
	}
	else
	{
		NewFunctionLibrary = NewObject<URigVMFunctionLibrary>(GetOuter(), SafeGraphName);
	}

	NewFunctionLibrary->SetSchemaClass(InSchemaClass);

	if (!NewFunctionLibrary->GetFunctionHostObjectPathDelegate.IsBound())
	{
		SetGetFunctionHostObjectPathDelegate(NewFunctionLibrary);
	}

	AddModel(NewFunctionLibrary, bCreateController);
	return NewFunctionLibrary;
}

TArray<FName> FRigVMClient::GetEntryNames(UScriptStruct* InUnitScriptStructFilter) const
{
	TArray<FName> EntryNames;
	for(const URigVMGraph* Model : GetModels())
	{
		for(const URigVMNode* Node : Model->GetNodes())
		{
			// Filter out unit nodes that are not of the specified type
			if(InUnitScriptStructFilter != nullptr)
			{
				if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
				{
					UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct();
					if(ScriptStruct == nullptr || !ScriptStruct->IsChildOf(InUnitScriptStructFilter))
					{
						continue;
					}
				}
			}

			const FName EntryName = Node->GetEventName();
			if(!EntryName.IsNone())
			{
				EntryNames.Add(EntryName);
			}
		}
	}
	return EntryNames;
}

UScriptStruct* FRigVMClient::GetDefaultExecuteContextStruct() const
{
	check(GetDefaultSchema());
	return GetDefaultSchema()->GetExecuteContextStruct();
}

void FRigVMClient::SetDefaultExecuteContextStruct(UScriptStruct* InExecuteContextStruct)
{
	GetDefaultSchema()->SetExecuteContextStruct(InExecuteContextStruct);
}

URigVMGraph* FRigVMClient::GetFocusedModel() const
{
#if WITH_EDITOR
	if (OnGetFocusedGraph().IsBound())
	{
		return OnGetFocusedGraph().Execute();
	}
#endif

	return GetDefaultModel();
}

bool FRigVMClient::RemoveModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return RemoveModel(InName, bSetupUndoRedo);
}

bool FRigVMClient::RemoveModel(const FString& InNodePathOrName, bool bSetupUndoRedo)
{
	if(URigVMGraph* Model = GetModel(InNodePathOrName))
	{
		UObject* ModelOuter = Model->GetOuter();
		check(ModelOuter);
		
		if(Model == GetDefaultModel() && !bDefaultModelCanBeRemoved)
		{
#if WITH_EDITOR
			static constexpr TCHAR Message[] = TEXT("Cannot remove the default model.");
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, Message, *FString());
#endif
			return false;
		}

		if(Model == FunctionLibrary)
		{
#if WITH_EDITOR
			static constexpr TCHAR Message[] = TEXT("Cannot remove the function library.");
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, Message, *FString());
#endif
			return false;
		}

#if WITH_EDITOR
		TSharedPtr<FScopedTransaction> Transaction;
		if(bSetupUndoRedo)
		{
			Transaction = MakeShared<FScopedTransaction>(NSLOCTEXT("RigVMClient", "RemoveModel", "Remove a root graph"));
			ModelOuter->Modify();
		}
#endif

		if (GetOuter()->Implements<URigVMClientHost>())
		{
			IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter());
			ClientHost->HandleRigVMGraphRemoved(this, InNodePathOrName);
		}

		if(bSetupUndoRedo)
		{
			ModelOuter->Modify();
			UndoRedoIndex++;

			FRigVMClientAction Action;
			Action.Type = ERigVMClientAction_RemoveModel;
			Action.NodePath = Model->GetNodePath();
			UndoStack.Push(Action);
			RedoStack.Reset();
		}

		// clean up the model
		if(ExternalModelHost == nullptr)
		{
			verify(Models.Remove(Model));
		}
		else
		{
			// Should have already been removed from external models
			verify(!ExternalModelHost->GetExternalModels().Contains(Model));
		}

		NotifyOuterOfPropertyChange();
		return true;
	}
	return false;
}

FName FRigVMClient::RenameModel(const FString& InNodePathOrName, const FName& InNewName, bool bSetupUndoRedo)
{
	if(URigVMGraph* Model = GetModel(InNodePathOrName))
	{
		if(Model == FunctionLibrary)
		{
#if WITH_EDITOR
			static constexpr TCHAR Message[] = TEXT("Cannot rename the function library.");
			FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, Message, *FString());
#endif
			return NAME_None;
		}

		if(Model->GetFName() == InNewName)
		{
			return InNewName;
		}

#if WITH_EDITOR
		TSharedPtr<FScopedTransaction> Transaction;
		if(bSetupUndoRedo)
		{
			Transaction = MakeShared<FScopedTransaction>(NSLOCTEXT("RigVMClient", "RenameModel", "Rename a root graph"));
		}
#endif

		const FString OldNodePath = Model->GetNodePath();
		const FName SafeNewName = GetUniqueName(InNewName);
		const FSoftObjectPath OldKey(Model);
		FString NewNodePath;

		{
			UE::TScopeLock Lock(ControllersLock);
			TObjectPtr<URigVMController>* Controller = Controllers.Find(Model);
			Model->Rename(*SafeNewName.ToString(), nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			NewNodePath = Model->GetNodePath();
			if (Controller)
			{
				Controllers.Add(Model, *Controller);
				Controllers.Remove(OldKey);
			}
		}

		if(bSetupUndoRedo)
		{
			GetOuter()->Modify();
			UndoRedoIndex++;

			FRigVMClientAction Action;
			Action.Type = ERigVMClientAction_RenameModel;
			Action.NodePath = OldNodePath;
			Action.OtherNodePath = NewNodePath;
			// Record the slot the model occupies in Models[] so PostTransacted can rebind it
			// back to the exact slot if the FPersistentObjectRef-by-name restore nulled it out.
			Action.ModelIndex = (ExternalModelHost == nullptr) ? Models.IndexOfByKey(Model) : INDEX_NONE;
			UndoStack.Push(Action);
			RedoStack.Reset();
		}

		if (GetOuter()->Implements<URigVMClientHost>())
		{
			IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter());
			ClientHost->HandleRigVMGraphRenamed(this, OldNodePath, NewNodePath);
		}
		
		NotifyOuterOfPropertyChange();
		return SafeNewName;
	}

	return NAME_None;
}

void FRigVMClient::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	IRigVMClientHost* ClientHost = CastChecked<IRigVMClientHost>(GetOuter());

	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		auto PerformAction = [this, ClientHost](const FRigVMClientAction& InAction, bool bUndo)
		{
			switch(InAction.Type)
			{
				case ERigVMClientAction_AddModel:
				case ERigVMClientAction_RemoveModel:
				{
					if(InAction.Type == ERigVMClientAction_RemoveModel)
					{
						bUndo = !bUndo;
					}
						
					if(URigVMGraph* Model = GetModel(InAction.NodePath))
					{
						if(bUndo)
						{
							ClientHost->HandleRigVMGraphRemoved(this, InAction.NodePath);
						}
						else
						{
							URigVMController* Controller = GetOrCreateController(Model);
							ClientHost->HandleRigVMGraphAdded(this, InAction.NodePath);
							if(Controller)
							{
								Controller->ResendAllNotifications();
							}
						}
					}
					break;
				}
				case ERigVMClientAction_RenameModel:
				{
					// Why this branch repairs Models[] instead of just calling RenameModel:
					//
					// The default URigVMGraph (and the URigVMFunctionLibrary) carry
					// RF_DefaultSubObject. The transaction system serializes RF_DefaultSubObject
					// references by FName — the captured name is the graph's name at outer-Modify
					// time, and resolution on undo/redo goes through StaticFindObjectFast(Outer, Name)
					// (see FPersistentObjectRef in CoreUObject/Public/Misc/TransactionCommon.h).
					//
					// FRigVMClient::RenameModel updates the graph's FName but does NOT touch the
					// captured snapshot. So on undo/redo, each pass leaves the graph at the OPPOSITE
					// name from what's captured, the name lookup fails, and Models[i] is restored
					// as nullptr. The graph itself is fine — only the array slot is bad.
					//
					// We do NOT call FRigVMClient::RenameModel(...) here because:
					//   - It looks the graph up via GetModel(), which iterates Models[]. The slot
					//     we need is exactly the nullptr one, so GetModel() returns nullptr.
					//   - It calls UObject::Rename without REN_NonTransactional, which would push
					//     another FObjectRecord into the in-flight transaction we're applying.
					//   - It routes the new name through GetUniqueName, which would mangle the
					//     target name we need to restore exactly.
					//
					// We DO rename the graph here directly via UObject::Rename when its current FName
					// isn't already TargetName. In some cases a prior step in FTransaction::Apply has
					// already restored the name; the GetFName() != TargetName guard makes our Rename
					// a no-op in that case.
					//
					// Steps:
					//   (1) Find the graph directly via StaticFindObjectFast on the outer (bypassing
					//       Models[], which may have a nullptr slot). Try SourceName first; fall
					//       back to TargetName for the already-renamed case.
					//   (2) Rename it to TargetName if it isn't already there. REN_NonTransactional
					//       so we don't pollute the in-flight transaction; REN_DoNotDirty because
					//       undo/redo isn't a user edit.
					//   (3) Repair Models[]: rebind to the EXACT slot recorded with the action
					//       (InAction.ModelIndex). Picking the first nullptr would corrupt the
					//       array order when two slots are simultaneously nullptr (e.g. two root
					//       renames inside a single outer transaction during redo).
					//   (4) Rekey the Controllers map. Look up by graph pointer (not by precomputed
					//       FSoftObjectPath), matching OnCollapseNodeRenamed. The map's existing
					//       key may not match the graph's current path.
					//   (5) Fire HandleRigVMGraphRenamed for UI / EdGraph sync.
					const FString SourcePath = bUndo ? InAction.OtherNodePath : InAction.NodePath;
					const FString TargetPath = bUndo ? InAction.NodePath : InAction.OtherNodePath;
					FString TargetNameStr = TargetPath;
					TargetNameStr.Split(TEXT("|"), nullptr, &TargetNameStr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					TargetNameStr.RemoveFromEnd(TEXT("::"));
					FString SourceNameStr = SourcePath;
					SourceNameStr.Split(TEXT("|"), nullptr, &SourceNameStr, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
					SourceNameStr.RemoveFromEnd(TEXT("::"));
					const FName SourceName(*SourceNameStr);
					const FName TargetName(*TargetNameStr);

					UObject* OuterObject = GetOuter();
					URigVMGraph* RenamedModel =
						Cast<URigVMGraph>(StaticFindObjectFast(URigVMGraph::StaticClass(), OuterObject, SourceName));
					if (RenamedModel == nullptr)
					{
						// Graph is already at TargetName (a prior step in Apply renamed it).
						RenamedModel =
							Cast<URigVMGraph>(StaticFindObjectFast(URigVMGraph::StaticClass(), OuterObject, TargetName));
					}
					if (RenamedModel == nullptr)
					{
						UE_LOG(LogRigVMDeveloper, Warning,
							TEXT("FRigVMClient::PostTransacted: failed to locate root graph by SourceName='%s' or TargetName='%s' under outer '%s' during %s; skipping repair."),
							*SourceNameStr, *TargetNameStr, *GetNameSafe(OuterObject), bUndo ? TEXT("undo") : TEXT("redo"));
						break;
					}

					if (RenamedModel->GetFName() != TargetName)
					{
						RenamedModel->Rename(
							*TargetNameStr,
							nullptr,
							REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch | REN_NonTransactional | REN_DoNotDirty);
					}

					// Repair Models[]: the FPersistentObjectRef-by-name lookup may have left a slot
					// nullptr (see comment above). Use the slot index recorded with the action so
					// we rebind to the EXACT slot (otherwise multi-rename transactions where two
					// slots are nullptr at once can swap their order).
					if (ExternalModelHost == nullptr)
					{
						const bool bAlreadyInArray = Models.Contains(RenamedModel);
						if (!bAlreadyInArray && Models.IsValidIndex(InAction.ModelIndex) && Models[InAction.ModelIndex] == nullptr)
						{
							Models[InAction.ModelIndex] = RenamedModel;
						}
					}

					// Rekey Controllers map. Look up by graph pointer rather than by a precomputed
					// FSoftObjectPath: the entry's key is whatever path the graph had when RenameModel
					// (or a prior PostTransacted pass) last rekeyed it, and that may not match the
					// graph's current path. Same idiom as OnCollapseNodeRenamed.
					{
						UE::TScopeLock Lock(ControllersLock);
						FSoftObjectPath StaleKey;
						TObjectPtr<URigVMController> ControllerForGraph = nullptr;
						for (const TPair<FSoftObjectPath, TObjectPtr<URigVMController>>& Pair : Controllers)
						{
							if (Pair.Value && Pair.Value->GetGraph() == RenamedModel)
							{
								StaleKey = Pair.Key;
								ControllerForGraph = Pair.Value;
								break;
							}
						}
						if (ControllerForGraph)
						{
							const FSoftObjectPath LiveKey(RenamedModel);
							if (StaleKey != LiveKey)
							{
								Controllers.Add(LiveKey, ControllerForGraph);
								Controllers.Remove(StaleKey);
							}
						}
					}

					ClientHost->HandleRigVMGraphRenamed(this, SourcePath, TargetPath);
					break;
				}
			}
		};
		
		while(UndoStack.Num() != UndoRedoIndex)
		{
			// do we need to undo - or redo?
			if(UndoStack.Num() > UndoRedoIndex)
			{
				FRigVMClientAction Action = UndoStack.Pop();
				PerformAction(Action, true);
				RedoStack.Push(Action);
			}
			else
			{
				FRigVMClientAction Action = RedoStack.Pop();
				PerformAction(Action, false);
				UndoStack.Push(Action);
			}
		}
	}
}

void FRigVMClient::OnCollapseNodeRenamed(const URigVMCollapseNode* InCollapseNode)
{
	UE::TScopeLock Lock(ControllersLock);

	TMap<FSoftObjectPath, TObjectPtr<URigVMController>> OldControllers = Controllers;
	for(const TPair<FSoftObjectPath, TObjectPtr<URigVMController>>& Pair : OldControllers)
	{
		if(URigVMController* Controller = Pair.Value)
		{
			if(const URigVMGraph* Graph = Controller->GetGraph())
			{
				if(Graph->IsInOuter(InCollapseNode))
				{
					const FSoftObjectPath& OldObjectPath = Pair.Key;
					const FSoftObjectPath NewObjectPath(Graph);

					if(OldObjectPath != NewObjectPath)
					{
						Controllers.Add(NewObjectPath, Controller);
						Controllers.Remove(OldObjectPath);
					}
				}
			}
		}
	}
}

void FRigVMClient::OnCollapseNodeRemoved(const URigVMCollapseNode* InCollapseNode)
{
	UE::TScopeLock Lock(ControllersLock);
	
	TMap<FSoftObjectPath, TObjectPtr<URigVMController>> OldControllers = Controllers;
	for(const TPair<FSoftObjectPath, TObjectPtr<URigVMController>>& Pair : OldControllers)
	{
		if(URigVMController* Controller = Pair.Value)
		{
			if(const URigVMGraph* Graph = Controller->GetGraph())
			{
				if(Graph->IsInOuter(InCollapseNode))
				{
					RemoveController(Graph);
				}
			}
		}
	}
}

URigVMNode* FRigVMClient::FindNode(const FString& InNodePathOrName) const
{
	for(const URigVMGraph* Model : GetModels())
	{
		if(URigVMNode* Node = Model->FindNode(InNodePathOrName))
		{
			return Node;
		}
	}
	if(FunctionLibrary)
	{
		return FunctionLibrary->FindNode(InNodePathOrName);
	}
	return nullptr;
}

URigVMPin* FRigVMClient::FindPin(const FString& InPinPath) const
{
	for(const URigVMGraph* Model : GetModels())
	{
		if(URigVMPin* Pin = Model->FindPin(InPinPath))
		{
			return Pin;
		}
	}
	if(FunctionLibrary)
	{
		return FunctionLibrary->FindPin(InPinPath);
	}
	return nullptr;
}

UObject* FRigVMClient::GetOuter() const
{
	UE::TScopeLock Lock(OuterClientHostLock);
	UObject* Outer = OuterClientHost.Get();
	check(Outer);
	return Outer;
}

FProperty* FRigVMClient::GetOuterClientProperty() const
{
	return GetOuter()->GetClass()->FindPropertyByName(OuterClientPropertyName);
}

void FRigVMClient::NotifyOuterOfPropertyChange(EPropertyChangeType::Type ChangeType) const
{
	if(bSuspendNotifications)
	{
		return;
	}
	FProperty* Property = GetOuterClientProperty();
	FPropertyChangedEvent PropertyChangedEvent(Property, ChangeType);
	GetOuter()->PostEditChangeProperty(PropertyChangedEvent);
}

URigVMController* FRigVMClient::CreateController(const URigVMGraph* InModel)
{
	const FSoftObjectPath Key(InModel);
	URigVMController* Controller = nullptr;

	{
		UE::TScopeLock Lock(ControllersLock);

		// check one more time if this controller already exists. given we are now deploying locks
		// this is important to avoid double registration of controllers.
		if(const TObjectPtr<URigVMController>* ExistingController = Controllers.Find(Key))
		{
			if  (!URigVMHost::IsGarbageOrDestroyed(ExistingController->Get()))
			{
				URigVMController* Existing = ExistingController->Get();
				if(Existing->GetGraph() == InModel)
				{
					return Existing;
				}
				// Stale slot — see PurgeStaleController. FindOrAdd below re-inserts under the same key.
				PurgeStaleController(Key, Existing);
			}
		}

		const FString ModelName = InModel ? InModel->GetName() : TEXT("__NullGraph");
		const FName SafeControllerName = GetUniqueName(*FString::Printf(TEXT("%s_Controller"), *ModelName));
		Controller = NewObject<URigVMController>(GetOuter(), ControllerClass, SafeControllerName);
		Controllers.FindOrAdd(Key, nullptr) = Controller;
		if(InModel && InModel->GetSchemaClass() != nullptr)
		{
			Controller->SetSchemaClass(InModel->GetSchemaClass());
		}
		else
		{
			check(DefaultSchemaClass);
			Controller->SetSchemaClass(DefaultSchemaClass);
		}

		Controller->SetActionStack(GetOrCreateActionStack());
		if(InModel)
		{
			Controller->SetGraph((URigVMGraph*)InModel);
		}
	}

	check(Controller);
	
	Controller->OnModified().AddLambda([this](ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
	{
		HandleGraphModifiedEvent(InNotifType, InGraph, InSubject);
	});

	if (GetOuter()->Implements<URigVMClientHost>())
	{
		IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter());
		ClientHost->HandleConfigureRigVMController(this, Controller);
	}

	if(InModel)
	{
		Controller->RemoveStaleNodes();
	}
	return Controller;
}

URigVMActionStack* FRigVMClient::GetOrCreateActionStack()
{
	if(UObject* Outer = GetOuter())
	{
		if (ActionStack && ActionStack->GetOuter() != Outer)
		{
			ResetActionStack();
		}
		
		if(ActionStack == nullptr)
		{
			ActionStack = NewObject<URigVMActionStack>(Outer, TEXT("ActionStack"));
		}
	}
	return ActionStack;
}

void FRigVMClient::ResetActionStack()
{
	DestroyObject(ActionStack);
	ActionStack = nullptr;
}

FName FRigVMClient::GetUniqueName(const FName& InDesiredName) const
{
	return GetUniqueName(GetOuter(), InDesiredName);
}

FName FRigVMClient::GetUniqueName(UObject* InOuter, const FName& InDesiredName)
{
	return URigVMSchema::GetUniqueName(*InDesiredName.ToString(), [InOuter](const FName& InName) -> bool
	{
		return FindObjectWithOuter(InOuter, nullptr, InName) == nullptr; 
	}, false, true);
}

void FRigVMClient::DestroyObject(UObject* InObject)
{
	if(InObject)
	{
		static int32 ObjectIndexToBeDestroyed = 0;
		static constexpr TCHAR ObjectNameFormat[] = TEXT("RigVMClient_ObjectToBeDestroyed_%d");
		const FString NewObjectName = FString::Printf(ObjectNameFormat, ObjectIndexToBeDestroyed++);
		InObject->Rename(*NewObjectName, GetTransientPackage(), REN_DontCreateRedirectors);
		if(!InObject->IsRooted())
		{
			InObject->MarkAsGarbage();
		}
	}
}

uint32 FRigVMClient::GetStructureHash() const
{
	uint32 Hash = 0;
	const TArray<URigVMGraph*> ModelsAndFunctionLibrary = GetAllModels(true, true);
	for(const URigVMGraph* Model : ModelsAndFunctionLibrary)
	{
		Hash = HashCombine(Hash, Model->GetStructureHash());
	}
	return Hash;
}

uint32 FRigVMClient::GetSerializedStructureHash() const
{
	uint32 Hash = 0;
	const TArray<URigVMGraph*> ModelsAndFunctionLibrary = GetAllModels(true, true);
	for(const URigVMGraph* Model : ModelsAndFunctionLibrary)
	{
		Hash = HashCombine(Hash, Model->GetSerializedStructureHash());
	}
	return Hash;
}

FRigVMClientPatchResult FRigVMClient::PatchModelsOnLoad()
{
	FRigVMClientPatchResult Result;
	
	TArray<URigVMGraph*> AllModels = GetAllModelsLeavesFirst(true);
	TGuardValue<bool> ClientIgnoreModificationsGuard(bIgnoreModelNotifications, true);
	for(URigVMGraph* Model : AllModels)
	{
		Model->PostLoad();
		
		// Temporary hack until we manage to allow multiple schemas inside a FRigVMClient
		if (GetDefaultSchemaClass()->GetDefaultObject<URigVMSchema>()->ShouldPatchGraphSchema())
		{
			Model->SetSchemaClass(GetDefaultSchemaClass());
		}
		
		URigVMController* Controller = GetOrCreateController(Model);
		TGuardValue<bool> GuardSuspendTemplateComputation(Controller->bSuspendTemplateComputation, true);
		TGuardValue<bool> GuardIsTransacting(Controller->bIsTransacting, true);
		{
			FRigVMDefaultValueTypeGuard DefaultValueTypeGuard(Controller, ERigVMPinDefaultValueType::KeepValueType);
			Result.Merge(Controller->PatchPinGuidInCollapseNodes());
			Result.Merge(Controller->PatchLocalVariableTypes());
			Result.Merge(Controller->PatchRerouteNodesOnLoad());
			Result.Merge(Controller->PatchUnitNodesOnLoad());
			Result.Merge(Controller->PatchDispatchNodesOnLoad());
			Result.Merge(Controller->PatchBranchNodesOnLoad());
			Result.Merge(Controller->PatchIfSelectNodesOnLoad());
			Result.Merge(Controller->PatchArrayNodesOnLoad());
			Result.Merge(Controller->PatchReduceArrayFloatDoubleConvertsionsOnLoad());
			Result.Merge(Controller->PatchInvalidLinksOnWildcards());
			Result.Merge(Controller->PatchExecutePins());
			Result.Merge(Controller->PatchLazyPins());
			Result.Merge(Controller->PatchConstantPins());
			Result.Merge(Controller->PatchUserDefinedStructPinNames());
			Result.Merge(Controller->PatchExternalVariablesToArguments());
		
			if (URigVMCollapseNode* CollapseNode = Model->GetTypedOuter<URigVMCollapseNode>())
			{
				Result.Merge(Controller->PatchFunctionsWithInvalidReturnPaths());	
			}
		}
		Result.Merge(Controller->PatchPinDefaultValues());
	}

	return Result;
}

#if WITH_EDITOR
void FRigVMClient::PatchFunctionReferencesOnLoad()
{
	// If the asset was copied from one project to another, the function referenced might have a different
	// path, even if the function is internal to the contorl rig. In that case, let's try to find the function
	// in the local function library.

	for (URigVMGraph* Model : *this)
	{
		TArray<URigVMNode*> Nodes = Model->GetNodes();
		for (URigVMLibraryNode* Library : GetFunctionLibrary()->GetFunctions())
		{
			Nodes.Append(Library->GetContainedNodes());
		}

		for (int32 i = 0; i < Nodes.Num(); ++i)
		{
			URigVMNode* Node = Nodes[i];
			if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
			{
				if (!FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.IsValid())
				{
					(void)FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.LoadSynchronous();
				}
				if (!FunctionReferenceNode->ReferencedNodePtr_DEPRECATED)
				{
					if (FunctionLibrary)
					{
						FString FunctionPath = FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.ToSoftObjectPath().GetSubPathString();

						FString Left, Right;
						if (FunctionPath.Split(TEXT("."), &Left, &Right))
						{
							FString LibraryNodePath = FunctionLibrary->GetNodePath();
							if (Left == FunctionLibrary->GetName())
							{
								if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionLibrary->FindNode(Right)))
								{
									FunctionReferenceNode->ReferencedNodePtr_DEPRECATED = LibraryNode;
								}
							}
						}
					}
				}

				if (FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.IsValid())
				{
					FunctionReferenceNode->ReferencedFunctionHeader = FunctionReferenceNode->ReferencedNodePtr_DEPRECATED->GetFunctionHeader();
				}
				else if (!FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.IsNull())
				{
					// At least lets make sure we store the path in the header
					FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer.SetLibraryNodePath(FunctionReferenceNode->ReferencedNodePtr_DEPRECATED.ToSoftObjectPath().ToString());
				}

				if (FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer.LibraryNode_DEPRECATED.IsValid())
				{
					FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer.SetLibraryNodePath(FunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer.LibraryNode_DEPRECATED.ToString());
				}

				for (URigVMPin* Pin : FunctionReferenceNode->GetPins())
				{
					if (Pin->IsArray())
					{
						Pin->bIsDynamicArray = true;
					}
				}
			}

			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
			{
				Nodes.Append(CollapseNode->GetContainedNodes());
			}
		}
	}
}

#endif // WITH_EDITOR

void FRigVMClient::PatchFunctionsOnLoad(IRigVMGraphFunctionHost* FunctionHost, TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders)
{
	if (!FunctionHost)
	{
		return;
	}
	
	if (FRigVMGraphFunctionStore* Store = FunctionHost->GetRigVMGraphFunctionStore())
	{
		// Lets rebuild the FunctionStore from the model
		if (FunctionLibrary)
		{
			Store->PublicFunctions.Reset();
			Store->PrivateFunctions.Reset();

			for (URigVMLibraryNode* LibraryNode : FunctionLibrary->GetFunctions())
			{
				// Fix array pins to be marked as dynamic array
				for (URigVMPin* Pin : LibraryNode->GetPins())
				{
					if (Pin->IsArray())
					{
						Pin->bIsDynamicArray = true;
					}
				}
				if (URigVMFunctionEntryNode* EntryNode = LibraryNode->GetEntryNode())
				{
					for (URigVMPin* Pin : EntryNode->GetPins())
					{
						if (Pin->IsArray())
						{
							Pin->bIsDynamicArray = true;
						}
					}
				}
				if (URigVMFunctionReturnNode* ReturnNode = LibraryNode->GetReturnNode())
				{
					for (URigVMPin* Pin : ReturnNode->GetPins())
					{
						if (Pin->IsArray())
						{
							Pin->bIsDynamicArray = true;
						}
					}
				}

				bool bIsPublic = FunctionLibrary->IsFunctionPublic(LibraryNode->GetFName());
				if (!bIsPublic)
				{
					bIsPublic = BackwardsCompatiblePublicFunctions.Contains(LibraryNode->GetFName());
					if (bIsPublic)
					{
						FunctionLibrary->PublicFunctionNames.Add(LibraryNode->GetFName());
					}
				}

				FRigVMGraphFunctionHeader Header = LibraryNode->GetFunctionHeader(FunctionHost);
				if (FRigVMGraphFunctionHeader* OldHeader = OldHeaders.Find(LibraryNode))
				{
					Header.ExternalVariables = OldHeader->ExternalVariables;
					Header.Dependencies = OldHeader->Dependencies;
					Header.Layout = OldHeader->Layout;
				}

				const FRigVMVariant* Variant = FunctionLibrary->GetFunctionVariant(LibraryNode->GetFName());
				if (!Variant)
				{
					Header.Variant.Guid = FRigVMVariant::GenerateGUID(Header.LibraryPointer.GetLibraryNodePath());
					FunctionLibrary->FunctionToVariant.FindOrAdd(Header.Name) = Header.Variant;
				}
				else
				{
					Header.Variant = *Variant;
				}

				FRigVMGraphFunctionData* FunctionData = Store->AddFunction(Header, bIsPublic);
				if (bIsPublic)
				{
					UpdateGraphFunctionSerializedGraph(LibraryNode);
				}
				else
				{
					FunctionData->SerializedCollapsedNode_DEPRECATED.Empty();
					FunctionData->CollapseNodeArchive.Empty();
				}
			}

			// Update dependencies and external variables if needed
			for (URigVMLibraryNode* LibraryNode : FunctionLibrary->GetFunctions())
			{
				UpdateExternalVariablesForFunction(LibraryNode);
				UpdateDependenciesForFunction(LibraryNode);
			}
		}
	}
}

FRigVMClientPatchResult FRigVMClient::PatchPinDefaultValues()
{
	FRigVMClientPatchResult Result;
	
	TArray<URigVMGraph*> AllModels = GetAllModelsLeavesFirst(true);
	TGuardValue<bool> ClientIgnoreModificationsGuard(bIgnoreModelNotifications, true);
	for(URigVMGraph* Model : AllModels)
	{
		URigVMController* Controller = GetOrCreateController(Model);
		TGuardValue<bool> GuardIsTransacting(Controller->bIsTransacting, true);
		Result.Merge(Controller->PatchPinDefaultValues());
	}

	return Result;
}

void FRigVMClient::ProcessDetachedLinks()
{
	TArray<URigVMGraph*> AllModels = GetAllModels(true, true);
	for(URigVMGraph* Model : AllModels)
	{
		URigVMController* Controller = GetOrCreateController(Model);
		Controller->ProcessDetachedLinks();
	}
}

void FRigVMClient::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	if (!ObjectSaveContext.IsCooking())
	{
		if (IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter()))
		{
			if (TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost())
			{
				if (FRigVMGraphFunctionStore* Store = FunctionHost->GetRigVMGraphFunctionStore())
				{
					for (URigVMNode* Node : FunctionLibrary->GetNodes())
					{
						if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Node))
						{
							FRigVMGraphFunctionIdentifier Identifier = LibraryNode->GetFunctionIdentifier();
							if (Store->IsFunctionPublic(Identifier))
							{
								UpdateGraphFunctionSerializedGraph(LibraryNode);
							}
						}
					}
				}
			}
		}
	}
}

void FRigVMClient::HandleGraphModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	if (bIgnoreModelNotifications)
	{
		return;
	}
	
#if WITH_EDITOR

	IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(GetOuter());
	TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = nullptr;
	FRigVMGraphFunctionStore* FunctionStore = nullptr;
	if (ClientHost)
	{
		FunctionHost = ClientHost->GetRigVMGraphFunctionHost();
		if (FunctionHost)
		{
			FunctionStore = FunctionHost->GetRigVMGraphFunctionStore();
		}
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::NodeAdded: // A node has been added to the graph (Subject == URigVMNode)
		{
			// A node was added directly into the function library
			if(InGraph->IsA<URigVMFunctionLibrary>())
			{
				if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
				{
					if (GetOuter()->Implements<URigVMClientHost>())
					{
						if (FunctionStore && FunctionHost)
						{
							FunctionStore->AddFunction(CollapseNode->GetFunctionHeader(FunctionHost.GetInterface()), false);
						}
					}
				}
			}
			// A node was added into the contained graph of a function
			else if(URigVMLibraryNode* LibraryNode = Cast<URigVMNode>(InSubject)->FindFunctionForNode())
			{
				DirtyGraphFunctionCompilationData(LibraryNode);
				if (URigVMFunctionReferenceNode* FunctionReference = Cast<URigVMFunctionReferenceNode>(InSubject))
				{
					UpdateDependenciesForFunction(LibraryNode);
					UpdateExternalVariablesForFunction(LibraryNode);
				}
			}
			break;	
		}
		case ERigVMGraphNotifType::NodeRemoved: // A node has been removed from the graph (Subject == URigVMNode)
		{		
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
			{
				if(CollapseNode->IsGraphFunctionDefinition())
				{
					if (GetOuter()->Implements<URigVMClientHost>())
					{
						if (FunctionStore && FunctionHost)
						{
							FunctionStore->RemoveFunction(FRigVMGraphFunctionIdentifier (FunctionHost.GetObject(), CollapseNode->GetPathName()));
						}
					}
				}
			}
			// A node was removed from the contained graph of a function
			else if(URigVMLibraryNode* LibraryNode = Cast<URigVMNode>(InSubject)->FindFunctionForNode())
			{
				DirtyGraphFunctionCompilationData(LibraryNode);
				if (URigVMFunctionReferenceNode* FunctionReference = Cast<URigVMFunctionReferenceNode>(InSubject))
				{
					UpdateDependenciesForFunction(LibraryNode);
					UpdateExternalVariablesForFunction(LibraryNode);
				}
			}
			break;	
		}
		case ERigVMGraphNotifType::LocalVariableAdded:
		case ERigVMGraphNotifType::LocalVariableRemoved:
		case ERigVMGraphNotifType::LocalVariableDefaultValueChanged:
		case ERigVMGraphNotifType::LocalVariableRenamed:
		case ERigVMGraphNotifType::LocalVariableTypeChanged:
		{
			if(URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InSubject))
			{
				DirtyGraphFunctionCompilationData(LibraryNode);
			}
			break;
		}
		case ERigVMGraphNotifType::VariableAdded: // A variable has been added (Subject == URigVMVariableNode)
		case ERigVMGraphNotifType::VariableRemoved: // A variable has been removed (Subject == URigVMVariableNode)
		case ERigVMGraphNotifType::VariableRenamed: // A variable has been renamed (Subject == URigVMVariableNode)
		{
			if(URigVMLibraryNode* LibraryNode = Cast<URigVMNode>(InSubject)->FindFunctionForNode())
			{
				DirtyGraphFunctionCompilationData(LibraryNode);
				UpdateExternalVariablesForFunction(LibraryNode);
			}
			break;
		}
		case ERigVMGraphNotifType::FunctionRenamed: // A node has been renamed in the graph (Subject == URigVMNode)
		{
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
			{
				if (CollapseNode->IsGraphFunctionDefinition())
				{
					URigVMBuildData* BuildData = URigVMBuildData::Get();
					if (IRigVMGraphFunctionHost* Host = Cast<IRigVMGraphFunctionHost>(CollapseNode->GetFunctionIdentifier().HostObject.ResolveObject()))
					{
						if (FRigVMGraphFunctionData* Data = Host->GetRigVMGraphFunctionStore()->FindFunctionByName(CollapseNode->GetPreviousFName()))
						{
							const FRigVMGraphFunctionIdentifier PreviousFunctionId = Data->Header.LibraryPointer;
							const FRigVMVariant Variant = Data->Header.Variant;
							Data->Header = CollapseNode->GetFunctionHeader();
							Data->Header.Variant = Variant;

							if (const FRigVMFunctionReferenceArray* FunctionReferencesPtr = BuildData->GraphFunctionReferences.Find(PreviousFunctionId))
							{
								const FRigVMFunctionReferenceArray& FunctionReferences = *FunctionReferencesPtr;
								BuildData->Modify();
								FRigVMFunctionReferenceArray& NewFunctionReferences = BuildData->GraphFunctionReferences.Add(Data->Header.LibraryPointer, FunctionReferences);
								BuildData->GraphFunctionReferences.Remove(PreviousFunctionId);
								BuildData->MarkPackageDirty();

								for (int32 i=0; i<NewFunctionReferences.Num(); ++i)
								{
									if (!NewFunctionReferences[i].IsValid())
									{
										NewFunctionReferences[i].LoadSynchronous();
									}
									if (NewFunctionReferences[i].IsValid())
									{
										NewFunctionReferences[i]->ReferencedFunctionHeader = Data->Header;
										NewFunctionReferences[i]->MarkPackageDirty();
									}
								}
							}

							UpdateFunctionReferences(Data->Header, true, false);
							UpdateGraphFunctionData(CollapseNode);
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::NodeColorChanged: // A node's color has changed (Subject == URigVMNode)
		case ERigVMGraphNotifType::NodeCategoryChanged: // A node's category has changed (Subject == URigVMNode)
		case ERigVMGraphNotifType::NodeKeywordsChanged: // A node's keywords have changed (Subject == URigVMNode)
		case ERigVMGraphNotifType::NodeDescriptionChanged: // A node's description has changed (Subject == URigVMNode)
		case ERigVMGraphNotifType::NodeTitleChanged: // A node's title has changed (Subject == URigVMNode)
		case ERigVMGraphNotifType::VariantTagsChanged: // The tags in the header of this function variant have changed (Subject == URigVMLibraryNode)
		{
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
			{
				if(CollapseNode->IsGraphFunctionDefinition())
				{
					UpdateGraphFunctionData(CollapseNode);
				}
			}
			break;
		}
		
		case ERigVMGraphNotifType::PinAdded: // A pin has been added to a given node (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinRemoved: // A pin has been removed from a given node (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinRenamed: // A pin has been renamed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinArraySizeChanged: // An array pin's size has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinDefaultValueChanged: // A pin's default value has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinDirectionChanged: // A pin's direction has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinTypeChanged: // A pin's data type has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinIndexChanged: // A pin's index has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinWatchedChanged: // A pin's watch state has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinDisplayNameChanged: // A pin's display name / UI label has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinCategoryChanged: // A pin's category has changed (Subject == URigVMPin)
		case ERigVMGraphNotifType::PinCategoryExpansionChanged: // A pin category has been collapsed / expanded on the function node (Subject == URigVMNode)
		case ERigVMGraphNotifType::FunctionVariantGuidChanged: // A function has changed its guid (Subject == URigVMLibraryNode)
		{
			URigVMNode* Node = Cast<URigVMNode>(InSubject);
			if (const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
			{
				Node = Cast<URigVMNode>(Pin->GetNode());
			}
			if(Node)
			{
				if (URigVMLibraryNode* LibraryNode = Node->FindFunctionForNode())
				{
					DirtyGraphFunctionCompilationData(LibraryNode);
				}
				if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
				{
					if(CollapseNode->IsGraphFunctionDefinition())
					{
						UpdateGraphFunctionData(CollapseNode);
						UpdateExternalVariablesForFunction(CollapseNode);
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinCategoriesChanged: // A node's category list has changed (Subject == URigVMNode)
		{
			if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
			{
				if(CollapseNode->IsGraphFunctionDefinition())
				{
					UpdateGraphFunctionData(CollapseNode);
				}
			}
			break;
		}
		case ERigVMGraphNotifType::LinkAdded: // A link has been added (Subject == URigVMLink)
		case ERigVMGraphNotifType::LinkRemoved: // A link has been removed (Subject == URigVMLink)
		{
			if (URigVMLink* Link = Cast<URigVMLink>(InSubject))
			{
				if (URigVMNode* OuterNode = Cast<URigVMNode>(Link->GetGraph()->GetOuter()))
				{
					if (URigVMLibraryNode* LibraryNode = OuterNode->FindFunctionForNode())
					{
						DirtyGraphFunctionCompilationData(LibraryNode);
					}
				}
			}

			break;
		}

		case ERigVMGraphNotifType::FunctionAccessChanged: // A function was made public/private
		{
			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InSubject))
			{
				if (URigVMFunctionLibrary* Library = Cast<URigVMFunctionLibrary>(InGraph))
				{
					bool bIsPublic = Library->IsFunctionPublic(LibraryNode->GetFName());
					if (FRigVMGraphFunctionStore* Store = FindFunctionStore(LibraryNode))
					{
						const FRigVMGraphFunctionIdentifier Id = LibraryNode->GetFunctionIdentifier();
						Store->MarkFunctionAsPublic(Id, bIsPublic);
						if(bIsPublic)
						{
							URigVMBuildData::Get()->RegisterPublicFunction(Id, Store->FindFunction(Id)->Header);
							UpdateGraphFunctionSerializedGraph(LibraryNode);
						}
						else
						{
							URigVMBuildData::Get()->UnregisterPublicFunction(Id);
						}
					}
				}
			}
			break;
		}
		
		default:
		{
			break;
		}
	}

	PostGraphModifiedDelegate.Broadcast(InNotifType, InGraph, InSubject);
#endif
}

FRigVMGraphFunctionStore* FRigVMClient::FindFunctionStore(const URigVMLibraryNode* InLibraryNode)
{
	if (IRigVMClientHost* ClientHost = InLibraryNode->GetImplementingOuter<IRigVMClientHost>())
	{
		if (TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost())
		{
			return FunctionHost->GetRigVMGraphFunctionStore();
		}
	}
	return nullptr;
}

bool FRigVMClient::UpdateFunctionReferences(const FRigVMGraphFunctionHeader& Header, bool bUpdateDependencies, bool bUpdateExternalVariables)
{
	URigVMBuildData* BuildData = URigVMBuildData::Get();
	if (const FRigVMFunctionReferenceArray* FunctionReferenceArray = BuildData->FindFunctionReferences(Header.LibraryPointer))
	{
		for (int32 i=0; i<FunctionReferenceArray->Num(); ++i)
		{
			const TSoftObjectPtr<URigVMFunctionReferenceNode>& Reference = FunctionReferenceArray->FunctionReferences[i];

			// Only update references that are loaded
			// Other references will be updated when they are loaded
			if (Reference.IsValid())
			{
				URigVMFunctionReferenceNode* Node = Reference.Get();

				Node->Modify();
				Node->ReferencedFunctionHeader = Header;
				Node->InvalidateCache();

				if (bUpdateDependencies || bUpdateExternalVariables)
				{
					if (URigVMLibraryNode* LibraryNode = Node->FindFunctionForNode())
					{
						IRigVMClientHost* OtherClientHost = LibraryNode->GetImplementingOuter<IRigVMClientHost>();
						if (bUpdateDependencies)
						{
							OtherClientHost->GetRigVMClient()->UpdateDependenciesForFunction(LibraryNode);
						}
						if (bUpdateExternalVariables)
						{								
							OtherClientHost->GetRigVMClient()->UpdateExternalVariablesForFunction(LibraryNode);
						}
					}
				}
				Node->MarkPackageDirty();
			}
		}
	}
	return true;
}

bool FRigVMClient::UpdateGraphFunctionData(const URigVMLibraryNode* InLibraryNode)
{
	if (IRigVMClientHost* ClientHost = InLibraryNode->GetImplementingOuter<IRigVMClientHost>())
	{
		if (TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost())
		{
			if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
			{
				if (FRigVMGraphFunctionData* Data = Store->UpdateFunctionInterface(InLibraryNode->GetFunctionHeader(FunctionHost.GetInterface())))
				{
					if (Store->IsFunctionPublic(Data->Header.LibraryPointer))
					{
						URigVMBuildData::Get()->UpdatePublicFunctionHeader(Data->Header.LibraryPointer, Data->Header);
					}
					UpdateFunctionReferences(Data->Header, false, false);
					return true;
				}
			}
		}
	}	
	
	return false;	
}

bool FRigVMClient::UpdateExternalVariablesForFunction(const URigVMLibraryNode* InLibraryNode)
{
	if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
	{
		FRigVMGraphFunctionIdentifier Identifier = InLibraryNode->GetFunctionIdentifier();
		if (Store->UpdateExternalVariables(Identifier, InLibraryNode->GetExternalVariables()))
		{
			bool bPublic;
			const FRigVMGraphFunctionData* Data = Store->FindFunction(Identifier, &bPublic);
			
			if (bPublic)
			{
				URigVMBuildData::Get()->UpdatePublicFunctionHeader(Identifier, Data->Header);
			}
			
			UpdateFunctionReferences(Data->Header, false, true);
			return true;
		}
	}
	
	return false;
}

bool FRigVMClient::UpdateDependenciesForFunction(const URigVMLibraryNode* InLibraryNode)
{
	if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
	{
		TMap<FRigVMGraphFunctionIdentifier, uint32> Dependencies = InLibraryNode->GetDependencies();
		FRigVMGraphFunctionIdentifier Identifier = InLibraryNode->GetFunctionIdentifier();
		if (Store->UpdateDependencies(Identifier, Dependencies))
		{
			bool bPublic;
			const FRigVMGraphFunctionData* Data = Store->FindFunction(Identifier, &bPublic);
			
			if (bPublic)
			{
				URigVMBuildData::Get()->UpdatePublicFunctionHeader(Identifier, Data->Header);
			}
			
			UpdateFunctionReferences(Data->Header, true, false);
			return true;
		}
	}
	
	return false;
}

bool FRigVMClient::DirtyGraphFunctionCompilationData(URigVMLibraryNode* InLibraryNode)
{
	if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
	{
		FRigVMGraphFunctionIdentifier Identifier = InLibraryNode->GetFunctionIdentifier();
		if (const FRigVMGraphFunctionData* Data = Store->FindFunction(Identifier))
		{
			Store->RemoveFunctionCompilationData(Identifier);

			// References to this function will check if the compilation hash matches, and will recompile if they
			// see a different compilation hash. No need to dirty their compilation data.
			
			return true;
		}
	}

	return false;
}

bool FRigVMClient::UpdateGraphFunctionSerializedGraph(URigVMLibraryNode* InLibraryNode)
{
	if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
	{
		const FRigVMGraphFunctionIdentifier Identifier = InLibraryNode->GetFunctionIdentifier();
		if (FRigVMGraphFunctionData* Data = Store->FindFunction(Identifier))
		{
			Data->SerializedCollapsedNode_DEPRECATED.Empty();

			URigVMController* Controller = GetOrCreateController(InLibraryNode->GetGraph());
			check(Controller);
			Data->CollapseNodeArchive.Reset();
			(void)Controller->ExportFunctionToArchive(Identifier.GetFunctionFName(), Data->CollapseNodeArchive);

			return true;
		}
	}
	return false;
}

bool FRigVMClient::IsFunctionPublic(URigVMLibraryNode* InLibraryNode)
{
	if (FRigVMGraphFunctionStore* Store = FindFunctionStore(InLibraryNode))
	{
		return Store->IsFunctionPublic(InLibraryNode->GetFunctionIdentifier());
	}
	return false;
}

#if WITH_EDITOR

bool FRigVMClient::UpgradeAllOccurencesOfNodes(const TArray<URigVMNode*>& InNodesToUpgrade, UObject* InOuter, bool bSetupUndoRedo)
{
	TArray<UScriptStruct*> ScriptStructsToFind;
	for(const URigVMNode* NodeToUpgrade : InNodesToUpgrade)
	{
		if(!NodeToUpgrade || !IsValid(NodeToUpgrade))
		{
			return false;
		}
		
		if(!NodeToUpgrade->CanBeUpgraded())
		{
			continue;
		}
		
		if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(NodeToUpgrade))
		{
			ScriptStructsToFind.AddUnique(UnitNode->GetScriptStruct());
		}
		else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(NodeToUpgrade))
		{
			if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
			{
				ScriptStructsToFind.AddUnique(Factory->GetScriptStruct());
			}
		}
	}

	if(ScriptStructsToFind.IsEmpty())
	{
		return false;
	}

	TArray<URigVMGraph*> AllGraphs = GetAllModelsLeavesFirst(true);
	TArray<TArray<URigVMNode*>> NodesPerGraph;
	NodesPerGraph.AddDefaulted(AllGraphs.Num());

	for(int32 GraphIndex = 0; GraphIndex < AllGraphs.Num(); GraphIndex++)
	{
		const URigVMGraph* Graph = AllGraphs[GraphIndex];
		
		for(URigVMNode* Node : Graph->GetNodes())
		{
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(ScriptStructsToFind.Contains(UnitNode->GetScriptStruct()))
				{
					NodesPerGraph[GraphIndex].Add(Node);
				}
			}
			else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					if(ScriptStructsToFind.Contains(Factory->GetScriptStruct()))
					{
						NodesPerGraph[GraphIndex].Add(Node);
					}
				}
			}
		}
	}

	TSharedPtr<FScopedTransaction> Transaction;
	if(bSetupUndoRedo)
	{
		Transaction = MakeShared<FScopedTransaction>(NSLOCTEXT("RigVMClient", "UpgradeAllOccurencesOfNodes", "Upgrade Nodes"));
		InOuter->Modify();
	}
	
	FScopedSlowTask UpgradeNodesSlowTask((float)NodesPerGraph.Num(), NSLOCTEXT("RigVMClient", "UpgradeAllOccurencesOfNodes", "Upgrade Nodes"));
	UpgradeNodesSlowTask.MakeDialog(true);

	bool bResult = true;
	for(int32 GraphIndex = 0; GraphIndex < AllGraphs.Num(); GraphIndex++)
	{
		UpgradeNodesSlowTask.EnterProgressFrame(1.0f, FText::FromString(FString::Printf(TEXT("Processing nodes of graph '%s'"), *AllGraphs[GraphIndex]->GetGraphName())));
		UpgradeNodesSlowTask.ForceRefresh();
		
		if(NodesPerGraph[GraphIndex].IsEmpty())
		{
			continue;
		}
		URigVMController* Controller = GetOrCreateController((AllGraphs[GraphIndex]));
		const TArray<URigVMNode*> UpgradedNodes = Controller->UpgradeNodes(NodesPerGraph[GraphIndex], true, bSetupUndoRedo);
		if(UpgradedNodes.Num() != NodesPerGraph[GraphIndex].Num())
		{
			bResult = false;
		}
	}

	return bResult;
}

#endif