// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMByteCode.h"
#include "RigVMFunctions/RigVMFunction_ControlFlow.h"
#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/RigVMDispatch_MakeStruct.h"
#include "RigVMFunctions/Execution/RigVMFunction_Sequence.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMDeveloperModule.h"
#include "RigVMHost.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Package.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringOutputDevice.h"
#include "Algo/Count.h"
#include "Algo/Reverse.h"
#include "Algo/Transform.h"
#include "RigVMPythonUtils.h"
#include "RigVMSettings.h"
#include "RigVMTypeUtils.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMStringUtils.h"
#include "StructUtils/UserDefinedStruct.h"
#include "RigVMFunctions/RigVMDispatch_If.h"
#include "RigVMFunctions/RigVMDispatch_Select.h"
#include "RigVMFunctions/RigVMDispatch_Switch.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/Nodes/RigVMBranchNode.h"
#include "RigVMModel/Nodes/RigVMArrayNode.h"
#include "RigVMModel/RigVMGraphSection.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "ScopedTransaction.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMController)

#if WITH_EDITOR
#include "Exporters/Exporter.h"
#include "UnrealExporter.h"
#include "Factories.h"
#include "UObject/CoreRedirects.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Styling/AppStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif

TMap<URigVMController::FRigVMStructPinRedirectorKey, FString> URigVMController::PinPathCoreRedirectors;

FRigVMControllerCompileBracketScope::FRigVMControllerCompileBracketScope(URigVMController* InController)
: Graph(nullptr), bSuspendNotifications(InController->bSuspendNotifications)
{
	check(InController);
	Graph = InController->GetGraph();
	check(Graph);
	
	if (bSuspendNotifications)
	{
		return;
	}
	Graph->Notify(ERigVMGraphNotifType::InteractionBracketOpened, nullptr);
}

FRigVMControllerCompileBracketScope::~FRigVMControllerCompileBracketScope()
{
	check(Graph);
	if (bSuspendNotifications)
	{
		return;
	}
	Graph->Notify(ERigVMGraphNotifType::InteractionBracketClosed, nullptr);
}

void FRigVMClientPatchResult::Merge(const FRigVMClientPatchResult& InOther)
{
	bSucceeded = Succeeded() && InOther.Succeeded();
	bChangedContent = ChangedContent() || InOther.ChangedContent();
	bRequiresToMarkPackageDirty = RequiresToMarkPackageDirty() || InOther.RequiresToMarkPackageDirty();
	ErrorMessages.Append(InOther.GetErrorMessages());
	RemovedNodes.Append(InOther.GetRemovedNodes());
	AddedNodes.Append(InOther.GetAddedNodes());
}

URigVMController::URigVMController()
	: bValidatePinDefaults(true)
	, bSuspendNotifications(false)
	, bSuspendMarkingPackageDirty(false)
	, bSuspendRefreshingFunctionReferences(false)
	, bReportWarningsAndErrors(true)
	, bIgnoreRerouteCompactnessChanges(false)
	, UserLinkDirection(ERigVMPinDirection::Invalid)
	, bEnableTypeCasting(true)
	, bAllowPrivateFunctions(false)
	, bSuspendASTLinkChecks(false)
	, bIsTransacting(false)
	, bIsRunningUnitTest(false)
	, bIsFullyResolvingTemplateNode(false)
	, bSuspendTemplateComputation(false)
#if WITH_EDITOR
	, bRegisterTemplateNodeUsage(true)
#endif
	, bEnableSchemaRemoveNodeCheck(true)
	, UpdateGraphSectionsBracket(0)
{
	SetActionStack(URigVMActionStack::GetDisabledActionStack());
}

URigVMController::URigVMController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bValidatePinDefaults(true)
	, bSuspendNotifications(false)
	, bSuspendRefreshingFunctionReferences(false)
	, bReportWarningsAndErrors(true)
	, bIgnoreRerouteCompactnessChanges(false)
	, UserLinkDirection(ERigVMPinDirection::Invalid)
	, bEnableTypeCasting(true)
	, bAllowPrivateFunctions(false)
	, bSuspendASTLinkChecks(false)
	, bIsTransacting(false)
	, bIsRunningUnitTest(false)
	, bIsFullyResolvingTemplateNode(false)
	, bSuspendTemplateComputation(false)
#if WITH_EDITOR
	, bRegisterTemplateNodeUsage(true)
#endif
	, bEnableSchemaRemoveNodeCheck(true)
	, UpdateGraphSectionsBracket(0)
{
	SetActionStack(URigVMActionStack::GetDisabledActionStack());
}

URigVMController::~URigVMController()
{
	{
		UE::TScopeLock Lock(ActionStackLock);
		if(URigVMActionStack* ActionStack = WeakActionStack.Get())
		{
			ActionStack->OnModified().Remove(ActionStackHandle);
			ActionStackHandle.Reset();
		}
	}
}

#if WITH_EDITORONLY_DATA
void URigVMController::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMActionStack::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMInjectionInfo::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMPin::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMVariableNode::StaticClass()));
	OutConstructClasses.Add(FTopLevelAssetPath(URigVMLink::StaticClass()));
}
#endif

URigVMGraph* URigVMController::GetGraph() const
{
	if (Graphs.Num() == 0)
	{
		return nullptr;
	}
	return Graphs.Last();
}

void URigVMController::SetGraph(URigVMGraph* InGraph)
{
	ensure(Graphs.Num() < 2);

	if (URigVMGraph* LastGraph = GetGraph())
	{
		if(LastGraph == InGraph)
		{
			return;
		}
		LastGraph->OnModified().RemoveAll(this);

		static constexpr TCHAR Message[] = TEXT("Usage of URigVMController::SetGraph to switch between graphs has been deprecated. Please rely on URigVMController::GetControllerForGraph instead.");
		ReportWarning(Message);
	}

	Graphs.Reset();
	if (InGraph != nullptr)
	{
		PushGraph(InGraph, false);
	}

	// make sure we have "some" action stack. this is mainly relevant for unit tests
	{
		UE::TScopeLock Lock(ActionStackLock);
		if(!WeakActionStack.IsValid())
		{
			SetActionStack_NoLock(NewObject<URigVMActionStack>(this, TEXT("ActionStack")));
		}
	}

	HandleModifiedEvent(ERigVMGraphNotifType::GraphChanged, GetGraph(), nullptr);
}

URigVMSchema* URigVMController::GetSchema() const
{
	if (SchemaClass)
	{
		return SchemaClass->GetDefaultObject<URigVMSchema>();
	}
	return nullptr;
}

bool URigVMController::PushGraph(URigVMGraph* InGraph, bool bSetupUndoRedo)
{
	if (URigVMGraph* LastGraph = GetGraph())
	{
		if(LastGraph == InGraph)
		{
			return false;
		}
		LastGraph->OnModified().RemoveAll(this);
		
		static constexpr TCHAR Message[] = TEXT("Usage of URigVMController::PushGraph to switch between graphs has been deprecated. Please rely on URigVMController::GetControllerForGraph instead.");
		ReportWarning(Message);
	}

	check(InGraph);
	Graphs.Push(InGraph);

	InGraph->OnModified().AddUObject(this, &URigVMController::HandleModifiedEvent);

	return true;
}

URigVMGraph* URigVMController::PopGraph(bool bSetupUndoRedo)
{
	ensure(Graphs.Num() > 1);
	if (Graphs.Num() == 1)
	{
		return nullptr;
	}
	
	URigVMGraph* LastGraph = GetGraph();
	if (LastGraph)
	{
		LastGraph->OnModified().RemoveAll(this);
	}

	Graphs.Pop();

	if (URigVMGraph* CurrentGraph = GetGraph())
	{
		CurrentGraph->OnModified().AddUObject(this, &URigVMController::HandleModifiedEvent);
	}

	return LastGraph;
}

URigVMGraph* URigVMController::GetTopLevelGraph() const
{
	URigVMGraph* Graph = GetGraph();
	UObject* Outer = Graph->GetOuter();
	while (Outer)
	{
		if (URigVMGraph* OuterGraph = Cast<URigVMGraph>(Outer))
		{
			Graph = OuterGraph;
			Outer = Outer->GetOuter();
		}
		else if (Outer->IsA<URigVMLibraryNode>())
		{
			Outer = Outer->GetOuter();
		}
		else
		{
			break;
		}
	}

	return Graph;
}

URigVMController* URigVMController::GetControllerForGraph(const URigVMGraph* InGraph) const
{
	if(InGraph == GetGraph())
	{
		return const_cast<URigVMController*>(this);
	}

	if(IRigVMClientHost* ClientHost = GetClientHost())
	{
		if(FRigVMClient* Client = ClientHost->GetRigVMClient())
		{
			URigVMController* Controller = Client->GetOrCreateController(InGraph);
			if(bIsRunningUnitTest)
			{
				Controller->bIsRunningUnitTest = true;
				Controller->bReportWarningsAndErrors = bReportWarningsAndErrors;
				if(!Controller->GetExternalVariablesDelegate.IsBound() && !InGraph->IsRootGraph())
				{
					// get the controller for the root graph
					if(const URigVMController* RootController =  Client->GetOrCreateController(InGraph->GetRootGraph()))
					{
						Controller->GetExternalVariablesDelegate = RootController->GetExternalVariablesDelegate;
					}
				}
			}
			return Controller;
		}
	}
	return nullptr;
}

IRigVMClientHost* URigVMController::GetClientHost() const
{
	return GetClientHost_Internal(GetGraph());
}

IRigVMClientHost* URigVMController::GetClientHost_Internal(const URigVMGraph* InGraph) const
{
	// get the client host from the graph rather than this controller,
	// since the graph may be in a different host / rigvmblueprint.
	IRigVMClientHost* ClientHost = nullptr;

	if(InGraph)
	{
		ClientHost = InGraph->GetImplementingOuter<IRigVMClientHost>();
	}

	// the graph may not be nested under a client in some unit tests
	if(ClientHost == nullptr)
	{
		ClientHost = GetImplementingOuter<IRigVMClientHost>();
	}

	return ClientHost;
}

TArray<FName> URigVMController::GetAllEventNames() const
{
	TArray<FName> AllEventNames;
	if(!IsValidGraph())
	{
		return AllEventNames;
	}

	TArray<const URigVMGraph*> GraphsToCheck = {GetGraph()};
	if(const IRigVMClientHost* ClientHost = GetClientHost())
	{
		const TArray<URigVMGraph*> ClientModels = ClientHost->GetAllModels();
		for(const URigVMGraph* ClientModel : ClientModels)
		{
			GraphsToCheck.AddUnique(ClientModel);
		}
	}
	for(const URigVMGraph* GraphToCheck : GraphsToCheck)
	{
		const TArray<FName> EventNames = GraphToCheck->GetEventNames();
		for(const FName& EventName : EventNames)
		{
			AllEventNames.AddUnique(EventName);
		}
	}

	return AllEventNames;
}

FRigVMGraphModifiedEvent& URigVMController::OnModified()
{
	return ModifiedEventStatic;
}

void URigVMController::Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject) const
{
	if (bSuspendNotifications)
	{
		return;
	}
	if (URigVMGraph* Graph = GetGraph())
	{
		Graph->Notify(InNotifType, InSubject);
	}
	else
	{
		const_cast<URigVMController*>(this)->HandleModifiedEvent(InNotifType, nullptr, InSubject);
	}
}

void URigVMController::ResendAllNotifications()
{
	if (URigVMGraph* Graph = GetGraph())
	{
		for (URigVMLink* Link : Graph->Links)
		{
			Notify(ERigVMGraphNotifType::LinkRemoved, Link);
		}

		for (URigVMNode* Node : Graph->Nodes)
		{
			Notify(ERigVMGraphNotifType::NodeRemoved, Node);
		}

		for (URigVMNode* Node : Graph->Nodes)
		{
			Notify(ERigVMGraphNotifType::NodeAdded, Node);

			if (Node->IsA<URigVMCommentNode>())
			{
				Notify(ERigVMGraphNotifType::CommentTextChanged, Node);
			}
		}

		for (URigVMLink* Link : Graph->Links)
		{
			Notify(ERigVMGraphNotifType::LinkAdded, Link);
		}
	}
}

void URigVMController::SetIsRunningUnitTest(bool bIsRunning)
{
	bIsRunningUnitTest = bIsRunning;

	if(URigVMBuildData* BuildData = URigVMBuildData::Get())
	{
		BuildData->SetIsRunningUnitTest(bIsRunning);
	}	
}

FRigVMPinInfo::FRigVMPinInfo()
	: ParentIndex(INDEX_NONE)
	, Name(NAME_None)
	, Direction(ERigVMPinDirection::Invalid)
	, TypeIndex(INDEX_NONE)
	, bIsArray(false)
	, Property(nullptr)
	, DefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
	, bIsExpanded(false)
	, bIsConstant(false)
	, bIsInputVariable(false)
	, bIsDynamicArray(false)
	, bIsTrait(false)
	, bIsLazy(false)
{
}

FRigVMPinInfo::FRigVMPinInfo(const URigVMPin* InPin, int32 InParentIndex, ERigVMPinDirection InDirection, ERigVMPinDefaultValueType InDefaultValueType)
	: ParentIndex(InParentIndex)
	, Name(InPin->GetName())
	, Direction(InDirection == ERigVMPinDirection::Invalid ? InPin->GetDirection() : InDirection)
	, TypeIndex(InPin->GetTypeIndex())
	, bIsArray(InPin->IsArray())
	, Property(nullptr)
	, DefaultValueType(InDefaultValueType)
	, bIsExpanded(InPin->IsExpanded())
	, bIsConstant(InPin->IsDefinedAsConstant())
	, bIsInputVariable(InPin->IsDefinedAsInputVariable())
	, bIsDynamicArray(InPin->IsDynamicArray())
	, bIsTrait(InPin->IsTraitPin() && InPin->IsRootPin())
	, bIsLazy(InPin->IsLazy() && InPin->IsRootPin())
{
	// this method describes the info as currently represented in the model.
	if (InPin)
	{
		const URigVMPin* Pin = InPin;
		while (Pin)
		{
			if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(Pin->GetCPPTypeObject()))
			{
				const FString SubPinPath = InPin->GetSubPinPath(Pin);
				if (const FProperty* StructProperty = UDStruct->FindPropertyByName(*SubPinPath))
				{
					DisplayName = *UDStruct->GetAuthoredNameForField(StructProperty);
					break;
				}
			}
			Pin = Pin->GetParentPin();
		}

		CorrectExecuteTypeIndex();
	
		DefaultValue = InPin->GetDefaultValue();
	
		if(DefaultValue.IsEmpty() && (InPin->IsArray() || InPin->IsStruct()))
		{
			static const FString EmptyBraces(TEXT("()"));
			DefaultValue = EmptyBraces;
		}
	}
}

FRigVMPinInfo::FRigVMPinInfo(FProperty* InProperty, ERigVMPinDirection InDirection, int32 InParentIndex, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory)
	: ParentIndex(InParentIndex)
	, Name(InProperty->GetFName())
	, Direction(InDirection)
	, TypeIndex(INDEX_NONE)
	, bIsArray(InProperty->IsA<FArrayProperty>())
	, Property(InProperty)
	, DefaultValueType(InDefaultValueType)
	, bIsExpanded(false)
	, bIsConstant(false)
	, bIsInputVariable(false)
	, bIsDynamicArray(false)
	, bIsTrait(false)
	, bIsLazy(false)
{
	// this method describes the info as needed based on the property structure

	if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(Property->GetOwnerStruct()))
	{
		DisplayName = *UDStruct->GetAuthoredNameForField(InProperty);
	}

	if (Direction == ERigVMPinDirection::Invalid)
	{
		Direction = FRigVMStruct::GetPinDirectionFromProperty(Property);
	}

#if WITH_EDITOR

	if (CastField<FArrayProperty>(InProperty->GetOwnerProperty()) == nullptr)
	{
		const FString DisplayNameText = InProperty->GetDisplayNameText().ToString();
		if (!DisplayNameText.IsEmpty())
		{
			DisplayName = *DisplayNameText;
		}
	}
	
	bIsConstant = InProperty->HasMetaData(TEXT("Constant"));
	bIsInputVariable = InProperty->HasMetaData(TEXT("ExternalVariable"));
	CustomWidgetName = InProperty->GetMetaData(TEXT("CustomWidget"));
	if (InProperty->HasMetaData(FRigVMStruct::ExpandPinByDefaultMetaName))
	{
		bIsExpanded = true;
	}

#endif

#if WITH_EDITOR
	if (Direction == ERigVMPinDirection::Hidden)
	{
		if (!InProperty->HasMetaData(TEXT("ArraySize")))
		{
			bIsDynamicArray = true;
		}
	}
	if (bIsDynamicArray)
	{
		if (InProperty->HasMetaData(FRigVMStruct::SingletonMetaName))
		{
			bIsDynamicArray = false;
		}
	}

	if (InProperty->HasMetaData(FRigVMStruct::ComputeLazilyMetaName))
	{
		bIsLazy = true;
	}
#endif

	UObject* CPPTypeObject = nullptr;
	
	FProperty* PropertyForType = InProperty;
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyForType))
	{
		PropertyForType = ArrayProperty->Inner;
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyForType))
	{
		CPPTypeObject = StructProperty->Struct;
	}
	else if (const FClassProperty* ClassProperty = CastField<FClassProperty>(PropertyForType))
	{
		if(RigVMCore::SupportsUObjects())
		{
			CPPTypeObject = ClassProperty->MetaClass;
		}
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyForType))
	{
		if(RigVMCore::SupportsUObjects())
		{
			CPPTypeObject = ObjectProperty->PropertyClass;
		}
	}
	else if (const FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(PropertyForType))
	{
		if (RigVMCore::SupportsUInterfaces())
		{
			CPPTypeObject = InterfaceProperty->InterfaceClass;
		}
	}
	else
	{
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyForType))
		{
			CPPTypeObject = EnumProperty->GetEnum();
		}
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyForType))
		{
			CPPTypeObject = ByteProperty->Enum;
		}

		if(InDefaultValueMemory)
		{
			InProperty->ExportText_Direct(DefaultValue, InDefaultValueMemory, InDefaultValueMemory, nullptr, PPF_None, nullptr);
		}
	}

	FString CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(InProperty);
	CPPType = RigVMTypeUtils::PostProcessCPPType(CPPType, CPPTypeObject);
	TypeIndex = FRigVMRegistry::Get().GetTypeIndexFromCPPType(CPPType);
	CorrectExecuteTypeIndex();
}

void FRigVMPinInfo::CorrectExecuteTypeIndex()
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	if(Registry.IsExecuteType(TypeIndex))
	{
		TRigVMTypeIndex DefaultExecuteType = RigVMTypeUtils::TypeIndex::Execute;
		if(Registry.IsArrayType(TypeIndex))
		{
			DefaultExecuteType = Registry.GetArrayTypeFromBaseTypeIndex(DefaultExecuteType);
		}
		TypeIndex = DefaultExecuteType;
	}
}

uint32 GetTypeHash(const FRigVMPinInfo& InPin)
{
	uint32 Hash = 0; //GetTypeHash(InPin.ParentIndex);
	Hash = HashCombine(Hash, GetTypeHash(InPin.Name.ToString()));
	Hash = HashCombine(Hash, GetTypeHash((int32)InPin.Direction));
	Hash = HashCombine(Hash, GetTypeHash((int32)InPin.TypeIndex));
	Hash = HashCombine(Hash, GetTypeHash(InPin.bIsArray));
	Hash = HashCombine(Hash, GetTypeHash(InPin.bIsTrait));
	Hash = HashCombine(Hash, GetTypeHash(InPin.bIsInputVariable));
	// we are not hashing the parent index,  pinpath, default value or the property since
	// it doesn't matter for the structure validity of the node
	return Hash;
}



FRigVMPinInfoArray::FRigVMPinInfoArray(const URigVMNode* InNode, URigVMController* InController)
{
	// this method adds all pins as currently represented in the model.
	for(const URigVMPin* Pin : InNode->GetPins())
	{
		const ERigVMPinDefaultValueType DefaultValueType = InController->GetDefaultValueType(Pin, Pin->GetDefaultValue());
		(void)AddPin(Pin, INDEX_NONE, ERigVMPinDirection::Invalid, DefaultValueType);
	}
}

int32 FRigVMPinInfoArray::AddPin(const URigVMPin* InPin, int32 InParentIndex, ERigVMPinDirection InDirection, ERigVMPinDefaultValueType InDefaultValueType)
{
	// this method adds all pins as currently represented in the model.
	const int32 Index = Pins.Emplace(InPin, InParentIndex, InDirection, InDefaultValueType);
	if (!Pins[Index].ShouldHideSubPins())
	{
		for(const URigVMPin* SubPin : InPin->GetSubPins())
		{
			const int32 SubPinIndex = AddPin(SubPin, Index, InDirection, InDefaultValueType);
			Pins[Index].SubPins.Add(SubPinIndex);
		}
	}
	Pins[Index].bIsInputVariable = InPin->IsDefinedAsInputVariable();
	return Index;
}

FRigVMPinInfoArray::FRigVMPinInfoArray(const URigVMNode* InNode, URigVMController* InController, const FRigVMPinInfoArray* InPreviousPinInfos)
{
	// this method adds pins as needed based on the property structure
	for(const URigVMPin* Pin : InNode->GetPins())
	{
		const FString DefaultValue = Pin->GetDefaultValue();
		const ERigVMPinDefaultValueType DefaultValueType = InController->GetDefaultValueType(Pin, DefaultValue);
		if (Pin->GetTypeIndex() == INDEX_NONE)
		{
			InController->ReportErrorf( TEXT("Invalid pin type %s for %s in %s"), *Pin->GetCPPType(), *Pin->GetPathName(), *InNode->GetPackage()->GetPathName());
		}
		const bool bAddSubPins = !InNode->IsA<URigVMRerouteNode>() && !Pin->ShouldHideSubPins();
		int32 Index = AddPin(InController, INDEX_NONE, Pin->GetFName(), Pin->GetDirection(), Pin->GetTypeIndex(), DefaultValue, DefaultValueType, nullptr, InPreviousPinInfos, bAddSubPins);
		if (InNode->IsA<URigVMLibraryNode>())
		{
			Pins[Index].bIsDynamicArray = Pins[Index].bIsArray;
		}
		if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode))
		{
			Pins[Index].Guid = CollapseNode->FindPinGuid(Pin);
		}
		Pins[Index].bIsInputVariable = Pin->IsDefinedAsInputVariable();
	}
}

FRigVMPinInfoArray::FRigVMPinInfoArray(const FRigVMGraphFunctionHeader& FunctionHeader,
	URigVMController* InController, const FRigVMPinInfoArray* InPreviousPinInfos)
{
	// this method adds pins as needed based on the property structure
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	for(const FRigVMGraphFunctionArgument& FunctionArgument : FunctionHeader.Arguments)
	{
		const TRigVMTypeIndex TypeIndex = Registry.GetTypeIndexFromCPPType(FunctionArgument.CPPType.ToString());
		if (TypeIndex == INDEX_NONE)
		{
			InController->ReportErrorf( TEXT("Invalid pin type %s for %s in %s"), *FunctionArgument.CPPType.ToString(), *FunctionHeader.LibraryPointer.GetLibraryNodePath(), *InController->GetPackage()->GetPathName());
		}
		ensureMsgf(TypeIndex != INDEX_NONE, TEXT("Invalid pin type %s in %s"), *FunctionArgument.CPPType.ToString(), *InController->GetPackage()->GetPathName());
		ERigVMPinDefaultValueType DefaultValueType = ERigVMPinDefaultValueType::Unset;
		if(const FRigVMPinInfo* PinInfo = InPreviousPinInfos->GetPinFromPinPath(FunctionArgument.Name.ToString()))
		{
			DefaultValueType = PinInfo->DefaultValueType;
		}
		const bool bAddSubPins = !(FunctionArgument.bIsInputVariable && FunctionArgument.Direction == ERigVMPinDirection::Input);
		int32 Index = AddPin(InController, INDEX_NONE, FunctionArgument.Name, FunctionArgument.Direction, TypeIndex, FunctionArgument.DefaultValue, DefaultValueType, nullptr, InPreviousPinInfos, bAddSubPins);
		Pins[Index].bIsDynamicArray = Pins[Index].bIsArray;
		Pins[Index].bIsInputVariable = FunctionArgument.bIsInputVariable;
	}
}

int32 FRigVMPinInfoArray::AddPin(FProperty* InProperty, URigVMController* InController,
    ERigVMPinDirection InDirection, int32 InParentIndex, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory, bool bAddSubPins)
{
	// this method adds pins as needed based on the property structure

	check(InDefaultValueMemory);
	
	const int32 Index = Pins.Emplace(InProperty, InDirection, InParentIndex, InDefaultValueType, InDefaultValueMemory);
	if(InParentIndex != INDEX_NONE)
	{
		Pins[InParentIndex].SubPins.Add(Index);
	}

#if WITH_EDITOR
	Pins[Index].bIsInputVariable = InProperty->HasMetaData(TEXT("ExternalVariable"));
#endif
	if (Pins[Index].ShouldHideSubPins())
	{
		bAddSubPins = false;
	}

	if(bAddSubPins)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [InDefaultValueType](const FName&)
			{
				return InDefaultValueType;
			};
			(void)AddPins(StructProperty->Struct, InController, Pins[Index].Direction, Index, DefaultValueTypeGetter, InDefaultValueMemory, bAddSubPins);
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, InDefaultValueMemory);
			for(int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
			{
				const uint8* ElementDefaultValueMemory = ArrayHelper.GetRawPtr(ElementIndex);
				const int32 SubIndex = AddPin(ArrayProperty->Inner, InController, Pins[Index].Direction, Index, InDefaultValueType, ElementDefaultValueMemory, bAddSubPins);
				Pins[SubIndex].Name = *FString::FormatAsNumber(ElementIndex);
			}
		}
	}
	return Index;
}

int32 FRigVMPinInfoArray::AddPin(URigVMController* InController, int32 InParentIndex, const FName& InName, ERigVMPinDirection InDirection,
	TRigVMTypeIndex InTypeIndex, const FString& InDefaultValue, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory, const FRigVMPinInfoArray* InPreviousPinInfos, bool bAddSubPins)
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		
	FRigVMPinInfo Info;
	Info.ParentIndex = InParentIndex;
	Info.Name = InName;
	Info.Direction = InDirection;
	Info.TypeIndex = InTypeIndex;
	Info.bIsArray = Registry.IsArrayType(InTypeIndex);
	Info.DefaultValue = InDefaultValue;
	Info.DefaultValueType = InDefaultValueType;
	Info.CorrectExecuteTypeIndex();

	const int32 Index = Pins.Add(Info);
	if (InTypeIndex == INDEX_NONE)
	{
		InController->ReportErrorf(TEXT("Cannot add pin %s due to invalid type in package %s."), *GetPinPath(Index), *InController->GetPackage()->GetPathName());
	}

	if(InPreviousPinInfos)
	{
		const FString& PinPath = GetPinPath(Index);
		const int32 PreviousIndex = InPreviousPinInfos->GetIndexFromPinPath(PinPath);
		if(PreviousIndex != INDEX_NONE)
		{
			const FRigVMPinInfo& PreviousPin = (*InPreviousPinInfos)[PreviousIndex];
			if(PreviousPin.TypeIndex == InTypeIndex)
			{
				Pins[Index].DefaultValue = PreviousPin.DefaultValue;
			}
		}
	}
	
	const FRigVMTemplateArgumentType& Type = Registry.GetType(InTypeIndex);
	if(!Type.IsWildCard() && bAddSubPins)
	{
		if(Info.bIsArray)
		{
			const TRigVMTypeIndex& ElementTypeIndex = Registry.GetBaseTypeFromArrayTypeIndex(InTypeIndex);
			const FRigVMTemplateArgumentType& ElementType = Registry.GetType(ElementTypeIndex);

			const TArray<FString> Elements = URigVMPin::SplitDefaultValue(Pins[Index].DefaultValue);
			for(int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
			{
				const FString& ElementDefaultValue = Elements[ElementIndex];
				const uint8* ElementDefaultValueMemory = nullptr;
				FStructOnScope ElementDefaultValueMemoryScope;

				if(UScriptStruct* ElementScriptStruct = Cast<UScriptStruct>(ElementType.CPPTypeObject))
				{
					ElementDefaultValueMemoryScope = FStructOnScope(ElementScriptStruct);
		
					FRigVMPinDefaultValueImportErrorContext ErrorPipe;
					ElementScriptStruct->ImportText(*ElementDefaultValue, ElementDefaultValueMemoryScope.GetStructMemory(), nullptr, PPF_None, &ErrorPipe, FString());
					ElementDefaultValueMemory = ElementDefaultValueMemoryScope.GetStructMemory();
				}

				(void)AddPin(InController, Index, *FString::FormatAsNumber(ElementIndex), InDirection, ElementTypeIndex, ElementDefaultValue, InDefaultValueType, ElementDefaultValueMemory, InPreviousPinInfos, bAddSubPins);
			}
		}
		else if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
		{
			const uint8* DefaultValueMemory = InDefaultValueMemory;
			FStructOnScope DefaultValueMemoryScope;
			if(DefaultValueMemory == nullptr)
			{
				FRigVMPinDefaultValueImportErrorContext ErrorPipe;
				DefaultValueMemoryScope = FStructOnScope(ScriptStruct);
				ScriptStruct->ImportText(*Pins[Index].DefaultValue, DefaultValueMemoryScope.GetStructMemory(), nullptr, PPF_None, &ErrorPipe, FString());
				DefaultValueMemory = DefaultValueMemoryScope.GetStructMemory();
			}
			const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [InDefaultValueType](const FName&)
			{
				return InDefaultValueType;
			};
			AddPins(ScriptStruct, InController, InDirection, Index, DefaultValueTypeGetter, DefaultValueMemory, bAddSubPins);
		}
	}

	if(InParentIndex != INDEX_NONE)
	{
		Pins[InParentIndex].SubPins.Add(Index);
	}

	return Index;
}

void FRigVMPinInfoArray::AddPins(UScriptStruct* InScriptStruct, URigVMController* InController,
	ERigVMPinDirection InDirection, int32 InParentIndex, TFunction<ERigVMPinDefaultValueType(const FName&)> InDefaultValueTypeGetter,
	const uint8* InDefaultValueMemory, bool bAddSubPins)
{
	TSharedPtr<FStructOnScope> StructOnScope;
	const FRigVMStruct* RigVMStructInstance = nullptr;
	if(InScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		StructOnScope = MakeShareable(new FStructOnScope(InScriptStruct));
		RigVMStructInstance = reinterpret_cast<FRigVMStruct*>(StructOnScope->GetStructMemory()); 
	}
	else if (InDefaultValueMemory == nullptr)
	{
		StructOnScope = MakeShareable(new FStructOnScope(InScriptStruct));
	}

	if (InController->GetSchema()->ShouldUnfoldStruct(InController, InScriptStruct))
	{
		TArray<UStruct*> StructsToVisit = FRigVMTemplate::GetSuperStructs(InScriptStruct, true);
		for(UStruct* StructToVisit : StructsToVisit)
		{
			// using EFieldIterationFlags::None excludes the
			// properties of the super struct in this iterator.
			for (TFieldIterator<FProperty> It(StructToVisit, EFieldIterationFlags::None); It; ++It)
			{
				// filter pins that structs want to hide
				if(RigVMStructInstance)
				{
					if(!RigVMStructInstance->ShouldCreatePinForProperty(*It))
					{
						continue;
					}
				}

				const uint8* DefaultValueMemory = nullptr;
				if(InDefaultValueMemory)
				{
					DefaultValueMemory = It->ContainerPtrToValuePtr<uint8>(InDefaultValueMemory);
				}
				else
				{
					DefaultValueMemory = It->ContainerPtrToValuePtr<uint8>(StructOnScope->GetStructMemory());
				}

				bool bAddSubPinsForProperty = bAddSubPins;
#if WITH_EDITOR
				if(bAddSubPins)
				{
					if(It->HasMetaData(FRigVMStruct::HideSubPinsMetaName))
					{
						bAddSubPinsForProperty = false;
					}
				}
#endif
				ERigVMPinDefaultValueType DefaultValueType = ERigVMPinDefaultValueType::AutoDetect;
				if(InDefaultValueTypeGetter)
				{
					DefaultValueType = InDefaultValueTypeGetter(It->GetFName());
				}
				(void)AddPin(*It, InController, InDirection, InParentIndex, DefaultValueType, DefaultValueMemory, bAddSubPinsForProperty);
			}
		}
	}
}

const FString& FRigVMPinInfoArray::GetPinPath(const int32 InIndex) const
{
	if(!Pins.IsValidIndex(InIndex))
	{
		static const FString EmptyString;
		return EmptyString;
	}

	if(Pins[InIndex].PinPath.IsEmpty())
	{
		if(Pins[InIndex].ParentIndex == INDEX_NONE)
		{
			Pins[InIndex].PinPath = Pins[InIndex].Name.ToString();
		}
		else
		{
			Pins[InIndex].PinPath = URigVMPin::JoinPinPath(GetPinPath(Pins[InIndex].ParentIndex), Pins[InIndex].Name.ToString());
		}
	}

	return Pins[InIndex].PinPath;
}

int32 FRigVMPinInfoArray::GetIndexFromPinPath(const FString& InPinPath) const
{
	if(PinPathLookup.Num() != Num())
	{
		PinPathLookup.Reset();
		for(int32 Index=0; Index < Num(); Index++)
		{
			PinPathLookup.Add(GetPinPath(Index), Index);
		}
	}

	if(const int32* Index = PinPathLookup.Find(InPinPath))
	{
		return *Index;
	}
	return INDEX_NONE;
}

const FRigVMPinInfo* FRigVMPinInfoArray::GetPinFromPinPath(const FString& InPinPath) const
{
	const int32 Index = GetIndexFromPinPath(InPinPath);
	if(Pins.IsValidIndex(Index))
	{
		return &Pins[Index];
	}
	return nullptr;
}

int32 FRigVMPinInfoArray::GetRootIndex(const int32 InIndex) const
{
	if(Pins.IsValidIndex(InIndex))
	{
		if(Pins[InIndex].ParentIndex == INDEX_NONE)
		{
			return InIndex;
		}
		return GetRootIndex(Pins[InIndex].ParentIndex);
	}
	return INDEX_NONE;
}

uint32 GetTypeHash(const FRigVMPinInfoArray& InPins)
{
	TArray<uint32> Hashes;
	Hashes.Reserve(InPins.Num());
	
	uint32 OverAllHash = GetTypeHash(InPins.Num());
	for(const FRigVMPinInfo& Info : InPins)
	{
		uint32 PinHash = GetTypeHash(Info);
		if(Info.ParentIndex != INDEX_NONE)
		{
			PinHash = HashCombine(PinHash, Hashes[Info.ParentIndex]);
		}
		Hashes.Add(PinHash);
		OverAllHash = HashCombine(OverAllHash, PinHash); 
	}
	return OverAllHash;
}

void URigVMController::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	switch (InNotifType)
	{
		case ERigVMGraphNotifType::GraphChanged:
		case ERigVMGraphNotifType::NodeAdded:
		case ERigVMGraphNotifType::NodeRemoved:
		case ERigVMGraphNotifType::LinkAdded:
		case ERigVMGraphNotifType::LinkRemoved:
		case ERigVMGraphNotifType::PinArraySizeChanged:
		{
			if (InGraph)
			{
				InGraph->ClearAST();
			}
			break;
		}
		case ERigVMGraphNotifType::PinDefaultValueChanged:
		{
			if (InGraph->RuntimeAST.IsValid())
			{
				URigVMPin* RootPin = CastChecked<URigVMPin>(InSubject)->GetRootPin();
				FRigVMASTProxy RootPinProxy = FRigVMASTProxy::MakeFromUObject(RootPin);
				const FRigVMExprAST* Expression = InGraph->GetRuntimeAST()->GetExprForSubject(RootPinProxy);
				if (Expression == nullptr)
				{
					InGraph->ClearAST();
					break;
				}
				else if(Expression->NumParents() > 1)
				{
					InGraph->ClearAST();
					break;
				}
			}
			break;
		}
		case ERigVMGraphNotifType::VariableAdded:
		case ERigVMGraphNotifType::VariableRemoved:
		{
			URigVMGraph* RootGraph = InGraph->GetRootGraph();
			if(URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(RootGraph->GetRootGraph()))
			{
				URigVMNode* Node = CastChecked<URigVMNode>(InSubject);
				check(Node);

				bool bIsLocal = false;
				if (InNotifType == ERigVMGraphNotifType::VariableAdded || InNotifType == ERigVMGraphNotifType::VariableRemoved)
				{
					if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
					{
						if (VariableNode->IsLocalVariable() || VariableNode->IsInputArgument())
						{
							bIsLocal = true;
						}
					}
				}
			}
		}
	}

	ModifiedEventStatic.Broadcast(InNotifType, InGraph, InSubject);
	if (ModifiedEventDynamic.IsBound())
	{
		ModifiedEventDynamic.Broadcast(InNotifType, InGraph, InSubject);
	}
}

TArray<FString> URigVMController::GeneratePythonCommands() 
{
	TArray<FString> Commands;

	const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

	// Add local variables
	for (const FRigVMGraphVariableDescription& Variable : GetGraph()->LocalVariables)
	{
		const FString VariableName = GetSchema()->GetSanitizedVariableName(Variable.Name.ToString());

		if (Variable.CPPTypeObject)
		{
			// FRigVMGraphVariableDescription AddLocalVariable(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, bool bSetupUndoRedo = true);
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_local_variable_from_object_path('%s', '%s', '%s', '%s')"),
						*GraphName,
						*VariableName,
						*Variable.CPPType,
						Variable.CPPTypeObject ? *Variable.CPPTypeObject->GetPathName() : TEXT(""),
						*Variable.DefaultValue));
		}
		else
		{
			// FRigVMGraphVariableDescription AddLocalVariable(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, bool bSetupUndoRedo = true);
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_local_variable('%s', '%s', None, '%s')"),
						*GraphName,
						*VariableName,
						*Variable.CPPType,
						*Variable.DefaultValue));
		}
	}
	
	
	// All nodes
	for (URigVMNode* Node : GetGraph()->GetNodes())
	{
		Commands.Append(GetAddNodePythonCommands(Node));
	}

	// All links
	for (URigVMLink* Link : GetGraph()->GetLinks())
	{
		URigVMPin* SourcePin = Link->GetSourcePin();
		URigVMPin* TargetPin = Link->GetTargetPin();
		
		if (SourcePin->GetInjectedNodes().Num() > 0 || TargetPin->GetInjectedNodes().Num() > 0)
		{
			continue;
		}

		const FString SourcePinPath = GetSchema()->GetSanitizedPinPath(SourcePin->GetPinPath());
		const FString TargetPinPath = GetSchema()->GetSanitizedPinPath(TargetPin->GetPinPath());

		//bool AddLink(const FString& InOutputPinPath, const FString& InInputPinPath, bool bSetupUndoRedo = true);
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_link('%s', '%s')"),
					*GraphName,
					*SourcePinPath,
					*TargetPinPath));
	}

	return Commands;
}

TArray<FString> URigVMController::GetAddNodePythonCommands(URigVMNode* Node) const
{
	TArray<FString> Commands;

	const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
	const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());

	auto GetResolveWildcardPinsPythonCommands = [](const FString& InGraphName, const URigVMTemplateNode* InNode, const FRigVMTemplate* InTemplate)
	{
		TArray<FString> Commands;

		// Lets minimize the number of commands by stopping when the number of permutations left is 1 (or less)
		TArray<int32> Permutations;
		Permutations.SetNumUninitialized(InTemplate->NumPermutations());
		FRigVMTemplate::FTypeMap TypeMap;
		
		for (int32 ArgIndex = 0; ArgIndex < InTemplate->NumArguments(); ++ArgIndex)
		{
			if (Permutations.Num() < 2)
			{
				break;
			}
			
			const FRigVMTemplateArgument* Argument = InTemplate->GetArgument(ArgIndex);
			if (!Argument->IsSingleton())
			{
				URigVMPin* Pin = InNode->FindPin(Argument->GetName().ToString());
				if (!Pin->IsWildCard())
				{
					Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').resolve_wild_card_pin('%s', '%s', '%s')"),
							*InGraphName,
							*Pin->GetPinPath(),
							*Pin->GetCPPType(),
							*Pin->GetCPPTypeObject()->GetPathName()));

					TypeMap.Add(Argument->GetName(), Pin->GetTypeIndex());
					InTemplate->Resolve(TypeMap, Permutations, false);
				}
			}
		}

		return Commands;
	};

	if (const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
	{
		if (const URigVMInjectionInfo* InjectionInfo = Cast<URigVMInjectionInfo>(UnitNode->GetOuter()))
		{
			const URigVMPin* InjectionInfoPin = InjectionInfo->GetPin();
			const FString InjectionInfoPinPath = GetSchema()->GetSanitizedPinPath(InjectionInfoPin->GetPinPath());
			const FString InjectionInfoInputPinName = InjectionInfo->InputPin ? GetSchema()->GetSanitizedPinName(InjectionInfo->InputPin->GetName()) : FString();
			const FString InjectionInfoOutputPinName = InjectionInfo->OutputPin ? GetSchema()->GetSanitizedPinName(InjectionInfo->OutputPin->GetName()) : FString();

			//URigVMInjectionInfo* AddInjectedNodeFromStructPath(const FString& InPinPath, bool bAsInput, const FString& InScriptStructPath, const FName& InMethodName, const FName& InInputPinName, const FName& InOutputPinName, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);
			Commands.Add(FString::Printf(TEXT("%s_info = blueprint.get_controller_by_name('%s').add_injected_node_from_struct_path('%s', %s, '%s', '%s', '%s', '%s', '%s')"),
					*NodeName, 
					*GraphName, 
					*InjectionInfoPinPath, 
					InjectionInfoPin->GetDirection() == ERigVMPinDirection::Input ? TEXT("True") : TEXT("False"), 
					*UnitNode->GetScriptStruct()->GetPathName(), 
					*UnitNode->GetMethodName().ToString(), 
					*InjectionInfoInputPinName, 
					*InjectionInfoOutputPinName, 
					*UnitNode->GetName()));
		}
		else if (UnitNode->IsSingleton())
		{
			// add_struct_node_from_struct_path(script_struct_path, method_name, position=[0.0, 0.0], node_name='', undo=True)
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_unit_node_from_struct_path('%s', 'Execute', %s, '%s')"),
					*GraphName,
					*UnitNode->GetScriptStruct()->GetPathName(),
					*RigVMPythonUtils::Vector2DToPythonString(UnitNode->GetPosition()),
					*NodeName));
		}
		else
		{
			// add_template_node(notation, position=[0.0, 0.0], node_name='', undo=True)
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_template_node('%s', %s, '%s')"),
							*GraphName,
							*UnitNode->GetNotation().ToString(),
							*RigVMPythonUtils::Vector2DToPythonString(UnitNode->GetPosition()),
							*NodeName));

			// Try to resolve wildcard pins			
			if (const FRigVMTemplate* Template = UnitNode->GetTemplate())
			{
				Commands.Append(GetResolveWildcardPinsPythonCommands(GraphName, UnitNode, Template));
			}			
		}		
	}
	else if (const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
	{
		// add_template_node(notation, position=[0.0, 0.0], node_name='', undo=True)
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_template_node('%s', %s, '%s')"),
						*GraphName,
						*DispatchNode->GetNotation().ToString(),
						*RigVMPythonUtils::Vector2DToPythonString(DispatchNode->GetPosition()),
						*NodeName));

		// Try to resolve wildcard pins			
		if (const FRigVMTemplate* Template = DispatchNode->GetTemplate())
		{
			Commands.Append(GetResolveWildcardPinsPythonCommands(GraphName, DispatchNode, Template));
		}			
	}
	else if (const URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(Node))
	{
		TArray<FString> InnerNodeCommands = GetAddNodePythonCommands(AggregateNode->GetFirstInnerNode());
		Commands.Append(InnerNodeCommands);

		// set_node_position(node_name, position)
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_position_by_name('%s', %s)"),
				*GraphName,
				*AggregateNode->GetName(),
				*RigVMPythonUtils::Vector2DToPythonString(AggregateNode->GetPosition())));

		// add commands for any additional aggregate pin
		const TArray<URigVMPin*> AggregatePins = AggregateNode->IsInputAggregate() ? AggregateNode->GetAggregateInputs() : AggregateNode->GetAggregateOutputs();

		for(int32 Index = 2; Index < AggregatePins.Num(); Index++)
		{
			// add_aggregate_pin(node_name, pin_name)
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_aggregate_pin('%s', '%s')"),
					*GraphName,
					*AggregateNode->GetName(),
					*AggregatePins[Index]->GetName()
				));
		}
	}
	else if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
	{
		if (!VariableNode->IsInjected())
		{
			const FString VariableName = GetSchema()->GetSanitizedVariableName(VariableNode->GetVariableName().ToString());

			// add_variable_node(variable_name, cpp_type, cpp_type_object, is_getter, default_value, position=[0.0, 0.0], node_name='', undo=True)
			if (VariableNode->GetVariableDescription().CPPTypeObject)
			{
				Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_variable_node_from_object_path('%s', '%s', '%s', %s, '%s', %s, '%s')"),
						*GraphName,
						*VariableName,
						*VariableNode->GetVariableDescription().CPPType,
						*VariableNode->GetVariableDescription().CPPTypeObject->GetPathName(),
						VariableNode->IsGetter() ? TEXT("True") : TEXT("False"),
						*VariableNode->GetVariableDescription().DefaultValue,
						*RigVMPythonUtils::Vector2DToPythonString(VariableNode->GetPosition()),
						*NodeName));	
			}
			else
			{
				Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_variable_node('%s', '%s', None, %s, '%s', %s, '%s')"),
						*GraphName,
						*VariableName,
						*VariableNode->GetVariableDescription().CPPType,
						VariableNode->IsGetter() ? TEXT("True") : TEXT("False"),
						*VariableNode->GetVariableDescription().DefaultValue,
						*RigVMPythonUtils::Vector2DToPythonString(VariableNode->GetPosition()),
						*NodeName));	
			}
		}
	}
	else if (const URigVMCommentNode* CommentNode = Cast<URigVMCommentNode>(Node))
	{
		// add_comment_node(comment_text, position=[0.0, 0.0], size=[400.0, 300.0], color=[0.0, 0.0, 0.0, 0.0], node_name='', undo=True)
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_comment_node('%s', %s, %s, %s, '%s')"),
					*GraphName,
					*CommentNode->GetCommentText().ReplaceCharWithEscapedChar(),
					*RigVMPythonUtils::Vector2DToPythonString(CommentNode->GetPosition()),
					*RigVMPythonUtils::Vector2DToPythonString(CommentNode->GetSize()),
					*RigVMPythonUtils::LinearColorToPythonString(CommentNode->GetNodeColor()),
					*NodeName));	
	}
	else if (const URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(Node))
	{
		// add_free_reroute_node(const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bIsConstant, const FName& InCustomWidgetName, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_free_reroute_node('%s', '%s', %s, '%s', '%s', %s, '%s')"),
				*GraphName,
				*RerouteNode->GetPins()[0]->GetCPPType(),
				*RerouteNode->GetPins()[0]->GetCPPTypeObject()->GetPathName(),
				RerouteNode->GetPins()[0]->IsDefinedAsConstant() ? TEXT("True") : TEXT("False"),
				*RerouteNode->GetPins()[0]->GetCustomWidgetName().ToString(),
				*RerouteNode->GetPins()[0]->GetDefaultValue(),
				*RigVMPythonUtils::Vector2DToPythonString(RerouteNode->GetPosition()),
				*NodeName));
	}
	else if (const URigVMEnumNode* EnumNode = Cast<URigVMEnumNode>(Node))
	{
		// add_enum_node(cpp_type_object_path, position=[0.0, 0.0], node_name='', undo=True)
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_enum_node('%s', %s, '%s')"),
						*GraphName,
						*EnumNode->GetCPPTypeObject()->GetPathName(),
						*RigVMPythonUtils::Vector2DToPythonString(EnumNode->GetPosition()),
						*NodeName));
	}
	else if (const URigVMFunctionReferenceNode* RefNode = Cast<URigVMFunctionReferenceNode>(Node))
	{
		if (RefNode->GetReferencedFunctionHeader().LibraryPointer.HostObject == GetGraph()->GetDefaultFunctionLibrary()->GetFunctionHostObjectPath())
		{
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_function_reference_node(function_%s, %s, '%s')"),
					*GraphName,
					*RigVMPythonUtils::PythonizeName(RefNode->LoadReferencedNode()->GetContainedGraph()->GetGraphName()),
					*RigVMPythonUtils::Vector2DToPythonString(RefNode->GetPosition()), 
					*NodeName));
		}
		else
		{
			Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_external_function_reference_node('%s', '%s', %s, '%s')"),
						*GraphName,
						*RefNode->GetReferencedFunctionHeader().LibraryPointer.HostObject.ToString(),
						*RefNode->GetReferencedFunctionHeader().Name.ToString(),
						*RigVMPythonUtils::Vector2DToPythonString(RefNode->GetPosition()), 
						*NodeName));
		}
	}
	else if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
	{
		const FString ContainedGraphName = GetSchema()->GetSanitizedGraphName(CollapseNode->GetContainedGraph()->GetGraphName());
		// AddFunctionReferenceNode(URigVMLibraryNode* InFunctionDefinition, const FVector2D& InNodePosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_function_reference_node(function_%s, %s, '%s')"),
					*GraphName,
					*RigVMPythonUtils::PythonizeName(ContainedGraphName),
					*RigVMPythonUtils::Vector2DToPythonString(CollapseNode->GetPosition()), 
					*NodeName));
	

		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').promote_function_reference_node_to_collapse_node('%s')"),
				*GraphName,
				*NodeName));
		Commands.Add(FString::Printf(TEXT("library_controller.remove_function_from_library('%s')"),
				*ContainedGraphName));
	
	}
	else if (const URigVMInvokeEntryNode* InvokeEntryNode = Cast<URigVMInvokeEntryNode>(Node))
	{
		// add_invoke_entry_node(entry_name, position=[0.0, 0.0], node_name='', undo=True)
		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_invoke_entry_node('%s', %s, '%s')"),
						*GraphName,
						*InvokeEntryNode->GetEntryName().ToString(),
						*RigVMPythonUtils::Vector2DToPythonString(InvokeEntryNode->GetPosition()),
						*NodeName));
	}
	else if (Node->IsA<URigVMFunctionInterfaceNode>())
	{
		
		
	}
	else
	{
		ensure(false);
	}

	if (!Commands.IsEmpty())
	{
		for (const URigVMPin* Pin : Node->GetPins())
		{
			if (Pin->GetDirection() == ERigVMPinDirection::Output || Pin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				continue;
			}

			if(Pin->IsTraitPin())
			{
				continue;
			}
			
			const FString DefaultValue = Pin->GetDefaultValue();
			if (!DefaultValue.IsEmpty() && DefaultValue != TEXT("()"))
			{
				const FString PinPath = GetSchema()->GetSanitizedPinPath(Pin->GetPinPath());

				Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_default_value('%s', '%s')"),
							*GraphName,
							*PinPath,
							*Pin->GetDefaultValue()));


				TArray<const URigVMPin*> SubPins = { Pin };
				for (int32 i = 0; i < SubPins.Num(); ++i)
				{
					if (SubPins[i]->IsStruct() || SubPins[i]->IsArray())
					{
						SubPins.Append(SubPins[i]->GetSubPins());
						const FString SubPinPath = GetSchema()->GetSanitizedPinPath(SubPins[i]->GetPinPath());

						Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_expansion('%s', %s)"),
							*GraphName,
							*SubPinPath,
							SubPins[i]->IsExpanded() ? TEXT("True") : TEXT("False")));
					}
				}
			}

			if (!Pin->GetBoundVariablePath().IsEmpty())
			{
				const FString PinPath = GetSchema()->GetSanitizedPinPath(Pin->GetPinPath());

				Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').bind_pin_to_variable('%s', '%s')"),
							*GraphName,
							*PinPath,
							*Pin->GetBoundVariablePath()));
			}
		}

		for(const FString& TraitName : Node->GetTraitNames())
		{
			Commands.Append(GetAddTraitPythonCommands(Node, *TraitName));
		}
	}

	return Commands;
}

TArray<FString> URigVMController::GetAddTraitPythonCommands(URigVMNode* Node, const FName& TraitName) const
{
	TArray<FString> Commands;

	const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
	const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());

	if(const URigVMPin* TraitPin = Node->FindTrait(TraitName))
	{
		const FString TraitStructPath = TraitPin->GetCPPTypeObject()->GetPathName();
		const FString DefaultValue = TraitPin->GetDefaultValue();

		Commands.Add(FString::Printf(TEXT("asset.get_controller_by_name('%s').add_trait('%s', '%s', '%s', '%s')"),
			*GraphName,
			*NodeName,
			*TraitStructPath,
			*TraitName.ToString(),
			*DefaultValue));
	}

	return Commands;
}

FRigVMGraphFunctionStore* URigVMController::GetGraphFunctionStore() const
{
	IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>();
	if (!ClientHost)
	{
		ReportErrorf(TEXT("Cannot find client host."));
		return nullptr;
	}

	TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost();
	if (!FunctionHost)
	{
		ReportErrorf(TEXT("Cannot find function host."));
		return nullptr;
	}

	return FunctionHost->GetRigVMGraphFunctionStore();
}

FRigVMGraphFunctionData* URigVMController::FindFunctionData(const FName& InFunctionName) const
{
	FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore();
	if (!FunctionStore)
	{
		return nullptr;
	}

	return FunctionStore->FindFunctionByName(InFunctionName);
}

#if WITH_EDITOR

URigVMUnitNode* URigVMController::AddUnitNode(UScriptStruct* InScriptStruct, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return AddUnitNode(InScriptStruct, URigVMUnitNode::StaticClass(), InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMUnitNode* URigVMController::AddUnitNode(UScriptStruct* InScriptStruct, TSubclassOf<URigVMUnitNode> InUnitNodeClass, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return AddUnitNodeWithDefaults(InScriptStruct, InUnitNodeClass, FRigStructScope(), InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

const UClass* URigVMController::GetNodeClassForTemplate(FRigVMTemplate* InTemplate) const
{
	// determine what kind of node we need to create
	if(InTemplate->UsesDispatch())
	{
		return URigVMDispatchNode::StaticClass();
	}
	if(const FRigVMFunction* FirstFunction = InTemplate->GetOrCreatePermutation(0))
	{
		const UScriptStruct* PotentialUnitStruct = FirstFunction->Struct;
		if(PotentialUnitStruct && PotentialUnitStruct->IsChildOf(FRigVMStruct::StaticStruct()))
		{
			return URigVMUnitNode::StaticClass();
		}
	}
	return nullptr;
}

URigVMUnitNode* URigVMController::AddUnitNodeFromStructPath(const FString& InScriptStructPath, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(InScriptStructPath);
	if (ScriptStruct == nullptr)
	{
		ReportErrorf(TEXT("Cannot find struct for path '%s'."), *InScriptStructPath);
		return nullptr;
	}

	return AddUnitNode(ScriptStruct, InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMUnitNode* URigVMController::AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, const FString& InDefaults,
	const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if(InScriptStruct == nullptr)
	{
		return nullptr;
	}

	FStructOnScope StructOnScope;

	if(!InDefaults.IsEmpty())
	{
		StructOnScope = FStructOnScope(InScriptStruct);
		
		FRigVMPinDefaultValueImportErrorContext ErrorPipe;
		InScriptStruct->ImportText(*InDefaults, StructOnScope.GetStructMemory(), nullptr, PPF_None, &ErrorPipe, FString());

		if(ErrorPipe.NumErrors > 0)
		{
			return nullptr;
		}
	}

	return AddUnitNodeWithDefaults(InScriptStruct, StructOnScope, InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMUnitNode* URigVMController::AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, const FRigStructScope& InDefaults,
														  const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo,
														  bool bPrintPythonCommand)
{
	return AddUnitNodeWithDefaults(InScriptStruct, URigVMUnitNode::StaticClass(), InDefaults, InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMUnitNode* URigVMController::AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, TSubclassOf<URigVMUnitNode> InUnitNodeClass, const FRigStructScope& InDefaults,
                                                          const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo,
                                                          bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if (GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add unit nodes to function library graphs."));
		return nullptr;
	}

	if (InScriptStruct == nullptr)
	{
		ReportError(TEXT("InScriptStruct is null."));
		return nullptr;
	}
	if (InMethodName == NAME_None)
	{
		ReportError(TEXT("InMethodName is None."));
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(InScriptStruct, *InMethodName.ToString());
	if (Function == nullptr)
	{
		ReportErrorf(TEXT("RIGVM_METHOD '%s::%s' cannot be found."), *InScriptStruct->GetStructCPPName(), *InMethodName.ToString());
		return nullptr;
	}

	if(!GetSchema()->SupportsUnitFunction(this, Function))
	{
		return nullptr;
	}

	if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if(const FRigVMClient* Client = ClientHost->GetRigVMClient())
		{
			if(!Function->SupportsExecuteContextStruct(GetSchema()->GetExecuteContextStruct()))
			{
				ReportErrorf(TEXT("Cannot add node for function '%s' - incompatible execute context: '%s' vs '%s'."),
						*Function->GetName(),
						*Function->GetExecuteContextStruct()->GetStructCPPName(),
						*GetSchema()->GetExecuteContextStruct()->GetStructCPPName());
				return nullptr;
			}
		}
	}

	FString StructureError;
	if (!FRigVMStruct::ValidateStruct(InScriptStruct, &StructureError))
	{
		ReportErrorf(TEXT("Failed to validate struct '%s': %s"), *InScriptStruct->GetName(), *StructureError);
		return nullptr;
	}

	if(const FRigVMTemplate* Template = Function->GetTemplate())
	{
		if(bSetupUndoRedo)
		{
			OpenUndoBracket(FString::Printf(TEXT("Add %s Node"), *Template->GetNodeName().ToString()));
		}

		const FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? InScriptStruct->GetName() : InNodeName);
		URigVMUnitNode* TemplateNode = Cast<URigVMUnitNode>(AddTemplateNode(Template->GetNotation(), InPosition, Name, bSetupUndoRedo, bPrintPythonCommand));
		if(TemplateNode == nullptr)
		{
			CancelUndoBracket();
			return nullptr;
		}

		int32 PermutationIndex = Template->FindPermutation(Function);
		FRigVMTemplateTypeMap Types = Template->GetTypesForPermutation(PermutationIndex);
		for (TPair<FName, TRigVMTypeIndex>& Pair : Types)
		{
			if (URigVMPin* Pin = TemplateNode->FindPin(Pair.Key.ToString()))
			{
				if (Pin->IsWildCard())
				{
					FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset, true);
					ResolveWildCardPin(Pin, Pair.Value, bSetupUndoRedo);
				}
			}
			if (!TemplateNode->HasWildCardPin())
			{
				break;
			}
		}

		if (UnitNodeCreatedContext.IsValid())
		{
			if (TSharedPtr<FStructOnScope> StructScope = TemplateNode->ConstructStructInstance())
			{
				TGuardValue<FName> NodeNameScope(UnitNodeCreatedContext.NodeName, TemplateNode->GetFName());
				FRigVMStruct* StructInstance = (FRigVMStruct*)StructScope->GetStructMemory();
				StructInstance->OnUnitNodeCreated(UnitNodeCreatedContext);
			}
		}

		if(InDefaults.IsValid() && (InDefaults.GetScriptStruct() == InScriptStruct))
		{
			if (!SetUnitNodeDefaults(TemplateNode, InDefaults, bSetupUndoRedo, bPrintPythonCommand))
			{
				CancelUndoBracket();
				return nullptr;
			}
		}

		if(bSetupUndoRedo)
		{
			CloseUndoBracket();
		}

		return TemplateNode;
	}

	// Use the defaults if they are passed
	FRigVMStruct* StructMemory = nullptr;
	FStructOnScope StructOnScope(InScriptStruct);
	if (InDefaults.IsValid())
	{
		StructMemory = (FRigVMStruct*)InDefaults.GetMemory();
	}
	else
	{
		StructMemory = (FRigVMStruct*)StructOnScope.GetStructMemory();
	}
	
	const bool bIsEventNode = (!StructMemory->GetEventName().IsNone());
	if (bIsEventNode)
	{
		// don't allow event nodes in anything but top level graphs
		if (!Graph->IsTopLevelGraph())
		{
			ReportAndNotifyError(TEXT("Event nodes can only be added to top level graphs."));
			return nullptr;
		}

		// make sure to check all graphs when adding an event - we cannot
		// have the same event twice under two graphs within the client.
		if(GetAllEventNames().Contains(StructMemory->GetEventName()))
		{
			ReportAndNotifyWarningf(TEXT("Event %s already exists in the graph."), *StructMemory->GetEventName().ToString());
			return nullptr;
		}

		if(StructMemory->CanOnlyExistOnce())
		{
			// don't allow several event nodes in the main graph
			TObjectPtr<URigVMNode> EventNode = GetSchema()->FindEventNode(this, InScriptStruct);
			if (EventNode != nullptr)
			{
				const FString ErrorMessage = FString::Printf(TEXT("Rig Graph can only contain one single %s node."),
																*InScriptStruct->GetDisplayNameText().ToString());
				ReportAndNotifyError(ErrorMessage);
				return Cast<URigVMUnitNode>(EventNode);
			}
		}
	}
	
	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? InScriptStruct->GetName() : InNodeName);
	URigVMUnitNode* Node = NewObject<URigVMUnitNode>(Graph, InUnitNodeClass, *Name);
	Node->ResolvedFunctionName = Function->GetName();
	Node->Position = InPosition;
	Node->NodeTitle = InScriptStruct->GetMetaData(TEXT("DisplayName"));
	
	FString NodeColorMetadata;
	InScriptStruct->GetStringMetaDataHierarchical(URigVMNode::NodeColorName, &NodeColorMetadata);
	if (!NodeColorMetadata.IsEmpty())
	{
		Node->NodeColor = GetColorFromMetadata(NodeColorMetadata);
		Node->NodeColorType = ERigVMNodeColorType::FromMetadata;
	}

	FString ExportedDefaultValue;
	if (InDefaults.IsValid())
	{
		InDefaults.GetScriptStruct()->ExportText(ExportedDefaultValue, InDefaults.GetMemory(), nullptr, nullptr, (PPF_ExternalEditor | PPF_IncludeTransient), nullptr, true);
	}
	else
	{
		CreateDefaultValueForStructIfRequired(InScriptStruct, ExportedDefaultValue);
	}
	{
		FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset, true);
		TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
		AddPinsForStruct(InScriptStruct, Node, nullptr, ERigVMPinDirection::Invalid, ExportedDefaultValue, true);
	}

	if(!AddGraphNode(Node, true))
	{
		return nullptr;
	}
	
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);

	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add %s Node"), *Node->GetNodeTitle()));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}

	if (UnitNodeCreatedContext.IsValid())
	{
		if (TSharedPtr<FStructOnScope> StructScope = Node->ConstructStructInstance())
		{
			TGuardValue<FName> NodeNameScope(UnitNodeCreatedContext.NodeName, Node->GetFName());
			FRigVMStruct* StructInstance = (FRigVMStruct*)StructScope->GetStructMemory();
			StructInstance->OnUnitNodeCreated(UnitNodeCreatedContext);
		}
	}

	Node->IncrementVersion();

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
				FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

bool URigVMController::SetUnitNodeDefaults(URigVMUnitNode* InNode, const FString& InDefaults, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if(InNode == nullptr)
	{
		return false; 
	}

	UScriptStruct* ScriptStruct = InNode->GetScriptStruct();
	if(ScriptStruct == nullptr)
	{
		return false;
	}
	
	FStructOnScope StructOnScope(ScriptStruct);
	FRigVMPinDefaultValueImportErrorContext ErrorPipe;
	ScriptStruct->ImportText(*InDefaults, StructOnScope.GetStructMemory(), nullptr, PPF_None, &ErrorPipe, FString());

	if(ErrorPipe.NumErrors > 0)
	{
		return false;
	}

	return SetUnitNodeDefaults(InNode, StructOnScope, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetUnitNodeDefaults(URigVMUnitNode* InNode, const FRigStructScope& InDefaults,
                                           bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(InNode == nullptr || !InDefaults.IsValid())
	{
		return false;
	}

	if(InNode->GetScriptStruct() != InDefaults.GetScriptStruct())
	{
		return false;
	}

	static constexpr TCHAR SetUnitNodeDefaultsTitle[] = TEXT("Set Unit Node Defaults");
	OpenUndoBracket(SetUnitNodeDefaultsTitle);

	for(URigVMPin* Pin : InNode->GetPins())
	{
		if(Pin->GetDirection() != ERigVMPinDirection::Input &&
			Pin->GetDirection() != ERigVMPinDirection::IO &&
			Pin->GetDirection() != ERigVMPinDirection::Visible)
		{
			continue;
		}
		
		if(const FProperty* Property = InDefaults.GetScriptStruct()->FindPropertyByName(Pin->GetFName()))
		{
			const uint8* MemberMemoryPtr = Property->ContainerPtrToValuePtr<uint8>(InDefaults.GetMemory());
			const FString NewDefault = FRigVMStruct::ExportToFullyQualifiedText(Property, MemberMemoryPtr);
			if(NewDefault != Pin->GetDefaultValue())
			{
				FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::AutoDetect);
				SetPinDefaultValue(Pin->GetPinPath(), NewDefault, true, bSetupUndoRedo, false, bPrintPythonCommand);
			}
		}
	}

	CloseUndoBracket();
	return true;
}

URigVMVariableNode* URigVMController::AddVariableNode(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return AddVariableNode(InVariableName, URigVMVariableNode::StaticClass(), InCPPType, InCPPTypeObject, bIsGetter, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMVariableNode* URigVMController::AddVariableNode(const FName& InVariableName, TSubclassOf<URigVMVariableNode> InVariableNodeClass, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add variables nodes to function library graphs."));
		return nullptr;
	}

	// check if the operation will cause to dirty assets
	if(bSetupUndoRedo)
	{
		if(URigVMFunctionLibrary* OuterLibrary = Graph->GetTypedOuter<URigVMFunctionLibrary>())
		{
			if(URigVMLibraryNode* OuterFunction = OuterLibrary->FindFunctionForNode(Graph->GetTypedOuter<URigVMCollapseNode>()))
			{
				// Make sure there is no local variable with that name
				bool bFoundLocalVariable = false;
				for (FRigVMGraphVariableDescription& LocalVariable : OuterFunction->GetContainedGraph()->GetLocalVariables(true))
				{
					if (LocalVariable.Name == InVariableName)
					{
						bFoundLocalVariable = true;
						break;
					}
				}

				if (!bFoundLocalVariable)
				{
					for (FRigVMGraphVariableDescription& InputVariable : OuterFunction->GetContainedGraph()->GetInputVariables())
					{
						if (InputVariable.Name == InVariableName)
						{
							bFoundLocalVariable = true;
							break;
						}
					}
				}
				
				if (!bFoundLocalVariable)
				{
					// Make sure there is no external variable with that name
					TArray<FRigVMExternalVariable> ExternalVariables = OuterFunction->GetContainedGraph()->GetExternalVariables();
					bool bFoundExternalVariable = false;
					for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
					{
						if(ExternalVariable.GetName() == InVariableName)
						{
							bFoundExternalVariable = true;
							break;
						}
					}

					if(!bFoundExternalVariable)
					{
						// Warn the user the changes are not undoable
						if(RequestBulkEditDialogDelegate.IsBound())
						{
							FRigVMController_BulkEditResult Result = RequestBulkEditDialogDelegate.Execute(OuterFunction, ERigVMControllerBulkEditType::AddVariable);
							if(Result.bCanceled)
							{
								return nullptr;
							}
							bSetupUndoRedo = Result.bSetupUndoRedo;
						}
					}
				}
			}
		}
	}

	if (InCPPTypeObject == nullptr)
	{
		InCPPTypeObject = URigVMCompiler::GetScriptStructForCPPType(InCPPType);
	}
	if (InCPPTypeObject == nullptr)
	{
		InCPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPType);
	}

	FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, InCPPTypeObject);

	FRigVMExternalVariable ExternalVariable = GetVariableByName(InVariableName, true);
	if(!ExternalVariable.IsValid(true))
	{
		static constexpr TCHAR Format[] = TEXT("Cannot add variable '%s' with type '%s' - variable does not exist.");
		ReportErrorf(Format, *InVariableName.ToString(), *CPPType);
		return nullptr;
	}

	if(!GetSchema()->SupportsExternalVariable(this, &ExternalVariable))
	{
		return nullptr;
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("VariableNode")) : InNodeName);
	URigVMVariableNode* Node = NewObject<URigVMVariableNode>(Graph, InVariableNodeClass.Get(), *Name);
	Node->VariableGuid = ExternalVariable.GetGuid();
	Node->Position = InPosition;

	if (!bIsGetter)
	{
		URigVMPin* ExecutePin = MakeExecutePin(Node, FRigVMStruct::ExecuteContextName);
		ExecutePin->Direction = ERigVMPinDirection::IO;
		AddNodePin(Node, ExecutePin);
	}

	URigVMPin* VariablePin = NewObject<URigVMPin>(Node, URigVMVariableNode::VariableName);
	VariablePin->CPPType = RigVMTypeUtils::FNameType;
	VariablePin->Direction = ERigVMPinDirection::Hidden;
	VariablePin->DefaultValue = InVariableName.ToString();
	VariablePin->CustomWidgetName = TEXT("VariableName");
	AddNodePin(Node, VariablePin);

	URigVMPin* ValuePin = NewObject<URigVMPin>(Node, URigVMVariableNode::ValueName);
	ValuePin->CPPType = ExternalVariable.GetExtendedCPPType().ToString();
	ValuePin->CPPTypeObject = ExternalVariable.GetCPPTypeObject();
	if (ValuePin->CPPTypeObject)
	{
		ValuePin->CPPTypeObjectPath = *ValuePin->CPPTypeObject->GetPathName();
	}
	ValuePin->bIsDynamicArray = ExternalVariable.IsArray();

	ValuePin->Direction = bIsGetter ? ERigVMPinDirection::Output : ERigVMPinDirection::Input;
	AddNodePin(Node, ValuePin);

	if(!AddGraphNode(Node, false))
	{
		return nullptr;
	}

	if (ValuePin->IsStruct())
	{
		FString DefaultValue = InDefaultValue;
		CreateDefaultValueForStructIfRequired(ValuePin->GetScriptStruct(), DefaultValue);
		{
			TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
			AddPinsForStruct(ValuePin->GetScriptStruct(), Node, ValuePin, ValuePin->Direction, DefaultValue, false);
		}
	}
	else if (!InDefaultValue.IsEmpty() && InDefaultValue != TEXT("()"))
	{
		SetPinDefaultValue(ValuePin, InDefaultValue, true, false, false);
	}

	ForEveryPinRecursively(Node, [](URigVMPin* Pin) {
		Pin->bIsExpanded = false;
	});

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add %s Variable"), *InVariableName.ToString()));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}

	Notify(ERigVMGraphNotifType::NodeAdded, Node);
	Notify(ERigVMGraphNotifType::VariableAdded, Node);

	if (URigVMLibraryNode* LibraryNode = Cast<URigVMNode>(Node)->FindFunctionForNode())
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			PatchExternalVariablesToArguments(bSetupUndoRedo);
		}
	}
	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
								FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

URigVMVariableNode* URigVMController::AddVariableNodeFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return nullptr;
		}
	}

	return AddVariableNode(InVariableName, InCPPType, CPPTypeObject, bIsGetter, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

void URigVMController::RefreshVariableNode(const FName& InNodeName, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins)
{
	const FRigVMExternalVariable Variable = GetVariableByName(InVariableName);
	RefreshVariableNode(InNodeName, Variable.GetGuid(), InVariableName, InCPPType, InCPPTypeObject, bSetupUndoRedo, bSetupOrphanPins);
}

void URigVMController::RefreshVariableNode(const FName& InNodeName, const FGuid& InVariableGuid, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins)
{
	if (!IsValidGraph())
	{
		return;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return;
	}
	
	if (!InVariableGuid.IsValid())
	{
		return;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Graph->FindNodeByName(InNodeName)))
	{
		if (VariableNode->VariableGuid != InVariableGuid)
		{
			if (bSetupUndoRedo)
			{
				VariableNode->Modify();
			}
			VariableNode->VariableGuid = InVariableGuid;
		}
		
		if (URigVMPin* VariablePin = VariableNode->FindPin(URigVMVariableNode::VariableName))
		{
			if (VariablePin->Direction == ERigVMPinDirection::Visible)
			{
				if (bSetupUndoRedo)
				{
					VariablePin->Modify();
				}
				VariablePin->Direction = ERigVMPinDirection::Hidden;
				Notify(ERigVMGraphNotifType::PinDirectionChanged, VariablePin);
			}

			if (InVariableName.IsValid() && VariablePin->DefaultValue != InVariableName.ToString())
			{
				SetPinDefaultValue(VariablePin, InVariableName.ToString(), false, bSetupUndoRedo, false);
				Notify(ERigVMGraphNotifType::PinDefaultValueChanged, VariablePin);
				Notify(ERigVMGraphNotifType::VariableRenamed, VariableNode);
			}

			if (!InCPPType.IsEmpty())
			{
				if (URigVMPin* ValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName))
				{
					if (ValuePin->CPPType != InCPPType || ValuePin->GetCPPTypeObject() != InCPPTypeObject)
					{
						if (bSetupUndoRedo)
						{
							ValuePin->Modify();
						}

						// if this is an unsupported datatype...
						if (InCPPType == FName(NAME_None).ToString())
						{
							RemoveNode(VariableNode, bSetupUndoRedo);
							return;
						}

						FString CPPTypeObjectPath;
						if(InCPPTypeObject)
						{
							CPPTypeObjectPath = InCPPTypeObject->GetPathName();
						}
						ChangePinType(ValuePin, InCPPType, *CPPTypeObjectPath, bSetupUndoRedo, bSetupOrphanPins);
					}
				}
			}
		}
	}
}

void URigVMController::OnExternalVariableRemoved(const FGuid& InVarGuid, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return;
	}

	if (!InVarGuid.IsValid())
	{
		return;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	// When transacting, the action stack will deal with the deletion of variable nodes
	if(GIsTransacting)
	{
		return;
	}

	for (const FRigVMGraphVariableDescription& LocalVariable : Graph->GetLocalVariables(true))
	{
		if (InVarGuid == LocalVariable.Guid)
		{
			return;
		}
	}
	
	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Remove Variable Nodes"));
	}

	TArray<URigVMNode*> Nodes = Graph->GetNodes();
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMVariableNode* VariableNode = ExactCast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableGuid() == InVarGuid)
			{
				RemoveNode(Node, bSetupUndoRedo, true);
				continue;
			}
		}
		else if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
			{
				FRigVMControllerCompileBracketScope CollapseCompileScope(CollapseController);
				TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);

				// call this function for the contained graph recursively 
				CollapseController->OnExternalVariableRemoved(InVarGuid, bSetupUndoRedo);
			}
		}			
	}

	if (bSetupUndoRedo)
	{
		CloseUndoBracket();
	}
}

bool URigVMController::OnExternalVariableRenamed(const FGuid& InVarGuid, const FName& InNewVarName, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (!InVarGuid.IsValid() || !InNewVarName.IsValid())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	for (const FRigVMGraphVariableDescription& LocalVariable : Graph->GetLocalVariables(true))
	{
		if (InVarGuid == LocalVariable.Guid)
		{
			return false;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Rename Variable Nodes"));
	}

	TArray<URigVMNode*> Nodes = Graph->GetNodes();
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableGuid() == InVarGuid)
			{
				RefreshVariableNode(Node->GetFName(), InVarGuid, InNewVarName, FString(), nullptr, bSetupUndoRedo, false);
				continue;
			}
		}
		else if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
			{
				FRigVMControllerCompileBracketScope CollapseCompileScope(CollapseController);
				TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);
				CollapseController->OnExternalVariableRenamed(InVarGuid, InNewVarName, bSetupUndoRedo);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return true;
}

void URigVMController::OnExternalVariableTypeChanged(const FName& InVarName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo)
{
	FRigVMExternalVariable Variable = GetVariableByName(InVarName);
	if (Variable.GetGuid().IsValid())
	{
		OnExternalVariableTypeChanged(Variable.GetGuid(), InCPPType, InCPPTypeObject, bSetupUndoRedo);
	}
}

void URigVMController::OnExternalVariableTypeChanged(const FGuid& InVarGuid, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return;
	}

	if (!InVarGuid.IsValid())
	{
		return;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	for (const FRigVMGraphVariableDescription& LocalVariable : Graph->GetLocalVariables(true))
	{
		if (InVarGuid == LocalVariable.Guid)
		{
			return;
		}
	}

	const FRigVMExternalVariable Variable = GetVariableByGuid(InVarGuid);

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Change Variable Nodes Type"));
	}

	TArray<URigVMNode*> Nodes = Graph->GetNodes();
	for (URigVMNode* Node : Nodes)
	{
		if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			const bool bMatches = InVarGuid.IsValid() && VariableNode->GetVariableGuid().IsValid() ? VariableNode->GetVariableGuid() == InVarGuid : VariableNode->GetVariableName() == Variable.GetName();
			if (bMatches) 
			{
				RefreshVariableNode(Node->GetFName(), InVarGuid, Variable.GetName(), InCPPType, InCPPTypeObject, bSetupUndoRedo, true);
				continue;
			}
		}
		else if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
			{
				FRigVMControllerCompileBracketScope CollapseCompileScope(CollapseController);
				TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);
				CollapseController->OnExternalVariableTypeChanged(InVarGuid, InCPPType, InCPPTypeObject, bSetupUndoRedo);
			}
		}

		TArray<URigVMPin*> AllPins = Node->GetAllPinsRecursively();
		for (URigVMPin* Pin : AllPins)
		{
			if (Pin->GetBoundVariableName() == Variable.GetName().ToString())
			{
				FString BoundVariablePath = Pin->GetBoundVariablePath();
				UnbindPinFromVariable(Pin, bSetupUndoRedo);
				// try to bind it again - maybe it can be bound (due to cast rules etc)
				BindPinToVariable(Pin, BoundVariablePath, bSetupUndoRedo);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		CloseUndoBracket();
	}
}

void URigVMController::OnExternalVariableTypeChangedFromObjectPath(const FName& InVarName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo)
{
	FRigVMExternalVariable Variable = GetVariableByName(InVarName);
	if (Variable.GetGuid().IsValid())
	{
		OnExternalVariableTypeChangedFromObjectPath(Variable.GetGuid(), InCPPType, InCPPTypeObjectPath, bSetupUndoRedo);
	}
}

void URigVMController::OnExternalVariableTypeChangedFromObjectPath(const FGuid& InVarGuid, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return;
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return;
		}
	}

	OnExternalVariableTypeChanged(InVarGuid, InCPPType, CPPTypeObject, bSetupUndoRedo);
}

URigVMVariableNode* URigVMController::ReplaceParameterNodeWithVariable(const FName& InNodeName, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMParameterNode* ParameterNode = Cast<URigVMParameterNode>(Graph->FindNodeByName(InNodeName)))
	{
		URigVMPin* ParameterValuePin = ParameterNode->FindPin(URigVMParameterNode::ValueName);
		check(ParameterValuePin);

		FRigVMGraphParameterDescription Description = ParameterNode->GetParameterDescription();
		
		URigVMVariableNode* VariableNode = AddVariableNode(
			InVariableName,
			InCPPType,
			InCPPTypeObject,
			ParameterValuePin->GetDirection() == ERigVMPinDirection::Output,
			ParameterValuePin->GetDefaultValue(),
			ParameterNode->GetPosition(),
			FString(),
			bSetupUndoRedo);

		if (VariableNode)
		{
			URigVMPin* VariableValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName);

			RewireLinks(
				ParameterValuePin,
				VariableValuePin,
				ParameterValuePin->GetDirection() == ERigVMPinDirection::Input,
				bSetupUndoRedo
			);

			RemoveNode(ParameterNode, bSetupUndoRedo, true);

			return VariableNode;
		}
	}

	return nullptr;
}

bool URigVMController::UnresolveTemplateNodes(const TArray<FName>& InNodeNames, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	TArray<URigVMNode*> Nodes;
	for (const FName& NodeName : InNodeNames)
	{
		if (URigVMNode* Node = GetGraph()->FindNodeByName(NodeName))
		{
			Nodes.Add(Node);
		}
	}

	if(UnresolveTemplateNodes(Nodes, bSetupUndoRedo))
	{
		if(bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			TArray<FString> NodeNames;
			for(const FName& NodeName : InNodeNames)
			{
				NodeNames.Add(GetSchema()->GetSanitizedNodeName(NodeName.ToString()));
			}
			const FString NodeNamesJoined = FString::Join(NodeNames, TEXT("','"));

			// UnresolveTemplateNodes(const TArray<FName>& InNodeNames)
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
					FString::Printf(TEXT("asset.get_controller_by_name('%s').unresolve_template_nodes(['%s'])"),
					*GraphName,
					*NodeNamesJoined));
		}

		return true;
	}

	return false;
}

bool URigVMController::UnresolveTemplateNodes(const TArray<URigVMNode*>& InNodes, bool bSetupUndoRedo)
{
	if (!IsValidGraph() || InNodes.IsEmpty())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	// check if any of the nodes needs to be unresolved
	const bool bHasNodeToResolve = InNodes.ContainsByPredicate( [](const URigVMNode* Node) -> bool
	{
		if (const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
		{
			if (!TemplateNode->IsSingleton())
			{
				return !TemplateNode->IsFullyUnresolved();
			}
		}
		return false;
	});
	if (!bHasNodeToResolve)
	{
		return false;
	}
	
	FRigVMBaseAction Action(this);
	if(bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Unresolve nodes"));
		GetActionStack()->BeginAction(Action);
	}

	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		for (URigVMNode* Node : InNodes)
		{
			EjectAllInjectedNodes(Node, bSetupUndoRedo);
			TArray<URigVMLink*> Links = Node->GetLinks();
			for (int32 i=0; i<Links.Num(); ++i)
			{
				URigVMLink* Link = Links[i];
				URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();

				URigVMNode* OtherNode = SourcePin->GetNode() == Node ? TargetPin->GetNode() : SourcePin->GetNode();
				if (!InNodes.Contains(OtherNode))
				{
					const URigVMPin* PinOnNode = SourcePin->GetNode() == Node ? SourcePin : TargetPin;
					if(PinOnNode->IsExecuteContext())
					{
						continue;
					}
					
					if (const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
					{
						if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
						{
							const URigVMPin* RootPin = PinOnNode->GetRootPin();
							if(const FRigVMTemplateArgument* Argument = Template->FindArgument(RootPin->GetFName()))
							{
								if(Argument->IsSingleton())
								{
									continue;
								}
							}
						}
					}
					
					BreakLink(SourcePin, TargetPin, bSetupUndoRedo);
				}
			}

			if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
			{
				TemplateNode->InvalidateCache();
				TemplateNode->ResolvedFunctionName.Reset();
				TemplateNode->ResolvedPermutation = INDEX_NONE;

				if (const FRigVMTemplate* Template = TemplateNode->GetTemplate())
				{
					for (int32 i=0; i<Template->NumArguments(); ++i)
					{
						const FRigVMTemplateArgument* Argument = Template->GetArgument(i);
						if (!Argument->IsSingleton())
						{
							if (URigVMPin* Pin = TemplateNode->FindPin(Argument->Name.ToString()))
							{
								TRigVMTypeIndex OldTypeIndex = Pin->GetTypeIndex();
								TRigVMTypeIndex NewTypeIndex = RigVMTypeUtils::TypeIndex::WildCard;
								while (Registry.IsArrayType(OldTypeIndex))
								{
									OldTypeIndex = Registry.GetBaseTypeFromArrayTypeIndex(OldTypeIndex);
									NewTypeIndex = Registry.GetArrayTypeFromBaseTypeIndex(NewTypeIndex);
								}
								ChangePinType(Pin, NewTypeIndex, bSetupUndoRedo, false, true, false);
							}
						}
					}
					UpdateTemplateNodePinTypes(TemplateNode, bSetupUndoRedo);
				}
			}
		}
	}

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

TArray<URigVMNode*> URigVMController::UpgradeNodes(const TArray<FName>& InNodeNames, bool bRecursive, bool bSetupUndoRedo,
                                                   bool bPrintPythonCommand)
{
	TArray<URigVMNode*> Nodes;
	if (!IsValidGraph())
	{
		return Nodes;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return Nodes;
	}

	for (const FName& NodeName : InNodeNames)
	{
		if (URigVMNode* Node = GetGraph()->FindNodeByName(NodeName))
		{
			Nodes.Add(Node);
		}
	}

	Nodes = UpgradeNodes(Nodes, bRecursive, bSetupUndoRedo);

	if(bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		TArray<FString> NodeNames;
		for(const FName& NodeName : InNodeNames)
		{
			NodeNames.Add(GetSchema()->GetSanitizedNodeName(NodeName.ToString()));
		}
		const FString NodeNamesJoined = FString::Join(NodeNames, TEXT("','"));

		// UpgradeNodes(const TArray<FName>& InNodeNames)
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
				FString::Printf(TEXT("asset.get_controller_by_name('%s').upgrade_nodes(['%s'])"),
				*GraphName,
				*NodeNamesJoined));
	}

	// log a warning for all nodes which are still marked deprecated
	for(URigVMNode* Node : Nodes)
	{
		if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
		{
			if(UnitNode->IsOutDated())
			{
				ReportWarningf(TEXT("Node %s cannot be upgraded. There is no automatic upgrade path available."), *UnitNode->GetNodePath());
			}
		}
	}

	return Nodes;
}

TArray<URigVMNode*> URigVMController::UpgradeNodes(const TArray<URigVMNode*>& InNodes, bool bRecursive, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return TArray<URigVMNode*>();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return TArray<URigVMNode*>();
	}

	bool bFoundAnyNodeToUpgrade = false;
	for(URigVMNode* Node : InNodes)
	{
		if(!IsValidNodeForGraph(Node))
		{
			return TArray<URigVMNode*>();
		}

		bFoundAnyNodeToUpgrade |= Node->CanBeUpgraded();
	}

	if(!bFoundAnyNodeToUpgrade)
	{
		return InNodes;
	}

	FRigVMBaseAction Action(this);
	if(bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Upgrade nodes"));
		GetActionStack()->BeginAction(Action);
	}

	FScopedSlowTask UpgradeNodesSlowTask((float)(InNodes.Num() + 2), NSLOCTEXT("RigVMController", "UpgradeNodes", "Upgrade Nodes"));
	UpgradeNodesSlowTask.MakeDialog(true);

	UpgradeNodesSlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("RigVMController", "DetachingLinks", "Detaching Links"));
	UpgradeNodesSlowTask.ForceRefresh();

	// find all links affecting the nodes to upgrade
	TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InNodes, true);
	if(!FastBreakLinkedPaths(LinkedPaths, bSetupUndoRedo))
	{
		if(bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return TArray<URigVMNode*>();
	}

	FRestoreLinkedPathSettings Settings;

	TArray<URigVMNode*> UpgradedNodes;
	TMap<FString,FRigVMController_PinPathRemapDelegate> RemapPinDelegates;
	for(URigVMNode* Node : InNodes)
	{
		const FName OriginalName = Node->GetFName();

		UpgradeNodesSlowTask.EnterProgressFrame(1.0f, FText::FromString(FString::Printf(TEXT("Upgrading node '%s'"), *OriginalName.ToString())));
		UpgradeNodesSlowTask.ForceRefresh();
		
		FRigVMController_PinPathRemapDelegate RemapPinDelegate;
		URigVMNode* UpgradedNode = UpgradeNode(Node, bSetupUndoRedo, &RemapPinDelegate);
		if(UpgradedNode)
		{
			const FName NewName = UpgradedNode->GetFName();
			if(NewName != OriginalName)
			{
				Settings.NodeNameMap.Add(OriginalName.ToString(), NewName.ToString());
			}
			
			UpgradedNodes.Add(UpgradedNode);
			if(RemapPinDelegate.IsBound())
			{
				RemapPinDelegates.Add(UpgradedNode->GetName(), RemapPinDelegate);
			}
		}
	}

	UpgradeNodesSlowTask.EnterProgressFrame(1.0f, NSLOCTEXT("RigVMController", "RestoringLinks", "Restoring Links"));
	UpgradeNodesSlowTask.ForceRefresh();

	Settings.RemapDelegates = RemapPinDelegates;
	RestoreLinkedPaths(LinkedPaths, Settings, bSetupUndoRedo);

	if(bRecursive)
	{
		UpgradedNodes = UpgradeNodes(UpgradedNodes, bRecursive, bSetupUndoRedo);
	}

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return UpgradedNodes;
}

URigVMNode* URigVMController::UpgradeNode(URigVMNode* InNode, bool bSetupUndoRedo, FRigVMController_PinPathRemapDelegate* OutRemapPinDelegate)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return nullptr;
	}

	if(!InNode->CanBeUpgraded())
	{
		return InNode; 
	}

	TMap<FString, FString> RedirectedPinPaths;
	TMap<FString, FPinState> PinStates = GetPinStates(InNode, true);
	EjectAllInjectedNodes(InNode, bSetupUndoRedo);

	const FString NodeName = InNode->GetName();
	const FString NodeTitle = InNode->GetNodeTitle();
	const FVector2D NodePosition = InNode->GetPosition();

	FRigVMBaseAction Action(this);
	if(bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Upgrade node"));
		GetActionStack()->BeginAction(Action);
	}

	URigVMNode* UpgradedNode = nullptr;

	const FRigVMStructUpgradeInfo UpgradeInfo = InNode->GetUpgradeInfo();
	check(UpgradeInfo.IsValid());
	
	FName MethodName = TEXT("Execute");
	if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode))
	{
		MethodName = UnitNode->GetMethodName();
	}

	if(OutRemapPinDelegate)
	{
		*OutRemapPinDelegate = FRigVMController_PinPathRemapDelegate::CreateLambda([UpgradeInfo](const FString& InPinPath, bool bIsInput) -> FString
		{
			return UpgradeInfo.RemapPin(InPinPath, bIsInput, true);
		});
	}

	if(!RemoveNode(InNode, bSetupUndoRedo, false))
	{
		if(bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		ReportErrorf(TEXT("Unable to remove node %s."), *NodeName);
		return nullptr;
	}

	URigVMNode* NewNode = nullptr;
	if(UpgradeInfo.GetNewStruct()->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		NewNode = AddUnitNode(UpgradeInfo.GetNewStruct(), MethodName, NodePosition, NodeName, bSetupUndoRedo, false);
	}
	else if(UpgradeInfo.GetNewStruct()->IsChildOf(FRigVMDispatchFactory::StaticStruct()) && !UpgradeInfo.NewDispatchFunction.IsNone())
	{
		if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(*UpgradeInfo.NewDispatchFunction.ToString()))
		{
			if(const FRigVMTemplate* Template = Function->GetTemplate())
			{
				if(const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory())
				{
					if(Factory->GetScriptStruct() == UpgradeInfo.GetNewStruct())
					{
						NewNode = AddTemplateNode(Template->GetNotation(), NodePosition, NodeName, bSetupUndoRedo, false);
						if(NewNode)
						{
							for(int32 ArgumentIndex=0;ArgumentIndex<Function->GetArguments().Num();ArgumentIndex++)
							{
								const FRigVMFunctionArgument& Argument = Function->GetArguments()[ArgumentIndex];
								if(URigVMPin* Pin = NewNode->FindPin(Argument.Name))
								{
									if(Pin->IsWildCard())
									{
										ResolveWildCardPin(Pin, Function->GetArgumentTypeIndices()[ArgumentIndex], bSetupUndoRedo, false);
									}
								}
							}
						}
					}
				}
			}
		}
	}
	
	if(NewNode == nullptr)
	{
		if(bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		ReportErrorf(TEXT("Unable to upgrade node %s."), *NodeName);
		return nullptr;
	}

	const TArray<FString>& AggregatePins = UpgradeInfo.GetAggregatePins();
	for(const FString& AggregatePinName : AggregatePins)
	{
		const FName PreviousName = NewNode->GetFName();
		AddAggregatePin(PreviousName.ToString(), AggregatePinName, FString(), bSetupUndoRedo, false);
		NewNode = GetGraph()->FindNodeByName(PreviousName);
	}

	for(URigVMPin* Pin : NewNode->GetPins())
	{
		const FString DefaultValue = UpgradeInfo.GetDefaultValueForPin(Pin->GetFName());
		if(!DefaultValue.IsEmpty())
		{
			SetPinDefaultValue(Pin, DefaultValue, true, bSetupUndoRedo, false);

			if(FPinState* PinState = PinStates.Find(Pin->GetPinPath()))
			{
				PinState->DefaultValue.Reset();
			}
		}
	}

	// redirect pin state paths
	for(TPair<FString, FPinState>& PinState : PinStates)
	{
		for(int32 TrueFalse = 0; TrueFalse < 2; TrueFalse++)
		{
			const FString RemappedInputPath = UpgradeInfo.RemapPin(PinState.Key, TrueFalse == 0, false);
			if(RemappedInputPath != PinState.Key)
			{
				if(!RedirectedPinPaths.Contains(PinState.Key))
				{
					RedirectedPinPaths.Add(PinState.Key, RemappedInputPath);
				}
			}
		}
	}

	UpgradedNode = NewNode;
	check(UpgradedNode);

	ApplyPinStates(UpgradedNode, PinStates, RedirectedPinPaths, bSetupUndoRedo);

	if(!NodeTitle.IsEmpty() && UpgradedNode->SupportsRenaming())
	{
		(void)RenameNode(UpgradedNode, *NodeTitle, bSetupUndoRedo, false);
	}

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return UpgradedNode;
}

URigVMParameterNode* URigVMController::AddParameterNode(const FName& InParameterName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsInput, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	AddVariableNode(InParameterName, InCPPType, InCPPTypeObject, bIsInput, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
	ReportWarning(TEXT("AddParameterNode has been deprecated. Adding a variable node instead."));
	return nullptr;
}

URigVMParameterNode* URigVMController::AddParameterNodeFromObjectPath(const FName& InParameterName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsInput, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return nullptr;
		}
	}

	return AddParameterNode(InParameterName, InCPPType, CPPTypeObject, bIsInput, InDefaultValue, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMCommentNode* URigVMController::AddCommentNode(const FString& InCommentText, const FVector2D& InPosition, const FVector2D& InSize, const FLinearColor& InColor, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add comment nodes to function library graphs."));
		return nullptr;
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("CommentNode")) : InNodeName);
	URigVMCommentNode* Node = NewObject<URigVMCommentNode>(Graph, *Name);
	Node->Position = InPosition;
	Node->Size = InSize;
	Node->NodeColor = InColor;
	Node->NodeColorType = ERigVMNodeColorType::UserDefined;
	Node->CommentText = InCommentText;

	if(!AddGraphNode(Node, false))
	{
		return nullptr;
	}
	
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Comment")));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}

	Notify(ERigVMGraphNotifType::NodeAdded, Node);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
								FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

URigVMRerouteNode* URigVMController::AddRerouteNodeOnLink(URigVMLink* InLink, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidLinkForGraph(InLink))
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if (GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add reroutes to function library graphs."));
		return nullptr;
	}

	URigVMPin* SourcePin = InLink->GetSourcePin();
	const URigVMPin* TargetPin = InLink->GetTargetPin();

	TGuardValue<bool> GuardCompactness(bIgnoreRerouteCompactnessChanges, true);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Reroute")));
		GetActionStack()->BeginAction(Action);
	}

	URigVMRerouteNode* Node = AddRerouteNodeOnPin(TargetPin->GetPinPath(), true, InPosition, InNodeName, bSetupUndoRedo);
	if (Node == nullptr)
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return nullptr;
	}

	URigVMPin* ValuePin = Node->Pins[0];
	AddLink(SourcePin, ValuePin, bSetupUndoRedo);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').add_reroute_node_on_link_path('%s', %s, '%s')"),
											*GraphName,
											*URigVMLink::GetPinPathRepresentation(SourcePin->GetPinPath(), TargetPin->GetPinPath()),
											*RigVMPythonUtils::Vector2DToPythonString(Node->GetPosition()),
											*NodeName));
	}

	return Node;
}

URigVMRerouteNode* URigVMController::AddRerouteNodeOnLinkPath(const FString& InLinkPinPathRepresentation, const FVector2D& InPosition, const FString&
                                                              InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMLink* Link = Graph->FindLink(InLinkPinPathRepresentation);
	return AddRerouteNodeOnLink(Link, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMRerouteNode* URigVMController::AddRerouteNodeOnPin(const FString& InPinPath, bool bAsInput, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add reroutes to function library graphs."));
		return nullptr;
	}

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if(Pin == nullptr)
	{
		return nullptr;
	}

	TGuardValue<bool> GuardCompactness(bIgnoreRerouteCompactnessChanges, true);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Reroute")));
		GetActionStack()->BeginAction(Action);
	}

	//in case an injected node is present, use its pins for any new links
	URigVMPin *PinForLink = Pin->GetPinForLink(); 
	if (bAsInput)
	{
		BreakAllLinks(PinForLink, bAsInput, bSetupUndoRedo);
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("RerouteNode")) : InNodeName);
	URigVMRerouteNode* Node = NewObject<URigVMRerouteNode>(Graph, *Name);
	Node->Position = InPosition;

	URigVMPin* ValuePin = NewObject<URigVMPin>(Node, URigVMRerouteNode::ValueName);
	ConfigurePinFromPin(ValuePin, Pin);
	ValuePin->Direction = ERigVMPinDirection::IO;
	AddNodePin(Node, ValuePin);

	FString DefaultValue = Pin->GetDefaultValue();
	if (!DefaultValue.IsEmpty())
	{
		FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Override, true);
		SetPinDefaultValue(ValuePin, Pin->GetDefaultValue(), true, false, false);
	}

	ForEveryPinRecursively(ValuePin, [](URigVMPin* Pin) {
		Pin->bIsExpanded = true;
	});

	if(!AddGraphNode(Node, true))
	{
		return nullptr;
	}
	
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node, false));
	}

	if (bAsInput)
	{
		AddLink(ValuePin, PinForLink, bSetupUndoRedo);
	}
	else
	{
		AddLink(PinForLink, ValuePin, bSetupUndoRedo);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());
		// AddRerouteNodeOnPin(const FString& InPinPath, bool bAsInput, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').add_reroute_node_on_pin('%s', %s, %s, '%s')"),
											*GraphName,
											*GetSchema()->GetSanitizedPinPath(InPinPath),
											(bAsInput) ? TEXT("True") : TEXT("False"),
											*RigVMPythonUtils::Vector2DToPythonString(Node->GetPosition()),
											*NodeName));
	}

	return Node;
}

URigVMInjectionInfo* URigVMController::AddInjectedNode(const FString& InPinPath, bool bAsInput, UScriptStruct* InScriptStruct, const FName& InMethodName, const FName& InInputPinName, const FName& InOutputPinName, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add injected nodes to function library graphs."));
		return nullptr;
	}

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		return nullptr;
	}

	if (Pin->IsArray())
	{
		return nullptr;
	}

	if (bAsInput && !(Pin->GetDirection() == ERigVMPinDirection::Input || Pin->GetDirection() == ERigVMPinDirection::IO))
	{
		ReportError(TEXT("Pin is not an input / cannot add injected input node."));
		return nullptr;
	}
	if (!bAsInput && !(Pin->GetDirection() == ERigVMPinDirection::Output))
	{
		ReportError(TEXT("Pin is not an output / cannot add injected output node."));
		return nullptr;
	}

	if (InScriptStruct == nullptr)
	{
		ReportError(TEXT("InScriptStruct is null."));
		return nullptr;
	}

	if (InMethodName == NAME_None)
	{
		ReportError(TEXT("InMethodName is None."));
		return nullptr;
	}

	// find the input and output pins to use
	FProperty* InputProperty = InScriptStruct->FindPropertyByName(InInputPinName);
	if (InputProperty == nullptr)
	{
		ReportErrorf(TEXT("Cannot find property '%s' on struct type '%s'."), *InInputPinName.ToString(), *InScriptStruct->GetName());
		return nullptr;
	}
	if (!InputProperty->HasMetaData(FRigVMStruct::InputMetaName))
	{
		ReportErrorf(TEXT("Property '%s' on struct type '%s' is not marked as an input."), *InInputPinName.ToString(), *InScriptStruct->GetName());
		return nullptr;
	}
	FProperty* OutputProperty = InScriptStruct->FindPropertyByName(InOutputPinName);
	if (OutputProperty == nullptr)
	{
		ReportErrorf(TEXT("Cannot find property '%s' on struct type '%s'."), *InOutputPinName.ToString(), *InScriptStruct->GetName());
		return nullptr;
	}
	if (!OutputProperty->HasMetaData(FRigVMStruct::OutputMetaName))
	{
		ReportErrorf(TEXT("Property '%s' on struct type '%s' is not marked as an output."), *InOutputPinName.ToString(), *InScriptStruct->GetName());
		return nullptr;
	}

	// 1.- Create unit node
	// 2.- Rewire links
	// 3.- Inject node into pin

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Injected Node")));
		GetActionStack()->BeginAction(Action);
	}

	// 1.- Create unit node
	URigVMUnitNode* UnitNode = nullptr;
	URigVMPin* InputPin = nullptr;
	URigVMPin* OutputPin = nullptr;
	{
		{
			TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
			UnitNode = AddUnitNode(InScriptStruct, InMethodName, FVector2D::ZeroVector, InNodeName, bSetupUndoRedo);
		}
		if (UnitNode == nullptr)
		{
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return nullptr;
		}
		else if (UnitNode->IsMutable())
		{
			ReportErrorf(TEXT("Injected node %s is mutable."), *InScriptStruct->GetName());
			RemoveNode(UnitNode, false);
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return nullptr;
		}

		InputPin = UnitNode->FindPin(InInputPinName.ToString());
		check(InputPin);
		OutputPin = UnitNode->FindPin(InOutputPinName.ToString());
		check(OutputPin);

		if (InputPin->GetCPPType() != OutputPin->GetCPPType() ||
			InputPin->IsArray() != OutputPin->IsArray())
		{
			ReportErrorf(TEXT("Injected node %s is using incompatible input and output pins."), *InScriptStruct->GetName());
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return nullptr;
		}

		if (InputPin->GetCPPType() != Pin->GetCPPType() ||
			InputPin->IsArray() != Pin->IsArray())
		{
			ReportErrorf(TEXT("Injected node %s is using incompatible pin."), *InScriptStruct->GetName());
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return nullptr;
		}
	}

	// 2.- Rewire links
	TArray<URigVMLink*> NewLinks;
	{
		URigVMPin* PreviousInputPin = Pin;
		URigVMPin* PreviousOutputPin = Pin;
		if (Pin->InjectionInfos.Num() > 0)
		{
			PreviousInputPin = Pin->InjectionInfos.Last()->InputPin;
			PreviousOutputPin = Pin->InjectionInfos.Last()->OutputPin;
		}
		if (bAsInput)
		{
			FString PinDefaultValue = PreviousInputPin->GetDefaultValue();
			if (!PinDefaultValue.IsEmpty())
			{
				SetPinDefaultValue(InputPin, PinDefaultValue, true, bSetupUndoRedo, false);
			}
			TArray<URigVMLink*> Links = PreviousInputPin->GetSourceLinks(true /* recursive */);
			if (Links.Num() > 0)
			{
				RewireLinks(PreviousInputPin, InputPin, true, bSetupUndoRedo, Links);
				NewLinks = InputPin->GetSourceLinks();
			}
			AddLink(OutputPin, PreviousInputPin, bSetupUndoRedo);
		}
		else
		{
			TArray<URigVMLink*> Links = PreviousOutputPin->GetTargetLinks(true /* recursive */);
			if (Links.Num() > 0)
			{
				RewireLinks(PreviousOutputPin, OutputPin, false, bSetupUndoRedo, Links);
				NewLinks = OutputPin->GetTargetLinks();
			}
			AddLink(PreviousOutputPin, InputPin, bSetupUndoRedo);
		}
	}

	// 3.- Inject node into pin
	URigVMInjectionInfo* InjectionInfo = InjectNodeIntoPin(InPinPath, bAsInput, InInputPinName, InOutputPinName, bSetupUndoRedo);

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}
	
	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').add_injected_node_from_struct_path('%s', %s, '%s', '%s', '%s', '%s', '%s')"),
											*GraphName,
											*GetSchema()->GetSanitizedPinPath(InPinPath),
											(bAsInput) ? TEXT("True") : TEXT("False"),
											*InScriptStruct->GetPathName(),
											*InMethodName.ToString(),
											*GetSchema()->GetSanitizedPinName(InInputPinName.ToString()),
											*GetSchema()->GetSanitizedPinName(InOutputPinName.ToString()),
											*GetSchema()->GetSanitizedNodeName(InNodeName)));
	}

	return InjectionInfo;

}

URigVMInjectionInfo* URigVMController::AddInjectedNodeFromStructPath(const FString& InPinPath, bool bAsInput, const FString& InScriptStructPath, const FName& InMethodName, const FName& InInputPinName, const FName& InOutputPinName, const FString& InNodeName, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	UScriptStruct* ScriptStruct = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UScriptStruct>(InScriptStructPath);
	if (ScriptStruct == nullptr)
	{
		ReportErrorf(TEXT("Cannot find struct for path '%s'."), *InScriptStructPath);
		return nullptr;
	}

	return AddInjectedNode(InPinPath, bAsInput, ScriptStruct, InMethodName, InInputPinName, InOutputPinName, InNodeName, bSetupUndoRedo);
}

bool URigVMController::RemoveInjectedNode(const FString& InPinPath, bool bAsInput, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add injected nodes to function library graphs."));
		return false;
	}

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		return false;
	}

	if (!Pin->HasInjectedNodes())
	{
		return false;
	}


	// 1.- Eject node
	// 2.- Rewire links
	// 3.- Remove node

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Remove Injected Node")));
		GetActionStack()->BeginAction(Action);
	}

	URigVMInjectionInfo* InjectionInfo = Pin->InjectionInfos.Last();
	URigVMPin* InputPin = InjectionInfo->InputPin;
	URigVMPin* OutputPin = InjectionInfo->OutputPin;

	// 1.- Eject node
	URigVMNode* NodeEjected = EjectNodeFromPin(InPinPath, bSetupUndoRedo);
	if (!NodeEjected)
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return false;
	}

	// 2.- Rewire links
	if (bAsInput)
	{
		BreakLink(OutputPin, Pin, bSetupUndoRedo);
		if (InputPin)
		{
			TArray<URigVMLink*> Links = InputPin->GetSourceLinks();
			RewireLinks(InputPin, Pin, true, bSetupUndoRedo, Links);
		}
	}
	else
	{
		BreakLink(Pin, InputPin, bSetupUndoRedo);
		TArray<URigVMLink*> Links = InputPin->GetTargetLinks();
		RewireLinks(OutputPin, Pin, false, bSetupUndoRedo, Links);
	}
	
	// 3.- Remove node
	if (!RemoveNode(NodeEjected, bSetupUndoRedo, false))
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return false;
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}
	
	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_injected_node('%s', %s)"),
											*GraphName,
											*GetSchema()->GetSanitizedPinPath(InPinPath),
											(bAsInput) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

URigVMInjectionInfo* URigVMController::InjectNodeIntoPin(const FString& InPinPath, bool bAsInput, const FName& InInputPinName, const FName& InOutputPinName, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (!Pin)
	{
		return nullptr;
	}

	return InjectNodeIntoPin(Pin, bAsInput, InInputPinName, InOutputPinName, bSetupUndoRedo);
}

URigVMInjectionInfo* URigVMController::InjectNodeIntoPin(URigVMPin* InPin, bool bAsInput, const FName& InInputPinName, const FName& InOutputPinName, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}
	
	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot inject nodes in function library graphs."));
		return nullptr;
	}

	URigVMPin* PinForLink = InPin->GetPinForLink();

	URigVMNode* NodeToInject = nullptr;
	TArray<URigVMPin*> ConnectedPins = bAsInput ? PinForLink->GetLinkedSourcePins(true) : PinForLink->GetLinkedTargetPins(true);
	if (ConnectedPins.Num() < 1)
	{
		ReportErrorf(TEXT("Cannot find node connected to pin '%s' as %s."), *InPin->GetPinPath(), bAsInput ? TEXT("input") : TEXT("output"));
		return nullptr;
	}

	NodeToInject = ConnectedPins[0]->GetNode();
	for (int32 i = 1; i < ConnectedPins.Num(); ++i)
	{
		if (ConnectedPins[i]->GetNode() != NodeToInject)
		{
			ReportErrorf(TEXT("Found more than one node connected to pin '%s' as %s."), *InPin->GetPinPath(), bAsInput ? TEXT("input") : TEXT("output"));
			return nullptr;
		}
	}

	URigVMPin* InputPin = nullptr;
	URigVMPin* OutputPin = nullptr;
	if (NodeToInject->IsA<URigVMUnitNode>())
	{
		InputPin = NodeToInject->FindPin(InInputPinName.ToString());
		if (!InputPin)
		{
			ReportErrorf(TEXT("Could not find pin '%s' in node %s."), *InInputPinName.ToString(), *NodeToInject->GetNodePath());
			return nullptr;
		}
	}
	OutputPin = NodeToInject->FindPin(InOutputPinName.ToString());
	if (!OutputPin)
	{
		ReportErrorf(TEXT("Could not find pin '%s' in node %s."), *InOutputPinName.ToString(), *NodeToInject->GetNodePath());
		return nullptr;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Inject Node")));
		GetActionStack()->BeginAction(Action);
	}
	
	URigVMInjectionInfo* InjectionInfo = NewObject<URigVMInjectionInfo>(InPin);
	{
		Notify(ERigVMGraphNotifType::NodeRemoved, NodeToInject);
		
		// re-parent the unit node to be under the injection info
		RenameObject(NodeToInject, nullptr, InjectionInfo);
		
		InjectionInfo->Node = NodeToInject;
		InjectionInfo->bInjectedAsInput = bAsInput;
		InjectionInfo->InputPin = InputPin;
		InjectionInfo->OutputPin = OutputPin;
	
		InPin->InjectionInfos.Add(InjectionInfo);
		InPin->IncrementVersion();

		Notify(ERigVMGraphNotifType::NodeAdded, NodeToInject);
	}

	// Notify the change in links (after the node is injected)
	{
		TArray<URigVMLink*> NewLinks;
		if (bAsInput)
		{
			if (InputPin)
			{
				NewLinks = InputPin->GetSourceLinks();
			}
		}
		else
		{
			NewLinks = OutputPin->GetTargetLinks();
		}
		for (URigVMLink* Link : NewLinks)
		{
			Notify(ERigVMGraphNotifType::LinkAdded, Link);
		}
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMInjectNodeIntoPinAction(this, InjectionInfo));
		GetActionStack()->EndAction(Action);
	}

	return InjectionInfo;
}

URigVMNode* URigVMController::EjectNodeFromPin(const FString& InPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (!Pin)
	{
		return nullptr;
	}

	return EjectNodeFromPin(Pin, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMNode* URigVMController::EjectNodeFromPin(URigVMPin* InPin, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot eject nodes in function library graphs."));
		return nullptr;
	}

	if (!InPin->HasInjectedNodes())
	{
		ReportErrorf(TEXT("Pin '%s' has no injected nodes."), *InPin->GetPinPath());
		return nullptr;
	}

	URigVMInjectionInfo* Injection = InPin->InjectionInfos.Last();

	
	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Eject node"));

		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMEjectNodeFromPinAction(this, Injection));
	}

	FVector2D Position = InPin->GetNode()->GetPosition() + FVector2D(0.f, 12.f) * float(InPin->GetPinIndex());
	if (InPin->GetDirection() == ERigVMPinDirection::Output)
	{
		Position += FVector2D(250.f, 0.f);
	}
	else
	{
		Position -= FVector2D(250.f, 0.f);
	}


	URigVMNode* NodeToEject = Injection->Node;
	URigVMPin* InputPin = Injection->InputPin;
	URigVMPin* OutputPin = Injection->OutputPin;
	
	URigVMLink* LinkToRemove = nullptr;
	
	if (Injection->bInjectedAsInput)
	{
		if (InputPin)
		{
			TArray<URigVMLink*> SourceLinks = InputPin->GetSourceLinks(true);
			if (SourceLinks.Num() > 0)
			{
				LinkToRemove = SourceLinks[0];
				
			}
		}
	}
	else
	{
		TArray<URigVMLink*> TargetLinks = OutputPin->GetTargetLinks(true);
		if (TargetLinks.Num() > 0)
		{
			LinkToRemove = TargetLinks[0];
		}
	}
	
	
	RenameObject(NodeToEject, nullptr, Graph);
	SetNodePosition(NodeToEject, Position, false);
	InPin->InjectionInfos.Remove(Injection);
	InPin->IncrementVersion();
	DestroyObject(Injection);
	
	Notify(ERigVMGraphNotifType::NodeRemoved, NodeToEject);
	if (LinkToRemove)
	{
		Notify(ERigVMGraphNotifType::LinkRemoved, LinkToRemove);
	}

	Notify(ERigVMGraphNotifType::NodeAdded, NodeToEject);
	if (InputPin)
	{
		TArray<URigVMLink*> SourceLinks = InputPin->GetSourceLinks(true);
		if (SourceLinks.Num() > 0)
		{
			Notify(ERigVMGraphNotifType::LinkAdded, SourceLinks[0]);
		}
	}
	TArray<URigVMLink*> TargetLinks = OutputPin->GetTargetLinks(true);
	if (TargetLinks.Num() > 0)
	{
		Notify(ERigVMGraphNotifType::LinkAdded, TargetLinks[0]);
	}
		
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').eject_node_from_pin('%s')"),
											*GraphName,
											*GetSchema()->GetSanitizedPinPath(InPin->GetPinPath())));
	}

	return NodeToEject;
}

bool URigVMController::EjectAllInjectedNodes(URigVMNode* InNode, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	bool bHasAnyInjectedNode = false;
	for(URigVMPin* Pin : InNode->GetPins())
	{
		bHasAnyInjectedNode = bHasAnyInjectedNode || Pin->HasInjectedNodes();
	}

	if(!bHasAnyInjectedNode)
	{
		return false;
	}

	FRigVMBaseAction EjectAllInjectedNodesAction(this);
	if (bSetupUndoRedo)
	{
		GetActionStack()->BeginAction(EjectAllInjectedNodesAction);
	}

	for(URigVMPin* Pin : InNode->GetPins())
	{
		if(Pin->HasInjectedNodes())
		{
			if(!EjectNodeFromPin(Pin, bSetupUndoRedo, bPrintPythonCommands))
			{
				return false;
			}
		}
	}

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(EjectAllInjectedNodesAction);
	}

	return true;
}


bool URigVMController::Undo()
{
	if (!IsValidGraph())
	{
		return false;
	}

	return GetActionStack()->Undo(this);
}

bool URigVMController::Redo()
{
	if (!IsValidGraph())
	{
		return false;
	}

	return GetActionStack()->Redo(this);
}

bool URigVMController::OpenUndoBracket(const FString& InTitle)
{
	if (!IsValidGraph())
	{
		return false;
	}
	return GetActionStack()->OpenUndoBracket(this, InTitle);
}

bool URigVMController::CloseUndoBracket()
{
	if (!IsValidGraph())
	{
		return false;
	}
	return GetActionStack()->CloseUndoBracket(this);
}

bool URigVMController::CancelUndoBracket()
{
	if (!IsValidGraph())
	{
		return false;
	}
	return GetActionStack()->CancelUndoBracket(this);
}

FString URigVMController::ExportNodesToText(const TArray<FName>& InNodeNames, bool bIncludeExteriorLinks)
{
	if (!IsValidGraph())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;

	// find all of the nodes
	TArray<URigVMNode*> Nodes;
	Nodes.Reserve(InNodeNames.Num());
	for (const FName& NodeName : InNodeNames)
	{
		if (URigVMNode* Node = Graph->FindNodeByName(NodeName))
		{
			Nodes.Add(Node);
		}
	}

	TArray<FName> AllNodeNames = InNodeNames;
	TArray<FName> FilteredNodeNames;
	FilteredNodeNames.Reserve(InNodeNames.Num());
	for (const FName& NodeName : InNodeNames)
	{
		if (URigVMNode* Node = Graph->FindNodeByName(NodeName))
		{
			for (URigVMPin* Pin : Node->GetPins())
			{
				for (URigVMInjectionInfo* Injection : Pin->GetInjectedNodes())
				{
					AllNodeNames.AddUnique(Injection->Node->GetFName());
				}
			}

			// skip injected nodes that would show up twice in the export
			if(const URigVMInjectionInfo* InjectionInfo = Node->GetInjectionInfo())
			{
				if(const URigVMNode* OuterNode = InjectionInfo->GetTypedOuter<URigVMNode>())
				{
					if(Nodes.Contains(OuterNode))
					{
						continue;
					}
				}
			}
		}

		FilteredNodeNames.Add(NodeName);
	}

	// Export each of the selected nodes
	for (const FName& NodeName : FilteredNodeNames)
	{
		if (URigVMNode* Node = Graph->FindNodeByName(NodeName))
		{
			UExporter::ExportToOutputDevice(&Context, Node, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Node->GetOuter());
		}
	}

	for (URigVMLink* Link : Graph->Links)
	{
		const URigVMPin* SourcePin = Link->GetSourcePin();
		const URigVMPin* TargetPin = Link->GetTargetPin();
		if (SourcePin && TargetPin && IsValid(SourcePin) && IsValid(TargetPin))
		{
			const bool bSourceNodeIsPartOfExport = AllNodeNames.Contains(SourcePin->GetNode()->GetFName());
			const bool bTargetNodeIsPartOfExport = AllNodeNames.Contains(TargetPin->GetNode()->GetFName());

			const bool bShouldExport = (bSourceNodeIsPartOfExport && bTargetNodeIsPartOfExport) ||
				(bIncludeExteriorLinks && (bSourceNodeIsPartOfExport != bTargetNodeIsPartOfExport));

			if(bShouldExport)
			{
				Link->UpdatePinPaths();
				UExporter::ExportToOutputDevice(&Context, Link, NULL, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Link->GetOuter());
			}
		}
	}

	return MoveTemp(Archive);
}

FString URigVMController::ExportNodeToText(const URigVMNode* InNode, bool bIncludeExteriorLinks)
{
	TArray<FName> AllNodeNames;
	if (InNode != nullptr)
	{
		AllNodeNames.Add(InNode->GetFName());
	}

	return ExportNodesToText(AllNodeNames, bIncludeExteriorLinks);
}

FString URigVMController::ExportSelectedNodesToText(bool bIncludeExteriorLinks)
{
	if (!IsValidGraph())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	return ExportNodesToText(Graph->GetSelectNodes(), bIncludeExteriorLinks);
}

struct FRigVMControllerObjectFactory : public FCustomizableTextObjectFactory
{
public:
	URigVMController* Controller;
	TArray<URigVMNode*> CreatedNodes;
	TMap<const URigVMGraph*, TArray<FName>> CreateNodeNamesPerGraph;
	TMap<const URigVMGraph*, TMap<FString, FString>> NodeNameMapPerGraph;
	URigVMController::FRestoreLinkedPathSettings RestoreLinksSettings; 
	TArray<URigVMLink*> CreatedLinks;
	TArray<URigVMGraph*> CreatedGraphs;
public:
	FRigVMControllerObjectFactory(URigVMController* InController)
		: FCustomizableTextObjectFactory(GWarn)
		, Controller(InController)
	{
		RestoreLinksSettings.bIsImportingFromText = true;
	}

protected:
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override
	{
		const UObject* DefaultObject = ObjectClass->GetDefaultObject(); 
		return DefaultObject->IsA<URigVMNode>() || 
			DefaultObject->IsA<URigVMGraph>() ||
			DefaultObject->IsA<URigVMLink>() ||
			DefaultObject->IsA<URigVMInjectionInfo>() ||
			DefaultObject->IsA<URigVMPin>();
	}

	virtual bool CanCreateObject(UObject* InParent, UClass* ObjectClass, const FName& InDesiredName) const override
	{
		if(const URigVMGraph* Graph = Cast<URigVMGraph>(InParent))
		{
			if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
			{
				// avoid creating duplicate entry / return nodes
				if(ObjectClass == URigVMFunctionEntryNode::StaticClass() && LibraryNode->GetEntryNode() != nullptr)
				{
					return false;
				}
				if(ObjectClass == URigVMFunctionReturnNode::StaticClass() && LibraryNode->GetReturnNode() != nullptr)
				{
					return false;
				}
			}		
		}
		return true;
	}

	virtual void UpdateObjectName(UClass* ObjectClass, UObject* InParent, FName& InOutObjName) override
	{
		if (ObjectClass->GetDefaultObject()->IsA<URigVMNode>())
		{
			const URigVMGraph* Graph = Cast<URigVMGraph>(InParent);
			if(Graph == nullptr)
			{
				if(const URigVMInjectionInfo* InjectionInfo = Cast<URigVMInjectionInfo>(InParent))
				{
					Graph = InjectionInfo->GetGraph();
				}
			}

			check(Graph);

			TArray<FName>& CreateNodeNames = CreateNodeNamesPerGraph.FindOrAdd(Graph);
			TMap<FString, FString>& NodeNameMap = NodeNameMapPerGraph.FindOrAdd(Graph);

			const FName ValidName = URigVMSchema::GetUniqueName(InOutObjName, [Graph, this, CreateNodeNames](const FName& InName) {
				return !CreateNodeNames.Contains(InName) && Graph->IsNameAvailable(InName.ToString());
			}, false, true);

			if(!InOutObjName.IsEqual(ValidName, ENameCase::CaseSensitive))
			{
				if(ObjectClass != URigVMFunctionEntryNode::StaticClass() &&
					ObjectClass != URigVMFunctionReturnNode::StaticClass())
				{
					NodeNameMap.Add(InOutObjName.ToString(), ValidName.ToString());
				}
			}
			
			CreateNodeNames.Add(ValidName);
			InOutObjName = ValidName;
		}
	}

	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (URigVMGraph* Graph = Cast<URigVMGraph>(CreatedObject))
		{
			CreatedGraphs.Add(Graph);
			(void)CreateNodeNamesPerGraph.FindOrAdd(Graph);
			(void)NodeNameMapPerGraph.FindOrAdd(Graph);

			for(URigVMNode* Node : Graph->GetNodes())
			{
				ProcessConstructedObject(Node);
			}
			for(URigVMLink* Link : Graph->GetLinks())
			{
				ProcessConstructedObject(Link);
			}
		}
		else if (URigVMNode* Node = Cast<URigVMNode>(CreatedObject))
		{
			CreatedNodes.Add(Node);

			for (URigVMPin* Pin : Node->GetPins())
			{
				ProcessConstructedObject(Pin);
			}

			if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
			{
				ProcessConstructedObject(CollapseNode->GetContainedGraph());
			}
		}
		else if (URigVMLink* Link = Cast<URigVMLink>(CreatedObject))
		{
			CreatedLinks.Add(Link);
		}
		else if (const URigVMPin* Pin = Cast<URigVMPin>(CreatedObject))
		{
			for(URigVMInjectionInfo* InjectionInfo : Pin->GetInjectedNodes())
			{
				ProcessConstructedObject(InjectionInfo);
			}
		}
		else if (URigVMInjectionInfo* Injection = Cast<URigVMInjectionInfo>(CreatedObject))
		{
			if (URigVMNode* InjectedNode = Injection->Node)
			{
				ProcessConstructedObject(InjectedNode);

				FName NewName = InjectedNode->GetFName();
				UpdateObjectName(URigVMNode::StaticClass(), Injection->GetGraph(), NewName);
				Controller->RenameObject(InjectedNode, *NewName.ToString(), nullptr);
				Injection->InputPin = Injection->InputPin ? Injection->Node->FindPin(Injection->InputPin->GetName()) : nullptr;
				Injection->OutputPin = Injection->OutputPin ? Injection->Node->FindPin(Injection->OutputPin->GetName()) : nullptr;
			}
		}
	}
};

bool URigVMController::CanImportNodesFromText(const FString& InText)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	FRigVMControllerObjectFactory Factory(this);
	return Factory.CanCreateObjectsFromText(InText);
}

TArray<FName> URigVMController::ImportNodesFromText(const FString& InText, bool bSetupUndoRedo, bool bPrintPythonCommands)
{
	if (!IsValidGraph())
	{
		return TArray<FName>();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return TArray<FName>();
	}

	check(GetGraph());

	const TArray<URigVMNode*> NodesPriorImport = GetGraph()->GetNodes();
	const TArray<URigVMLink*> LinksPriorImport = GetGraph()->GetLinks();

	FRigVMControllerObjectFactory Factory(this);
	Factory.ProcessBuffer(GetGraph(), RF_Transactional, InText);

	if (Factory.CreatedNodes.IsEmpty() && Factory.CreatedLinks.IsEmpty())
	{
		return TArray<FName>();
	}
	
	// sort the graphs, nodes and links based on the depth - first resolve the links in leaf graphs
	// and then work your way outward
	const TArray<URigVMGraph*> ImportedGraphs = Factory.CreatedGraphs;
	const TArray<URigVMNode*> ImportedNodes = Factory.CreatedNodes;
	const TArray<URigVMLink*> ImportedLinks = Factory.CreatedLinks;
	SortGraphElementsByGraphDepth(Factory.CreatedGraphs, true);
	SortGraphElementsByGraphDepth(Factory.CreatedNodes, true);
	SortGraphElementsByGraphDepth(Factory.CreatedLinks, true);

	FRigVMControllerCompileBracketScope CompileScope(this);

	TArray<TGuardValue<bool>> EditGuards;
	for (URigVMGraph* CreatedGraph : Factory.CreatedGraphs)
	{
		EditGuards.Emplace(CreatedGraph->bEditable, true);
	}

	TMap<URigVMGraph*, TArray<FName>> NodeNamesPerGraph;
	{
		TArray<URigVMNode*> FilteredNodes;
		for (URigVMNode* CreatedNode : Factory.CreatedNodes)
		{
			URigVMGraph* Graph = CreatedNode->GetTypedOuter<URigVMGraph>();
			if(Graph == nullptr)
			{
				DestroyObject(CreatedNode);
				continue;
			}

			if(URigVMController* ControllerForGraph = GetControllerForGraph(Graph))
			{
				if(!ControllerForGraph->AddGraphNode(CreatedNode, false))
				{
					continue;
				}

				FilteredNodes.Add(CreatedNode);
			}
		}

		Swap(Factory.CreatedNodes, FilteredNodes);
	}

	// Some nodes inside these graphs might have been deleted, we need to clean up the graph
	for (URigVMGraph* CreatedGraph : Factory.CreatedGraphs)
	{
		CreatedGraph->Nodes = CreatedGraph->Nodes.FilterByPredicate([](const URigVMNode* Node)
		{
			return IsValid(Node);
		});
	}

	// the links may already be in the graph's links property
	// due to the serialization - remove them since we need.
	// but we also want to maintain the order of the links as they
	// are getting created. we'll create an array 
	for (URigVMLink* CreatedLink : Factory.CreatedLinks)
	{
		if(URigVMGraph* Graph = CreatedLink->GetTypedOuter<URigVMGraph>())
		{
			Graph->Links.Remove(CreatedLink);
		}
	}

	FRigVMUnitNodeCreatedContext::FScope UnitNodeCreatedScope(UnitNodeCreatedContext, ERigVMNodeCreatedReason::Paste, GetClientHost());
	for (URigVMNode* CreatedNode : Factory.CreatedNodes)
	{
		URigVMGraph* Graph = CreatedNode->GetTypedOuter<URigVMGraph>();
		check(Graph);

		if(URigVMController* ControllerForGraph = GetControllerForGraph(Graph))
		{
			// Refresh the unit node to account for changes in node color, pin additions, pin order, etc
			if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(CreatedNode))
			{
				if(UnitNode->ResolvedFunctionName.IsEmpty())
				{
					if(UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
					{
						const FName MethodName = UnitNode->GetMethodName();
						if(!MethodName.IsNone())
						{
							if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(ScriptStruct, *MethodName.ToString()))
							{
								UnitNode->ResolvedFunctionName = Function->GetName();

								if(const FRigVMTemplate* Template = Function->GetTemplate())
								{
									UnitNode->TemplateNotation = Template->GetNotation();
								}
							}
						}
					}
				}
				
				TGuardValue<bool> SuspendNotifications(ControllerForGraph->bSuspendNotifications, true);
				ControllerForGraph->RepopulatePinsOnNode(UnitNode);
			}

			if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(CreatedNode))
			{
				if (ControllerForGraph->UnitNodeCreatedContext.IsValid())
				{
					if (TSharedPtr<FStructOnScope> StructScope = UnitNode->ConstructStructInstance())
					{
						TGuardValue<FName> NodeNameScope(ControllerForGraph->UnitNodeCreatedContext.NodeName, UnitNode->GetFName());
						FRigVMStruct* StructInstance = (FRigVMStruct*)StructScope->GetStructMemory();
						StructInstance->OnUnitNodeCreated(ControllerForGraph->UnitNodeCreatedContext);
					}
				}
			}

			if (URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(CreatedNode))
			{
				if(URigVMBuildData* BuildData = URigVMBuildData::Get())
				{
					BuildData->RegisterFunctionReference(FunctionRefNode->GetReferencedFunctionHeader().LibraryPointer, FunctionRefNode);
				}
			}

			for(URigVMPin* Pin : CreatedNode->Pins)
			{
				ControllerForGraph->EnsurePinValidity(Pin, true);
				Pin->InjectionInfos.Remove(nullptr);
			}

			ControllerForGraph->Notify(ERigVMGraphNotifType::NodeAdded, CreatedNode);

			TArray<FName>& NodeNames = NodeNamesPerGraph.FindOrAdd(Graph);
			NodeNames.Add(CreatedNode->GetFName());
		}
	}

	if (Factory.CreatedLinks.Num() > 0)
	{
		URigVMGraph* LastGraph = nullptr;
		for (URigVMLink* CreatedLink : Factory.CreatedLinks)
		{
			URigVMGraph* Graph = CreatedLink->GetTypedOuter<URigVMGraph>();
			if(Graph == nullptr)
			{
				DestroyObject(CreatedLink);
				continue;
			}

			if(Graph != LastGraph)
			{
				if (TMap<FString, FString>* Map = Factory.NodeNameMapPerGraph.Find(Graph))
				{
					Factory.RestoreLinksSettings.NodeNameMap = *Map;
				}
				else
				{
					Factory.RestoreLinksSettings.NodeNameMap.Reset();
				}
				LastGraph = Graph;
			}

			if(URigVMController* ControllerForGraph = GetControllerForGraph(Graph))
			{
				// this takes care of mapping from the old to the new nodes etc
				const TArray<FLinkedPath> LinkedPaths = ControllerForGraph->RemapLinkedPaths(
					ControllerForGraph->GetLinkedPaths({CreatedLink}), 
					Factory.RestoreLinksSettings,
					false);
				const FLinkedPath& LinkedPath = LinkedPaths[0]; 

				URigVMPin* SourcePin = LinkedPath.GetSourcePin();
				URigVMPin* TargetPin = LinkedPath.GetTargetPin();
				
				if (SourcePin && TargetPin)
				{
					// update the paths on the link to allow reuse of the UObject
					CreatedLink->SetSourceAndTargetPinPaths(LinkedPath.SourcePinPath, LinkedPath.TargetPinPath);
					
					if (TargetPin->IsBoundToVariable())
					{
						const FString VariableNodeName = TargetPin->GetBoundVariableNode()->GetName();
						const FString BindingPath = TargetPin->GetBoundVariablePath();

						// The current situation is that the outer pin has an injection info, and the injected node exists
						// but the injected node is not linked to the outer pin. BreakAllLinks will try to unbind the outer pin,
						// for that to be successful, the binding needs to be complete
						// Connect it so that the unbound is successful
						check(!SourcePin->IsLinkedTo(TargetPin));
						Graph->DetachedLinks.Add(CreatedLink);
						ControllerForGraph->AddLink(SourcePin, TargetPin, false);

						// recreate binding
						ControllerForGraph->UnbindPinFromVariable(TargetPin, false);
						ControllerForGraph->BindPinToVariable(TargetPin, BindingPath, false, VariableNodeName);
					}
					else
					{
						Graph->DetachedLinks.Add(CreatedLink);
						ControllerForGraph->AddLink(SourcePin, TargetPin, false);
					}
				}
				else
				{
					Graph->DetachedLinks.AddUnique(CreatedLink);
				}

				// if the link is still part of the detached link array
				// the restore was not successful.
				if(Graph->DetachedLinks.Remove(CreatedLink))
				{
					ControllerForGraph->ReportErrorf(TEXT("Cannot import link '%s'."), *URigVMLink::GetPinPathRepresentation(CreatedLink->GetSourcePinPath(), CreatedLink->GetTargetPinPath()));
					DestroyObject(CreatedLink);
				}
			}
		}
	}

	// order nodes and links per graph again in the order that they were originally created.
	SortGraphElementsByImportOrder(GetGraph()->Nodes, NodesPriorImport, ImportedNodes);
	for(URigVMGraph* CreatedGraph : Factory.CreatedGraphs)
	{
		SortGraphElementsByImportOrder(CreatedGraph->Nodes, {}, ImportedNodes);
	}

	SortGraphElementsByImportOrder(GetGraph()->Links, LinksPriorImport, ImportedLinks);
	for(URigVMGraph* CreatedGraph : Factory.CreatedGraphs)
	{
		SortGraphElementsByImportOrder(CreatedGraph->Links, {}, ImportedLinks);
	}

	if(!NodeNamesPerGraph.Contains(GetGraph()))
	{
		ReportError(TEXT("Unexpected failure during ImportNodesFromText"));
		return TArray<FName>();
	}
	
	const TArray<FName>& NodeNames = NodeNamesPerGraph.FindChecked(GetGraph());

	if (bSetupUndoRedo)
	{
		FRigVMImportFromTextAction Action(this, InText, NodeNames);
		Action.SetTitle(TEXT("Importing Nodes from Text"));
		GetActionStack()->AddAction(Action);
	}

#if WITH_EDITOR
	if (bPrintPythonCommands && !NodeNames.IsEmpty())
	{
		FString PythonContent = InText.Replace(TEXT("\\\""), TEXT("\\\\\""));
		PythonContent = InText.Replace(TEXT("'"), TEXT("\\'"));
		PythonContent = PythonContent.Replace(TEXT("\r\n"), TEXT("\\r\\n'\r\n'"));

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').import_nodes_from_text('%s')"),
			*GraphName,
			*PythonContent));
	}
#endif

	return NodeNames;
}

bool URigVMController::ExportFunctionToArchive(const FName& InFunctionName, FRigVMObjectArchive& OutArchive)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph);
	if (FunctionLibrary == nullptr)
	{
		ReportError(TEXT("Cannot export functions from non-function-libraries"));
		return false;
	}

	URigVMLibraryNode* Function = FunctionLibrary->FindFunction(InFunctionName);
	if(Function == nullptr)
	{
		ReportErrorf(TEXT("Cannot find function '%s'"), *InFunctionName.ToString());
		return false;
	}

	OutArchive.Reset();
	{
		FRigVMObjectArchiveWriter Writer(OutArchive, Function);
		Writer << Function;
	}
	OutArchive.Compress();
	
	return true;
}

URigVMLibraryNode* URigVMController::ImportFunctionFromArchive(const FRigVMObjectArchive& InArchive, const FName& InFunctionName)
{
	if (!IsValidGraph())
	{
		return {};
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return {};
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph);
	if (FunctionLibrary == nullptr)
	{
		ReportError(TEXT("Cannot import functions into non-function-libraries"));
		return {};
	}

	if(InArchive.IsEmpty())
	{
		ReportError(TEXT("Provided archive is empty."));
		return {};
	}

	// test if the archive contains the right thin.
	// since the archive is passed as const - we need to copy it.
	// the process here may also decompress the archive internally.
	FRigVMObjectArchive CopyOfArchive = InArchive;
	FRigVMObjectArchiveReader::FObjectHeader ObjectHeader;
	{
		FRigVMObjectArchiveReader ReaderForHeader(CopyOfArchive, nullptr);
		ObjectHeader = ReaderForHeader.GetRootObjectHeader();
	}
	if(!ObjectHeader.IsValid())
	{
		ReportError(TEXT("Cannot read object header from archive. Corrupt data?"));
		return {};
	}
	
	if(ObjectHeader.Class != URigVMCollapseNode::StaticClass())
	{
		ReportError(TEXT("Archive doesn't contain a collapse node."));
		return {};
	}

	const FName NameForFunction = URigVMSchema::GetUniqueName(InFunctionName.IsNone() ? ObjectHeader.Name : InFunctionName, [Graph](const FName& InName)
	{
		return Graph->IsNameAvailable(InName.ToString());
	}, false, true);

	FRigVMVariant Variant;
	if (FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore())
	{
		if (const FRigVMGraphFunctionData* FunctionData = FunctionStore->FindFunctionByName(NameForFunction))
		{
			Variant = FunctionData->Header.Variant;
		}
	}
	
	if (!Variant.Guid.IsValid())
	{
		Variant.Guid = FRigVMVariant::GenerateGUID();
	}

	URigVMCollapseNode * CollapseNode = NewObject<URigVMCollapseNode>(FunctionLibrary, NameForFunction);

	// reinitialize the reader
	FRigVMObjectArchiveReader Reader(CopyOfArchive, CollapseNode);

	// perform the import
	Reader << CollapseNode;

	if(CollapseNode->GetContainedGraph() == nullptr)
	{
		ReportError(TEXT("Unable to deserialize archive."));
		RemoveNode(CollapseNode, false, false);
		return nullptr;
	}

	if(!AddGraphNode(CollapseNode, false))
	{
		ReportError(TEXT("Unable to add imported function to function library."));
		DestroyObject(CollapseNode);
		return nullptr;
	}

	// add the variant information back
	FunctionLibrary->FunctionToVariant.Add(NameForFunction, Variant);

	// notify everyone
	Notify(ERigVMGraphNotifType::NodeAdded, CollapseNode);
	if(URigVMController* ContainedGraphController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
	{
		ContainedGraphController->ResendAllNotifications();
	}

	return CollapseNode;
}

URigVMLibraryNode* URigVMController::LocalizeFunctionFromPath(const FString& InHostPath, const FName& InFunctionName, bool bLocalizeDependentPrivateFunctions, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add function reference nodes to function library graphs."));
		return nullptr;
	}

	UObject* HostObject = StaticLoadObject(UObject::StaticClass(), NULL, *InHostPath, NULL, LOAD_None, NULL);
	if (!HostObject)
	{
		ReportErrorf(TEXT("Failed to load the Host object %s."), *InHostPath);
		return nullptr;
	}

	IRigVMGraphFunctionHost* FunctionHost = Cast<IRigVMGraphFunctionHost>(HostObject);
	if (!FunctionHost)
	{
		ReportError(TEXT("Host object is not a IRigVMGraphFunctionHost."));
		return nullptr;
	}

	FRigVMGraphFunctionData* Data = FunctionHost->GetRigVMGraphFunctionStore()->FindFunctionByName(InFunctionName);
	if (!Data)
	{
		ReportErrorf(TEXT("Function %s not found in host %s."), *InFunctionName.ToString(), *InHostPath);
		return nullptr;
	}

	return LocalizeFunction(Data->Header.LibraryPointer, bLocalizeDependentPrivateFunctions, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMLibraryNode* URigVMController::LocalizeFunction(
	const FRigVMGraphFunctionIdentifier& InFunctionDefinition,
	bool bLocalizeDependentPrivateFunctions,
	bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	TArray<FRigVMGraphFunctionIdentifier> FunctionsToLocalize;
	FunctionsToLocalize.Add(InFunctionDefinition);

	TMap<FRigVMGraphFunctionIdentifier, URigVMLibraryNode*> Results = LocalizeFunctions(FunctionsToLocalize, bLocalizeDependentPrivateFunctions, bSetupUndoRedo, bPrintPythonCommand);

	URigVMLibraryNode** LocalizedFunctionPtr = Results.Find(FunctionsToLocalize[0]);
	if(LocalizedFunctionPtr)
	{
		return *LocalizedFunctionPtr;
	}
	return nullptr;
}

TMap<FRigVMGraphFunctionIdentifier, URigVMLibraryNode*> URigVMController::LocalizeFunctions(
	TArray<FRigVMGraphFunctionIdentifier> InFunctionDefinitions,
	bool bLocalizeDependentPrivateFunctions,
	bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	TMap<FRigVMGraphFunctionIdentifier, URigVMLibraryNode*> LocalizedFunctions;

	if(!IsValidGraph())
	{
		return LocalizedFunctions;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return LocalizedFunctions;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* ThisLibrary = Graph->GetDefaultFunctionLibrary();
	if(ThisLibrary == nullptr)
	{
		return LocalizedFunctions;
	}

	TArray<FRigVMGraphFunctionData*> FunctionsToLocalize;

	TArray<FRigVMGraphFunctionIdentifier> NodesToVisit;
	for(const FRigVMGraphFunctionIdentifier& FunctionDefinition : InFunctionDefinitions)
	{
		NodesToVisit.AddUnique(FunctionDefinition);
		FunctionsToLocalize.AddUnique(FRigVMGraphFunctionData::FindFunctionData(FunctionDefinition));
	}

	const int32 InputNodesToVisitCount = NodesToVisit.Num();

	const FSoftObjectPath ThisFunctionHost = ThisLibrary->GetFunctionHostObjectPath();
	
	// find all functions to localize
	for(int32 NodeToVisitIndex=0; NodeToVisitIndex<NodesToVisit.Num(); NodeToVisitIndex++)
	{
		const FRigVMGraphFunctionIdentifier NodeToVisit = NodesToVisit[NodeToVisitIndex];

		// Already local
		if (NodeToVisit.HostObject == ThisFunctionHost)
		{
			continue;
		}

		bool bIsPublic;
		FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(NodeToVisit, &bIsPublic);
		if (!FunctionData)
		{
			ReportAndNotifyErrorf(TEXT("Cannot localize function - could not find function %s in host %s."), *NodeToVisit.GetLibraryNodePath(), *NodeToVisit.HostObject.ToString());
			return LocalizedFunctions;
		}

		// Do not localize public functions if they are not part of the input set of functions
		if (bIsPublic && NodeToVisitIndex >= InputNodesToVisitCount)
		{
			continue;
		}

		if (!bLocalizeDependentPrivateFunctions)
		{
			ReportAndNotifyErrorf(TEXT("Cannot localize function - dependency %s is private."), *NodeToVisit.GetLibraryNodePath());
			return LocalizedFunctions;
		}

		FunctionsToLocalize.AddUnique(FunctionData);

		// Look for its dependencies
		for (TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : FunctionData->Header.Dependencies)
		{
			NodesToVisit.AddUnique(Pair.Key);
		}
	}
	
	// sort the functions to localize based on their nesting
	Algo::Sort(FunctionsToLocalize, [](FRigVMGraphFunctionData* A, FRigVMGraphFunctionData* B) -> bool
	{
		check(A);
		check(B);
		return B->Header.Dependencies.Contains(A->Header.LibraryPointer);
	});

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Localize functions"));
	}

	// import the functions to our local function library
	if(URigVMController* LibraryController = GetControllerForGraph(ThisLibrary))
	{
		for(FRigVMGraphFunctionData* FunctionToLocalize : FunctionsToLocalize)
		{
			if (URigVMLibraryNode* ReferencedFunction = Cast<URigVMLibraryNode>(FunctionToLocalize->Header.LibraryPointer.GetNodeSoftPath().TryLoad()))
			{
				if (IRigVMClientHost* ClientHost = ReferencedFunction->GetImplementingOuter<IRigVMClientHost>())
				{
					ClientHost->GetRigVMClient()->UpdateGraphFunctionSerializedGraph(ReferencedFunction);
				}
			}

			URigVMLibraryNode* LocalizedFunction = nullptr;
			if(FunctionToLocalize->CollapseNodeArchive.IsEmpty())
			{
				TArray<FName> NodeNames = LibraryController->ImportNodesFromText(FunctionToLocalize->SerializedCollapsedNode_DEPRECATED, false);
				if (NodeNames.Num() > 0)
				{
					LocalizedFunction = ThisLibrary->FindFunction(NodeNames[0]);
				}
			}
			else
			{
				LocalizedFunction = LibraryController->ImportFunctionFromArchive(FunctionToLocalize->CollapseNodeArchive);
			}

			if(LocalizedFunction)
			{
				LocalizedFunctions.Add(FunctionToLocalize->Header.LibraryPointer, LocalizedFunction);
				ThisLibrary->LocalizedFunctions.FindOrAdd(FunctionToLocalize->Header.LibraryPointer.GetLibraryNodePath(), LocalizedFunction);
			}
		}
	}

	// once we have all local functions available, clean up the references
	TArray<URigVMGraph*> GraphsToUpdate;
	if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if(FRigVMClient* Client = ClientHost->GetRigVMClient())
		{
			GraphsToUpdate = Client->GetAllModels(true, true);
		}
	}

	for(int32 GraphToUpdateIndex=0; GraphToUpdateIndex<GraphsToUpdate.Num(); GraphToUpdateIndex++)
	{
		URigVMGraph* GraphToUpdate = GraphsToUpdate[GraphToUpdateIndex];
		if(URigVMController* GraphController = GetControllerForGraph(GraphToUpdate))
		{
			const TArray<URigVMNode*> NodesToUpdate = GraphToUpdate->GetNodes();
			for(URigVMNode* NodeToUpdate : NodesToUpdate)
			{
				if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(NodeToUpdate))
				{
					URigVMLibraryNode** RemappedNodePtr = LocalizedFunctions.Find(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer);
					if(RemappedNodePtr)
					{
						URigVMLibraryNode* RemappedNode = *RemappedNodePtr;
						GraphController->SetReferencedFunction(FunctionReferenceNode, RemappedNode, bSetupUndoRedo);
					}
				}
			}
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	if (bPrintPythonCommand)
	{
		for (const FRigVMGraphFunctionIdentifier& Identifier : InFunctionDefinitions)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(Identifier);
			const FString FunctionDefinitionName = GetSchema()->GetSanitizedNodeName(FunctionData->Header.Name.ToString());

			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("asset.get_controller_by_name('%s').localize_function_from_path('%s', '%s', %s)"),
						*GraphName,
						*Identifier.HostObject.ToString(),
						*FunctionDefinitionName,
						bLocalizeDependentPrivateFunctions ? TEXT("True") : TEXT("False")));
		}
	}

	return LocalizedFunctions;
}

URigVMCollapseNode* URigVMController::CollapseNodes(const TArray<FName>& InNodeNames, const FString& InCollapseNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bIsAggregate)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<URigVMNode*> Nodes;
	for (const FName& NodeName : InNodeNames)
	{
		URigVMNode* Node = Graph->FindNodeByName(NodeName);
		if (Node == nullptr)
		{
			ReportErrorf(TEXT("Cannot find node '%s'."), *NodeName.ToString());
			return nullptr;
		}
		Nodes.AddUnique(Node);
	}

	URigVMCollapseNode* Node = CollapseNodes(Nodes, InCollapseNodeName, bSetupUndoRedo, bIsAggregate);
	if (Node && bPrintPythonCommand)
	{
		FString ArrayStr = TEXT("[");
		for (auto It = InNodeNames.CreateConstIterator(); It; ++It)
		{
			ArrayStr += TEXT("'") + It->ToString() + TEXT("'");
			if (It.GetIndex() < InNodeNames.Num() - 1)
			{
				ArrayStr += TEXT(", ");
			}
		}
		ArrayStr += TEXT("]");

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').collapse_nodes(%s, '%s')"),
											*GraphName,
											*ArrayStr,
											*InCollapseNodeName));
	}

	return Node;
}

TArray<URigVMNode*> URigVMController::ExpandLibraryNode(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return TArray<URigVMNode*>();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return TArray<URigVMNode*>();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find collapse node '%s'."), *InNodeName.ToString());
		return TArray<URigVMNode*>();
	}

	URigVMLibraryNode* LibNode = Cast<URigVMLibraryNode>(Node);
	if (LibNode == nullptr)
	{
		ReportErrorf(TEXT("Node '%s' is not a library node (not collapse nor function)."), *InNodeName.ToString());
		return TArray<URigVMNode*>();
	}

	TArray<URigVMNode*> Nodes = ExpandLibraryNode(LibNode, bSetupUndoRedo);

	if (!Nodes.IsEmpty() && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').expand_library_node('%s')"),
											*GraphName,
											*NodeName));
	}

	return Nodes;
}

#endif

URigVMCollapseNode* URigVMController::CollapseNodes(const TArray<URigVMNode*>& InNodes, const FString& InCollapseNodeName, bool bSetupUndoRedo, bool bIsAggregate)
{
	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot collapse nodes in function library graphs."));
		return nullptr;
	}

	if (InNodes.IsEmpty())
	{
		ReportError(TEXT("No nodes specified to collapse."));
		return nullptr;
	}

	{
		const TArrayView<URigVMNode* const> NodesView((URigVMNode** const)InNodes.GetData(), InNodes.Num());
		if(!GetSchema()->CanCollapseNodes(this, NodesView))
		{
			return nullptr;
		}
	}

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if (bIsAggregate)
	{
		if(InNodes.Num() != 1)
		{
			return nullptr;
		}

		if(!InNodes[0]->IsAggregate())
		{
			ReportError(TEXT("Cannot aggregate the given node."));
			return nullptr;
		}
	}
#endif

	TArray<URigVMNode*> Nodes;
	for (URigVMNode* Node : InNodes)
	{
		if (!IsValidNodeForGraph(Node))
		{
			return nullptr;
		}

		// filter out certain nodes
		if (Node->IsEvent())
		{
			continue;
		}

		if (Node->IsA<URigVMFunctionInterfaceNode>())
		{
			continue;
		}

		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->IsInputArgument())
			{
				continue;
			}
		}

		Nodes.Add(Node);
	}

	if (Nodes.Num() == 0)
	{
		return nullptr;
	}

	FBox2D Bounds = FBox2D(EForceInit::ForceInit);
	TArray<FName> NodeNames;
	for (URigVMNode* Node : Nodes)
	{
		NodeNames.Add(Node->GetFName());
		Bounds += Node->GetPosition();
	}

  	FVector2D Diagonal = Bounds.Max - Bounds.Min;
	FVector2D Center = (Bounds.Min + Bounds.Max) * 0.5f;

	TMap<const URigVMPin*, const URigVMPin*> CollapsedPins;
	TArray<URigVMLink*> InputLinksToRewire;
	TArray<URigVMLink*> OutputLinksToRewire;
	TArray<URigVMLink*> AllLinks = Graph->GetLinks();

	auto NodeToBeCollapsed = [&Nodes](URigVMNode* InNode) -> bool
	{
		check(InNode);
		
		if(Nodes.Contains(InNode))
		{
			return true;
		}
		
		if(InNode->IsInjected()) 
		{
			InNode = InNode->GetTypedOuter<URigVMNode>();
			if(Nodes.Contains(InNode))
			{
				return true;
			}
		}

		return false;
	};

	TArray<const URigVMNode*> BoundaryNodes;
	
	// find all pins to collapse. we need this to find out if
	// we might have a parent pin of a given linked pin already 
	// collapsed.
	for (URigVMLink* Link : AllLinks)
	{
		URigVMPin* SourcePin = Link->GetSourcePin();
		URigVMPin* TargetPin = Link->GetTargetPin();
		bool bSourceToBeCollapsed = NodeToBeCollapsed(SourcePin->GetNode());
		bool bTargetToBeCollapsed = NodeToBeCollapsed(TargetPin->GetNode());
		if (bSourceToBeCollapsed == bTargetToBeCollapsed)
		{
			continue;
		}

		if (bSourceToBeCollapsed)
		{
			OutputLinksToRewire.Add(Link);
		}
		else
		{
			InputLinksToRewire.Add(Link);
		}

		BoundaryNodes.AddUnique(SourcePin->GetNode());
		BoundaryNodes.AddUnique(TargetPin->GetNode());
	}

	auto ExecuteLinkCountPredicate = [](const URigVMLink* Link) { return Link->GetSourcePin()->IsExecuteContext(); };
	const int32 NumInputExecuteLinks = static_cast<int32>(Algo::CountIf(InputLinksToRewire, ExecuteLinkCountPredicate));
	const int32 NumOutputExecuteLinks = static_cast<int32>(Algo::CountIf(OutputLinksToRewire, ExecuteLinkCountPredicate));

	if (NumInputExecuteLinks > 1 || NumOutputExecuteLinks > 1)
	{
		ReportAndNotifyWarning(TEXT("Cannot collapse nodes. More than one chain of execute links selected."));
		return nullptr;
	}

	// introduce an order for the node based on the screen position
	TMap<const URigVMNode*, int64> NodeScores;
	int32 MaxNumPinsPerNode = 0;
	for (const URigVMNode* BoundaryNode : BoundaryNodes)
	{
		NodeScores.Add(BoundaryNode, static_cast<int64>(BoundaryNode->GetPosition().Y));
		MaxNumPinsPerNode = FMath::Max<int32>(MaxNumPinsPerNode, BoundaryNode->GetAllPinsRecursively().Num());
	}
	
	// introduce an order for the pins on all nodes based on screen position
	TMap<const URigVMPin*, int64> PinScores;
	for (const URigVMNode* BoundaryNode : BoundaryNodes)
	{
		const TArray<URigVMPin*> AllPins = BoundaryNode->GetAllPinsRecursively();
		TArray<int64> MaxScorePerDirection;
		MaxScorePerDirection.AddZeroed(static_cast<int32>(ERigVMPinDirection::Invalid));

		const int64 NodeScore = NodeScores.FindChecked(BoundaryNode) * static_cast<int64>(MaxNumPinsPerNode);
		
		for (const URigVMPin* Pin : AllPins)
		{
			int64& MaxScore = MaxScorePerDirection[static_cast<int64>(Pin->GetDirection())];
			PinScores.Add(Pin, NodeScore + (MaxScore++));
		}
	}

	// sort both link arrays based on the score of the target pin
	Algo::Sort(InputLinksToRewire, [&PinScores](URigVMLink* A, URigVMLink* B) -> bool
	{
		return PinScores.FindChecked(A->GetTargetPin()) < PinScores.FindChecked(B->GetTargetPin());
	});
	Algo::Sort(OutputLinksToRewire, [&PinScores](URigVMLink* A, URigVMLink* B) -> bool
	{
		return PinScores.FindChecked(A->GetTargetPin()) < PinScores.FindChecked(B->GetTargetPin());
	});

	struct FLinkCollapseInfoPin
	{
		FLinkCollapseInfoPin()
		: Pin(nullptr)
		{
		}

		FLinkCollapseInfoPin(const FString& InSegmentPath, const URigVMPin* InPin)
		: SegmentPath(InSegmentPath)
		, Pin(InPin)
		{
		}
		
		FString SegmentPath;
		const URigVMPin* Pin;
	};
	struct FLinkCollapseInfo
	{
		FLinkCollapseInfo()
		: Pin(nullptr)
		{
		}

		FLinkCollapseInfo(const URigVMPin* InPin, const URigVMPin* InLinkedPin)
		: Pin(InPin)
		, LinkedPins({{FString(), InLinkedPin}})
		{
		}
		
		const URigVMPin* Pin;
		TArray<FLinkCollapseInfoPin> LinkedPins;
	};

	TArray<FLinkCollapseInfo> InputLinkInfos;
	for (const URigVMLink* Link : InputLinksToRewire)
	{
		InputLinkInfos.Emplace(Link->GetSourcePin(), Link->GetTargetPin());
	}

	TArray<FLinkCollapseInfo> OutputLinkInfos;
	for (const URigVMLink* Link : OutputLinksToRewire)
	{
		OutputLinkInfos.Emplace(Link->GetSourcePin(), Link->GetTargetPin());
	}

	auto CombineLinkInfos = [](const TArray<FLinkCollapseInfo>& InFlatInfos) -> TArray<FLinkCollapseInfo>
	{
		TArray<FLinkCollapseInfo> Result;

		for (const FLinkCollapseInfo& FlatInfo : InFlatInfos)
		{
			check(FlatInfo.LinkedPins.Num() == 1);
			check(FlatInfo.LinkedPins[0].SegmentPath.IsEmpty());
			
			const URigVMPin* Pin = FlatInfo.Pin;

			// try to find a mapped pin using this pin exactly
			FLinkCollapseInfo* MatchingInfo = Result.FindByPredicate([Pin](const FLinkCollapseInfo& Info)
			{
				return Info.Pin == Pin;
			});
			
			if (MatchingInfo)
			{
				MatchingInfo->LinkedPins.Emplace(FString(), FlatInfo.LinkedPins[0].Pin);
				continue;
			}

			// get all pins in this pin tree
 			URigVMPin* RootPin = Pin->GetRootPin();
			TArray<URigVMPin*> ChildPins = {RootPin};
			ChildPins.Append(RootPin->GetAllSubPinsRecursively());
			ChildPins.Remove(const_cast<URigVMPin*>(Pin));

			bool bCombinedWithExistingLinkInfo = false;
			for (const URigVMPin* ChildPin : ChildPins)
			{
				MatchingInfo = Result.FindByPredicate([ChildPin](const FLinkCollapseInfo& Info)
				{
					return Info.Pin == ChildPin;
				});

				if (!MatchingInfo)
				{
					continue;
				}

				// if we found a match we are either a sub pin of the pin in question,
				// or a parent pin. ajust out mapping accordingly
				if (ChildPin->IsInOuter(Pin))
				{
					FString SegmentPath = ChildPin->GetName();
					ChildPin = ChildPin->GetParentPin();
					while (ChildPin && ChildPin != Pin)
					{
						SegmentPath = URigVMPin::JoinPinPath(ChildPin->GetName(), SegmentPath);
						ChildPin = ChildPin->GetParentPin();
					}

					MatchingInfo->Pin = Pin;
					for (FLinkCollapseInfoPin& PinInfo : MatchingInfo->LinkedPins)
					{
						if (PinInfo.SegmentPath.IsEmpty())
						{
							PinInfo.SegmentPath = SegmentPath;
						}
						else
						{
							PinInfo.SegmentPath = URigVMPin::JoinPinPath(SegmentPath, PinInfo.SegmentPath);
						}
					}

					MatchingInfo->LinkedPins.Emplace(SegmentPath, FlatInfo.LinkedPins[0].Pin);
				}
				else
				{
					FString SegmentPath = Pin->GetName();
					Pin = Pin->GetParentPin();
					while (Pin && ChildPin != Pin)
					{
						SegmentPath = URigVMPin::JoinPinPath(Pin->GetName(), SegmentPath);
						Pin = Pin->GetParentPin();
					}

					MatchingInfo->LinkedPins.Emplace(SegmentPath, FlatInfo.LinkedPins[0].Pin);
				}

				bCombinedWithExistingLinkInfo = true;
				break;
			}

			if (!bCombinedWithExistingLinkInfo)
			{
				Result.Add(FlatInfo);
			}
		}

		return Result;
	};

	InputLinkInfos = CombineLinkInfos(InputLinkInfos);
	OutputLinkInfos = CombineLinkInfos(OutputLinkInfos);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMCollapseNodesAction CollapseAction;

	FString CollapseNodeName = GetSchema()->GetValidNodeName(Graph, InCollapseNodeName.IsEmpty() ? FString(TEXT("CollapseNode")) : InCollapseNodeName);

	if (bSetupUndoRedo)
	{
		CollapseAction = FRigVMCollapseNodesAction(this, Nodes, CollapseNodeName, bIsAggregate); 
		CollapseAction.SetTitle(TEXT("Collapse Nodes"));
		GetActionStack()->BeginAction(CollapseAction);
	}

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	URigVMCollapseNode* CollapseNode = nullptr;
	if (bIsAggregate)
	{
		CollapseNode = NewObject<URigVMAggregateNode>(Graph, *CollapseNodeName);		
	}
	else
	{
		CollapseNode = NewObject<URigVMCollapseNode>(Graph, *CollapseNodeName);		
	}
#else
	URigVMCollapseNode* CollapseNode = NewObject<URigVMCollapseNode>(Graph, *CollapseNodeName);
#endif
	FString ContainedGraphName = CollapseNodeName + TEXT("_ContainedGraph");

	if (IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if (FRigVMClient* RigVMClient = ClientHost->GetRigVMClient())
		{
			CollapseNode->ContainedGraph = RigVMClient->CreateContainedGraphModel(CollapseNode, *ContainedGraphName);
		}
	}

	if (CollapseNode->ContainedGraph == nullptr)
	{
		return nullptr;
	}

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	if (bIsAggregate)
	{
		CollapseNode->ContainedGraph->bEditable = false;
	}
	TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);
#endif
	
	CollapseNode->Position = Center;

	if(!AddGraphNode(CollapseNode, false))
	{
		return nullptr;
	}

	auto CreateExposedPins = [this, &CollapsedPins, CollapseNode](const FLinkCollapseInfo& CollapseInfo, bool bIsInput)
	{
		check(!CollapseInfo.LinkedPins.IsEmpty());
		
		const URigVMPin* PinToCollapse = CollapseInfo.Pin;
		const URigVMPin* PinForConfiguration = bIsInput ? CollapseInfo.LinkedPins[0].Pin : CollapseInfo.Pin;
		
		if(PinToCollapse->IsExecuteContext())
		{
			for(const TPair<const URigVMPin*, const URigVMPin*>& Pair : CollapsedPins)
			{
				if(Pair.Key != PinToCollapse && Pair.Key->IsExecuteContext())
				{
					CollapsedPins.Add(PinToCollapse, Pair.Value);
					break;
				}
			}
		}

		if (CollapsedPins.Contains(PinToCollapse))
		{
			return;
		}

		const FName DesiredPinName = PinForConfiguration->GetFName();
		const FName PinName = URigVMSchema::GetUniqueName(DesiredPinName, [CollapseNode](const FName& InName) {
			return CollapseNode->FindPin(InName.ToString()) == nullptr;
		}, false, true);

		URigVMPin* CollapsedPin = NewObject<URigVMPin>(CollapseNode, PinName);
		ConfigurePinFromPin(CollapsedPin, PinForConfiguration, false);

		if (PinToCollapse->IsExecuteContext())
		{
			CollapsedPin->SetDisplayName(FRigVMStruct::ExecuteName.Resolve());
			CollapsedPin->Direction = ERigVMPinDirection::IO;
		}
		else if (CollapsedPin->GetDirection() == ERigVMPinDirection::IO)
		{
			if(!CollapsedPin->IsExecuteContext())
			{
				CollapsedPin->Direction = bIsInput ? ERigVMPinDirection::Input : ERigVMPinDirection::Output;
			}
		}

		if (CollapsedPin->IsStruct())
		{
			TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
			AddPinsForStruct(CollapsedPin->GetScriptStruct(), CollapseNode, CollapsedPin, CollapsedPin->GetDirection(), FString(), false);
		}

		AddNodePin(CollapseNode, CollapsedPin);

		FPinState PinState = GetPinState(PinForConfiguration);
		ApplyPinState(CollapsedPin, PinState);

		CollapsedPins.Add(PinToCollapse, CollapsedPin);
	};

	for (const FLinkCollapseInfo& InputLinkInfo : InputLinkInfos)
	{
		CreateExposedPins(InputLinkInfo, true);
	}
	for (const FLinkCollapseInfo& OutputLinkInfo : OutputLinkInfos)
	{
		CreateExposedPins(OutputLinkInfo, false);
	}

	Notify(ERigVMGraphNotifType::NodeAdded, CollapseNode);

	URigVMFunctionEntryNode* EntryNode = nullptr;
	URigVMFunctionReturnNode* ReturnNode = nullptr;

	// import the functions to our local function library
	if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
	{
		EntryNode = NewObject<URigVMFunctionEntryNode>(CollapseNode->ContainedGraph, TEXT("Entry"));

		if(CollapseController->AddGraphNode(EntryNode, false))
		{
			EntryNode->Position = -Diagonal * 0.5f - FVector2D(250.f, 0.f);
			{
				TGuardValue<bool> SuspendNotifications(CollapseController->bSuspendNotifications, true);
				CollapseController->RefreshFunctionPins(EntryNode);
			}

			CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, EntryNode);
		}

		ReturnNode = NewObject<URigVMFunctionReturnNode>(CollapseNode->ContainedGraph, TEXT("Return"));

		if(CollapseController->AddGraphNode(ReturnNode, false))
		{
			ReturnNode->Position = FVector2D(Diagonal.X, -Diagonal.Y) * 0.5f + FVector2D(300.f, 0.f);
			{
				TGuardValue<bool> SuspendNotifications(CollapseController->bSuspendNotifications, true);
				CollapseController->RefreshFunctionPins(ReturnNode);
			}

			CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, ReturnNode);
		}
	}

	auto RewireLinksForInfo = [CollapsedPins, EntryNode, ReturnNode](URigVMController* InController, const FLinkCollapseInfo& CollapseInfo, bool bIsInput, bool bIsInterior)
	{
		const URigVMPin* OriginalPin = CollapseInfo.Pin;
		OriginalPin = CollapsedPins.FindChecked(OriginalPin);

		if (bIsInterior)
		{
			if (bIsInput)
			{
				OriginalPin = EntryNode->FindPin(OriginalPin->GetSegmentPath(true));
			}
			else
			{
				OriginalPin = ReturnNode->FindPin(OriginalPin->GetSegmentPath(true));
				if (OriginalPin == nullptr)
				{
					return;
				}

				URigVMPin* SourcePin = InController->GetGraph()->FindPin(CollapseInfo.Pin->GetPinPath());
				if (SourcePin == nullptr)
				{
					return;
				}

				URigVMPin* TargetPin = const_cast<URigVMPin*>(OriginalPin);

				if (!SourcePin->IsLinkedTo(TargetPin))
				{
					InController->AddLink(SourcePin, TargetPin, false);
				}
				return;
			}
		}
		else if (bIsInput)
		{
			URigVMPin* SourcePin = const_cast<URigVMPin*>(CollapseInfo.Pin);
			URigVMPin* TargetPin = const_cast<URigVMPin*>(OriginalPin);
			if (!SourcePin->IsLinkedTo(TargetPin))
			{
				InController->AddLink(SourcePin, TargetPin, false);
			}
			return;
		}

		if (OriginalPin == nullptr)
		{
			return;
		}

		for (const FLinkCollapseInfoPin& LinkedPinInfo : CollapseInfo.LinkedPins)
		{
			const URigVMPin* Pin = OriginalPin;
			if (!LinkedPinInfo.SegmentPath.IsEmpty())
			{
				Pin = Pin->FindSubPin(LinkedPinInfo.SegmentPath);
				if (!Pin)
				{
					continue;
				}
			}
				
			const URigVMPin* LinkedPin = LinkedPinInfo.Pin;
			if (bIsInterior)
			{
				LinkedPin = InController->GetGraph()->FindPin(LinkedPin->GetPinPath());
				if (LinkedPin == nullptr)
				{
					continue;
				}
			}

			URigVMPin* SourcePin = const_cast<URigVMPin*>(Pin);
			URigVMPin* TargetPin = const_cast<URigVMPin*>(LinkedPin);
			if (!SourcePin->IsLinkedTo(TargetPin))
			{
				InController->AddLink(SourcePin, TargetPin, false);
			}
		}
	};

	// create the new nodes within the collapse node
	if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
	{
		const FString TextContent = ExportNodesToText(NodeNames);
		const TArray<FName> ContainedNodeNames = CollapseController->ImportNodesFromText(TextContent, false);

		// move the nodes to the right place
		for (const FName& ContainedNodeName : ContainedNodeNames)
		{
			if (URigVMNode* ContainedNode = CollapseNode->GetContainedGraph()->FindNodeByName(ContainedNodeName))
			{
				if(!ContainedNode->IsInjected())
				{
					CollapseController->SetNodePosition(ContainedNode, ContainedNode->Position - Center, false, false);
				}
			}
		}

		for (const FLinkCollapseInfo& InputLinkInfo : InputLinkInfos)
		{
			RewireLinksForInfo(CollapseController, InputLinkInfo, true, true);
		}
		for (const FLinkCollapseInfo& OutputLinkInfo : OutputLinkInfos)
		{
			RewireLinksForInfo(CollapseController, OutputLinkInfo, false, true);
		}
	}

	for (const FLinkCollapseInfo& InputLinkInfo : InputLinkInfos)
	{
		RewireLinksForInfo(this, InputLinkInfo, true, false);
	}
	for (const FLinkCollapseInfo& OutputLinkInfo : OutputLinkInfos)
	{
		RewireLinksForInfo(this, OutputLinkInfo, false, false);
	}

	if (ReturnNode)
	{
		struct Local
		{
			static bool IsLinkedToEntryNode(URigVMNode* InNode, TMap<URigVMNode*, bool>& CachedMap)
			{
				if (InNode->IsA<URigVMFunctionEntryNode>())
				{
					return true;
				}

				if (!CachedMap.Contains(InNode))
				{
					CachedMap.Add(InNode, false);

					if (URigVMPin* ExecuteContextPin = InNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()))
					{
						TArray<URigVMPin*> SourcePins = ExecuteContextPin->GetLinkedSourcePins();
						for (URigVMPin* SourcePin : SourcePins)
						{
							if (IsLinkedToEntryNode(SourcePin->GetNode(), CachedMap))
							{
								CachedMap.FindOrAdd(InNode) = true;
								break;
							}
						}
					}
				}

				return CachedMap.FindChecked(InNode);
			}
		};

		// check if there is a last node on the top level block what we need to hook up
		TMap<URigVMNode*, bool> IsContainedNodeLinkedToEntryNode;

		TArray<URigVMNode*> NodesForExecutePin;
		NodesForExecutePin.Add(EntryNode);
		for (int32 NodeForExecutePinIndex = 0; NodeForExecutePinIndex < NodesForExecutePin.Num(); NodeForExecutePinIndex++)
		{
			URigVMNode* NodeForExecutePin = NodesForExecutePin[NodeForExecutePinIndex];
			if (!NodeForExecutePin->IsMutable())
			{
				continue;
			}

			TArray<URigVMNode*> TargetNodes = NodeForExecutePin->GetLinkedTargetNodes();
			for(URigVMNode* TargetNode : TargetNodes)
			{
				NodesForExecutePin.AddUnique(TargetNode);
			}

			// make sure the node doesn't have any mutable nodes connected to its executecontext
			URigVMPin* ExecuteContextPin = nullptr;
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(NodeForExecutePin))
			{
				TSharedPtr<FStructOnScope> UnitScope = UnitNode->ConstructStructInstance();
				if(UnitScope.IsValid())
				{
					FRigVMStruct* Unit = (FRigVMStruct*)UnitScope->GetStructMemory();
					if(Unit->IsForLoop())
					{
						ExecuteContextPin = NodeForExecutePin->FindPin(FRigVMStruct::ForLoopCompletedPinName.ToString());
					}
				}
			}

			if(ExecuteContextPin == nullptr)
			{
				ExecuteContextPin = NodeForExecutePin->FindPin(FRigVMStruct::ExecuteContextName.ToString());
			}

			if(ExecuteContextPin)
			{
				if(!ExecuteContextPin->IsExecuteContext())
				{
					continue;
				}

				if (ExecuteContextPin->GetDirection() != ERigVMPinDirection::IO &&
					ExecuteContextPin->GetDirection() != ERigVMPinDirection::Output)
				{
					continue;
				}

				if (ExecuteContextPin->GetTargetLinks().Num() > 0)
				{
					continue;
				}

				if (!Local::IsLinkedToEntryNode(NodeForExecutePin, IsContainedNodeLinkedToEntryNode))
				{
					continue;
				}

				if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
				{
					CollapseController->AddLink(ExecuteContextPin, ReturnNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), false);
				}
				break;
			}
		}
	}

	RemoveNodesByName(NodeNames, false, false);

	if (!InCollapseNodeName.IsEmpty() && CollapseNodeName != InCollapseNodeName)
	{		
		FString ValidName = GetSchema()->GetValidNodeName(Graph, InCollapseNodeName);
		if (ValidName == InCollapseNodeName)
		{
			RenameNode(CollapseNode, *ValidName, false);
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(CollapseAction);
	}

	return CollapseNode;
}

TArray<URigVMNode*> URigVMController::ExpandLibraryNode(URigVMLibraryNode* InNode, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return TArray<URigVMNode*>();
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return TArray<URigVMNode*>();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if(!GetSchema()->CanExpandNode(this, InNode))
	{
		return TArray<URigVMNode*>();
	}

	URigVMGraph* InnerGraph = InNode->GetContainedGraph();
	if (URigVMFunctionReferenceNode* RefNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		if (URigVMLibraryNode* LibraryNode = RefNode->LoadReferencedNode())
		{
			InnerGraph = LibraryNode->GetContainedGraph();
		}
		else
		{
			ReportError(TEXT("Cannot expand nodes from function reference because the source graph is not found."));
			return TArray<URigVMNode*>();			
		}
	}
	else if(InnerGraph == nullptr)
	{
		ReportError(TEXT("Cannot expand nodes from collapse node because the source graph is not found."));
		return TArray<URigVMNode*>();
	}

	TArray<URigVMNode*> ContainedNodes = InnerGraph->GetNodes();
	TArray<URigVMLink*> ContainedLinks = InnerGraph->GetLinks();
	if (ContainedNodes.Num() == 0)
	{
		return TArray<URigVMNode*>();
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMExpandNodeAction ExpandAction;

	if (bSetupUndoRedo)
	{
		ExpandAction = FRigVMExpandNodeAction(this, InNode);
		ExpandAction.SetTitle(FString::Printf(TEXT("Expand '%s' Node"), *InNode->GetName()));
		GetActionStack()->BeginAction(ExpandAction);
	}

	TArray<FName> NodeNames;
	TArray<FName> InjectedNodeNames;
	FBox2D Bounds = FBox2D(EForceInit::ForceInit);
	{
		TArray<URigVMNode*> FilteredNodes;
		for (URigVMNode* Node : ContainedNodes)
		{
			if (Cast<URigVMFunctionEntryNode>(Node) != nullptr ||
				Cast<URigVMFunctionReturnNode>(Node) != nullptr)
			{
				continue;
			}

			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (VariableNode->IsInputArgument())
				{
					continue;
				}
			}

			if(Node->IsInjected())
			{
				InjectedNodeNames.Add(Node->GetFName());
				continue;
			}
			
			NodeNames.Add(Node->GetFName());
			FilteredNodes.Add(Node);
			Bounds += Node->GetPosition();
		}
		ContainedNodes = FilteredNodes;
	}

	if (ContainedNodes.Num() == 0)
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(ExpandAction);
		}
		return TArray<URigVMNode*>();
	}

	// Find local variables that need to be added as member variables. If member variables of same name and type already
	// exist, they will be reused. If a local variable is not used, it will not be created.
	if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		TArray<FRigVMGraphVariableDescription> LocalVariables = InnerGraph->LocalVariables;
		TArray<FRigVMExternalVariable> CurrentVariables = GetAllVariables();
		TArray<FRigVMGraphVariableDescription> VariablesToAdd;
		for (const URigVMNode* Node : InnerGraph->GetNodes())
		{
			if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (VariableNode->IsInputArgument())
				{
					continue;
				}
				
				for (FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
				{
					if (LocalVariable.Name == VariableNode->GetVariableName())
					{
						bool bVariableExists = false;
						bool bVariableIncompatible = false;
						FRigVMExternalVariable LocalVariableExternalType = LocalVariable.ToExternalVariable();
						for (FRigVMExternalVariable& CurrentVariable : CurrentVariables)
						{						
							if (CurrentVariable.GetName() == LocalVariable.Name)
							{
								if (!CurrentVariable.IsSameType(LocalVariableExternalType))
								{
									bVariableIncompatible = true;	
								}
								bVariableExists = true;
								break;
							}
						}

						if (!bVariableExists)
						{
							VariablesToAdd.Add(LocalVariable);	
						}
						else if(bVariableIncompatible)
						{
							ReportErrorf(TEXT("Found variable %s of incompatible type with a local variable inside function %s"), *LocalVariable.Name.ToString(), *FunctionReferenceNode->GetReferencedFunctionHeader().Name.ToString());
							if (bSetupUndoRedo)
							{
								GetActionStack()->CancelAction(ExpandAction);
							}
							return TArray<URigVMNode*>();
						}
						break;
					}
				}
			}
		}

		if (RequestNewExternalVariableDelegate.IsBound())
		{
			for (const FRigVMGraphVariableDescription& OldVariable : VariablesToAdd)
			{
				RequestNewExternalVariableDelegate.Execute(OldVariable, false, false);
			}
		}
	}

	FVector2D Diagonal = Bounds.Max - Bounds.Min;
	FVector2D Center = (Bounds.Min + Bounds.Max) * 0.5f;

	FString TextContent;
	if(URigVMController* InnerController = GetControllerForGraph(InnerGraph))
	{
		TextContent = InnerController->ExportNodesToText(NodeNames);
	}

	TArray<FName> ExpandedNodeNames = ImportNodesFromText(TextContent, false);
	TArray<URigVMNode*> ExpandedNodes;
	for (const FName& ExpandedNodeName : ExpandedNodeNames)
	{
		URigVMNode* ExpandedNode = Graph->FindNodeByName(ExpandedNodeName);
		check(ExpandedNode);
		ExpandedNodes.Add(ExpandedNode);
	}

	if (ExpandedNodeNames.Num() < NodeNames.Num()) // if a private function is found and the user decides not to localize, then the nodes count will not match
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(ExpandAction);
		}
		return TArray<URigVMNode*>();
	}

	TMap<FName, FName> NodeNameMap;
	for (int32 NodeNameIndex = 0, ExpandedNodeNameIndex = 0, InjectedNodeName = 0; ExpandedNodeNameIndex < ExpandedNodeNames.Num(); ExpandedNodeNameIndex++)
	{
		if (ExpandedNodes[ExpandedNodeNameIndex]->IsInjected())
		{
			NodeNameMap.Add(InjectedNodeNames[InjectedNodeName], ExpandedNodeNames[ExpandedNodeNameIndex]);
			InjectedNodeName++;
			continue;
		}
		NodeNameMap.Add(NodeNames[NodeNameIndex], ExpandedNodeNames[ExpandedNodeNameIndex]);
		SetNodePosition(ExpandedNodes[ExpandedNodeNameIndex], InNode->Position + ContainedNodes[NodeNameIndex]->Position - Center, false, false);
		NodeNameIndex++;
	}

	// a) store all of the pin defaults off the library node
	TMap<FString, FPinState> PinStates = GetPinStates(InNode);

	// b) create a map of new links to create by following the links to / from the library node
	TMap<FString, TArray<FString>> ToLibraryNode;
	TMap<FString, TArray<FString>> FromLibraryNode;
	TArray<URigVMPin*> LibraryPinsToTurnIntoConstant;

	TArray<URigVMLink*> LibraryLinks = InNode->GetLinks();
	for (URigVMLink* Link : LibraryLinks)
	{
		if (Link->GetTargetPin()->GetNode() == InNode)
		{
			if (!Link->GetTargetPin()->IsRootPin())
			{
				LibraryPinsToTurnIntoConstant.AddUnique(Link->GetTargetPin()->GetRootPin());
			}

			FString NodeName, PinPath;
			URigVMPin::SplitPinPathAtStart(Link->GetTargetPin()->GetPinPath(), NodeName, PinPath);
			ToLibraryNode.FindOrAdd(PinPath).Add(Link->GetSourcePin()->GetPinPath());
		}
		else
		{
			if (!Link->GetSourcePin()->IsRootPin())
			{
				LibraryPinsToTurnIntoConstant.AddUnique(Link->GetSourcePin()->GetRootPin());
			}

			FString NodeName, PinPath;
			URigVMPin::SplitPinPathAtStart(Link->GetSourcePin()->GetPinPath(), NodeName, PinPath);
			FromLibraryNode.FindOrAdd(PinPath).Add(Link->GetTargetPin()->GetPinPath());
		}
	}

	// c) create a map from the entry node to the contained graph
	TMap<FString, TArray<FString>> FromEntryNode;
	if (URigVMFunctionEntryNode* EntryNode = InnerGraph->GetEntryNode())
	{
		TArray<URigVMLink*> EntryLinks = EntryNode->GetLinks();

		for (URigVMNode* Node : InnerGraph->GetNodes())
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (VariableNode->IsInputArgument())
				{
					EntryLinks.Append(VariableNode->GetLinks());
				}
			}
		}
		
		for (URigVMLink* Link : EntryLinks)
		{
			if (Link->GetSourcePin()->GetNode() != EntryNode && !Link->GetSourcePin()->GetNode()->IsA<URigVMVariableNode>())
			{
				continue;
			}

			if (!Link->GetSourcePin()->IsRootPin())
			{
				if(URigVMPin* RootPinOnOtherNode = InNode->FindPin(Link->GetSourcePin()->GetRootPin()->GetName()))
				{
					LibraryPinsToTurnIntoConstant.AddUnique(RootPinOnOtherNode);
				}
			}

			FString NodeName, PinPath;
			URigVMPin::SplitPinPathAtStart(Link->GetSourcePin()->GetPinPath(), NodeName, PinPath);

			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Link->GetSourcePin()->GetNode()))
			{
				PinPath = VariableNode->GetVariableName().ToString();
			}

			TArray<FString>& LinkedPins = FromEntryNode.FindOrAdd(PinPath);

			URigVMPin::SplitPinPathAtStart(Link->GetTargetPin()->GetPinPath(), NodeName, PinPath);
			
			if (NodeNameMap.Contains(*NodeName))
			{
				NodeName = NodeNameMap.FindChecked(*NodeName).ToString();
				LinkedPins.Add(URigVMPin::JoinPinPath(NodeName, PinPath));	
			}
			else if (NodeName == TEXT("Return"))
			{
				LinkedPins.Add(URigVMPin::JoinPinPath(NodeName, PinPath));
			}
		}
	}

	// d) create a map from the contained graph from to the return node
	TMap<FString, TArray<FString>> ToReturnNode;
	if (URigVMFunctionReturnNode* ReturnNode = InnerGraph->GetReturnNode())
	{
		TArray<URigVMLink*> ReturnLinks = ReturnNode->GetLinks();
		for (URigVMLink* Link : ReturnLinks)
		{
			if (Link->GetTargetPin()->GetNode() != ReturnNode)
			{
				continue;
			}

			if (!Link->GetTargetPin()->IsRootPin())
			{
				if(URigVMPin* RootPinOnOtherNode = InNode->FindPin(Link->GetTargetPin()->GetRootPin()->GetName()))
				{
					LibraryPinsToTurnIntoConstant.AddUnique(RootPinOnOtherNode);
				}
			}

			FString NodeName, PinPath;
			URigVMPin::SplitPinPathAtStart(Link->GetTargetPin()->GetPinPath(), NodeName, PinPath);

			TArray<FString>& LinkedPins = ToReturnNode.FindOrAdd(PinPath);

			URigVMPin::SplitPinPathAtStart(Link->GetSourcePin()->GetPinPath(), NodeName, PinPath);
			
			if (NodeNameMap.Contains(*NodeName))
			{
				NodeName = NodeNameMap.FindChecked(*NodeName).ToString();
				LinkedPins.Add(URigVMPin::JoinPinPath(NodeName, PinPath));	
			}
			else if (NodeName == TEXT("Entry"))
			{
				LinkedPins.Add(URigVMPin::JoinPinPath(NodeName, PinPath));
			}
		}
	}

	// e) restore all pin states on pins linked to the entry node
	for (const TPair<FString, TArray<FString>>& FromEntryPair : FromEntryNode)
	{
		FString EntryPinPath = FromEntryPair.Key;
		const FPinState* CollapsedPinState = PinStates.Find(EntryPinPath);
		if (CollapsedPinState == nullptr)
		{
			continue;
		}

		for (const FString& EntryTargetLinkPinPath : FromEntryPair.Value)
		{
			if (URigVMPin* TargetPin = GetGraph()->FindPin(EntryTargetLinkPinPath))
			{
				ApplyPinState(TargetPin, *CollapsedPinState);
			}
		}
	}

	// f) create constants for all pins which had wires on sub pins
	TMap<FString, URigVMPin*> ConstantInputPins;
	TMap<FString, URigVMPin*> ConstantOutputPins;
	FVector2D ConstantInputPosition = InNode->Position + FVector2D(-Diagonal.X, -Diagonal.Y) * 0.5 + FVector2D(-200.f, 0.f);
	FVector2D ConstantOutputPosition = InNode->Position + FVector2D(Diagonal.X, -Diagonal.Y) * 0.5 + FVector2D(250.f, 0.f);
	for (URigVMPin* LibraryPinToTurnIntoConstant : LibraryPinsToTurnIntoConstant)
	{
		if (LibraryPinToTurnIntoConstant->GetDirection() == ERigVMPinDirection::Input ||
			LibraryPinToTurnIntoConstant->GetDirection() == ERigVMPinDirection::IO)
		{
			URigVMTemplateNode* ConstantNode =
				AddConstantNode(
					LibraryPinToTurnIntoConstant->GetCPPType(),
					*LibraryPinToTurnIntoConstant->GetCPPTypeObject()->GetPathName(),
					LibraryPinToTurnIntoConstant->GetDefaultValue(),
					ConstantInputPosition,
					FString::Printf(TEXT("Constant_%s"), *LibraryPinToTurnIntoConstant->GetName()),
					false);

			ConstantInputPosition += FVector2D(0.f, 150.f);

			URigVMPin* ValuePin = ConstantNode->FindPin(FRigVMDispatch_Constant::ValueName.ToString());
			ApplyPinState(ValuePin, GetPinState(LibraryPinToTurnIntoConstant));
			ConstantInputPins.Add(LibraryPinToTurnIntoConstant->GetName(), ValuePin);
			ExpandedNodes.Add(ConstantNode);
		}

		if (LibraryPinToTurnIntoConstant->GetDirection() == ERigVMPinDirection::Output ||
			LibraryPinToTurnIntoConstant->GetDirection() == ERigVMPinDirection::IO)
		{
			URigVMTemplateNode* ConstantNode =
				AddConstantNode(
					LibraryPinToTurnIntoConstant->GetCPPType(),
					*LibraryPinToTurnIntoConstant->GetCPPTypeObject()->GetPathName(),
					LibraryPinToTurnIntoConstant->GetDefaultValue(),
					ConstantOutputPosition,
					FString::Printf(TEXT("Constant_%s"), *LibraryPinToTurnIntoConstant->GetName()),
					false);

			ConstantOutputPosition += FVector2D(0.f, 150.f);

			URigVMPin* ValuePin = ConstantNode->FindPin(FRigVMDispatch_Constant::ValueName.ToString());
			ApplyPinState(ValuePin, GetPinState(LibraryPinToTurnIntoConstant));
			ConstantOutputPins.Add(LibraryPinToTurnIntoConstant->GetName(), ValuePin);
			ExpandedNodes.Add(ConstantNode);
		}
	}

	// g) remap all output / source pins and create a final list of links to create
	TMap<FString, FString> RemappedSourcePinsForInputs;
	TMap<FString, FString> RemappedSourcePinsForOutputs;
	TArray<URigVMPin*> LibraryPins = InNode->GetAllPinsRecursively();
	for (URigVMPin* LibraryPin : LibraryPins)
	{
		FString LibraryPinPath = LibraryPin->GetPinPath();
		FString LibraryNodeName;
		URigVMPin::SplitPinPathAtStart(LibraryPinPath, LibraryNodeName, LibraryPinPath);


		struct Local
		{
			static void UpdateRemappedSourcePins(FString SourcePinPath, FString TargetPinPath, TMap<FString, FString>& RemappedSourcePins)
			{
				while (!SourcePinPath.IsEmpty() && !TargetPinPath.IsEmpty())
				{
					RemappedSourcePins.FindOrAdd(SourcePinPath) = TargetPinPath;

					FString SourceLastSegment, TargetLastSegment;
					if (!URigVMPin::SplitPinPathAtEnd(SourcePinPath, SourcePinPath, SourceLastSegment))
					{
						break;
					}
					if (!URigVMPin::SplitPinPathAtEnd(TargetPinPath, TargetPinPath, TargetLastSegment))
					{
						break;
					}
				}
			}
		};

		if (LibraryPin->GetDirection() == ERigVMPinDirection::Input ||
			LibraryPin->GetDirection() == ERigVMPinDirection::IO)
		{
			if (const TArray<FString>* LibraryPinLinksPtr = ToLibraryNode.Find(LibraryPinPath))
			{
				const TArray<FString>& LibraryPinLinks = *LibraryPinLinksPtr;
				ensure(LibraryPinLinks.Num() == 1);

				const FString SourcePinPath = LibraryPinPath;
				FString TargetPinPath = LibraryPinLinks[0];

				// if the pin on the library node is represented by a constant
				// we need to remap to that instead.
				if(URigVMPin** ConstantPinPtr = ConstantInputPins.Find(SourcePinPath))
				{
					if(URigVMPin* ConstantPin = *ConstantPinPtr)
					{
						TargetPinPath = ConstantPin->GetPinPath();
					}
				}

				Local::UpdateRemappedSourcePins(SourcePinPath, TargetPinPath, RemappedSourcePinsForInputs);
			}
		}
		if (LibraryPin->GetDirection() == ERigVMPinDirection::Output ||
			LibraryPin->GetDirection() == ERigVMPinDirection::IO)
		{
			if (const TArray<FString>* LibraryPinLinksPtr = ToReturnNode.Find(LibraryPinPath))
			{
				const TArray<FString>& LibraryPinLinks = *LibraryPinLinksPtr;
				ensure(LibraryPinLinks.Num() == 1);

				const FString SourcePinPath = LibraryPinPath;
				FString TargetPinPath = LibraryPinLinks[0];

				// if the pin on the library node is represented by a constant
				// we need to remap to that instead.
				if(URigVMPin** ConstantPinPtr = ConstantOutputPins.Find(SourcePinPath))
				{
					if(URigVMPin* ConstantPin = *ConstantPinPtr)
					{
						TargetPinPath = ConstantPin->GetPinPath();
					}
				}

				Local::UpdateRemappedSourcePins(SourcePinPath, TargetPinPath, RemappedSourcePinsForOutputs);
			}
		}
	}

	// h) re-establish all of the links going to the left of the library node
	//    in this pass we only care about pins which have constants
	for (const TPair<FString, TArray<FString>>& ToLibraryNodePair : ToLibraryNode)
	{
		FString LibraryNodePinName, LibraryNodePinPathSuffix;
		if (!URigVMPin::SplitPinPathAtStart(ToLibraryNodePair.Key, LibraryNodePinName, LibraryNodePinPathSuffix))
		{
			LibraryNodePinName = ToLibraryNodePair.Key;
		}

		if (!ConstantInputPins.Contains(LibraryNodePinName))
		{
			continue;
		}

		URigVMPin* ConstantPin = ConstantInputPins.FindChecked(LibraryNodePinName);
		URigVMPin* TargetPin = LibraryNodePinPathSuffix.IsEmpty() ? ConstantPin : ConstantPin->FindSubPin(LibraryNodePinPathSuffix);
		check(TargetPin);

		for (const FString& SourcePinPath : ToLibraryNodePair.Value)
		{
			URigVMPin* SourcePin = GetGraph()->FindPin(*SourcePinPath);
			if (SourcePin && TargetPin)
			{
				if (!SourcePin->IsLinkedTo(TargetPin))
				{
					AddLink(SourcePin, TargetPin, false);
				}
			}
		}
	}

	// i) re-establish all of the links going to the left of the library node (based on the entry node)
	for (const TPair<FString, TArray<FString>>& FromEntryNodePair : FromEntryNode)
	{
		FString EntryPinPath = FromEntryNodePair.Key;
		FString EntryPinPathSuffix;

		const FString* RemappedSourcePin = RemappedSourcePinsForInputs.Find(EntryPinPath);
		while (RemappedSourcePin == nullptr)
		{
			FString LastSegment;
			if (!URigVMPin::SplitPinPathAtEnd(EntryPinPath, EntryPinPath, LastSegment))
			{
				break;
			}

			if (EntryPinPathSuffix.IsEmpty())
			{
				EntryPinPathSuffix = LastSegment;
			}
			else
			{
				EntryPinPathSuffix = URigVMPin::JoinPinPath(LastSegment, EntryPinPathSuffix);
			}

			RemappedSourcePin = RemappedSourcePinsForInputs.Find(EntryPinPath);
		}

		if (RemappedSourcePin == nullptr)
		{
			continue;
		}

		FString RemappedSourcePinPath = *RemappedSourcePin;
		if (!EntryPinPathSuffix.IsEmpty())
		{
			RemappedSourcePinPath = URigVMPin::JoinPinPath(RemappedSourcePinPath, EntryPinPathSuffix);
		}

		// remap the top level pin in case we need to insert a constant
		FString EntryPinName;
		if (!URigVMPin::SplitPinPathAtStart(FromEntryNodePair.Key, EntryPinPath, EntryPinPathSuffix))
		{
			EntryPinName = FromEntryNodePair.Key;
			EntryPinPathSuffix.Reset();
		}
		if (ConstantInputPins.Contains(EntryPinName))
		{
			URigVMPin* ConstantPin = ConstantInputPins.FindChecked(EntryPinName);
			URigVMPin* TargetPin = EntryPinPathSuffix.IsEmpty() ? ConstantPin : ConstantPin->FindSubPin(EntryPinPathSuffix);
			check(TargetPin);
			RemappedSourcePinPath = TargetPin->GetPinPath();
		}

		for (const FString& FromEntryNodeTargetPinPath : FromEntryNodePair.Value)
		{
			TArray<URigVMPin*> TargetPins;

			URigVMPin* SourcePin = GetGraph()->FindPin(RemappedSourcePinPath);
			URigVMPin* TargetPin = GetGraph()->FindPin(FromEntryNodeTargetPinPath);

			// potentially the target pin was on the entry node,
			// so there's no node been added for it. we'll have to look into the remapped
			// pins for the "FromLibraryNode" map.
			if(TargetPin == nullptr)
			{
				FString RemappedTargetPinPath = FromEntryNodeTargetPinPath;
				FString ReturnNodeName, ReturnPinPath;
				if (URigVMPin::SplitPinPathAtStart(RemappedTargetPinPath, ReturnNodeName, ReturnPinPath))
				{
					if(Cast<URigVMFunctionReturnNode>(InnerGraph->FindNode(ReturnNodeName)))
					{
						if(FromLibraryNode.Contains(ReturnPinPath))
						{
							const TArray<FString>& FromLibraryNodeTargetPins = FromLibraryNode.FindChecked(ReturnPinPath);
							for(const FString& FromLibraryNodeTargetPin : FromLibraryNodeTargetPins)
							{
								if(URigVMPin* MappedTargetPin = GetGraph()->FindPin(FromLibraryNodeTargetPin))
								{
									TargetPins.Add(MappedTargetPin);
								}
							}
						}
					}
				}
			}
			else
			{
				TargetPins.Add(TargetPin);
			}
			
			if (SourcePin)
			{
				for(URigVMPin* EachTargetPin : TargetPins)
				{
					if (!SourcePin->IsLinkedTo(EachTargetPin))
					{
						AddLink(SourcePin, EachTargetPin, false);
					}
				}
			}
		}
	}

	// j) re-establish all of the links going from the right of the library node
	//    in this pass we only check pins which have a constant
	for (const TPair<FString, TArray<FString>>& ToReturnNodePair : ToReturnNode)
	{
		FString LibraryNodePinName, LibraryNodePinPathSuffix;
		if (!URigVMPin::SplitPinPathAtStart(ToReturnNodePair.Key, LibraryNodePinName, LibraryNodePinPathSuffix))
		{
			LibraryNodePinName = ToReturnNodePair.Key;
		}

		if (!ConstantOutputPins.Contains(LibraryNodePinName))
		{
			continue;
		}

		URigVMPin* ConstantPin = ConstantOutputPins.FindChecked(LibraryNodePinName);
		URigVMPin* TargetPin = LibraryNodePinPathSuffix.IsEmpty() ? ConstantPin : ConstantPin->FindSubPin(LibraryNodePinPathSuffix);
		check(TargetPin);

		for (const FString& SourcePinpath : ToReturnNodePair.Value)
		{
			URigVMPin* SourcePin = GetGraph()->FindPin(*SourcePinpath);
			if (SourcePin && TargetPin)
			{
				if (!SourcePin->IsLinkedTo(TargetPin))
				{
					AddLink(SourcePin, TargetPin, false);
				}
			}
		}
	}

	// k) re-establish all of the links going from the right of the library node
	for (const TPair<FString, TArray<FString>>& FromLibraryNodePair : FromLibraryNode)
	{
		FString FromLibraryNodePinPath = FromLibraryNodePair.Key;
		FString FromLibraryNodePinPathSuffix;

		const FString* RemappedSourcePin = RemappedSourcePinsForOutputs.Find(FromLibraryNodePinPath);
		while (RemappedSourcePin == nullptr)
		{
			FString LastSegment;
			if (!URigVMPin::SplitPinPathAtEnd(FromLibraryNodePinPath, FromLibraryNodePinPath, LastSegment))
			{
				break;
			}

			if (FromLibraryNodePinPathSuffix.IsEmpty())
			{
				FromLibraryNodePinPathSuffix = LastSegment;
			}
			else
			{
				FromLibraryNodePinPathSuffix = URigVMPin::JoinPinPath(LastSegment, FromLibraryNodePinPathSuffix);
			}

			RemappedSourcePin = RemappedSourcePinsForOutputs.Find(FromLibraryNodePinPath);
		}

		if (RemappedSourcePin == nullptr)
		{
			continue;
		}

		FString RemappedSourcePinPath = *RemappedSourcePin;
		if (!FromLibraryNodePinPathSuffix.IsEmpty())
		{
			RemappedSourcePinPath = URigVMPin::JoinPinPath(RemappedSourcePinPath, FromLibraryNodePinPathSuffix);
		}

		// remap the top level pin in case we need to insert a constant
		FString ReturnPinName, ReturnPinPathSuffix;
		if (!URigVMPin::SplitPinPathAtStart(FromLibraryNodePair.Key, ReturnPinName, ReturnPinPathSuffix))
		{
			ReturnPinName = FromLibraryNodePair.Key;
			ReturnPinPathSuffix.Reset();
		}
		if (ConstantOutputPins.Contains(ReturnPinName))
		{
			URigVMPin* ConstantPin = ConstantOutputPins.FindChecked(ReturnPinName);
			URigVMPin* SourcePin = ReturnPinPathSuffix.IsEmpty() ? ConstantPin : ConstantPin->FindSubPin(ReturnPinPathSuffix);
			check(SourcePin);
			RemappedSourcePinPath = SourcePin->GetPinPath();
		}

		for (const FString& FromLibraryNodeTargetPinPath : FromLibraryNodePair.Value)
		{
			URigVMPin* SourcePin = GetGraph()->FindPin(*RemappedSourcePinPath);
			URigVMPin* TargetPin = GetGraph()->FindPin(FromLibraryNodeTargetPinPath);
			if (SourcePin && TargetPin)
			{
				if (!SourcePin->IsLinkedTo(TargetPin))
				{
					AddLink(SourcePin, TargetPin, false);
				}
			}
		}
	}

	// l) remove the library node from the graph
	RemoveNode(InNode, false, true);

	if (bSetupUndoRedo)
	{
		for (URigVMNode* ExpandedNode : ExpandedNodes)
		{
			ExpandAction.ExpandedNodePaths.Add(ExpandedNode->GetName());
		}
		GetActionStack()->EndAction(ExpandAction);
	}

	return ExpandedNodes;
}

FName URigVMController::PromoteCollapseNodeToFunctionReferenceNode(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand, const FString& InExistingFunctionDefinitionPath)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NAME_None;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Result = PromoteCollapseNodeToFunctionReferenceNode(Cast<URigVMCollapseNode>(Graph->FindNodeByName(InNodeName)), bSetupUndoRedo, InExistingFunctionDefinitionPath);
	if (Result)
	{
		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
								FString::Printf(TEXT("asset.get_controller_by_name('%s').promote_collapse_node_to_function_reference_node('%s')"),
													*GraphName,
													*GetSchema()->GetSanitizedNodeName(InNodeName.ToString())));
		}
		
		return Result->GetFName();
	}
	return NAME_None;
}

FName URigVMController::PromoteFunctionReferenceNodeToCollapseNode(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bRemoveFunctionDefinition)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NAME_None;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Result = PromoteFunctionReferenceNodeToCollapseNode(Cast<URigVMFunctionReferenceNode>(Graph->FindNodeByName(InNodeName)), bSetupUndoRedo, bRemoveFunctionDefinition);
	if (Result)
	{
		return Result->GetFName();
	}
	return NAME_None;
}

URigVMFunctionReferenceNode* URigVMController::PromoteCollapseNodeToFunctionReferenceNode(URigVMCollapseNode* InCollapseNode, bool bSetupUndoRedo, const FString& InExistingFunctionDefinitionPath)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}
	
	if (!IsValidNodeForGraph(InCollapseNode))
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* FunctionLibrary = Graph->GetDefaultFunctionLibrary();
	if (FunctionLibrary == nullptr)
	{
		return nullptr;
	}

	for (URigVMPin* Pin : InCollapseNode->GetPins())
	{
		if (Pin->IsWildCard())
		{
			ReportAndNotifyErrorf(TEXT("Cannot create function %s because it contains a wildcard pin %s"), *InCollapseNode->GetName(), *Pin->GetName());
			return nullptr;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	URigVMFunctionReferenceNode* FunctionRefNode = nullptr;

	// Create Function
	URigVMLibraryNode* FunctionDefinition = nullptr;
	if (!InExistingFunctionDefinitionPath.IsEmpty() && 
		ensureAlwaysMsgf(!FPackageName::IsShortPackageName(InExistingFunctionDefinitionPath), TEXT("Expected full path name for function definition path: \"%s\")"), *InExistingFunctionDefinitionPath))
	{
		FunctionDefinition = FindObject<URigVMLibraryNode>(nullptr, *InExistingFunctionDefinitionPath);
	}

	if (FunctionDefinition == nullptr)
	{
		if(URigVMController* LibraryController = GetControllerForGraph(FunctionLibrary))
		{
			FRigVMControllerCompileBracketScope LibraryCompileScope(LibraryController);

			const FString FunctionName = LibraryController->GetSchema()->GetValidNodeName(FunctionLibrary, InCollapseNode->GetName());
			FunctionDefinition = LibraryController->AddFunctionToLibrary(
				*FunctionName,
				InCollapseNode->GetPins().ContainsByPredicate([](const URigVMPin* Pin) -> bool
				{
					return Pin->IsExecuteContext() && (Pin->GetDirection() == ERigVMPinDirection::IO);
				}),
				FVector2D::ZeroVector, false);		
		}
	
		// Add interface pins in function
		if (FunctionDefinition)
		{
			if(URigVMController* DefinitionController = GetControllerForGraph(FunctionDefinition->GetContainedGraph()))
			{
				for(const URigVMPin* Pin : InCollapseNode->GetPins())
				{
					DefinitionController->AddExposedPin_Internal(Pin->GetFName(), Pin->GetDirection(), Pin->GetCPPType(), (Pin->GetCPPTypeObject() ? *Pin->GetCPPTypeObject()->GetPathName() : TEXT("")), Pin->GetDefaultValue(), false, /*bPrintPythonCommand=*/ false, /*bIsInputVariable=*/ false, /*bAllowExecutePinManagement=*/ true);
				}
			}
		}
	}

	// Copy inner graph from collapsed node to function
	if (FunctionDefinition)
	{
		FString TextContent;
		if(URigVMController* CollapseController = GetControllerForGraph(InCollapseNode->GetContainedGraph()))
		{
			TArray<FName> NodeNames;
			for (const URigVMNode* Node : InCollapseNode->GetContainedNodes())
			{
				if (Node->IsInjected())
				{
					continue;
				}
				
				NodeNames.Add(Node->GetFName());
			}
			TextContent = CollapseController->ExportNodesToText(NodeNames);
		}

		if(URigVMController* DefinitionController = GetControllerForGraph(FunctionDefinition->GetContainedGraph()))
		{
			DefinitionController->ImportNodesFromText(TextContent, false);
			if (FunctionDefinition->GetContainedGraph()->GetEntryNode() && InCollapseNode->GetContainedGraph()->GetEntryNode())
			{ 
				DefinitionController->SetNodePosition(FunctionDefinition->GetContainedGraph()->GetEntryNode(), InCollapseNode->GetContainedGraph()->GetEntryNode()->GetPosition(), false);
			}
			
			if (FunctionDefinition->GetContainedGraph()->GetReturnNode() && InCollapseNode->GetContainedGraph()->GetReturnNode())
			{ 
				DefinitionController->SetNodePosition(FunctionDefinition->GetContainedGraph()->GetReturnNode(), InCollapseNode->GetContainedGraph()->GetReturnNode()->GetPosition(), false);
			}

			for (const URigVMLink* InnerLink : InCollapseNode->GetContainedGraph()->GetLinks())
			{
				URigVMPin* SourcePin = InCollapseNode->GetGraph()->FindPin(InnerLink->GetSourcePinPath());
				URigVMPin* TargetPin = InCollapseNode->GetGraph()->FindPin(InnerLink->GetTargetPinPath());
				if (SourcePin && TargetPin)
				{
					if (!SourcePin->IsLinkedTo(TargetPin))
					{
						DefinitionController->AddLink(InnerLink->GetSourcePinPath(), InnerLink->GetTargetPinPath(), false);	
					}
				}				
			}
		}
	}

	// Remove collapse node, add function reference, and add external links
	if (FunctionDefinition)
	{
 		FString NodeName = InCollapseNode->GetName();
		FVector2D NodePosition = InCollapseNode->GetPosition();
		TMap<FString, FPinState> PinStates = GetPinStates(InCollapseNode);

		const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InCollapseNode, true);

		FastBreakLinkedPaths(LinkedPaths);
		RemoveNode(InCollapseNode, false, true);

		FunctionRefNode = AddFunctionReferenceNode(FunctionDefinition, NodePosition, NodeName, false);

		if (FunctionRefNode)
		{
			ApplyPinStates(FunctionRefNode, PinStates);
			RestoreLinkedPaths(LinkedPaths);
		}

		if (bSetupUndoRedo)
		{
			GetActionStack()->AddAction(FRigVMPromoteNodeAction(this, InCollapseNode, NodeName, FString()));
		}
	}

	return FunctionRefNode;
}

URigVMCollapseNode* URigVMController::PromoteFunctionReferenceNodeToCollapseNode(URigVMFunctionReferenceNode* InFunctionRefNode, bool bSetupUndoRedo, bool bRemoveFunctionDefinition)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}
	
	if (!IsValidNodeForGraph(InFunctionRefNode))
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMCollapseNode* FunctionDefinition = Cast<URigVMCollapseNode>(InFunctionRefNode->LoadReferencedNode());
	if (FunctionDefinition == nullptr)
	{
		return nullptr;
	}

	// Find local variables that need to be added as member variables. If member variables of same name and type already
	// exist, they will be reused. If a local variable is not used, it will not be created.
	TArray<FRigVMGraphVariableDescription> LocalVariables = FunctionDefinition->GetContainedGraph()->LocalVariables;
	TArray<FRigVMExternalVariable> CurrentVariables = GetAllVariables();
	TArray<FRigVMGraphVariableDescription> VariablesToAdd;
	for (const URigVMNode* Node : FunctionDefinition->GetContainedGraph()->GetNodes())
	{
		if (const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			for (FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
			{
				if (LocalVariable.Name == VariableNode->GetVariableName())
				{
					bool bVariableExists = false;
					bool bVariableIncompatible = false;
					FRigVMExternalVariable LocalVariableExternalType = LocalVariable.ToExternalVariable();
					for (FRigVMExternalVariable& CurrentVariable : CurrentVariables)
					{						
						if (CurrentVariable.GetName() == LocalVariable.Name)
						{
							if (!CurrentVariable.IsSameType(LocalVariableExternalType))
							{
								bVariableIncompatible = true;	
							}
							bVariableExists = true;
							break;
						}
					}

					if (!bVariableExists)
					{
						VariablesToAdd.Add(LocalVariable);	
					}
					else if(bVariableIncompatible)
					{
						ReportErrorf(TEXT("Found variable %s of incompatible type with a local variable inside function %s"), *LocalVariable.Name.ToString(), *FunctionDefinition->GetName());
						return nullptr;
					}
					break;
				}
			}
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);

	FString NodeName = InFunctionRefNode->GetName();
	FVector2D NodePosition = InFunctionRefNode->GetPosition();
	TMap<FString, FPinState> PinStates = GetPinStates(InFunctionRefNode);

	const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InFunctionRefNode, true);

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMPromoteNodeAction(this, InFunctionRefNode, NodeName, FunctionDefinition->GetPathName()));
	}

	FastBreakLinkedPaths(LinkedPaths);
	RemoveNode(InFunctionRefNode, false, true);

	if (RequestNewExternalVariableDelegate.IsBound())
	{
		for (const FRigVMGraphVariableDescription& OldVariable : VariablesToAdd)
		{
			RequestNewExternalVariableDelegate.Execute(OldVariable, false, false);
		}
	}

	URigVMCollapseNode* CollapseNode = DuplicateObject<URigVMCollapseNode>(FunctionDefinition, Graph, *NodeName);
	if(CollapseNode)
	{
		FRestoreLinkedPathSettings RestoreSettings;

		if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
		{
			TGuardValue<bool> SuspendNotifications(CollapseController->bSuspendNotifications, true);

			for (URigVMNode* Node : CollapseNode->GetContainedGraph()->GetNodes())
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
				{
					const TArray<FLinkedPath> VariableLinkedPaths = GetLinkedPaths(VariableNode);
					CollapseController->FastBreakLinkedPaths(VariableLinkedPaths);
					CollapseController->RepopulatePinsOnNode(VariableNode, true, false, true);
					CollapseController->RestoreLinkedPaths(VariableLinkedPaths, RestoreSettings);
				}
			}

			CollapseNode->GetContainedGraph()->LocalVariables.Empty();
		}		
				
		CollapseNode->NodeColor = FLinearColor::White;
		CollapseNode->Position = NodePosition;

		if(!AddGraphNode(CollapseNode, false))
		{
			return nullptr;
		}

		ApplyPinStates(CollapseNode, PinStates);

		Notify(ERigVMGraphNotifType::NodeAdded, CollapseNode);
		
		RestoreLinkedPaths(LinkedPaths, RestoreSettings);
	}

	if(bRemoveFunctionDefinition)
	{
		if(URigVMController* LibraryController = GetControllerForGraph(FunctionDefinition->GetRootGraph()))
		{
			LibraryController->RemoveFunctionFromLibrary(FunctionDefinition->GetFName(), false);
		}
	}

	return CollapseNode;
}

void URigVMController::SetReferencedFunction(URigVMFunctionReferenceNode* InFunctionRefNode, URigVMLibraryNode* InNewReferencedNode, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return;
	}
	
	FRigVMGraphFunctionHeader OldReferencedNode = InFunctionRefNode->GetReferencedFunctionHeader();
	IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>();
	InFunctionRefNode->ReferencedFunctionHeader = InNewReferencedNode->GetFunctionHeader(ClientHost ? ClientHost->GetRigVMGraphFunctionHost().GetInterface() : nullptr);
	
	if(!(OldReferencedNode == InFunctionRefNode->ReferencedFunctionHeader))
	{
		if(URigVMBuildData* BuildData = URigVMBuildData::Get())
		{
			BuildData->UnregisterFunctionReference(OldReferencedNode.LibraryPointer, InFunctionRefNode);
			BuildData->RegisterFunctionReference(InFunctionRefNode->ReferencedFunctionHeader.LibraryPointer, InFunctionRefNode);
		}
	}


	if(URigVMController* ReferenceController = GetControllerForGraph(InFunctionRefNode->GetGraph()))
	{
		ReferenceController->Notify(ERigVMGraphNotifType::NodeReferenceChanged, InFunctionRefNode);
	}
}

void URigVMController::RefreshFunctionPins(URigVMNode* InNode, bool bSetupUndoRedo)
{
	if (InNode == nullptr)
	{
		return;
	}

	URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(InNode);
	URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(InNode);

	if (EntryNode || ReturnNode)
	{
		const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InNode);
		FastBreakLinkedPaths(LinkedPaths, bSetupUndoRedo);
		RepopulatePinsOnNode(InNode, false, false, true);
		FRestoreLinkedPathSettings Settings;
		RestoreLinkedPaths(LinkedPaths, Settings, bSetupUndoRedo);
	}
}

void URigVMController::ReportRemovedLink(const FString& InSourcePinPath, const FString& InTargetPinPath, const FString& Reason)
{
	if(bSuspendNotifications)
	{
		return;
	}
	if(!IsValidGraph())
	{
		return;
	}
	
	const URigVMPin* TargetPin = GetGraph()->FindPin(InTargetPinPath);
	FString TargetNodeName, TargetSegmentPath;
	if(!URigVMPin::SplitPinPathAtStart(InTargetPinPath, TargetNodeName, TargetSegmentPath))
	{
		TargetSegmentPath = InTargetPinPath;
	}
	
	ReportWarningf(TEXT("Link '%s' -> '%s' was removed.%s"), *InSourcePinPath, *InTargetPinPath, *(Reason.IsEmpty() ? Reason : TEXT(" ") + Reason));
	SendUserFacingNotification(
		FString::Printf(TEXT("Link to target pin '%s' was removed."), *TargetSegmentPath),
		0.f, TargetPin, TEXT("MessageLog.Note")
	);
}

bool URigVMController::RemoveNodes(TArray<URigVMNode*> InNodes, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if(InNodes.Num() == 0)
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	bool bBulkEditDialogShown = false;
	for(const URigVMNode* InNode : InNodes)
	{
		if (!IsValidNodeForGraph(InNode))
		{
			return false;
		}

		if(bEnableSchemaRemoveNodeCheck)
		{
			if(!GetSchema()->CanRemoveNode(this, InNode))
			{
				ReportErrorf(TEXT("Our schema does not allow to remove node %s"), *InNode->GetNodePath());
				return false;
			}
		}

		if (InNode->IsInjected())
		{
			URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo();
			if (InjectionInfo->GetPin()->GetInjectedNodes().Last() != InjectionInfo)
			{
				ReportErrorf(TEXT("Cannot remove injected node %s as it is not the last injection on the pin"), *InNode->GetNodePath());
				return false;
			}
		}

		if (bSetupUndoRedo)
		{
			// don't allow deletion of function entry / return nodes
			if ((Cast<URigVMFunctionEntryNode>(InNode) != nullptr && InNode->GetName() == TEXT("Entry")) ||
				(Cast<URigVMFunctionReturnNode>(InNode) != nullptr && InNode->GetName() == TEXT("Return")))
			{
				// due to earlier bugs in the copy & paste code entry and return nodes could end up in
				// root graphs - in those cases we allow deletion
				if(!Graph->IsRootGraph())
				{
					return false;
				}
			}

			// check if the operation will cause to dirty assets
			if(!bBulkEditDialogShown)
			{
				if(const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode))
				{
					if(URigVMFunctionLibrary* OuterLibrary = Graph->GetTypedOuter<URigVMFunctionLibrary>())
					{
						if(URigVMLibraryNode* OuterFunction = OuterLibrary->FindFunctionForNode(Graph->GetTypedOuter<URigVMCollapseNode>()))
						{
							const FName VariableToRemove = VariableNode->GetVariableName();
							bool bIsLocalVariable = false;
							for (const FRigVMGraphVariableDescription& VariableDescription : OuterFunction->GetContainedGraph()->GetLocalVariables(true))
							{
								if (VariableDescription.Name == VariableToRemove)
								{
									bIsLocalVariable = true;
									break;
								}
							}

							if (!bIsLocalVariable)
							{
								TArray<FRigVMExternalVariable> ExternalVariablesWithoutVariableNode;
								{
									URigVMGraph* EditedGraph = InNode->GetGraph();
									TGuardValue<TArray<TObjectPtr<URigVMNode>>> TemporaryRemoveNodes(EditedGraph->Nodes, TArray<TObjectPtr<URigVMNode>>());
									ExternalVariablesWithoutVariableNode = EditedGraph->GetExternalVariables();
								}

								bool bFoundExternalVariable = false;
								for(const FRigVMExternalVariable& ExternalVariable : ExternalVariablesWithoutVariableNode)
								{
									if(ExternalVariable.GetName() == VariableToRemove)
									{
										bFoundExternalVariable = true;
										break;
									}
								}

								if(!bFoundExternalVariable)
								{
									for (const FRigVMGraphVariableDescription& VariableDescription : OuterFunction->GetContainedGraph()->GetInputVariables())
									{
										if (VariableDescription.Name == VariableToRemove)
										{
											bFoundExternalVariable = true;
											break;
										}
									}
								}

								if(!bFoundExternalVariable)
								{
									if(URigVMController* FunctionController = GetControllerForGraph(OuterFunction->GetContainedGraph()))
									{
										if(FunctionController->RequestBulkEditDialogDelegate.IsBound())
										{
											FRigVMController_BulkEditResult Result = FunctionController->RequestBulkEditDialogDelegate.Execute(OuterFunction, ERigVMControllerBulkEditType::RemoveVariable);
											if(Result.bCanceled)
											{
												return false;
											}
											bSetupUndoRedo = Result.bSetupUndoRedo;
											bBulkEditDialogShown = true;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
	
	TGuardValue<bool> GuardCompactness(bIgnoreRerouteCompactnessChanges, true);

	// remove injected nodes from the list as needed 
	TArray<URigVMNode*> FilteredNodes;
	FilteredNodes.Reserve(InNodes.Num());
	for(URigVMNode* InNode : InNodes)
	{
		if(const URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo())
		{
			if(const URigVMNode* OuterNode = InjectionInfo->GetTypedOuter<URigVMNode>())
			{
				if(InNodes.Contains(OuterNode))
				{
					continue;
				}
			}
		}
		FilteredNodes.Add(InNode);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		if(FilteredNodes.Num() == 1)
		{
			Action.SetTitle(FString::Printf(TEXT("Remove %s Node"), *FilteredNodes[0]->GetNodeTitle()));
		}
		else
		{
			static const FString RemoveNodesTitle = TEXT("Remove nodes");
			Action.SetTitle(RemoveNodesTitle);
		}
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMRemoveNodesAction(this, FilteredNodes));
	}

	for(URigVMNode* InNode : FilteredNodes)
	{
		if (URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo())
		{
			URigVMPin* Pin = InjectionInfo->GetPin();
			check(Pin);

			if (!EjectNodeFromPin(Pin->GetPinPath(), bSetupUndoRedo))
			{
				if (bSetupUndoRedo)
				{
					GetActionStack()->CancelAction(Action);
				}
					
				return false;
			}
			
			if (InjectionInfo->bInjectedAsInput)
			{
				if (InjectionInfo->InputPin)
				{
					URigVMPin* LastInputPin = Pin->GetPinForLink();
					RewireLinks(InjectionInfo->InputPin, LastInputPin, true, bSetupUndoRedo);
				}
			}
			else
			{
				if (InjectionInfo->OutputPin)
				{
					URigVMPin* LastOutputPin = Pin->GetPinForLink();
					RewireLinks(InjectionInfo->OutputPin, LastOutputPin, false, bSetupUndoRedo);
				}
			}
		}

		
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
		{
			// If we are removing a reference, remove the function references to this node in the function library
			if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(LibraryNode))
			{
				if(URigVMBuildData* BuildData = URigVMBuildData::Get())
				{
					BuildData->UnregisterFunctionReference(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer, FunctionReferenceNode);
				}
			}
			// If we are removing a function, remove all the references first
			else if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
			{
				if(URigVMBuildData* BuildData = URigVMBuildData::Get())
				{
					FRigVMGraphFunctionIdentifier Identifier = LibraryNode->GetFunctionIdentifier();
					if (const FRigVMFunctionReferenceArray* ReferencesEntry = BuildData->FindFunctionReferences(Identifier))
					{
						// make a copy since we'll be modifying the array
						TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > FunctionReferences = ReferencesEntry->FunctionReferences;
						for (const TSoftObjectPtr<URigVMFunctionReferenceNode>& FunctionReferencePtr : FunctionReferences)
						{
							if (!ReferencesEntry->FunctionReferences.Contains(FunctionReferencePtr))
							{
								continue;
							}
							
							if (FunctionReferencePtr.IsValid())
							{
								if(URigVMController* ReferenceController = GetControllerForGraph(FunctionReferencePtr->GetGraph()))
								{
									ReferenceController->RemoveNode(FunctionReferencePtr.Get(), bSetupUndoRedo, false);
								}
							}
						}
					}

					BuildData->UnregisterPublicFunction(Identifier);
					BuildData->GraphFunctionReferences.Remove(Identifier);
				}
				
				for(const auto& Pair : FunctionLibrary->LocalizedFunctions)
				{
					if(Pair.Value == LibraryNode)
					{
						FunctionLibrary->LocalizedFunctions.Remove(Pair.Key);
						break;
					}
				}

				if (FunctionLibrary->PublicFunctionNames.Contains(LibraryNode->GetFName()))
				{
					FunctionLibrary->PublicFunctionNames.Remove(LibraryNode->GetFName());

					if (bSetupUndoRedo)
					{
						GetActionStack()->AddAction(FRigVMMarkFunctionPublicAction(this, LibraryNode->GetFName(), true));
					}
				}

				if (FunctionLibrary->FunctionToVariant.Contains(LibraryNode->GetFName()))
				{
					FunctionLibrary->FunctionToVariant.Remove(LibraryNode->GetFName());
				}
			}
		}

		SelectNode(InNode, false, false);

		TArray<URigVMPin*> Pins = InNode->GetPins();
		Pins.Append(InNode->GetOrphanedPins());
		for (URigVMPin* Pin : Pins)
		{
			// Remove injected nodes
			while (Pin->HasInjectedNodes())
			{
				RemoveInjectedNode(Pin->GetPinPath(), Pin->GetDirection() != ERigVMPinDirection::Output, false);
			}
		
			// breaking links also removes injected nodes 
			BreakAllLinks(Pin, true, false);
			BreakAllLinks(Pin, false, false);
			BreakAllLinksRecursive(Pin, true, false, false);
			BreakAllLinksRecursive(Pin, false, false, false);
		}

		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode))
		{
			URigVMGraph* SubGraph = CollapseNode->GetContainedGraph();

			if(URigVMController* SubGraphController = GetControllerForGraph(SubGraph))
			{
				TArray<URigVMNode*> ContainedNodes = SubGraph->GetNodes();
				TGuardValue<bool> SuspendTemplateComputation(bSuspendTemplateComputation, true);
				TGuardValue<bool> SubGraphSuspendTemplateComputation(SubGraphController->bSuspendTemplateComputation, true);
				TGuardValue<bool> SubGraphDisableSchemaRemoveNodeCheck(SubGraphController->bEnableSchemaRemoveNodeCheck, false);
				SubGraphController->RemoveNodes(ContainedNodes, false, false);

				if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
				{
					if(FRigVMClient* Client = ClientHost->GetRigVMClient())
					{
						(void)Client->RemoveController(SubGraph);
					}
				}
			}
		}
		
		Graph->Nodes.Remove(InNode);
		Notify(ERigVMGraphNotifType::NodeRemoved, InNode);
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		if (Graph->IsA<URigVMFunctionLibrary>())
		{
			for(const URigVMNode* InNode : FilteredNodes)
			{
				const FString NodeName = GetSchema()->GetSanitizedNodeName(InNode->GetName());
				
				RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
									FString::Printf(TEXT("library_controller.remove_function_from_library('%s')"),
													*NodeName));
			}
		}
		else
		{
			TArray<FString> NodePaths;
			Algo::Transform(FilteredNodes, NodePaths, [this](const URigVMNode* Node)
			{
				static constexpr TCHAR Format[] = TEXT("'%s'");
				return FString::Printf(Format, *GetSchema()->GetSanitizedPinPath(Node->GetNodePath()));
			});

			const FString NodePathsJoined = FString::Join(NodePaths, TEXT(", "));
			
			FString PythonCmd = FString::Printf(TEXT("asset.get_controller_by_name('%s')."), *GraphName );
			PythonCmd += FString::Printf(TEXT("remove_node_by_name([%s])"), *NodePathsJoined);
			
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), PythonCmd );
		}
	}

	for(URigVMNode* InNode : FilteredNodes)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode))
		{
			Notify(ERigVMGraphNotifType::VariableRemoved, VariableNode);

			if (URigVMLibraryNode* LibraryNode = Cast<URigVMNode>(VariableNode)->FindFunctionForNode())
			{
				if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
				{
					PatchExternalVariablesToArguments(bSetupUndoRedo);
				}
			}
		}
		
		if (URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo())
		{
			DestroyObject(InjectionInfo);
		}

		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode))
		{
			DestroyObject(CollapseNode->GetContainedGraph());
		}

		DestroyObject(InNode);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::RemoveNodesByName(const TArray<FName>& InNodeNames, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<URigVMNode*> Nodes;
	Algo::Transform(InNodeNames, Nodes, [this, Graph](const FName& NodeName) -> URigVMNode*
	{
		URigVMNode* Node = Graph->FindNodeByName(NodeName);
		if(Node == nullptr)
		{
			ReportErrorf(TEXT("Cannot find node '%s'."), *NodeName.ToString());
		}
		return Node;
	});

	if(Nodes.Contains(nullptr))
	{
		return false;
	}
	
	return RemoveNodes(Nodes, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::RemoveNode(URigVMNode* InNode, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return RemoveNodes({InNode}, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::RemoveNodeByName(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return RemoveNodesByName({InNodeName}, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::RenameNode(URigVMNode* InNode, const FName& InNewName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	FName ValidNewName = *GetSchema()->GetValidNodeName(GetGraph(), InNewName.ToString());

	if(!GetSchema()->CanRenameNode(this, InNode, ValidNewName))
	{
		return false;
	}

	const FString OldName = InNode->GetName();
	FRigVMRenameNodeAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMRenameNodeAction(this, InNode->GetFName(), ValidNewName);
		GetActionStack()->BeginAction(Action);
	}

	// loop over all links and remove them
	const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InNode);
	FastBreakLinkedPaths(LinkedPaths);

	const FSoftObjectPath PreviousObjectPath(InNode);
	InNode->PreviousName = InNode->GetFName();
	if (!RenameObject(InNode, *ValidNewName.ToString()))
	{
		GetActionStack()->CancelAction(Action);
		return false;
	}

	InNode->IncrementVersion();
	Notify(ERigVMGraphNotifType::FunctionRenamed, InNode);
	Notify(ERigVMGraphNotifType::NodeRenamed, InNode);

	// update the links once more
	FRestoreLinkedPathSettings Settings;
	Settings.NodeNameMap.Add(InNode->PreviousName.ToString(), InNode->GetName());
	RestoreLinkedPaths(LinkedPaths, Settings);

	if(URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
		{
			// rename each function reference
			if(URigVMBuildData* BuildData = URigVMBuildData::Get())
			{
				BuildData->ForEachFunctionReference(LibraryNode->GetFunctionIdentifier(), [this, InNewName](URigVMFunctionReferenceNode* ReferenceNode)
				{
					if(URigVMController* ReferenceController = GetControllerForGraph(ReferenceNode->GetGraph()))
					{
						ReferenceController->RenameNode(ReferenceNode, InNewName, false);
					}
				});
			}

			if (FunctionLibrary->PublicFunctionNames.Contains(InNode->PreviousName))
			{
				FunctionLibrary->PublicFunctionNames.Remove(InNode->PreviousName);
				FunctionLibrary->PublicFunctionNames.Add(ValidNewName);
			}

			if (FunctionLibrary->FunctionToVariant.Contains(InNode->PreviousName))
			{
				FunctionLibrary->FunctionToVariant.Remove(InNode->PreviousName);
				FunctionLibrary->FunctionToVariant.Add(ValidNewName);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("library_controller.rename_function('%s', '%s')"),
										*OldName,
										*InNewName.ToString()));
	}

	return true;
}

bool URigVMController::SelectNode(URigVMNode* InNode, bool bSelect, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (InNode->IsSelected() == bSelect)
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FName> NewSelection = Graph->GetSelectNodes();
	if (bSelect)
	{
		NewSelection.AddUnique(InNode->GetFName());
	}
	else
	{
		NewSelection.Remove(InNode->GetFName());
	}

	return SetNodeSelection(NewSelection, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SelectNodeByName(const FName& InNodeName, bool bSelect, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	return SelectNode(Graph->FindNodeByName(InNodeName), bSelect, bSetupUndoRedo);
}

bool URigVMController::ClearNodeSelection(bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	return SetNodeSelection(TArray<FName>(), bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetNodeSelection(const TArray<FName>& InNodeNames, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FRigVMSetNodeSelectionAction Action(this, Graph, InNodeNames);
	bool bSelectionChanged = false;

	TArray<FName> PreviousSelection = Graph->GetSelectNodes();
	for (const FName& PreviouslySelectedNode : PreviousSelection)
	{
		if (!InNodeNames.Contains(PreviouslySelectedNode))
		{
			if(Graph->SelectedNodes.Remove(PreviouslySelectedNode) > 0)
			{
				bSelectionChanged = true;
			}
		}
	}

	for (const FName& InNodeName : InNodeNames)
	{
		if (URigVMNode* NodeToSelect = Graph->FindNodeByName(InNodeName))
		{
			int32 PreviousNum = Graph->SelectedNodes.Num();
			Graph->SelectedNodes.AddUnique(InNodeName);
			if (PreviousNum != Graph->SelectedNodes.Num())
			{
				bSelectionChanged = true;
			}
		}
	}

	if (bSetupUndoRedo)
	{
		if (bSelectionChanged)
		{
			const TArray<FName>& SelectedNodes = Graph->GetSelectNodes();
			if (SelectedNodes.Num() == 0)
			{
				Action.SetTitle(TEXT("Deselect all nodes."));
			}
			else
			{
				if (SelectedNodes.Num() == 1)
				{
					Action.SetTitle(FString::Printf(TEXT("Selected node '%s'."), *SelectedNodes[0].ToString()));
				}
				else
				{
					Action.SetTitle(TEXT("Selected multiple nodes."));
				}
			}
			GetActionStack()->AddAction(Action, false, false);
		}
	}

	if (bSelectionChanged)
	{
		UpdateGraphSectionsIfRequired();
		
		Notify(ERigVMGraphNotifType::NodeSelectionChanged, nullptr);
	}

	if (bPrintPythonCommand)
	{
		FString ArrayStr = TEXT("[");
		for (auto It = InNodeNames.CreateConstIterator(); It; ++It)
		{
			ArrayStr += TEXT("'") + GetSchema()->GetSanitizedNodeName(It->ToString()) + TEXT("'");
			if (It.GetIndex() < InNodeNames.Num() - 1)
			{
				ArrayStr += TEXT(", ");
			}
		}
		ArrayStr += TEXT("]");

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_selection(%s)"),
											*GraphName,
											*ArrayStr));
	}

	return bSelectionChanged;
}

bool URigVMController::SelectLinkedNodes(const TArray<FName>& InNodeNames, bool bSelectSourceNodes, bool bClearSelection, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FName> NewSelection;
	for(const FName& NodeName : InNodeNames)
	{
		NewSelection.AddUnique(NodeName);
	} 
	
	for (int32 Index = 0; Index < NewSelection.Num(); Index++)
	{
		const URigVMNode* Node = Graph->FindNodeByName(NewSelection[Index]);
		if (!Node)
		{
			continue;
		}

		const bool bIsLinkedNode = !InNodeNames.Contains(Node->GetFName());
		if (bSelectSourceNodes || bIsLinkedNode)
		{
			const bool bSkipMutableNodes = bIsLinkedNode && !bSelectSourceNodes;
			const TArray<URigVMNode*> LinkedNodes = Node->GetLinkedSourceNodes();
			for (const URigVMNode* LinkedNode : LinkedNodes)
			{
				if(bSkipMutableNodes && LinkedNode->IsMutable())
				{
					continue;
				}
				NewSelection.AddUnique(LinkedNode->GetFName());
			}
		}
		if (!bSelectSourceNodes)
		{
			const TArray<URigVMNode*> LinkedNodes = Node->GetLinkedTargetNodes();
			for (const URigVMNode* LinkedNode : LinkedNodes)
			{
				NewSelection.AddUnique(LinkedNode->GetFName());
			}
		}
	}

	if(!bClearSelection)
	{
		const TArray<FName> PreviousSelection = Graph->GetSelectNodes();	
		for(const FName& NodeName : PreviousSelection)
		{
			NewSelection.AddUnique(NodeName);
		} 
	}

	const bool bSelectionChanged = SetNodeSelection(NewSelection, bSetupUndoRedo, false);

	if (bPrintPythonCommand)
	{
		FString ArrayStr = TEXT("[");
		for (auto It = InNodeNames.CreateConstIterator(); It; ++It)
		{
			ArrayStr += TEXT("'") + GetSchema()->GetSanitizedNodeName(It->ToString()) + TEXT("'");
			if (It.GetIndex() < InNodeNames.Num() - 1)
			{
				ArrayStr += TEXT(", ");
			}
		}
		ArrayStr += TEXT("]");

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').select_linked_nodes(%s, %s, %s)"),
											*GraphName,
											*ArrayStr,
											(bSelectSourceNodes) ? TEXT("True") : TEXT("False"),
											(bClearSelection) ? TEXT("True") : TEXT("False")));
	}

	return bSelectionChanged;
}

bool URigVMController::SelectNodeIslands(const TArray<FName>& InNodeNames, bool bClearSelection, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FName> NewSelection;
	for(const FName& NodeName : InNodeNames)
	{
		NewSelection.AddUnique(NodeName);
	} 
	
	if(!bClearSelection)
	{
		const TArray<FName> PreviousSelection = Graph->GetSelectNodes();	
		for(const FName& NodeName : PreviousSelection)
		{
			NewSelection.AddUnique(NodeName);
		} 
	}

	for (int32 Index = 0; Index < NewSelection.Num(); Index++)
	{
		const URigVMNode* Node = Graph->FindNodeByName(NewSelection[Index]);
		if (!Node)
		{
			continue;
		}
		
		TArray<URigVMNode*> LinkedNodes;
		LinkedNodes.Append(Node->GetLinkedSourceNodes());
		LinkedNodes.Append(Node->GetLinkedTargetNodes());
		
		for (const URigVMNode* LinkedNode : LinkedNodes)
		{
			NewSelection.AddUnique(LinkedNode->GetFName());
		}
	}

	const bool bSelectionChanged = SetNodeSelection(NewSelection, bSetupUndoRedo, false);

	if (bPrintPythonCommand)
	{
		FString ArrayStr = TEXT("[");
		for (auto It = InNodeNames.CreateConstIterator(); It; ++It)
		{
			ArrayStr += TEXT("'") + GetSchema()->GetSanitizedNodeName(It->ToString()) + TEXT("'");
			if (It.GetIndex() < InNodeNames.Num() - 1)
			{
				ArrayStr += TEXT(", ");
			}
		}
		ArrayStr += TEXT("]");

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').select_node_islands(%s, %s)"),
											*GraphName,
											*ArrayStr,
											(bClearSelection) ? TEXT("True") : TEXT("False")));
	}

	return bSelectionChanged;
}

bool URigVMController::SetNodePosition(URigVMNode* InNode, const FVector2D& InPosition, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(!GetSchema()->CanMoveNode(this, InNode, InPosition))
	{
		return false;
	}

	FRigVMSetNodePositionAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodePositionAction(this, InNode, InPosition);
		Action.SetTitle(FString::Printf(TEXT("Set Node Position")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->Position = InPosition;
	Notify(ERigVMGraphNotifType::NodePositionChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_position_by_name('%s', %s)"),
											*GraphName,
											*NodePath,
											*RigVMPythonUtils::Vector2DToPythonString(InPosition)));
	}

	return true;
}

bool URigVMController::SetNodePositionByName(const FName& InNodeName, const FVector2D& InPosition, bool bSetupUndoRedo, bool bMergeUndoAction, bool
                                             bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	return SetNodePosition(Node, InPosition, bSetupUndoRedo, bMergeUndoAction, bPrintPythonCommand);
}

bool URigVMController::SetNodeSize(URigVMNode* InNode, const FVector2D& InSize, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(!GetSchema()->CanResizeNode(this, InNode, InSize))
	{
		return false;
	}

	FRigVMSetNodeSizeAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeSizeAction(this, InNode, InSize);
		Action.SetTitle(FString::Printf(TEXT("Set Node Size")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->Size = InSize;
	Notify(ERigVMGraphNotifType::NodeSizeChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
							FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_size_by_name('%s', %s)"),
											*GraphName,
											*NodePath,
											*RigVMPythonUtils::Vector2DToPythonString(InSize)));
	}

	return true;
}

bool URigVMController::SetNodeSizeByName(const FName& InNodeName, const FVector2D& InSize, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	return SetNodeSize(Node, InSize, bSetupUndoRedo, bMergeUndoAction, bPrintPythonCommand);
}

bool URigVMController::SetNodeTitle(URigVMNode* InNode, const FString InNodeTitle, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (!GetSchema()->CanSetNodeTitle(this, InNode))
	{
		return false;
	}

	FRigVMSetNodeTitleAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeTitleAction(this, InNode, InNodeTitle);
		Action.SetTitle(FString::Printf(TEXT("Set Node Title")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->NodeTitle = InNodeTitle;
	Notify(ERigVMGraphNotifType::NodeTitleChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_title_by_name('%s', %s)"),
				*GraphName,
				*NodePath,
				*InNodeTitle));
	}

	return true;
}

bool URigVMController::SetNodeTitleByName(const FName& InNodeName, const FString InNodeTitle, bool bSetupUndoRedo, bool bMergeUndoAction)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	return SetNodeTitle(Node, InNodeTitle, bSetupUndoRedo, bMergeUndoAction);
}

bool URigVMController::SetNodeColor(URigVMNode* InNode, const FLinearColor& InColor, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(!GetSchema()->CanRecolorNode(this, InNode, InColor))
	{
		return false;
	}

	FRigVMSetNodeColorAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeColorAction(this, InNode, InColor);
		Action.SetTitle(FString::Printf(TEXT("Set Node Color")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->NodeColor = InColor;
	InNode->NodeColorType = ERigVMNodeColorType::UserDefined;
	Notify(ERigVMGraphNotifType::NodeColorChanged, InNode);

	if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
		{
			FunctionLibrary->ForEachReference(LibraryNode->GetFName(), [this](URigVMFunctionReferenceNode* ReferenceNode)
            {
				if(URigVMController* ReferenceController = GetControllerForGraph(ReferenceNode->GetGraph()))
				{
					ReferenceController->Notify(ERigVMGraphNotifType::NodeColorChanged, ReferenceNode);
				}
            });
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_color_by_name('%s', %s)"),
				*GraphName,
				*NodePath,
				*RigVMPythonUtils::LinearColorToPythonString(InColor)));
	}

	return true;
}

bool URigVMController::SetNodeColorByName(const FName& InNodeName, const FLinearColor& InColor, bool bSetupUndoRedo, bool bMergeUndoAction)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	return SetNodeColor(Node, InColor, bSetupUndoRedo, bMergeUndoAction);
}

bool URigVMController::SetNodeCategory(URigVMCollapseNode* InNode, const FString& InCategory, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (InNode->GetNodeCategory().Equals(InCategory))
	{
		return false;
	}

	FRigVMSetNodeCategoryAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeCategoryAction(this, InNode, InCategory);
		Action.SetTitle(FString::Printf(TEXT("Set Node Category")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->NodeCategory = InCategory;
	Notify(ERigVMGraphNotifType::NodeCategoryChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_category_by_name('%s', '%s')"),
				*GraphName,
				*NodePath,
				*InCategory));
	}

	return true;
}

bool URigVMController::SetNodeCategoryByName(const FName& InNodeName, const FString& InCategory, bool bSetupUndoRedo, bool bMergeUndoAction)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMCollapseNode* Node = Cast<URigVMCollapseNode>(Graph->FindNodeByName(InNodeName));
	return SetNodeCategory(Node, InCategory, bSetupUndoRedo, bMergeUndoAction);
}

bool URigVMController::SetNodeKeywords(URigVMCollapseNode* InNode, const FString& InKeywords, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (InNode->GetNodeKeywords() == InKeywords)
	{
		return false;
	}

	FRigVMSetNodeKeywordsAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeKeywordsAction(this, InNode, InKeywords);
		Action.SetTitle(FString::Printf(TEXT("Set Node Keywords")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->NodeKeywords = InKeywords;
	Notify(ERigVMGraphNotifType::NodeKeywordsChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_keywords_by_name('%s', '%s')"),
				*GraphName,
				*NodePath,
				*InKeywords));
	}

	return true;
}

bool URigVMController::SetNodeKeywordsByName(const FName& InNodeName, const FString& InKeywords, bool bSetupUndoRedo, bool bMergeUndoAction)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMCollapseNode* Node = Cast<URigVMCollapseNode>(Graph->FindNodeByName(InNodeName));
	return SetNodeKeywords(Node, InKeywords, bSetupUndoRedo, bMergeUndoAction);
}

bool URigVMController::SetNodeDescription(URigVMCollapseNode* InNode, const FString& InDescription, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (InNode->GetNodeDescription() == InDescription)
	{
		return false;
	}

	FRigVMSetNodeDescriptionAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetNodeDescriptionAction(this, InNode, InDescription);
		Action.SetTitle(FString::Printf(TEXT("Set Node Description")));
		GetActionStack()->BeginAction(Action);
	}

	InNode->NodeDescription = InDescription;
	Notify(ERigVMGraphNotifType::NodeDescriptionChanged, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action, bMergeUndoAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_node_description_by_name('%s', '%s')"),
				*GraphName,
				*NodePath,
				*InDescription));
	}

	return true;
}

bool URigVMController::SetNodeDescriptionByName(const FName& InNodeName, const FString& InDescription, bool bSetupUndoRedo, bool bMergeUndoAction)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMCollapseNode* Node = Cast<URigVMCollapseNode>(Graph->FindNodeByName(InNodeName));
	return SetNodeDescription(Node, InDescription, bSetupUndoRedo, bMergeUndoAction);
}

bool URigVMController::SetCommentText(URigVMNode* InNode, const FString& InCommentText, const int32& InCommentFontSize, const bool& bInCommentBubbleVisible, const bool& bInCommentColorBubble, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (URigVMCommentNode* CommentNode = Cast<URigVMCommentNode>(InNode))
	{
		if(CommentNode->CommentText == InCommentText && CommentNode->FontSize == InCommentFontSize && CommentNode->bBubbleVisible == bInCommentBubbleVisible && CommentNode->bColorBubble == bInCommentColorBubble)
		{
			return false;
		}

		FRigVMSetCommentTextAction Action;
		if (bSetupUndoRedo)
		{
			Action = FRigVMSetCommentTextAction(this, CommentNode, InCommentText, InCommentFontSize, bInCommentBubbleVisible, bInCommentColorBubble);
			Action.SetTitle(FString::Printf(TEXT("Set Comment Text")));
			GetActionStack()->BeginAction(Action);
		}

		CommentNode->CommentText = InCommentText;
		CommentNode->FontSize = InCommentFontSize;
		CommentNode->bBubbleVisible = bInCommentBubbleVisible;
		CommentNode->bColorBubble = bInCommentColorBubble;
		Notify(ERigVMGraphNotifType::CommentTextChanged, InNode);

		if (bSetupUndoRedo)
		{
			GetActionStack()->EndAction(Action);
		}

		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			const FString NodePath = GetSchema()->GetSanitizedPinPath(CommentNode->GetNodePath());

			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
				FString::Printf(TEXT("asset.get_controller_by_name('%s').set_comment_text_by_name('%s', '%s')"),
				*GraphName,
				*NodePath,
				*InCommentText));
		}

		return true;
	}

	return false;
}

bool URigVMController::SetCommentTextByName(const FName& InNodeName, const FString& InCommentText, const int32& InCommentFontSize, const bool& bInCommentBubbleVisible, const bool& bInCommentColorBubble, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	return SetCommentText(Node, InCommentText, InCommentFontSize, bInCommentBubbleVisible, bInCommentColorBubble, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::RenameVariable(const FName& InOldName, const FName& InNewName, bool bSetupUndoRedo)
{
	if (InOldName == InNewName)
	{
		ReportWarning(TEXT("RenameVariable: InOldName and InNewName are equal."));
		return false;
	}
	
	const FRigVMExternalVariable Variable = GetVariableByName(InOldName);
	if (Variable.GetGuid().IsValid())
	{
		return RenameVariable(Variable.GetGuid(), InNewName, bSetupUndoRedo);
	}
	return false;
}

bool URigVMController::RenameVariable(const FGuid& InGuid, const FName& InNewName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription> ExistingVariables = Graph->GetVariableDescriptions();
	for (const FRigVMGraphVariableDescription& ExistingVariable : ExistingVariables)
	{
		if (ExistingVariable.Name == InNewName)
		{
			ReportErrorf(TEXT("Cannot rename variable to '%s' - variable already exists."), *InNewName.ToString());
			return false;
		}
	}

	// If there is a local variable with the old name, a rename of the blueprint member variable does not affect this graph
	for (FRigVMGraphVariableDescription& LocalVariable : Graph->GetLocalVariables(true))
	{
		if (LocalVariable.Guid == InGuid)
		{
			return false;
		}
	}

	const FRigVMExternalVariable Variable = GetVariableByGuid(InGuid);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMRenameVariableAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMRenameVariableAction(this, InGuid, Variable.GetName(), InNewName);
		Action.SetTitle(FString::Printf(TEXT("Rename Variable")));
		GetActionStack()->BeginAction(Action);
	}

	TArray<URigVMNode*> RenamedNodes;
	for (URigVMNode* Node : Graph->Nodes)
	{
		if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableGuid() == InGuid)
			{
				VariableNode->FindPin(URigVMVariableNode::VariableName)->DefaultValue = InNewName.ToString();
				RenamedNodes.Add(Node);
			}
		}
	}

	for (URigVMNode* RenamedNode : RenamedNodes)
	{
		Notify(ERigVMGraphNotifType::VariableRenamed, RenamedNode);
		if (!bSuspendNotifications)
		{
			(void)MarkPackageDirty(Graph);
		}
	}

	if (bSetupUndoRedo)
	{
		if (RenamedNodes.Num() > 0)
		{
			GetActionStack()->EndAction(Action);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	return RenamedNodes.Num() > 0;
}

bool URigVMController::RenameParameter(const FName& InOldName, const FName& InNewName, bool bSetupUndoRedo)
{
	ReportWarning(TEXT("RenameParameter has been deprecated. Please use RenameVariable instead."));
	return false;
}

bool URigVMController::SetPinExpansion(const FString& InPinPath, bool bIsExpanded, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = SetPinExpansion(Pin, bIsExpanded, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_expansion('%s', %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			(bIsExpanded) ? TEXT("True") : TEXT("False")));
	}

	return bSuccess;
}

bool URigVMController::SetPinExpansion(URigVMPin* InPin, bool bIsExpanded, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	// If there is nothing to do, just return success
	if (InPin->GetSubPins().Num() == 0 || InPin->IsExpanded() == bIsExpanded)
	{
		return true;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FRigVMSetPinExpansionAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinExpansionAction(this, InPin, bIsExpanded);
		Action.SetTitle(bIsExpanded ? TEXT("Expand Pin") : TEXT("Collapse Pin"));
		GetActionStack()->BeginAction(Action);
	}

	InPin->bIsExpanded = bIsExpanded;

	Notify(ERigVMGraphNotifType::PinExpansionChanged, InPin);
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetPinIsWatched(const FString& InPinPath, bool bIsWatched, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	return SetPinIsWatched(Pin, bIsWatched, bSetupUndoRedo);
}

bool URigVMController::SetPinIsWatched(URigVMPin* InPin, bool bIsWatched, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidPinForGraph(InPin))
	{
		return false;
	}

	if (InPin->GetParentPin() != nullptr)
	{
		return false;
	}

	if (InPin->RequiresWatch() == bIsWatched)
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinWatchAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinWatchAction(this, InPin, bIsWatched);
		Action.SetTitle(bIsWatched ? TEXT("Watch Pin") : TEXT("Unwatch Pin"));
		GetActionStack()->BeginAction(Action);
	}

	InPin->bRequiresWatch = bIsWatched;

	Notify(ERigVMGraphNotifType::PinWatchedChanged, InPin);
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetPinDisplayName(const FString& InPinPath, const FString& InDisplayName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = SetPinDisplayName(Pin, InDisplayName, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_display_name('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			*InDisplayName));
	}

	return bSuccess;
}

bool URigVMController::SetPinDisplayName(URigVMPin* InPin, const FString& InDisplayName, bool bSetupUndoRedo)
{
	if(!IsValidPinForGraph(InPin))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(const URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(InPin->GetNode()))
	{
		if(const URigVMPin* OuterPin = InterfaceNode->FindReferencedPin(InPin))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterPin->GetGraph()))
			{
				return OuterController->SetPinDisplayName(OuterPin->GetPinPath(), InDisplayName, bSetupUndoRedo);
			}
		}
	}
	else if(InPin->GetNode()->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	const FName OldDisplayFName = InPin->GetDisplayName();
	const FString OldDisplayName = OldDisplayFName.IsNone() ? FString() : OldDisplayFName.ToString();

	if(InDisplayName.Equals(OldDisplayName, ESearchCase::CaseSensitive))
	{
		return true;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinDisplayNameAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinDisplayNameAction(this, InPin, InDisplayName);
		Action.SetTitle(TEXT("Set Pin Display Name"));
		GetActionStack()->BeginAction(Action);
	}

	InPin->DisplayName = InDisplayName.IsEmpty() ? FName(NAME_None) : FName(*InDisplayName);
	InPin->IncrementVersion();
	
	Notify(ERigVMGraphNotifType::PinDisplayNameChanged, InPin);

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InPin->GetNode()))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinDisplayNameChanged, InPin, false);
		}
	}

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::AddEmptyPinCategory(const FName& InNodeName, const FString& InCategory, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	const bool bSuccess = AddEmptyPinCategory(Node, InCategory, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_empty_pin_category('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString()),
			*InCategory));
	}

	return bSuccess;
}

bool URigVMController::AddEmptyPinCategory(const URigVMNode* InNode, const FString& InPinCategory, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		if(const URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(InNode->GetGraph()->GetOuter()))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterNode->GetGraph()))
			{
				return OuterController->AddEmptyPinCategory(OuterNode, InPinCategory, bSetupUndoRedo);
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMChangeNodePinCategoriesAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMChangeNodePinCategoriesAction(this, InNode);
		Action.SetTitle(TEXT("Add Empty Pin Category"));
		GetActionStack()->BeginAction(Action);
	}

	const FName PinCategoryName = URigVMSchema::GetUniqueName(*InPinCategory, [InNode](const FName& InName) {
		return !InNode->PinCategories.Contains(InName.ToString());
	}, false, true);

	const FString PinCategoryNameString = PinCategoryName.ToString();
	
	const_cast<URigVMNode*>(InNode)->PinCategories.Add(PinCategoryNameString);
	Notify(ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode));

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode), false);
		}
	}

	// if this was the first category and it is the default category
	// add all of the pins to it
	if(InNode->PinCategories.Num() == 1)
	{
		if(PinCategoryNameString.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase))
		{
			// apply this to all pins - it's going to filter invalid pins anyway
			for(URigVMPin* RootPin : InNode->Pins)
			{
				(void)SetPinCategory(RootPin, PinCategoryNameString, INDEX_NONE, bSetupUndoRedo);
			}
		}
	}

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		Action.UpdateAfterModification(InNode);
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetPinCategory(const FString& InPinPath, const FString& InCategory, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = SetPinCategory(Pin, InCategory, INDEX_NONE, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_category('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			*InCategory));
	}

	return bSuccess;
}

bool URigVMController::ClearPinCategory(const FString& InPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return SetPinCategory(InPinPath, FString(), bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetPinCategory(URigVMPin* InPin, const FString& InCategory, int32 InIndexInCategory, bool bSetupUndoRedo)
{
	if(!IsValidPinForGraph(InPin))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(const URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(InPin->GetNode()))
	{
		if(const URigVMPin* OuterPin = InterfaceNode->FindReferencedPin(InPin))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterPin->GetGraph()))
			{
				return OuterController->SetPinCategory(OuterPin->GetPinPath(), InCategory, bSetupUndoRedo);
			}
		}
	}
	else if(InPin->GetNode()->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	TArray<FString> Categories;
	RigVMStringUtils::SplitNodePath(InCategory, Categories);

	// trim each category name
	for(FString& Category : Categories)
	{
		Category.TrimStartAndEndInline();
	}

	// remove empty categories
	Categories.Remove(FString());

	// re-join the category path
	FString NewCategory;
	if(!Categories.IsEmpty())
	{
		NewCategory = RigVMStringUtils::JoinNodePath(Categories);
	}

	if(NewCategory.Equals(InPin->UserDefinedCategory, ESearchCase::CaseSensitive))
	{
		return true;
	}

	// we only allow one category per pin path - so we need to check all parents and children for set categories
	if(!NewCategory.IsEmpty())
	{
		TArray<URigVMPin*> PinsToCheck;
		URigVMPin* ParentPin = InPin->GetParentPin();
		while(ParentPin)
		{
			PinsToCheck.Add(ParentPin);
			ParentPin = ParentPin->GetParentPin();
		}
		PinsToCheck.Append(InPin->GetAllSubPinsRecursively());

		for(const URigVMPin* ParentOrSubPin : PinsToCheck)
		{
			if(!ParentOrSubPin->UserDefinedCategory.IsEmpty())
			{
				ReportErrorf(
					TEXT("Cannot set category (%s) on pin '%s' since pin '%s' already has a category set (%s)."),
					*NewCategory,
					*InPin->GetPinPath(),
					*ParentOrSubPin->GetPinPath(),
					*ParentOrSubPin->UserDefinedCategory
				);
				return false;
			}
		}
	}

	// you cannot set categories on IO, outputs or hidden pins
	if(InPin->GetDirection() == ERigVMPinDirection::IO ||
		InPin->GetDirection() == ERigVMPinDirection::Output ||
		InPin->GetDirection() == ERigVMPinDirection::Hidden)
	{
		ReportErrorf(
			TEXT("Cannot set category (%s) on pin '%s' - categories not allowed for pin direction."),
			*NewCategory,
			*InPin->GetPinPath()
		);
		return false;
	}

	TArray<FString> ParentAndThisCategories = InPin->GetNode()->GetParentPinCategories(NewCategory, false, true);
	Algo::Reverse(ParentAndThisCategories);
	for(const FString& CategoryOnPath : ParentAndThisCategories)
	{
		auto IsInput = [](const URigVMPin* InPin)
		{
			return InPin->GetDirection() == ERigVMPinDirection::Input || InPin->GetDirection() == ERigVMPinDirection::Visible;
		};
		
		// you cannot have inputs and IO on the same category or on parent categories
		const bool bIsInput = IsInput(InPin);
		const TArray<URigVMPin*> AllPins = InPin->GetNode()->GetAllPinsRecursively();
		for(const URigVMPin* OtherPin : AllPins)
		{
			if(OtherPin != InPin)
			{
				if(OtherPin->GetCategory().Equals(CategoryOnPath, ESearchCase::CaseSensitive))
				{
					if(bIsInput != IsInput(OtherPin))
					{
						ReportErrorf(
							TEXT("Cannot set category (%s) on pin '%s', it's already used on pin '%s' with a different direction."),
							*NewCategory,
							*InPin->GetPinPath(),
							*OtherPin->GetPinPath()
						);
					}
				}
			}
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinCategoryAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinCategoryAction(this, InPin, NewCategory);
		Action.SetTitle(TEXT("Set Pin Category"));
		GetActionStack()->BeginAction(Action);
	}

	// move all pins in the old category
	if(!InPin->UserDefinedCategory.IsEmpty() && InPin->IndexInCategory != INDEX_NONE)
	{
		const TArray<URigVMPin*> PinsInOldCategory = InPin->GetNode()->GetPinsForCategory(InPin->UserDefinedCategory);
		for(URigVMPin* OtherPin : PinsInOldCategory)
		{
			if(OtherPin != InPin && OtherPin->IndexInCategory != INDEX_NONE && OtherPin->IndexInCategory > InPin->IndexInCategory)
			{
				OtherPin->IndexInCategory--;
				Notify(ERigVMGraphNotifType::PinCategoryChanged, OtherPin);
			}
		}
	}

	int32 IndexInCategory = INDEX_NONE;
	if (!NewCategory.IsEmpty())
	{
		if(InIndexInCategory == INDEX_NONE)
		{
			IndexInCategory = InPin->GetNode()->GetPinsForCategory(NewCategory).Num();
		}
		else
		{
			const TArray<URigVMPin*> PinsInNewCategory = InPin->GetNode()->GetPinsForCategory(NewCategory);
			IndexInCategory = FMath::Min(InIndexInCategory, PinsInNewCategory.Num());

			// move all pins in the new category unless we are adding to the end
			if (IndexInCategory != PinsInNewCategory.Num())
			{
				for(URigVMPin* OtherPin : PinsInNewCategory)
				{
					if(OtherPin != InPin && OtherPin->IndexInCategory != INDEX_NONE && OtherPin->IndexInCategory >= IndexInCategory)
					{
						OtherPin->IndexInCategory++;
						Notify(ERigVMGraphNotifType::PinCategoryChanged, OtherPin);
					}
				}
			}
		}
	}

	InPin->UserDefinedCategory = NewCategory;
	InPin->IndexInCategory = IndexInCategory;
	InPin->IncrementVersion();
	Notify(ERigVMGraphNotifType::PinCategoryChanged, InPin);

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InPin->GetNode()))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoryChanged, InPin, false);
		}
	}

	if(NewCategory.IsEmpty())
	{
		(void)SetPinDisplayName(InPin, FString(), bSetupUndoRedo);
	}
	
	if(!NewCategory.IsEmpty())
	{
		if(bSetupUndoRedo)
		{
			TArray<FString> CombinedCategories = InPin->GetNode()->PinCategories;
			const TArray<FString> CategoriesToAdd = InPin->GetNode()->GetParentPinCategories(NewCategory, false, true);
			Algo::Reverse(CategoriesToAdd);
			for(const FString& CategoryToAdd : CategoriesToAdd)
			{
				CombinedCategories.AddUnique(CategoryToAdd);
			}
			(void)SetPinCategories(InPin->GetNode(), CombinedCategories, bSetupUndoRedo);
		}
	}

	if(!NewCategory.IsEmpty())
	{
		(void)SetPinCategoryExpansion(InPin->GetNode(), NewCategory, true, bSetupUndoRedo);
	}

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InPin->GetNode()))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
		}
	}

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::RemovePinCategory(const FName& InNodeName, const FString& InPinCategory, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	const bool bSuccess = RemovePinCategory(Node, InPinCategory, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_pin_category('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString()),
			*InPinCategory));
	}

	return bSuccess;
}

bool URigVMController::RemovePinCategory(const URigVMNode* InNode, const FString& InPinCategory, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		if(const URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(InNode->GetGraph()->GetOuter()))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterNode->GetGraph()))
			{
				return OuterController->RemovePinCategory(OuterNode, InPinCategory, bSetupUndoRedo);
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	const TArray<FString> AllCategories = InNode->GetPinCategories();
	TArray<FString> CategoriesToRemove = { InPinCategory };
	CategoriesToRemove.Append(InNode->GetSubPinCategories(InPinCategory, false, true));

	CategoriesToRemove.RemoveAll([&AllCategories](const FString InCategory) -> bool
	{
		return !AllCategories.Contains(InCategory);
	});

	TArray<URigVMPin*> PinsForCategory;
	for(const FString& CategoryToRemove : CategoriesToRemove)
	{
		PinsForCategory.Append(InNode->GetPinsForCategory(CategoryToRemove));
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMChangeNodePinCategoriesAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMChangeNodePinCategoriesAction(this, InNode);
		Action.SetTitle(TEXT("Remove Pin Category"));
		GetActionStack()->BeginAction(Action);
	}

	int32 NumPinsAffected = 0;
	for(URigVMPin* PinInCategory : PinsForCategory)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 
		if(SetPinCategory(PinInCategory, FString(), INDEX_NONE, bSetupUndoRedo))
		{
			NumPinsAffected++;
		}
	}

	int32 NumCategoriesRemoved = 0;
	for(const FString& CategoryToRemove : CategoriesToRemove)
	{
		NumCategoriesRemoved += const_cast<URigVMNode*>(InNode)->PinCategories.Remove(CategoryToRemove);
	}

	if(NumCategoriesRemoved > 0)
	{
		Notify(ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode));
	}

	const bool bHasMadeChange = (NumCategoriesRemoved + NumPinsAffected) > 0;

	if(bHasMadeChange)
	{
		if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
		{
			if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
			{
				RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
				PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode), false);
			}
		}
	}
	
	if (!bSuspendNotifications && bHasMadeChange)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		if(bHasMadeChange)
		{
			Action.UpdateAfterModification(InNode);
			GetActionStack()->EndAction(Action);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	return true;
}

bool URigVMController::RenamePinCategory(const FName& InNodeName, const FString& InOldPinCategory, const FString& InNewPinCategory, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	const bool bSuccess = RenamePinCategory(Node, InOldPinCategory, InNewPinCategory, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').rename_pin_category('%s', '%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString()),
			*InOldPinCategory,
			*InNewPinCategory));
	}

	return bSuccess;
}

bool URigVMController::RenamePinCategory(const URigVMNode* InNode, const FString& InOldPinCategory, const FString& InNewPinCategory, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	if(InOldPinCategory.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase))
	{
		ReportErrorf(TEXT("Cannot rename '%s' category."), *InOldPinCategory);
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		if(const URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(InNode->GetGraph()->GetOuter()))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterNode->GetGraph()))
			{
				return OuterController->RenamePinCategory(OuterNode, InOldPinCategory, InNewPinCategory, bSetupUndoRedo);
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	const FString OldPrefix = InOldPinCategory + TEXT("|");
	const FString NewPrefix = InNewPinCategory + TEXT("|");

	TArray<FString> CategoriesToRename;
	for(const FString& ExistingCategory : InNode->GetPinCategories())
	{
		if(ExistingCategory.Equals(InOldPinCategory, ESearchCase::CaseSensitive) ||
			ExistingCategory.StartsWith(OldPrefix, ESearchCase::CaseSensitive))
		{
			CategoriesToRename.Add(ExistingCategory);
		}
	}

	if(CategoriesToRename.IsEmpty())
	{
		return false;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMChangeNodePinCategoriesAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMChangeNodePinCategoriesAction(this, InNode);
		Action.SetTitle(TEXT("Rename Pin Category"));
		GetActionStack()->BeginAction(Action);
	}

	TArray<URigVMPin*> PinsToUpdate;
	TArray<FString> PinsCategoriesToRename;
	
	// first obtain the pins to update
	for (const FString& CategoryToRename : CategoriesToRename)
	{
		PinsToUpdate.Append(InNode->GetPinsForCategory(CategoryToRename));
	}

	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true);

		// Set all pins temporarily to the default category
		for (URigVMPin* PinToUpdate : PinsToUpdate)
		{
			const FString CategoryToRename = PinToUpdate->GetCategory();

			// store the original pin category name as it will be needed later at the final set
			PinsCategoriesToRename.Add(CategoryToRename);

			(void)SetPinCategory(PinToUpdate, FString(), INDEX_NONE, bSetupUndoRedo);
		}
	}

	// update the category list on the node
	for(const FString& CategoryToRename : CategoriesToRename)
	{
		const bool bWasExpanded = InNode->IsPinCategoryExpanded(CategoryToRename);
		const int32 Index = InNode->PinCategories.Find(CategoryToRename);
		if(CategoryToRename.Equals(InOldPinCategory, ESearchCase::CaseSensitive))
        {
			const_cast<URigVMNode*>(InNode)->PinCategories[Index] = InNewPinCategory;
        }
        else if(CategoryToRename.StartsWith(OldPrefix, ESearchCase::CaseSensitive))
        {
        	const FString NewCategory = NewPrefix + CategoryToRename.RightChop(OldPrefix.Len());
			const_cast<URigVMNode*>(InNode)->PinCategories[Index] = NewCategory;
        }

		const_cast<URigVMNode*>(InNode)->PinCategoryExpansion.Remove(CategoryToRename);
		const_cast<URigVMNode*>(InNode)->PinCategoryExpansion.FindOrAdd(InNode->PinCategories[Index]) = bWasExpanded;
	}

	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true);

		// then update the pins to their new categories
		for (int PinIndex = 0; PinIndex < PinsToUpdate.Num(); ++PinIndex)
		{
			URigVMPin* PinToUpdate = PinsToUpdate[PinIndex];

			const FString& CategoryToRename = PinsCategoriesToRename[PinIndex];
			if(CategoryToRename.Equals(InOldPinCategory, ESearchCase::CaseSensitive))
			{
				(void)SetPinCategory(PinToUpdate, InNewPinCategory, INDEX_NONE, bSetupUndoRedo);
			}
			else if(CategoryToRename.StartsWith(OldPrefix, ESearchCase::CaseSensitive))
			{
				const FString NewCategory = NewPrefix + CategoryToRename.RightChop(OldPrefix.Len());
				(void)SetPinCategory(PinToUpdate, NewCategory, INDEX_NONE, bSetupUndoRedo);
			}
		}
	}
	
	Notify(ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode));

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode), false);
		}
	}
	
	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		Action.UpdateAfterModification(InNode);
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetPinCategoryIndex(const FName& InNodeName, const FString& InPinCategory, int32 InNewIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	const bool bSuccess = SetPinCategoryIndex(Node, InPinCategory, InNewIndex, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_category_index('%s', '%s', %d)"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString()),
			*InPinCategory,
			InNewIndex));
	}

	return bSuccess;
}

bool URigVMController::SetPinCategoryIndex(const URigVMNode* InNode, const FString& InPinCategory, int32 InNewIndex, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	// for entry or return nodes we relay to the outer pins
	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		if(const URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(InNode->GetGraph()->GetOuter()))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterNode->GetGraph()))
			{
				return OuterController->SetPinCategoryIndex(OuterNode, InPinCategory, InNewIndex, bSetupUndoRedo);
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	TArray<FString> Categories = InNode->GetPinCategories();
	if(!Categories.Contains(InPinCategory))
	{
		return false;
	}

	InNewIndex = FMath::Clamp(InNewIndex, 0, Categories.Num() - 1);
	if(Categories.Find(InPinCategory) == InNewIndex)
	{
		return false;
	}

	Categories.Remove(InPinCategory);
	if(InNewIndex >= Categories.Num())
	{
		Categories.Add(InPinCategory);
	}
	else
	{
		Categories.Insert(InPinCategory, InNewIndex);
	}

	return SetPinCategories(InNode, Categories, bSetupUndoRedo);
}

bool URigVMController::SetPinCategoryExpansion(const FName& InNodeName, const FString& InPinCategory, bool bIsExpanded, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	const bool bSuccess = SetPinCategoryExpansion(Node, InPinCategory, bIsExpanded, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_category_expansion('%s', '%s', %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString()),
			*InPinCategory,
			(bIsExpanded) ? TEXT("True") : TEXT("False")));
	}

	return bSuccess;
}

bool URigVMController::SetPinCategoryExpansion(const URigVMNode* InNode, const FString& InPinCategory, bool bIsExpanded, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	const TArray<FString> AllCategories = InNode->GetPinCategories();
	if(!AllCategories.Contains(InPinCategory))
	{
		ReportErrorf(TEXT("Cannot find category '%s' on node '%s'."), *InPinCategory, *InNode->GetName());
		return false;
	}

	if(InPinCategory.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase))
	{
		// nothing to do here - the default category is always expanded
		return false;
	}

	if(InNode->IsPinCategoryExpanded(InPinCategory) == bIsExpanded)
	{
		return false;
	}

	if(InNode->GetPinsForCategory(InPinCategory).IsEmpty() &&
		InNode->GetSubPinCategories(InPinCategory).IsEmpty())
	{
		return false;
	}
	
	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinCategoryExpansionAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinCategoryExpansionAction(this, InNode, InPinCategory);
		if(bIsExpanded)
		{
			Action.SetTitle(TEXT("Collapse Pin Category"));
		}
		else
		{
			Action.SetTitle(TEXT("Expand Pin Category"));
		}
		GetActionStack()->BeginAction(Action);
	}

	if(bIsExpanded)
	{
		const_cast<URigVMNode*>(InNode)->PinCategoryExpansion.FindOrAdd(InPinCategory) = true;
	}
	else
	{
		const_cast<URigVMNode*>(InNode)->PinCategoryExpansion.Remove(InPinCategory);
	}
	const_cast<URigVMNode*>(InNode)->LastAffectedPinCategory = InPinCategory;
	Notify(ERigVMGraphNotifType::PinCategoryExpansionChanged, const_cast<URigVMNode*>(InNode));

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetPinIndexInCategory(const FString& InPinPath, int32 InIndexInCategory, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = SetPinIndexInCategory(Pin, InIndexInCategory, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_index_in_category('%s', %d)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			InIndexInCategory));
	}

	return bSuccess;
}

bool URigVMController::SetPinIndexInCategory(URigVMPin* InPin, int32 InIndexInCategory, bool bSetupUndoRedo)
{
	if(!IsValidPinForGraph(InPin))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(const URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(InPin->GetNode()))
	{
		if(const URigVMPin* OuterPin = InterfaceNode->FindReferencedPin(InPin))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterPin->GetGraph()))
			{
				return OuterController->SetPinIndexInCategory(OuterPin->GetPinPath(), InIndexInCategory, bSetupUndoRedo);
			}
		}
	}
	else if(InPin->GetNode()->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	if(InPin->GetCategory().IsEmpty())
	{
		InIndexInCategory = INDEX_NONE;
	}

	if(InPin->GetIndexInCategory() == InIndexInCategory)
	{
		return true;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinIndexInCategoryAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinIndexInCategoryAction(this, InPin, InIndexInCategory);
		Action.SetTitle(TEXT("Set Pin Index in Category"));
		GetActionStack()->BeginAction(Action);
	}

	if(!InPin->GetCategory().IsEmpty() && InIndexInCategory != INDEX_NONE)
	{
		TArray<URigVMPin*> PinsInCategory = InPin->GetNode()->GetPinsForCategory(InPin->GetCategory());
		check(PinsInCategory.Contains(InPin));

		PinsInCategory.Remove(InPin);
		if(InIndexInCategory >= PinsInCategory.Num())
		{
			PinsInCategory.Add(InPin);
		}
		else
		{
			PinsInCategory.Insert(InPin, InIndexInCategory);
		}

		TArray<bool> HasChanged;
		for(int32 Index = 0; Index < PinsInCategory.Num(); Index++)
		{
			HasChanged.Add(PinsInCategory[Index]->IndexInCategory != Index);
			PinsInCategory[Index]->IndexInCategory = Index;
		}
		for(int32 Index = 0; Index < PinsInCategory.Num(); Index++)
		{
			if(HasChanged[Index])
			{
				Notify(ERigVMGraphNotifType::PinCategoryChanged, PinsInCategory[Index]);
			}
		}
	}
	else
	{
		InPin->IndexInCategory = InIndexInCategory;
		InPin->IncrementVersion();
		Notify(ERigVMGraphNotifType::PinCategoryChanged, InPin);
	}

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InPin->GetNode()))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoryChanged, InPin, false);
		}
	}

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::SetNodeLayout(const FName& InNodeName, FRigVMNodeLayout InLayout, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	if(const URigVMSchema* Schema = GetGraph()->GetSchema())
	{
		if(!Schema->SupportsNodeLayouts(GetGraph()))
		{
			return false;
		}
	}

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	return SetNodeLayout(Node, InLayout, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetNodeLayout(const URigVMNode* InNode, FRigVMNodeLayout InLayout, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	int32 NumChanges = 0;
	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Setting Pin Layout"));
	}

	// first remove any pin layout from the node
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true);
		const TGuardValue<bool> GuardNotifications(bSuspendNotifications, true); 

		if(ClearNodeLayout(InNode->GetFName(), bSetupUndoRedo, bPrintPythonCommand))
		{
			NumChanges++;
		}
	}

	for(const TPair<FString,FString>& DisplayNamePair : InLayout.DisplayNames)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 
		const TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
		
		if(const URigVMPin* Pin = InNode->FindPin(DisplayNamePair.Key))
		{
			if(SetPinDisplayName(Pin->GetPinPath(), DisplayNamePair.Value, bSetupUndoRedo, bPrintPythonCommand))
			{
				NumChanges++;
			}
		}
		else
		{
			// just a warning - not an error. the layout can still be applied
			ReportWarningf(TEXT("Cannot find pin '%s' on node '%s' - ignoring display name provided by layout."), *DisplayNamePair.Key, *InNode->GetName());
		}
	}

	for(const FRigVMPinCategory& Category : InLayout.Categories)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 
		const TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
		
		for(const FString& SegmentPath : Category.Elements)
		{
			if(const URigVMPin* Pin = InNode->FindPin(SegmentPath))
			{
				if(SetPinCategory(Pin->GetPinPath(), Category.Path, bSetupUndoRedo, bPrintPythonCommand))
				{
					NumChanges++;
				}
			}
			else
			{
				// just a warning - not an error. the layout can still be applied
				ReportWarningf(TEXT("Cannot find pin '%s' on node '%s' - ignoring pin category provided by layout."), *SegmentPath, *InNode->GetName());
			}
		}
	}

	for(const TPair<FString,int32>& PinIndexPair : InLayout.PinIndexInCategory)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 
		const TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
		
		if(const URigVMPin* Pin = InNode->FindPin(PinIndexPair.Key))
		{
			if(SetPinIndexInCategory(Pin->GetPinPath(), PinIndexPair.Value, bSetupUndoRedo, bPrintPythonCommand))
			{
				NumChanges++;
			}
		}
		else
		{
			// just a warning - not an error. the layout can still be applied
			ReportWarningf(TEXT("Cannot find pin '%s' on node '%s' - ignoring pin index provided by layout."), *PinIndexPair.Key, *InNode->GetName());
		}
	}

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
		}
	}

	if (NumChanges > 0)
	{
		Notify(ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode));
	}

	if(bSetupUndoRedo)
	{
		if(NumChanges > 0)
		{
			CloseUndoBracket();
		}
		else
		{
			CancelUndoBracket();
		}
	}

	return NumChanges > 0;
}

bool URigVMController::ClearNodeLayout(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	return ClearNodeLayout(Node, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::ClearNodeLayout(const URigVMNode* InNode, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Clearing Pin Layout"));
	}

	int32 NumChanges = 0;

	TArray<FString> PinCategories = InNode->GetPinCategories();
	// sort by length and reverse
	PinCategories.Sort();
	Algo::Reverse(PinCategories);

	// remove all pin categories
	for(const FString& PinCategory : PinCategories)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 

		if(RemovePinCategory(InNode->GetFName(), PinCategory, bSetupUndoRedo, bPrintPythonCommand))
		{
			NumChanges++;
		}
	}

	// reset all pin display names
	const TArray<URigVMPin*> AllPins = InNode->GetAllPinsRecursively();
	for(const URigVMPin* Pin : AllPins)
	{
		const TGuardValue<bool> GuardRefreshFunctions(bSuspendRefreshingFunctionReferences, true); 

		const FName DefaultDisplayName = InNode->GetDisplayNameForPin(Pin->GetSegmentPath(true));
		if(!Pin->DisplayName.IsEqual(DefaultDisplayName, ENameCase::CaseSensitive))
		{
			if(SetPinDisplayName(Pin->GetPinPath(), FString(), bSetupUndoRedo, bPrintPythonCommand))
			{
				NumChanges++;
			}
		}
	}

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
		}
	}
	
	if(bSetupUndoRedo)
	{
		if(NumChanges > 0)
		{
			CloseUndoBracket();
		}
		else
		{
			CancelUndoBracket();
		}
	}

	return NumChanges > 0;
}

bool URigVMController::SetPinCategories(const FName& InNodeName, const TArray<FString>& InCategories, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	return SetPinCategories(Node, InCategories, bSetupUndoRedo);
}

bool URigVMController::SetPinCategories(const URigVMNode* InNode, const TArray<FString>& InCategories, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	// for entry or return nodes we relay to the outer pins
	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		if(const URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(InNode->GetGraph()->GetOuter()))
		{
			if(URigVMController* OuterController = GetControllerForGraph(OuterNode->GetGraph()))
			{
				return OuterController->SetPinCategories(OuterNode, InCategories, bSetupUndoRedo);
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionReferenceNode>())
	{
		return false;
	}

	const TArray<FString> PreviousCategories = InNode->GetPinCategories();
	if(PreviousCategories.Num() == InCategories.Num())
	{
		bool bFoundDifference = false;
		for(int32 Index = 0; Index < PreviousCategories.Num(); Index++)
		{
			if(!PreviousCategories[Index].Equals(InCategories[Index], ESearchCase::CaseSensitive))
			{
				bFoundDifference = true;
				break;
			}
		}
		if(!bFoundDifference)
		{
			return false;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMChangeNodePinCategoriesAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMChangeNodePinCategoriesAction(this, InNode);
		Action.SetTitle(TEXT("Set Pin Categories"));
		GetActionStack()->BeginAction(Action);
	}

	const_cast<URigVMNode*>(InNode)->PinCategories = InCategories;
	Notify(ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode));

	if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>())
		{
			RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
			PropagateNotificationToFunctionReferences(LibraryNode, ERigVMGraphNotifType::PinCategoriesChanged, const_cast<URigVMNode*>(InNode), false);
		}
	}

	if (!bSuspendNotifications)
	{
		const URigVMGraph* Graph = GetGraph();
		check(Graph);
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		Action.UpdateAfterModification(InNode);
		GetActionStack()->EndAction(Action);
	}

	return true;
}

FString URigVMController::GetPinDefaultValue(const FString& InPinPath)
{
	if (!IsValidGraph())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return FString();
	}
	Pin = Pin->GetPinForLink();

	return Pin->GetDefaultValue();
}

bool URigVMController::SetPinDefaultValue(const FString& InPinPath, const FString& InDefaultValue, bool bResizeArrays, bool bSetupUndoRedo, bool bMergeUndoAction, bool bPrintPythonCommand, bool bSetValueOnLinkedPins)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Pin->GetNode()))
	{
		if (Pin->GetName() == URigVMVariableNode::VariableName)
		{
			return SetVariableName(VariableNode, *InDefaultValue, bSetupUndoRedo);
		}
	}

	// by default treat pin default value changes as overrides
	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Override);
	if (!SetPinDefaultValue(Pin, InDefaultValue, bResizeArrays, bSetupUndoRedo, bMergeUndoAction, bSetValueOnLinkedPins))
	{
		return false;
	}

	URigVMPin* PinForLink = Pin->GetPinForLink();
	if (PinForLink != Pin)
	{
		if (!SetPinDefaultValue(PinForLink, InDefaultValue, bResizeArrays, false, bMergeUndoAction, bSetValueOnLinkedPins))
		{
			return false;
		}
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_pin_default_value('%s', '%s', %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			*InDefaultValue,
			(bResizeArrays) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

bool URigVMController::SetPinDefaultValue(URigVMPin* InPin, const FString& InDefaultValue, bool bResizeArrays, bool bSetupUndoRedo, bool bMergeUndoAction, bool bSetValueOnLinkedPins)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if(!bSetValueOnLinkedPins)
	{
		if(!InPin->GetSourceLinks(false).IsEmpty())
		{
			return false;
		}
	}
	
	check(InPin);

	if(!InPin->IsUObject()
		&& InPin->GetCPPType() != RigVMTypeUtils::FStringType
		&& InPin->GetCPPType() != RigVMTypeUtils::FNameType
		&& bValidatePinDefaults)
	{
		if(InDefaultValue.IsEmpty())
		{
			return false;
		}
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);
 
	if (bValidatePinDefaults)
	{
		if (!InPin->IsValidDefaultValue(InDefaultValue))
		{
			return false;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMSetPinDefaultValueAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMSetPinDefaultValueAction(this, InPin, InDefaultValue);
		Action.SetTitle(FString::Printf(TEXT("Set Pin Default Value")));
		GetActionStack()->BeginAction(Action);
	}

	const FString ClampedDefaultValue = InPin->IsRootPin() ? InPin->ClampDefaultValueFromMetaData(InDefaultValue) : InDefaultValue;

	bool bSetPinDefaultValueSucceeded = false;
	bool bFoundSkippedDefaultValue = false;
	if (InPin->IsArray())
	{
		if (GetSchema()->CanUnfoldPin(this, InPin))
		{
			TArray<FString> Elements = URigVMPin::SplitDefaultValue(ClampedDefaultValue);

			if (bResizeArrays)
			{
				TGuardValue<bool> GuardReporting(bReportWarningsAndErrors, false);
				while (Elements.Num() > InPin->SubPins.Num())
				{
					if(!InsertArrayPin(InPin, INDEX_NONE, FString(), bSetupUndoRedo))
					{
						break;
					}
					bSetPinDefaultValueSucceeded = true;
				}
				while (Elements.Num() < InPin->SubPins.Num())
				{
					if(!RemoveArrayPin(InPin->SubPins.Last()->GetPinPath(), bSetupUndoRedo))
					{
						break;
					}
					bSetPinDefaultValueSucceeded = true;
				}
			}
			else
			{
				ensure(Elements.Num() == InPin->SubPins.Num());
			}

			for (int32 ElementIndex = 0; ElementIndex < Elements.Num(); ElementIndex++)
			{
				if(URigVMPin* SubPin = InPin->FindSubPin(FString::FromInt(ElementIndex)))
				{
					PostProcessDefaultValue(SubPin, Elements[ElementIndex]);
					if (!Elements[ElementIndex].IsEmpty() || SubPin->IsWildCard() || SubPin->IsUObject())
					{
						SetPinDefaultValue(SubPin, Elements[ElementIndex], bResizeArrays, false, false, bSetValueOnLinkedPins);
						bSetPinDefaultValueSucceeded = true;
					}
				}
			}
		}
	}
	else if (InPin->IsStruct())
	{
		TArray<FString> MemberValuePairs = URigVMPin::SplitDefaultValue(ClampedDefaultValue);

		for (const FString& MemberValuePair : MemberValuePairs)
		{
			FString MemberName, MemberValue;
			if (MemberValuePair.Split(TEXT("="), &MemberName, &MemberValue))
			{
				URigVMPin* SubPin = InPin->FindSubPin(MemberName);
				if (!SubPin)
				{
					bFoundSkippedDefaultValue = true;
				}
				else if( !MemberValue.IsEmpty())
				{
					PostProcessDefaultValue(SubPin, MemberValue);
					if (!MemberValue.IsEmpty() || SubPin->IsWildCard() || SubPin->IsUObject())
					{
						SetPinDefaultValue(SubPin, MemberValue, bResizeArrays, false, false, bSetValueOnLinkedPins);
						bSetPinDefaultValueSucceeded = true;
					}
				}
			}
		}
	}
	
	// set default value on the current pin if there is no subpin to store some of the default values
	if(!bSetPinDefaultValueSucceeded || bFoundSkippedDefaultValue)
	{
		// always mark the pin's default value type as user provided
		// even if the value didn't change. this is done to remember
		// the value when switching versions of nodes / functions.
		const ERigVMPinDefaultValueType NewDefaultValueType = GetDefaultValueType(InPin, ClampedDefaultValue);
		if(InPin->DefaultValueType != NewDefaultValueType)
		{
			InPin->DefaultValueType = NewDefaultValueType;
			InPin->IncrementVersion();
			bSetPinDefaultValueSucceeded = true;
		}
		// no need to send notifications if not changing the value
		if(InPin->GetDefaultValue() != ClampedDefaultValue)
		{
			if (InPin->IsUObject())
			{
				InPin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(ClampedDefaultValue);
				InPin->DefaultValue.Empty();
			}
			else
			{
				InPin->DefaultValue = ClampedDefaultValue;
				InPin->DefaultValueObject = nullptr;
			}
			InPin->IncrementVersion();
			
			// root trait pin store their default value in a separate property bag so that
			// things like soft object ptr can be used and tracked in a uproperty
			if (InPin->IsTraitPin() && InPin->IsRootPin())
			{
				FRigVMTraitDefaultValueStruct* DefaultValueStructPtr = InPin->GetNode()->TraitDefaultValues.Find(InPin->GetName());
				if (ensure(DefaultValueStructPtr))
				{
					DefaultValueStructPtr->SetValue(ClampedDefaultValue);
				}
			}
			
			Notify(ERigVMGraphNotifType::PinDefaultValueChanged, InPin);
			if (!bSuspendNotifications)
			{
				(void)MarkPackageDirty(Graph);
			}
			bSetPinDefaultValueSucceeded = true;
		}
	}

	if (bSetupUndoRedo)
	{
		if(bSetPinDefaultValueSucceeded)
		{
			GetActionStack()->EndAction(Action, bMergeUndoAction);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	return true;
}

bool URigVMController::ResetPinDefaultValue(const FString& InPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = ResetPinDefaultValue(Pin, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').reset_pin_default_value('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath)));
	}

	return bSuccess;
}

bool URigVMController::ResetDefaultValueForPins(const TArray<FString>& InPinPaths, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph() || InPinPaths.IsEmpty())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	OpenUndoBracket(TEXT("Reset Default Value On Pins"));

	int32 NumPinsAffected = 0;
	for(const FString& PinPath : InPinPaths)
	{
		if(ResetPinDefaultValue(PinPath, bSetupUndo, false))
		{
			NumPinsAffected++;
		}
	}

	if(NumPinsAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedPinPaths;
		SanitizedPinPaths.Reserve(InPinPaths.Num());
		for(const FString& PinPath : InPinPaths)
		{
			SanitizedPinPaths.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedPinPath(PinPath)));
		}

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').reset_default_value_for_pins([%s])"),
			*GraphName,
			*FString::Join(SanitizedPinPaths, TEXT(","))));
	}

	return true;
}

bool URigVMController::ResetDefaultValueForAllPinsOnNode(const FName& InNodeName, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	TArray<FString> PinPaths;
	for(const URigVMPin* Pin : Node->GetPins())
	{
		if(Pin->CanProvideDefaultValue() && !Pin->HasOriginalDefaultValue())
		{
			PinPaths.Add(Pin->GetPinPath());
		}
	}

	if(!ResetDefaultValueForPins(PinPaths, bSetupUndo, false))
	{
		return false;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').reset_default_value_for_all_pins_on_node('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString())));
	}

	return true;
}

bool URigVMController::ResetDefaultValueForAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	OpenUndoBracket(TEXT("Reset Default Value For All Pins on Nodes"));

	int32 NumNodesAffected = 0;
	for(const FName& NodeName : InNodeNames)
	{
		if(ResetDefaultValueForAllPinsOnNode(NodeName, bSetupUndo, false))
		{
			NumNodesAffected++;
		}
	}

	if(NumNodesAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedNodeNames;
		SanitizedNodeNames.Reserve(InNodeNames.Num());
		for(const FName& NodeName : InNodeNames)
		{
			SanitizedNodeNames.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedNodeName(NodeName.ToString())));
		}
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').reset_default_value_for_all_pins_on_nodes([%s])"),
			*GraphName,
			*FString::Join(SanitizedNodeNames, TEXT(","))));
	}

	return true;
}

bool URigVMController::AddOverrideToPin(const FString& InPinPath, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Override, true);
	if(!SetPinDefaultValue(Pin, Pin->GetDefaultValue(), false, bSetupUndo, false, false))
	{
		return false;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_override_to_pin('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath)));
	}

	return true;
}

bool URigVMController::AddOverrideToPins(const TArray<FString>& InPinPaths, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph() || InPinPaths.IsEmpty())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	OpenUndoBracket(TEXT("Add Override To Pins"));

	int32 NumPinsAffected = 0;
	for(const FString& PinPath : InPinPaths)
	{
		if(AddOverrideToPin(PinPath, bSetupUndo, false))
		{
			NumPinsAffected++;
		}
	}

	if(NumPinsAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedPinPaths;
		SanitizedPinPaths.Reserve(InPinPaths.Num());
		for(const FString& PinPath : InPinPaths)
		{
			SanitizedPinPaths.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedPinPath(PinPath)));
		}

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_override_to_pins([%s])"),
			*GraphName,
			*FString::Join(SanitizedPinPaths, TEXT(","))));
	}

	return true;
}

bool URigVMController::AddOverrideToAllPinsOnNode(const FName& InNodeName, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	TArray<FString> PinPaths;
	for(const URigVMPin* Pin : Node->GetPins())
	{
		if(Pin->CanProvideDefaultValue() && Pin->GetDefaultValueType() != ERigVMPinDefaultValueType::Override)
		{
			PinPaths.Add(Pin->GetPinPath());
		}
	}

	if(!AddOverrideToPins(PinPaths, bSetupUndo, false))
	{
		return false;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_override_to_all_pins_on_node('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString())));
	}

	return true;
}

bool URigVMController::AddOverrideToAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	OpenUndoBracket(TEXT("Add Override To All Pins On Nodes"));

	int32 NumNodesAffected = 0;
	for(const FName& NodeName : InNodeNames)
	{
		if(AddOverrideToAllPinsOnNode(NodeName, bSetupUndo, false))
		{
			NumNodesAffected++;
		}
	}

	if(NumNodesAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedNodeNames;
		SanitizedNodeNames.Reserve(InNodeNames.Num());
		for(const FName& NodeName : InNodeNames)
		{
			SanitizedNodeNames.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedNodeName(NodeName.ToString())));
		}
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_override_to_all_pins_on_nodes([%s])"),
			*GraphName,
			*FString::Join(SanitizedNodeNames, TEXT(","))));
	}

	return true;
}

bool URigVMController::ClearOverrideOnPin(const FString& InPinPath, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset, true);
	if(!ResetPinDefaultValue(Pin->GetPinPath(), bSetupUndo, false))
	{
		return false;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').clear_override_on_pin('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath)));
	}

	return true;
}

bool URigVMController::ClearOverrideOnPins(const TArray<FString>& InPinPaths, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph() || InPinPaths.IsEmpty())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	OpenUndoBracket(TEXT("Clear Override On Pins"));

	int32 NumPinsAffected = 0;
	for(const FString& PinPath : InPinPaths)
	{
		if(ClearOverrideOnPin(PinPath, bSetupUndo, false))
		{
			NumPinsAffected++;
		}
	}

	if(NumPinsAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedPinPaths;
		SanitizedPinPaths.Reserve(InPinPaths.Num());
		for(const FString& PinPath : InPinPaths)
		{
			SanitizedPinPaths.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedPinPath(PinPath)));
		}

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').clear_override_on_pins([%s])"),
			*GraphName,
			*FString::Join(SanitizedPinPaths, TEXT(","))));
	}

	return true;
}

bool URigVMController::ClearOverrideOnAllPinsOnNode(const FName& InNodeName, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(InNodeName);
	if (Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
		return false;
	}

	TArray<FString> PinPaths;
	TArray<URigVMPin*> Pins = Node->GetAllPinsRecursively();
	for(const URigVMPin* Pin : Pins)
	{
		if(Pin->CanProvideDefaultValue() && Pin->HasDefaultValueOverride())
		{
			PinPaths.Add(Pin->GetPinPath());
		}
	}

	if(!ClearOverrideOnPins(PinPaths, bSetupUndo, false))
	{
		return false;
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').clear_override_on_all_pins_on_node('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedNodeName(InNodeName.ToString())));
	}

	return true;
}

bool URigVMController::ClearOverrideOnAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	OpenUndoBracket(TEXT("Add Override To All Pins On Nodes"));

	int32 NumNodesAffected = 0;
	for(const FName& NodeName : InNodeNames)
	{
		if(ClearOverrideOnAllPinsOnNode(NodeName, bSetupUndo, false))
		{
			NumNodesAffected++;
		}
	}

	if(NumNodesAffected == 0)
	{
		CancelUndoBracket();
		return false;
	}

	CloseUndoBracket();
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		TArray<FString> SanitizedNodeNames;
		SanitizedNodeNames.Reserve(InNodeNames.Num());
		for(const FName& NodeName : InNodeNames)
		{
			SanitizedNodeNames.Add(FString::Printf(TEXT("'%s'"), *GetSchema()->GetSanitizedNodeName(NodeName.ToString())));
		}
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').clear_override_on_all_pins_on_nodes([%s])"),
			*GraphName,
			*FString::Join(SanitizedNodeNames, TEXT(","))));
	}

	return true;
}

bool URigVMController::ResetPinDefaultValue(URigVMPin* InPin, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	check(InPin);

	URigVMNode* RigVMNode = InPin->GetNode();

	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset);

	FString DefaultValue = RigVMNode->GetOriginalPinDefaultValue(InPin);
	PostProcessDefaultValue(InPin, DefaultValue);
	if(!DefaultValue.IsEmpty() || InPin->IsWildCard() || (InPin->IsUObject() && DefaultValue.IsEmpty()))
	{
		SetPinDefaultValue(InPin, DefaultValue, true, bSetupUndoRedo, false);
		return true;
	}

	return false;
}

FString URigVMController::GetPinInitialDefaultValue(const URigVMPin* InPin)
{
	static const FString EmptyValue;
	static const FString TArrayInitValue( TEXT("()") );
	static const FString TObjectInitValue( TEXT("()") );
	static const TMap<FString, FString> InitValues =
	{
		{ RigVMTypeUtils::BoolType,	TEXT("False") },
		{ RigVMTypeUtils::Int32Type,	TEXT("0") },
		{ RigVMTypeUtils::FloatType,	TEXT("0.000000") },
		{ RigVMTypeUtils::DoubleType,	TEXT("0.000000") },
		{ RigVMTypeUtils::FNameType,	FName(NAME_None).ToString() },
		{ RigVMTypeUtils::FStringType,	TEXT("") }
	};

	if (InPin->IsStruct())
	{
		// offset is useless here as we are going to get the full struct default value
		static const uint32 Offset = 0;
		return GetPinInitialDefaultValueFromStruct(InPin->GetScriptStruct(), InPin, Offset);
	}
		
	if (InPin->IsStructMember())
	{
		if (URigVMPin* ParentPin = InPin->GetParentPin())
		{
			// cut off node's and parent struct's paths if func reference node, only node instead
			static const uint32 Offset = InPin->GetNode()->IsA<URigVMFunctionReferenceNode>() ? 2 : 1;
			return GetPinInitialDefaultValueFromStruct(ParentPin->GetScriptStruct(), InPin, Offset);
		}
	}

	if (InPin->IsArray())
	{
		return TArrayInitValue;
	}
		
	if (InPin->IsUObject())
	{
		return TObjectInitValue;
	}
		
	if (UEnum* Enum = InPin->GetEnum())
	{
		return Enum->GetNameStringByIndex(0);
	}
	
	if (const FString* BasicDefault = InitValues.Find(InPin->GetCPPType()))
	{
		return *BasicDefault;
	}
	
	return EmptyValue;
}

FString URigVMController::GetPinInitialDefaultValueFromStruct(UScriptStruct* ScriptStruct, const URigVMPin* InPin, uint32 InOffset)
{
	FString DefaultValue;
	if (InPin && ScriptStruct)
	{
		TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(ScriptStruct));
		uint8* Memory = (uint8*)StructOnScope->GetStructMemory();

		if (InPin->GetScriptStruct() == ScriptStruct)
		{
			ScriptStruct->ExportText(DefaultValue, Memory, nullptr, nullptr, PPF_None, nullptr, true);
			return DefaultValue;
		}

		const FString PinPath = InPin->GetPinPath();

		TArray<FString> Parts;
		if (!URigVMPin::SplitPinPath(PinPath, Parts))
		{
			return DefaultValue;
		}

		const uint32 NumParts = Parts.Num();
		if (InOffset >= NumParts)
		{
			return DefaultValue;
		}

		uint32 PartIndex = InOffset;

		UStruct* Struct = ScriptStruct;
		FProperty* Property = Struct->FindPropertyByName(*Parts[PartIndex++]);
		check(Property);

		Memory = Property->ContainerPtrToValuePtr<uint8>(Memory);

		while (PartIndex < NumParts && Property != nullptr)
		{
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Property = ArrayProperty->Inner;
				check(Property);
				PartIndex++;

				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					UScriptStruct* InnerStruct = StructProperty->Struct;
					StructOnScope = MakeShareable(new FStructOnScope(InnerStruct));
					Memory = (uint8 *)StructOnScope->GetStructMemory();
				}
				continue;
			}

			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				Struct = StructProperty->Struct;
				Property = Struct->FindPropertyByName(*Parts[PartIndex++]);
				check(Property);
				Memory = Property->ContainerPtrToValuePtr<uint8>(Memory);
				continue;
			}

			break;
		}

		if (Memory)
		{
			check(Property);
			Property->ExportTextItem_Direct(DefaultValue, Memory, nullptr, nullptr, PPF_None);
		}
	}

	return DefaultValue;
}

FString URigVMController::AddAggregatePin(const FString& InNodeName, const FString& InPinName, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	 
	if (!IsValidGraph())
	{
		return FString();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMNode* Node = Graph->FindNodeByName(*InNodeName);
	if (!Node)
	{
		return FString();
	}

	return AddAggregatePin(Node, InPinName, InDefaultValue, bSetupUndoRedo, bPrintPythonCommand);
}

FString URigVMController::AddAggregatePin(URigVMNode* InNode, const FString& InPinName, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return FString();
	}
	
	if (!InNode)
	{
		return FString();
	}

	if (!IsValidNodeForGraph(InNode))
	{
		return FString();
	}

	URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(InNode);
	if (AggregateNode == nullptr)
	{
		if(!InNode->IsAggregate())
		{
			return FString();
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);

	FRigVMReplaceNodesAction Action;
	if(bSetupUndoRedo)
	{
		Action = FRigVMReplaceNodesAction(this, {InNode});
		Action.SetTitle(TEXT("Add aggregate pin"));
	}

	const TMap<FString, FPinState> PinStates = GetPinStates(InNode);

	if (AggregateNode == nullptr)
	{
		bool bAggregateInputs = false;
		URigVMPin* Arg1 = nullptr;
		URigVMPin* Arg2 = nullptr;
		URigVMPin* ArgOpposite = nullptr;

		const TArray<URigVMPin*> AggregateInputs = InNode->GetAggregateInputs();
		const TArray<URigVMPin*> AggregateOutputs = InNode->GetAggregateOutputs();

		if (AggregateInputs.Num() == 2 && AggregateOutputs.Num() == 1)
		{
			bAggregateInputs = true;
			Arg1 = AggregateInputs[0];
			Arg2 = AggregateInputs[1];
			ArgOpposite = AggregateOutputs[0];
		}
		else if (AggregateInputs.Num() == 1 && AggregateOutputs.Num() == 2)
		{
			bAggregateInputs = false;
			Arg1 = AggregateOutputs[0];
			Arg2 = AggregateOutputs[1];
			ArgOpposite = AggregateInputs[0];
		}
		else
		{
			return FString();
		}

		if (!Arg1 || !Arg2 || !ArgOpposite)
		{
			return FString();
		}

		if (Arg1->GetCPPType() != Arg2->GetCPPType() || Arg1->GetCPPTypeObject() != Arg2->GetCPPTypeObject() ||
			Arg1->GetCPPType() != ArgOpposite->GetCPPType() || Arg1->GetCPPTypeObject() != ArgOpposite->GetCPPTypeObject())
		{
			return FString();
		}

		TMap<FName, URigVMNode*> EjectedNodes;
		for (URigVMPin* Pin : InNode->GetPins())
		{
			if (Pin->HasInjectedNodes())
			{
				if (URigVMNode* EjectedNode = EjectNodeFromPin(Pin, false, false))
				{
					EjectedNodes.Add(Pin->GetFName(), EjectedNode);
				}
			}
		}
		(void)EjectAllInjectedNodes(InNode, false);
		
		const FString AggregateArg1 = Arg1->GetName();
		const FString AggregateArg2 = Arg2->GetName();
		const FString AggregateOppositeArg = ArgOpposite->GetName();

		TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InNode);
		if(!FastBreakLinkedPaths(LinkedPaths, false))
		{
			return FString();
		}

		// We must resolve the type before we continue
		if (Arg1->IsWildCard())
		{
			TRigVMTypeIndex AnswerType = INDEX_NONE;
			if (RequestPinTypeSelectionDelegate.IsBound())
			{
				if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InNode))
				{
					if (const FRigVMTemplate* Template = TemplateNode->GetTemplate())
					{
						if (const FRigVMTemplateArgument* Argument = Template->FindArgument(Arg1->GetFName()))
						{
							TArray<TRigVMTypeIndex> Types; Argument->GetAllTypes(Types);
							AnswerType = RequestPinTypeSelectionDelegate.Execute(Types);
						}
					}
				}
			}

			if (AnswerType != INDEX_NONE)
			{
				ResolveWildCardPin(Arg1, AnswerType, false);
			}

		}

		if (Arg1->IsWildCard() || Arg2->IsWildCard() || ArgOpposite->IsWildCard())
		{
			return FString();
		}

		const FName PreviousNodeName = InNode->GetFName();
		URigVMCollapseNode* CollapseNode = CollapseNodes({InNode}, InNode->GetName(), false, true);
		if (!CollapseNode)
		{
			return FString();
		}

		InNode = CollapseNode->GetContainedGraph()->FindNodeByName(PreviousNodeName);

		AggregateNode = Cast<URigVMAggregateNode>(CollapseNode);
		if (AggregateNode)
		{
			if(URigVMController* AggregateController = GetControllerForGraph(AggregateNode->GetContainedGraph()))
			{
				TGuardValue<bool> GuardEditGraph(AggregateController->GetGraph()->bEditable, true);

				for(int32 Index = 0; Index < InNode->GetPins().Num(); Index++)
				{
					URigVMPin* Pin = InNode->GetPins()[Index];
					const FName PinName = Pin->GetFName();
				
					if (URigVMPin* AggregatePin = AggregateNode->FindPin(PinName.ToString()))
					{
						AggregateController->SetExposedPinIndex(PinName, Index, false);
						continue;
					}

					const FName ExposedPinName = AggregateController->AddExposedPin(PinName, Pin->GetDirection(), Pin->GetCPPType(), *Pin->GetCPPTypeObject()->GetPathName(), Pin->GetDefaultValue(), false, false);

					const FString PinNameStr = PinName.ToString();
					const FString ExposedPinNameStr = ExposedPinName.ToString();

					if(URigVMPin* ExposedPin = AggregateNode->FindPin(ExposedPinNameStr))
					{
						ExposedPin->SetDisplayName(Pin->GetDisplayName());
					}

					if(URigVMPin* ExposedPin = AggregateNode->GetEntryNode()->FindPin(ExposedPinNameStr))
					{
						ExposedPin->SetDisplayName(Pin->GetDisplayName());
					}

					if(URigVMPin* ExposedPin = AggregateNode->GetReturnNode()->FindPin(ExposedPinNameStr))
					{
						ExposedPin->SetDisplayName(Pin->GetDisplayName());
					}

					if (Pin->GetDirection() == ERigVMPinDirection::Input)
					{
						AggregateController->AddLink(FString::Printf(TEXT("Entry.%s"), *ExposedPinNameStr), FString::Printf(TEXT("%s.%s"), *InNode->GetName(), *PinNameStr), false);
					}
					else
					{
						AggregateController->AddLink(FString::Printf(TEXT("%s.%s"), *InNode->GetName(), *PinNameStr), FString::Printf(TEXT("Return.%s"), *ExposedPinNameStr), false);
					}
				}
			}
		}
		else
		{
			return FString();
		}

		FRestoreLinkedPathSettings Settings;
		Settings.NodeNameMap = {{PreviousNodeName.ToString(), AggregateNode->GetName()}};
		RestoreLinkedPaths(LinkedPaths, Settings, false);

		for (const TPair<FName, URigVMNode*>& Pair : EjectedNodes)
		{
			if (URigVMPin* Pin = AggregateNode->FindRootPinByName(Pair.Key))
			{
				URigVMNode* EjectedNode = Pair.Value;
				const bool bIsInput = Pin->GetNode()->GetLinkedSourceNodes().Contains(EjectedNode);
				FName OtherPinName = NAME_None;

				TArray<URigVMPin*> LinkedPins = bIsInput ? Pin->GetLinkedSourcePins() : Pin->GetLinkedTargetPins();
				for (URigVMPin* LinkedPin : LinkedPins)
				{
					if (LinkedPin->GetNode() == EjectedNode)
					{
						OtherPinName = LinkedPin->GetFName();
						break;
					}
				}
				
				if (!InjectNodeIntoPin(Pin->GetPinPath(), bIsInput, bIsInput ? FName() : OtherPinName, bIsInput ? OtherPinName : FName(), bSetupUndoRedo))
				{
					(void)RemoveNode(Pair.Value, false, false);
				}
			}
		}
	}
	
	if (!AggregateNode)
	{
		return FString();
	}

	URigVMPin* NewPin = nullptr;	
	if(URigVMController* AggregateController = GetControllerForGraph(AggregateNode->GetContainedGraph()))
	{
		TGuardValue<bool> GuardEditGraph(AggregateController->GetGraph()->bEditable, true);

		URigVMNode* InnerNode = (AggregateNode == nullptr) ? InNode : AggregateNode->GetFirstInnerNode();

		const FString InnerNodeContent = AggregateController->ExportNodesToText({InnerNode->GetFName()});
		const TArray<FName> NewNodeNames = AggregateController->ImportNodesFromText(InnerNodeContent, false);
		
		if(NewNodeNames.IsEmpty())
		{
			return FString();
		}

		URigVMNode* NewNode = AggregateNode->GetContainedGraph()->FindNodeByName(NewNodeNames[0]);

		FName NewPinName = *InPinName;
		if (NewPinName.IsNone())
		{
			URigVMNode* LastInnerNode = AggregateNode->GetLastInnerNode();
			URigVMPin* SecondAggregateInnerPin = LastInnerNode->GetSecondAggregatePin();
			FString LastAggregateName;
			if (AggregateNode->IsInputAggregate())
			{
				TArray<URigVMPin*> SourcePins = SecondAggregateInnerPin->GetLinkedSourcePins();
				if (SourcePins.Num() > 0)
				{
					LastAggregateName = SourcePins[0]->GetName();
				}
			}
			else
			{
				TArray<URigVMPin*> TargetPins = SecondAggregateInnerPin->GetLinkedTargetPins();
				if (TargetPins.Num() > 0)
				{
					LastAggregateName = TargetPins[0]->GetName();
				}
			}

			NewPinName = InnerNode->GetNextAggregateName(*LastAggregateName);
		}
		
		if (NewPinName.IsNone())
		{
			NewPinName = InnerNode->GetSecondAggregatePin()->GetFName();
		}
		
		const URigVMPin* Arg1 = AggregateNode->GetFirstAggregatePin();
		FName NewExposedPinName = AggregateController->AddExposedPin(NewPinName, Arg1->GetDirection(), Arg1->GetCPPType(), *Arg1->GetCPPTypeObject()->GetPathName(), InDefaultValue, false);
		NewPin = AggregateNode->FindPin(NewExposedPinName.ToString());
		URigVMPin* NewUnitPinArg1 = NewNode->GetFirstAggregatePin();
		URigVMPin* NewUnitPinArg2 = NewNode->GetSecondAggregatePin();
		URigVMPin* NewUnitPinOppositeArg = NewNode->GetOppositeAggregatePin();
		URigVMNode* PreviousNode = nullptr;
		if(AggregateNode->IsInputAggregate())
		{
			URigVMFunctionEntryNode* EntryNode = AggregateNode->GetEntryNode();
			URigVMPin* EntryPin = EntryNode->FindPin(NewExposedPinName.ToString());
			URigVMPin* ReturnPin = AggregateNode->GetReturnNode()->FindPin(NewUnitPinOppositeArg->GetName());
			URigVMPin* PreviousReturnPin = ReturnPin->GetLinkedSourcePins()[0];
			PreviousNode = PreviousReturnPin->GetNode();
		
			AggregateController->BreakAllLinks(ReturnPin, true, false);
			AggregateController->AddLink(PreviousReturnPin, NewUnitPinArg1, false);						
			AggregateController->AddLink(EntryPin, NewUnitPinArg2, false);
			AggregateController->AddLink(NewUnitPinOppositeArg, ReturnPin, false);
		}
		else
		{
			URigVMFunctionReturnNode* ReturnNode = AggregateNode->GetReturnNode();
			URigVMPin* NewReturnPin = ReturnNode->FindPin(NewExposedPinName.ToString());
			URigVMPin* OldReturnPin = ReturnNode->GetPins()[ReturnNode->GetPins().Num()-2];
			URigVMPin* PreviousReturnPin = OldReturnPin->GetLinkedSourcePins()[0];
			PreviousNode = PreviousReturnPin->GetNode();

			AggregateController->BreakAllLinks(OldReturnPin, true, false);
			AggregateController->AddLink(PreviousReturnPin, NewUnitPinOppositeArg, false);						
			AggregateController->AddLink(NewUnitPinArg1, OldReturnPin, false);
			AggregateController->AddLink(NewUnitPinArg2, NewReturnPin, false);
		}

		// Rearrange the graph nodes
		URigVMFunctionReturnNode* ReturnNode = AggregateNode->GetReturnNode();
		FVector2D NodeDimensions(200, 150);
		AggregateController->SetNodePosition(NewNode, PreviousNode->GetPosition() + NodeDimensions, false);
		AggregateController->SetNodePosition(ReturnNode, NewNode->GetPosition() + NodeDimensions, false);

		// Connect other input pins
		for (URigVMPin* OtherInputPin : AggregateNode->GetFirstInnerNode()->GetPins())
		{
			if (OtherInputPin->GetName() != NewUnitPinArg1->GetName() &&
				OtherInputPin->GetName() != NewUnitPinArg2->GetName() &&
				OtherInputPin->GetName() != NewUnitPinOppositeArg->GetName())
			{
				URigVMPin* OtherEntryPin = AggregateNode->GetEntryNode()->FindPin(OtherInputPin->GetName());
				AggregateController->AddLink(OtherEntryPin, NewNode->FindPin(OtherEntryPin->GetName()), false);
			}
		}

		AggregateNode->LastInnerNodeCache = NewNode;
	}

	if (!NewPin)
	{
		return FString();
	}

	if(!PinStates.IsEmpty())
	{
		if(URigVMController* AggregateController = GetControllerForGraph(AggregateNode->GetContainedGraph()))
		{
			AggregateController->ApplyPinStates(AggregateNode, PinStates);
		}
	}

	if (bSetupUndoRedo)
	{
		Action.StoreNode(AggregateNode, false);
		GetActionStack()->AddAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString NodePath = GetSchema()->GetSanitizedPinPath(InNode->GetNodePath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_aggregate_pin('%s', '%s', '%s')"),
			*GraphName,
			*NodePath,
			*InPinName,
			*InDefaultValue));
	}
	
	return NewPin->GetPinPath();

#else
	return FString();
#endif
}

bool URigVMController::RemoveAggregatePin(const FString& InPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(*InPinPath);
	if (!Pin)
	{
		return false;
	}

	return RemoveAggregatePin(Pin, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::RemoveAggregatePin(URigVMPin* InPin, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!InPin)
	{
		return false;
	}

	if (InPin->GetParentPin())
	{
		return false;
	}

	FRigVMReplaceNodesAction Action;
	if(bSetupUndoRedo)
	{
		Action = FRigVMReplaceNodesAction(this, {InPin->GetNode()});
		Action.SetTitle(TEXT("Remove aggregate pin"));
	}

	URigVMNode* Node = InPin->GetNode();
	FRigVMControllerCompileBracketScope CompileScope(this);

	bool bSuccess = false;
	if (URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(Node))
	{
		URigVMGraph* Graph = AggregateNode->GetContainedGraph();
		if (AggregateNode->IsInputAggregate())
		{
			if (URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode())
			{
				if (URigVMPin* EntryPin = EntryNode->FindPin(InPin->GetName()))
				{
					if (EntryPin->GetLinkedTargetPins().Num() > 0)
					{
						if(URigVMController* AggregateController = GetControllerForGraph(AggregateNode->GetContainedGraph()))
						{
							TGuardValue<bool> GuardEditGraph(AggregateController->GetGraph()->bEditable, true);
						
							URigVMPin* TargetPin = EntryPin->GetLinkedTargetPins()[0];
						
							URigVMNode* NodeToRemove = TargetPin->GetNode();
							URigVMPin* ResultPin = NodeToRemove->GetOppositeAggregatePin();
							URigVMPin* NextNodePin = ResultPin->GetLinkedTargetPins()[0];

							if (NodeToRemove == AggregateNode->FirstInnerNodeCache || NodeToRemove == AggregateNode->LastInnerNodeCache)
							{
								AggregateNode->InvalidateCache();
							}

							const FString FirstAggregatePin = AggregateNode->GetFirstAggregatePin()->GetName();
							const FString SecondAggregatePin = AggregateNode->GetSecondAggregatePin()->GetName();
							FString OtherArg = TargetPin->GetName() == FirstAggregatePin ? SecondAggregatePin : FirstAggregatePin;
							AggregateController->BreakAllLinks(NextNodePin, true, false);
							AggregateController->RewireLinks(NodeToRemove->FindPin(OtherArg), NextNodePin, true, false);
							AggregateController->RemoveNode(NodeToRemove, false);
							AggregateController->RemoveExposedPin(*InPin->GetName(), false);
							bSuccess = true;
						}
					}
				}
			}
		}
		else
		{
			if (URigVMFunctionReturnNode* ReturnNode = Graph->GetReturnNode())
			{
				if (URigVMPin* ReturnPin = ReturnNode->FindPin(InPin->GetName()))
				{
					if (ReturnPin->GetLinkedSourcePins().Num() > 0)
					{
						if(URigVMController* AggregateController = GetControllerForGraph(AggregateNode->GetContainedGraph()))
						{
							TGuardValue<bool> GuardEditGraph(AggregateController->GetGraph()->bEditable, true);
						
							URigVMPin* SourcePin = ReturnPin->GetLinkedSourcePins()[0];
					
							URigVMNode* NodeToRemove = SourcePin->GetNode();
							URigVMPin* OppositePin = NodeToRemove->GetOppositeAggregatePin();
							URigVMPin* NextNodePin = OppositePin->GetLinkedSourcePins()[0];
							URigVMNode* NextNode = NextNodePin->GetNode();

							if (NodeToRemove == AggregateNode->FirstInnerNodeCache || NodeToRemove == AggregateNode->LastInnerNodeCache)
							{
								AggregateNode->InvalidateCache();
							}

							const FString FirstAggregatePin = AggregateNode->GetFirstAggregatePin()->GetName();
							const FString SecondAggregatePin = AggregateNode->GetSecondAggregatePin()->GetName();
							FString OtherArg = SourcePin->GetName() == FirstAggregatePin ? SecondAggregatePin : FirstAggregatePin;
							AggregateController->BreakAllLinks(NextNodePin, false, false);
							AggregateController->RewireLinks(NodeToRemove->FindPin(OtherArg), NextNodePin, false, false);
							AggregateController->RemoveNode(NodeToRemove, false);
							AggregateController->RemoveExposedPin(*InPin->GetName(), false);
							bSuccess = true;
						}
					}
				}
			}			
		}

		if (bSuccess && AggregateNode->GetContainedNodes().Num() == 3)
		{
			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(AggregateNode);
			if(!FastBreakLinkedPaths(LinkedPaths, false))
			{
				return false;
			}

			TMap<FString, FString> PinNameMap;
			for(URigVMPin* Pin : AggregateNode->GetPins())
			{
				if(URigVMPin* EntryPin = AggregateNode->GetEntryNode()->FindPin(Pin->GetName()))
				{
					TArray<URigVMPin*> TargetPins = EntryPin->GetLinkedTargetPins();
					if(TargetPins.Num() > 0)
					{
						PinNameMap.Add(EntryPin->GetName(), TargetPins[0]->GetName());
					}
				}
				else if(URigVMPin* ReturnPin = AggregateNode->GetReturnNode()->FindPin(Pin->GetName()))
				{
					TArray<URigVMPin*> SourcePins = ReturnPin->GetLinkedSourcePins();
					if(SourcePins.Num() > 0)
					{
						PinNameMap.Add(ReturnPin->GetName(), SourcePins[0]->GetName());
					}
				}
			}

			const FName PreviousNodeName = AggregateNode->GetFName();
			TArray<URigVMNode*> NodesEjected = ExpandLibraryNode(AggregateNode, false);
			bSuccess = NodesEjected.Num() == 1;

			if(bSuccess)
			{
				URigVMNode* EjectedNode = NodesEjected[0];
				RenameNode(EjectedNode, PreviousNodeName, false, false);
				Node = EjectedNode;

				FRestoreLinkedPathSettings Settings;
				Settings.RemapDelegates = {{
					PreviousNodeName.ToString(), FRigVMController_PinPathRemapDelegate::CreateLambda([
						EjectedNode,
						PinNameMap
					](const FString& InPinPath, bool bIsInput) -> FString
					{
						static constexpr TCHAR PinPrefixFormat[] = TEXT("%s.");

						TArray<FString> Segments;
						URigVMPin::SplitPinPath(InPinPath, Segments);
						Segments[0] = EjectedNode->GetName();

						if(const FString* RemappedPin = PinNameMap.Find(Segments[1]))
						{
							Segments[1] = *RemappedPin;
						}
						return URigVMPin::JoinPinPath(Segments);
					})
				}};
				RestoreLinkedPaths(LinkedPaths, Settings, false);
			}
			else
			{
				Node = nullptr;
			}
		}		
	}

	if (bSetupUndoRedo)
	{
		if (bSuccess)
		{
			Action.StoreNode(Node, false);
			GetActionStack()->AddAction(Action);
		}
	}

	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString PinPath = GetSchema()->GetSanitizedPinPath(InPin->GetPinPath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_aggregate_pin('%s')"),
			*GraphName,
			*PinPath));
	}

	return bSuccess;

#else
	return false;
#endif
}

FString URigVMController::AddArrayPin(const FString& InArrayPinPath, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return InsertArrayPin(InArrayPinPath, INDEX_NONE, InDefaultValue, bSetupUndoRedo, bPrintPythonCommand);
}

FString URigVMController::DuplicateArrayPin(const FString& InArrayElementPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return FString();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* ElementPin = Graph->FindPin(InArrayElementPinPath);
	if (ElementPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InArrayElementPinPath);
		return FString();
	}

	if (!ElementPin->IsArrayElement())
	{
		ReportErrorf(TEXT("Pin '%s' is not an array element."), *InArrayElementPinPath);
		return FString();
	}

	URigVMPin* ArrayPin = ElementPin->GetParentPin();
	check(ArrayPin);
	ensure(ArrayPin->IsArray());

	FString DefaultValue = ElementPin->GetDefaultValue();
	return InsertArrayPin(ArrayPin->GetPinPath(), ElementPin->GetPinIndex() + 1, DefaultValue, bSetupUndoRedo, bPrintPythonCommand);
}

FString URigVMController::InsertArrayPin(const FString& InArrayPinPath, int32 InIndex, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return FString();
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return FString();
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* ArrayPin = Graph->FindPin(InArrayPinPath);
	if (ArrayPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InArrayPinPath);
		return FString();
	}

	URigVMPin* ElementPin = InsertArrayPin(ArrayPin, InIndex, InDefaultValue, bSetupUndoRedo);
	if (ElementPin)
	{
		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
				FString::Printf(TEXT("asset.get_controller_by_name('%s').insert_array_pin('%s', %d, '%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedPinPath(InArrayPinPath),
				InIndex,
				*InDefaultValue));
		}
		
		return ElementPin->GetPinPath();
	}

	return FString();
}

URigVMPin* URigVMController::InsertArrayPin(URigVMPin* ArrayPin, int32 InIndex, const FString& InDefaultValue, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}
	
	if (!ArrayPin->IsArray())
	{
		ReportErrorf(TEXT("Pin '%s' is not an array."), *ArrayPin->GetPinPath());
		return nullptr;
	}

	if (!GetSchema()->CanUnfoldPin(this, ArrayPin))
	{
		ReportErrorf(TEXT("Cannot insert array pin under '%s'."), *ArrayPin->GetPinPath());
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (InIndex == INDEX_NONE)
	{
		InIndex = ArrayPin->GetSubPins().Num();
	}

	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Override);
	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMInsertArrayPinAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMInsertArrayPinAction(this, ArrayPin, InIndex, InDefaultValue, OptionalDefaultValueType.Get(ERigVMPinDefaultValueType::Override));
		Action.SetTitle(FString::Printf(TEXT("Insert Array Pin")));
		GetActionStack()->BeginAction(Action);
	}

	for (int32 ExistingIndex = ArrayPin->GetSubPins().Num() - 1; ExistingIndex >= InIndex; ExistingIndex--)
	{
		URigVMPin* ExistingPin = ArrayPin->GetSubPins()[ExistingIndex];
		RenameObject(ExistingPin, *FString::FormatAsNumber(ExistingIndex + 1));

		// temporarily force a the index based display name
		// so that the ed graph node can pick that up during handling the notify
		// (the default is that it's based the index in the sub pin array,
		// which at this point hasn't changed)
		const TGuardValue<FName> ForcePinIndexDisplayName(ExistingPin->DisplayName, *FString::FromInt(ExistingIndex+1));
		Notify(ERigVMGraphNotifType::PinRenamed, ExistingPin);
	}

	URigVMPin* Pin = NewObject<URigVMPin>(ArrayPin, *FString::FormatAsNumber(InIndex));
	ConfigurePinFromPin(Pin, ArrayPin);
	Pin->CPPType = ArrayPin->GetArrayElementCppType();
	ArrayPin->SubPins.Insert(Pin, InIndex);
	ArrayPin->IncrementVersion();

	if (Pin->IsStruct())
	{
		UScriptStruct* ScriptStruct = Pin->GetScriptStruct();
		if (ScriptStruct)
		{
			FString DefaultValue = InDefaultValue;
			CreateDefaultValueForStructIfRequired(ScriptStruct, DefaultValue);
			{
				TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
				AddPinsForStruct(ScriptStruct, Pin->GetNode(), Pin, Pin->Direction, DefaultValue, false);
			}
		}
	}
	else if (Pin->IsArray())
	{
		FArrayProperty * ArrayProperty = CastField<FArrayProperty>(FindPropertyForPin(Pin->GetPinPath()));
		if (ArrayProperty)
		{
			TArray<FString> ElementDefaultValues = URigVMPin::SplitDefaultValue(InDefaultValue);
			AddPinsForArray(ArrayProperty, Pin->GetNode(), Pin, Pin->Direction, ElementDefaultValues, false);
		}
	}
	else
	{
		FString DefaultValue = InDefaultValue;
		PostProcessDefaultValue(Pin, DefaultValue);
		if(Pin->CanProvideDefaultValue())
		{
			Pin->DefaultValueType = GetDefaultValueType(Pin, DefaultValue);
		}

		if (Pin->IsUObject())
		{
			Pin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(DefaultValue);
			Pin->DefaultValue.Empty();
		}
		else
		{
			Pin->DefaultValue = DefaultValue;
			Pin->DefaultValueObject = nullptr;
		}
	}

	// set the array pin's default value type based on the resulting array pin list
	{
		const FRigVMDefaultValueTypeGuard AutoDetectGuard(this, ERigVMPinDefaultValueType::AutoDetect, true);
		ArrayPin->DefaultValueType = GetDefaultValueType(ArrayPin, ArrayPin->GetDefaultValue());
	}

	Notify(ERigVMGraphNotifType::PinAdded, Pin);
	Notify(ERigVMGraphNotifType::PinArraySizeChanged, ArrayPin);
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return Pin;
}

bool URigVMController::RemoveArrayPin(const FString& InArrayElementPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* ArrayElementPin = Graph->FindPin(InArrayElementPinPath);
	if (ArrayElementPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InArrayElementPinPath);
		return false;
	}

	if (!ArrayElementPin->IsArrayElement())
	{
		ReportErrorf(TEXT("Pin '%s' is not an array element."), *InArrayElementPinPath);
		return false;
	}

	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Override);

	URigVMPin* ArrayPin = ArrayElementPin->GetParentPin();
	check(ArrayPin);
	ensure(ArrayPin->IsArray());

	// we need to keep at least one element for fixed size arrays
	if(ArrayPin->IsExecuteContext() || ArrayPin->IsFixedSizeArray())
	{
		if(ArrayPin->GetArraySize() == 1)
		{
			ReportErrorf(TEXT("Cannot remove last element of a fixed size array %s."), *InArrayElementPinPath);
			return false;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Remove Array Pin")));
		GetActionStack()->BeginAction(Action);
	}

	int32 IndexToRemove = ArrayElementPin->GetPinIndex();
	FRigVMRemoveArrayPinAction RemovePinAction(this, ArrayElementPin);
	if (!RemovePin(ArrayElementPin, bSetupUndoRedo))
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return false;
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(RemovePinAction);
	}

	for (int32 ExistingIndex = IndexToRemove; ExistingIndex < ArrayPin->GetArraySize(); ExistingIndex++)
	{
		URigVMPin* ExistingPin = ArrayPin->GetSubPins()[ExistingIndex];
		ExistingPin->SetNameFromIndex();
		ExistingPin->ClearPinPathCache();
		Notify(ERigVMGraphNotifType::PinRenamed, ExistingPin);
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	// set the array pin's default value type based on the resulting array pin list
	{
		const FRigVMDefaultValueTypeGuard AutoDetectGuard(this, ERigVMPinDefaultValueType::AutoDetect, true);
		ArrayPin->DefaultValueType = GetDefaultValueType(ArrayPin, ArrayPin->GetDefaultValue());
	}
	Notify(ERigVMGraphNotifType::PinArraySizeChanged, ArrayPin);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_array_pin('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InArrayElementPinPath)));
	}

	return true;
}

bool URigVMController::RemovePin(URigVMPin* InPinToRemove, bool bSetupUndoRedo, bool bForceBreakLinks)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	check(!(InPinToRemove->IsRootPin() && InPinToRemove->IsTraitPin()));

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (InPinToRemove->IsDefinedAsInputVariable())
	{
		UnbindPinFromVariable(InPinToRemove, bSetupUndoRedo);
	}

	if (bSetupUndoRedo || bForceBreakLinks)
	{
		BreakAllLinks(InPinToRemove, true, bSetupUndoRedo);
		BreakAllLinks(InPinToRemove, false, bSetupUndoRedo);
		BreakAllLinksRecursive(InPinToRemove, true, false, bSetupUndoRedo);
		BreakAllLinksRecursive(InPinToRemove, false, false, bSetupUndoRedo);
	}

	if (URigVMPin* ParentPin = InPinToRemove->GetParentPin())
	{
		ParentPin->SubPins.Remove(InPinToRemove);
		ParentPin->IncrementVersion();
	}
	
	else if(URigVMNode* Node = InPinToRemove->GetNode())
	{
		Node->Pins.Remove(InPinToRemove);
		Node->OrphanedPins.Remove(InPinToRemove);
		Node->IncrementVersion();
	}

	TArray<URigVMPin*> SubPins = InPinToRemove->GetSubPins();
	for (URigVMPin* SubPin : SubPins)
	{
		if (!RemovePin(SubPin, bSetupUndoRedo, bForceBreakLinks))
		{
			return false;
		}
	}

	if (!bSuspendNotifications)
	{
		Notify(ERigVMGraphNotifType::PinRemoved, InPinToRemove);
	}

	DestroyObject(InPinToRemove);

	return true;
}

bool URigVMController::ClearArrayPin(const FString& InArrayPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return SetArrayPinSize(InArrayPinPath, 0, FString(), bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetArrayPinSize(const FString& InArrayPinPath, int32 InSize, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InArrayPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InArrayPinPath);
		return false;
	}

	if (!Pin->IsArray())
	{
		ReportErrorf(TEXT("Pin '%s' is not an array."), *InArrayPinPath);
		return false;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Set Array Pin Size (%d)"), InSize));
		GetActionStack()->BeginAction(Action);
	}

	int32 MinSize = 0;
	if(Pin->IsExecuteContext() || Pin->IsFixedSizeArray())
	{
		MinSize = 1;
	}

	InSize = FMath::Max<int32>(InSize, MinSize);
	int32 AddedPins = 0;
	int32 RemovedPins = 0;

	FString DefaultValue = InDefaultValue;
	if (DefaultValue.IsEmpty())
	{
		if (Pin->GetSubPins().Num() > 0)
		{
			DefaultValue = Pin->GetSubPins().Last()->GetDefaultValue();
		}
		CreateDefaultValueForStructIfRequired(Pin->GetScriptStruct(), DefaultValue);
	}

	while (Pin->GetSubPins().Num() > InSize)
	{
		if (!RemoveArrayPin(Pin->GetSubPins()[Pin->GetSubPins().Num()-1]->GetPinPath(), bSetupUndoRedo))
		{
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return false;
		}
		RemovedPins++;
	}

	while (Pin->GetSubPins().Num() < InSize)
	{
		if (AddArrayPin(Pin->GetPinPath(), DefaultValue, bSetupUndoRedo).IsEmpty())
		{
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return false;
		}
		AddedPins++;
	}

	if (!InDefaultValue.IsEmpty() && InDefaultValue != TEXT("()"))
	{
		SetPinDefaultValue(Pin, InDefaultValue, false, bSetupUndoRedo, true);
	}

	if (bSetupUndoRedo)
	{
		if (RemovedPins > 0 || AddedPins > 0)
		{
			GetActionStack()->EndAction(Action);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	return RemovedPins > 0 || AddedPins > 0;
}

bool URigVMController::BindPinToVariable(const FString& InPinPath, const FString& InNewBoundVariablePath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	bool bSuccess = false;
	if (InNewBoundVariablePath.IsEmpty())
	{
		bSuccess = UnbindPinFromVariable(Pin, bSetupUndoRedo);
	}
	else
	{
		bSuccess = BindPinToVariable(Pin, InNewBoundVariablePath, bSetupUndoRedo);
	}
	
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').bind_pin_to_variable('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			*InNewBoundVariablePath));
	}
	
	return bSuccess;
}

bool URigVMController::BindPinToVariable(URigVMPin* InPin, const FString& InNewBoundVariablePath, bool bSetupUndoRedo, const FString& InVariableNodeName)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidPinForGraph(InPin))
	{
		return false;
	}

	FString VariableName = InNewBoundVariablePath, SegmentPath;
	InNewBoundVariablePath.Split(TEXT("."), &VariableName, &SegmentPath);

	FRigVMExternalVariable Variable;
	const TArray<FRigVMExternalVariable> AllVariables = GetAllVariables(true);
	for (const FRigVMExternalVariable& VariableDescription : AllVariables)
	{
		if (VariableDescription.GetName().ToString().Equals(VariableName, ESearchCase::IgnoreCase))
		{
			Variable = VariableDescription;
			break;
		}
	}

	if (!Variable.IsValid(true))
	{
		ReportErrorf(TEXT("Cannot find variable '%s' in this graph."), *VariableName);
		return false;
	}

	if(!GetSchema()->CanBindVariable(this, InPin, &Variable, InNewBoundVariablePath))
	{
		return false;
	}

	if (!RigVMTypeUtils::AreCompatible(Variable, InPin->ToExternalVariable(), SegmentPath))
	{
		ReportErrorf(TEXT("Pin '%s' is not compatible with Variable '%s'"), *InPin->GetPinPath(), *Variable.GetName().ToString());
		return false;
	}
	
	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Bind pin to variable"));
		GetActionStack()->BeginAction(Action);
	}

	// Unbind any other variables, remove any other injections, and break all links to the input pin
	{
		if (InPin->IsBoundToVariable())
		{
			UnbindPinFromVariable(InPin, bSetupUndoRedo);
		}
		TArray<URigVMInjectionInfo*> Infos = InPin->GetInjectedNodes();
		for (URigVMInjectionInfo* Info : Infos)
		{
			RemoveInjectedNode(Info->GetPin()->GetPinPath(), Info->bInjectedAsInput, bSetupUndoRedo);
		}
		BreakAllLinks(InPin, true, bSetupUndoRedo);
	}

	// Create variable node
	URigVMVariableNode* VariableNode = nullptr;
	{
		{
			TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
			FString CPPType;
			UObject* CPPTypeObject;
			RigVMTypeUtils::CPPTypeFromExternalVariable(Variable, CPPType, &CPPTypeObject);
			VariableNode = AddVariableNode(*VariableName, CPPType, CPPTypeObject, true, FString(), FVector2D::ZeroVector, InVariableNodeName, bSetupUndoRedo);
		}
		if (VariableNode == nullptr)
		{
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return false;
		}
	}
	
	URigVMPin* ValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName);
	// Connect value pin to input pin
	{
		if (!SegmentPath.IsEmpty())
		{
			ValuePin = ValuePin->FindSubPin(SegmentPath);
		}

		{
			GetGraph()->ClearAST(true, false);
			if (!AddLink(ValuePin, InPin, bSetupUndoRedo))
			{
				if (bSetupUndoRedo)
				{
					GetActionStack()->CancelAction(Action);
				}
				return false;
			}
		}
	}

	// Inject into pin
	if (!InjectNodeIntoPin(InPin->GetPinPath(), true, FName(), ValuePin->GetFName(), bSetupUndoRedo))
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return false;
	}
	
	// remove the mapping from the variable map since it may still be in there.
	// we also need to remove it here since the bound variable now acts as the variable mapping instead.
	if (InPin->IsDefinedAsInputVariable())
	{
		if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InPin->GetNode()))
		{
			FunctionReferenceNode->VariableMap_DEPRECATED.Remove(InPin->GetFName());
		}
	}
	
	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::UnbindPinFromVariable(const FString& InPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = UnbindPinFromVariable(Pin, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').unbind_pin_from_variable('%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath)));
	}
	
	return bSuccess;
}

bool URigVMController::UnbindPinFromVariable(URigVMPin* InPin, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidPinForGraph(InPin))
	{
		return false;
	}

	if(!GetSchema()->CanUnbindVariable(this, InPin))
	{
		return false;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Unbind pin from variable"));
		GetActionStack()->BeginAction(Action);
	}

	RemoveInjectedNode(InPin->GetPinPath(), true, bSetupUndoRedo);

	TArray<URigVMPin*> SourcePins = InPin->GetLinkedSourcePins();
	if(SourcePins.Num() == 1)
	{
		BreakLink(SourcePins[0], InPin, bSetupUndoRedo);
	}
	
	// remove the mapping from the variable map
	if (InPin->IsDefinedAsInputVariable())
	{
		if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InPin->GetNode()))
		{
			FunctionReferenceNode->VariableMap_DEPRECATED.Remove(InPin->GetFName());
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

bool URigVMController::MakeBindingsFromVariableNode(const FName& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Graph->FindNodeByName(InNodeName)))
	{
		return MakeBindingsFromVariableNode(VariableNode, bSetupUndoRedo);
	}

	return false;
}

bool URigVMController::MakeBindingsFromVariableNode(URigVMVariableNode* InNode, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	check(InNode);

	TArray<TPair<URigVMPin*, URigVMPin*>> Pairs;
	TArray<URigVMNode*> NodesToRemove;
	NodesToRemove.Add(InNode);

	if (URigVMPin* ValuePin = InNode->FindPin(URigVMVariableNode::ValueName))
	{
		TArray<URigVMLink*> Links = ValuePin->GetTargetLinks(true);
		for (URigVMLink* Link : Links)
		{
			URigVMPin* SourcePin = Link->GetSourcePin();

			TArray<URigVMPin*> TargetPins;
			TargetPins.Add(Link->GetTargetPin());

			for (int32 TargetPinIndex = 0; TargetPinIndex < TargetPins.Num(); TargetPinIndex++)
			{
				URigVMPin* TargetPin = TargetPins[TargetPinIndex];
				if (Cast<URigVMRerouteNode>(TargetPin->GetNode()))
				{
					NodesToRemove.AddUnique(TargetPin->GetNode());
					TargetPins.Append(TargetPin->GetLinkedTargetPins(false /* recursive */));
				}
				else
				{
					Pairs.Add(TPair<URigVMPin*, URigVMPin*>(SourcePin, TargetPin));
				}
			}
		}
	}

	FName VariableName = InNode->GetVariableName();
	FRigVMExternalVariable Variable = GetVariableByName(VariableName);
	if (!Variable.IsValid(true /* allow nullptr */))
	{
		return false;
	}

	if (Pairs.Num() > 0)
	{
		FRigVMControllerCompileBracketScope CompileScope(this);
		if (bSetupUndoRedo)
		{
			OpenUndoBracket(TEXT("Turn Variable Node into Bindings"));
		}

		for (const TPair<URigVMPin*, URigVMPin*>& Pair : Pairs)
		{
			URigVMPin* SourcePin = Pair.Key;
			URigVMPin* TargetPin = Pair.Value;
			FString SegmentPath = SourcePin->GetSegmentPath();
			FString VariablePathToBind = VariableName.ToString();
			if (!SegmentPath.IsEmpty())
			{
				VariablePathToBind = FString::Printf(TEXT("%s.%s"), *VariablePathToBind, *SegmentPath);
			}

			if (!BindPinToVariable(TargetPin, VariablePathToBind, bSetupUndoRedo))
			{
				CancelUndoBracket();
			}
		}

		for (URigVMNode* NodeToRemove : NodesToRemove)
		{
			RemoveNode(NodeToRemove, bSetupUndoRedo, true);
		}

		if (bSetupUndoRedo)
		{
			CloseUndoBracket();
		}
		return true;
	}

	return false;

}

bool URigVMController::MakeVariableNodeFromBinding(const FString& InPinPath, const FVector2D& InNodePosition, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return PromotePinToVariable(InPinPath, true, InNodePosition, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::PromotePinToVariable(const FString& InPinPath, bool bCreateVariableNode, const FVector2D& InNodePosition, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = PromotePinToVariable(Pin, false, bCreateVariableNode, InNodePosition, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').promote_pin_to_variable('%s', %s, %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			(bCreateVariableNode) ? TEXT("True") : TEXT("False"),
			*RigVMPythonUtils::Vector2DToPythonString(InNodePosition)));
	}
	
	return bSuccess;
}

bool URigVMController::PromotePinToLocalVariable(const FString& InPinPath, bool bCreateVariableNode, const FVector2D& InNodePosition, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}

	const bool bSuccess = PromotePinToVariable(Pin, true, bCreateVariableNode, InNodePosition, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').promote_pin_to_local_variable('%s', %s, %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(InPinPath),
			(bCreateVariableNode) ? TEXT("True") : TEXT("False"),
			*RigVMPythonUtils::Vector2DToPythonString(InNodePosition)));
	}
	
	return bSuccess;
}

bool URigVMController::PromotePinToVariable(URigVMPin* InPin, bool bCreateLocalVariable, bool bCreateVariableNode, const FVector2D& InNodePosition, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	check(InPin);

	if (GetGraph()->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot promote pins to variables in function library graphs."));
		return false;
	}

	if (InPin->GetDirection() != ERigVMPinDirection::Input && InPin->GetDirection() != ERigVMPinDirection::Output && InPin->GetDirection() != ERigVMPinDirection::IO)
	{
		return false;
	}

	FRigVMExternalVariable VariableForPin;
	FString SegmentPath;
	if (InPin->IsBoundToVariable())
	{
		VariableForPin = GetVariableByName(*InPin->GetBoundVariableName());
		check(VariableForPin.IsValid(true /* allow nullptr */));
		SegmentPath = InPin->GetBoundVariablePath();
		if (SegmentPath.StartsWith(VariableForPin.GetName().ToString() + TEXT(".")))
		{
			SegmentPath = SegmentPath.RightChop(VariableForPin.GetName().ToString().Len());
		}
		else
		{
			SegmentPath.Empty();
		}
	}
	else
	{
		FName VariableName;
		VariableForPin = InPin->ToExternalVariable();
		if (bCreateLocalVariable)
		{
			const FRigVMGraphVariableDescription VariableDescription = AddLocalVariable(VariableForPin.GetName(),VariableForPin.GetExtendedCPPType().ToString(), VariableForPin.GetCPPTypeObject(), InPin->GetDefaultValue());
			VariableName = VariableDescription.Name;
		}
		else
		{
			if (!UnitNodeCreatedContext.GetCreateExternalVariableDelegate().IsBound())
			{
				return false;
			}

			VariableName = UnitNodeCreatedContext.GetCreateExternalVariableDelegate().Execute(VariableForPin, InPin->GetDefaultValue());
			if (VariableName.IsNone())
			{
				return false;
			}
		}

		VariableForPin = GetVariableByName(VariableName);
		if (!VariableForPin.IsValid(true /* allow nullptr*/))
		{
			return false;
		}
	}

	if (bCreateVariableNode)
	{
		FRigVMControllerCompileBracketScope CompileScope(this);
		if (URigVMVariableNode* VariableNode = AddVariableNode(
			VariableForPin.GetName(),
			VariableForPin.GetExtendedCPPType().ToString(),
			const_cast<UObject*>(VariableForPin.GetCPPTypeObject()),
			InPin->GetDirection() != ERigVMPinDirection::Output,
			FString(),
			InNodePosition,
			FString(),
			bSetupUndoRedo))
		{
			if (URigVMPin* ValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName))
			{
				if (InPin->GetDirection() == ERigVMPinDirection::Output)
				{
					return AddLink(InPin->GetPinPath(), ValuePin->GetPinPath() + SegmentPath, bSetupUndoRedo);
				}
				return AddLink(ValuePin->GetPinPath() + SegmentPath, InPin->GetPinPath(), bSetupUndoRedo);
			}
		}
	}
	else if (InPin->GetDirection() == ERigVMPinDirection::Output)
	{
		// cannot bind output pin to variable, so just return true
		return true;
	}
	else
	{
		FRigVMControllerCompileBracketScope CompileScope(this);
		return BindPinToVariable(InPin, VariableForPin.GetName().ToString(), bSetupUndoRedo);
	}

	return false;
}

bool URigVMController::AddLink(const FString& InOutputPinPath, const FString& InInputPinPath, bool bSetupUndoRedo,
	bool bPrintPythonCommand, ERigVMPinDirection InUserDirection, bool bCreateCastNode)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);


	FString OutputPinPath = InOutputPinPath;
	FString InputPinPath = InInputPinPath;

	if (FString* RedirectedOutputPinPath = OutputPinRedirectors.Find(OutputPinPath))
	{
		OutputPinPath = *RedirectedOutputPinPath;
	}
	if (FString* RedirectedInputPinPath = InputPinRedirectors.Find(InputPinPath))
	{
		InputPinPath = *RedirectedInputPinPath;
	}

	URigVMPin* OutputPin = Graph->FindPin(OutputPinPath);
	if (OutputPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *OutputPinPath);
		return false;
	}
	OutputPin = OutputPin->GetPinForLink();

	URigVMPin* InputPin = Graph->FindPin(InputPinPath);
	if (InputPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InputPinPath);
		return false;
	}
	InputPin = InputPin->GetPinForLink();

	const bool bSuccess = AddLink(OutputPin, InputPin, bSetupUndoRedo, InUserDirection, bCreateCastNode);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		const FString SanitizedInputPinPath = GetSchema()->GetSanitizedPinPath(InputPin->GetPinPath());
		const FString SanitizedOutputPinPath = GetSchema()->GetSanitizedPinPath(OutputPin->GetPinPath());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_link('%s', '%s')"),
			*GraphName,
			*SanitizedOutputPinPath,
			*SanitizedInputPinPath));
	}
	
	return bSuccess;
}

bool URigVMController::AddLink(URigVMPin* OutputPin, URigVMPin* InputPin, bool bSetupUndoRedo, ERigVMPinDirection InUserDirection, bool bCreateCastNode, bool bIsRestoringLinks, FString* OutFailureReason)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		static const FString FailureReason = TEXT("Cannot add links in non-editable graph."); 
		if(OutFailureReason)
		{
			*OutFailureReason = FailureReason;
		}
		ReportError(FailureReason);
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TGuardValue<ERigVMPinDirection> UserLinkDirectionGuard(UserLinkDirection,	
	InUserDirection == ERigVMPinDirection::Invalid ? UserLinkDirection : InUserDirection);

	if(!GetSchema()->CanAddLink(this, OutputPin, InputPin, nullptr, UserLinkDirection, true, true, OutFailureReason))
	{
		if(OutFailureReason)
		{
			ReportError(*OutFailureReason);
		}
		return false;
	}
	
	if (!bIsTransacting)
	{
		FString FailureReason;

		bool bCanLink = true;
		if(bIsRestoringLinks || bSuspendASTLinkChecks)
		{
			bCanLink = URigVMPin::CanLink(OutputPin, InputPin, &FailureReason, nullptr, UserLinkDirection, bCreateCastNode);
		}
		else
		{
			bCanLink = Graph->CanLink(OutputPin, InputPin, &FailureReason, GetCurrentByteCode(), UserLinkDirection, bCreateCastNode);
		}
		
		if (!bCanLink)
		{
			if(OutFailureReason)
			{
				*OutFailureReason = FailureReason;
			}
			
			if(OutputPin->IsExecuteContext() && InputPin->IsExecuteContext())
			{
				if(OutputPin->GetNode()->IsA<URigVMFunctionEntryNode>() &&
					InputPin->GetNode()->IsA<URigVMFunctionReturnNode>())
				{
					return false;
				}
			}
			ReportErrorf(TEXT("Cannot link '%s' to '%s': %s."), *OutputPin->GetPinPath(), *InputPin->GetPinPath(), *FailureReason);
			return false;
		}
	}

	ensure(!OutputPin->IsLinkedTo(InputPin));
	ensure(!InputPin->IsLinkedTo(OutputPin));

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Add Link"));
		GetActionStack()->BeginAction(Action);
	}

	// check if we need to inject cast node
	if(bCreateCastNode &&
		bEnableTypeCasting &&
		OutputPin->GetTypeIndex() != InputPin->GetTypeIndex() &&
		!OutputPin->IsWildCard() &&
		!InputPin->IsWildCard())
	{
		if(!FRigVMRegistry::Get().CanMatchTypes(OutputPin->GetTypeIndex(), InputPin->GetTypeIndex(), true))
		{
			if(bCreateCastNode)
			{
				const FRigVMFunction* CastFunction = RigVMTypeUtils::GetCastForTypeIndices(OutputPin->GetTypeIndex(), InputPin->GetTypeIndex());

				if(CastFunction == nullptr)
				{
					// this is potentially a template node which has more types available.
					// look through the filtered types and find a matching cast.
				}

				if(CastFunction == nullptr)
				{
					if (bSetupUndoRedo)
					{
						GetActionStack()->CancelAction(Action);
					}
					return false;
				}

				const FVector2D OutputPinPosition = OutputPin->GetNode()->GetPosition() + FVector2D(150, 40.f) + FVector2D(0, 16) * OutputPin->GetRootPin()->GetPinIndex();
				const FVector2D InputPinPosition = InputPin->GetNode()->GetPosition() + FVector2D(-75, 40.f) + FVector2D(0, 16) * InputPin->GetRootPin()->GetPinIndex();
				const FVector2D CastPosition = (OutputPinPosition + InputPinPosition) * 0.5; 

				const URigVMNode* CastNode = nullptr;

				// try to find an existing cast node
				if(CastNode == nullptr)
				{
					for(URigVMLink* ExistingLink : OutputPin->GetLinks())
					{
						if(ExistingLink->GetSourcePin() == OutputPin)
						{
							if(URigVMUnitNode* ExistingCastNode = Cast<URigVMUnitNode>(ExistingLink->GetTargetPin()->GetNode()))
							{
								if(ExistingCastNode->GetScriptStruct() == CastFunction->Struct)
								{
									CastNode = ExistingCastNode;
									break;
								}
							}
						}
					}
				}

				if(CastNode == nullptr)
				{
					if(CastFunction->Factory)
					{
						CastNode = AddTemplateNode(CastFunction->Factory->GetTemplateNotation(), CastPosition, FString(), bSetupUndoRedo, false);
					}
					else
					{
						CastNode = AddUnitNode(CastFunction->Struct, CastFunction->GetMethodName(), CastPosition, FString(), bSetupUndoRedo, false);
					}
				}
				
				if(CastNode == nullptr)
				{
					if (bSetupUndoRedo)
					{
						GetActionStack()->CancelAction(Action);
					}
					return false;
				}

				static const FString CastTemplateValueName = RigVMTypeUtils::GetCastTemplateValueName().ToString();
				static const FString CastTemplateResultName = RigVMTypeUtils::GetCastTemplateResultName().ToString();
				URigVMPin* ValuePin = CastNode->FindPin(CastTemplateValueName);
				URigVMPin* ResultPin = CastNode->FindPin(CastTemplateResultName);

				if(!OutputPin->IsLinkedTo(ValuePin))
				{
					if(!AddLink(OutputPin, ValuePin, bSetupUndoRedo))
					{
						if (bSetupUndoRedo)
						{
							GetActionStack()->CancelAction(Action);
						}
						return false;
					}
				}

				if(!ResultPin->IsLinkedTo(InputPin))
				{
					if(!AddLink(ResultPin, InputPin, bSetupUndoRedo))
					{
						if (bSetupUndoRedo)
						{
							GetActionStack()->CancelAction(Action);
						}
						return false;
					}
				}

				if(bSetupUndoRedo)
				{
					Action.SetTitle(TEXT("Add Link with Cast"));
					GetActionStack()->EndAction(Action);
				}
				return true;
			}
		}
	}

	if (OutputPin->IsExecuteContext())
	{
		BreakAllLinks(OutputPin, false, bSetupUndoRedo);
	}

	BreakAllLinks(InputPin, true, bSetupUndoRedo);
	if (bSetupUndoRedo)
	{
		BreakAllLinksRecursive(InputPin, true, true, bSetupUndoRedo);
		BreakAllLinksRecursive(InputPin, true, false, bSetupUndoRedo);
	}

	// resolve types on the pins if needed
	if((InputPin->GetCPPTypeObject() != OutputPin->GetCPPTypeObject() ||
		OutputPin->GetCPPType() != InputPin->GetCPPType()) &&
		!InputPin->IsExecuteContext() &&
		!OutputPin->IsExecuteContext())
	{
		bool bOutputPinCanChangeType = OutputPin->IsWildCard();
		bool bInputPinCanChangeType = InputPin->IsWildCard();

		if(!bOutputPinCanChangeType && !bInputPinCanChangeType)
		{
			bInputPinCanChangeType = UserLinkDirection == ERigVMPinDirection::Output && InputPin->GetNode()->IsA<URigVMTemplateNode>(); 
			bOutputPinCanChangeType = UserLinkDirection == ERigVMPinDirection::Input && OutputPin->GetNode()->IsA<URigVMTemplateNode>(); 
		}
		
		if(bOutputPinCanChangeType)
		{
			bool bInteractionBracketOpened = false;
			if(OutputPin->GetNode()->IsA<URigVMRerouteNode>())
			{
				if(!bInteractionBracketOpened)
				{
					Notify(ERigVMGraphNotifType::InteractionBracketOpened, nullptr);
					bInteractionBracketOpened = true;
				}
				SetPinDefaultValue(OutputPin, InputPin->GetDefaultValue(), true, bSetupUndoRedo, false);
			}
			if(InputPin->GetNode()->IsA<URigVMRerouteNode>())
			{
				if(!bInteractionBracketOpened)
				{
					Notify(ERigVMGraphNotifType::InteractionBracketOpened, nullptr);
					bInteractionBracketOpened = true;
				}
				SetPinDefaultValue(OutputPin, OutputPin->GetDefaultValue(), true, bSetupUndoRedo, false);
			}

			if(bInteractionBracketOpened)
			{
				Notify(ERigVMGraphNotifType::InteractionBracketClosed, nullptr);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		ExpandPinRecursively(OutputPin->GetParentPin(), bSetupUndoRedo);
		ExpandPinRecursively(InputPin->GetParentPin(), bSetupUndoRedo);
	}

	// Before adding the link, let's resolve input and ouput pin types
	// If templates, we will filter the permutations that support this link
	// If any links need to be broken before perfoming this connection, try to find them and break them
	if (!bIsTransacting)
	{
		if (!InputPin->IsExecuteContext() && !OutputPin->IsExecuteContext())
		{
			URigVMPin* FirstToResolve = (InUserDirection == ERigVMPinDirection::Input) ? OutputPin : InputPin;
			URigVMPin* SecondToResolve = (FirstToResolve == OutputPin) ? InputPin : OutputPin;
			if (!PrepareToLink(FirstToResolve, SecondToResolve, bSetupUndoRedo))
			{
				if (bSetupUndoRedo)
				{
					GetActionStack()->CancelAction(Action);
				}
				return false;
			}
		}
		else if (InputPin->GetNode()->IsA<URigVMRerouteNode>() || OutputPin->GetNode()->IsA<URigVMRerouteNode>())
		{
			URigVMPin* PinToResolve = (OutputPin->GetNode()->IsA<URigVMRerouteNode>()) ? OutputPin : InputPin;
			URigVMPin* PinToSkip = (PinToResolve == OutputPin) ? InputPin : OutputPin;
			if (PinToResolve->GetTypeIndex() != PinToSkip->GetTypeIndex())
			{
				if (!ResolveWildCardPin(PinToResolve, PinToSkip->GetTypeIndex(), bSetupUndoRedo))
				{
					if (bSetupUndoRedo)
					{
						GetActionStack()->CancelAction(Action);
					}
					return false;
				}
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMAddLinkAction(this, OutputPin, InputPin));
	}

	URigVMLink* Link = nullptr;
	
	// try to reuse UObjects that may still be sitting in the detached links array
	const FString OutputPinPath = OutputPin->GetPinPath();
	const FString InputPinPath = InputPin->GetPinPath();
	TObjectPtr<URigVMLink>* ExistingLink = Graph->DetachedLinks.FindByPredicate(
		[OutputPinPath, InputPinPath](const URigVMLink* Link) -> bool
		{
			return Link->GetSourcePinPath().Equals(OutputPinPath, ESearchCase::CaseSensitive) &&
				Link->GetTargetPinPath().Equals(InputPinPath, ESearchCase::CaseSensitive);
		});

	if(ExistingLink)
	{
		Link = *ExistingLink;
		Graph->DetachedLinks.Remove(Link);
	}
	else
	{
		Link = NewObject<URigVMLink>(Graph);
	}
	
	Link->SetSourceAndTargetPinPaths(OutputPin->GetPinPath(), InputPin->GetPinPath());
	Graph->Links.Add(Link);
	OutputPin->Links.Add(Link);
	InputPin->Links.Add(Link);
	OutputPin->IncrementVersion();
	InputPin->IncrementVersion();

	UpdateGraphSectionsIfRequired();

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}
	Notify(ERigVMGraphNotifType::LinkAdded, Link);

	if (bSetupUndoRedo)
	{
#if WITH_EDITOR
		if (!bSuspendTemplateComputation)
		{
			auto ResolveTemplateNodeToCommonTypes = [this, bSetupUndoRedo](URigVMPin* Pin)
			{
				if(!Pin->IsExecuteContext())
				{
					return;
				}
			
				URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Pin->GetNode());
				if(TemplateNode == nullptr)
				{
					return;
				}

				const FRigVMTemplate* Template = TemplateNode->GetTemplate();
				if(Template == nullptr)
				{
					return;
				}

				if(!TemplateNode->HasWildCardPin())
				{
					return;
				}

				const FRigVMTemplate::FTypeMap PreferredTypes = GetCommonlyUsedTypesForTemplate(TemplateNode);
				if(PreferredTypes.IsEmpty())
				{
					return;
				}

				const int32 PreferredPermutation = Template->FindPermutation(PreferredTypes);
				if(PreferredPermutation != INDEX_NONE)
				{
					const TGuardValue<bool> DisableRegisterUseOfTemplate(bRegisterTemplateNodeUsage, false);
					if(FullyResolveTemplateNode(TemplateNode, PreferredPermutation, bSetupUndoRedo))
					{
						static const FString Message = TEXT("Template node was automatically resolved to commonly used types.");
						SendUserFacingNotification(Message, 0.f, TemplateNode, TEXT("MessageLog.Note"));
					}
				}
			};

			ResolveTemplateNodeToCommonTypes(OutputPin);
			ResolveTemplateNodeToCommonTypes(InputPin);
		}
#endif
		
		GetActionStack()->EndAction(Action);
	}

	if (!bIsTransacting)
	{
		ensureMsgf(RigVMTypeUtils::AreCompatible(*OutputPin->GetCPPType(), OutputPin->GetCPPTypeObject(), *InputPin->GetCPPType(), InputPin->GetCPPTypeObject()),
		   TEXT("Incompatible types after successful link %s (%s) -> %s (%s) created in %s")
		   , *OutputPin->GetPinPath(true)
		   , *OutputPin->GetCPPType()
		   , *InputPin->GetPinPath(true)
		   , *InputPin->GetCPPType()
		   , *GetPackage()->GetPathName());
	}

	return true;
}

void URigVMController::RelinkSourceAndTargetPins(URigVMNode* Node, bool bSetupUndoRedo)
{
	TArray<URigVMPin*> SourcePins;
	TArray<URigVMPin*> TargetPins;
	TArray<URigVMLink*> LinksToRemove;

	// store source and target links 
	const TArray<URigVMLink*> RigVMLinks = Node->GetLinks();
	for (URigVMLink* Link: RigVMLinks)
	{
		URigVMPin* SrcPin = Link->GetSourcePin();
		if (SrcPin && SrcPin->GetNode() != Node)
		{
			SourcePins.AddUnique(SrcPin);
			LinksToRemove.AddUnique(Link);
		}

		URigVMPin* DstPin = Link->GetTargetPin();
		if (DstPin && DstPin->GetNode() != Node)
		{
			TargetPins.AddUnique(DstPin);
			LinksToRemove.AddUnique(Link);
		}
	}

	if( SourcePins.Num() > 0 && TargetPins.Num() > 0 )
	{
		// remove previous links 
		for (URigVMLink* Link: LinksToRemove)
		{
			BreakLink(Link->GetSourcePin(), Link->GetTargetPin(), bSetupUndoRedo); 
		}

		// relink pins if feasible 
		TArray<bool> TargetHandled;
		TargetHandled.AddZeroed(TargetPins.Num());
		for (URigVMPin* Src: SourcePins)
		{
			for (int32 Index = 0; Index < TargetPins.Num(); Index++)
			{
				if (!TargetHandled[Index])
				{
					if (URigVMPin::CanLink(Src, TargetPins[Index], nullptr, nullptr))
					{
						// execute pins can be linked to one target only so link to the 1st compatible target
						const bool bNeedNewLink = Src->IsExecuteContext() ? (Src->GetTargetLinks().Num() == 0) : true;
						if (bNeedNewLink)
						{
							AddLink(Src, TargetPins[Index], bSetupUndoRedo);
							TargetHandled[Index] = true;								
						}
					}
				}
			}
		}
	}
}

bool URigVMController::BreakLink(const FString& InOutputPinPath, const FString& InInputPinPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* OutputPin = Graph->FindPin(InOutputPinPath);
	if (OutputPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InOutputPinPath);
		return false;
	}
	OutputPin = OutputPin->GetPinForLink();

	URigVMPin* InputPin = Graph->FindPin(InInputPinPath);
	if (InputPin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InInputPinPath);
		return false;
	}
	InputPin = InputPin->GetPinForLink();

	const bool bSuccess = BreakLink(OutputPin, InputPin, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').break_link('%s', '%s')"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(OutputPin->GetPinPath()),
			*GetSchema()->GetSanitizedPinPath(InputPin->GetPinPath())));
	}
	return bSuccess;
}

bool URigVMController::BreakLink(URigVMPin* OutputPin, URigVMPin* InputPin, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if(!GetSchema()->CanBreakLink(this, OutputPin, InputPin))
	{
		return false;
	}

	for (URigVMLink* Link : InputPin->Links)
	{
		if (Link->GetSourcePin() == OutputPin && Link->GetTargetPin() == InputPin)
		{
			FRigVMControllerCompileBracketScope CompileScope(this);
			FRigVMControllerGraphSectionsScope GraphSectionsScope(this);
			FRigVMBreakLinkAction Action;
			if (bSetupUndoRedo)
			{
				Action = FRigVMBreakLinkAction(this, OutputPin, InputPin);
				Action.SetTitle(FString::Printf(TEXT("Break Link")));
				GetActionStack()->BeginAction(Action);
			}

			OutputPin->Links.Remove(Link);
			InputPin->Links.Remove(Link);
			Graph->Links.Remove(Link);

			// each time a link is removed, existing orphaned pins
			// may become unused and thus can be removed
			RemoveUnusedOrphanedPins(OutputPin->GetNode());
			RemoveUnusedOrphanedPins(InputPin->GetNode());
			
			if (!bSuspendNotifications)
			{
				(void)MarkPackageDirty(Graph);
			}
			Notify(ERigVMGraphNotifType::LinkRemoved, Link);

			DestroyObject(Link);

			if (bSetupUndoRedo)
			{
				GetActionStack()->EndAction(Action);
			}

			return true;
		}
	}

	return false;
}

bool URigVMController::BreakAllLinks(const FString& InPinPath, bool bAsInput, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}


	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return false;
	}
	Pin = Pin->GetPinForLink();

	if (!IsValidPinForGraph(Pin))
	{
		return false;
	}

	const bool bSuccess = BreakAllLinks(Pin, bAsInput, bSetupUndoRedo);
	if (bSuccess && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').break_all_links('%s', %s)"),
			*GraphName,
			*GetSchema()->GetSanitizedPinPath(Pin->GetPinPath()),
			bAsInput ? TEXT("True") : TEXT("False")));
	}
	return bSuccess;
}

bool URigVMController::BreakAllLinks(URigVMPin* Pin, bool bAsInput, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValid(Pin))
	{
		return false;
	}
	
	if(!Pin->IsLinked(false))
	{
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);
	
	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Break All Links")));
		GetActionStack()->BeginAction(Action);
	}

	int32 LinksBroken = 0;
	{
		if (Pin->IsBoundToVariable() && bAsInput && bSetupUndoRedo)
		{
			UnbindPinFromVariable(Pin, bSetupUndoRedo);
			LinksBroken++;
		}

		TArray<URigVMLink*> Links = Pin->GetLinks();
		for (int32 LinkIndex = Links.Num() - 1; LinkIndex >= 0; LinkIndex--)
		{
			URigVMLink* Link = Links[LinkIndex];
			if (bAsInput && Link->GetTargetPin() == Pin)
			{
				LinksBroken += BreakLink(Link->GetSourcePin(), Pin, bSetupUndoRedo) ? 1 : 0;
			}
			else if (!bAsInput && Link->GetSourcePin() == Pin)
			{
				LinksBroken += BreakLink(Pin, Link->GetTargetPin(), bSetupUndoRedo) ? 1 : 0;
			}
		}
	}

	if (bSetupUndoRedo)
	{
		if (LinksBroken > 0)
		{
			GetActionStack()->EndAction(Action);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	if (LinksBroken > 0)
	{
		UpdateGraphSectionsIfRequired();
	}

	return LinksBroken > 0;
}

bool URigVMController::BreakAllLinksRecursive(URigVMPin* Pin, bool bAsInput, bool bTowardsParent, bool bSetupUndoRedo)
{
	bool bBrokenLinks = false;
	{
		if (bTowardsParent)
		{
			URigVMPin* ParentPin = Pin->GetParentPin();
			if (ParentPin)
			{
				bBrokenLinks |= BreakAllLinks(ParentPin, bAsInput, bSetupUndoRedo);
				bBrokenLinks |= BreakAllLinksRecursive(ParentPin, bAsInput, bTowardsParent, bSetupUndoRedo);
			}
		}
		else
		{
			TArray<URigVMPin*> SubPins = Pin->SubPins;
			for (URigVMPin* SubPin : SubPins)
			{
				bBrokenLinks |= BreakAllLinks(SubPin, bAsInput, bSetupUndoRedo);
				bBrokenLinks |= BreakAllLinksRecursive(SubPin, bAsInput, bTowardsParent, bSetupUndoRedo);
			}
		}
	}

	return bBrokenLinks;
}

FName URigVMController::AddExposedPin(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bIsInputVariable)
{
	return AddExposedPin_Internal(InPinName, InDirection, InCPPType, InCPPTypeObjectPath, InDefaultValue, bSetupUndoRedo, bPrintPythonCommand, bIsInputVariable, /*bAllowExecutePinManagement=*/ false);
}

FName URigVMController::AddExposedPin_Internal(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bIsInputVariable, bool bAllowExecutePinManagement)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NAME_None;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsTopLevelGraph())
	{
		ReportError(TEXT("Exposed pins can only be edited on nested graphs."));
		return NAME_None;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot expose pins in function library graphs."));
		return NAME_None;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	check(LibraryNode);

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsNone())
	{
		if (CPPTypeObject == nullptr)
		{
			CPPTypeObject = URigVMCompiler::GetScriptStructForCPPType(InCPPTypeObjectPath.ToString());
		}
		if (CPPTypeObject == nullptr)
		{
			CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
		}
	}

	// For now we do not allow wildcards for library nodes
	if (CPPTypeObject)
	{
		if(CPPTypeObject == RigVMTypeUtils::GetWildCardCPPTypeObject())
		{
			ReportError(TEXT("Cannot expose pins of wildcard type in functions."));
			return NAME_None;
		}
	}
	
	// only allow one IO / input exposed pin of type execute context per direction
	// except for aggregate nodes that can have multiple exec outputs
	bool bCheckForExecUniqueness = true;
	if (LibraryNode->IsA<URigVMAggregateNode>())
	{
		bCheckForExecUniqueness = InDirection != ERigVMPinDirection::Output;
	}
	
	if (bCheckForExecUniqueness)
	{
		if(const UScriptStruct* CPPTypeStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			if(CPPTypeStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				if (!bAllowExecutePinManagement)
				{
					if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
					{
						if (CollapseNode->IsGraphFunctionDefinition())
						{
							ReportError(TEXT("Execute pins cannot be manually added to functions. Use SetFunctionIsPure() to change mutability."));
							return NAME_None;
						}
					}
				}

				for(const URigVMPin* ExistingPin : LibraryNode->Pins)
				{
					if(ExistingPin->IsExecuteContext())
					{
						return NAME_None;
					}
				}
			}
		}
	}

	FName PinName = URigVMSchema::GetUniqueName(InPinName, [LibraryNode](const FName& InName) {

		if(LibraryNode->FindPin(InName.ToString()) != nullptr)
		{
			return false;
		}

		const TArray<FRigVMGraphVariableDescription>& LocalVariables = LibraryNode->GetContainedGraph()->GetLocalVariables(true);
		for(const FRigVMGraphVariableDescription& VariableDescription : LocalVariables)
		{
			if (VariableDescription.Name == InName)
			{
				return false;
			}
		}
		return true;

	}, false, true);

	const FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, CPPTypeObject);
	const FRigVMTemplateArgumentType Type(*CPPType, CPPTypeObject);
	const TRigVMTypeIndex TypeIndex = FRigVMRegistry::Get().FindOrAddType(Type);
	if(!GetSchema()->SupportsType(this, TypeIndex))
	{
		return NAME_None;
	}

	URigVMPin* Pin = NewObject<URigVMPin>(LibraryNode, PinName);
	Pin->CPPType = CPPType;
	Pin->CPPTypeObjectPath = InCPPTypeObjectPath;
	Pin->bIsConstant = false;
	Pin->bIsInputVariable = bIsInputVariable;
	if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
	{
		CollapseNode->PinNameToGuid.FindOrAdd(PinName) = FGuid::NewGuid();
	}

	Pin->Direction = InDirection;
	Pin->bIsDynamicArray = RigVMTypeUtils::IsArrayType(CPPType);
	AddNodePin(LibraryNode, Pin);

	if (Pin->IsStruct())
	{
		if(URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
		{
			FString DefaultValue = InDefaultValue;
			LibraryController->CreateDefaultValueForStructIfRequired(Pin->GetScriptStruct(), DefaultValue);
			{
				TGuardValue<bool> SuspendNotifications(LibraryController->bSuspendNotifications, true);
				LibraryController->AddPinsForStruct(Pin->GetScriptStruct(), LibraryNode, Pin, Pin->Direction, DefaultValue, false);
			}
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMAddExposedPinAction Action(this, Pin);
	if (bSetupUndoRedo)
	{
		GetActionStack()->BeginAction(Action);
	}

	if(URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
	{
		LibraryController->Notify(ERigVMGraphNotifType::PinAdded, Pin);
	}

	if (!InDefaultValue.IsEmpty())
	{
		if(URigVMController* PinController = GetControllerForGraph(Pin->GetGraph()))
		{
			PinController->SetPinDefaultValue(Pin, InDefaultValue, true, bSetupUndoRedo, false);
		}
	}

	URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode();
	if (!EntryNode)
	{
		EntryNode = NewObject<URigVMFunctionEntryNode>(Graph, TEXT("Entry"));

		if(AddGraphNode(EntryNode, false))
		{
			TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
			RefreshFunctionPins(EntryNode);
			Notify(ERigVMGraphNotifType::NodeAdded, EntryNode);
		}
	}

	URigVMFunctionReturnNode* ReturnNode = Graph->GetReturnNode();
	if (!ReturnNode)
	{
		ReturnNode = NewObject<URigVMFunctionReturnNode>(Graph, TEXT("Return"));
		if(AddGraphNode(ReturnNode, false))
		{
			TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
			RefreshFunctionPins(ReturnNode);
			Notify(ERigVMGraphNotifType::NodeAdded, ReturnNode);
		}
	}

	
	RefreshFunctionPins(EntryNode);
	RefreshFunctionPins(ReturnNode);

	RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		//AddExposedPin(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)

		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		static constexpr TCHAR AddExposedPinFormat[] = TEXT("asset.get_controller_by_name('%s').add_exposed_pin('%s', %s, '%s', '%s', '%s', %s, %s, %s)");
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(AddExposedPinFormat,
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InPinName.ToString()),
				*RigVMPythonUtils::EnumValueToPythonString<ERigVMPinDirection>((int64)InDirection),
				*InCPPType,
				*InCPPTypeObjectPath.ToString(),
				*InDefaultValue,
				bSetupUndoRedo ? TEXT("True") : TEXT("False"),
				bPrintPythonCommand ? TEXT("True") : TEXT("False"),
				bIsInputVariable ? TEXT("True") : TEXT("False")
			));
	}
	
	return PinName;
}

bool URigVMController::RemoveExposedPin(const FName& InPinName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return RemoveExposedPin_Internal(InPinName, bSetupUndoRedo, bPrintPythonCommand, /*bAllowExecutePinManagement=*/ false);
}

bool URigVMController::RemoveExposedPin_Internal(const FName& InPinName, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bAllowExecutePinManagement)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsTopLevelGraph())
	{
		ReportError(TEXT("Exposed pins can only be edited on nested graphs."));
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot remove exposed pins in function library graphs."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	check(LibraryNode);

	URigVMPin* Pin = LibraryNode->FindPin(InPinName.ToString());
	if (Pin == nullptr)
	{
		return false;
	}

	if (Pin->IsExecuteContext() && !bAllowExecutePinManagement)
	{
		if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
		{
			if (CollapseNode->IsGraphFunctionDefinition())
			{
				ReportError(TEXT("Execute pins cannot be manually removed from functions. Use SetFunctionIsPure() to change mutability."));
				return false;
			}
		}
	}

	if (URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(LibraryNode))
	{
		if (AggregateNode->IsInputAggregate())
		{
			if (Pin->GetDirection() != ERigVMPinDirection::Input)
			{
				return false;
			}

			if (AggregateNode->GetAggregateInputs().Num() <= 2)
			{
				return false;
			}
		}
		else
		{
			if (Pin->GetDirection() != ERigVMPinDirection::Output)
			{
				return false;
			}

			if (AggregateNode->GetAggregateOutputs().Num() <= 2)
			{
				return false;
			}
		}
	}

	if(bSetupUndoRedo)
	{
		if(RequestBulkEditDialogDelegate.IsBound())
		{
			FRigVMController_BulkEditResult Result = RequestBulkEditDialogDelegate.Execute(LibraryNode, ERigVMControllerBulkEditType::RemoveExposedPin);
			if(Result.bCanceled)
			{
				return false;
			}
			bSetupUndoRedo = Result.bSetupUndoRedo;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMRemoveExposedPinAction Action(this, Pin);
	if (bSetupUndoRedo)
	{
		GetActionStack()->BeginAction(Action);
	}

	bool bSuccessfullyRemovedPin = false;
	{
		FGuid Guid;
		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
		{
			CollapseNode->PinNameToGuid.RemoveAndCopyValue(InPinName, Guid);
		}
		
		if(URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
		{
			bSuccessfullyRemovedPin = LibraryController->RemovePin(Pin, bSetupUndoRedo, true);
		}

		TArray<URigVMVariableNode*> NodesToRemove;
		for (URigVMNode* Node : Graph->GetNodes())
		{
			if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (Guid.IsValid())
				{
					if (VariableNode->GetVariableGuid() == Guid)
					{
						NodesToRemove.AddUnique(VariableNode);
					}
				}

				// Guid accuracy relies on PatchPinGuidInCollapseNodes, but some RigVM users don't patch. So allow for variable name fallback.
				if (VariableNode->GetVariableName() == InPinName)
				{
					NodesToRemove.AddUnique(VariableNode);
				}
			}
		}
		for (int32 i=NodesToRemove.Num()-1; i >= 0; --i)
		{
			RemoveNode(NodesToRemove[i], bSetupUndoRedo);
		}

		RefreshFunctionPins(Graph->GetEntryNode(), bSetupUndoRedo);
		RefreshFunctionPins(Graph->GetReturnNode(), bSetupUndoRedo);
		RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
	}

	if (bSetupUndoRedo)
	{
		if (bSuccessfullyRemovedPin)
		{
			GetActionStack()->EndAction(Action);
		}
		else
		{
			GetActionStack()->CancelAction(Action);
		}
	}

	if (bSuccessfullyRemovedPin && bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_exposed_pin('%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InPinName.ToString())));
	}

	return bSuccessfullyRemovedPin;
}

bool URigVMController::RenameExposedPin(const FName& InOldPinName, const FName& InNewPinName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsTopLevelGraph())
	{
		ReportError(TEXT("Exposed pins can only be edited on nested graphs."));
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot rename exposed pins in function library graphs."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	check(LibraryNode);

	URigVMPin* Pin = LibraryNode->FindPin(InOldPinName.ToString());
	if (Pin == nullptr)
	{
		return false;
	}

	if (Pin->GetFName() == InNewPinName)
	{
		return false;
	}
	
	if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
	{
		FGuid Guid;
		CollapseNode->PinNameToGuid.RemoveAndCopyValue(InOldPinName, Guid);
		ensure(Guid.IsValid());
		CollapseNode->PinNameToGuid.FindOrAdd(InNewPinName) = Guid;
	}

	if(bSetupUndoRedo)
	{
		if(RequestBulkEditDialogDelegate.IsBound())
		{
			FRigVMController_BulkEditResult Result = RequestBulkEditDialogDelegate.Execute(LibraryNode, ERigVMControllerBulkEditType::RenameExposedPin); 
			if(Result.bCanceled)
			{
				return false;
			}
			bSetupUndoRedo = Result.bSetupUndoRedo;
		}
	}

	FName PinName = GetSchema()->GetUniqueName(InNewPinName, [LibraryNode](const FName& InName) {
		const TArray<FRigVMGraphVariableDescription>& LocalVariables = LibraryNode->GetContainedGraph()->GetLocalVariables(true);
		for(const FRigVMGraphVariableDescription& VariableDescription : LocalVariables)
		{
			if (VariableDescription.Name == InName)
			{
				return false;
			}
		}
		return true;
	}, false, true);

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMRenameExposedPinAction Action;
	if (bSetupUndoRedo)
	{
		Action = FRigVMRenameExposedPinAction(this, Pin->GetFName(), PinName);
		GetActionStack()->BeginAction(Action);
	}

	struct Local
	{
		static bool RenamePin(const URigVMController* InController, URigVMPin* InPin, const FName& InNewName)
		{
			check(!InPin->IsTraitPin());
			
			URigVMController* PinController = InController->GetControllerForGraph(InPin->GetGraph());
			if(PinController == nullptr)
			{
				return false;
			}
			TArray<FLinkedPath> LinkedPaths = PinController->GetLinkedPaths(InPin, true, true);

			PinController->FastBreakLinkedPaths(LinkedPaths);

			const FString OldPinPath = InPin->GetPinPath();
			static constexpr TCHAR PinPrefixFormat[] = TEXT("%s.");
			const FString OldPinPathPrefix = FString::Printf(PinPrefixFormat, *OldPinPath);
			
			if (!InController->RenameObject(InPin, *InNewName.ToString()))
			{
				return false;
			}

			InPin->SetDisplayName(InNewName);
			InPin->IncrementVersion();
			
			PinController->Notify(ERigVMGraphNotifType::PinRenamed, InPin);

			FRestoreLinkedPathSettings Settings;
			Settings.RemapDelegates = {{
				InPin->GetNode()->GetName(), FRigVMController_PinPathRemapDelegate::CreateLambda([
					OldPinPath,
					OldPinPathPrefix,
					InPin
				](const FString& InPinPath, bool bIsInput) -> FString
				{
					if(InPinPath == OldPinPath)
					{
						return InPin->GetPinPath();
					}

					if(InPinPath.StartsWith(OldPinPathPrefix, ESearchCase::CaseSensitive))
					{
						return InPin->GetPinPath() + InPinPath.Mid(OldPinPath.Len());
					}

					return InPinPath;
				})
			}};
			PinController->RestoreLinkedPaths(LinkedPaths, Settings);

			return true;
		}
	};

	if (!Local::RenamePin(this, Pin, PinName))
	{
		GetActionStack()->CancelAction(Action);
		return false;
	}

	TArray<URigVMTemplateNode*> InterfaceNodes = {Graph->GetEntryNode(), Graph->GetReturnNode()};
	for (URigVMTemplateNode* InterfaceNode : InterfaceNodes)
	{
		if (InterfaceNode)
		{
			if (URigVMPin* InterfacePin = InterfaceNode->FindPin(InOldPinName.ToString()))
			{
				Local::RenamePin(this, InterfacePin, PinName);
			}
		}
	}

	if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
	{
		FunctionLibrary->ForEachReference(LibraryNode->GetFName(), [this, InOldPinName, PinName](URigVMFunctionReferenceNode* ReferenceNode)
        {
			if (URigVMPin* EntryPin = ReferenceNode->FindPin(InOldPinName.ToString()))
			{
				if(URigVMController* ReferenceController = GetControllerForGraph(ReferenceNode->GetGraph()))
				{
					Local::RenamePin(ReferenceController, EntryPin, PinName);
				}
            }
        });
	}

	for (URigVMNode* Node : Graph->GetNodes())
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableName() == InOldPinName)
			{
				SetVariableName(VariableNode, InNewPinName, bSetupUndoRedo);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').rename_exposed_pin('%s', '%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InOldPinName.ToString()),
				*GetSchema()->GetSanitizedPinName(InNewPinName.ToString())));
	}

	return true;
}

bool URigVMController::ChangeExposedPinType(const FName& InPinName, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool& bSetupUndoRedo, bool bSetupOrphanPins, bool bPrintPythonCommand)
{
	return ChangeExposedPinType_Internal(InPinName, InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupOrphanPins, bPrintPythonCommand, /*bAllowExecutePinManagement=*/ false);
}

bool URigVMController::ChangeExposedPinType_Internal(const FName& InPinName, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool& bSetupUndoRedo, bool bSetupOrphanPins, bool bPrintPythonCommand, bool bAllowExecutePinManagement)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsTopLevelGraph())
	{
		ReportError(TEXT("Exposed pins can only be edited on nested graphs."));
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot change exposed pin types in function library graphs."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	check(LibraryNode);

	URigVMPin* Pin = LibraryNode->FindPin(InPinName.ToString());
	if (Pin == nullptr)
	{
		return false;
	}

	// Guard: cannot change existing execute pin type on function definitions
	if (!bAllowExecutePinManagement)
	{
		if (Pin->IsExecuteContext())
		{
			if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
			{
				if (CollapseNode->IsGraphFunctionDefinition())
				{
					ReportError(TEXT("Cannot change execute pin type on functions. Use SetFunctionIsPure() to change mutability."));
					return false;
				}
			}
		}
	}

	// We do not allow unresolving exposed pins
	if (InCPPType == RigVMTypeUtils::GetWildCardCPPType())
	{
		ReportError(TEXT("Cannot change exposed pin type to wildcard."));
		return false;
	}
	
	// only allow one exposed pin of type execute context per direction
	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsNone())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
		if(CPPTypeObject)
		{
			if(const UScriptStruct* CPPTypeStruct = Cast<UScriptStruct>(CPPTypeObject))
			{
				if(CPPTypeStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					if (!bAllowExecutePinManagement)
					{
						if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
						{
							if (CollapseNode->IsGraphFunctionDefinition())
							{
								ReportError(TEXT("Cannot change pin type to execute context on functions. Use SetFunctionIsPure() to change mutability."));
								return false;
							}
						}
					}

					for(URigVMPin* ExistingPin : LibraryNode->Pins)
					{
						if(ExistingPin != Pin)
						{
							if(ExistingPin->IsExecuteContext())
							{
								return false;
							}
						}
					}
				}
			}
		}
	}

	const FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, CPPTypeObject);
	const FRigVMTemplateArgumentType Type(*CPPType, CPPTypeObject);
	const TRigVMTypeIndex TypeIndex = FRigVMRegistry::Get().FindOrAddType(Type);
	if(!GetSchema()->SupportsType(this, TypeIndex))
	{
		return false;
	}

	bool bIsExecute = false;
	if (CPPTypeObject)
	{
		if(const UScriptStruct* CPPTypeStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			if(CPPTypeStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				bIsExecute = true;
			}
		}
	}

	if (bIsExecute)
	{
		if (Pin->GetDirection() != ERigVMPinDirection::IO)
		{
			Pin->Direction = ERigVMPinDirection::IO;
		}
	}
	else if(Pin->GetDirection() == ERigVMPinDirection::IO)
	{
		ReportAndNotifyError(TEXT("Input/Output pins only allow Execute Context types."));
		return false;
	}

	if(bSetupUndoRedo)
	{
		if(RequestBulkEditDialogDelegate.IsBound())
		{
			const FRigVMController_BulkEditResult Result = RequestBulkEditDialogDelegate.Execute(LibraryNode, ERigVMControllerBulkEditType::ChangeExposedPinType); 
			if(Result.bCanceled)
			{
				return false;
			}
			bSetupUndoRedo = Result.bSetupUndoRedo;
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Change Exposed Pin Type")));
		GetActionStack()->BeginAction(Action);
	}

	FRigVMRegistry& Registry = FRigVMRegistry::Get();

	// If the pin does not support the type, first break all links in the contained graph to this pin (and subpins)
	TArray<URigVMTemplateNode*> InterfaceNodes = {Graph->GetEntryNode(), Graph->GetReturnNode()};

	// Break all links to this pin
	{
		TArray<URigVMLink*> InterfacePinLinks;
		TArray<URigVMPin*> ExtendedInterfacePins;
		for (URigVMNode* Node : Graph->GetNodes())
		{
			if (Node->IsA<URigVMFunctionInterfaceNode>())
			{
				if (URigVMPin* InterfacePin = Node->FindPin(Pin->GetName()))
				{
					ExtendedInterfacePins.Add(InterfacePin);
				}
			}
			else if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
			{
				if (VariableNode->GetVariableName() == InPinName)
				{
					ExtendedInterfacePins.Add(VariableNode->GetValuePin());
				}
			}
		}

		for (URigVMPin* InterfacePin : ExtendedInterfacePins)
		{
			TArray<URigVMPin*> PinsToProcess;
			PinsToProcess.Add(InterfacePin);
			for (int32 i=0; i<PinsToProcess.Num(); ++i)
			{
				InterfacePinLinks.Append(PinsToProcess[i]->GetLinks());
				PinsToProcess.Append(PinsToProcess[i]->GetSubPins());
			}
		}
		
		for (int32 i=0; i<InterfacePinLinks.Num(); ++i)
		{
			BreakLink(InterfacePinLinks[i]->GetSourcePin(), InterfacePinLinks[i]->GetTargetPin(), bSetupUndoRedo);
		}
	}

	// Change pin type of the library node in the function library
	{
		bool bSuccessChangingType = false;
		if(URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
		{
			bSuccessChangingType = LibraryController->ChangePinType(Pin, InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupOrphanPins);

			if (bSuccessChangingType)
			{
				LibraryController->RemoveUnusedOrphanedPins(LibraryNode);
			}
		}
		if (!bSuccessChangingType)
		{
			if (bSetupUndoRedo)
			{
				GetActionStack()->CancelAction(Action);
			}
			return false;
		}
	}

	// Repopulate pin on interface nodes
	for (URigVMTemplateNode* InterfaceNode : InterfaceNodes)
	{
		if (InterfaceNode)
		{
			const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InterfaceNode);
			FastBreakLinkedPaths(LinkedPaths);
			RepopulatePinsOnNode(InterfaceNode, true, bSetupOrphanPins, true);
			FRestoreLinkedPathSettings Settings;
			RestoreLinkedPaths(LinkedPaths, Settings);
			RemoveUnusedOrphanedPins(InterfaceNode);
		}
	}

	// Change pin type on function references
	if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
	{
		RefreshFunctionReferences(LibraryNode, bSetupUndoRedo, false);
	}

	// Change pin types on input variable nodes
	for (URigVMNode* Node : Graph->GetNodes())
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableName() == InPinName)
			{
				URigVMPin* ValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName);
				if (ValuePin)
				{
					ChangePinType(ValuePin, InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupOrphanPins);
					RemoveUnusedOrphanedPins(VariableNode);
				}
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').change_exposed_pin_type('%s', '%s', '%s', %s)"),
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InPinName.ToString()),
				*InCPPType,
				*InCPPTypeObjectPath.ToString(),
				(bSetupUndoRedo) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

bool URigVMController::SetExposedPinIndex(const FName& InPinName, int32 InNewIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FString PinPath = InPinName.ToString();
	if (PinPath.Contains(TEXT(".")))
	{
		ReportError(TEXT("Cannot change pin index for pins on nodes for now - only within collapse nodes."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	if (LibraryNode == nullptr)
	{
		ReportError(TEXT("Graph is not under a Collapse Node"));
		return false;
	}

	URigVMPin* Pin = LibraryNode->FindPin(PinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find exposed pin '%s'."), *PinPath);
		return false;
	}

	if (Pin->GetPinIndex() == InNewIndex)
	{
		return true; // Nothing to do, do not fail
	}

	if (InNewIndex < 0 || InNewIndex >= LibraryNode->GetPins().Num())
	{
		ReportErrorf(TEXT("Invalid new pin index '%d'."), InNewIndex);
		return false;
	}

	FRigVMControllerCompileBracketScope CompileBracketScope(this);

	FRigVMSetPinIndexAction PinIndexAction(this, Pin, InNewIndex);
	{
		LibraryNode->Pins.Remove(Pin);
		LibraryNode->Pins.Insert(Pin, InNewIndex);

		if(const URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
		{
			LibraryController->Notify(ERigVMGraphNotifType::PinIndexChanged, Pin);
		}
	}

	RefreshFunctionPins(LibraryNode->GetEntryNode());
	RefreshFunctionPins(LibraryNode->GetReturnNode());
	RefreshFunctionReferences(LibraryNode, false, false);
	
	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(PinIndexAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_exposed_pin_index('%s', %d)"),
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InPinName.ToString()),
				InNewIndex));
	}

	return true;
}

bool URigVMController::MarkExposedPinAsInputVariable(const FName& InPinName, bool bDefinedAsExternalVariable, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FString PinPath = InPinName.ToString();
	if (PinPath.Contains(TEXT(".")))
	{
		ReportError(TEXT("Cannot change pin index for pins on nodes for now - only within collapse nodes."));
		return false;
	}

	URigVMCollapseNode* LibraryNode = Cast<URigVMCollapseNode>(Graph->GetOuter());
	if (LibraryNode == nullptr)
	{
		ReportError(TEXT("Graph is not under a Collapse Node"));
		return false;
	}

	if (!Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
	{
		ReportError(TEXT("Input Variables can only be used on functions - not on normal collapse nodes."));
		return false;
	}

	URigVMPin* Pin = LibraryNode->FindPin(PinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find exposed pin '%s'."), *PinPath);
		return false;
	}

	if (Pin->IsDefinedAsInputVariable() == bDefinedAsExternalVariable)
	{
		return true; // Nothing to do, do not fail
	}

	if (Pin->IsExecuteContext())
	{
		ReportErrorf(TEXT("Pin '%s' is an execute context. Only non-execute pins can be marked as external variable."), *PinPath);
		return false;
	}

	if (Pin->GetDirection() != ERigVMPinDirection::Input)
	{
		ReportErrorf(TEXT("Pin '%s' is not an input pin."), *PinPath);
		return false;
	}

	FRigVMControllerCompileBracketScope CompileBracketScope(this);

	FRigVMMarkExposedPinAsInputVariableAction PinExternalVariableAction(this, Pin, bDefinedAsExternalVariable);
	if (bSetupUndoRedo)
	{
		GetActionStack()->BeginAction(PinExternalVariableAction);
	}
	
	{
		Pin->Modify();
		Pin->bIsInputVariable = bDefinedAsExternalVariable;
		Pin->IncrementVersion();
		Notify(ERigVMGraphNotifType::PinTypeChanged, Pin);

		if(URigVMController* LibraryController = GetControllerForGraph(LibraryNode->GetGraph()))
		{
			// if we are marking an input variable as a normal argument
			// we need to run the code below to clean up obsolete variable nodes (like SetVariable)
			if (!bDefinedAsExternalVariable)
			{
				FGuid Guid = LibraryNode->FindPinGuid(Pin);
				if (Guid.IsValid())
				{
					LibraryController->OnExternalVariableRemoved(Guid, bSetupUndoRedo);
				}
			}
		}
	}

	RefreshFunctionPins(LibraryNode->GetEntryNode());
	RefreshFunctionPins(LibraryNode->GetReturnNode());
	RefreshFunctionReferences(LibraryNode, false, false);
	
	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(PinExternalVariableAction);
	}
	
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').mark_exposed_pin_as_input_variable('%s', %s)"),
				*GraphName,
				*GetSchema()->GetSanitizedPinName(InPinName.ToString()),
				(bDefinedAsExternalVariable) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

FRigVMGraphFunctionHeader URigVMController::FindGraphFunctionHeaderByName(FString InHostPath, FName InFunctionName) const
{
	FString ErrorMessage;
	const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(InHostPath, InFunctionName, nullptr, &ErrorMessage);
	if(!Header.IsValid() && !ErrorMessage.IsEmpty())
	{
		ReportError(ErrorMessage);
	}
	return Header;
}

FRigVMGraphFunctionHeader URigVMController::FindGraphFunctionHeader(FRigVMGraphFunctionIdentifier InFunctionIdentifier) const
{
	FString ErrorMessage;
	const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(InFunctionIdentifier, nullptr, &ErrorMessage);
	if(!Header.IsValid() && !ErrorMessage.IsEmpty())
	{
		ReportError(ErrorMessage);
	}
	return Header;
}

FRigVMGraphFunctionIdentifier URigVMController::FindGraphFunctionIdentifier(FString InHostPath, FName InFunctionName) const
{
	return FindGraphFunctionHeaderByName(InHostPath, InFunctionName).LibraryPointer;
}

URigVMFunctionReferenceNode* URigVMController::AddFunctionReferenceNode(URigVMLibraryNode* InFunctionDefinition, const FVector2D& InNodePosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!InFunctionDefinition)
	{
		return nullptr;
	}

	if (URigVMFunctionReferenceNode* ReferenceNode = AddFunctionReferenceNodeFromDescription(InFunctionDefinition->GetFunctionHeader(), InNodePosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand))
	{
		return ReferenceNode;
	}

	return nullptr;
}

bool URigVMController::SwapFunctionReferenceByName(const FName& InFunctionReferenceNodeName, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMFunctionReferenceNode* Node = Cast<URigVMFunctionReferenceNode>(GetGraph()->FindNodeByName(InFunctionReferenceNodeName));
	if(Node == nullptr)
	{
		ReportErrorf(TEXT("Cannot find function reference node '%s'."), *InFunctionReferenceNodeName.ToString());
		return false;
	}

	return SwapFunctionReference(Node, InNewFunctionIdentifier, bSetupOrphanPins, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SwapFunctionReference(URigVMFunctionReferenceNode* InFunctionReferenceNode, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidNodeForGraph(InFunctionReferenceNode))
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if(!InNewFunctionIdentifier.IsValid())
	{
		ReportWarning(TEXT("The provided new function identifier is not valid."));
		return false;
	}

	if(InFunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer == InNewFunctionIdentifier)
	{
		ReportWarning(TEXT("The old and new function definitions are the same. No swap required."));
		return false;
	}

	FRigVMReplaceNodesAction Action;
	if(bSetupUndoRedo)
	{
		Action = FRigVMReplaceNodesAction(this, {InFunctionReferenceNode});
		Action.SetTitle(TEXT("Swap Function Reference"));
	}

	URigVMBuildData::Get()->UnregisterFunctionReference(InFunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer, InFunctionReferenceNode);
	InFunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer = InNewFunctionIdentifier;
	InFunctionReferenceNode->UpdateFunctionHeaderFromHost();
	URigVMBuildData::Get()->RegisterFunctionReference(InFunctionReferenceNode->ReferencedFunctionHeader.LibraryPointer, InFunctionReferenceNode);
	RepopulatePinsOnNode(InFunctionReferenceNode, false, bSetupOrphanPins, true);

	TArray<FRigVMExternalVariable> ExternalVariables;
	if (GetExternalVariablesDelegate.IsBound())
	{
		ExternalVariables.Append(GetExternalVariablesDelegate.Execute(GetGraph()));
	}

	Notify(ERigVMGraphNotifType::NodeReferenceChanged, InFunctionReferenceNode);
	
	if (bSetupUndoRedo)
	{
		Action.StoreNode(InFunctionReferenceNode, false);
		GetActionStack()->AddAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString FunctionRefNodeName = GetSchema()->GetSanitizedNodeName(InFunctionReferenceNode->GetName());
		const FString NewFunctionDefinitionName = GetSchema()->GetSanitizedNodeName(InNewFunctionIdentifier.GetFunctionName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("new_definition = blueprint.get_controller_by_name('%s').find_graph_function_header('%s', '%s')"),
					*GraphName,
					*InNewFunctionIdentifier.HostObject.ToString(),
					*NewFunctionDefinitionName));

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').swap_function_reference_by_name('%s', new_definition, %s)"),
					*GraphName,
					*FunctionRefNodeName,
					(bSetupOrphanPins) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

bool URigVMController::SwapAllFunctionReferences(const FRigVMGraphFunctionIdentifier& InOldFunctionIdentifier, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if(!InOldFunctionIdentifier.IsValid())
	{
		ReportWarning(TEXT("The provided old function identifier is not valid."));
		return false;
	}

	if(!InNewFunctionIdentifier.IsValid())
	{
		ReportWarning(TEXT("The provided new function identifier is not valid."));
		return false;
	}

	if(InOldFunctionIdentifier == InNewFunctionIdentifier)
	{
		ReportWarning(TEXT("The old and new function identifier are the same. No swap required."));
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot swap function reference nodes within function library graphs."));
		return false;
	}

	TArray<URigVMFunctionReferenceNode*> FunctionReferenceNodes;
	for(TObjectPtr<URigVMNode>& Node : Graph->Nodes)
	{
		if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
		{
			if(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer == InOldFunctionIdentifier)
			{
				FunctionReferenceNodes.Add(FunctionReferenceNode);
			}
		}
	}

	if(FunctionReferenceNodes.IsEmpty())
	{
		return false;
	}
	
	if (bSetupUndoRedo && FunctionReferenceNodes.Num() > 1)
	{
		OpenUndoBracket(TEXT("Swap All Function References"));
	}

	for(URigVMFunctionReferenceNode* Node : FunctionReferenceNodes)
	{
		if(!SwapFunctionReference(Node, InNewFunctionIdentifier, bSetupOrphanPins, bSetupUndoRedo, false))
		{
			if (bSetupUndoRedo && FunctionReferenceNodes.Num() > 1)
			{
				CancelUndoBracket();
			}
			return false;
		}
	}

	if (bSetupUndoRedo && FunctionReferenceNodes.Num() > 1)
	{
		CloseUndoBracket();
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString OldFunctionDefinitionName = GetSchema()->GetSanitizedNodeName(InOldFunctionIdentifier.GetFunctionName());
		const FString NewFunctionDefinitionName = GetSchema()->GetSanitizedNodeName(InNewFunctionIdentifier.GetFunctionName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("old_definition = blueprint.get_controller_by_name('%s').find_graph_function_header('%s', '%s')"),
					*GraphName,
					*InOldFunctionIdentifier.HostObject.ToString(),
					*OldFunctionDefinitionName));

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("new_definition = blueprint.get_controller_by_name('%s').find_graph_function_header('%s', '%s')"),
					*GraphName,
					*InNewFunctionIdentifier.HostObject.ToString(),
					*NewFunctionDefinitionName));

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').swap_all_function_references(old_definition, new_definition, %s)"),
					*GraphName,
					(bSetupOrphanPins) ? TEXT("True") : TEXT("False")));
	}
	
	return true;
}

URigVMFunctionReferenceNode* URigVMController::AddFunctionReferenceNodeFromDescription(const FRigVMGraphFunctionHeader& InFunctionDefinition, const FVector2D& InNodePosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add function reference nodes to function library graphs."));
		return nullptr;
	}

	// Update the function header from the host itself (in case the spawner has outdated information)
	const FRigVMGraphFunctionHeader* FunctionHeader = &InFunctionDefinition;
	if (const FRigVMGraphFunctionData* FunctionData = InFunctionDefinition.GetFunctionData())
	{
		FunctionHeader = &FunctionData->Header;
	}
	
	if(!GetSchema()->SupportsGraphFunction(this, FunctionHeader))
	{
		return nullptr;
	}

	FString NodeName = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FunctionHeader->Name.ToString() : InNodeName);
	URigVMFunctionReferenceNode* FunctionRefNode = NewObject<URigVMFunctionReferenceNode>(Graph, *NodeName);
	FunctionRefNode->Position = InNodePosition;
	FunctionRefNode->ReferencedFunctionHeader = *FunctionHeader;

	if(!AddGraphNode(FunctionRefNode, false))
	{
		return nullptr;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	
	RepopulatePinsOnNode(FunctionRefNode, false, false, false);

	Notify(ERigVMGraphNotifType::NodeAdded, FunctionRefNode);

	if (URigVMBuildData* BuildData = URigVMBuildData::Get())
	{
		BuildData->RegisterFunctionReference(FunctionRefNode->GetReferencedFunctionHeader().LibraryPointer, FunctionRefNode);
	}
	
	for (const FRigVMGraphFunctionArgument& Argument : FunctionHeader->Arguments)
	{
		if (URigVMPin* TargetPin = FunctionRefNode->FindPin(Argument.Name.ToString()))
		{
			const FString& DefaultValue = Argument.DefaultValue;
			if (!DefaultValue.IsEmpty())
			{
				FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset, true);
				SetPinDefaultValue(TargetPin, DefaultValue, true, false, false);
			}
		}
	}

	// auto expand categories as needed
	for(const FRigVMPinCategory& Category : FunctionRefNode->ReferencedFunctionHeader.Layout.Categories)
	{
		(void)SetPinCategoryExpansion(FunctionRefNode, Category.Path, Category.bExpandedByDefault, bSetupUndoRedo);
	}

	if (bSetupUndoRedo)
	{
		FRigVMImportFromTextAction Action(this, FunctionRefNode);
		Action.SetTitle(TEXT("Add function node"));
		GetActionStack()->AddAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		const FString FunctionDefinitionName = GetSchema()->GetSanitizedNodeName(FunctionHeader->Name.ToString());

		bool bLocal = false;
		if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
		{
			if (FunctionHeader->LibraryPointer.HostObject == ClientHost->GetRigVMGraphFunctionHost().GetObject())
			{
				bLocal = true;
				RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
					FString::Printf(TEXT("asset.get_controller_by_name('%s').add_function_reference_node(library.find_function('%s'), %s, '%s')"),
							*GraphName,
							*FunctionDefinitionName,
							*RigVMPythonUtils::Vector2DToPythonString(InNodePosition),
							*NodeName));
			}
		}
		
		if (!bLocal)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("asset.get_controller_by_name('%s').add_external_function_reference_node('%s', '%s', %s, '%s')"),
						*GraphName,
						*FunctionHeader->LibraryPointer.HostObject.ToString(),
						*FunctionDefinitionName,
						*RigVMPythonUtils::Vector2DToPythonString(InNodePosition), 
						*NodeName));
		}
	}

	return FunctionRefNode;
}

URigVMFunctionReferenceNode* URigVMController::AddExternalFunctionReferenceNode(const FString& InHostPath, const FName& InFunctionName, const FVector2D& InNodePosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add function reference nodes to function library graphs."));
		return nullptr;
	}

	const FRigVMGraphFunctionHeader Header = FindGraphFunctionHeaderByName(InHostPath, InFunctionName);
	if(!Header.IsValid())
	{
		return nullptr;
	}

	return AddFunctionReferenceNodeFromDescription(Header, InNodePosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetRemappedVariable(URigVMFunctionReferenceNode* InFunctionRefNode,
	const FName& InInnerVariableName, const FName& InOuterVariableName, bool bSetupUndoRedo)
{
	if (!InFunctionRefNode)
	{
		return false;
	}
	return BindPinToVariable(InFunctionRefNode->FindRootPinByName(InInnerVariableName), InOuterVariableName.ToString(), bSetupUndoRedo);
}

URigVMLibraryNode* URigVMController::AddFunctionToLibrary(const FName& InFunctionName, bool bMutable, const FVector2D& InNodePosition, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if(!GetSchema()->CanAddFunction(this, nullptr))
	{
		return nullptr;
	}

	FString FunctionName = GetSchema()->GetValidNodeName(Graph, InFunctionName.IsNone() ? FString(TEXT("Function")) : InFunctionName.ToString());
	URigVMCollapseNode* CollapseNode = NewObject<URigVMCollapseNode>(Graph, *FunctionName);
	FString ContainedGraphName = FunctionName + TEXT("_ContainedGraph");

	if (IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if (FRigVMClient* RigVMClient = ClientHost->GetRigVMClient())
		{
			CollapseNode->ContainedGraph = RigVMClient->CreateContainedGraphModel(CollapseNode, *ContainedGraphName);
		}
	}

	if (CollapseNode->ContainedGraph == nullptr)
	{
		return nullptr;
	}

	CollapseNode->Position = InNodePosition;

	if(!AddGraphNode(CollapseNode, true))
	{
		return nullptr;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	
	if (bMutable)
	{
		const UScriptStruct* ExecuteContextStruct = FRigVMExecuteContext::StaticStruct();

		if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
		{
			CollapseController->AddExposedPin_Internal(FRigVMStruct::ExecuteContextName,
				ERigVMPinDirection::IO,
				FString::Printf(TEXT("F%s"), *ExecuteContextStruct->GetName()),
				*ExecuteContextStruct->GetPathName(),
				FString(),
				false,
				/*bPrintPythonCommand=*/ false,
				/*bIsInputVariable=*/ false,
				/*bAllowExecutePinManagement=*/ true);
		}
	}

	if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
	{
		TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);

		URigVMNode* EntryNode = CollapseNode->ContainedGraph->FindNode(TEXT("Entry"));
		URigVMNode* ReturnNode = CollapseNode->ContainedGraph->FindNode(TEXT("Return"));

		if (EntryNode == nullptr)
		{
			EntryNode = NewObject<URigVMFunctionEntryNode>(CollapseNode->ContainedGraph, TEXT("Entry"));
			if(CollapseController->AddGraphNode(EntryNode, false))
			{
				TGuardValue<bool> SuspendNotifications(CollapseController->bSuspendNotifications, true);
				CollapseController->RefreshFunctionPins(EntryNode);
				CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, EntryNode);
			}
		}
		CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, EntryNode);

		EntryNode->Position = FVector2D(-250.f, 0.f);
		CollapseController->Notify(ERigVMGraphNotifType::NodePositionChanged, EntryNode);

		if (ReturnNode == nullptr)
		{
			ReturnNode = NewObject<URigVMFunctionReturnNode>(CollapseNode->ContainedGraph, TEXT("Return"));
			if(CollapseController->AddGraphNode(ReturnNode, false))
			{
				TGuardValue<bool> SuspendNotifications(CollapseController->bSuspendNotifications, true);
				CollapseController->RefreshFunctionPins(ReturnNode);
				CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, ReturnNode);
			}
		}
		CollapseController->Notify(ERigVMGraphNotifType::NodeAdded, ReturnNode);

		ReturnNode->Position = FVector2D(250.f, 0.f);
		CollapseController->Notify(ERigVMGraphNotifType::NodePositionChanged, ReturnNode);

		if (EntryNode->IsMutable() && ReturnNode->IsMutable())
		{
			CollapseController->AddLink(EntryNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), ReturnNode->FindPin(FRigVMStruct::ExecuteContextName.ToString()), false);
		}
	}

	if (URigVMFunctionLibrary* Library = Cast<URigVMFunctionLibrary>(Graph))
	{
		if (bSetupUndoRedo)
		{
			Library->Modify();
		}

		FRigVMVariant Variant;
		if (FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore())
		{
			if (FRigVMGraphFunctionData* FunctionData = FunctionStore->FindFunctionByName(CollapseNode->GetFName()))
			{
				Variant = FunctionData->Header.Variant;
			}
		}
		
		if (!Variant.Guid.IsValid())
		{
			Variant.Guid = FRigVMVariant::GenerateGUID();
		}
		
		Library->FunctionToVariant.Add(CollapseNode->GetFName(), Variant);
	}

	if (bSetupUndoRedo)
	{
		FRigVMImportFromTextAction Action(this, CollapseNode);
		Action.SetTitle(TEXT("Add function to library"));
		GetActionStack()->AddAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		//AddFunctionToLibrary(const FName& InFunctionName, bool bMutable, const FVector2D& InNodePosition, bool bSetupUndoRedo, bool bPrintPythonCommand)
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("library_controller.add_function_to_library('%s', %s, %s)"),
				*GetSchema()->GetSanitizedNodeName(InFunctionName.ToString()),
				(bMutable) ? TEXT("True") : TEXT("False"),
				*RigVMPythonUtils::Vector2DToPythonString(InNodePosition)));
	}

	return CollapseNode;
}

bool URigVMController::RemoveFunctionFromLibrary(const FName& InFunctionName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if(URigVMNode* Node = Graph->FindNodeByName(InFunctionName))
	{
		if(!GetSchema()->CanRemoveFunction(this, Node))
		{
			return false;
		}
		return RemoveNodeByName(InFunctionName, bSetupUndoRedo, bPrintPythonCommand);
	}

	ReportErrorf(TEXT("Cannot find node '%s'."), *InFunctionName.ToString());
	return false;
}

bool URigVMController::RenameFunction(const FName& InOldFunctionName, const FName& InNewFunctionName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);


	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Can only remove function definitions from function library graphs."));
		return false;
	}

	URigVMNode* Node = Graph->FindNode(InOldFunctionName.ToString());
	if (!Node)
	{
		ReportErrorf(TEXT("Could not find function called '%s'."), *InOldFunctionName.ToString());
		return false;
	}

	return RenameNode(Node, InNewFunctionName, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::MarkFunctionAsPublic(const FName& InFunctionName, bool bInIsPublic, bool bSetupUndoRedo,	bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);


	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Can only change function definitions from function library graphs."));
		return false;
	}

	URigVMNode* Node = Graph->FindNode(InFunctionName.ToString());
	if (!Node)
	{
		ReportErrorf(TEXT("Could not find function called '%s'."), *InFunctionName.ToString());
		return false;
	}

	if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph))
	{
		bool bOldIsPublic = FunctionLibrary->PublicFunctionNames.Contains(InFunctionName); 
		if ((bInIsPublic && bOldIsPublic) || (!bInIsPublic && !bOldIsPublic))
		{
			return true;
		}

		if (bSetupUndoRedo)
		{
			FRigVMBaseAction BaseAction(this);
			BaseAction.SetTitle(FString::Printf(TEXT("Mark function %s as %s"), *InFunctionName.ToString(), (bInIsPublic) ? TEXT("Public") : TEXT("Private")));
			GetActionStack()->BeginAction(BaseAction);
			GetActionStack()->AddAction(FRigVMMarkFunctionPublicAction(this, InFunctionName, bInIsPublic));
			GetActionStack()->EndAction(BaseAction);
		}

		if (bInIsPublic)
		{
			FunctionLibrary->PublicFunctionNames.Add(InFunctionName);
		}
		else
		{
			FunctionLibrary->PublicFunctionNames.Remove(InFunctionName);
		}
	}

	Notify(ERigVMGraphNotifType::FunctionAccessChanged, Node);

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("library_controller.mark_function_as_public('%s', %s)"),
				*GetSchema()->GetSanitizedNodeName(InFunctionName.ToString()),
				(bInIsPublic) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

bool URigVMController::IsFunctionPublic(const FName& InFunctionName)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);


	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Can only check function definitions from function library graphs."));
		return false;
	}

	URigVMNode* Node = Graph->FindNode(InFunctionName.ToString());
	if (!Node)
	{
		ReportErrorf(TEXT("Could not find function called '%s'."), *InFunctionName.ToString());
		return false;
	}

	if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph))
	{
		return FunctionLibrary->PublicFunctionNames.Contains(InFunctionName);
	}

	return false;
}

bool URigVMController::SetFunctionIsPure(const FName& InFunctionName, bool bIsPure, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Can only change function purity from function library graphs."));
		return false;
	}

	URigVMNode* Node = Graph->FindNode(InFunctionName.ToString());
	if (!Node)
	{
		ReportErrorf(TEXT("Could not find function called '%s'."), *InFunctionName.ToString());
		return false;
	}

	URigVMLibraryNode* FunctionNode = Cast<URigVMLibraryNode>(Node);
	if (FunctionNode == nullptr)
	{
		ReportErrorf(TEXT("Node '%s' is not a function."), *InFunctionName.ToString());
		return false;
	}

	// Check current state
	const bool bCurrentlyHasExecutePin = FunctionNode->FindExecutePin() != nullptr;
	const bool bWantExecutePin = !bIsPure;

	// Already in desired state?
	if (bCurrentlyHasExecutePin == bWantExecutePin)
	{
		return true;
	}

	// Get the controller for the function's contained graph to modify exposed pins
	URigVMController* FunctionController = GetControllerForGraph(FunctionNode->GetContainedGraph());
	if (FunctionController == nullptr)
	{
		ReportErrorf(TEXT("Could not get controller for function '%s'."), *InFunctionName.ToString());
		return false;
	}

	FRigVMControllerCompileBracketScope CompileScope(FunctionController);

	if (bIsPure)
	{
		// Group the execute-pin removal and any subsequent link breaks into a single
		// undo action so that a single "Undo" reverses the entire operation.
		if (bSetupUndoRedo)
		{
			FunctionController->OpenUndoBracket(TEXT("Set Function Pure"));
		}

		// Remove the execute pin to make the function pure
		if (!FunctionController->RemoveExposedPin_Internal(FRigVMStruct::ExecuteContextName, bSetupUndoRedo, bPrintPythonCommand, /*bAllowExecutePinManagement=*/ true))
		{
			if (bSetupUndoRedo)
			{
				FunctionController->CancelUndoBracket();
			}
			ReportError(TEXT("Failed to remove execute pin from function."));
			return false;
		}

		// After removing the execute pin, break any links to InputVariable pins that are
		// now illegal. A non-local variable getter or entry node pin feeding an InputVariable
		// pin would let the pure function mutate external state — those links must be broken.
		// Local variable getters remain valid (function-scoped, not external state).
		{
			URigVMGraph* ContainedGraph = FunctionNode->GetContainedGraph();
			const TArray<URigVMLink*>& Links = ContainedGraph->GetLinks();

			TArray<TPair<FString, FString>> LinkPathsToBreak;
			for (URigVMLink* Link : Links)
			{
				URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();
				if (SourcePin == nullptr || TargetPin == nullptr)
				{
					continue;
				}

				if (!TargetPin->IsDefinedAsInputVariable())
				{
					continue;
				}

				// Reject if the source is a non-local variable getter
				if (const URigVMVariableNode* VarNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
				{
					if (VarNode->IsGetter() && !VarNode->IsLocalVariable())
					{
						LinkPathsToBreak.Emplace(SourcePin->GetPinPath(), TargetPin->GetPinPath());
						continue;
					}
				}

				// Reject if the source is an entry node pin
				if (SourcePin->GetNode()->IsA<URigVMFunctionEntryNode>())
				{
					LinkPathsToBreak.Emplace(SourcePin->GetPinPath(), TargetPin->GetPinPath());
				}
			}

			for (const TPair<FString, FString>& PathPair : LinkPathsToBreak)
			{
				// ignore the return - non broken links can be discarded
				(void)FunctionController->BreakLink(PathPair.Key, PathPair.Value, bSetupUndoRedo);
			}
		}

		if (bSetupUndoRedo)
		{
			FunctionController->CloseUndoBracket();
		}
	}
	else
	{
		// Add the execute pin to make the function mutable
		const UScriptStruct* ExecuteContextStruct = FRigVMExecuteContext::StaticStruct();
		const FName PinName = FunctionController->AddExposedPin_Internal(
			FRigVMStruct::ExecuteContextName,
			ERigVMPinDirection::IO,
			FString::Printf(TEXT("F%s"), *ExecuteContextStruct->GetName()),
			*ExecuteContextStruct->GetPathName(),
			FString(),
			bSetupUndoRedo,
			bPrintPythonCommand,
			/*bIsInputVariable=*/ false,
			/*bAllowExecutePinManagement=*/ true);

		if (PinName.IsNone())
		{
			ReportError(TEXT("Failed to add execute pin to function."));
			return false;
		}
	}

	if (bPrintPythonCommand)
	{
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("library_controller.set_function_is_pure('%s', %s)"),
				*GetSchema()->GetSanitizedNodeName(InFunctionName.ToString()),
				(bIsPure) ? TEXT("True") : TEXT("False")));
	}

	return true;
}

URigVMLibraryNode* URigVMController::CreateFunctionVariant(const FName& InFunctionName, const FName& InVariantName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	URigVMLibraryNode* Result = nullptr;
	if (!IsValidGraph())
	{
		return Result;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return Result;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Can only add function variant in library graphs."));
		return Result;
	}

	IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>();
	if (!ClientHost)
	{
		ReportErrorf(TEXT("Cannot find client host."));
		return Result;
	}

	TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = ClientHost->GetRigVMGraphFunctionHost();
	if (!FunctionHost)
	{
		ReportErrorf(TEXT("Cannot find function host."));
		return Result;
	}

	bool bIsPublic;
	FRigVMGraphFunctionData* OriginalFunction = FunctionHost->GetRigVMGraphFunctionStore()->FindFunctionByName(InFunctionName, &bIsPublic);
	if (!OriginalFunction)
	{
		ReportErrorf(TEXT("Cannot find function %s."), *InFunctionName.ToString());
		return Result;
	}

	URigVMLibraryNode* FunctionToClone = Cast<URigVMLibraryNode>(Graph->FindNodeByName(InFunctionName));
	if (!FunctionToClone)
	{
		ReportErrorf(TEXT("Cannot find function node %s."), *InFunctionName.ToString());
		return Result;
	}

	const FName VariantName = *GetSchema()->GetValidNodeName(Graph, InVariantName.IsNone() ? InFunctionName.ToString() : InVariantName.ToString());

	FRigVMBaseAction BaseAction(this);
	if(bSetupUndoRedo)
	{
		BaseAction.SetTitle(FString::Printf(TEXT("Create function variant %s with name %s"), *InFunctionName.ToString(), *InVariantName.ToString()));
		GetActionStack()->BeginAction(BaseAction);
		GetActionStack()->AddAction(FRigVMCreateFunctionVariantAction(this, InFunctionName, VariantName));
	}

	ClientHost->GetRigVMClient()->UpdateGraphFunctionSerializedGraph(FunctionToClone);

	TArray<FName> NodeNames;
	if(OriginalFunction->CollapseNodeArchive.IsEmpty())
	{
		NodeNames = ImportNodesFromText(OriginalFunction->SerializedCollapsedNode_DEPRECATED, false);
	}
	else
	{
		if(const URigVMLibraryNode* NewCollapseNode = ImportFunctionFromArchive(OriginalFunction->CollapseNodeArchive, VariantName))
		{
			NodeNames.Add(NewCollapseNode->GetFName());
		}
	}

	if (!NodeNames.IsEmpty())
	{
		if (VariantName != NodeNames[0])
		{
			RenameFunction(NodeNames[0], VariantName, false);
		}
		FRigVMVariant NewVariant(OriginalFunction->Header.Variant);
		if (FRigVMGraphFunctionData* NewFunction = FunctionHost->GetRigVMGraphFunctionStore()->FindFunctionByName(VariantName))
		{
			const FName NewName = NewFunction->Header.Name;
			const FRigVMGraphFunctionIdentifier Identifier = NewFunction->Header.LibraryPointer;
			NewFunction->Header = OriginalFunction->Header;
			NewFunction->Header.Name = NewName;
			NewFunction->Header.LibraryPointer = Identifier;
			
			if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph))
			{
				FunctionLibrary->FunctionToVariant.Add(NewName, NewVariant);
			}

			MarkFunctionAsPublic(NewName, bIsPublic, false);
		}

		Result = Cast<URigVMLibraryNode>(Graph->FindNodeByName(VariantName));
	}
	else if(bSetupUndoRedo)
	{
		GetActionStack()->CancelAction(BaseAction);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(BaseAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').create_function_variant('%s', '%s')"),
				*GraphName,
				*InFunctionName.ToString(),
				*InVariantName.ToString()));
	}

	return Result;
}

bool URigVMController::AddDefaultTagToFunctionVariant(const FName& InFunctionName, const FName& InTagName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	URigVMProjectSettings* Settings = GetMutableDefault<URigVMProjectSettings>(URigVMProjectSettings::StaticClass());
	if (!Settings)
	{
		return false;
	}

	const FRigVMTag* Tag = Settings->FindTag(InTagName);
	if (!Tag)
	{
		ReportErrorf(TEXT("Could not find default tag with name %s."), *InTagName.ToString());
		return false;
	}

	if (AddTagToFunctionVariant(InFunctionName, *Tag, bSetupUndoRedo, false))
	{
		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_default_tag_to_function_variant('%s', %s)"),
				*GraphName,
				*InFunctionName.ToString(),
				*InTagName.ToString()));
		}
		return true;
	}
	return false;
}

bool URigVMController::AddTagToFunctionVariant(const FName& InFunctionName, const FRigVMTag& InTag, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph);
	if (!FunctionLibrary)
	{
		ReportError(TEXT("Can only add tag to variant in library graphs."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionLibrary->FindFunction(InFunctionName));
	if (!LibraryNode)
	{
		ReportErrorf(TEXT("Could not find library node for function %s."), *InFunctionName.ToString());
		return false;
	}

	FRigVMVariant* Variant = FunctionLibrary->GetFunctionVariant(InFunctionName);
	if (!Variant)
	{
		ReportErrorf(TEXT("Could not find function variant for function %s."), *InFunctionName.ToString());
		return false;
	}

	if(Variant->Tags.ContainsByPredicate([InTag](const FRigVMTag& Tag) -> bool
	{
		return Tag.Name == InTag.Name; 
	}))
	{
		ReportErrorf(TEXT("Cannot add tag %s to function %s - tag already applied."), *InTag.Name.ToString(), *InFunctionName.ToString());
		return false;
	}

	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add tag to function variant")));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMAddFunctionVariantTagAction(this, InFunctionName, InTag));
	}

	Variant->Tags.Add(InTag);

	Notify(ERigVMGraphNotifType::VariantTagsChanged, LibraryNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
	
		const FString TagPythonString = FString::Printf(TEXT("unreal.RigVMTag(name=\"%s\", label=\"%s\", tool_tip=\"%s\", color=unreal.LinearColor(r=%f, g=%f, b=%f, a=%f), show_in_user_interface=%s, marks_subject_as_invalid=%s)"),
			*InTag.Name.ToString(),
			*InTag.Label,
			*InTag.ToolTip.ToString(),
			InTag.Color.R, InTag.Color.G, InTag.Color.B, InTag.Color.A,
			(InTag.bShowInUserInterface) ? TEXT("True") : TEXT("False"),
			(InTag.bMarksSubjectAsInvalid) ? TEXT("True") : TEXT("False"));
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_tag_to_function_variant('%s', %s)"),
				*GraphName,
				*InFunctionName.ToString(),
				*TagPythonString));
	}

	return true;
}

bool URigVMController::RemoveTagFromFunctionVariant(const FName& InFunctionName, const FName& InTagName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph);
	if (!FunctionLibrary)
	{
		ReportError(TEXT("Can only add tag to variant in library graphs."));
		return false;
	}

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(FunctionLibrary->FindFunction(InFunctionName));
	if (!LibraryNode)
	{
		ReportErrorf(TEXT("Could not find library node for function %s."), *InFunctionName.ToString());
		return false;
	}

	FRigVMVariant* Variant = FunctionLibrary->GetFunctionVariant(InFunctionName);
	if (!Variant)
	{
		ReportErrorf(TEXT("Could not find function variant for function %s."), *InFunctionName.ToString());
		return false;
	}

	const int32 Index = Variant->Tags.IndexOfByPredicate([InTagName](const FRigVMTag& InTag)
	{
		return InTag.Name == InTagName;
	});
	if (Index == INDEX_NONE)
	{
		ReportErrorf(TEXT("Could not find tag %s for function %s."), *InTagName.ToString(), *InFunctionName.ToString());
		return false;
	}
	
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Remove tag from function variant")));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMRemoveFunctionVariantTagAction(this, InFunctionName, InTagName));
	}

	Variant->Tags.RemoveAll([InTagName](const FRigVMTag& Tag) -> bool
	{
		return Tag.Name == InTagName;
	});

	Notify(ERigVMGraphNotifType::VariantTagsChanged, LibraryNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
	
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_tag_from_function_variant('%s', '%s')"),
				*GraphName,
				*InFunctionName.ToString(),
				*InTagName.ToString()));
	}

	return true;
}

TArray<FRigVMVariantRef> URigVMController::FindVariantsOfFunction(const FName& InFunctionName)
{
	TArray<FRigVMVariantRef> Result;
	
    FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore();
    if (!FunctionStore)
    {
    	ReportErrorf(TEXT("Cannot find function store."));
    	return Result;
    }

    FRigVMGraphFunctionData* FunctionData = FunctionStore->FindFunctionByName(InFunctionName);
    if (!FunctionData)
    {
    	ReportErrorf(TEXT("Cannot find function %s."), *InFunctionName.ToString());
    	return Result;
    }

	return FunctionData->Header.LibraryPointer.GetVariants(true);
}

bool URigVMController::SplitFunctionVariant(const FName& InFunctionName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot split function variants on normal (non-function-library) graphs."));
		return false;
	}

	URigVMFunctionLibrary* Library = CastChecked<URigVMFunctionLibrary>(Graph);
	const URigVMLibraryNode* LibraryNode = Library->FindFunction(InFunctionName);
	if(LibraryNode == nullptr)
	{
		ReportError(TEXT("Function '%s' cannot be found"));
		return false;
	}

	FRigVMVariant Variant;
	if (FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore())
	{
		if (FRigVMGraphFunctionData* FunctionData = FunctionStore->FindFunctionByName(InFunctionName))
		{
			Variant = FunctionData->Header.Variant;
		}
	}

	// prefer the deterministic path based guid - and only fall back on
	// random guids if necessary
	const FGuid PreviousGuid = Variant.Guid; 
	Variant.Guid = FRigVMVariant::GenerateGUID(LibraryNode->GetPathName());
	if(Variant.Guid == PreviousGuid)
	{
		Variant.Guid = FRigVMVariant::GenerateGUID();
	}

	FScopedTransaction Transaction(NSLOCTEXT("RigVMController", "SplitAssetVariant", "Split Asset Variant"));
	if(bSetupUndoRedo)
	{
		Library->Modify();
	}
	Library->FunctionToVariant.FindOrAdd(InFunctionName) = Variant;

	Notify(ERigVMGraphNotifType::FunctionVariantGuidChanged, Library->FindFunction(InFunctionName));

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').split_function_variant('%s')"),
				*GraphName,
				*InFunctionName.ToString()));
	}

	return true;
}

bool URigVMController::JoinFunctionVariant(const FName& InFunctionName, const FGuid& InGuid, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot join function variants on normal (non-function-library) graphs."));
		return false;
	}

	URigVMFunctionLibrary* Library = CastChecked<URigVMFunctionLibrary>(Graph);
	if(Library->FindFunction(InFunctionName) == nullptr)
	{
		ReportError(TEXT("Function '%s' cannot be found"));
		return false;
	}

	FRigVMVariant Variant;
	if (FRigVMGraphFunctionStore* FunctionStore = GetGraphFunctionStore())
	{
		if (FRigVMGraphFunctionData* FunctionData = FunctionStore->FindFunctionByName(InFunctionName))
		{
			Variant = FunctionData->Header.Variant;
		}
	}

	if(Variant.Guid == InGuid)
	{
		return false;
	}
	Variant.Guid = InGuid;

	FScopedTransaction Transaction(NSLOCTEXT("RigVMController", "SplitAssetVariant", "Split Asset Variant"));
	if(bSetupUndoRedo)
	{
		Library->Modify();
	}
	Library->FunctionToVariant.FindOrAdd(InFunctionName) = Variant;

	Notify(ERigVMGraphNotifType::FunctionVariantGuidChanged, Library->FindFunction(InFunctionName));

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("guid = unreal.Guid()")));
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("guid.import_text(\"%s\")"), *Variant.Guid.ToString()));
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').join_function_variant('%s', guid)"),
				*GraphName,
				*InFunctionName.ToString()));
	}

	return true;
}

FRigVMGraphVariableDescription URigVMController::AddLocalVariable(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	FRigVMGraphVariableDescription NewVariable;
	if (!IsValidGraph())
	{
		return NewVariable;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NewVariable;
	}
	
	URigVMGraph* Graph = GetGraph();
	check(Graph);

	// Check this is the main graph of a function
	{
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			if (!LibraryNode->GetOuter()->IsA<URigVMFunctionLibrary>())
			{
				return NewVariable;
			}
		}
		else
		{
			return NewVariable;
		}
	}

	FName VariableName = URigVMSchema::GetUniqueName(InVariableName, [Graph](const FName& InName) {
		for (FRigVMGraphVariableDescription LocalVariable : Graph->GetLocalVariables(true))
		{
			if (LocalVariable.Name == InName)
			{
				return false;
			}
		}
		return true;
	}, false, true);

	NewVariable.Guid = FGuid::NewGuid();
	NewVariable.Name = VariableName;
	NewVariable.CPPType = InCPPType;
	NewVariable.CPPTypeObject = InCPPTypeObject;
	if (InCPPTypeObject)
	{
		NewVariable.CPPTypeObjectPath = *InCPPTypeObject->GetPathName();
	}
	NewVariable.DefaultValue = InDefaultValue;

	Graph->LocalVariables.Add(NewVariable);

	FRigVMControllerCompileBracketScope CompileScope(this);
	
	for (URigVMNode* Node : Graph->GetNodes())
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableName == VariableNode->GetVariableName())
			{
				RefreshVariableNode(VariableNode->GetFName(), NewVariable.Guid, VariableName, InCPPType, InCPPTypeObject, bSetupUndoRedo, false);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		FRigVMAddLocalVariableAction Action(this, NewVariable);
		Action.SetTitle(FString::Printf(TEXT("Add Local Variable %s"), *InVariableName.ToString()));
		GetActionStack()->AddAction(Action);
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);

		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			Notify(ERigVMGraphNotifType::LocalVariableAdded, LibraryNode);
		}
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').add_local_variable_from_object_path('%s', '%s', '%s', '%s')"),
				*GraphName,
				*NewVariable.Name.ToString(),
				*NewVariable.CPPType,
				(NewVariable.CPPTypeObject) ? *NewVariable.CPPTypeObject->GetPathName() : *FString(),
				*NewVariable.DefaultValue));
	}

	return NewVariable;
}

FRigVMGraphVariableDescription URigVMController::AddLocalVariableFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo)
{
	FRigVMGraphVariableDescription Description;
	if (!IsValidGraph())
	{
		return Description;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return Description;
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return Description;
		}
	}

	return AddLocalVariable(InVariableName, InCPPType, CPPTypeObject, InDefaultValue, bSetupUndoRedo);
}

bool URigVMController::RemoveLocalVariable(const FName& InVariableName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;
	int32 FoundIndex = INDEX_NONE;
	for (int32 Index = 0; Index < LocalVariables.Num(); ++Index)
	{
		if (LocalVariables[Index].Name == InVariableName)
		{
			FoundIndex = Index;
			break;
		}
	}

	if (FoundIndex != INDEX_NONE)
	{
		FRigVMControllerCompileBracketScope CompileScope(this);
		FRigVMBaseAction BaseAction(this);
		if (bSetupUndoRedo)
		{
			BaseAction.SetTitle(FString::Printf(TEXT("Remove Local Variable %s"), *InVariableName.ToString()));
			GetActionStack()->BeginAction(BaseAction);			
		}	
		
		const FString VarNameStr = InVariableName.ToString();

		bool bSwitchToMemberVariable = false;
		FRigVMExternalVariable ExternalVariableToSwitch;
		{
			TArray<FRigVMExternalVariable> ExternalVariables;
			if (GetExternalVariablesDelegate.IsBound())
			{
				ExternalVariables.Append(GetExternalVariablesDelegate.Execute(GetGraph()));
			}

			for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
			{
				if (ExternalVariable.GetName() == InVariableName)
				{
					bSwitchToMemberVariable = true;
					ExternalVariableToSwitch = ExternalVariable;
					break;
				}	
			}
		}

		if (!bSwitchToMemberVariable)
		{
			TArray<URigVMNode*> Nodes = Graph->GetNodes();
			for (URigVMNode* Node : Nodes)
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
				{
					if (URigVMPin* VariablePin = VariableNode->FindPin(URigVMVariableNode::VariableName))
					{
						if (VariablePin->GetDefaultValue() == VarNameStr)
						{
							RemoveNode(Node, bSetupUndoRedo, true);
							continue;
						}
					}
				}
			}
		}
		else
		{
			TArray<URigVMNode*> Nodes = Graph->GetNodes();
			for (URigVMNode* Node : Nodes)
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
				{
					if (URigVMPin* VariablePin = VariableNode->FindPin(URigVMVariableNode::VariableName))
					{
						if (VariablePin->GetDefaultValue() == VarNameStr)
						{
							RefreshVariableNode(VariableNode->GetFName(),  ExternalVariableToSwitch.GetGuid(), ExternalVariableToSwitch.GetName(), ExternalVariableToSwitch.GetExtendedCPPType().ToString(), ExternalVariableToSwitch.GetCPPTypeObject(), bSetupUndoRedo, false);
							continue;
						}
					}
				}

				TArray<URigVMPin*> AllPins = Node->GetAllPinsRecursively();
				for (URigVMPin* Pin : AllPins)
				{
					if (Pin->GetBoundVariableName() == InVariableName.ToString())
					{
						if (Pin->GetCPPType() != ExternalVariableToSwitch.GetExtendedCPPType().ToString() || Pin->GetCPPTypeObject() == ExternalVariableToSwitch.GetCPPTypeObject())
						{
							UnbindPinFromVariable(Pin, bSetupUndoRedo);
						}
					}
				}
			}		
		}

		if (!bSuspendNotifications)
		{
			(void)MarkPackageDirty(Graph);
			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
			{
				Notify(ERigVMGraphNotifType::LocalVariableRemoved, LibraryNode);
			}
		}

		if (bSetupUndoRedo)
		{
			GetActionStack()->AddAction(FRigVMRemoveLocalVariableAction(this, LocalVariables[FoundIndex]));
		}
		LocalVariables.RemoveAt(FoundIndex);

		if (bSetupUndoRedo)
		{
			GetActionStack()->EndAction(BaseAction);
		}

		if (bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_local_variable('%s')"),
					*GraphName,
					*GetSchema()->GetSanitizedVariableName(InVariableName.ToString())));
		}
		return true;
	}

	return false;
}

bool URigVMController::RenameLocalVariable(const FName& InVariableName, const FName& InNewVariableName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;
	int32 FoundIndex = INDEX_NONE;
	for (int32 Index = 0; Index < LocalVariables.Num(); ++Index)
	{
		if (LocalVariables[Index].Name == InVariableName)
		{
			FoundIndex = Index;
			break;
		}
	}

	if (FoundIndex == INDEX_NONE)
	{
		return false;
	}

	for (int32 Index = 0; Index < LocalVariables.Num(); ++Index)
	{
		if (LocalVariables[Index].Name == InNewVariableName)
		{
			return false;
		}
	}
	
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		FRigVMBaseAction BaseAction(this);
		BaseAction.SetTitle(FString::Printf(TEXT("Rename Local Variable %s to %s"), *InVariableName.ToString(), *InNewVariableName.ToString()));

		GetActionStack()->BeginAction(BaseAction);
		GetActionStack()->AddAction(FRigVMRenameLocalVariableAction(this, LocalVariables[FoundIndex].Name, InNewVariableName));
		GetActionStack()->EndAction(BaseAction);
	}
	
	LocalVariables[FoundIndex].Name = InNewVariableName;

	TArray<URigVMNode*> RenamedNodes;
	for (URigVMNode* Node : Graph->Nodes)
	{
		if(URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (VariableNode->GetVariableName() == InVariableName)
			{
				VariableNode->FindPin(URigVMVariableNode::VariableName)->DefaultValue = InNewVariableName.ToString();
				RenamedNodes.Add(Node);
			}
		}
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			Notify(ERigVMGraphNotifType::LocalVariableRenamed, LibraryNode);
		}
		for (URigVMNode* RenamedNode : RenamedNodes)
		{
			Notify(ERigVMGraphNotifType::VariableRenamed, RenamedNode);
		}
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').rename_local_variable('%s', '%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedVariableName(InVariableName.ToString()),
				*GetSchema()->GetSanitizedVariableName(InNewVariableName.ToString())));
	}

	return true;
}

bool URigVMController::SetLocalVariableType(const FName& InVariableName, const FString& InCPPType,
                                            UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;
	int32 FoundIndex = INDEX_NONE;
	for (int32 Index = 0; Index < LocalVariables.Num(); ++Index)
	{
		if (LocalVariables[Index].Name == InVariableName)
		{
			FoundIndex = Index;
			break;
		}
	}

	if (FoundIndex == INDEX_NONE)
	{
		return false;
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			Notify(ERigVMGraphNotifType::LocalVariableTypeChanged, LibraryNode);
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction BaseAction(this);
	if (bSetupUndoRedo)
	{
		BaseAction.SetTitle(FString::Printf(TEXT("Change Local Variable type %s to %s"), *InVariableName.ToString(), *InCPPType));
		GetActionStack()->BeginAction(BaseAction);

		GetActionStack()->AddAction(FRigVMChangeLocalVariableTypeAction(this, LocalVariables[FoundIndex], InCPPType, InCPPTypeObject));
	}	
	
	LocalVariables[FoundIndex].CPPType = InCPPType;
	LocalVariables[FoundIndex].CPPTypeObject = InCPPTypeObject;
	if (InCPPTypeObject)
	{
		LocalVariables[FoundIndex].CPPTypeObjectPath = *InCPPTypeObject->GetPathName();
	}
	else
	{
		LocalVariables[FoundIndex].CPPTypeObjectPath = NAME_None;
	}

	// Set default value
	if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InCPPTypeObject))
	{
		FString DefaultValue;
		CreateDefaultValueForStructIfRequired(ScriptStruct, DefaultValue);
		LocalVariables[FoundIndex].DefaultValue = DefaultValue;
	}
	else
	{
		LocalVariables[FoundIndex].DefaultValue = FString();
	}

	// Change pin types on variable nodes
	TArray<URigVMNode*> Nodes = Graph->GetNodes();
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (URigVMPin* VariablePin = VariableNode->FindPin(URigVMVariableNode::VariableName))
			{
				if (VariablePin->GetDefaultValue() == InVariableName.ToString())
				{
					RefreshVariableNode(Node->GetFName(), LocalVariables[FoundIndex].Guid, InVariableName, InCPPType, InCPPTypeObject, bSetupUndoRedo, false);
					continue;
				}
			}
		}

		const TArray<URigVMPin*> AllPins = Node->GetAllPinsRecursively();
		for (URigVMPin* Pin : AllPins)
		{
			if (Pin->GetBoundVariableName() == InVariableName.ToString())
			{
				UnbindPinFromVariable(Pin, bSetupUndoRedo);
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(BaseAction);
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		//bool URigVMController::SetLocalVariableType(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bPrintPythonCommand)
		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_local_variable_type_from_object_path('%s', '%s', '%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedVariableName(InVariableName.ToString()),
				*InCPPType,
				(InCPPTypeObject) ? *InCPPTypeObject->GetPathName() : *FString()));
	}
	
	return true;
}

bool URigVMController::SetLocalVariableTypeFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return false;
		}
	}

	return SetLocalVariableType(InVariableName, InCPPType, CPPTypeObject, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetLocalVariableDefaultValue(const FName& InVariableName, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;
	int32 FoundIndex = INDEX_NONE;
	for (int32 Index = 0; Index < LocalVariables.Num(); ++Index)
	{
		if (LocalVariables[Index].Name == InVariableName)
		{
			FoundIndex = Index;
			break;
		}
	}

	if (FoundIndex == INDEX_NONE)
	{
		return false;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		FRigVMChangeLocalVariableDefaultValueAction Action(this, LocalVariables[FoundIndex], InDefaultValue);
		Action.SetTitle(FString::Printf(TEXT("Change Local Variable %s default value"), *InVariableName.ToString()));
		GetActionStack()->AddAction(Action);
	}

	FRigVMGraphVariableDescription& VariableDescription = LocalVariables[FoundIndex];
	VariableDescription.DefaultValue = InDefaultValue;
	
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);

		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			Notify(ERigVMGraphNotifType::LocalVariableDefaultValueChanged, LibraryNode);
		}
	}

	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_local_variable_default_value('%s', '%s')"),
				*GraphName,
				*GetSchema()->GetSanitizedVariableName(InVariableName.ToString()),
				*InDefaultValue));
	}
	
	return true;
}

bool URigVMController::SetLocalVariableIndex(const FName& InVariableName, int32 InNewIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	// Look up variable by name
	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;
	const FRigVMGraphVariableDescription* Variable = LocalVariables.FindByPredicate(
		[InVariableName](const FRigVMGraphVariableDescription& Var) {
			return Var.Name == InVariableName;
		});

	if (!Variable)
	{
		ReportErrorf(TEXT("Cannot find local variable '%s'."), *InVariableName.ToString());
		return false;
	}

	// Call GUID-based version
	return SetLocalVariableIndex(Variable->Guid, InNewIndex, bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::SetLocalVariableIndex(const FGuid& InVariableGuid, int32 InNewIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMGraphVariableDescription>& LocalVariables = Graph->LocalVariables;

	// Validate new index is within bounds
	if (InNewIndex < 0 || InNewIndex >= LocalVariables.Num())
	{
		ReportErrorf(TEXT("Invalid new variable index '%d'. Valid range is 0 to %d."), InNewIndex, LocalVariables.Num() - 1);
		return false;
	}

	// Find variable by GUID and get current index
	int32 OldIndex = INDEX_NONE;
	for (int32 i = 0; i < LocalVariables.Num(); ++i)
	{
		if (LocalVariables[i].Guid == InVariableGuid)
		{
			OldIndex = i;
			break;
		}
	}

	if (OldIndex == INDEX_NONE)
	{
		ReportError(TEXT("Cannot find local variable by GUID."));
		return false;
	}

	// Nothing to do if already at target index
	if (OldIndex == InNewIndex)
	{
		return true;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);

	// Setup undo/redo action
	if (bSetupUndoRedo)
	{
		FRigVMSetLocalVariableIndexAction Action(this, InVariableGuid, OldIndex, InNewIndex);
		Action.SetTitle(FString::Printf(TEXT("Reorder Local Variable '%s'"), *LocalVariables[OldIndex].Name.ToString()));
		GetActionStack()->AddAction(Action);
	}

	// Perform the reorder: remove from old position, insert at new position
	FRigVMGraphVariableDescription Variable = LocalVariables[OldIndex];
	LocalVariables.RemoveAt(OldIndex);
	LocalVariables.Insert(Variable, InNewIndex);

	// Notify listeners
	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);

		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			Notify(ERigVMGraphNotifType::LocalVariableIndexChanged, LibraryNode);
		}
	}

	// Print Python command
	if (bPrintPythonCommand)
	{
		const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

		RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
			FString::Printf(TEXT("asset.get_controller_by_name('%s').set_local_variable_index('%s', %d)"),
				*GraphName,
				*GetSchema()->GetSanitizedVariableName(Variable.Name.ToString()),
				InNewIndex));
	}

	return true;
}

URigVMUserWorkflowOptions* URigVMController::MakeOptionsForWorkflow(UObject* InSubject, const FRigVMUserWorkflow& InWorkflow)
{
	URigVMUserWorkflowOptions* Options = nullptr;

	UClass* Class = InWorkflow.GetOptionsClass();
	if(Class == nullptr)
	{
		return Options;
	}

	if(!Class->IsChildOf(URigVMUserWorkflowOptions::StaticClass()))
	{
		return Options;
	}
	
	Options = NewObject<URigVMUserWorkflowOptions>(GetTransientPackage(), Class, NAME_None, RF_Transient);
	Options->Subject = InSubject;
	Options->Workflow = InWorkflow;

	TWeakObjectPtr<URigVMController> WeakThis = this;
	Options->ReportDelegate = FRigVMReportDelegate::CreateLambda([WeakThis](
		EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
		{
			if(URigVMController* StrongThis = WeakThis.Get())
			{
				if(InSeverity == EMessageSeverity::Error)
				{
					StrongThis->ReportAndNotifyError(InMessage);
				}
				else if(InSeverity == EMessageSeverity::Warning ||
					InSeverity == EMessageSeverity::PerformanceWarning)
				{
					StrongThis->ReportAndNotifyWarning(InMessage);
				}
				else 
				{
					StrongThis->ReportInfo(InMessage);
				}
			}
		}
	);

	if(ConfigureWorkflowOptionsDelegate.IsBound())
	{
		ConfigureWorkflowOptionsDelegate.Execute(Options);
	}
	
	return Options;
}

bool URigVMController::PerformUserWorkflow(const FRigVMUserWorkflow& InWorkflow,
	const URigVMUserWorkflowOptions* InOptions, bool bSetupUndoRedo)
{
	if(!InWorkflow.IsValid() || !ensure(InOptions != nullptr))
	{
		return false;
	}


	FRigVMBaseAction Bracket(this);
	Bracket.SetTitle(InWorkflow.GetTitle());
	GetActionStack()->BeginAction(Bracket);

	const bool bSuccess = InWorkflow.Perform(InOptions, this);

	GetActionStack()->EndAction(Bracket);

	if(!bSuccess)
	{
		// if the workflow was run as the top level action we'll undo
		if(GetActionStack()->CurrentActions.IsEmpty())
		{
			GetActionStack()->Undo(this);
		}
	}

	return bSuccess;
}

TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> URigVMController::GetAffectedReferences(ERigVMControllerBulkEditType InEditType, bool bForceLoad)
{
	TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> FunctionReferencePtrs;
	
#if WITH_EDITOR

	check(IsValidGraph());
	URigVMGraph* Graph = GetGraph();
	URigVMFunctionLibrary* FunctionLibrary = Graph->GetTypedOuter<URigVMFunctionLibrary>();
	if(FunctionLibrary == nullptr)
	{
		return FunctionReferencePtrs;
	}

	URigVMLibraryNode* Function = FunctionLibrary->FindFunctionForNode(Graph->GetTypedOuter<URigVMCollapseNode>());
	if(Function == nullptr)
	{
		return FunctionReferencePtrs;
	}

	// get the immediate references
	FunctionReferencePtrs = FunctionLibrary->GetReferencesForFunction(Function->GetFName());
	TMap<FString, int32> VisitedPaths;
	
	for(int32 FunctionReferenceIndex = 0; FunctionReferenceIndex < FunctionReferencePtrs.Num(); FunctionReferenceIndex++)
	{
		TSoftObjectPtr<URigVMFunctionReferenceNode> FunctionReferencePtr = FunctionReferencePtrs[FunctionReferenceIndex];
		VisitedPaths.Add(FunctionReferencePtr.ToSoftObjectPath().ToString(), FunctionReferenceIndex);
	}

	for(int32 FunctionReferenceIndex = 0; FunctionReferenceIndex < FunctionReferencePtrs.Num(); FunctionReferenceIndex++)
	{
		TSoftObjectPtr<URigVMFunctionReferenceNode> FunctionReferencePtr = FunctionReferencePtrs[FunctionReferenceIndex];

		if(bForceLoad)
		{
			if(OnBulkEditProgressDelegate.IsBound() && !bSuspendNotifications)
			{
				OnBulkEditProgressDelegate.Execute(FunctionReferencePtr, InEditType, ERigVMControllerBulkEditProgress::BeginLoad, FunctionReferenceIndex, FunctionReferencePtrs.Num());
			}

			if(!FunctionReferencePtr.IsValid())
			{
				FunctionReferencePtr.LoadSynchronous();
			}

			if(OnBulkEditProgressDelegate.IsBound() && !bSuspendNotifications)
			{
				OnBulkEditProgressDelegate.Execute(FunctionReferencePtr, InEditType, ERigVMControllerBulkEditProgress::FinishedLoad, FunctionReferenceIndex, FunctionReferencePtrs.Num());
			}
		}

		// adding pins / renaming doesn't cause any recursion, so we can stop here
		if((InEditType == ERigVMControllerBulkEditType::AddExposedPin) ||
			(InEditType == ERigVMControllerBulkEditType::RemoveExposedPin) ||
			(InEditType == ERigVMControllerBulkEditType::RenameExposedPin) ||
			(InEditType == ERigVMControllerBulkEditType::ChangeExposedPinType) ||
            (InEditType == ERigVMControllerBulkEditType::RenameVariable))
		{
			continue;
		}

		// for loaded assets we'll recurse now
		if(FunctionReferencePtr.IsValid())
		{
			if(URigVMFunctionReferenceNode* AffectedFunctionReferenceNode = FunctionReferencePtr.Get())
			{
				if(URigVMLibraryNode* AffectedFunction = AffectedFunctionReferenceNode->FindFunctionForNode())
				{
					if(URigVMController* FunctionController = GetControllerForGraph(AffectedFunction->GetContainedGraph()))
					{
						TGuardValue<bool> SuspendNotifications(FunctionController->bSuspendNotifications, true);
						TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> AffectedFunctionReferencePtrs = FunctionController->GetAffectedReferences(InEditType, bForceLoad);
						for(TSoftObjectPtr<URigVMFunctionReferenceNode> AffectedFunctionReferencePtr : AffectedFunctionReferencePtrs)
						{
							const FString Key = AffectedFunctionReferencePtr.ToSoftObjectPath().ToString();
							if(VisitedPaths.Contains(Key))
							{
								continue;
							}
							VisitedPaths.Add(Key, FunctionReferencePtrs.Add(AffectedFunctionReferencePtr));
						}
					}
				}
			}
		}
	}
	
#endif

	return FunctionReferencePtrs;
}

TArray<FAssetData> URigVMController::GetAffectedAssets(ERigVMControllerBulkEditType InEditType, bool bForceLoad)
{
	TArray<FAssetData> Assets;

#if WITH_EDITOR

	if(!IsValidGraph())
	{
		return Assets;
	}

	TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> FunctionReferencePtrs = GetAffectedReferences(InEditType, bForceLoad);
	TMap<FString, int32> VisitedAssets;

	URigVMGraph* Graph = GetGraph();
	TSoftObjectPtr<URigVMGraph> GraphPtr = Graph;
	const FString ThisAssetPath = GraphPtr.ToSoftObjectPath().GetAssetPath().ToString();

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	for(int32 FunctionReferenceIndex = 0; FunctionReferenceIndex < FunctionReferencePtrs.Num(); FunctionReferenceIndex++)
	{
		TSoftObjectPtr<URigVMFunctionReferenceNode> FunctionReferencePtr = FunctionReferencePtrs[FunctionReferenceIndex];
		const FString AssetPath = FunctionReferencePtr.ToSoftObjectPath().GetAssetPath().ToString();
		if(AssetPath.StartsWith(TEXT("/Engine/Transient")))
		{
			continue;
		}
		if(VisitedAssets.Contains(AssetPath))
		{
			continue;
		}
		if(AssetPath == ThisAssetPath)
		{
			continue;
		}
					
		const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(AssetPath));
		if(AssetData.IsValid())
		{
			VisitedAssets.Add(AssetPath, Assets.Add(AssetData));
		}
	}
	
#endif

	return Assets;
}

void URigVMController::ExpandPinRecursively(URigVMPin* InPin, bool bSetupUndoRedo)
{
	if (InPin == nullptr)
	{
		return;
	}

	if (bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Expand Pin Recursively"));
	}

	bool bExpandedSomething = false;
	while (InPin)
	{
		if (SetPinExpansion(InPin, true, bSetupUndoRedo))
		{
			bExpandedSomething = true;
		}
		InPin = InPin->GetParentPin();
	}

	if (bSetupUndoRedo)
	{
		if (bExpandedSomething)
		{
			CloseUndoBracket();
		}
		else
		{
			CancelUndoBracket();
		}
	}
}

bool URigVMController::SetVariableName(URigVMVariableNode* InVariableNode, const FName& InVariableName, bool bSetupUndoRedo)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (!IsValidNodeForGraph(InVariableNode))
	{
		return false;
	}

	if (InVariableNode->GetVariableName() == InVariableName)
	{
		return false;
	}

	if (InVariableName == NAME_None)
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	TArray<FRigVMExternalVariable> Descriptions = GetAllVariables();
	TMap<FName, int32> NameToIndex;
	for (int32 VariableIndex = 0; VariableIndex < Descriptions.Num(); VariableIndex++)
	{
		NameToIndex.Add(Descriptions[VariableIndex].GetName(), VariableIndex);
	}

	const FRigVMExternalVariable VariableType = RigVMTypeUtils::ExternalVariableFromCPPType(FGuid(), InVariableName, InVariableNode->GetCPPType(), InVariableNode->GetCPPTypeObject());
	FName VariableName = URigVMSchema::GetUniqueName(InVariableName, [Descriptions, NameToIndex, VariableType](const FName& InName) {
		const int32* FoundIndex = NameToIndex.Find(InName);
		if (FoundIndex == nullptr)
		{
			return true;
		}
		return VariableType.IsSameType(Descriptions[*FoundIndex]);
	}, false, true);

	int32 NodesSharingName = 0;
	for (URigVMNode* Node : Graph->Nodes)
	{
		if (URigVMVariableNode* OtherVariableNode = Cast<URigVMVariableNode>(Node))
		{
			if (OtherVariableNode->GetVariableName() == InVariableNode->GetVariableName())
			{
				NodesSharingName++;
			}
		}
	}

	if (NodesSharingName == 1)
	{
		Notify(ERigVMGraphNotifType::VariableRemoved, InVariableNode);
	}

	SetPinDefaultValue(InVariableNode->FindPin(URigVMVariableNode::VariableName), VariableName.ToString(), false, bSetupUndoRedo, false);

	Notify(ERigVMGraphNotifType::VariableAdded, InVariableNode);
	Notify(ERigVMGraphNotifType::VariableRenamed, InVariableNode);

	return true;
}

URigVMRerouteNode* URigVMController::AddFreeRerouteNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bIsConstant, const FName& InCustomWidgetName, const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add reroutes to function library graphs."));
		return nullptr;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Reroute")));
		GetActionStack()->BeginAction(Action);
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("RerouteNode")) : InNodeName);
	URigVMRerouteNode* Node = NewObject<URigVMRerouteNode>(Graph, *Name);
	Node->Position = InPosition;

	URigVMPin* ValuePin = NewObject<URigVMPin>(Node, URigVMRerouteNode::ValueName);
	ValuePin->CPPType = InCPPType;
	ValuePin->CPPTypeObjectPath = InCPPTypeObjectPath;
	ValuePin->bIsConstant = bIsConstant;
	ValuePin->CustomWidgetName = InCustomWidgetName;
	ValuePin->Direction = ERigVMPinDirection::IO;
	AddNodePin(Node, ValuePin);
	if(!AddGraphNode(Node, false))
	{
		return nullptr;
	}

	if (!InDefaultValue.IsEmpty() && InDefaultValue != TEXT("()"))
	{
		SetPinDefaultValue(ValuePin, InDefaultValue, true, false, false);
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}

	Notify(ERigVMGraphNotifType::NodeAdded, Node);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return Node;
}

URigVMTemplateNode* URigVMController::AddConstantNode(const FString& InCPPType, const FName& InCPPTypeObjectPath,
	const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Constant Node"));
	}

	URigVMTemplateNode* TemplateNode = AddTemplateNode(FRigVMDispatch_Constant().GetTemplateNotation(), InPosition, InNodeName, bSetupUndoRedo, bSetupUndoRedo);
	if(!TemplateNode)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* ValuePin = TemplateNode->FindPin(FRigVMDispatch_Constant::ValueName.ToString()))
	{
		ResolveWildCardPin(ValuePin->GetPinPath(), InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupUndoRedo);
		if(!InDefaultValue.IsEmpty())
		{
			SetPinDefaultValue(ValuePin->GetPinPath(), InDefaultValue, true, bSetupUndoRedo, false, bSetupUndoRedo);
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMTemplateNode* URigVMController::AddMakeStructNode(const FString& InCPPType, const FName& InCPPTypeObjectPath,
                                                        const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Make Struct Node"));
	}

	URigVMTemplateNode* TemplateNode = AddTemplateNode(FRigVMDispatch_MakeStruct().GetTemplateNotation(), InPosition, InNodeName, bSetupUndoRedo, bSetupUndoRedo);
	if(!TemplateNode)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* StructPin = TemplateNode->FindPin(FRigVMDispatch_MakeStruct::StructName.ToString()))
	{
		ResolveWildCardPin(StructPin->GetPinPath(), InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupUndoRedo);
	}
	if(!InDefaultValue.IsEmpty())
	{
		if(const URigVMPin* ElementsPin = TemplateNode->FindPin(FRigVMDispatch_MakeStruct::ElementsName.ToString()))
		{
			SetPinDefaultValue(ElementsPin->GetPinPath(), InDefaultValue, true, bSetupUndoRedo, false, bSetupUndoRedo);
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMTemplateNode* URigVMController::AddBreakStructNode(const FString& InCPPType, const FName& InCPPTypeObjectPath,
	const FString& InDefaultValue, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Break Struct Node"));
	}

	URigVMTemplateNode* TemplateNode = AddTemplateNode(FRigVMDispatch_BreakStruct().GetTemplateNotation(), InPosition, InNodeName, bSetupUndoRedo, bSetupUndoRedo);
	if(!TemplateNode)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* StructPin = TemplateNode->FindPin(FRigVMDispatch_BreakStruct::StructName.ToString()))
	{
		ResolveWildCardPin(StructPin->GetPinPath(), InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupUndoRedo);
		if(!InDefaultValue.IsEmpty())
		{
			SetPinDefaultValue(StructPin->GetPinPath(), InDefaultValue, true, bSetupUndoRedo, false, bSetupUndoRedo);
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMTemplateNode* URigVMController::AddConstantNodeOnPin(const FString& InPinPath, const FVector2D& InPosition,
	const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMPin* Pin = Graph->FindPin(InPinPath);
	if(Pin == nullptr)
	{
		ReportErrorf(TEXT("Pin '%s' cannot be found."), *InPinPath);
		return nullptr;
	}

	if(Pin->GetDirection() != ERigVMPinDirection::Input && Pin->GetDirection() != ERigVMPinDirection::IO)
	{
		ReportError(TEXT("Constant nodes can only be added to input / io pins.."));
		return nullptr;
	}

	const FString& CPPType = Pin->GetCPPType();
	FName CPPTypeObjectPath = NAME_None;
	if(const UObject* CPPTypeObject = Pin->GetCPPTypeObject())
	{
		CPPTypeObjectPath = *CPPTypeObject->GetPathName();
	}
	const FString DefaultValue = Pin->GetDefaultValue();

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Constant Node On Pin"));
	}

	URigVMTemplateNode* TemplateNode = AddConstantNode(CPPType, CPPTypeObjectPath, DefaultValue, InPosition, InNodeName, bSetupUndoRedo);
	if(TemplateNode == nullptr)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* ValuePin = TemplateNode->FindPin(FRigVMDispatch_Constant::ValueName.ToString()))
	{
		AddLink(ValuePin->GetPinPath(), Pin->GetPinPath(), bSetupUndoRedo, bSetupUndoRedo);
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMTemplateNode* URigVMController::AddMakeStructNodeOnPin(const FString& InPinPath, const FVector2D& InPosition,
	const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMPin* Pin = Graph->FindPin(InPinPath);
	if(Pin == nullptr)
	{
		ReportErrorf(TEXT("Pin '%s' cannot be found."), *InPinPath);
		return nullptr;
	}

	if(Pin->GetDirection() != ERigVMPinDirection::Input && Pin->GetDirection() != ERigVMPinDirection::IO)
	{
		ReportError(TEXT("Make Struct nodes can only be added to input / io pins.."));
		return nullptr;
	}

	const FString& CPPType = Pin->GetCPPType();
	FName CPPTypeObjectPath = NAME_None;
	if(const UObject* CPPTypeObject = Pin->GetCPPTypeObject())
	{
		CPPTypeObjectPath = *CPPTypeObject->GetPathName();
	}
	const FString DefaultValue = Pin->GetDefaultValue();

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Make Struct Node On Pin"));
	}

	URigVMTemplateNode* TemplateNode = AddMakeStructNode(CPPType, CPPTypeObjectPath, DefaultValue, InPosition, InNodeName, bSetupUndoRedo);
	if(TemplateNode == nullptr)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* StructPin = TemplateNode->FindPin(FRigVMDispatch_MakeStruct::StructName.ToString()))
	{
		AddLink(StructPin->GetPinPath(), Pin->GetPinPath(), bSetupUndoRedo, bSetupUndoRedo);
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMTemplateNode* URigVMController::AddBreakStructNodeOnPin(const FString& InPinPath, const FVector2D& InPosition,
	const FString& InNodeName, bool bSetupUndoRedo)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}
	
	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	const URigVMPin* Pin = Graph->FindPin(InPinPath);
	if(Pin == nullptr)
	{
		ReportErrorf(TEXT("Pin '%s' cannot be found."), *InPinPath);
		return nullptr;
	}

	if(Pin->GetDirection() != ERigVMPinDirection::Output && Pin->GetDirection() != ERigVMPinDirection::IO)
	{
		ReportError(TEXT("Break Struct nodes can only be added to output / io pins.."));
		return nullptr;
	}

	const FString& CPPType = Pin->GetCPPType();
	FName CPPTypeObjectPath = NAME_None;
	if(const UObject* CPPTypeObject = Pin->GetCPPTypeObject())
	{
		CPPTypeObjectPath = *CPPTypeObject->GetPathName();
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Break Struct Node On Pin"));
	}

	URigVMTemplateNode* TemplateNode = AddBreakStructNode(CPPType, CPPTypeObjectPath, FString(), InPosition, InNodeName, bSetupUndoRedo);
	if(TemplateNode == nullptr)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	if(const URigVMPin* StructPin = TemplateNode->FindPin(FRigVMDispatch_MakeStruct::StructName.ToString()))
	{
		AddLink(Pin->GetPinPath(), StructPin->GetPinPath(), bSetupUndoRedo, bSetupUndoRedo);
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return TemplateNode;
}

URigVMNode* URigVMController::AddBranchNode(const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	return AddUnitNode(FRigVMFunction_ControlFlowBranch::StaticStruct(), FRigVMStruct::ExecuteName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand);
}

URigVMNode* URigVMController::AddIfNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FVector2D& InPosition, const FString& InNodeName, bool  bSetupUndoRedo, bool bPrintPythonCommand)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	ensure(!InCPPType.IsEmpty());

	UObject* CPPTypeObject = nullptr;
	if(!InCPPTypeObjectPath.IsNone())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath.ToString());
			return nullptr;
		}
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add If Node"));
	}

	const FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, CPPTypeObject);
	const FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("IfNode")) : InNodeName);
	const TRigVMTypeIndex& TypeIndex = FRigVMRegistry::Get().FindOrAddType({*CPPType, CPPTypeObject});
	
	const FRigVMDispatchFactory* Factory = FRigVMRegistry::Get().FindOrAddDispatchFactory(FRigVMDispatch_If::StaticStruct());
	URigVMNode* Node = AddTemplateNode(Factory->GetTemplate()->GetNotation(), InPosition, Name, bSetupUndoRedo, bPrintPythonCommand);
	if(Node)
	{
		ResolveWildCardPin(Node->GetPins().Last(), TypeIndex, bSetupUndoRedo, bPrintPythonCommand);
	}
	
	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return Node;
}

URigVMNode* URigVMController::AddIfNodeFromStruct(UScriptStruct* InScriptStruct, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo)
{
	if (!InScriptStruct)
	{
		return nullptr;
	}

	return AddIfNode(RigVMTypeUtils::GetUniqueStructTypeName(InScriptStruct), FName(InScriptStruct->GetPathName()), InPosition, InNodeName, bSetupUndoRedo);
}

URigVMNode* URigVMController::AddSelectNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	ensure(!InCPPType.IsEmpty());

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsNone())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath.ToString());
			return nullptr;
		}
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Select Node"));
	}

	const FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, CPPTypeObject);
	const FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("SelectNode")) : InNodeName);
	const TRigVMTypeIndex& TypeIndex = FRigVMRegistry::Get().FindOrAddType({*CPPType, CPPTypeObject});
	
	const FRigVMDispatchFactory* Factory = FRigVMRegistry::Get().FindOrAddDispatchFactory(FRigVMDispatch_SelectInt32::StaticStruct());
	URigVMNode* Node = AddTemplateNode(Factory->GetTemplate()->GetNotation(), InPosition, Name, bSetupUndoRedo, bPrintPythonCommand);
	if(Node)
	{
		ResolveWildCardPin(Node->GetPins().Last(), TypeIndex, bSetupUndoRedo, bPrintPythonCommand);
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}
	return Node;
}

URigVMNode* URigVMController::AddSelectNodeFromStruct(UScriptStruct* InScriptStruct, const FVector2D& InPosition,
	const FString& InNodeName, bool bSetupUndoRedo)
{
	if (!InScriptStruct)
	{
		return nullptr;
	}

	return AddSelectNode(RigVMTypeUtils::GetUniqueStructTypeName(InScriptStruct), FName(InScriptStruct->GetPathName()), InPosition, InNodeName, bSetupUndoRedo);
}

URigVMTemplateNode* URigVMController::AddTemplateNode(const FName& InNotation, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	ensure(!InNotation.IsNone());

	FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(FRigVMRegistry::Get().FindTemplate(InNotation));
	if (Template == nullptr)
	{
		ReportErrorf(TEXT("Template '%s' cannot be found."), *InNotation.ToString());
		return nullptr;
	}

	if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		if(const FRigVMClient* Client = ClientHost->GetRigVMClient())
		{
			if(!Template->SupportsExecuteContextStruct(GetSchema()->GetExecuteContextStruct()))
			{
				ReportErrorf(TEXT("Cannot add node for template '%s' - incompatible execute context: '%s' vs '%s'."),
						*Template->GetNotation().ToString(),
						*Template->GetExecuteContextStruct()->GetStructCPPName(),
						*GetSchema()->GetExecuteContextStruct()->GetStructCPPName());
				return nullptr;
			}
		}
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? Template->GetNodeName().ToString() : InNodeName);
	URigVMTemplateNode* Node = nullptr;

	if(const UClass* NodeClassToUse = GetNodeClassForTemplate(Template))
	{
		check(NodeClassToUse == URigVMTemplateNode::StaticClass() ||
			NodeClassToUse->IsChildOf(URigVMTemplateNode::StaticClass()));
		Node = NewObject<URigVMTemplateNode>(Graph, NodeClassToUse, *Name);
	}

	if(Node == nullptr)
	{
		ReportErrorf(TEXT("Template node '%s' cannot be created. Unknown template."), *InNotation.ToString());
		return nullptr;
	}
	
	Node->TemplateNotation = Template->GetNotation();
	Node->Position = InPosition;

	int32 PermutationIndex = INDEX_NONE;
	FRigVMTemplate::FTypeMap Types;
	Template->FullyResolve(Types, PermutationIndex);

	FRigVMRegistry& Registry = FRigVMRegistry::Get();
	AddPinsForTemplate(Template, Types, Node);

	if (Node->HasWildCardPin())
	{
		UpdateTemplateNodePinTypes(Node, false);
	}
	else
	{
		if (!Node->IsA<URigVMFunctionEntryNode>() && !Node->IsA<URigVMFunctionReturnNode>())
		{
			FullyResolveTemplateNode(Node, INDEX_NONE, false);
		}
	}

	if(!AddGraphNode(Node, true))
	{
		return nullptr;
	}

	FRigVMBaseAction Action(this);
	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}
	
	ResolveTemplateNodeMetaData(Node, bSetupUndoRedo);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

TArray<UScriptStruct*> URigVMController::GetRegisteredUnitStructs()
{
	TArray<UScriptStruct*> UnitStructs;

	for(const FRigVMFunction& Function : FRigVMRegistry::Get().GetFunctions())
	{
		if(!Function.IsValid())
		{
			continue;
		}
		if(UScriptStruct* Struct = Function.Struct)
		{
			if (!Struct->IsChildOf(FRigVMStruct::StaticStruct()))
			{
				continue;
			}
			UnitStructs.Add(Struct);
		}
	}
	
	return UnitStructs; 
}

TArray<FString> URigVMController::GetRegisteredTemplates()
{
	TArray<FString> Templates;

	for(const FRigVMTemplate& Template : FRigVMRegistry::Get().GetTemplates())
	{
		if(!Template.IsValid() || Template.NumPermutations() < 2)
		{
			continue;
		}
		Templates.Add(Template.GetNotation().ToString());
	}
	
	return Templates; 
}

TArray<UScriptStruct*> URigVMController::GetUnitStructsForTemplate(const FName& InNotation)
{
	TArray<UScriptStruct*> UnitStructs;

	FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(FRigVMRegistry::Get().FindTemplate(InNotation));
	if(Template)
	{
		if(!Template->UsesDispatch())
		{
			for(int32 PermutationIndex = 0; PermutationIndex < Template->NumPermutations(); PermutationIndex++)
			{
				UnitStructs.Add(Template->GetOrCreatePermutation(PermutationIndex)->Struct);
			}
		}
	}
	
	return UnitStructs;
}

FString URigVMController::GetTemplateForUnitStruct(UScriptStruct* InFunction, const FString& InMethodName)
{
	if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(InFunction, *InMethodName))
	{
		if(const FRigVMTemplate* Template = Function->GetTemplate())
		{
			return Template->GetNotation().ToString();
		}
	}
	return FString();
}

URigVMEnumNode* URigVMController::AddEnumNode(const FName& InCPPTypeObjectPath, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

    UObject* CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
	if (CPPTypeObject == nullptr)
	{
		ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath.ToString());
		return nullptr;
	}

	UEnum* Enum = Cast<UEnum>(CPPTypeObject);
	if(Enum == nullptr)
	{
		ReportErrorf(TEXT("Cpp type object for path '%s' is not an enum."), *InCPPTypeObjectPath.ToString());
		return nullptr;
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("IfNode")) : InNodeName);
	URigVMEnumNode* Node = NewObject<URigVMEnumNode>(Graph, *Name);
	Node->Position = InPosition;

	URigVMPin* EnumValuePin = NewObject<URigVMPin>(Node, URigVMEnumNode::EnumValueName);
	EnumValuePin->CPPType = CPPTypeObject->GetName();
	EnumValuePin->CPPTypeObject = CPPTypeObject;
	EnumValuePin->CPPTypeObjectPath = InCPPTypeObjectPath;
	EnumValuePin->Direction = ERigVMPinDirection::Visible;
	EnumValuePin->DefaultValue = Enum->GetNameStringByValue(0);
	AddNodePin(Node, EnumValuePin);

	URigVMPin* EnumIndexPin = NewObject<URigVMPin>(Node, URigVMEnumNode::EnumIndexName);
	EnumIndexPin->CPPType = RigVMTypeUtils::Int32Type;
	EnumIndexPin->Direction = ERigVMPinDirection::Output;
	EnumIndexPin->DisplayName = TEXT("Result");
	AddNodePin(Node, EnumIndexPin);

	if(!AddGraphNode(Node, true))
	{
		return nullptr;
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMImportFromTextAction(this, Node));
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
				FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

URigVMNode* URigVMController::AddArrayNode(ERigVMOpCode InOpCode, const FString& InCPPType,
	UObject* InCPPTypeObject, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo,
	bool bPrintPythonCommand, bool bIsPatching)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	const FString CPPType = RigVMTypeUtils::IsArrayType(InCPPType) ? RigVMTypeUtils::BaseTypeFromArrayType(InCPPType) : InCPPType;
	const TRigVMTypeIndex& ElementTypeIndex = FRigVMRegistry::Get().FindOrAddType({*CPPType, InCPPTypeObject});
	if(ElementTypeIndex == INDEX_NONE)
	{
		return nullptr;
	}
	const TRigVMTypeIndex& ArrayTypeIndex = FRigVMRegistry::Get().GetArrayTypeFromBaseTypeIndex(ElementTypeIndex);

	const FName FactoryName = FRigVMDispatch_ArrayBase::GetFactoryNameForOpCode(InOpCode);
	if(FactoryName.IsNone())
	{
		ReportErrorf(TEXT("OpCode '%s' is not valid for Array Node."), *StaticEnum<ERigVMOpCode>()->GetNameStringByValue((int64)InOpCode));
		return nullptr;
	}

	const FRigVMDispatchFactory* Factory = FRigVMRegistry::Get().FindDispatchFactory(FactoryName);
	if(Factory == nullptr)
	{
		ReportErrorf(TEXT("Cannot find array dispatch '%s'."), *FactoryName.ToString());
		return nullptr;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Add Array Node"));
	}
	
	FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(Factory->GetTemplate());
	URigVMTemplateNode* Node = AddTemplateNode(
		Template->GetNotation(),
		InPosition, 
		InNodeName, 
		bSetupUndoRedo, 
		bPrintPythonCommand);

	if(!FRigVMRegistry::Get().IsWildCardType(ElementTypeIndex))
	{
		FName ArgumentNameToResolve = NAME_None;
		TRigVMTypeIndex TypeIndex = INDEX_NONE;
		for(int32 Index = 0; Index < Template->NumArguments(); Index++)
		{
			const FRigVMTemplateArgument* Argument = Template->GetArgument(Index);
			if(Argument->IsSingleton())
			{
				continue;
			}
			if(Argument->GetArrayType() == FRigVMTemplateArgument::EArrayType_SingleValue)
			{
				ArgumentNameToResolve = Argument->GetName();
				TypeIndex = ElementTypeIndex;
				break;
			}
			if(Argument->GetArrayType() == FRigVMTemplateArgument::EArrayType_ArrayValue)
			{
				ArgumentNameToResolve = Argument->GetName();
				TypeIndex = ArrayTypeIndex;
				break;
			}
		}

		if(!ArgumentNameToResolve.IsNone() && TypeIndex != INDEX_NONE)
		{
			if(bIsPatching)
			{
				FRigVMTemplate::FTypeMap TypeMap;
				TypeMap.Add(ArgumentNameToResolve, TypeIndex);

				TArray<int32> Permutations;
				Template->Resolve(TypeMap, Permutations, false);
				check(Permutations.Num() == 1);
				Template->GetOrCreatePermutation(Permutations[0]);

				for(const FRigVMTemplate::FTypePair& Pair : TypeMap)
				{
					if(!FRigVMRegistry::Get().IsWildCardType(Pair.Value))
					{
						if(URigVMPin* Pin = Node->FindPin(Pair.Key.ToString()))
						{
							ChangePinType(Pin, Pair.Value, false, false);
						}
					}
				}

				FullyResolveTemplateNode(Node, Permutations[0], false);
			}
			else if(const URigVMPin* Pin = Node->FindPin(ArgumentNameToResolve.ToString()))
			{
				ResolveWildCardPin(Pin->GetPinPath(), TypeIndex, bSetupUndoRedo, bPrintPythonCommand);
			}
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}

	return Node;
}

URigVMNode* URigVMController::AddArrayNodeFromObjectPath(ERigVMOpCode InOpCode, const FString& InCPPType,
	const FString& InCPPTypeObjectPath, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo,
	bool bPrintPythonCommand, bool bIsPatching)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsEmpty())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath);
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath);
			return nullptr;
		}
	}

	return AddArrayNode(InOpCode, InCPPType, CPPTypeObject, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand, bIsPatching);
}

URigVMInvokeEntryNode* URigVMController::AddInvokeEntryNode(const FName& InEntryName, const FVector2D& InPosition,
	const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return nullptr;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		ReportError(TEXT("Cannot add invoke entry nodes to function library graphs."));
		return nullptr;
	}

	FString Name = GetSchema()->GetValidNodeName(Graph, InNodeName.IsEmpty() ? FString(TEXT("InvokeEntryNode")) : InNodeName);
	URigVMInvokeEntryNode* Node = NewObject<URigVMInvokeEntryNode>(Graph, *Name);
	Node->Position = InPosition;

	URigVMPin* ExecutePin = MakeExecutePin(Node, FRigVMStruct::ExecuteContextName);
	ExecutePin->Direction = ERigVMPinDirection::IO;
	AddNodePin(Node, ExecutePin);

	URigVMPin* EntryNamePin = NewObject<URigVMPin>(Node, URigVMInvokeEntryNode::EntryName);
	EntryNamePin->CPPType = RigVMTypeUtils::FNameType;
	EntryNamePin->Direction = ERigVMPinDirection::Input;
	EntryNamePin->bIsConstant = true;
	EntryNamePin->DefaultValue = InEntryName.ToString();
	EntryNamePin->CustomWidgetName = TEXT("EntryName");
	AddNodePin(Node, EntryNamePin);

	if(!AddGraphNode(Node, true))
	{
		return nullptr;
	}

	if (!bSuspendNotifications)
	{
		(void)MarkPackageDirty(Graph);
	}

	FRigVMControllerCompileBracketScope CompileScope(this);

	Notify(ERigVMGraphNotifType::VariableAdded, Node);

	if (bSetupUndoRedo)
	{
		FRigVMImportFromTextAction Action(this, Node);
		Action.SetTitle(FString::Printf(TEXT("Add Invoke %s Entry"), *InEntryName.ToString()));
		GetActionStack()->AddAction(Action);
	}

	if (bPrintPythonCommand)
	{
		TArray<FString> Commands = GetAddNodePythonCommands(Node);
		for (const FString& Command : Commands)
		{
			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
								FString::Printf(TEXT("%s"), *Command));
		}
	}

	return Node;
}

FName URigVMController::AddTrait(const FName& InNodeName, const FName& InTraitTypeObjectPath,
	const FName& InTraitName, const FString& InDefaultValue, int32 InPinIndex, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return NAME_None;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NAME_None;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMNode* Node = Graph->FindNodeByName(InNodeName))
	{
		UObject* TraitCPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InTraitTypeObjectPath.ToString());
		if(TraitCPPTypeObject == nullptr)
		{
			TraitCPPTypeObject = URigVMCompiler::GetScriptStructForCPPType(InTraitTypeObjectPath.ToString());
		}
		if(TraitCPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find trait script struct '%s'."), *InTraitTypeObjectPath.ToString());
			return NAME_None;
		}

		UScriptStruct* TraitScriptStruct = Cast<UScriptStruct>(TraitCPPTypeObject);
		if(TraitScriptStruct == nullptr)
		{
			ReportErrorf(TEXT("CPP Type Object '%s' is not a struct."), *InTraitTypeObjectPath.ToString());
			return NAME_None;
		}

		const FName TraitName = AddTrait(Node, TraitScriptStruct, InTraitName, InDefaultValue, InPinIndex, bSetupUndoRedo);
		if(!TraitName.IsNone() && bPrintPythonCommand)
		{
			const TArray<FString> TraitCommands = GetAddTraitPythonCommands(Node, TraitName);
			for(const FString& TraitCommand : TraitCommands)
			{
				RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), TraitCommand);
			}
		}
		return TraitName;
	}

	ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
	return NAME_None;
}

FName URigVMController::AddTrait(URigVMNode* InNode, UScriptStruct* InTraitScriptStruct, const FName& InTraitName, const FString& InDefaultValue, int32 InPinIndex, bool bSetupUndoRedo)
{
	if (!IsValidNodeForGraph(InNode))
	{
		return NAME_None;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return NAME_None;
	}

	check(InTraitScriptStruct);

	if(!InTraitScriptStruct->IsChildOf(FRigVMTrait::StaticStruct()))
	{
		ReportErrorf(TEXT("CPP Type Object '%s' is not a struct."), *InTraitScriptStruct->GetPathName());
		return NAME_None;
	}

	const FRigVMTemplateArgumentType TraitType(InTraitScriptStruct);
	const TRigVMTypeIndex TraitTypeIndex = FRigVMRegistry::Get().FindOrAddType(TraitType);

	if(const URigVMSchema* Schema = GetSchema())
	{
		if(!Schema->SupportsType(this, TraitTypeIndex))
		{
			ReportError(TEXT("Trait cannot be added to node: Schema doesn't support the type."));
			return NAME_None;
		}
	}

	const FName ValidTraitName = URigVMSchema::GetUniqueName(InTraitName, [InNode](const FName& InName) {
		return InNode->FindPin(InName.ToString()) == nullptr;
	}, false, false);

	TSharedPtr<FStructOnScope> TraitScope(new FStructOnScope(InTraitScriptStruct));
	FRigVMTrait* Trait = (FRigVMTrait*)TraitScope->GetStructMemory();

	FString DefaultValue = InDefaultValue;
	// Combine user overrides with struct defaults into a single default value string
	CreateDefaultValueForStructIfRequired(InTraitScriptStruct, DefaultValue);
	OverrideDefaultValueMember(TEXT("Name"), TEXT("\"")+ValidTraitName.ToString()+TEXT("\""), DefaultValue);
	
	FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
	{
		// force logging to the error pipe for error detection
		LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity()); 
		InTraitScriptStruct->ImportText(*DefaultValue, Trait, nullptr, PPF_None, &ErrorPipe, InTraitScriptStruct->GetName()); 
	}

	check(Trait->Name == ValidTraitName.ToString());

	FString FailureReason;
	if(!Trait->CanBeAddedToNode(InNode, &FailureReason))
	{
		ReportErrorf(TEXT("Trait cannot be added to node: %s"), *FailureReason);
		return NAME_None;
	}

	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Add Trait")));
		GetActionStack()->BeginAction(Action);
		GetActionStack()->AddAction(FRigVMAddTraitAction(this, InNode, ValidTraitName, InTraitScriptStruct, DefaultValue, InPinIndex == INDEX_NONE ? InNode->GetPins().Num() : InPinIndex));
	}

	InNode->TraitRootPinNames.Add(ValidTraitName.ToString());
	
	// root trait pin store their default value in a separate property bag so that
	// things like soft object ptr can be used and tracked in a uproperty
	FRigVMTraitDefaultValueStruct& TraitDefaultValueStruct = InNode->TraitDefaultValues.Add(ValidTraitName.ToString());
	TraitDefaultValueStruct.Init(InTraitScriptStruct);
	TraitDefaultValueStruct.SetValue(DefaultValue);

	URigVMPin* TraitPin = NewObject<URigVMPin>(InNode, ValidTraitName);
	const FString DisplayName = Trait->GetDisplayName();
	TraitPin->DisplayName = DisplayName.IsEmpty() ? FName(NAME_None) : FName(*DisplayName); 
	TraitPin->CPPType = InTraitScriptStruct->GetStructCPPName();
	TraitPin->CPPTypeObject = InTraitScriptStruct;
	TraitPin->CPPTypeObjectPath = *TraitPin->CPPTypeObject->GetPathName();
	TraitPin->Direction = ERigVMPinDirection::Input;
	TraitPin->DefaultValueObject = TraitPin->CPPTypeObject;

	{
		const TGuardValue<bool> GuardNotifications(bSuspendNotifications, true);
	
		AddNodePin(InNode, TraitPin);

		AddPinsForStruct(InTraitScriptStruct, InNode, TraitPin, TraitPin->GetDirection(), DefaultValue, true);

		FRigVMPinInfoArray ProgrammaticPins;
		Trait->GetProgrammaticPins(this, INDEX_NONE, TraitPin, DefaultValue, ProgrammaticPins);

		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		const FRigVMPinInfoArray PreviousPins;

		for (int32 PinIndex = 0; PinIndex < ProgrammaticPins.Num(); ++PinIndex)
		{
			const FString& PinPath = ProgrammaticPins.GetPinPath(PinIndex);
			FString ParentPinPath, PinName;
			UObject* OuterForPin = TraitPin;
			if (URigVMPin::SplitPinPathAtEnd(PinPath, ParentPinPath, PinName))
			{
				OuterForPin = TraitPin->FindSubPin(ParentPinPath);
			}

			CreatePinFromPinInfo(Registry, PreviousPins, ProgrammaticPins[PinIndex], PinPath, OuterForPin);
		}

		// move the the pin to the right index as required
		if(TraitPin->GetPinIndex() != InPinIndex &&
			InPinIndex >=0 && InPinIndex < InNode->GetPins().Num())
		{
			URigVMPin* LastPin =  InNode->Pins.Pop();
			InNode->Pins.Insert(LastPin, InPinIndex);
		}

		InNode->UpdateTraitRootPinNames();
	}

	Notify(ERigVMGraphNotifType::PinAdded, TraitPin);

	Trait->OnTraitAdded(this, InNode);

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return ValidTraitName;
}

bool URigVMController::RemoveTrait(const FName& InNodeName, const FName& InTraitName, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	const URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMNode* Node = Graph->FindNodeByName(InNodeName))
	{
		const bool bSuccess = RemoveTrait(Node, InTraitName, bSetupUndoRedo);
		if(bSuccess && bPrintPythonCommand)
		{
			const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());
        	const FString NodeName = GetSchema()->GetSanitizedNodeName(Node->GetName());

			RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()),
				FString::Printf(TEXT("asset.get_controller_by_name('%s').remove_trait('%s', '%s')"),
				*GraphName,
				*NodeName,
				*InTraitName.ToString()));
		}
		return bSuccess;
	}

	ReportErrorf(TEXT("Cannot find node '%s'."), *InNodeName.ToString());
	return false;

}

bool URigVMController::RemoveTrait(URigVMNode* InNode, const FName& InTraitName, bool bSetupUndoRedo)
{
	if(!IsValidNodeForGraph(InNode))
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMPin* TraitPin = InNode->FindTrait(InTraitName);
	if(TraitPin == nullptr)
	{
		return false;
	}

	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(FString::Printf(TEXT("Remove Trait")));
		GetActionStack()->BeginAction(Action);

		const UScriptStruct* TraitScriptStruct = TraitPin->GetScriptStruct();
		const FString TraitDefaultValue = TraitPin->GetDefaultValue();
		GetActionStack()->AddAction(FRigVMRemoveTraitAction(this, InNode, InTraitName, TraitScriptStruct, TraitDefaultValue, TraitPin->GetPinIndex()));
	}

	const FString TraitNameString = InTraitName.ToString();
	(void)InNode->TraitRootPinNames.RemoveAll([TraitNameString](const FString& TraitRootPinName) -> bool
	{
		return TraitNameString.Equals(TraitRootPinName, ESearchCase::CaseSensitive);
	});

	(void)InNode->TraitDefaultValues.Remove(TraitNameString);

	RemovePin(TraitPin, bSetupUndoRedo, true);

	if(bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	return true;
}

void URigVMController::ForEveryPinRecursively(URigVMPin* InPin, TFunction<void(URigVMPin*)> OnEachPinFunction)
{
	OnEachPinFunction(InPin);
	for (URigVMPin* SubPin : InPin->SubPins)
	{
		ForEveryPinRecursively(SubPin, OnEachPinFunction);
	}
}

void URigVMController::ForEveryPinRecursively(URigVMNode* InNode, TFunction<void(URigVMPin*)> OnEachPinFunction)
{
	for (URigVMPin* Pin : InNode->GetPins())
	{
		ForEveryPinRecursively(Pin, OnEachPinFunction);
	}
}

bool URigVMController::IsValidGraph() const
{
	if(!IsValidSchema())
	{
		return false;
	}
	
	URigVMGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		ReportError(TEXT("Controller does not have a graph associated - use SetGraph / set_graph."));
		return false;
	}

	if (!IsValid(Graph))
	{
		return false;
	}

	return true;
}

bool URigVMController::IsValidSchema() const
{
	const URigVMSchema* SchemaPtr = GetSchema();
	if (SchemaPtr == nullptr)
	{
		ReportError(TEXT("Controller does not have a schema associated."));
		return false;
	}

	if (!IsValid(SchemaPtr))
	{
		return false;
	}

	return true;
}

bool URigVMController::IsGraphEditable() const
{
	if(const URigVMSchema* Schema = GetSchema())
	{
		if(const URigVMGraph* Graph = GetGraph())
		{
			return Schema->IsGraphEditable(Graph);
		}
	}
	return false;
}

bool URigVMController::IsValidNodeForGraph(const URigVMNode* InNode)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (InNode == nullptr)
	{
		ReportError(TEXT("InNode is nullptr."));
		return false;
	}

	if (InNode->GetGraph() != GetGraph())
	{
		ReportWarningf(TEXT("InNode '%s' is on a different graph. InNode graph is %s, this graph is %s"), *InNode->GetNodePath(), *GetNameSafe(InNode->GetGraph()), *GetNameSafe(GetGraph()));
		return false;
	}

	if (InNode->GetNodeIndex() == INDEX_NONE && !InNode->GetOuter()->IsA<URigVMInjectionInfo>())
	{
		ReportErrorf(TEXT("InNode '%s' is transient (not yet nested to a graph)."), *InNode->GetName());
		return false;
	}

	return true;
}

bool URigVMController::IsValidPinForGraph(const URigVMPin* InPin)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (InPin == nullptr)
	{
		ReportError(TEXT("InPin is nullptr."));
		return false;
	}

	if (!IsValidNodeForGraph(InPin->GetNode()))
	{
		return false;
	}

	if (InPin->GetPinIndex() == INDEX_NONE)
	{
		ReportErrorf(TEXT("InPin '%s' is transient (not yet nested properly)."), *InPin->GetName());
	}

	return true;
}

bool URigVMController::IsValidLinkForGraph(const URigVMLink* InLink)
{
	if(!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	if (InLink == nullptr)
	{
		ReportError(TEXT("InLink is nullptr."));
		return false;
	}

	if (InLink->GetGraph() != GetGraph())
	{
		ReportError(TEXT("InLink is on a different graph."));
		return false;
	}

	if(InLink->GetSourcePin() == nullptr)
	{
		ReportError(TEXT("InLink has no source pin."));
		return false;
	}

	if(InLink->GetTargetPin() == nullptr)
	{
		ReportError(TEXT("InLink has no target pin."));
		return false;
	}

	if (InLink->GetLinkIndex() == INDEX_NONE)
	{
		ReportError(TEXT("InLink is transient (not yet nested properly)."));
	}

	if(!IsValidPinForGraph(InLink->GetSourcePin()))
	{
		return false;
	}

	if(!IsValidPinForGraph(InLink->GetTargetPin()))
	{
		return false;
	}

	return true;
}

void URigVMController::AddPinsForStruct(UStruct* InStruct, URigVMNode* InNode, URigVMPin* InParentPin, ERigVMPinDirection InPinDirection, const FString& InDefaultValue, bool bAutoExpandArrays, const FRigVMPinInfoArray* PreviousPins)
{
	if(!InStruct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		if(!GetSchema()->ShouldUnfoldStruct(this, InStruct))
		{
			return;
		}
	}

	if(InParentPin && InParentPin->ShouldHideSubPins())
	{
		return;
	}

	// todo: reuse the default values when creating the pins
	
	TArray<FString> MemberNameValuePairs = URigVMPin::SplitDefaultValue(InDefaultValue);
	TMap<FName, FString> MemberValues;
	for (const FString& MemberNameValuePair : MemberNameValuePairs)
	{
		FString MemberName, MemberValue;
		if (MemberNameValuePair.Split(TEXT("="), &MemberName, &MemberValue))
		{
			MemberValues.Add(*MemberName, MemberValue);
		}
	}

	TSharedPtr<FStructOnScope> StructOnScope;
	const FRigVMStruct* RigVMStructInstance = nullptr;
	if(InStruct->IsChildOf(FRigVMStruct::StaticStruct()))
	{
		StructOnScope = MakeShareable(new FStructOnScope(InStruct));
		RigVMStructInstance = reinterpret_cast<FRigVMStruct*>(StructOnScope->GetStructMemory()); 
	}

	TArray<UStruct*> StructsToVisit = FRigVMTemplate::GetSuperStructs(InStruct, true);
	for(UStruct* StructToVisit : StructsToVisit)
	{
		// using EFieldIterationFlags::None excludes the
		// properties of the super struct in this iterator.
		for (TFieldIterator<FProperty> It(StructToVisit, EFieldIterationFlags::None); It; ++It)
		{
			// filter pins that structs want to hide
			if(RigVMStructInstance)
			{
				if(!RigVMStructInstance->ShouldCreatePinForProperty(*It))
				{
					continue;
				}
			}
			
			FName PropertyName = It->GetFName();

			URigVMPin* Pin = NewObject<URigVMPin>(InParentPin == nullptr ? Cast<UObject>(InNode) : Cast<UObject>(InParentPin), PropertyName);
			ConfigurePinFromProperty(*It, Pin, InPinDirection);

			if (InParentPin)
			{
				AddSubPin(InParentPin, Pin);
			}
			else
			{
				AddNodePin(InNode, Pin);
			}

			FString* DefaultValuePtr = MemberValues.Find(Pin->GetFName());

			FStructProperty* StructProperty = CastField<FStructProperty>(*It);
			if (StructProperty)
			{
				if (GetSchema()->ShouldUnfoldStruct(this, StructProperty->Struct))
				{
					FString DefaultValue;
					if (DefaultValuePtr != nullptr)
					{
						DefaultValue = *DefaultValuePtr;
					}
					CreateDefaultValueForStructIfRequired(StructProperty->Struct, DefaultValue);
					{
						TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
						AddPinsForStruct(StructProperty->Struct, InNode, Pin, Pin->GetDirection(), DefaultValue, bAutoExpandArrays);
					}
				}
				else if(DefaultValuePtr != nullptr)
				{
					Pin->DefaultValue = *DefaultValuePtr;
					if(Pin->CanProvideDefaultValue() && !Pin->DefaultValue.IsEmpty())
					{
						Pin->DefaultValueType = GetDefaultValueType(Pin, Pin->DefaultValue);
					}
				}
			}

			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*It);
			if (ArrayProperty)
			{
				ensure(Pin->IsArray());

				if (DefaultValuePtr)
				{
					if (GetSchema()->CanUnfoldPin(this, Pin))
					{
						TArray<FString> ElementDefaultValues = URigVMPin::SplitDefaultValue(*DefaultValuePtr);
						AddPinsForArray(ArrayProperty, InNode, Pin, Pin->Direction, ElementDefaultValues, bAutoExpandArrays);
					}
					else
					{
						FString DefaultValue = *DefaultValuePtr;
						PostProcessDefaultValue(Pin, DefaultValue);
						Pin->DefaultValue = *DefaultValuePtr;
						if(Pin->CanProvideDefaultValue() && !Pin->DefaultValue.IsEmpty())
						{
							Pin->DefaultValueType = GetDefaultValueType(Pin, Pin->DefaultValue);
						}
					}
				}
			}
			
			if (!Pin->IsArray() && !Pin->IsStruct() && DefaultValuePtr != nullptr)
			{
				FString DefaultValue = *DefaultValuePtr;
				PostProcessDefaultValue(Pin, DefaultValue);
				if (Pin->IsUObject())
				{
					Pin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(DefaultValue);
					Pin->DefaultValue.Empty();
				}
				else
				{
					Pin->DefaultValue = DefaultValue;
					Pin->DefaultValueObject = nullptr;
				}
				
				if(Pin->CanProvideDefaultValue() && !DefaultValue.IsEmpty())
				{
					Pin->DefaultValueType = GetDefaultValueType(Pin, DefaultValue);
				}
			}

			if (!bSuspendNotifications)
			{
				Notify(ERigVMGraphNotifType::PinAdded, Pin);
			}
		}
	}
}

void URigVMController::AddPinsForArray(FArrayProperty* InArrayProperty, URigVMNode* InNode, URigVMPin* InParentPin, ERigVMPinDirection InPinDirection, const TArray<FString>& InDefaultValues, bool bAutoExpandArrays)
{
	check(InParentPin);
	if (!GetSchema()->CanUnfoldPin(this, InParentPin))
	{
		return;
	}

	if(InParentPin && InParentPin->ShouldHideSubPins())
	{
		return;
	}

	for (int32 ElementIndex = 0; ElementIndex < InDefaultValues.Num(); ElementIndex++)
	{
		FString ElementName = FString::FormatAsNumber(InParentPin->SubPins.Num());
		URigVMPin* Pin = NewObject<URigVMPin>(InParentPin, *ElementName);

		ConfigurePinFromProperty(InArrayProperty->Inner, Pin, InPinDirection);
		FString DefaultValue = InDefaultValues[ElementIndex];

		AddSubPin(InParentPin, Pin);

		if (bAutoExpandArrays)
		{
			TGuardValue<bool> ErrorGuard(bReportWarningsAndErrors, false);
			ExpandPinRecursively(Pin, false);
		}

		FStructProperty* StructProperty = CastField<FStructProperty>(InArrayProperty->Inner);
		if (StructProperty)
		{
			if (GetSchema()->CanUnfoldPin(this, Pin))
			{
				// DefaultValue before this point only contains parent struct overrides,
				// see comments in CreateDefaultValueForStructIfRequired
				UScriptStruct* ScriptStruct = Pin->GetScriptStruct();
				if (ScriptStruct)
				{
					CreateDefaultValueForStructIfRequired(ScriptStruct, DefaultValue);
				}
				{
					TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
					AddPinsForStruct(StructProperty->Struct, InNode, Pin, Pin->Direction, DefaultValue, bAutoExpandArrays);
				}
			}
			else if (!DefaultValue.IsEmpty())
			{
				PostProcessDefaultValue(Pin, DefaultValue);
				Pin->DefaultValue = DefaultValue;
				if(Pin->CanProvideDefaultValue() && !Pin->DefaultValue.IsEmpty())
				{
					Pin->DefaultValueType = GetDefaultValueType(Pin, Pin->DefaultValue);
				}
			}
		}

		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InArrayProperty->Inner);
		if (ArrayProperty)
		{
			if (GetSchema()->CanUnfoldPin(this, Pin))
			{
				TArray<FString> ElementDefaultValues = URigVMPin::SplitDefaultValue(DefaultValue);
				AddPinsForArray(ArrayProperty, InNode, Pin, Pin->Direction, ElementDefaultValues, bAutoExpandArrays);
			}
			else if (!DefaultValue.IsEmpty())
			{
				PostProcessDefaultValue(Pin, DefaultValue);
				Pin->DefaultValue = DefaultValue;
				if(Pin->CanProvideDefaultValue() && !Pin->DefaultValue.IsEmpty())
				{
					Pin->DefaultValueType = GetDefaultValueType(Pin, Pin->DefaultValue);
				}
			}
		}

		if (!Pin->IsArray() && !Pin->IsStruct())
		{
			PostProcessDefaultValue(Pin, DefaultValue);
			if (Pin->IsUObject())
			{
				Pin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(DefaultValue);
				Pin->DefaultValue.Empty();
			}
			else
			{
				Pin->DefaultValue = DefaultValue;
				Pin->DefaultValueObject = nullptr;
			}
			
			if(Pin->CanProvideDefaultValue() && !DefaultValue.IsEmpty())
			{
				Pin->DefaultValueType = GetDefaultValueType(Pin, DefaultValue);
			}
		}
	}
}

void URigVMController::AddPinsForTemplate(const FRigVMTemplate* InTemplate, const FRigVMTemplateTypeMap& InPinTypeMap, URigVMNode* InNode)
{
	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::Unset, true);

	FRigVMDispatchContext DispatchContext;
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		DispatchContext = DispatchNode->GetDispatchContext();
	}

	auto AddExecutePins = [InTemplate, InNode, &Registry, &DispatchContext, this](ERigVMPinDirection InPinDirection)
	{
		for (int32 ArgIndex = 0; ArgIndex < InTemplate->NumExecuteArguments(DispatchContext); ArgIndex++)
		{
			const FRigVMExecuteArgument* Arg = InTemplate->GetExecuteArgument(ArgIndex, DispatchContext);
			if(Arg->Direction != InPinDirection)
			{
				continue;
			}
			
			URigVMPin* Pin = NewObject<URigVMPin>(InNode, Arg->Name);
			const FRigVMTemplateArgumentType Type = Registry.GetType(Arg->TypeIndex);
	        
			Pin->CPPType = Type.CPPType.ToString();
			Pin->CPPTypeObject = Type.CPPTypeObject;
			if (Pin->CPPTypeObject)
			{
				Pin->CPPTypeObjectPath = *Pin->CPPTypeObject->GetPathName();
			}
			Pin->Direction = Arg->Direction;
			Pin->LastKnownTypeIndex = Arg->TypeIndex;
			Pin->LastKnownCPPType = Pin->CPPType;

			AddNodePin(InNode, Pin);

			if(Registry.IsArrayType(Arg->TypeIndex))
			{
				if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Pin->GetNode()))
				{
					if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
					{
						const FString DefaultValue =  Factory->GetArgumentDefaultValue(Pin->GetFName(), Arg->TypeIndex);
						if(!DefaultValue.IsEmpty())
						{
							SetPinDefaultValue(Pin, DefaultValue, true, false, false);
						}
					}
				}
			}
		}
	};

	AddExecutePins(ERigVMPinDirection::IO);
	AddExecutePins(ERigVMPinDirection::Input);

	for (int32 ArgIndex = 0; ArgIndex < InTemplate->NumArguments(); ArgIndex++)
	{
		const FRigVMTemplateArgument* Arg = InTemplate->GetArgument(ArgIndex);

		URigVMPin* Pin = NewObject<URigVMPin>(InNode, Arg->GetName());
		const TRigVMTypeIndex& TypeIndex = InPinTypeMap.FindChecked(Arg->GetName());
		const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(TypeIndex);
		Pin->CPPType = Type.CPPType.ToString();
		Pin->CPPTypeObject = Type.CPPTypeObject;
		if (Pin->CPPTypeObject)
		{
			Pin->CPPTypeObjectPath = *Pin->CPPTypeObject->GetPathName();
		}
		Pin->Direction = Arg->GetDirection();

		Pin->bIsDynamicArray = FRigVMRegistry::Get().IsArrayType(TypeIndex);
		if (Pin->Direction == ERigVMPinDirection::Hidden)
		{
			if (InTemplate->GetArgumentMetaData(Arg->Name, FRigVMStruct::ArraySizeMetaName).IsEmpty())
			{
				Pin->bIsDynamicArray = true;
			}
		}

		if (Pin->bIsDynamicArray)
		{
			if (!InTemplate->GetArgumentMetaData(Arg->Name, FRigVMStruct::SingletonMetaName).IsEmpty())
			{
				Pin->bIsDynamicArray = false;
			}
		}

		if (Pin->Direction == ERigVMPinDirection::Input &&
			!InTemplate->GetArgumentMetaData(Arg->Name, FRigVMStruct::ComputeLazilyMetaName).IsEmpty())
		{
			Pin->bIsLazy = true;
		}
		
		if (Pin->Direction != ERigVMPinDirection::Output &&
			!InTemplate->GetArgumentMetaData(Arg->Name, FRigVMStruct::ConstantMetaName).IsEmpty())
		{
			Pin->bIsConstant = true;
		}

		AddNodePin(InNode, Pin);

		if(!Pin->IsWildCard() && !Pin->IsArray())
		{
			FString DefaultValue;
			if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InNode))
			{
				DefaultValue = TemplateNode->GetInitialDefaultValueForPin(Pin->GetFName());
			}

			TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
			if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Pin->CPPTypeObject))
			{
				AddPinsForStruct(ScriptStruct, Pin->GetNode(), Pin, Pin->Direction, DefaultValue, false);
			}
			else if(!DefaultValue.IsEmpty())
			{
				SetPinDefaultValue(Pin, DefaultValue, true, false, false);
			}
		}
		else if(Pin->IsFixedSizeArray())
		{
			if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Pin->GetNode()))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					const FString DefaultValue =  Factory->GetArgumentDefaultValue(Pin->GetFName(), RigVMTypeUtils::TypeIndex::WildCardArray);
					if(!DefaultValue.IsEmpty())
					{
						SetPinDefaultValue(Pin, DefaultValue, true, false, false);
					}
				}
			}
		}
	}

	AddExecutePins(ERigVMPinDirection::Output);

	InNode->IncrementVersion();
}

void URigVMController::ConfigurePinFromProperty(FProperty* InProperty, URigVMPin* InOutPin, ERigVMPinDirection InPinDirection) const
{
	if (InPinDirection == ERigVMPinDirection::Invalid)
	{
		InOutPin->Direction = FRigVMStruct::GetPinDirectionFromProperty(InProperty);
	}
	else
	{
		InOutPin->Direction = InPinDirection;
	}

	// If this property wants to be explicitly hidden, hide it
	if (InProperty->HasMetaData(FRigVMStruct::HiddenMetaName))
	{
		InOutPin->Direction = ERigVMPinDirection::Hidden;
	}

#if WITH_EDITOR

	if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(InProperty->GetOwnerStruct()))
	{
		const FString DisplayName = Struct->GetAuthoredNameForField(InProperty);
		InOutPin->DisplayName =  DisplayName.IsEmpty() ? NAME_None : FName(*DisplayName);
	}
	
	InOutPin->bIsConstant = InProperty->HasMetaData(TEXT("Constant"));
	InOutPin->bIsInputVariable = InProperty->HasMetaData(TEXT("ExternalVariable"));
	FString CustomWidgetName = InProperty->GetMetaData(TEXT("CustomWidget"));
	InOutPin->CustomWidgetName = CustomWidgetName.IsEmpty() ? FName(NAME_None) : FName(*CustomWidgetName);

	if (InProperty->HasMetaData(FRigVMStruct::ExpandPinByDefaultMetaName))
	{
		InOutPin->bIsExpanded = true;
	}

#endif

	InOutPin->CPPType = RigVMTypeUtils::GetCPPTypeFromProperty(InProperty);

	InOutPin->bIsDynamicArray = false;
	FProperty* PropertyForType = InProperty;
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyForType);
	if (ArrayProperty)
	{
		PropertyForType = ArrayProperty->Inner;
		InOutPin->bIsDynamicArray = true;
	}
#if WITH_EDITOR
	if (InOutPin->Direction == ERigVMPinDirection::Hidden)
	{
		if (!InProperty->HasMetaData(FRigVMStruct::ArraySizeMetaName))
		{
			InOutPin->bIsDynamicArray = true;
		}
	}

	if (InOutPin->bIsDynamicArray)
	{
		if (InProperty->HasMetaData(FRigVMStruct::SingletonMetaName))
		{
			InOutPin->bIsDynamicArray = false;
		}
	}

	if (InOutPin->Direction == ERigVMPinDirection::Input)
	{
		// fixed array elements are treated as lazy elements
		// if the original argument is also marked as lazy
		if (const URigVMPin* ParentPin = InOutPin->GetParentPin())
		{
			if (ParentPin->IsFixedSizeArray())
			{
				InOutPin->bIsLazy = ParentPin->IsLazy();
			}
		}

		if (InProperty->HasMetaData(FRigVMStruct::ComputeLazilyMetaName))
		{
			InOutPin->bIsLazy = true;
		}
	}
	
	if (InProperty->HasMetaData(FRigVMStruct::ConstantMetaName))
	{
		InOutPin->bIsConstant = true;
	}
#endif

	if (FStructProperty* StructProperty = CastField<FStructProperty>(PropertyForType))
	{
		InOutPin->CPPTypeObject = StructProperty->Struct;
	}
	else if (FClassProperty* ClassProperty = CastField<FClassProperty>(PropertyForType))
	{
		if(RigVMCore::SupportsUObjects())
		{
			InOutPin->CPPTypeObject = ClassProperty->MetaClass;
		}
		else
		{
			ReportErrorf(TEXT("Unsupported type '%s' for pin '%s'."), *ClassProperty->MetaClass->GetName(), *InOutPin->GetName());
			InOutPin->CPPType = FString();
			InOutPin->CPPTypeObject = nullptr;
		}
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(PropertyForType))
	{
		if(RigVMCore::SupportsUObjects())
		{
			InOutPin->CPPTypeObject = ObjectProperty->PropertyClass;
		}
		else
		{
			ReportErrorf(TEXT("Unsupported type '%s' for pin '%s'."), *ObjectProperty->PropertyClass->GetName(), *InOutPin->GetName());
			InOutPin->CPPType = FString();
			InOutPin->CPPTypeObject = nullptr;
		}
	}
	else if (FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(PropertyForType))
	{
		if (RigVMCore::SupportsUInterfaces())
		{
			InOutPin->CPPTypeObject = InterfaceProperty->InterfaceClass;
		}
		else
		{
			ReportErrorf(TEXT("Unsupported type '%s' for pin '%s'."), *InterfaceProperty->InterfaceClass->GetName(), *InOutPin->GetName());
			InOutPin->CPPType = FString();
			InOutPin->CPPTypeObject = nullptr;
		}
	}
	else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(PropertyForType))
	{
		InOutPin->CPPTypeObject = EnumProperty->GetEnum();
	}
	else if (FByteProperty* ByteProperty = CastField<FByteProperty>(PropertyForType))
	{
		InOutPin->CPPTypeObject = ByteProperty->Enum;
	}

	if (InOutPin->CPPTypeObject)
	{
		InOutPin->CPPTypeObjectPath = *InOutPin->CPPTypeObject->GetPathName();
	}

	InOutPin->CPPType = RigVMTypeUtils::PostProcessCPPType(InOutPin->CPPType, InOutPin->GetCPPTypeObject());

	if(InOutPin->IsExecuteContext() && InOutPin->CPPTypeObject != FRigVMExecuteContext::StaticStruct())
	{
		MakeExecutePin(InOutPin);
	}
}

void URigVMController::ConfigurePinFromPin(URigVMPin* InOutPin, const URigVMPin* InPin, bool bCopyDisplayName)
{
	// it is important we copy things that define the identity of the pin
	// things that defines the state of the pin is copied during GetPinState()
	// though addmittedly these two functions have overlaps currently
	InOutPin->bIsConstant = InPin->bIsConstant;
	InOutPin->Direction = InPin->Direction;
	InOutPin->CPPType = InPin->CPPType;
	InOutPin->CPPTypeObjectPath = InPin->CPPTypeObjectPath;
	InOutPin->CPPTypeObject = InPin->CPPTypeObject;
	InOutPin->DefaultValue = InPin->DefaultValue;
	InOutPin->DefaultValueObject = InPin->DefaultValueObject;
	InOutPin->bIsDynamicArray = InPin->bIsDynamicArray;
	InOutPin->bIsLazy = InPin->bIsLazy;
	InOutPin->bIsInputVariable = InPin->bIsInputVariable;
	if(bCopyDisplayName)
	{
		InOutPin->SetDisplayName(InPin->GetDisplayName());
	}

	if(InOutPin->IsExecuteContext() && InOutPin->CPPTypeObject != FRigVMExecuteContext::StaticStruct())
	{
		MakeExecutePin(InOutPin);
	}
}

void URigVMController::ConfigurePinFromArgument(URigVMPin* InOutPin, const FRigVMGraphFunctionArgument& InArgument, bool bCopyDisplayName)
{
	// it is important we copy things that define the identity of the pin
	// things that defines the state of the pin is copied during GetPinState()
	// though addmittedly these two functions have overlaps currently
	InOutPin->bIsConstant = InArgument.bIsConst;
	InOutPin->bIsInputVariable = InArgument.bIsInputVariable;
	InOutPin->Direction = InArgument.Direction;
	InOutPin->CPPType = InArgument.CPPType.ToString();
	InOutPin->CPPTypeObjectPath = *InArgument.CPPTypeObject.ToSoftObjectPath().ToString();
	InOutPin->CPPTypeObject = InArgument.CPPTypeObject.Get();
	if (InOutPin->IsUObject())
	{
		InOutPin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InArgument.DefaultValue);
		InOutPin->DefaultValue.Empty();
	}
	else
	{
		InOutPin->DefaultValueObject = nullptr;
		InOutPin->DefaultValue = InArgument.DefaultValue;
	}
	InOutPin->bIsDynamicArray = InArgument.bIsArray;
	if(bCopyDisplayName)
	{
		InOutPin->SetDisplayName(InArgument.DisplayName);
	}

	if(InOutPin->IsExecuteContext() && InOutPin->CPPTypeObject != FRigVMExecuteContext::StaticStruct())
	{
		MakeExecutePin(InOutPin);
	}
}

FProperty* URigVMController::FindPropertyForPin(const FString& InPinPath)
{
	if(!IsValidGraph())
	{
		return nullptr;
	}

	TArray<FString> Parts;
	if (!URigVMPin::SplitPinPath(InPinPath, Parts))
	{
		return nullptr;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	URigVMPin* Pin = Graph->FindPin(InPinPath);
	if (Pin == nullptr)
	{
		ReportErrorf(TEXT("Cannot find pin '%s'."), *InPinPath);
		return nullptr;
	}

	URigVMNode* Node = Pin->GetNode();

	URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node);
	if (UnitNode)
	{
		int32 PartIndex = 1; // cut off the first one since it's the node

		UStruct* Struct = UnitNode->GetScriptStruct();
		FProperty* Property = Struct->FindPropertyByName(*Parts[PartIndex++]);

		while (PartIndex < Parts.Num() && Property != nullptr)
		{
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Property = ArrayProperty->Inner;
				PartIndex++;
				continue;
			}

			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				Struct = StructProperty->Struct;
				Property = Struct->FindPropertyByName(*Parts[PartIndex++]);
				continue;
			}

			break;
		}

		if (PartIndex == Parts.Num())
		{
			return Property;
		}
	}

	return nullptr;
}

void URigVMController::RemoveStaleNodes()
{
	if (!IsValidGraph())
	{
		return;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	Graph->Nodes.Remove(nullptr);
}

void URigVMController::AddPinRedirector(bool bInput, bool bOutput, const FString& OldPinPath, const FString& NewPinPath)
{
	if (OldPinPath.IsEmpty() || NewPinPath.IsEmpty() || OldPinPath == NewPinPath)
	{
		return;
	}

	if (bInput)
	{
		InputPinRedirectors.FindOrAdd(OldPinPath) = NewPinPath;
	}
	if (bOutput)
	{
		OutputPinRedirectors.FindOrAdd(OldPinPath) = NewPinPath;
	}
}

void URigVMController::ClearPinRedirectors()
{
	InputPinRedirectors.Reset();
	OutputPinRedirectors.Reset();
}

#if WITH_EDITOR

bool URigVMController::ShouldRedirectPin(UScriptStruct* InOwningStruct, const FString& InOldRelativePinPath, FString& InOutNewRelativePinPath) const
{
	if(InOwningStruct == nullptr) // potentially a template node
	{
		return false;
	}
	
	FRigVMStructPinRedirectorKey RedirectorKey(InOwningStruct, InOldRelativePinPath);
	if (const FString* RedirectedPinPath = PinPathCoreRedirectors.Find(RedirectorKey))
	{
		InOutNewRelativePinPath = *RedirectedPinPath;
		return InOutNewRelativePinPath != InOldRelativePinPath;
	}

	FString RelativePinPath = InOldRelativePinPath;
	FString PinName, SubPinPath;
	if (!URigVMPin::SplitPinPathAtStart(RelativePinPath, PinName, SubPinPath))
	{
		PinName = RelativePinPath;
		SubPinPath.Empty();
	}

	bool bShouldRedirect = false;
	FCoreRedirectObjectName OldObjectName(*PinName, InOwningStruct->GetFName(), *InOwningStruct->GetOutermost()->GetPathName());
	FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Property, OldObjectName);
	if (OldObjectName != NewObjectName)
	{
		PinName = NewObjectName.ObjectName.ToString();
		bShouldRedirect = true;
	}

	FProperty* Property = InOwningStruct->FindPropertyByName(*PinName);
	if (Property == nullptr)
	{
		return false;
	}

	if (!SubPinPath.IsEmpty())
	{
		if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			FString NewSubPinPath;
			if (ShouldRedirectPin(StructProperty->Struct, SubPinPath, NewSubPinPath))
			{
				SubPinPath = NewSubPinPath;
				bShouldRedirect = true;
			}
		}
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FString SubPinName, SubSubPinPath;
			if (URigVMPin::SplitPinPathAtStart(SubPinPath, SubPinName, SubSubPinPath))
			{
				if (FStructProperty* InnerStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
				{
					FString NewSubSubPinPath;
					if (ShouldRedirectPin(InnerStructProperty->Struct, SubSubPinPath, NewSubSubPinPath))
					{
						SubSubPinPath = NewSubSubPinPath;
						SubPinPath = URigVMPin::JoinPinPath(SubPinName, SubSubPinPath);
						bShouldRedirect = true;
					}
				}
			}
		}
	}

	if (bShouldRedirect)
	{
		if (SubPinPath.IsEmpty())
		{
			InOutNewRelativePinPath = PinName;
			PinPathCoreRedirectors.Add(RedirectorKey, InOutNewRelativePinPath);
		}
		else
		{
			InOutNewRelativePinPath = URigVMPin::JoinPinPath(PinName, SubPinPath);

			TArray<FString> OldParts, NewParts;
			if (URigVMPin::SplitPinPath(InOldRelativePinPath, OldParts) &&
				URigVMPin::SplitPinPath(InOutNewRelativePinPath, NewParts))
			{
				ensure(OldParts.Num() == NewParts.Num());

				FString OldPath = OldParts[0];
				FString NewPath = NewParts[0];
				for (int32 PartIndex = 0; PartIndex < OldParts.Num(); PartIndex++)
				{
					if (PartIndex > 0)
					{
						OldPath = URigVMPin::JoinPinPath(OldPath, OldParts[PartIndex]);
						NewPath = URigVMPin::JoinPinPath(NewPath, NewParts[PartIndex]);
					}

					// this is also going to cache paths which haven't been redirected.
					// consumers of the table have to still compare old != new
					FRigVMStructPinRedirectorKey SubRedirectorKey(InOwningStruct, OldPath);
					if (!PinPathCoreRedirectors.Contains(SubRedirectorKey))
					{
						PinPathCoreRedirectors.Add(SubRedirectorKey, NewPath);
					}
				}
			}
		}
	}

	return bShouldRedirect;
}

bool URigVMController::ShouldRedirectPin(const FString& InOldPinPath, FString& InOutNewPinPath) const
{
	URigVMGraph* Graph = GetGraph();
	check(Graph);

	FString PinPathInNode, NodeName;
	URigVMPin::SplitPinPathAtStart(InOldPinPath, NodeName, PinPathInNode);

	URigVMNode* Node = Graph->FindNode(NodeName);
	if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
	{
		FString NewPinPathInNode;
		if (ShouldRedirectPin(UnitNode->GetScriptStruct(), PinPathInNode, NewPinPathInNode))
		{
			InOutNewPinPath = URigVMPin::JoinPinPath(NodeName, NewPinPathInNode);
			return true;
		}
	}
	else if (URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(Node))
	{
		URigVMPin* ValuePin = RerouteNode->Pins[0];
		if (ValuePin->IsStruct())
		{
			FString ValuePinPath = ValuePin->GetPinPath();
			if (InOldPinPath == ValuePinPath)
			{
				return false;
			}
			else if (!InOldPinPath.StartsWith(ValuePinPath))
			{
				return false;
			}

			FString PinPathInStruct, NewPinPathInStruct;
			if (URigVMPin::SplitPinPathAtStart(PinPathInNode, NodeName, PinPathInStruct))
			{
				if (ShouldRedirectPin(ValuePin->GetScriptStruct(), PinPathInStruct, NewPinPathInStruct))
				{
					InOutNewPinPath = URigVMPin::JoinPinPath(ValuePin->GetPinPath(), NewPinPathInStruct);
					return true;
				}
			}
		}
	}

	return false;
}

void URigVMController::GenerateRepopulatePinsNodeData(TArray<FRepopulatePinsNodeData>& NodesPinData, URigVMNode* InNode, bool bInFollowCoreRedirectors, bool bInSetupOrphanedPins, bool bInRecreateLinks, bool bInRecurseIntoCollapseNodes)
{
	if (InNode == nullptr)
	{
		ReportError(TEXT("InNode is nullptr."));
		return;
	}

	FRigVMControllerCompileBracketScope CompileBracketScope(this);
	UE::TScopeLock Lock(PinPathCoreRedirectorsLock);

	FRepopulatePinsNodeData NodeData;
	NodeData.Node = InNode;
	NodeData.bFollowCoreRedirectors = bInFollowCoreRedirectors;
	NodeData.bRecreateLinks = bInRecreateLinks;
	NodeData.bSetupOrphanPinsForThisNode = bInSetupOrphanedPins;

	URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode);
	if (CollapseNode)
	{
		if (CollapseNode->IsGraphFunctionDefinition())
		{
			NodeData.bSetupOrphanPinsForThisNode = false;
		}
	}

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();

	NodeData.PreviousPinInfos = FRigVMPinInfoArray(InNode, this);
	NodeData.PreviousPinHash = GetTypeHash(NodeData.PreviousPinInfos);
	if (!GenerateNewPinInfos(Registry, InNode, NodeData.PreviousPinInfos, NodeData.NewPinInfos, NodeData.bSetupOrphanPinsForThisNode))
	{
		return; // skip this node if no Infos can be generated
	}

	NodeData.bRequireRecreateLinks = false;
	NodeData.bRequirePinStates = false;

	GenerateRepopulatePinLists(Registry, NodeData);

	NodesPinData.Add(NodeData);

	// Recurse if a collapse node. Callers that only need the top-level node's pin info pass
	// bInRecurseIntoCollapseNodes=false to skip this — recursing would otherwise call GetControllerForGraph
	// on the contained graph, which routes through FRigVMClient::GetOrCreateController and creates a cached
	// controller in the asset's Controllers map.
	if (CollapseNode != nullptr && bInRecurseIntoCollapseNodes)
	{
		if (URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
		{
			TGuardValue<bool> GuardEditGraph(CollapseNode->ContainedGraph->bEditable, true);
			// need to get a copy of the node array since the following function could remove nodes from the graph
			// we don't want to remove elements from the array we are iterating over.
			TArray<URigVMNode*> ContainedNodes = CollapseNode->GetContainedNodes();
			for (URigVMNode* ContainedNode : ContainedNodes)
			{
				CollapseController->GenerateRepopulatePinsNodeData(NodesPinData, ContainedNode, bInFollowCoreRedirectors, false, false, bInRecurseIntoCollapseNodes);
			}
		}
	}
}

void URigVMController::OrphanPins(const TArray<FRepopulatePinsNodeData>& NodesPinData)
{
	FRigVMControllerCompileBracketScope CompileBracketScope(this);
	UE::TScopeLock Lock(PinPathCoreRedirectorsLock);

	for (const FRepopulatePinsNodeData& NodeData : NodesPinData)
	{
		if (NodeData.Node == nullptr)
		{
			ReportError(TEXT("InNode is nullptr orphaning pins."));
			continue;
		}

		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		
		// orphan pins
		for (int32 Index = 0; Index < NodeData.PreviousPinsToOrphan.Num(); Index++)
		{
			const FString& PinPath = NodeData.PreviousPinInfos.GetPinPath(NodeData.PreviousPinsToOrphan[Index]);
			if (URigVMPin* Pin = NodeData.Node->FindPin(PinPath))
			{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
				UE_LOGF(LogRigVMDeveloper, Display, "Orphaning pin '%ls'.", *PinPath);
#endif
				const bool bIsTraitPin = Pin->GetRootPin()->IsTraitPin();
				check(Pin->IsRootPin() || bIsTraitPin);

				//const FString OrphanedName = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *Pin->GetName());
				const FString OrphanedName = bIsTraitPin
					? FString::Printf(TEXT("%s%s.%s"), URigVMPin::OrphanPinPrefix, *Pin->GetParentPin()->GetName(), *Pin->GetName())
					: FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *Pin->GetName());

				if (!NodeData.Node->FindPin(OrphanedName))
				{
					URigVMPin* OrphanPin = NewObject<URigVMPin>(NodeData.Node, *OrphanedName);
					ConfigurePinFromPin(OrphanPin, Pin, true);

					for (URigVMPin* SubPin : Pin->SubPins)
					{
						const FString SubPinName = SubPin->GetName();
						URigVMPin* OrphanedSubPin = NewObject<URigVMPin>(OrphanPin, *SubPinName);
						ConfigurePinFromPin(OrphanedSubPin, SubPin, true);
						OrphanPin->SubPins.Add(OrphanedSubPin);
					}

					NodeData.Node->OrphanedPins.Add(OrphanPin);
				}
			}
		}
	}
}

void URigVMController::RepopulatePins(const TArray<FRepopulatePinsNodeData>& NodesPinData)
{
	FRigVMControllerCompileBracketScope CompileBracketScope(this);
	UE::TScopeLock Lock(PinPathCoreRedirectorsLock);

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	for (const FRepopulatePinsNodeData& NodeData : NodesPinData)
	{
		RepopulatePinsOnNode(Registry, NodeData);
	}
}

void URigVMController::RepopulatePinsOnNode(URigVMNode* InNode, bool bFollowCoreRedirectors, bool bSetupOrphanedPins, bool bRecreateLinks)
{
	if (InNode == nullptr)
	{
		ReportError(TEXT("InNode is nullptr."));
		return;
	}

	FRigVMControllerCompileBracketScope CompileBracketScope(this);
	UE::TScopeLock Lock(PinPathCoreRedirectorsLock);

	TArray<FRepopulatePinsNodeData> NodesPinData;
	GenerateRepopulatePinsNodeData(NodesPinData, InNode, bFollowCoreRedirectors, bSetupOrphanedPins, bRecreateLinks);
	RepopulatePins(NodesPinData);
}

bool URigVMController::GenerateNewPinInfos(const FRigVMRegistry& Registry, URigVMNode* InNode, const FRigVMPinInfoArray& PreviousPinInfos, FRigVMPinInfoArray& NewPinInfos, const bool bSetupOrphanPinsForThisNode)
{
	URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode);
	URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(InNode);
	URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(InNode);
	URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(InNode);
	URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode);
	URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InNode);
	URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode);
	URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode);

	// step 2/3: clear pins on the node and repopulate the node with new pins
	if (UnitNode != nullptr)
	{
		UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct();
		if (ScriptStruct == nullptr)
		{
			// this may be an unresolved template node
			// in that case there's nothing we can do here
			return false;
		}

		// Only update the node color from struct metadata if it is indicated that the color still comes from there
		if(UnitNode->NodeColorType == ERigVMNodeColorType::FromMetadata)
		{
			FString NodeColorMetadata;
			ScriptStruct->GetStringMetaDataHierarchical(URigVMNode::NodeColorName, &NodeColorMetadata);
			if (!NodeColorMetadata.IsEmpty())
			{
				UnitNode->NodeColor = GetColorFromMetadata(NodeColorMetadata);
			}
		}

		const TSharedPtr<FStructOnScope> DefaultValueContent = UnitNode->ConstructStructInstance(false);
		const TFunction<ERigVMPinDefaultValueType(const FName&)> DefaultValueTypeGetter = [InNode, this](const FName& InPropertyName)
		{
			if(const URigVMPin* Pin = InNode->FindPin(InPropertyName.ToString()))
			{
				return GetDefaultValueType(Pin, Pin->GetDefaultValue());
			}
			return ERigVMPinDefaultValueType::AutoDetect;
		};
		NewPinInfos.AddPins(ScriptStruct, this, ERigVMPinDirection::Invalid, INDEX_NONE, DefaultValueTypeGetter, DefaultValueContent->GetStructMemory(), true);
	}
	else if (DispatchNode)
	{
		TMap<FName, TRigVMTypeIndex> PinTypeMap;
		for (const URigVMPin* Pin : DispatchNode->Pins)
		{
			PinTypeMap.Add(Pin->GetFName(), Pin->GetTypeIndex());
		}

		const FRigVMTemplate* Template = DispatchNode->GetTemplate();
		if (!Template)
		{
			return false;
		}

		FRigVMDispatchContext DispatchContext = DispatchNode->GetDispatchContext();
		auto AddExecutePins = [Template, DispatchNode, &Registry, &DispatchContext, &NewPinInfos, &PreviousPinInfos, this](ERigVMPinDirection InPinDirection)
			{
				for (int32 ArgIndex = 0; ArgIndex < Template->NumExecuteArguments(DispatchContext); ArgIndex++)
				{
					const FRigVMExecuteArgument* Arg = Template->GetExecuteArgument(ArgIndex, DispatchContext);
					if (Arg->Direction == InPinDirection)
					{
						const FRigVMTemplateArgumentType Type = Registry.GetType(Arg->TypeIndex);
						const TRigVMTypeIndex TypeIndex = Registry.GetTypeIndex(Type);

						FString DefaultValue;
						if (Registry.IsArrayType(Arg->TypeIndex))
						{
							if (const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
							{
								DefaultValue = Factory->GetArgumentDefaultValue(Arg->Name, Arg->TypeIndex);
							}
						}

						(void)NewPinInfos.AddPin(this, INDEX_NONE, Arg->Name, Arg->Direction, TypeIndex, DefaultValue, ERigVMPinDefaultValueType::Unset, nullptr, &PreviousPinInfos, true);
					}
				}
			};

		AddExecutePins(ERigVMPinDirection::IO);
		AddExecutePins(ERigVMPinDirection::Input);

		for (int32 ArgIndex = 0; ArgIndex < Template->NumArguments(); ArgIndex++)
		{
			const FRigVMTemplateArgument* Arg = Template->GetArgument(ArgIndex);

			TRigVMTypeIndex TypeIndex = INDEX_NONE;
			if (const TRigVMTypeIndex* ExistingTypeIndex = PinTypeMap.Find(Arg->GetName()))
			{
				TypeIndex = *ExistingTypeIndex;
				if (!Arg->SupportsTypeIndex(TypeIndex))
				{
					TypeIndex = INDEX_NONE;
				}
			}

			if (TypeIndex == INDEX_NONE)
			{
				if (Arg->IsSingleton())
				{
					TypeIndex = Arg->GetSupportedTypeIndices()[0];
				}
				else if (Arg->GetArrayType() == FRigVMTemplateArgument::EArrayType_ArrayValue)
				{
					TypeIndex = RigVMTypeUtils::TypeIndex::WildCardArray;
				}
				else
				{
					TypeIndex = RigVMTypeUtils::TypeIndex::WildCard;
				}
			}

			FString DefaultValue;
			UScriptStruct* ArgumentScriptStruct = nullptr;
			const uint8* DefaultValueMemory = nullptr;

			if (const URigVMPin* ArgumentPin = DispatchNode->FindPin(Arg->Name.ToString()))
			{
				DefaultValue = ArgumentPin->GetDefaultValue();
				ArgumentScriptStruct = Cast<UScriptStruct>(ArgumentPin->GetCPPTypeObject());
			}
			else if (const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
			{
				if (Arg->IsSingleton())
				{
					const TRigVMTypeIndex Type0 = Arg->GetTypeIndex(0);
					DefaultValue = Factory->GetArgumentDefaultValue(Arg->Name, Type0);
					const FRigVMTemplateArgumentType& Type = Registry.GetType(Type0);
					ArgumentScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject);
				}
			}

			FStructOnScope DefaultValueMemoryScope; // has to be in this scope so that DefaultValueMemory is valid
			if (ArgumentScriptStruct && !DefaultValue.IsEmpty())
			{
				DefaultValueMemoryScope = FStructOnScope(ArgumentScriptStruct);

				FRigVMPinDefaultValueImportErrorContext ErrorPipe;
				ArgumentScriptStruct->ImportText(*DefaultValue, DefaultValueMemoryScope.GetStructMemory(), nullptr, PPF_None, &ErrorPipe, FString());
				DefaultValueMemory = DefaultValueMemoryScope.GetStructMemory();
			}

			bool bAddSubPinsForArgument = true;
#if WITH_EDITOR
			if (!Template->GetArgumentMetaData(Arg->Name, FRigVMStruct::HideSubPinsMetaName).IsEmpty())
			{
				bAddSubPinsForArgument = false;
			}
#endif
			ERigVMPinDefaultValueType DefaultValueType = ERigVMPinDefaultValueType::AutoDetect;
			if(const FRigVMPinInfo* PinInfo = PreviousPinInfos.GetPinFromPinPath(Arg->Name.ToString()))
			{
				DefaultValueType = PinInfo->DefaultValueType;
			}
			(void)NewPinInfos.AddPin(this, INDEX_NONE, Arg->Name, Arg->GetDirection(), TypeIndex, DefaultValue, DefaultValueType, DefaultValueMemory, &PreviousPinInfos, bAddSubPinsForArgument);
		}

		AddExecutePins(ERigVMPinDirection::Output);
	}
	else if ((RerouteNode != nullptr) || (VariableNode != nullptr))
	{
		if (InNode->GetPins().Num() == 0)
		{
			return false;
		}

		URigVMPin* ValuePin = nullptr;
		if (RerouteNode)
		{
			ValuePin = RerouteNode->Pins[0];
		}
		else
		{
			ValuePin = VariableNode->FindPin(URigVMVariableNode::ValueName);
		}
		check(ValuePin);
		EnsurePinValidity(ValuePin, false);

		if (VariableNode)
		{
			// this includes local variables for validation
			const TArray<FRigVMExternalVariable> ExternalVariables = GetAllVariables(false);
			const FRigVMGraphVariableDescription VariableDescription = VariableNode->GetVariableDescription();
			const FRigVMExternalVariable CurrentExternalVariable = VariableDescription.ToExternalVariable();

			FRigVMExternalVariable Variable;
			if (VariableNode->IsInputArgument())
			{
				URigVMGraph* Graph = GetGraph();
				check(Graph);

				if (URigVMFunctionEntryNode* GraphEntryNode = Graph->GetEntryNode())
				{
					if (URigVMPin* EntryPin = GraphEntryNode->FindPin(VariableDescription.Name.ToString()))
					{
						Variable = RigVMTypeUtils::ExternalVariableFromCPPType(VariableDescription.Guid, VariableDescription.Name, EntryPin->GetCPPType(), EntryPin->GetCPPTypeObject());
					}
				}
			}
			else
			{
				for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
				{
					if (ExternalVariable.GetGuid().IsValid() && CurrentExternalVariable.GetGuid().IsValid())
					{
						if (ExternalVariable.GetGuid() == CurrentExternalVariable.GetGuid())
						{
							Variable = ExternalVariable;
							break;
						}
					}
					else
					{
						if (ExternalVariable.GetName() == CurrentExternalVariable.GetName())
						{
							Variable = ExternalVariable;
							break;
						}
					}
				}
			}

			if (Variable.IsValid(true))
			{
				if (!VariableNode->IsGetter())
				{
					(void) NewPinInfos.AddPin(VariableNode->FindExecutePin(), INDEX_NONE, ERigVMPinDirection::IO, ERigVMPinDefaultValueType::AutoDetect);
				}
				URigVMPin* NamePin = VariableNode->GetVariableNamePin();
				(void) NewPinInfos.AddPin(NamePin, INDEX_NONE, NamePin->GetDirection(), NamePin->GetDefaultValueType());
				(void) NewPinInfos.AddPin(this, INDEX_NONE, URigVMVariableNode::ValueName, ValuePin->GetDirection(), Variable.GetTypeIndex(), ValuePin->GetDefaultValue(), ValuePin->GetDefaultValueType(), nullptr, &PreviousPinInfos, true);
			}
			else
			{
				ReportWarningf(
					TEXT("Control Rig '%s', Variable '%s' not found."),
					*InNode->GetOutermost()->GetPathName(),
					*CurrentExternalVariable.GetName().ToString()
				);
			}
		}
		else
		{
			NewPinInfos = FRigVMPinInfoArray(InNode, this, &PreviousPinInfos);
		}

	}
	else if (EntryNode || ReturnNode)
	{
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode->GetGraph()->GetOuter()))
		{
			bool bIsEntryNode = EntryNode != nullptr;

			TArray<URigVMPin*> SortedLibraryPins;

			// add execute pins first
			bool bHasExecutePin = false;
			for (URigVMPin* LibraryPin : LibraryNode->GetPins())
			{
				if (LibraryPin->IsExecuteContext())
				{
					SortedLibraryPins.Add(LibraryPin);
					bHasExecutePin = true;
				}
			}

			// For function definitions (pure functions), entry/return nodes need execute pins
			// for internal sequencing even if the external interface has no execute pin.
			if (!bHasExecutePin)
			{
				if (URigVMCollapseNode* OuterCollapseNode = Cast<URigVMCollapseNode>(LibraryNode))
				{
					if (OuterCollapseNode->IsGraphFunctionDefinition())
					{
						// Add an execute pin for internal sequencing
						const TRigVMTypeIndex ExecuteTypeIndex = RigVMTypeUtils::TypeIndex::Execute;
						ERigVMPinDirection Direction = bIsEntryNode ? ERigVMPinDirection::Output : ERigVMPinDirection::Input;

						(void)NewPinInfos.AddPin(this, INDEX_NONE, FRigVMStruct::ExecuteContextName, Direction, ExecuteTypeIndex, FString(), ERigVMPinDefaultValueType::Unset, nullptr, &PreviousPinInfos, false);
					}
				}
			}

			// add remaining pins
			for (URigVMPin* LibraryPin : LibraryNode->GetPins())
			{
				SortedLibraryPins.AddUnique(LibraryPin);
			}

			for (URigVMPin* LibraryPin : SortedLibraryPins)
			{
				if (LibraryPin->GetDirection() == ERigVMPinDirection::IO && !LibraryPin->IsExecuteContext())
				{
					continue;
				}

				if (bIsEntryNode)
				{
					if (LibraryPin->GetDirection() == ERigVMPinDirection::Output)
					{
						continue;
					}
				}
				else
				{
					if (LibraryPin->GetDirection() == ERigVMPinDirection::Input)
					{
						continue;
					}
				}

				ERigVMPinDirection Direction = bIsEntryNode ? ERigVMPinDirection::Output : ERigVMPinDirection::Input;
				const int32 NewPinIndex = NewPinInfos.AddPin(LibraryPin, INDEX_NONE, Direction, LibraryPin->GetDefaultValueType());

				if (LibraryPin->IsRootPin() && LibraryPin->GetSubPins().IsEmpty() && LibraryPin->IsDefinedAsInputVariable() &&
					LibraryPin->IsStruct() && !NewPinInfos[NewPinIndex].ShouldHideSubPins())
				{
					NewPinInfos.AddPins(Cast<UScriptStruct>(LibraryPin->GetCPPTypeObject()), this, Direction, NewPinIndex, nullptr, nullptr, true);
				}
			}
		}
		else
		{
			// due to earlier bugs with copy and paste we can find entry and return nodes under the top level
			// graph. we'll ignore these for now.
		}
	}
	else if (CollapseNode)
	{
		NewPinInfos = FRigVMPinInfoArray(InNode, this, &PreviousPinInfos);
	}
	else if (FunctionRefNode)
	{
		FunctionRefNode->UpdateFunctionHeaderFromHost();
		const FRigVMGraphFunctionHeader& FunctionHeader = FunctionRefNode->GetReferencedFunctionHeader();
		if (FunctionHeader.IsValid())
		{
			NewPinInfos = FRigVMPinInfoArray(FunctionHeader, this, &PreviousPinInfos);
		}
		// if we can't find referenced node anymore
		// let's keep all pins
		else
		{
			NewPinInfos = FRigVMPinInfoArray(FunctionRefNode, this, &PreviousPinInfos);
		}
	}
	else
	{
		return false;
	}

	// make sure the new pin infos contains the trait pins from the last run
	for (int32 Index = 0; Index < PreviousPinInfos.Num(); Index++)
	{
		const FRigVMPinInfo& PreviousPin = PreviousPinInfos[Index];
		if (PreviousPin.bIsTrait)
		{
			const bool bAddSubPins = !PreviousPin.ShouldHideSubPins();
			const int32 NewPinIndex = NewPinInfos.AddPin(this, INDEX_NONE, PreviousPin.Name, PreviousPin.Direction, PreviousPin.TypeIndex, PreviousPin.DefaultValue, PreviousPin.DefaultValueType, nullptr, &PreviousPinInfos, bAddSubPins);
			NewPinInfos[NewPinIndex].bIsTrait = true;

			if (URigVMPin* Pin = InNode->FindPin(PreviousPin.PinPath))
			{
				TSharedPtr<FStructOnScope> TraitScope = Pin->GetTraitInstance();
				FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();

				VMTrait->GetProgrammaticPins(this, NewPinIndex, Pin, Pin->GetDefaultValue(), NewPinInfos);
			}
		}
	}

	return true;
}

void URigVMController::GenerateRepopulatePinLists(const FRigVMRegistry& Registry, FRepopulatePinsNodeData& NodeData)
{
	URigVMNode* InNode = NodeData.Node;

	for (int32 Index = 0; Index < NodeData.PreviousPinInfos.Num(); Index++)
	{
		const FString PinPath = NodeData.PreviousPinInfos.GetPinPath(Index);
		const int32 NewIndex = NodeData.NewPinInfos.GetIndexFromPinPath(PinPath);
		int32 RootIndex = NodeData.PreviousPinInfos.GetRootIndex(Index);

		if (NewIndex == INDEX_NONE)
		{
			if (NodeData.PreviousPinInfos[Index].Direction != ERigVMPinDirection::Hidden)
			{
				if (URigVMPin* Pin = InNode->FindPin(PinPath))
				{
					const bool bIsTraitPin = Pin->GetRootPin()->IsTraitPin();
					if (bIsTraitPin)
					{
						RootIndex = Index;
					}
					if (!Pin->GetLinks().IsEmpty() || !Pin->InjectionInfos.IsEmpty())
					{
						NodeData.bRequireRecreateLinks = true;
						NodeData.bRequirePinStates = true;
					}

					if (NodeData.bSetupOrphanPinsForThisNode)
					{
						if (!NodeData.PreviousPinsToOrphan.Contains(RootIndex))
						{
							URigVMPin* RootPin = bIsTraitPin ? Pin : Pin->GetRootPin();

							if (RootPin->GetSourceLinks(true).Num() > 0 ||
								RootPin->GetTargetLinks(true).Num() > 0)
							{
								NodeData.PreviousPinsToOrphan.Add(RootIndex);

								NodeData.bRequireRecreateLinks = true;
								NodeData.bRequirePinStates = true;
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
								UE_LOGF(LogRigVMDeveloper, Display, "Previously existing pin '%ls' needs to be orphaned.", *RootPin->GetPinPath());
#endif
							}
						}
					}
				}
			}

			if (!NodeData.PreviousPinsToOrphan.Contains(RootIndex))
			{
				NodeData.PreviousPinsToRemove.Add(Index);
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
				UE_LOGF(LogRigVMDeveloper, Display, "Previously existing pin '%ls' is now obsolete.", *PinPath);
#endif
			}
		}
		else if (GetTypeHash(NodeData.PreviousPinInfos[Index]) != GetTypeHash(NodeData.NewPinInfos[NewIndex]))
		{
			const bool bTypesDiffer = !Registry.CanMatchTypes(NodeData.PreviousPinInfos[Index].TypeIndex, NodeData.NewPinInfos[NewIndex].TypeIndex, true);
			if (NodeData.PreviousPinInfos[Index].Direction != NodeData.NewPinInfos[NewIndex].Direction)
			{
				NodeData.bRequireRecreateLinks = true;
			}
			else if (NodeData.PreviousPinInfos[Index].bIsInputVariable != NodeData.NewPinInfos[NewIndex].bIsInputVariable)
			{
				NodeData.bRequireRecreateLinks = true;
			}
			else if (NodeData.PreviousPinInfos[Index].Direction != ERigVMPinDirection::Hidden)
			{
				NodeData.bRequireRecreateLinks |= bTypesDiffer;
			}

			if (Registry.CanMatchTypes(NodeData.PreviousPinInfos[Index].TypeIndex, NodeData.NewPinInfos[NewIndex].TypeIndex, true))
			{
				NodeData.PreviousPinsToUpdate.Add(Index);
			}
			else
			{
				NodeData.PreviousPinsToRemove.Add(Index);
				NodeData.NewPinsToAdd.Add(NewIndex);

				// we also need to setup orphan pins if a pin CPP type has changed
				if (URigVMPin* Pin = InNode->FindPin(PinPath))
				{
					// Preserve pin state when the pin is being recreated rather
					// than updated in-place - injection infos are owned by the
					// old pin UObject and would otherwise be lost.
					if (!Pin->InjectionInfos.IsEmpty())
					{
						NodeData.bRequirePinStates = true;
					}

					if (!Pin->GetLinks().IsEmpty())
					{
						if (NodeData.bSetupOrphanPinsForThisNode)
						{
							if (!NodeData.PreviousPinsToOrphan.Contains(RootIndex))
							{
								URigVMPin* RootPin = Pin->GetRootPin();

								if (RootPin->GetSourceLinks(true).Num() > 0 ||
									RootPin->GetTargetLinks(true).Num() > 0)
								{
									NodeData.PreviousPinsToOrphan.Add(RootIndex);

									NodeData.bRequireRecreateLinks = true;
									NodeData.bRequirePinStates = true;
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
									UE_LOGF(LogRigVMDeveloper, Display, "Previously existing pin '%ls' needs to be orphaned.", *RootPin->GetPinPath());
#endif
								}
							}
						}

					}
				}
			}

#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
			const FName& PreviousCPPType = Registry.GetType(NodeData.PreviousPinInfos[Index].TypeIndex).CPPType;
			const FName& NewCPPType = Registry.GetType(NodeData.NewPinInfos[NewIndex].TypeIndex).CPPType;

			const FString PreviousDirection = StaticEnum<ERigVMPinDirection>()->GetDisplayNameTextByValue((int64)NodeData.PreviousPinInfos[Index].Direction).ToString();
			const FString NewDirection = StaticEnum<ERigVMPinDirection>()->GetDisplayNameTextByValue((int64)NodeData.NewPinInfos[NewIndex].Direction).ToString();

			UE_LOGF(LogRigVMDeveloper, Display,
				"Previous pin '%ls' (Index %d, %ls, %ls) differs with new pin (Index %d, %ls, %ls).",
				*PinPath,
				Index,
				*PreviousCPPType.ToString(),
				*PreviousDirection,
				NewIndex,
				*NewCPPType.ToString(),
				*NewDirection
			);
#endif
		}
	}
	for (int32 Index = 0; Index < NodeData.NewPinInfos.Num(); Index++)
	{
		const FString PinPath = NodeData.NewPinInfos.GetPinPath(Index);
		const int32 PreviousIndex = NodeData.PreviousPinInfos.GetIndexFromPinPath(PinPath);
		if (PreviousIndex == INDEX_NONE)
		{
			NodeData.NewPinsToAdd.Add(Index);
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
			UE_LOGF(LogRigVMDeveloper, Display, "Newly required pin '%ls' needs to be added.", *PinPath);
#endif
		}
		else
		{
			// the previous pin exists - but it has been orphaned
			const int32 PreviousRootIndex = NodeData.PreviousPinInfos.GetRootIndex(PreviousIndex);
			if (NodeData.PreviousPinsToOrphan.Contains(PreviousRootIndex))
			{
				NodeData.NewPinsToAdd.AddUnique(Index);

#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
				UE_LOGF(LogRigVMDeveloper, Display, "Orphaned pin '%ls' needs to be re-added.", *PinPath);
#endif
			}
		}
	}
}

void URigVMController::RepopulatePinsOnNode(const FRigVMRegistry& Registry, const FRepopulatePinsNodeData& NodeData)
{
	URigVMNode* InNode = NodeData.Node;

	if (InNode == nullptr)
	{
		ReportError(TEXT("InNode is nullptr repopulating pins."));
		return;
	}

	// step 0/3: update execute pins
	for (URigVMPin* Pin : InNode->Pins)
	{
		if (Pin->IsExecuteContext())
		{
			MakeExecutePin(Pin);
		}
	}

	// if the nodes does not match in structure repopulate
	if (GetTypeHash(NodeData.NewPinInfos) == NodeData.PreviousPinHash)
	{
		return;
	}

	URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode);
	URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InNode);
	URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode);

	// step 1/3: keep a record of the current state of the node's pins
	TMap<FString, FString> RedirectedPinPaths;
	if (NodeData.bFollowCoreRedirectors)
	{
		RedirectedPinPaths = GetRedirectedPinPaths(InNode);
	}

	// also in case this node is part of an injection
	FName InjectionInputPinName = NAME_None;
	FName InjectionOutputPinName = NAME_None;
	if (URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo())
	{
		InjectionInputPinName = InjectionInfo->InputPin ? InjectionInfo->InputPin->GetFName() : NAME_None;
		InjectionOutputPinName = InjectionInfo->OutputPin ? InjectionInfo->OutputPin->GetFName() : NAME_None;
	}

	TMap<FString, FPinState> PinStates;
	TArray<FLinkedPath> LinkedPaths;

	if (NodeData.bRequirePinStates)
	{
		PinStates = GetPinStates(InNode);
	}

	if (NodeData.bRecreateLinks && NodeData.bRequireRecreateLinks)
	{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
		UE_LOGF(LogRigVMDeveloper, Display, "Detaching links of node %ls.", *InNode->GetPathName());
#endif
		LinkedPaths = GetLinkedPaths(InNode);
		FastBreakLinkedPaths(LinkedPaths);
	}

	// we can do a simpler version of the update and simply
	// add new pins, remove obsolete pins and change types as needed 
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
	UE_LOGF(LogRigVMDeveloper, Display, "Performing fast update of node %ls ...", *InNode->GetPathName());
#endif

	// orphan pins
	for (int32 Index = 0; Index < NodeData.PreviousPinsToOrphan.Num(); Index++)
	{
		const FString& PinPath = NodeData.PreviousPinInfos.GetPinPath(NodeData.PreviousPinsToOrphan[Index]);
		if (URigVMPin* Pin = InNode->FindPin(PinPath))
		{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
			UE_LOGF(LogRigVMDeveloper, Display, "Orphaning pin '%ls'.", *PinPath);
#endif
			const bool bIsTraitPin = Pin->GetRootPin()->IsTraitPin();
			check(Pin->IsRootPin() || bIsTraitPin);

			//const FString OrphanedName = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *Pin->GetName());
			const FString OrphanedName = bIsTraitPin
				? FString::Printf(TEXT("%s%s.%s"), URigVMPin::OrphanPinPrefix, *Pin->GetParentPin()->GetName(), *Pin->GetName())
				: FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *Pin->GetName());
			if (InNode->FindPin(OrphanedName) == nullptr)
			{
				Pin->DisplayName = Pin->GetFName();
				InNode->Pins.Remove(Pin);
				
				RenameObject(Pin, *OrphanedName, nullptr);

				InNode->OrphanedPins.Add(Pin);
				Notify(ERigVMGraphNotifType::PinRenamed, Pin);
			}
			else
			{
				RemovePin(Pin, false);
			}
		}
	}

	// remove obsolete pins
	for (int32 Index = NodeData.PreviousPinsToRemove.Num() - 1; Index >= 0; Index--)
	{
		const FString& PinPath = NodeData.PreviousPinInfos.GetPinPath(NodeData.PreviousPinsToRemove[Index]);
		URigVMPin* Pin = InNode->FindPin(PinPath);
		if (Pin == nullptr)
		{
			const FString OrphanedPinPath = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *PinPath);
			Pin = InNode->FindPin(OrphanedPinPath);
		}
		if (Pin)
		{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
			UE_LOGF(LogRigVMDeveloper, Display, "Removing pin '%ls'.", *PinPath);
#endif
			RemovePin(Pin, false);
		}
	}
	// add missing pins
	for (int32 Index = 0; Index < NodeData.NewPinsToAdd.Num(); Index++)
	{
		const FString& PinPath = NodeData.NewPinInfos.GetPinPath(NodeData.NewPinsToAdd[Index]);
		FString ParentPinPath, PinName;
		UObject* OuterForPin = InNode;
		if (URigVMPin::SplitPinPathAtEnd(PinPath, ParentPinPath, PinName))
		{
			OuterForPin = InNode->FindPin(ParentPinPath);
			if (OuterForPin == nullptr)
			{
				const FString OrphanedParentPinPath = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *ParentPinPath);
				OuterForPin = InNode->FindPin(OrphanedParentPinPath);
			}
		}

		CreatePinFromPinInfo(Registry, NodeData.PreviousPinInfos, NodeData.NewPinInfos[NodeData.NewPinsToAdd[Index]], PinPath, OuterForPin);
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
		UE_LOGF(LogRigVMDeveloper, Display, "Adding new pin '%ls'.", *PinPath);
#endif
	}
	// update existing pins
	for (int32 Index = 0; Index < NodeData.PreviousPinsToUpdate.Num(); Index++)
	{
		const FString& PinPath = NodeData.PreviousPinInfos.GetPinPath(NodeData.PreviousPinsToUpdate[Index]);
		const FRigVMPinInfo* NewPinInfo = NodeData.NewPinInfos.GetPinFromPinPath(PinPath);
		check(NewPinInfo);

		if (URigVMPin* Pin = InNode->FindPin(PinPath))
		{
			if (Pin->IsExecuteContext())
			{
				MakeExecutePin(Pin);
			}

			if (Pin->GetTypeIndex() != NewPinInfo->TypeIndex)
			{
				// we expect these changes to only apply to float and double pins.
				check((NewPinInfo->TypeIndex == RigVMTypeUtils::TypeIndex::Float) ||
					(NewPinInfo->TypeIndex == RigVMTypeUtils::TypeIndex::FloatArray) ||
					(NewPinInfo->TypeIndex == RigVMTypeUtils::TypeIndex::Double) ||
					(NewPinInfo->TypeIndex == RigVMTypeUtils::TypeIndex::DoubleArray));

				const FRigVMTemplateArgumentType& NewType = Registry.GetType(NewPinInfo->TypeIndex);
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
				UE_LOGF(LogRigVMDeveloper, Display, "Changing pin '%ls' type from %ls to %ls.",
					*PinPath,
					*Pin->CPPType,
					*NewType.CPPType.ToString()
				);
#endif
				Pin->CPPType = NewType.CPPType.ToString();
				Pin->CPPTypeObject = NewType.CPPTypeObject;
				check(Pin->CPPTypeObject == nullptr);
				Pin->CPPTypeObjectPath = NAME_None;
				Pin->LastKnownTypeIndex = NewPinInfo->TypeIndex;
				Pin->bIsInputVariable = NewPinInfo->bIsInputVariable;
				Pin->IncrementVersion();

				Notify(ERigVMGraphNotifType::PinTypeChanged, Pin);
			}
			else if (Pin->bIsInputVariable != NewPinInfo->bIsInputVariable)
			{
				Pin->bIsInputVariable = NewPinInfo->bIsInputVariable;
				Pin->IncrementVersion();

				Notify(ERigVMGraphNotifType::PinTypeChanged, Pin);
			}
		}
	}

	// create a map representing the order of expected pins
	TMap<FString, TArray<FName>> PinOrder;
	for (int32 Index = 0; Index < NodeData.NewPinInfos.Num(); Index++)
	{
		const FRigVMPinInfo& NewPin = NodeData.NewPinInfos[Index];
		FString ParentPinPath;
		if (NewPin.ParentIndex != INDEX_NONE)
		{
			ParentPinPath = NodeData.NewPinInfos.GetPinPath(NewPin.ParentIndex);
			if (NodeData.NewPinInfos[NewPin.ParentIndex].bIsArray)
			{
				continue;
			}
		}

		TArray<FName>& SubPinOrder = PinOrder.FindOrAdd(ParentPinPath);
		SubPinOrder.Add(NewPin.Name);
	}

	auto SortPinArray = [this](TArray<TObjectPtr<URigVMPin>>& Pins, const TArray<FName>* PinOrder)
		{
			if (PinOrder == nullptr)
			{
				return;
			}

			if (Pins.Num() < 2)
			{
				return;
			}

			const TArray<TObjectPtr<URigVMPin>> PreviousPins = Pins;

			if (Pins[0]->IsArrayElement())
			{
				Algo::Sort(Pins, [PinOrder](const TObjectPtr<URigVMPin>& A, const TObjectPtr<URigVMPin>& B) -> bool
					{
						return A->GetFName().Compare(B->GetFName()) < 0;
					});
			}
			else
			{
				Algo::Sort(Pins, [PinOrder](const TObjectPtr<URigVMPin>& A, const TObjectPtr<URigVMPin>& B) -> bool
					{
						const int32 IndexA = PinOrder->Find(A->GetFName());
						const int32 IndexB = PinOrder->Find(B->GetFName());
						return IndexA < IndexB;
					});
			}

			for (int32 Index = 0; Index < Pins.Num(); Index++)
			{
				if (PreviousPins[Index] != Pins[Index])
				{
					Notify(ERigVMGraphNotifType::PinIndexChanged, Pins[Index]);
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
					UE_LOGF(LogRigVMDeveloper, Display, "Pin '%ls' changed index from %d to %d.",
						*Pins[Index]->GetPinPath(),
						PreviousPins.Find(Pins[Index]),
						Index
					);
#endif
				}
			}
		};

	SortPinArray(InNode->Pins, PinOrder.Find(FString()));
	for (URigVMPin* Pin : InNode->Pins)
	{
		SortPinArray(Pin->SubPins, PinOrder.Find(Pin->GetPinPath()));
	}

	if (DispatchNode)
	{
		ResolveTemplateNodeMetaData(DispatchNode, false);
	}
	else if (CollapseNode)
	{
		if (!CollapseNode->IsGraphFunctionDefinition())
		{
			// no need to notify since the function library graph is invisible anyway
			RemoveUnusedOrphanedPins(CollapseNode);
		}
	}
	else if (FunctionRefNode)
	{
		// we want to make sure notify the graph of a potential name change
		// when repopulating the function ref node
		Notify(ERigVMGraphNotifType::NodeRenamed, FunctionRefNode);
	}

	if (!PinStates.IsEmpty())
	{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
		UE_LOGF(LogRigVMDeveloper, Display, "Reapplying pin-states of node %ls...", *InNode->GetPathName());
#endif
		ApplyPinStates(InNode, PinStates, RedirectedPinPaths);
	}

	InNode->UpdateTraitRootPinNames();

	if (!LinkedPaths.IsEmpty())
	{
#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
		UE_LOGF(LogRigVMDeveloper, Display, "Reattaching links of node %ls...", *InNode->GetPathName());
#endif

		FRestoreLinkedPathSettings Settings;
		Settings.bFollowCoreRedirectors = true;
		Settings.bRelayToOrphanPins = true;
		RestoreLinkedPaths(LinkedPaths, Settings);
	}

	if (URigVMInjectionInfo* InjectionInfo = InNode->GetInjectionInfo())
	{
		InjectionInfo->InputPin = InNode->FindPin(InjectionInputPinName.ToString());
		InjectionInfo->OutputPin = InNode->FindPin(InjectionOutputPinName.ToString());
	}

#if UE_RIGVMCONTROLLER_VERBOSE_REPOPULATE
	UE_LOGF(LogRigVMDeveloper, Display, "Repopulate of node %ls is completed.\n", *InNode->GetPathName());
#endif
}

void URigVMController::RemovePinsDuringRepopulate(URigVMNode* InNode, TArray<URigVMPin*>& InPins, bool bSetupOrphanedPins)
{
	TArray<URigVMPin*> Pins = InPins;
	for (URigVMPin* Pin : Pins)
	{
		if(bSetupOrphanedPins && !Pin->IsExecuteContext())
		{
			const bool bIsTraitPin = Pin->GetRootPin()->IsTraitPin();
			URigVMPin* RootPin = bIsTraitPin ? Pin : Pin->GetRootPin();
			const FString OrphanedName = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *RootPin->GetName());

			URigVMPin* OrphanedRootPin = nullptr;
			
			for(URigVMPin* OrphanedPin : InNode->OrphanedPins)
			{
				if(OrphanedPin->GetName() == OrphanedName)
				{
					OrphanedRootPin = OrphanedPin;
					break;
				}
			}

			if(OrphanedRootPin == nullptr)
			{
				if(Pin->IsRootPin() || bIsTraitPin) // if we are passing root pins we can reparent them directly
				{
					check(RootPin->GetLinkedSourcePins(true).IsEmpty());
					check(RootPin->GetLinkedTargetPins(true).IsEmpty());
					
					RootPin->DisplayName = RootPin->GetFName();
					RenameObject(RootPin, *OrphanedName, nullptr);
					InNode->Pins.Remove(RootPin);

					if(!bSuspendNotifications)
					{
						Notify(ERigVMGraphNotifType::PinRemoved, RootPin);
					}

					InNode->OrphanedPins.Add(RootPin);

					if(!bSuspendNotifications)
					{
						Notify(ERigVMGraphNotifType::PinAdded, RootPin);
					}
				}
				else // while if we are iterating over sub pins - we should reparent them
				{
					OrphanedRootPin = NewObject<URigVMPin>(RootPin->GetNode(), *OrphanedName);
					ConfigurePinFromPin(OrphanedRootPin, RootPin);
					OrphanedRootPin->DisplayName = RootPin->GetFName();
				
					OrphanedRootPin->GetNode()->OrphanedPins.Add(OrphanedRootPin);

					if(!bSuspendNotifications)
					{
						Notify(ERigVMGraphNotifType::PinAdded, OrphanedRootPin);
					}
				}
			}

			if(!Pin->IsRootPin() && (OrphanedRootPin != nullptr))
			{
				RenameObject(Pin, nullptr, OrphanedRootPin);
				RootPin->SubPins.Remove(Pin);
				EnsurePinValidity(Pin, false);
				AddSubPin(OrphanedRootPin, Pin);
			}
		}
	}

	for (URigVMPin* Pin : Pins)
	{
		if(!Pin->IsOrphanPin())
		{
			RemovePin(Pin, false);
		}
	}
	InPins.Reset();
}

bool URigVMController::RemoveUnusedOrphanedPins(URigVMNode* InNode, bool bRelayLinks)
{
	if(!InNode->HasOrphanedPins())
	{
		return true;
	}

	TArray<URigVMPin*> OrphanedPins = InNode->OrphanedPins; 
	TArray<URigVMPin*> RemainingOrphanPins;
	for(int32 PinIndex=0; PinIndex < OrphanedPins.Num(); PinIndex++)
	{
		URigVMPin* OrphanedPin = OrphanedPins[PinIndex];

		int32 NumSourceLinks = OrphanedPin->GetSourceLinks(true).Num(); 
		int32 NumTargetLinks = OrphanedPin->GetTargetLinks(true).Num();

		if(bRelayLinks && (NumSourceLinks + NumTargetLinks > 0))
		{
			TArray<URigVMLink*> Links = OrphanedPin->GetSourceLinks(true); 
			Links.Append(OrphanedPin->GetTargetLinks(true));

			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(Links);
			FastBreakLinkedPaths(LinkedPaths);

			FRestoreLinkedPathSettings Settings;
			Settings.bFollowCoreRedirectors = true;
			Settings.bRelayToOrphanPins = true;

			RestoreLinkedPaths(LinkedPaths, Settings);

			NumSourceLinks = OrphanedPin->GetSourceLinks(true).Num();
			NumTargetLinks = OrphanedPin->GetTargetLinks(true).Num();
		}

		if(NumSourceLinks + NumTargetLinks == 0)
		{
			RemovePin(OrphanedPin, false);
		}
		else
		{
			RemainingOrphanPins.Add(OrphanedPin);
		}
	}

	InNode->OrphanedPins = RemainingOrphanPins;
	
	return !InNode->HasOrphanedPins();
}

#endif

void URigVMController::SetupDefaultUnitNodeDelegates(TDelegate<FName(FRigVMExternalVariable, FString)> InCreateExternalVariableDelegate)
{
	TWeakObjectPtr<URigVMController> WeakThis(this);

#if WITH_EDITOR
	UnitNodeCreatedContext.Controller = this;
#endif
	UnitNodeCreatedContext.GetAllExternalVariablesDelegate().BindLambda([WeakThis]() -> TArray<FRigVMExternalVariable> {
		if (WeakThis.IsValid())
		{
			return WeakThis->GetAllVariables();
		}
		return TArray<FRigVMExternalVariable>();
	});

	UnitNodeCreatedContext.GetBindPinToExternalVariableDelegate().BindLambda([WeakThis](FString InPinPath, FString InVariablePath) -> bool {
		if (WeakThis.IsValid())
		{
			return WeakThis->BindPinToVariable(InPinPath, InVariablePath, true);
		}
		return false;
	});

	UnitNodeCreatedContext.GetCreateExternalVariableDelegate() = InCreateExternalVariableDelegate;
}

void URigVMController::ResetUnitNodeDelegates()
{
	UnitNodeCreatedContext.GetAllExternalVariablesDelegate().Unbind();
	UnitNodeCreatedContext.GetBindPinToExternalVariableDelegate().Unbind();
	UnitNodeCreatedContext.GetCreateExternalVariableDelegate().Unbind();
}

FLinearColor URigVMController::GetColorFromMetadata(const FString& InMetadata)
{
	FLinearColor Color = FLinearColor::Black;

	FString Metadata = InMetadata;
	Metadata.TrimStartAndEndInline();
	FString SplitString(TEXT(" "));
	FString Red, Green, Blue, GreenAndBlue;
	if (Metadata.Split(SplitString, &Red, &GreenAndBlue))
	{
		Red.TrimEndInline();
		GreenAndBlue.TrimStartInline();
		if (GreenAndBlue.Split(SplitString, &Green, &Blue))
		{
			Green.TrimEndInline();
			Blue.TrimStartInline();

			float RedValue = FCString::Atof(*Red);
			float GreenValue = FCString::Atof(*Green);
			float BlueValue = FCString::Atof(*Blue);
			Color = FLinearColor(RedValue, GreenValue, BlueValue);
		}
	}

	return Color;
}

TMap<FString, FString> URigVMController::GetRedirectedPinPaths(URigVMNode* InNode) const
{
	TMap<FString, FString> RedirectedPinPaths;
	URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode);
	URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(InNode);

	UScriptStruct* OwningStruct = nullptr;
	if (UnitNode)
	{
		OwningStruct = UnitNode->GetScriptStruct();
	}
	else if (RerouteNode)
	{
		URigVMPin* ValuePin = RerouteNode->Pins[0];
		if (ValuePin->IsStruct())
		{
			OwningStruct = ValuePin->GetScriptStruct();
		}
	}

	if (OwningStruct)
	{
		TArray<URigVMPin*> AllPins = InNode->GetAllPinsRecursively();
		for (URigVMPin* Pin : AllPins)
		{
			FString NodeName, PinPath;
			URigVMPin::SplitPinPathAtStart(Pin->GetPinPath(), NodeName, PinPath);

			if (RerouteNode)
			{
				FString ValuePinName, SubPinPath;
				if (URigVMPin::SplitPinPathAtStart(PinPath, ValuePinName, SubPinPath))
				{
					FString RedirectedSubPinPath;
					if (ShouldRedirectPin(OwningStruct, SubPinPath, RedirectedSubPinPath))
					{
						FString RedirectedPinPath = URigVMPin::JoinPinPath(ValuePinName, RedirectedSubPinPath);
						RedirectedPinPaths.Add(PinPath, RedirectedPinPath);
					}
				}
			}
			else
			{
				FString RedirectedPinPath;
				if (ShouldRedirectPin(OwningStruct, PinPath, RedirectedPinPath))
				{
					RedirectedPinPaths.Add(PinPath, RedirectedPinPath);
				}
			}
		}
	};
	return RedirectedPinPaths;
}

URigVMController::FPinState URigVMController::GetPinState(const URigVMPin* InPin, bool bStoreWeakInjectionInfos) const
{
	FPinState State;
	State.Direction = InPin->GetDirection();
	State.CPPType = InPin->GetCPPType();
	State.CPPTypeObject = InPin->GetCPPTypeObject();
	State.DefaultValue = InPin->GetDefaultValue();
	State.DefaultValueType = InPin->GetDefaultValueType();
	State.bIsExpanded = InPin->IsExpanded();
	State.InjectionInfos = InPin->GetInjectedNodes();

	if(bStoreWeakInjectionInfos)
	{
		for(URigVMInjectionInfo* InjectionInfo : State.InjectionInfos)
		{
			State.WeakInjectionInfos.Add(InjectionInfo->GetWeakInfo());
		}
		State.InjectionInfos.Reset();
	}
	else
	{
		// move the injection info under the graph so that they don't get lost
		// when the pins get destroyed. the injection info will get renamed / reparented
		// under the pin again once it is re-populated - or destroyed if the pin
		// no longer exists during ApplyPinState.
		for(URigVMInjectionInfo* InjectionInfo : State.InjectionInfos)
		{
#if WITH_EDITOR
			// Copy over the old unit node if the content is from prior to that property being deprecated.
			if (!InjectionInfo->Node)
			{
				InjectionInfo->Node = InjectionInfo->UnitNode_DEPRECATED;
			}
#endif
			RenameObject(InjectionInfo, nullptr, InPin->GetGraph());
		}
	}

	return State;
}

TMap<FString, URigVMController::FPinState> URigVMController::GetPinStates(URigVMNode* InNode, bool bStoreWeakInjectionInfos) const
{
	TMap<FString, FPinState> PinStates;

	TArray<URigVMPin*> AllPins = InNode->GetAllPinsRecursively();
	for (URigVMPin* Pin : AllPins)
	{
		FString PinPath, NodeName;
		URigVMPin::SplitPinPathAtStart(Pin->GetPinPath(), NodeName, PinPath);

		// we need to ensure validity here because GetPinState()-->GetDefaultValue() needs pin to be in a valid state.
		// some additional context:
		// right after load, some pins will be a invalid state because they don't have their CPPTypeObject,
		// which is expected since it is a transient property.
		// if the CPPTypeObject is not there, those pins may struggle with producing a valid default value
		// because Pin->IsStruct() will always be false if the pin does not have a valid type object.
		if (Pin->IsRootPin())
		{
			EnsurePinValidity(Pin, true);
		}
		FPinState State = GetPinState(Pin, bStoreWeakInjectionInfos);
		PinStates.Add(PinPath, State);
	}

	return PinStates;
}

void URigVMController::ApplyPinState(URigVMPin* InPin, const FPinState& InPinState, bool bSetupUndoRedo)
{
	for (URigVMInjectionInfo* InjectionInfo : InPinState.InjectionInfos)
	{
		// Copy over the old unit node if the content is from prior to that property being deprecated.
		if (InjectionInfo && !InjectionInfo->Node)
		{
			InjectionInfo->Node = InjectionInfo->UnitNode_DEPRECATED;
		}

		// If this node has been destroyed already or is marked as garbage
		if (InjectionInfo && URigVMHost::IsGarbageOrDestroyed(InjectionInfo->Node))
		{
			continue;
		}

		if (InjectionInfo && InjectionInfo->Node)
		{
			RenameObject(InjectionInfo, nullptr, InPin);
			InjectionInfo->InputPin = InjectionInfo->InputPin ? InjectionInfo->Node->FindPin(InjectionInfo->InputPin->GetName()) : nullptr;
			InjectionInfo->OutputPin = InjectionInfo->OutputPin ? InjectionInfo->Node->FindPin(InjectionInfo->OutputPin->GetName()) : nullptr;
			InPin->InjectionInfos.Add(InjectionInfo);
		}
	}

	// alternatively if the injection infos are not provided as strong pointers
	// we can fall back onto the weak ptr information and try again
	if(InPinState.InjectionInfos.IsEmpty())
	{
		for (const URigVMInjectionInfo::FWeakInfo& InjectionInfo : InPinState.WeakInjectionInfos)
		{
			if(URigVMNode* FormerlyInjectedNode = InjectionInfo.Node.Get())
			{
				if (FormerlyInjectedNode->IsInjected())
				{
					URigVMInjectionInfo* Injection = Cast<URigVMInjectionInfo>(FormerlyInjectedNode->GetOuter());
					if(URigVMPin* OuterPin = Cast<URigVMPin>(Injection->GetOuter()))
					{
						OuterPin->InjectionInfos.Remove(Injection);
						OuterPin->IncrementVersion();
					}
					RenameObject(FormerlyInjectedNode, nullptr, InPin->GetGraph());
					DestroyObject(Injection);
				}
				if(InjectionInfo.bInjectedAsInput)
				{
					const FString OutputPinPath = URigVMPin::JoinPinPath(FormerlyInjectedNode->GetNodePath(), InjectionInfo.OutputPinName.ToString()); 
					AddLink(OutputPinPath, InPin->GetPinPath(), bSetupUndoRedo, false);
				}
				else
				{
					const FString InputPinPath = URigVMPin::JoinPinPath(FormerlyInjectedNode->GetNodePath(), InjectionInfo.InputPinName.ToString()); 
					AddLink(InPin->GetPinPath(), InputPinPath, bSetupUndoRedo, false);
				}

				if(InPin->IsRootPin())
				{
					InjectNodeIntoPin(InPin, InjectionInfo.bInjectedAsInput, InjectionInfo.InputPinName, InjectionInfo.OutputPinName, bSetupUndoRedo);
				}
			}
		}
	}

	if (!InPinState.DefaultValue.IsEmpty() || InPin->IsWildCard())
	{
		if(InPinState.Direction != ERigVMPinDirection::Output)
		{
			FString DefaultValue = InPinState.DefaultValue;
			PostProcessDefaultValue(InPin, DefaultValue);
			if(!DefaultValue.IsEmpty() || InPin->IsWildCard() || InPin->IsUObject())
			{
				FRigVMDefaultValueTypeGuard _(this, InPinState.DefaultValueType, true);
				SetPinDefaultValue(InPin, DefaultValue, true, bSetupUndoRedo, false);
			}
		}
	}

	SetPinExpansion(InPin, InPinState.bIsExpanded, bSetupUndoRedo);
	InPin->IncrementVersion();
}

void URigVMController::ApplyPinStates(URigVMNode* InNode, const TMap<FString, URigVMController::FPinState>& InPinStates, const TMap<FString, FString>& InRedirectedPinPaths, bool bSetupUndoRedo)
{
	FRigVMControllerCompileBracketScope CompileBracketScope(this);
	for (const TPair<FString, FPinState>& PinStatePair : InPinStates)
	{
		FString PinPath = PinStatePair.Key;
		const FPinState& PinState = PinStatePair.Value;

		if (InRedirectedPinPaths.Contains(PinPath))
		{
			PinPath = InRedirectedPinPaths.FindChecked(PinPath);
		}

		if (URigVMPin* Pin = InNode->FindPin(PinPath))
		{
			ApplyPinState(Pin, PinState, bSetupUndoRedo);
		}
		else
		{
			for (URigVMInjectionInfo* InjectionInfo : PinState.InjectionInfos)
			{
				if(URigVMPin* OuterPin = Cast<URigVMPin>(InjectionInfo->GetOuter()))
				{
					OuterPin->InjectionInfos.Remove(InjectionInfo);
					OuterPin->IncrementVersion();
				}
				RenameObject(InjectionInfo->Node, nullptr, InNode->GetGraph());
				DestroyObject(InjectionInfo);
			}
		}
	}
}

URigVMPin* URigVMController::CreatePinFromPinInfo(const FRigVMRegistry& InRegistry, const FRigVMPinInfoArray& InPreviousPinInfos, const FRigVMPinInfo& InPinInfo, const FString& InPinPath, UObject* InOuter)
{
	check(InOuter);
	URigVMPin* Pin = NewObject<URigVMPin>(InOuter, InPinInfo.Name);
	if (InPinInfo.Property)
	{
		ConfigurePinFromProperty(InPinInfo.Property, Pin, InPinInfo.Direction);
	}
	else
	{
		const FRigVMTemplateArgumentType& Type = InRegistry.GetType(InPinInfo.TypeIndex);
		Pin->CPPType = Type.CPPType.ToString();
		Pin->CPPTypeObject = Type.CPPTypeObject;
		if (Pin->CPPTypeObject)
		{
			Pin->CPPTypeObjectPath = *Pin->CPPTypeObject->GetPathName();
		}
		if (InRegistry.IsExecuteType(InPinInfo.TypeIndex))
		{
			MakeExecutePin(Pin);
		}

		Pin->Direction = InPinInfo.Direction;
		Pin->DisplayName = InPinInfo.DisplayName.IsEmpty() ? NAME_None : FName(*InPinInfo.DisplayName);
		Pin->bIsConstant = InPinInfo.bIsConstant;
		Pin->bIsInputVariable = InPinInfo.bIsInputVariable;
		Pin->bIsDynamicArray = InPinInfo.bIsDynamicArray;
		Pin->bIsLazy = InPinInfo.bIsLazy;
		Pin->CustomWidgetName = InPinInfo.CustomWidgetName.IsEmpty() ? NAME_None : FName(*InPinInfo.CustomWidgetName);
	}

	Pin->bIsExpanded = InPinInfo.bIsExpanded;
	if (Pin->IsUObject())
	{
		Pin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InPinInfo.DefaultValue);
	}
	else
	{
		Pin->DefaultValue = InPinInfo.DefaultValue;
	}
	Pin->DefaultValueType = InPinInfo.DefaultValueType;

	// reuse expansion state and default value
	if (const FRigVMPinInfo* PreviousPin = InPreviousPinInfos.GetPinFromPinPath(InPinPath))
	{
		if (PreviousPin->TypeIndex == InPinInfo.TypeIndex)
		{
			Pin->bIsExpanded = PreviousPin->bIsExpanded;
			if (Pin->IsUObject())
			{
				Pin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(PreviousPin->DefaultValue);
			}
			else
			{
				Pin->DefaultValue = PreviousPin->DefaultValue;
			}
			Pin->DefaultValueType = PreviousPin->DefaultValueType;
		}
	}

	if (URigVMPin* ParentPin = Cast<URigVMPin>(InOuter))
	{
		AddSubPin(ParentPin, Pin);
	}
	else if (URigVMNode* OwnerNode = Cast<URigVMNode>(InOuter))
	{
		AddNodePin(OwnerNode, Pin);
	}
	else
	{
		ensureMsgf(false, TEXT("Outer %s of pin info %s is not a pin or a node"), *InOuter->GetPathName(), *InPinPath);
	}

	Notify(ERigVMGraphNotifType::PinAdded, Pin);

	if (Pin->IsDefinedAsInputVariable())
	{
		if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Pin->GetNode()))
		{
			const FString MappedVariablePath = FunctionReferenceNode->GetOuterVariablePath(Pin->GetFName());
			if (!MappedVariablePath.IsEmpty())
			{
				(void)BindPinToVariable(Pin, MappedVariablePath, false);
			}
		}
	}

	return Pin;
}

void URigVMController::UpdateGraphSectionsIfRequired()
{
	if (UpdateGraphSectionsBracket > 1)
	{
		return;
	}
	
	URigVMGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		return;
	}
	Graph->UpdateSections();
}

void URigVMController::ReportInfo(const FString& InMessage) const
{
	if (URigVMGraph* Graph = GetGraph())
	{
		if (UPackage* Package = Cast<UPackage>(Graph->GetOutermost()))
		{
			UE_LOGF(LogRigVMDeveloper, Display, "%ls : %ls", *Package->GetPathName(), *InMessage);
			return;
		}
	}

	UE_LOGF(LogRigVMDeveloper, Display, "%ls", *InMessage);
}

void URigVMController::ReportWarning(const FString& InMessage) const
{
	if(!bReportWarningsAndErrors)
	{
		return;
	}

	FString Message = InMessage;
	if (URigVMGraph* Graph = GetGraph())
	{
		if (UPackage* Package = Cast<UPackage>(Graph->GetOutermost()))
		{
			Message = FString::Printf(TEXT("%s : %s"), *Package->GetPathName(), *InMessage);
		}
	}

	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Warning, *Message, *FString());
}

void URigVMController::ReportError(const FString& InMessage) const
{
	if(!bReportWarningsAndErrors)
	{
		return;
	}

	FString Message = InMessage;
	if (URigVMGraph* Graph = GetGraph())
	{
		if (UPackage* Package = Cast<UPackage>(Graph->GetOutermost()))
		{
			Message = FString::Printf(TEXT("%s : %s"), *Package->GetPathName(), *InMessage);
		}
	}

	FScriptExceptionHandler::Get().HandleException(ELogVerbosity::Error, *Message, *FString());
}

void URigVMController::ReportAndNotifyInfo(const FString& InMessage) const
{
	ReportWarning(InMessage);
	SendUserFacingNotification(InMessage, 0.f, nullptr, TEXT("MessageLog.Note"));
}

void URigVMController::ReportAndNotifyWarning(const FString& InMessage) const
{
	if (!bReportWarningsAndErrors)
	{
		return;
	}

	ReportWarning(InMessage);
	SendUserFacingNotification(InMessage, 0.f, nullptr, TEXT("MessageLog.Warning"));
}

void URigVMController::ReportAndNotifyError(const FString& InMessage) const
{
	if (!bReportWarningsAndErrors)
	{
		return;
	}

	ReportError(InMessage);
	SendUserFacingNotification(InMessage, 0.f, nullptr, TEXT("MessageLog.Error"));
}

void URigVMController::ReportPinTypeChange(URigVMPin* InPin, const FString& InNewCPPType)
{
	/*
	UE_LOGF(LogRigVMDeveloper,
		Warning,
		"Pin '%ls' is about to change type from '%ls' to '%ls'.",
		*InPin->GetPinPath(),
		*InPin->GetCPPType(),
		*InNewCPPType
	);
	//*/
}

void URigVMController::SendUserFacingNotification(const FString& InMessage, float InDuration, const UObject* InSubject, const FName& InBrushName) const
{
#if WITH_EDITOR

	if(InDuration < SMALL_NUMBER)
	{
		InDuration = FMath::Clamp(0.1f * static_cast<float>(InMessage.Len()), 5.0f, 20.0f);
	}
	
	FNotificationInfo Info(FText::FromString(InMessage));
	Info.bUseSuccessFailIcons = true;
	Info.Image = FAppStyle::GetBrush(InBrushName);
	Info.bFireAndForget = true;
	Info.bUseThrobber = true;
	// longer message needs more time to read
	Info.FadeOutDuration = FMath::Min(InDuration, 1.f);
	Info.ExpireDuration = InDuration;

	if(InSubject)
	{
		if(const URigVMNode* Node = Cast<URigVMNode>(InSubject))
		{
			Info.HyperlinkText = FText::FromString(Node->GetNodePath());
		}
		else if(const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
		{
			Info.HyperlinkText = FText::FromString(Pin->GetPinPath());
		}
		else if(const URigVMLink* Link = Cast<URigVMLink>(InSubject))
		{
			Info.HyperlinkText = FText::FromString(((URigVMLink*)Link)->GetPinPathRepresentation());
		}
		else
		{
			Info.HyperlinkText = FText::FromName(InSubject->GetFName());
		}

		Info.Hyperlink = FSimpleDelegate::CreateLambda([InSubject, this]()
		{
			if(RequestJumpToHyperlinkDelegate.IsBound())
			{
				RequestJumpToHyperlinkDelegate.Execute(InSubject);
			}
		});
	}
	
	TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
	if (NotificationPtr)
	{
		NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
	}
#endif
}

void URigVMController::RequestJumpToHyperLink(const URigVMNode* InSubject)
{
	if(RequestJumpToHyperlinkDelegate.IsBound())
	{
		RequestJumpToHyperlinkDelegate.Execute(InSubject);
	}
}

void URigVMController::CreateDefaultValueForStructIfRequired(UScriptStruct* InStruct, FString& InOutDefaultValue)
{
	if (InStruct != nullptr)
	{
		TArray<uint8, TAlignedHeapAllocator<16>> TempBuffer;
		TempBuffer.AddUninitialized(InStruct->GetStructureSize());

		// call the struct constructor to initialize the struct
		InStruct->InitializeDefaultValue(TempBuffer.GetData());

		// apply any higher-level value overrides
		// for example,  
		// struct B { int Test; B() {Test = 1;}}; ----> This is the constructor initialization, applied first in InitializeDefaultValue() above 
		// struct A 
		// {
		//		Array<B> TestArray;
		//		A() 
		//		{
		//			TestArray.Add(B());
		//			TestArray[0].Test = 2;  ----> This is the overrride, applied below in ImportText()
		//		}
		// }
		// See UnitTest RigVM->Graph->UnitNodeDefaultValue for more use case.
		
		if (!InOutDefaultValue.IsEmpty() && InOutDefaultValue != TEXT("()"))
		{ 
			FRigVMPinDefaultValueImportErrorContext ErrorPipe;
			InStruct->ImportText(*InOutDefaultValue, TempBuffer.GetData(), nullptr, PPF_None, &ErrorPipe, FString());
		}

		// in case InOutDefaultValue is not empty, it needs to be cleared
		// before ExportText() because ExportText() appends to it.
		InOutDefaultValue.Reset();

		InStruct->ExportText(InOutDefaultValue, TempBuffer.GetData(), TempBuffer.GetData(), nullptr, (PPF_ExternalEditor | PPF_IncludeTransient), nullptr);
		InStruct->DestroyStruct(TempBuffer.GetData());
	}
}

void URigVMController::PostProcessDefaultValue(const URigVMPin* Pin, FString& OutDefaultValue)
{
	static const FString NoneString = FName(NAME_None).ToString();
	static const FString QuotedNoneString = FString::Printf(TEXT("\"%s\""), *NoneString);
	if(OutDefaultValue == NoneString || OutDefaultValue == QuotedNoneString)
	{
		if(!Pin->IsStringType())
		{
			OutDefaultValue.Reset();
		}
	}
	if (Pin->IsStruct() || Pin->IsArray())
	{
		// We cannot sanitize struct text defaults that have native import/export, as they do not have to follow conventional syntax 
		const UScriptStruct* ScriptStruct = Pin->GetScriptStruct();
		const bool bHasNativeImportExport = ScriptStruct && ((ScriptStruct->StructFlags & (STRUCT_ImportTextItemNative | STRUCT_ExportTextItemNative)) != 0);

		if (!OutDefaultValue.IsEmpty() && !bHasNativeImportExport)
		{
			if (OutDefaultValue[0] != TCHAR('(') || OutDefaultValue[OutDefaultValue.Len()-1] != TCHAR(')'))
			{
				OutDefaultValue.Reset();
			}
		}
	}

	if (Pin->IsArray() && OutDefaultValue.IsEmpty())
	{
		OutDefaultValue = TEXT("()");
	}
	else if (Pin->IsEnum() && OutDefaultValue.IsEmpty())
	{
		int32 EnumIndex = Pin->GetEnum()->GetIndexByName(*NoneString);
		// make sure that none is a valid enum value
		if (EnumIndex != INDEX_NONE)
		{
			OutDefaultValue = NoneString;
		}
		else
		{
			// when none string is given but none is not a valid enum value,
			// it implies that the pin should use the default value of the enum
			// the value at index 0 is the best guess that can be made.
			// usually a user provided pin default is given later to override this value
			OutDefaultValue = Pin->GetEnum()->GetNameStringByIndex(0);
		}
	}
	else if (Pin->IsStruct() && (OutDefaultValue.IsEmpty() || OutDefaultValue == TEXT("()")))
	{
		CreateDefaultValueForStructIfRequired(Pin->GetScriptStruct(), OutDefaultValue);
	}
	else if (Pin->IsStringType())
	{
		while (OutDefaultValue.StartsWith(TEXT("\"")))
		{
			OutDefaultValue = OutDefaultValue.RightChop(1);
		}
		while (OutDefaultValue.EndsWith(TEXT("\"")))
		{
			OutDefaultValue = OutDefaultValue.LeftChop(1);
		}
		if(OutDefaultValue.IsEmpty() && Pin->GetCPPType() == RigVMTypeUtils::FNameType)
		{
			OutDefaultValue = FName(NAME_None).ToString();
		}
	}
	else if (OutDefaultValue.IsEmpty())
	{
		if (Pin->GetTypeIndex() == RigVMTypeUtils::TypeIndex::Bool)
		{
			OutDefaultValue = TEXT("false");
		}
		else if (Pin->GetTypeIndex() == RigVMTypeUtils::TypeIndex::Float || Pin->GetTypeIndex() == RigVMTypeUtils::TypeIndex::Double)
		{
			OutDefaultValue = TEXT("0.000000");
		}
		else if (Pin->GetTypeIndex() == RigVMTypeUtils::TypeIndex::Int32 || Pin->GetTypeIndex() == RigVMTypeUtils::TypeIndex::UInt32)
		{
			OutDefaultValue = TEXT("0");
		}
	}
}

void URigVMController::OverrideDefaultValueMember(const FString& InMemberName, const FString& InMemberValue, FString& InOutDefaultValue)
{
	FString NewMemberValuePair = FString::Printf(TEXT("%s=%s"), *InMemberName, *InMemberValue);
	TArray<FString> DefaultValues = URigVMPin::SplitDefaultValue(InOutDefaultValue);
	
	bool bFound = false;
	for (FString& MemberNameValuePair : DefaultValues)
	{
		FString MemberName, MemberValue;
		if (MemberNameValuePair.Split(TEXT("="), &MemberName, &MemberValue))
		{
			if (MemberName == InMemberName)
			{
				MemberNameValuePair = NewMemberValuePair;
				bFound = true;
				break;
			}
		}
	}

	if (!bFound)
	{
		DefaultValues.Add(NewMemberValuePair);
	}
	
	InOutDefaultValue = FString::Printf(TEXT("(%s)"), *FString::Join(DefaultValues, TEXT(",")));	
}

void URigVMController::ResolveTemplateNodeMetaData(URigVMTemplateNode* InNode, const TArray<int32>& InFilteredPermutationIndices, bool bSetupUndoRedo)
{
	check(InNode);

	if(InNode->IsA<URigVMUnitNode>())
	{
		const FLinearColor PreviousColor = InNode->NodeColor;
#if WITH_EDITOR
		InNode->NodeColor = InNode->GetTemplate()->GetColor(InFilteredPermutationIndices);
#endif
		if(!InNode->NodeColor.Equals(PreviousColor, 0.01f))
		{
			Notify(ERigVMGraphNotifType::NodeColorChanged, InNode);
		}
	}

	for(URigVMPin* Pin : InNode->GetPins())
	{
		const FName DisplayName = InNode->GetDisplayNameForPin(Pin, InFilteredPermutationIndices);
		if(Pin->DisplayName != DisplayName)
		{
			Pin->DisplayName = DisplayName;
			Notify(ERigVMGraphNotifType::PinRenamed, Pin);
		}
	}

	if(InNode->IsResolved())
	{
		for(URigVMPin* Pin : InNode->GetPins())
		{
			if(Pin->IsWildCard() || Pin->ContainsWildCardSubPin() || Pin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				continue;
			}
			if(!Pin->IsValidDefaultValue(Pin->GetDefaultValue()))
			{
				const FString NewDefaultValue = InNode->GetInitialDefaultValueForPin(Pin->GetFName(), InFilteredPermutationIndices);
				if(!NewDefaultValue.IsEmpty())
				{
					SetPinDefaultValue(Pin, NewDefaultValue, true, bSetupUndoRedo, false);
				}
			}

			// Update constant flag based on the resolved struct's property metadata.
			const bool bShouldBeConstant = InNode->IsPinConstant(Pin);
			if(Pin->bIsConstant != bShouldBeConstant)
			{
				Pin->bIsConstant = bShouldBeConstant;
			}
		}
	}
}

void URigVMController::ResolveTemplateNodeMetaData(URigVMTemplateNode* InNode, bool bSetupUndoRedo)
{
	TArray<int32> FilteredPermutationIndices = InNode->GetResolvedPermutationIndices(false);
	ResolveTemplateNodeMetaData(InNode, FilteredPermutationIndices, bSetupUndoRedo);
}

bool URigVMController::FullyResolveTemplateNode(URigVMTemplateNode* InNode, int32 InPermutationIndex, bool bSetupUndoRedo)
{
	if(bIsFullyResolvingTemplateNode)
	{
		return false;
	}
	TGuardValue<bool> ReentryGuard(bIsFullyResolvingTemplateNode, true);
	
	check(InNode);

	if (InNode->IsSingleton())
	{
		return true;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(InNode->GetTemplate());
	const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory();
	
	InNode->ResolvedPermutation = InPermutationIndex;
	
	// Figure out the permutation index from the pin types
	TArray<int32> Permutations;
	if (InPermutationIndex == INDEX_NONE)
	{
		Permutations = InNode->GetResolvedPermutationIndices(false);

		// If some float/double pin needs to change type, permutations might be empty
		// Try running again allowing that change
		if (Permutations.IsEmpty())
		{
			Permutations = InNode->GetResolvedPermutationIndices(true);
		}
		
		check(!Permutations.IsEmpty());
		InNode->ResolvedPermutation = Permutations[0];

		// make sure the permutation exists
		if(Factory)
		{
			FRigVMTemplateTypeMap TypeMap = InNode->GetTemplate()->GetTypesForPermutation(InNode->ResolvedPermutation);
			const FRigVMFunctionPtr DispatchFunction = Factory->GetOrCreateDispatchFunction(TypeMap);
			const FRigVMFunction* ResolvedFunction = Template->GetOrCreatePermutation(InNode->ResolvedPermutation);
			check(DispatchFunction);
			// When a referenced type has been removed from the registry (e.g. a UDS deleted
			// or migrated while a rig is open), Template->GetOrCreatePermutation can return null even
			// though Permutations[0] is a valid index. Soft-fail here to match the caller's existing
			// "node could not be resolved" handling at FRigVMClient::RefreshAllModels (RigVMClient.cpp).
			// Reset ResolvedPermutation to INDEX_NONE so the node is left in a clean unresolved state
			// instead of a half-resolved one that would crash on the next resolve attempt.
			if (!ResolvedFunction)
			{
				ReportErrorf(TEXT("Template node %s: permutation %d could not be materialized — a referenced type is likely missing from the RigVM registry."),
					*InNode->GetNodePath(),
					InNode->ResolvedPermutation);
				InNode->ResolvedPermutation = INDEX_NONE;
				return false;
			}
			check(ResolvedFunction->FunctionPtr == DispatchFunction);
		}
	}
	

	const FRigVMFunction* ResolvedFunction = Template->GetOrCreatePermutation(InNode->ResolvedPermutation);
	const TArray<int32> PermutationIndices = {InNode->ResolvedPermutation};

	// find all existing pins that we may need to change
	TArray<FRigVMTemplateArgument> MissingPins;
	TArray<URigVMPin*> PinsToRemove;
	TMap<URigVMPin*, TRigVMTypeIndex> PinTypesToChange;
	for(int32 ArgIndex = 0; ArgIndex < Template->NumArguments(); ArgIndex++)
	{
		const FRigVMTemplateArgument* Argument = Template->GetArgument(ArgIndex);
		const TRigVMTypeIndex ResolvedTypeIndex = Argument->GetSupportedTypeIndices(PermutationIndices)[0];

		URigVMPin* Pin = InNode->FindPin(Argument->GetName().ToString());
		if(Pin == nullptr)
		{
			ReportErrorf(TEXT("Template node %s is missing a pin for argument %s"),
				*InNode->GetNodePath(),
				*Argument->GetName().ToString()
			);
			return false;
		}

		if(Pin->GetTypeIndex() != ResolvedTypeIndex && ResolvedTypeIndex != RigVMTypeUtils::TypeIndex::Execute)
		{
			PinTypesToChange.Add(Pin, ResolvedTypeIndex);
		}
	}

	// find all missing pins which are not arguments for the template
	if(ResolvedFunction)
	{
		if(ResolvedFunction->Struct == nullptr)
		{
			const TArray<FRigVMTemplateArgument>& Arguments = Template->Arguments;
			for(const FRigVMTemplateArgument& Argument : Arguments)
			{
				const TRigVMTypeIndex ExpectedTypeIndex = Argument.GetTypeIndex(InNode->ResolvedPermutation);
				if(URigVMPin* Pin = InNode->FindPin(Argument.GetName().ToString()))
				{
					if(Pin->GetTypeIndex() != ExpectedTypeIndex && ExpectedTypeIndex != RigVMTypeUtils::TypeIndex::Execute)
					{
						PinTypesToChange.Add(Pin, ExpectedTypeIndex);
					}
				}
				else
				{
					MissingPins.Add(Argument);
				}
			}
		}
		else
		{
			TArray<UStruct*> StructsToVisit = FRigVMTemplate::GetSuperStructs(ResolvedFunction->Struct, true);
			for(UStruct* StructToVisit : StructsToVisit)
			{
				for (TFieldIterator<FProperty> It(StructToVisit, EFieldIterationFlags::None); It; ++It)
				{
					const FRigVMTemplateArgument ExpectedArgument = FRigVMTemplateArgument::Make(*It);
					const TRigVMTypeIndex ExpectedTypeIndex = ExpectedArgument.GetSupportedTypeIndices()[0];

					if(URigVMPin* Pin = InNode->FindPin(It->GetFName().ToString()))
					{
						if(Pin->GetTypeIndex() != ExpectedTypeIndex && ExpectedTypeIndex != RigVMTypeUtils::TypeIndex::Execute)
						{
							PinTypesToChange.Add(Pin, ExpectedTypeIndex);
						}
					}
					else
					{
						MissingPins.Add(ExpectedArgument);
					}
				}
			}
		}
	}

	// find all pins which don't have a matching arg on the function
	if(ResolvedFunction)
	{
		if(ResolvedFunction->Struct == nullptr)
		{
			FRigVMDispatchContext DispatchContext;
			if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
			{
				DispatchContext = DispatchNode->GetDispatchContext();
			}

			const TArray<FRigVMTemplateArgument>& Arguments = Template->Arguments;
			for(URigVMPin* Pin : InNode->GetPins())
			{
				bool bFound = Arguments.ContainsByPredicate([Pin](const FRigVMTemplateArgument& Argument)
				{
					return Pin->GetFName() == Argument.GetName();
				});
				if(!bFound)
				{
					bFound = Template->GetExecuteArguments(DispatchContext).ContainsByPredicate([Pin](const FRigVMExecuteArgument& Argument)
					{
						return Pin->GetFName() == Argument.Name;
					});
				}
				if(!bFound)
				{
					PinsToRemove.Add(Pin);
				}
			}
		}
		else
		{
			for(URigVMPin* Pin : InNode->GetPins())
			{
				// don't remove execute contexts, they are not part of the template's
				// wildcard pins - so they can't change between resolvals.
				if(Pin->IsExecuteContext())
				{
					continue;
				}
				
				if(ResolvedFunction->Struct->FindPropertyByName(Pin->GetFName()) == nullptr)
				{
					PinsToRemove.Add(Pin);
				}
			}
		}

		// update the cached resolved function name
		InNode->ResolvedFunctionName = ResolvedFunction->Name;
	}

	// exit out early if there's nothing to do
	if(PinTypesToChange.IsEmpty() && MissingPins.IsEmpty() && PinsToRemove.IsEmpty())
	{
		ResolveTemplateNodeMetaData(InNode, bSetupUndoRedo);
		return true;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Resolve Template Node"));
	}

	// update the incorrectly typed pins
	for(const TPair<URigVMPin*, TRigVMTypeIndex>& Pair : PinTypesToChange)
	{
		URigVMPin* Pin = Pair.Key;
		const TRigVMTypeIndex& ExpectedTypeIndex = Pair.Value;
		
		if(!Pin->IsWildCard())
		{
			if(Pin->GetTypeIndex() != ExpectedTypeIndex)
			{
				if (Pin->GetDirection() != ERigVMPinDirection::Hidden)
				{
					const int32 WildCardIndex = Pin->IsArray() ? RigVMTypeUtils::TypeIndex::WildCardArray : RigVMTypeUtils::TypeIndex::WildCard;
					if(!ChangePinType(Pin, WildCardIndex, bSetupUndoRedo, false, true, true))
					{
						if(bSetupUndoRedo)
						{
							CancelUndoBracket();
						}
						return false;
					}
				}
			}
		}
		if(!ChangePinType(Pin, ExpectedTypeIndex, bSetupUndoRedo, false, true, true))
		{
			if(bSetupUndoRedo)
			{
				CancelUndoBracket();
			}
			return false;
		}
	}

	// remove obsolete pins
	for(URigVMPin* PinToRemove : PinsToRemove)
	{
		RemovePin(PinToRemove, false);
	}

	InNode->IncrementVersion();

	// add missing pins
	if(ResolvedFunction)
	{
		if(ResolvedFunction->Struct == nullptr)
		{
			for(const FRigVMTemplateArgument& MissingPin : MissingPins)
			{
				check(MissingPin.GetDirection() == ERigVMPinDirection::Hidden);
				
				URigVMPin* Pin = NewObject<URigVMPin>(Cast<UObject>(InNode), MissingPin.GetName());

				const TRigVMTypeIndex TypeIndex = MissingPin.GetTypeIndex(InNode->ResolvedPermutation);
				const FRigVMTemplateArgumentType& Type = FRigVMRegistry::Get().GetType(TypeIndex); 
				
				Pin->Direction = MissingPin.GetDirection();
				Pin->CPPType = Pin->LastKnownCPPType = Type.CPPType.ToString();
				Pin->CPPTypeObject = Type.CPPTypeObject;
				Pin->CPPTypeObjectPath = Type.GetCPPTypeObjectPath();
				Pin->LastKnownTypeIndex = TypeIndex;

				if(Factory)
				{
					const FName DisplayNameText = Factory->GetDisplayNameForArgument(MissingPin.GetName(), InNode->GetTemplatePinTypeMap(true));
					if(!DisplayNameText.IsNone())
					{
						Pin->DisplayName = *DisplayNameText.ToString();
					}
				}
				
				AddNodePin(InNode, Pin);
				Notify(ERigVMGraphNotifType::PinAdded, Pin);

				// we don't need to set the default value here since the pin is hidden
			}
		}
		else
		{
			for(const FRigVMTemplateArgument& MissingPin : MissingPins)
			{
				check(MissingPin.GetDirection() == ERigVMPinDirection::Hidden);
				
				FProperty* Property = ResolvedFunction->Struct->FindPropertyByName(MissingPin.GetName());
				check(Property);

				URigVMPin* Pin = NewObject<URigVMPin>(Cast<UObject>(InNode), MissingPin.GetName());
				ConfigurePinFromProperty(Property, Pin, MissingPin.GetDirection());

				AddNodePin(InNode, Pin);
				Notify(ERigVMGraphNotifType::PinAdded, Pin);

				// we don't need to set the default value here since the pin is hidden
			}
		}
	}

	if(bSetupUndoRedo)
	{
#if WITH_EDITOR
		RegisterUseOfTemplate(InNode);
#endif
		CloseUndoBracket();
	}

	UpdateGraphSectionsIfRequired();

	return true;
}

bool URigVMController::ResolveWildCardPin(const FString& InPinPath, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	UObject* CPPTypeObject = nullptr;
	if (!InCPPTypeObjectPath.IsNone())
	{
		CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath<UObject>(InCPPTypeObjectPath.ToString());
		if (CPPTypeObject == nullptr)
		{
			ReportErrorf(TEXT("Cannot find cpp type object for path '%s'."), *InCPPTypeObjectPath.ToString());
			return false;
		}
	}

	const FString CPPType = RigVMTypeUtils::PostProcessCPPType(InCPPType, CPPTypeObject);

	if (URigVMPin* Pin = Graph->FindPin(InPinPath))
	{
		if (ResolveWildCardPin(Pin, FRigVMTemplateArgumentType(*CPPType, CPPTypeObject), bSetupUndoRedo, bPrintPythonCommand))
		{
			if (bPrintPythonCommand)
			{
				const FString GraphName = GetSchema()->GetSanitizedGraphName(GetGraph()->GetGraphName());

				// bool ResolveWildCardPin(const FString& InPinPath, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
				RigVMPythonUtils::Print(GetSchema()->GetGraphOuterName(GetGraph()), 
									FString::Printf(TEXT("asset.get_controller_by_name('%s').resolve_wild_card_pin('%s', '%s', '%s')"),
													*GraphName,
													*InPinPath,
													*InCPPType,
													*InCPPTypeObjectPath.ToString()));
			}
			
			return true;
		}
	}

	return false;
}

bool URigVMController::ResolveWildCardPin(URigVMPin* InPin, const FRigVMTemplateArgumentType& InType, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	return ResolveWildCardPin(InPin, FRigVMRegistry::Get().GetTypeIndex(InType), bSetupUndoRedo, bPrintPythonCommand);
}

bool URigVMController::ResolveWildCardPin(const FString& InPinPath, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo,
	bool bPrintPythonCommand)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMPin* Pin = Graph->FindPin(InPinPath))
	{
		return ResolveWildCardPin(Pin, InTypeIndex, bSetupUndoRedo, bPrintPythonCommand);
	}
	return false;
}

bool URigVMController::ResolveWildCardPin(URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	if (InPin->IsStructMember())
	{
		return false;
	}

	if(FRigVMRegistry::Get().IsWildCardType(InTypeIndex))
	{
		return false;
	}

	if (InPin->GetTypeIndex() == InTypeIndex)
	{
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	URigVMPin* RootPin = InPin;
	TRigVMTypeIndex Type = InTypeIndex;	
	while (RootPin->IsArrayElement())
	{
		RootPin = InPin->GetParentPin();
		Type = FRigVMRegistry::Get().GetArrayTypeFromBaseTypeIndex(Type);
	}
	
	ensure(RootPin->GetNode()->IsA<URigVMTemplateNode>());
	URigVMTemplateNode* TemplateNode = CastChecked<URigVMTemplateNode>(RootPin->GetNode());

	TRigVMTypeIndex NewType = INDEX_NONE;
	TemplateNode->SupportsType(RootPin, Type, &NewType);
	if (NewType != INDEX_NONE)
	{
		// We support the new type, and its different than the pin type
		if (NewType == Type && InPin->GetTypeIndex() != NewType)
		{
			 
		}
		// We support the new type, but its different than the one provided
		else if (NewType != Type)
		{
			// if its the same as the pin, we are done
			if (InPin->GetTypeIndex() == NewType)
			{
				return false;
			}
			Type = NewType;
		}
	}
	else
	{
		return false;
	}
	
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Resolve Wildcard Pin"));
		GetActionStack()->BeginAction(Action);
	}
	
	FRigVMControllerCompileBracketScope CompileScope(this);

	if (!InPin->IsWildCard())
	{
		if (!UnresolveTemplateNodes({TemplateNode}, bSetupUndoRedo))
		{
			return false;
		}
	}
	
	if (!ChangePinType(RootPin, Type, bSetupUndoRedo, true, true, false))
	{
		if (bSetupUndoRedo)
		{
			GetActionStack()->CancelAction(Action);
		}
		return false;
	}

	UpdateTemplateNodePinTypes(TemplateNode, bSetupUndoRedo);
	
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(TemplateNode))
	{
		if (DispatchNode->GetFactory()->GetScriptStruct() == FRigVMDispatch_SwitchEnum::StaticStruct())
		{
			URigVMPin* IndexPin = DispatchNode->FindPin(FRigVMDispatch_SwitchEnum::IndexName.ToString());
			URigVMPin* CasesPin = DispatchNode->FindPin(FRigVMDispatch_SwitchEnum::CasesName.ToString());
			if (IndexPin && CasesPin)
			{
				if (UEnum* Enum = Cast<UEnum>(IndexPin->GetCPPTypeObject()))
				{
					const int32 NumCases = Enum->NumEnums() - 1; // Exclude 'MAX' enum value.
					int32 SubPinIndex = 0;
					
					ClearArrayPin(CasesPin->GetPinPath(), bSetupUndoRedo);
					CasesPin->GetSubPins()[0]->Rename(TEXT("OldName"), nullptr); // Make sure no subpins will have a name clash
					TArray<FString> Names;
					Names.Reserve(NumCases);
					for (int32 i=0; i<NumCases; ++i)
					{
						if (Enum->HasMetaData(TEXT("Hidden"), i))
						{
							continue;
						}
					
						if (SubPinIndex > 0)
						{
							AddArrayPin(CasesPin->GetPinPath(), FString(), bSetupUndoRedo);
						}
						Names.Add(FString::FromInt(i));
						SubPinIndex++;
					}
					
					for (int32 i=Names.Num()-1; i>= 0; --i)
					{
						URigVMPin* SubPin = CasesPin->GetSubPins()[i];
						SubPin->Rename(*Names[i], nullptr);
						SubPin->CachedDisplayName.ResetCachedValue();
						SubPin->CachedPinPath.ResetCachedValue();
						SubPin->CachedPinPathWithNodePath.ResetCachedValue();
						Notify(ERigVMGraphNotifType::PinDisplayNameChanged, SubPin);
					}
				}
			}
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	UpdateGraphSectionsIfRequired();
	
	return true;

}

bool URigVMController::UpdateTemplateNodePinTypes(URigVMTemplateNode* InNode, bool bSetupUndoRedo, bool bInitializeDefaultValue, TMap<URigVMPin*, TArray<TRigVMTypeIndex>> ProposedTypes)
{
	URigVMGraph* Graph = GetGraph();
	check(InNode->GetGraph() == Graph);
	bool bAnyTypeChanged = false;

	const FRigVMTemplate* Template = InNode->GetTemplate();
	if(Template == nullptr)
	{
		return false;
	}

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();

	if (InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		return false;
	}

	TArray<int32> ResolvedPermutations = InNode->GetResolvedPermutationIndices(true);
	InNode->ResolvedPermutation = ResolvedPermutations.Num() == 1 ? ResolvedPermutations[0] : INDEX_NONE;

	const TArray<URigVMPin*>& Pins = InNode->GetPins();
	TArray<const FRigVMTemplateArgument*> Arguments;
	const int32 NumPins = Pins.Num();
	Arguments.SetNumUninitialized(NumPins);
	for(int32 PinIndex=0; PinIndex < NumPins; ++PinIndex)
	{
		URigVMPin* Pin = Pins[PinIndex];
		Arguments[PinIndex] = Template->FindArgument(Pin->GetFName());
	}

	// Remove invalid permutations
	{
		FRigVMRegistryReadLock RegistryReadLock(Registry);
		ResolvedPermutations.RemoveAllSwap([&](const int32& Permutation)
		{
			for (const FRigVMTemplateArgument* Argument : Arguments)
			{
				if (Argument && Argument->GetTypeIndex_NoLock(Permutation, RegistryReadLock) == INDEX_NONE)
				{
					return true;
				}
			}
			return false;
		}, EAllowShrinking::No);
	}

	FRigVMDispatchContext DispatchContext;
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(InNode))
	{
		DispatchContext = DispatchNode->GetDispatchContext();
	}

	// Find the types for each possible permutation
	TMap<int32, TArray<TRigVMTypeIndex>> PinTypes; // permutation to pin types
	PinTypes.Reserve(ResolvedPermutations.Num() * NumPins);

	{
		FRigVMRegistryReadLock RegistryReadLock(Registry);
		for (int32 ResolvedPermutation : ResolvedPermutations)
		{
			for(int32 PinIndex=0; PinIndex < NumPins; ++PinIndex)
			{
				URigVMPin* Pin = Pins[PinIndex];
				if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
				{
					PinTypes.FindOrAdd(ResolvedPermutation).Add(INDEX_NONE);
					continue;
				}

				bool bAddedType = false;
				TArray<TRigVMTypeIndex>& Types = PinTypes.FindOrAdd(ResolvedPermutation);
				if (const FRigVMTemplateArgument* Argument = Arguments[PinIndex])
				{
					Types.Add(Argument->GetTypeIndex_NoLock(ResolvedPermutation, RegistryReadLock));
					bAddedType = true;
				}
				else if (const FRigVMExecuteArgument* ExecuteArgument = Template->FindExecuteArgument_NoLock(Pin->GetFName(), DispatchContext, RegistryReadLock))
				{
					Types.Add(ExecuteArgument->TypeIndex);
					bAddedType = true;
				}
				
				if (!bAddedType)
				{
					// This is a pin with no argument
					// Its marked as invalid, and will show wildcard type
					Types.Add(INDEX_NONE);
				}
			}
		}
	}

	// Some pins can benefit from reducing types to a single option
	// Even if some pins reduce to a single type (maybe represented by a single permutation), the rest should still display a wildcard pin
	// If reduction happens for multiple pins, we need to make sure that they reduce to the same permutation
	// If possible, we want to respect the current pin type if its not wildcard
	TArray<bool> WasReduced;
	WasReduced.SetNumZeroed(NumPins);

	TMap<int32, TArray<TRigVMTypeIndex>> ReducedTypes = PinTypes;
	TArray<TRigVMTypeIndex> FinalPinTypes;
	FinalPinTypes.SetNumUninitialized(Pins.Num());
	
	// Reduce compatible types, try to find the permutation in the reduced types (from the proposed types)
	FRigVMTemplateTypeMap AlreadyResolvedTypeMap;
	for(int32 PinIndex=0; PinIndex < NumPins; ++PinIndex)
	{
		if (WasReduced[PinIndex])
		{
			continue;
		}
		
		URigVMPin* Pin = Pins[PinIndex];
		if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			continue;
		}
		
		if (TArray<TRigVMTypeIndex>* Proposed = ProposedTypes.Find(Pin))
		{
			TArray<TRigVMTypeIndex> ReducedPinTypes;
			ReducedPinTypes.Reserve(ReducedTypes.Num());
			for (auto Pair : ReducedTypes)
			{
				ReducedPinTypes.Add(Pair.Value[PinIndex]);
			}
			// Intersect the types with the input proposed types
			TArray<TRigVMTypeIndex> Intersection = ReducedPinTypes.FilterByPredicate([Proposed](const TRigVMTypeIndex& Type) { return Proposed->Contains(Type); });
			if (Intersection.Num() == 1)
			{
				WasReduced[PinIndex] = true;
				FinalPinTypes[PinIndex] = Intersection[0];
				AlreadyResolvedTypeMap.Add(Pin->GetFName(), FinalPinTypes[PinIndex]);

				// If the argument is hidden, the available PinTypes will all be INDEX_NONE
				if (Pin->Direction != ERigVMPinDirection::Hidden)
				{
					ReducedTypes = ReducedTypes.FilterByPredicate([Intersection, PinIndex](const TPair<int32, TArray<TRigVMTypeIndex>>& Permutation)
					{
						return Permutation.Value[PinIndex] == Intersection[0];
					});
				}
			}
		}
	}

	if (PinTypes.Num() > ReducedTypes.Num())
	{
		ensureMsgf(!ReducedTypes.IsEmpty(), TEXT("Found incompatible preferred types"));
	}

	for(int32 PinIndex=0; PinIndex < NumPins; ++PinIndex)
	{
		URigVMPin* Pin = Pins[PinIndex];
		if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			continue;
		}
		TArray<TRigVMTypeIndex> Types;
		Types.Reserve(ResolvedPermutations.Num());
		TRigVMTypeIndex PreferredType = INDEX_NONE;
		int32 TypesFoundInReduced = 0;
		for (int32 ResolvedPermutation : ResolvedPermutations)
		{
			Types.Add(PinTypes.FindChecked(ResolvedPermutation)[PinIndex]);
			if (ReducedTypes.Contains(ResolvedPermutation))
			{
				TypesFoundInReduced++;
				if (PreferredType == INDEX_NONE)
				{
					PreferredType = PinTypes.FindChecked(ResolvedPermutation)[PinIndex];
				}
			}
		}

		// Find the options left in the already reduced pins
		if (TypesFoundInReduced > 1 && !AlreadyResolvedTypeMap.IsEmpty())
		{
			FRigVMTemplateTypeMap OutTypeMap = AlreadyResolvedTypeMap;
			TArray<int32> OutPermutations;
			Template->Resolve(OutTypeMap, OutPermutations, false);
			if (TRigVMTypeIndex* TypeIndex = OutTypeMap.Find(Pin->GetFName()))
			{
				if (*TypeIndex != INDEX_NONE && !Registry.IsWildCardType(*TypeIndex))
				{
					PreferredType = *TypeIndex;
					TypesFoundInReduced = 1;
				}
			}
		}

		if (TypesFoundInReduced > 1)
		{
			PreferredType = Pin->GetTypeIndex();
		}
		FinalPinTypes[PinIndex] = InNode->TryReduceTypesToSingle(Types, PreferredType);
		if (FinalPinTypes[PinIndex] != INDEX_NONE)
		{
			WasReduced[PinIndex] =  true;
			AlreadyResolvedTypeMap.Add(Pin->GetFName(), FinalPinTypes[PinIndex]);
		}

		// Remove reduced types which do not match this type
		TArray<int32> PermutationToRemove;
		PermutationToRemove.Reserve(ReducedTypes.Num());
		for (TPair<int32, TArray<TRigVMTypeIndex>>& Pair : ReducedTypes)
		{
			if (Pair.Value[PinIndex] != FinalPinTypes[PinIndex])
			{
				PermutationToRemove.Add(Pair.Key);
			}
		}
		for (int32& ToRemove : PermutationToRemove)
		{
			ReducedTypes.Remove(ToRemove);
		}
	}
	
	// First unresolve any pins that need unresolving, then resolve everything else
	for (int32 UnresolveResolve=0; UnresolveResolve<2; ++UnresolveResolve)
	{
		for(int32 PinIndex=0; PinIndex < NumPins; ++PinIndex)
		{
			URigVMPin* Pin = Pins[PinIndex];
			if (UnresolveResolve == 0 && Pin->IsWildCard())
			{
				continue;
			}
			
			if (Pin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				continue;
			}

			bool bShouldUnresolve = FinalPinTypes[PinIndex] == INDEX_NONE;
			// If we are about to resolve the pin to a different type, first unresolve it
			if (!bShouldUnresolve && UnresolveResolve == 0 && Pin->GetTypeIndex() != FinalPinTypes[PinIndex])
			{
				bShouldUnresolve = true;
			}

			if (bShouldUnresolve)
			{
				// Unresolve
				if (Pin->HasInjectedNodes())
				{
					EjectNodeFromPin(Pin, bSetupUndoRedo);
				}
				
				const FRigVMTemplateArgument* Argument = Template->FindArgument(*Pin->GetName());
				FRigVMTemplateArgument::EArrayType ArrayType;
				if (Argument)
				{
					ArrayType = Argument->GetArrayType();
				}
				else
				{
					ArrayType = Pin->IsArray() ? FRigVMTemplateArgument::EArrayType::EArrayType_ArrayValue : FRigVMTemplateArgument::EArrayType::EArrayType_SingleValue;
				}
				
				FString CPPType = RigVMTypeUtils::GetWildCardCPPType();
				UObject* CPPObjectType = RigVMTypeUtils::GetWildCardCPPTypeObject();

				if (ArrayType == FRigVMTemplateArgument::EArrayType_ArrayValue)
				{
					CPPType = RigVMTypeUtils::GetWildCardArrayCPPType();
				}
				else if(ArrayType == FRigVMTemplateArgument::EArrayType_Mixed)
				{
					CPPType = Pin->IsArray() ? RigVMTypeUtils::GetWildCardArrayCPPType() : RigVMTypeUtils::GetWildCardCPPType();
				}

				// execute pins are no longer part of the template, avoid changing the type
				if(Pin->IsExecuteContext() && !Pin->GetNode()->IsA<URigVMRerouteNode>())
				{
					check(Argument == nullptr);
					MakeExecutePin(Pin);
					continue;
				}

				if (Pin->GetCPPType() != CPPType || Pin->GetCPPTypeObject() != CPPObjectType)
				{
					check(!Pin->IsExecuteContext() || Pin->GetNode()->IsA<URigVMRerouteNode>());
					ReportPinTypeChange(Pin, CPPType);
					ChangePinType(Pin, CPPType, CPPObjectType, bSetupUndoRedo, false, false, false, bInitializeDefaultValue);
					bAnyTypeChanged = true;
				}
			}
			else 
			{
				// Resolve
				if (Pin->GetTypeIndex() != FinalPinTypes[PinIndex])
				{
					bAnyTypeChanged = !Pin->IsExecuteContext();
					if(bAnyTypeChanged)
					{
						const FString CPPType = Registry.GetType(FinalPinTypes[PinIndex]).CPPType.ToString();
						ReportPinTypeChange(Pin, CPPType);
					}
					ChangePinType(Pin, FinalPinTypes[PinIndex], bSetupUndoRedo, false, false, false, bInitializeDefaultValue);
				}
			}
		}
	}

	bool bHasWildcard = InNode->HasWildCardPin();
	
	if (bHasWildcard)
	{
		InNode->ResolvedPermutation = INDEX_NONE;
	}
	else
	{
		// If we got into a resolved permutation through reducing types, we need to set the result in the node
		ResolvedPermutations = InNode->GetResolvedPermutationIndices(false);
		check(ResolvedPermutations.Num() == 1);
		InNode->ResolvedPermutation = ResolvedPermutations[0];
	}
	
	if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InNode))
	{
		if (const FRigVMFunction* Function = UnitNode->GetResolvedFunction())
		{
			UnitNode->ResolvedFunctionName = Function->GetName();
		}
	}
	
	return bAnyTypeChanged;
}

bool URigVMController::PrepareToLink(URigVMPin* FirstToResolve, URigVMPin* SecondToResolve, bool bSetupUndoRedo)
{
	FRigVMRegistry& Registry = FRigVMRegistry::Get();
	
	// Check if there is anything to do
	if (!FirstToResolve->IsWildCard() &&
		!SecondToResolve->IsWildCard() &&
		Registry.CanMatchTypes(FirstToResolve->GetTypeIndex(), SecondToResolve->GetTypeIndex(), true))
	{
		return true;
	}
	
	// Find out the matching supported types
	TArray<TRigVMTypeIndex> MatchingTypes;
	{
		auto GetPinSupportedTypes = [&Registry](URigVMPin* Pin) -> TArray<TRigVMTypeIndex>
		{
			URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Pin->GetNode());
			if (!TemplateNode || TemplateNode->IsSingleton() || !Pin->IsWildCard())
			{
				return {Pin->GetTypeIndex()};
			}
			else
			{
				if (const FRigVMTemplate* Template = TemplateNode->GetTemplate())
				{
					URigVMPin* RootPin = Pin;
					uint8 ArrayLevels = 0;
					while (RootPin->GetParentPin() != nullptr)
					{
						ArrayLevels++;
						RootPin = RootPin->GetParentPin();
					}
					if (const FRigVMTemplateArgument* Argument = Template->FindArgument(RootPin->GetFName()))
					{
						TArray<int32> ResolvedPermutations = TemplateNode->GetResolvedPermutationIndices(true);
						TArray<TRigVMTypeIndex> Types = Argument->GetSupportedTypeIndices(ResolvedPermutations);
						for (int32 i=0; i<ArrayLevels; ++i)
						{
							for (TRigVMTypeIndex& Type : Types)
							{
								Type = Registry.GetBaseTypeFromArrayTypeIndex(Type);
							}
						}
						return Types;
					}
				}
			}
			return {};
		};
		MatchingTypes = GetPinSupportedTypes(FirstToResolve);
		TArray<TRigVMTypeIndex> SecondTypes = GetPinSupportedTypes(SecondToResolve);
		MatchingTypes.RemoveAll([&Registry, SecondTypes](const TRigVMTypeIndex& FirstType)
		{
			return !SecondTypes.ContainsByPredicate([&Registry, FirstType](const TRigVMTypeIndex& SecondType)
			{
				return Registry.CanMatchTypes(FirstType, SecondType, true);
			});
		});
	}

	if (MatchingTypes.IsEmpty())
	{
		return false;
	}

	// reduce matching types by duplicate entries for float / double
	{
		TArray<TRigVMTypeIndex> FilteredMatchingTypes;
		FilteredMatchingTypes.Reserve(MatchingTypes.Num());
		for(const TRigVMTypeIndex& MatchingType : MatchingTypes)
		{
			// special case float singe & array
			if(MatchingType == RigVMTypeUtils::TypeIndex::Float)
			{
				if(MatchingTypes.Contains(RigVMTypeUtils::TypeIndex::Double))
				{
					continue;
				}
			}
			if(MatchingType == RigVMTypeUtils::TypeIndex::FloatArray)
			{
				if(MatchingTypes.Contains(RigVMTypeUtils::TypeIndex::DoubleArray))
				{
					continue;
				}
			}

			bool bAlreadyContainsMatch = false;
			for(const TRigVMTypeIndex& FilteredType : FilteredMatchingTypes)
			{
				if(Registry.CanMatchTypes(MatchingType, FilteredType, true))
				{
					bAlreadyContainsMatch = true;
					break;
				}
			}
			if(bAlreadyContainsMatch)
			{
				continue;
			}
			FilteredMatchingTypes.Add(MatchingType);
		}
		Swap(FilteredMatchingTypes, MatchingTypes);
	}

	TRigVMTypeIndex FinalType = INDEX_NONE;
	if (MatchingTypes.Num() > 1)
	{
		// Query the user for the type to resolve the pin (from the list of MatchingTypes)
		if (RequestPinTypeSelectionDelegate.IsBound())
		{
			FinalType = RequestPinTypeSelectionDelegate.Execute(MatchingTypes);
		}
		else
		{
			return false;
		}
	}
	else
	{
		FinalType = MatchingTypes[0];
	}
	
	// If only one match, resolve both pins to that
	bool bSuccess = true;
	if (FinalType != INDEX_NONE)
	{
		if (FirstToResolve->IsWildCard())
		{
			bSuccess &= ResolveWildCardPin(FirstToResolve, FinalType, bSetupUndoRedo);
		}
		if (SecondToResolve->IsWildCard())
		{
			bSuccess &= ResolveWildCardPin(SecondToResolve, FinalType, bSetupUndoRedo);
		}
	}
	else
	{
		return false;
	}

	return bSuccess;
}

bool URigVMController::ChangePinType(const FString& InPinPath, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bSetupOrphanPins, bool bBreakLinks, bool bRemoveSubPins, bool bInitializeDefaultValue)
{
	if (!IsValidGraph())
	{
		return false;
	}

	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}

	URigVMGraph* Graph = GetGraph();
	check(Graph);

	if (URigVMPin* Pin = Graph->FindPin(InPinPath))
	{
		return ChangePinType(Pin, InCPPType, InCPPTypeObjectPath, bSetupUndoRedo, bSetupOrphanPins, bBreakLinks, bRemoveSubPins, bInitializeDefaultValue);
	}

	return false;
}

bool URigVMController::ChangePinType(URigVMPin* InPin, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bSetupOrphanPins, bool bBreakLinks, bool bRemoveSubPins, bool bInitializeDefaultValue)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (InCPPType == TEXT("None") || InCPPType.IsEmpty())
	{
		return false;
	}

	FRigVMControllerGraphSectionsScope GraphSectionsScope(this);

	UObject* CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InCPPTypeObjectPath.ToString());

	// always refresh pin type if it is a user defined struct, whose internal layout can change at anytime
	bool bForceRefresh = false;
	if (CPPTypeObject && CPPTypeObject->IsA<UUserDefinedStruct>())
	{
		bForceRefresh = true;
	}

	if (!bForceRefresh)
	{
		if (InPin->CPPType == InCPPType && InPin->CPPTypeObject == CPPTypeObject)
		{
			return true;
		}
	}

	return ChangePinType(InPin, InCPPType, CPPTypeObject, bSetupUndoRedo, bSetupOrphanPins, bBreakLinks, bRemoveSubPins, bInitializeDefaultValue);
}

bool URigVMController::ChangePinType(URigVMPin* InPin, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins, bool bBreakLinks, bool bRemoveSubPins, bool bInitializeDefaultValue)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (InCPPType == TEXT("None") || InCPPType.IsEmpty())
	{
		return false;
	}

	if (RigVMTypeUtils::RequiresCPPTypeObject(InCPPType) && !InCPPTypeObject)
	{
		return false;
	}

	const FRigVMTemplateArgumentType Type(*InCPPType, InCPPTypeObject);
	// pin types are chosen from the graph pin type menu so it is not guaranteed that the chosen type
	// is registered, hence the use of FindOrAddType
	const TRigVMTypeIndex TypeIndex = FRigVMRegistry::Get().FindOrAddType(Type);
	return ChangePinType(InPin, TypeIndex, bSetupUndoRedo, bSetupOrphanPins, bBreakLinks, bRemoveSubPins, bInitializeDefaultValue);
}

bool URigVMController::ChangePinType(URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo, bool bSetupOrphanPins, bool bBreakLinks, bool bRemoveSubPins, bool bInitializeDefaultValue)
{
	if (!bIsTransacting && !IsGraphEditable())
	{
		return false;
	}
	
	if (InTypeIndex == INDEX_NONE)
	{
		return false;
	}

	check(InPin->GetGraph() == GetGraph());
	if(InPin->IsExecuteContext() && FRigVMRegistry::Get().IsExecuteType(InTypeIndex))
	{
		return false;
	}

	if(!GetSchema()->SupportsType(this, InTypeIndex))
	{
		return false;
	}

	if(InPin->IsTraitPin() && InPin->GetRootPin())
	{
		return false;
	}

	// only allow valid pin cpp types on template nodes
	TRigVMTypeIndex TypeIndex = InTypeIndex;
	if(URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InPin->GetNode()))
	{
		if (!bIsTransacting)
		{
			if (InPin->GetDirection() != ERigVMPinDirection::Hidden)
			{
				if (TemplateNode->IsA<URigVMUnitNode>() || TemplateNode->IsA<URigVMDispatchNode>())
				{
					if(!TemplateNode->SupportsType(InPin, InTypeIndex))
					{
						ReportErrorf(TEXT("ChangePinType: %s doesn't support type '%s'."), *InPin->GetPinPath(), *FRigVMRegistry::Get().GetType(InTypeIndex).CPPType.ToString());
						return false;
					}
				}
			}
		}

		// If changing to wildcard, try to maintain the container type
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		if (Registry.IsWildCardType(TypeIndex)) 
		{
			const bool bIsArrayType = FRigVMRegistry::Get().IsArrayType(TypeIndex);
			if(InPin->IsRootPin() && bIsArrayType != InPin->IsArray())
			{
				// nothing to do here - leave the type as is 
			}
			else
			{
				const TRigVMTypeIndex BaseTypeIndex = bIsArrayType ? FRigVMRegistry::Get().GetBaseTypeFromArrayTypeIndex(TypeIndex) : TypeIndex;
				TypeIndex = InPin->IsArray() ? Registry.GetArrayTypeFromBaseTypeIndex(BaseTypeIndex) : BaseTypeIndex;
			}
		}
	}

	FRigVMControllerCompileBracketScope CompileScope(this);
	FRigVMBaseAction Action(this);
	if (bSetupUndoRedo)
	{
		Action.SetTitle(TEXT("Change pin type"));
		GetActionStack()->BeginAction(Action);
	}

	TArray<FLinkedPath> LinkedPaths;

	if (bSetupUndoRedo)
	{
		if(!bSetupOrphanPins && bBreakLinks)
		{
			BreakAllLinks(InPin, true, true);
			BreakAllLinks(InPin, false, true);
			BreakAllLinksRecursive(InPin, true, false, true);
			BreakAllLinksRecursive(InPin, false, false, true);
		}
	}
	
	if(bSetupOrphanPins)
	{
		LinkedPaths = GetLinkedPaths(InPin, true, true);
		if(!LinkedPaths.IsEmpty())
		{
			BreakLinkedPaths(LinkedPaths, bSetupUndoRedo, bBreakLinks);

			const FString OrphanedName = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *InPin->GetName());
			if(InPin->GetNode()->FindPin(OrphanedName) == nullptr)
			{
				URigVMPin* OrphanedPin = NewObject<URigVMPin>(InPin->GetNode(), *OrphanedName);
				ConfigurePinFromPin(OrphanedPin, InPin);
				OrphanedPin->DisplayName = InPin->GetFName();

				InPin->GetNode()->OrphanedPins.Add(OrphanedPin);
				Notify(ERigVMGraphNotifType::PinAdded, OrphanedPin);

				if(OrphanedPin->IsStruct())
				{
					AddPinsForStruct(OrphanedPin->GetScriptStruct(), OrphanedPin->GetNode(), OrphanedPin, OrphanedPin->Direction, OrphanedPin->GetDefaultValue(), false);
				}
				
				OrphanedPin->ClearPinPathCache();
			}
		}
	}

	if(bRemoveSubPins || !InPin->IsArray())
	{
		TArray<URigVMPin*> Pins = InPin->SubPins;
		for (URigVMPin* Pin : Pins)
		{
			RemovePin(Pin, bSetupUndoRedo);
		}
		
		InPin->SubPins.Reset();
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->AddAction(FRigVMChangePinTypeAction(this, InPin, TypeIndex, bSetupOrphanPins, bBreakLinks, bRemoveSubPins));
	}
	
	// compute the number of remaining wildcard pins
	auto WildCardPinCountPredicate = [](const URigVMPin* Pin) { return Pin->IsWildCard(); };
	TArray<URigVMPin*> AllPins = InPin->GetNode()->GetAllPinsRecursively();
	int32 RemainingWildCardPins = static_cast<int32>(Algo::CountIf(AllPins, WildCardPinCountPredicate));
	const bool bPinWasWildCard = InPin->IsWildCard();

	const FPinState PreviousPinState = GetPinState(InPin);
	const FString PreviousCPPType = InPin->CPPType;
	
	const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(TypeIndex);
	InPin->CPPType = Type.CPPType.ToString();
	InPin->CPPTypeObjectPath = Type.GetCPPTypeObjectPath();
	InPin->CPPTypeObject = Type.CPPTypeObject;
	InPin->bIsDynamicArray = FRigVMRegistry::Get().IsArrayType(TypeIndex);

	if (bInitializeDefaultValue)
	{
		InPin->DefaultValue = FString();

		if(InPin->IsRootPin() && !InPin->IsWildCard())
		{
			if(URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InPin->GetNode()))
			{
				const FString InitialDefaultValue = TemplateNode->GetInitialDefaultValueForPin(InPin->GetFName());
				if (InPin->IsUObject())
				{
					InPin->DefaultValueObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(InitialDefaultValue);
				}
				else
				{
					InPin->DefaultValue = InitialDefaultValue;
				}
			}
		}

		const FString DefaultValue = InPin->GetDefaultValue();
		if(!DefaultValue.IsEmpty())
		{
			InPin->DefaultValueType = GetDefaultValueType(InPin, DefaultValue);
		}
	}

	if (InPin->IsExecuteContext() && !InPin->GetNode()->IsA<URigVMFunctionEntryNode>() && !InPin->GetNode()->IsA<URigVMFunctionReturnNode>())
	{
		InPin->Direction = ERigVMPinDirection::IO;
	}

	if (InPin->IsStruct() && !InPin->IsArray())
	{
		FString DefaultValue = InPin->DefaultValue;
		CreateDefaultValueForStructIfRequired(InPin->GetScriptStruct(), DefaultValue);
		AddPinsForStruct(InPin->GetScriptStruct(), InPin->GetNode(), InPin, InPin->Direction, DefaultValue, false);
	}

	if (InPin->IsArray())
	{
		const TRigVMTypeIndex BaseTypeIndex = FRigVMRegistry::Get().GetBaseTypeFromArrayTypeIndex(TypeIndex);
		for (int32 i=0; i<InPin->GetSubPins().Num(); ++i)
		{
			URigVMPin* SubPin = InPin->GetSubPins()[i];
			if (SubPin->GetDirection() == ERigVMPinDirection::Hidden)
			{
				continue;
			}
			ChangePinType(SubPin, BaseTypeIndex, bSetupUndoRedo, bSetupOrphanPins, bBreakLinks, bRemoveSubPins, bInitializeDefaultValue);
		}
	}

	// if the pin didn't change type - let's maintain the pin state
	if(PreviousCPPType == InPin->CPPType && !InPin->IsWildCard())
	{
		ApplyPinState(InPin, PreviousPinState, false);
	}

	// if this is a template clear its caches
	if(URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InPin->GetNode()))
	{
		TemplateNode->InvalidateCache();
	}

	InPin->IncrementVersion();
	Notify(ERigVMGraphNotifType::PinTypeChanged, InPin);
	Notify(ERigVMGraphNotifType::PinDefaultValueChanged, InPin);

	// let's see if this was the last resolved wildcard pin
	if(RemainingWildCardPins > 0)
	{
		// compute the number of current wildcard pins
		RemainingWildCardPins = 0;
		if(InPin->GetNode()->IsA<URigVMTemplateNode>())
		{
			AllPins = InPin->GetNode()->GetAllPinsRecursively();
			RemainingWildCardPins = static_cast<int32>(Algo::CountIf(AllPins, WildCardPinCountPredicate));
		}

		// if this is the first time that there are no wild card pins left
		if(RemainingWildCardPins == 0)
		{
			struct Local
			{
				static bool IsPinDefaultEmpty(URigVMPin* InPin)
				{
					const FString DefaultValue = InPin->GetDefaultValue();
					static const FString EmptyBraces = TEXT("()");
					return DefaultValue.IsEmpty() || DefaultValue == EmptyBraces;
				}
				
				static void ApplyResolvedDefaultValue(
					URigVMController* InController, 
					URigVMPin* InPin, 
					const FString& RemainingPinPath, 
					const FString& InDefaultValue, 
					bool bSetupUndoRedo)
				{
					if(InDefaultValue.IsEmpty())
					{
						return;
					}
					
					if(RemainingPinPath.IsEmpty())
					{
						InController->SetPinDefaultValue(InPin, InDefaultValue, true, bSetupUndoRedo, false);
						return;
					}

					FString PinName;
					FString SubPinPath;
					if(!URigVMPin::SplitPinPathAtStart(RemainingPinPath, PinName, SubPinPath))
					{
						PinName = RemainingPinPath;
						SubPinPath.Empty();
					}

					TArray<FString> MemberValuePairs = URigVMPin::SplitDefaultValue(InDefaultValue);
					for (const FString& MemberValuePair : MemberValuePairs)
					{
						FString MemberName, MemberValue;
						if (MemberValuePair.Split(TEXT("="), &MemberName, &MemberValue))
						{
							if(MemberName.Equals(PinName))
							{
								PostProcessDefaultValue(InPin, MemberValue);
								ApplyResolvedDefaultValue(InController, InPin, SubPinPath, MemberValue, bSetupUndoRedo);
								break;
							}
						}
					}
				}
			};
			
			for(URigVMPin* Pin : AllPins)
			{
				// skip struct pins or array pins
				if(Pin->GetSubPins().Num() > 0)
				{
					continue;
				}
				
				if(!Local::IsPinDefaultEmpty(Pin))
				{
					continue;
				}

				if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Pin->GetNode()))
				{
					if(UnitNode->GetScriptStruct())
					{
						TSharedPtr<FStructOnScope> StructOnScope = UnitNode->ConstructStructInstance(true);
						const FString StructDefaultValue = FRigVMStruct::ExportToFullyQualifiedText(UnitNode->GetScriptStruct(), StructOnScope->GetStructMemory());
						Local::ApplyResolvedDefaultValue(this, Pin, Pin->GetSegmentPath(true), StructDefaultValue, bSetupUndoRedo);
						if(!Local::IsPinDefaultEmpty(Pin))
						{
							continue;
						}
					}
				}

				// create the default value for the parent struct pin
				if(Pin->IsStructMember())
				{
					const URigVMPin* ParentPin = Pin->GetParentPin();
					TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(ParentPin->GetScriptStruct()));
					const FString StructDefaultValue = FRigVMStruct::ExportToFullyQualifiedText(ParentPin->GetScriptStruct(), StructOnScope->GetStructMemory());
					Local::ApplyResolvedDefaultValue(this, Pin, Pin->GetName(), StructDefaultValue, bSetupUndoRedo);
				}
				else
				{
					// plain types within an array or at the root
					FString SimpleTypeDefaultValue;
					if(Pin->GetCPPType() == RigVMTypeUtils::BoolType)
					{
						static const FString BoolDefaultValue = TEXT("False");
						SimpleTypeDefaultValue = BoolDefaultValue;
					}
					else if(Pin->GetCPPType() == RigVMTypeUtils::FloatType || Pin->GetCPPType() == RigVMTypeUtils::DoubleType)
					{
						static const FString FloatingPointDefaultValue = TEXT("0.000000");
						SimpleTypeDefaultValue = FloatingPointDefaultValue;
					}
					else if(Pin->GetCPPType() == RigVMTypeUtils::Int32Type)
					{
						static const FString IntegerDefaultValue = TEXT("0");
						SimpleTypeDefaultValue = IntegerDefaultValue;
					}
					Local::ApplyResolvedDefaultValue(this, Pin, FString(), SimpleTypeDefaultValue, bSetupUndoRedo);
				}
			}

			if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InPin->GetNode()))
			{
				if (!TemplateNode->IsA<URigVMFunctionEntryNode>() && !TemplateNode->IsA<URigVMFunctionReturnNode>())
				{
					// Figure out the permutation from the pin types. During undo, the filtered permutations are not
					// reliable as to which permutation we are resolving to.
					FullyResolveTemplateNode(TemplateNode, INDEX_NONE, bSetupUndoRedo);
				}
			}
		}
	}

	// since the resolved pin may affect the node title we need to let
	// graph views know to invalidate the node title text widget
	Notify(ERigVMGraphNotifType::NodeDescriptionChanged, InPin->GetNode());

	// in cases where we are just changing the type we have to let the
	// clients know that the links are still there
	if(!bSetupOrphanPins && !bBreakLinks && !bRemoveSubPins)
	{
		const TArray<URigVMLink*> CurrentLinks = InPin->GetLinks();
		for(URigVMLink* CurrentLink : CurrentLinks)
		{
			Notify(ERigVMGraphNotifType::LinkRemoved, CurrentLink);
			Notify(ERigVMGraphNotifType::LinkAdded, CurrentLink);
		}
	}

	if (bSetupUndoRedo)
	{
		GetActionStack()->EndAction(Action);
	}

	if(!LinkedPaths.IsEmpty())
	{
		FRestoreLinkedPathSettings Settings;
		Settings.bRelayToOrphanPins = true;
		RestoreLinkedPaths(LinkedPaths, Settings);
		RemoveUnusedOrphanedPins(InPin->GetNode());
	}

	return true;
}

#if WITH_EDITOR

void URigVMController::RewireLinks(URigVMPin* InOldPin, URigVMPin* InNewPin, bool bAsInput, bool bSetupUndoRedo, TArray<URigVMLink*> InLinks)
{
	ensure(InOldPin->GetRootPin() == InOldPin);
	ensure(InNewPin->GetRootPin() == InNewPin);
	FRigVMControllerCompileBracketScope CompileScope(this);

 	if (bAsInput)
	{
		TArray<URigVMLink*> Links = InLinks;
		if (Links.Num() == 0)
		{
			Links = InOldPin->GetSourceLinks(true /* recursive */);
		}

		for (URigVMLink* Link : Links)
		{
			FString SegmentPath = Link->GetTargetPin()->GetSegmentPath();
			URigVMPin* NewPin = SegmentPath.IsEmpty() ? InNewPin : InNewPin->FindSubPin(SegmentPath);
			check(NewPin);

			BreakLink(Link->GetSourcePin(), Link->GetTargetPin(), bSetupUndoRedo);
			AddLink(Link->GetSourcePin(), NewPin, bSetupUndoRedo);
		}
	}
	else
	{
		TArray<URigVMLink*> Links = InLinks;
		if (Links.Num() == 0)
		{
			Links = InOldPin->GetTargetLinks(true /* recursive */);
		}

		for (URigVMLink* Link : Links)
		{
			FString SegmentPath = Link->GetSourcePin()->GetSegmentPath();
			URigVMPin* NewPin = SegmentPath.IsEmpty() ? InNewPin : InNewPin->FindSubPin(SegmentPath);
			check(NewPin);

			BreakLink(Link->GetSourcePin(), Link->GetTargetPin(), bSetupUndoRedo);
			AddLink(NewPin, Link->GetTargetPin(), bSetupUndoRedo);
		}
	}
}

#endif

bool URigVMController::MarkPackageDirty(const UObject* InObjectToDirty) const
{
	if (!bSuspendMarkingPackageDirty && InObjectToDirty)
	{
		return InObjectToDirty->MarkPackageDirty();
	}
	return false;
}

bool URigVMController::RenameObject(UObject* InObjectToRename, const TCHAR* InNewName, UObject* InNewOuter, ERenameFlags InFlags) const
{
	// Renames are commonly done to temporarily move objects to a new outer and then move back (often during loading). 
	// The extra flags are used to support that flow avoiding the creation of redirectors, marking objects for save or affecting the linker state.
	const bool bSuccess = InObjectToRename->Rename(InNewName, InNewOuter, InFlags | REN_AllowPackageLinkerMismatch | REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	if(bSuccess)
	{
		if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InObjectToRename))
		{
			if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
			{
				if(FRigVMClient* Client = ClientHost->GetRigVMClient())
				{
					Client->OnCollapseNodeRenamed(CollapseNode);
				}
			}
		}
	}
	
	if (URigVMPin* Pin = Cast<URigVMPin>(InObjectToRename))
	{
		Pin->ClearPinPathCache();
	}
	
	return bSuccess;
}

void URigVMController::DestroyObject(UObject* InObjectToDestroy) const
{
	// disable the on object renamed callback
	// to avoid flushing the compilation queue during graph changes. 
	static FCoreUObjectDelegates::FOnObjectRenamed EmptyOnObjectRenamed;
	const TGuardValue<FCoreUObjectDelegates::FOnObjectRenamed> _(FCoreUObjectDelegates::OnObjectRenamed, EmptyOnObjectRenamed);
	
	if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InObjectToDestroy))
	{
		if(IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
		{
			if(FRigVMClient* Client = ClientHost->GetRigVMClient())
			{
				Client->OnCollapseNodeRemoved(CollapseNode);
			}
		}
	}

	if (URigVMNode* Node = Cast<URigVMNode>(InObjectToDestroy))
	{
		TArray<URigVMPin*> PinsToDestroy = Node->GetAllPinsRecursively();
		for (int32 i=PinsToDestroy.Num()-1; i>=0; --i)
		{
			const FString DeletedName = GetSchema()->GetValidNodeName(GetGraph(), FString::Printf(TEXT("%s_Deleted"), *PinsToDestroy[i]->GetPinPath()));
			RenameObject(PinsToDestroy[i], *DeletedName, nullptr, false);
			DestroyObject(PinsToDestroy[i]);
		}
	}
	
	if (URigVMPin* Pin = Cast<URigVMPin>(InObjectToDestroy))
	{
		if (URigVMNode* Node = Pin->GetNode())
		{
			Node->Pins.Remove(Pin);
			Node->OrphanedPins.Remove(Pin);
		}
	}

	// Invalidate the future loading of the destroyed object
	FLinkerLoad::InvalidateExport(InObjectToDestroy);
	
	// We are renaming an object that may be loading, but ultimately want to destroy.
	// Pass REN_AllowPackageLinkerMismatch to allow the linker to remain on the object
	// so we may rename without forcing the load to complete.
	RenameObject(InObjectToDestroy, nullptr, GetTransientPackage(), REN_AllowPackageLinkerMismatch);
	InObjectToDestroy->RemoveFromRoot();
	InObjectToDestroy->MarkAsGarbage();
}

URigVMPin* URigVMController::MakeExecutePin(URigVMNode* InNode, const FName& InName)
{
	URigVMPin* ExecutePin = NewObject<URigVMPin>(InNode, InName);
	ExecutePin->DisplayName = FRigVMStruct::ExecuteName;
	MakeExecutePin(ExecutePin);
	return ExecutePin;
}

bool URigVMController::MakeExecutePin(URigVMPin* InOutPin)
{
	if(InOutPin->CPPTypeObject != FRigVMExecuteContext::StaticStruct())
	{
		const bool bIsArray = InOutPin->IsArray();
		InOutPin->CPPType = FRigVMExecuteContext::StaticStruct()->GetStructCPPName();
		InOutPin->CPPTypeObject = FRigVMExecuteContext::StaticStruct();
		InOutPin->CPPTypeObjectPath = *InOutPin->CPPTypeObject->GetPathName();

		if(bIsArray)
		{
			InOutPin->CPPType = RigVMTypeUtils::ArrayTypeFromBaseType(InOutPin->CPPType);
			InOutPin->LastKnownTypeIndex = FRigVMRegistry::Get().GetArrayTypeFromBaseTypeIndex(RigVMTypeUtils::TypeIndex::Execute);
		}
		else
		{
			InOutPin->LastKnownTypeIndex = RigVMTypeUtils::TypeIndex::Execute;
		}
		InOutPin->LastKnownCPPType = InOutPin->CPPType;
		InOutPin->IncrementVersion();
		return true;
	}
	return false;
}

bool URigVMController::CorrectExecutePinsOnNode(URigVMNode* InOutNode)
{
	bool bModified = false;
	for (URigVMPin* Pin : InOutNode->Pins)
	{
		if (Pin->IsExecuteContext())
		{
			bModified |= MakeExecutePin(Pin);
		}
	}
	return bModified;
}

bool URigVMController::AddGraphNode(URigVMNode* InNode, bool bNotify)
{
	URigVMGraph* Graph = GetGraph();
	
	check(Graph);
	check(InNode);

	if(!GetSchema()->CanAddNode(this, InNode))
	{
		if (InNode->IsInjected())
		{
			URigVMInjectionInfo* Info = InNode->GetInjectionInfo();
			if (URigVMPin* Pin = Info->GetPin())
			{
				RemoveInjectedNode(Pin->GetPinPath(), Info->bInjectedAsInput, false);
			}
		}
	
		Graph->Nodes.Remove(InNode);
		DestroyObject(InNode);
		return false;
	}

	Graph->Nodes.AddUnique(InNode);

	if(bNotify)
	{
		Notify(ERigVMGraphNotifType::NodeAdded, InNode);
	}
	return true;
}

void URigVMController::AddNodePin(URigVMNode* InNode, URigVMPin* InPin)
{
	ValidatePin(InPin);
	checkf(!InNode->Pins.Contains(InPin), TEXT("Node %s already contains pin %s"), *InNode->GetPathName(), *InPin->GetName());
	InNode->Pins.Add(InPin);

	// the first time we add a fixed size array we want to expand it
	if(InPin->IsFixedSizeArray())
	{
		InPin->bIsExpanded = true;
	}

	if(InPin->DefaultValueType == ERigVMPinDefaultValueType::AutoDetect)
	{
		const FString DefaultValue = InPin->GetDefaultValue();
		if(DefaultValue.IsEmpty())
		{
			InPin->DefaultValueType = ERigVMPinDefaultValueType::Unset;
		}
		else
		{
			FRigVMDefaultValueTypeGuard _(this, ERigVMPinDefaultValueType::AutoDetect, true);
			InPin->DefaultValueType = GetDefaultValueType(InPin, DefaultValue);
		}
	}

	InNode->IncrementVersion();
}

void URigVMController::AddSubPin(URigVMPin* InParentPin, URigVMPin* InPin)
{
	ValidatePin(InPin);
	checkf(!InParentPin->SubPins.Contains(InPin), TEXT("Parent pin %s already contains subpin %s"), *InParentPin->GetPathName(), *InPin->GetName());
	InParentPin->SubPins.Add(InPin);
	InParentPin->IncrementVersion();
}

bool URigVMController::EnsurePinValidity(URigVMPin* InPin, bool bRecursive)
{
	check(InPin);
	
	// check if the CPPTypeObject is set up correctly.
	if(RigVMTypeUtils::RequiresCPPTypeObject(InPin->GetCPPType()))
	{
		// GetCPPTypeObject attempts to update pin type information to the latest
		// without testing for redirector
		if(InPin->GetCPPTypeObject() == nullptr)
		{
			FRigVMUserDefinedTypeResolver TypeResolver;
			if(const IRigVMClientHost* ClientHost = InPin->GetImplementingOuter<IRigVMClientHost>())
			{
				TypeResolver = FRigVMUserDefinedTypeResolver([ClientHost](const FString& InTypeName) -> UObject*
				{
					return ClientHost->ResolveUserDefinedTypeById(InTypeName);
				});
			}
			
			FString CPPType = InPin->GetCPPType();
			InPin->CPPTypeObject = RigVMTypeUtils::ObjectFromCPPType(CPPType, true, &TypeResolver);
			if(CPPType.IsEmpty())
			{
				return false;
			}
			InPin->CPPType = CPPType;
			InPin->IncrementVersion();
		}
		else
		{
			InPin->CPPType = RigVMTypeUtils::PostProcessCPPType(InPin->CPPType, InPin->GetCPPTypeObject());
			InPin->IncrementVersion();
		}
	}

	if (InPin->GetCPPType().IsEmpty() || InPin->GetCPPType() == FName().ToString())
	{
		return false;
	}
	
	if(bRecursive)
	{
		for(URigVMPin* SubPin : InPin->SubPins)
		{
			if(!EnsurePinValidity(SubPin, bRecursive))
			{
				return false;
			}
		}
	}

	return true;
}


void URigVMController::ValidatePin(URigVMPin* InPin)
{
	check(InPin);
	
	// create a property description from the pin here as a test,
	// since the compiler needs this
	FRigVMPropertyDescription(InPin->GetFName(), InPin->GetCPPType(), InPin->GetCPPTypeObject(), InPin->GetDefaultValue());

	if(InPin->IsExecuteContext())
	{
		ensure(InPin->GetCPPTypeObject() == FRigVMExecuteContext::StaticStruct());
	}
}

bool URigVMController::EnsureLocalVariableValidity()
{
	bool bChanged = false;
	if (URigVMGraph* Graph = GetGraph())
	{
		for (FRigVMGraphVariableDescription& Variable : Graph->LocalVariables)
		{
			// CPPType can become invalid when the type object is defined by
			// an asset that have changed name or asset path, user defined struct is one possibility
			if (RigVMTypeUtils::FixCPPTypeAndObject(Variable.CPPType, Variable.CPPTypeObject))
			{
				bChanged = true;
			}
			
			if (Variable.CPPTypeObject)
			{
				Variable.CPPTypeObjectPath = *Variable.CPPTypeObject.GetPathName();
			}
			else if(!Variable.CPPTypeObjectPath.IsNone())
			{
				Variable.CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(Variable.CPPTypeObjectPath.ToString());
				RigVMTypeUtils::FixCPPTypeAndObject(Variable.CPPType, Variable.CPPTypeObject);
				if (Variable.CPPTypeObject)
				{
					bChanged = true;
				}
			}
			
			if (!Variable.CPPTypeObject)
			{
				Variable.CPPTypeObjectPath = NAME_None;
			}
		}
	}
	return bChanged;
}

FRigVMExternalVariable URigVMController::GetVariableByProperty(const FProperty* InExternalVariableProperty) const
{
	if (GetExternalVariablesDelegate.IsBound())
	{
		const TArray<FRigVMExternalVariable> Variables = GetExternalVariablesDelegate.Execute(GetGraph());
		for (const FRigVMExternalVariable& Variable : Variables)
		{
			if (Variable.GetProperty() == InExternalVariableProperty)
			{
				return Variable;
			}
		}
	}
	
	return FRigVMExternalVariable();
}

FRigVMExternalVariable URigVMController::GetVariableByName(const FName& InExternalVariableName, const bool bIncludeInputArguments) const
{
	TArray<FRigVMExternalVariable> Variables = GetAllVariables(bIncludeInputArguments);
	for (const FRigVMExternalVariable& Variable : Variables)
	{
		if (Variable.GetName() == InExternalVariableName)
		{
			return Variable;
		}
	}	
	
	return FRigVMExternalVariable();
}

FRigVMExternalVariable URigVMController::GetVariableByGuid(const FGuid& InExternalVariableGuid, const bool bIncludeInputArguments) const
{
	TArray<FRigVMExternalVariable> Variables = GetAllVariables(bIncludeInputArguments);
	for (const FRigVMExternalVariable& Variable : Variables)
	{
		if (Variable.GetGuid() == InExternalVariableGuid)
		{
			return Variable;
		}
	}
	
	return FRigVMExternalVariable();
}

TArray<FRigVMExternalVariable> URigVMController::GetAllVariables(const bool bIncludeInputArguments) const
{
	TArray<FRigVMExternalVariable> ExternalVariables;

	if(URigVMGraph* Graph = GetGraph())
	{
		for (FRigVMGraphVariableDescription LocalVariable : Graph->GetLocalVariables(bIncludeInputArguments))
		{
			FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, LocalVariable.ToExternalVariable());
		}
		for (FRigVMGraphVariableDescription InputVariable : Graph->GetInputVariables())
		{
			FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, InputVariable.ToExternalVariable());
		}
	}
	
	if (GetExternalVariablesDelegate.IsBound())
	{
		const TArray<FRigVMExternalVariable> ExternalVariablesFromDelegate = GetExternalVariablesDelegate.Execute(GetGraph());
		for (const FRigVMExternalVariable& ExternalVariableFromDelegate : ExternalVariablesFromDelegate)
		{
			FRigVMExternalVariable::MergeExternalVariable(ExternalVariables, ExternalVariableFromDelegate);
		}
	}

	return ExternalVariables;
}

const FRigVMByteCode* URigVMController::GetCurrentByteCode() const
{
	if (GetCurrentByteCodeDelegate.IsBound())
	{
		return GetCurrentByteCodeDelegate.Execute();
	}
	return nullptr;
}

void URigVMController::RefreshFunctionReferences(const URigVMLibraryNode* InFunctionDefinition, bool bSetupUndoRedo, bool bLoadIfNecessary)
{
	if(bSuspendRefreshingFunctionReferences)
	{
		return;
	}
	
	check(InFunctionDefinition);

	if (const URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(InFunctionDefinition->GetGraph()))
	{
		TMap<URigVMController*,TSharedPtr<FRigVMControllerCompileBracketScope>> CompilationBrackets;
		FunctionLibrary->ForEachReference(InFunctionDefinition->GetFName(), [this, bSetupUndoRedo, &CompilationBrackets](URigVMFunctionReferenceNode* ReferenceNode)
		{
			if(URigVMController* ReferenceController = GetControllerForGraph(ReferenceNode->GetGraph()))
			{
				if (!CompilationBrackets.Contains(ReferenceController))
				{
					CompilationBrackets.FindOrAdd(ReferenceController) = MakeShared<FRigVMControllerCompileBracketScope>(ReferenceController);
				}
				const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(ReferenceNode->GetLinks());
				ReferenceController->FastBreakLinkedPaths(LinkedPaths, bSetupUndoRedo);
				ReferenceController->RepopulatePinsOnNode(ReferenceNode, false, false, true);
				TGuardValue<bool> ReportGuard(ReferenceController->bReportWarningsAndErrors, false);
				ReferenceController->RestoreLinkedPaths(LinkedPaths, FRestoreLinkedPathSettings(), bSetupUndoRedo);
			}
		}, bLoadIfNecessary);
	}
}

void URigVMController::PropagateExposedPinTypeChangeToReferencesAndInterface(const URigVMLibraryNode* InFunctionDefinition, bool bSetupOrphanPins)
{
	if (InFunctionDefinition == nullptr)
	{
		return;
	}

	// Repopulate the function entry/return nodes from the now-current library node layout,
	// preserving and restoring links to keep the contained graph valid. The compile bracket
	// matches ChangeExposedPinType_Internal's batching so we don't fire intermediate
	// notifications/recompiles mid-undo.
	if (URigVMGraph* ContainedGraph = InFunctionDefinition->GetContainedGraph())
	{
		if (URigVMController* ContainedController = GetControllerForGraph(ContainedGraph))
		{
			FRigVMControllerCompileBracketScope ContainedCompileScope(ContainedController);

			URigVMNode* InterfaceNodes[] = { ContainedGraph->GetEntryNode(), ContainedGraph->GetReturnNode() };
			for (URigVMNode* InterfaceNode : InterfaceNodes)
			{
				if (InterfaceNode == nullptr)
				{
					continue;
				}
				const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InterfaceNode);
				ContainedController->FastBreakLinkedPaths(LinkedPaths);
				ContainedController->RepopulatePinsOnNode(InterfaceNode, true, bSetupOrphanPins, true);
				ContainedController->RestoreLinkedPaths(LinkedPaths, FRestoreLinkedPathSettings());
				ContainedController->RemoveUnusedOrphanedPins(InterfaceNode);
			}
		}
	}

	// Function reference nodes rebuild their pin layout from the (now-current) function header.
	RefreshFunctionReferences(InFunctionDefinition, false, false);
}

void URigVMController::PropagateNotificationToFunctionReferences(const URigVMLibraryNode* InFunctionDefinition, ERigVMGraphNotifType InNotifType, UObject* InSubject, bool bLoadIfNecessary)
{
	check(InFunctionDefinition);

	if (const URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(InFunctionDefinition->GetGraph()))
	{
		FunctionLibrary->ForEachReference(InFunctionDefinition->GetFName(), [this, InNotifType, InSubject](URigVMFunctionReferenceNode* ReferenceNode)
		{
			if(const URigVMController* ReferenceController = GetControllerForGraph(ReferenceNode->GetGraph()))
			{
				UObject* TargetSubject = nullptr;
				if(InSubject)
				{
					if(const URigVMNode* NodeSubject = Cast<URigVMNode>(InSubject))
					{
						TargetSubject = ReferenceNode;
					}
					if(const URigVMPin* PinSubject = Cast<URigVMPin>(InSubject))
					{
						const FString SegmentPath = PinSubject->GetSegmentPath(true);
						TargetSubject = ReferenceNode->FindPin(SegmentPath);
					}
				}
				if(InSubject != nullptr && TargetSubject == nullptr)
				{
					return;
				}
				ReferenceController->Notify(InNotifType, TargetSubject);
			}
		}, bLoadIfNecessary);
	}
}

URigVMController::FLinkedPath::FLinkedPath(URigVMLink* InLink)
	: GraphPtr(InLink->GetGraph())
	, SourcePinPath(InLink->GetSourcePinPath())
	, TargetPinPath(InLink->GetTargetPinPath())
	, bSourceNodeIsInjected(false)
	, bTargetNodeIsInjected(false)
{
	if(const URigVMNode* SourceNode = InLink->GetSourceNode())
	{
		bSourceNodeIsInjected = SourceNode->IsInjected();
	}
	if(const URigVMNode* TargetNode = InLink->GetTargetNode())
	{
		bTargetNodeIsInjected = TargetNode->IsInjected();
	}
}

URigVMGraph* URigVMController::FLinkedPath::GetGraph(URigVMGraph* InGraph) const
{
	if(InGraph)
	{
		return InGraph;
	}
	if(GraphPtr.IsValid())
	{
		return GraphPtr.Get();
	}
	return nullptr;
}

FString URigVMController::FLinkedPath::GetPinPathRepresentation() const
{
	return URigVMLink::GetPinPathRepresentation(SourcePinPath, TargetPinPath);
}

URigVMPin* URigVMController::FLinkedPath::GetSourcePin(URigVMGraph* InGraph) const
{
	const URigVMGraph* Graph = GetGraph(InGraph);
	if(Graph == nullptr)
	{
		return nullptr;
	}
	return Graph->FindPin(SourcePinPath);
}

URigVMPin* URigVMController::FLinkedPath::GetTargetPin(URigVMGraph* InGraph) const
{
	const URigVMGraph* Graph = GetGraph(InGraph);
	if(Graph == nullptr)
	{
		return nullptr;
	}
	return Graph->FindPin(TargetPinPath);
}

uint32 GetTypeHash(const URigVMController::FLinkedPath& InPath)
{
	uint32 Hash = GetTypeHash(InPath.GraphPtr.ToSoftObjectPath().ToString());
	Hash = HashCombine(Hash, GetTypeHash(InPath.SourcePinPath));
	Hash = HashCombine(Hash, GetTypeHash(InPath.TargetPinPath));
	return Hash;
}

TArray<URigVMController::FLinkedPath> URigVMController::GetLinkedPaths() const
{
	if(const URigVMGraph* Graph = GetGraph())
	{
		return GetLinkedPaths(Graph->GetLinks());
	}
	return TArray<FLinkedPath>();
}

TArray<URigVMController::FLinkedPath> URigVMController::GetLinkedPaths(const TArray<URigVMLink*>& InLinks)
{
	TArray<FLinkedPath> LinkedPaths;
	LinkedPaths.Reserve(InLinks.Num());
	for(URigVMLink* Link : InLinks)
	{
		LinkedPaths.Emplace(Link);
	}
	return LinkedPaths;
}

TArray<URigVMController::FLinkedPath> URigVMController::GetLinkedPaths(URigVMNode* InNode, bool bIncludeInjectionNodes)
{
	const TArray<URigVMNode*> Nodes = {InNode};
	return GetLinkedPaths(Nodes, bIncludeInjectionNodes);
}

TArray<URigVMController::FLinkedPath> URigVMController::GetLinkedPaths(const TArray<URigVMNode*>& InNodes, bool bIncludeInjectionNodes)
{
	TArray<FLinkedPath> LinkedPaths;
	for(const URigVMNode* Node : InNodes)
	{
		TArray<URigVMLink*> Links = Node->GetLinks();
		for(URigVMLink* Link : Links)
		{
			if(!bIncludeInjectionNodes)
			{
				if(Link->GetSourcePin()->GetNode()->IsInjected() ||
					Link->GetTargetPin()->GetNode()->IsInjected())
				{
					continue;
				}
			}
			const FLinkedPath LinkedPath(Link);
			LinkedPaths.AddUnique(LinkedPath);
		}
	}
	return LinkedPaths;
}

TArray<URigVMController::FLinkedPath> URigVMController::GetLinkedPaths(const URigVMPin* InPin, bool bSourceLinksRecursive, bool bTargetLinksRecursive)
{
	TArray<URigVMLink*> Links;
	Links.Append(InPin->GetSourceLinks(bSourceLinksRecursive));
	Links.Append(InPin->GetTargetLinks(bTargetLinksRecursive));
	return GetLinkedPaths(Links);
}

bool URigVMController::BreakLinkedPaths(const TArray<FLinkedPath>& InLinkedPaths, bool bSetupUndoRedo, bool bRelyOnBreakLink)
{
	TArray<uint32> ProcessedLinks;

	for(const FLinkedPath& LinkedPath : InLinkedPaths)
	{
		// avoid duplicate links
		const uint32 Hash = GetTypeHash(LinkedPath);
		if(ProcessedLinks.Contains(Hash))
		{
			continue;
		}
		ProcessedLinks.Add(Hash);

		if(bRelyOnBreakLink)
		{
			if(!BreakLink(LinkedPath.SourcePinPath, LinkedPath.TargetPinPath, bSetupUndoRedo))
			{
				ReportErrorf(TEXT("Couldn't remove link '%s'"), *LinkedPath.GetPinPathRepresentation());
				return false;
			}
		}
		// if we are trying to perform this as quickly as possible - let's move the links from
		// the graph's main Links storage to a temporary location to avoid UObject creation.
		else
		{
			const FString PinPathRepresentation = LinkedPath.GetPinPathRepresentation();
			URigVMLink* Link = FindLinkFromPinPathRepresentation(PinPathRepresentation, false);
			if(Link)
			{
				URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();

				if ((SourcePin == nullptr) != (TargetPin == nullptr))
				{
					ReportErrorf(TEXT("Cannot break link %s in package %s"), *PinPathRepresentation, *GetPackage()->GetPathName());
				}

				if(SourcePin)
				{
					SourcePin->Links.Remove(Link);
				}
				if(TargetPin)
				{
					TargetPin->Links.Remove(Link);
				}
				
				Link->Detach();
				GetGraph()->Links.Remove(Link);
				GetGraph()->DetachedLinks.Add(Link);

				if(bSetupUndoRedo && SourcePin && TargetPin)
				{
					GetActionStack()->AddAction(FRigVMBreakLinkAction(this, SourcePin, TargetPin));
				}
			}
			else
			{
				ReportErrorf(TEXT("Couldn't remove link '%s'"), *LinkedPath.GetPinPathRepresentation());
				return false;
			}
		}
	}

	return true;
}

bool URigVMController::FastBreakLinkedPaths(const TArray<FLinkedPath>& InLinkedPaths, bool bSetupUndoRedo)
{
	return BreakLinkedPaths(InLinkedPaths, bSetupUndoRedo, false);
}

URigVMLink* URigVMController::FindLinkFromPinPathRepresentation(const FString& InPinPathRepresentation, bool bLookForDetachedLink) const
{
	TArray<TObjectPtr<URigVMLink>>& LinksToSearch = bLookForDetachedLink ? GetGraph()->DetachedLinks : GetGraph()->Links;
	const TObjectPtr<URigVMLink>* LinkPtr = LinksToSearch.FindByPredicate([InPinPathRepresentation](const URigVMLink* Link) -> bool
	{
		return Link->GetPinPathRepresentation().Equals(InPinPathRepresentation, ESearchCase::CaseSensitive);
	});

	if(LinkPtr)
	{
		return LinkPtr->Get();
	}
	
	return nullptr;
}

TArray<URigVMController::FLinkedPath> URigVMController::RemapLinkedPaths(
	const TArray<FLinkedPath>& InLinkedPaths,
	const FRestoreLinkedPathSettings& InSettings,
	bool bSetupUndoRedo)
{
	auto RemapPinPath = [this, InSettings, bSetupUndoRedo](const FString& InPinPath, bool bAsInput) -> FString
	{
		FString NodeName, SegmentPath;
		if(!URigVMPin::SplitPinPathAtStart(InPinPath, NodeName, SegmentPath))
		{
			return InPinPath;
		}

		const FString OldNodeName = NodeName;
		FString RemappedPinPath = InPinPath;

		if (InSettings.bFollowCoreRedirectors)
		{
			FString RedirectedSourcePinPath;
			if (ShouldRedirectPin(RemappedPinPath, RedirectedSourcePinPath))
			{
				RemappedPinPath = RedirectedSourcePinPath;
			}
		}

		if(const FRigVMController_PinPathRemapDelegate* RemapDelegateForOldName = InSettings.RemapDelegates.Find(OldNodeName))
		{
			FString NewRemappedPinPath = RemapDelegateForOldName->Execute(RemappedPinPath, bAsInput);
			if (!NewRemappedPinPath.IsEmpty() && !NewRemappedPinPath.Equals(RemappedPinPath, ESearchCase::CaseSensitive))
			{
				RemappedPinPath = NewRemappedPinPath;
				(void)URigVMPin::SplitPinPathAtStart(RemappedPinPath, NodeName, SegmentPath);
			}
		}

		if(const FString* RemappedNodeName = InSettings.NodeNameMap.Find(NodeName))
		{
			NodeName = *RemappedNodeName;
			RemappedPinPath = URigVMPin::JoinPinPath(*RemappedNodeName, SegmentPath);
		}

		if(const FRigVMController_PinPathRemapDelegate* RemapDelegateForNewName = InSettings.RemapDelegates.Find(NodeName))
		{
			FString NewRemappedPinPath = RemapDelegateForNewName->Execute(RemappedPinPath, bAsInput);
			if (!NewRemappedPinPath.IsEmpty() && !NewRemappedPinPath.Equals(RemappedPinPath, ESearchCase::CaseSensitive))
			{
				RemappedPinPath = NewRemappedPinPath;
				(void)URigVMPin::SplitPinPathAtStart(RemappedPinPath, NodeName, SegmentPath);
			}
		}
		
		// if the pin cannot be found based on this path the pin may be a sub pin on something that
		// hasn't resolved its type yet.
		if(InSettings.bIsImportingFromText && !RemappedPinPath.IsEmpty() && GetGraph()->FindPin(RemappedPinPath) == nullptr)
		{
			if (const URigVMPin* OriginalPin = GetGraph()->FindPin(InPinPath))
			{
				if (OriginalPin->IsStructMember())
				{
					const URigVMPin* OldRootPin = OriginalPin->GetRootPin();
					
					FString RemappedNodeName, RemappedSegmentPath, RemappedRootPinName, RemappedRemainingSegmentPath;
					if(URigVMPin::SplitPinPathAtStart(RemappedPinPath, RemappedNodeName, RemappedSegmentPath))
					{
						if(URigVMPin::SplitPinPathAtStart(RemappedSegmentPath, RemappedRootPinName, RemappedRemainingSegmentPath))
						{
							const FString RemappedRootPinPath = URigVMPin::JoinPinPath(RemappedNodeName, RemappedRootPinName);
							if (URigVMPin* RemappedRootPin = GetGraph()->FindPin(RemappedRootPinPath))
							{
								// todoooo
								//PrepareTemplatePinForType(RemappedRootPin, {OldRootPin->GetTypeIndex()}, bSetupUndoRedo);
							}
						}
					}
				}
			}
		}

		if(!RemappedPinPath.IsEmpty() && GetGraph()->FindPin(RemappedPinPath))
		{
			return RemappedPinPath;
		}

		return InPinPath;
	};

	TArray<FLinkedPath> RemappedLinkedPaths;
	RemappedLinkedPaths.Reserve(InLinkedPaths.Num());
	
	for(const FLinkedPath& LinkedPath : InLinkedPaths)
	{
		RemappedLinkedPaths.Add(LinkedPath);
		FLinkedPath& RemappedLinkedPath = RemappedLinkedPaths.Last();
		RemappedLinkedPath.OriginalPinPathRepresentation = LinkedPath.GetPinPathRepresentation();
		RemappedLinkedPath.SourcePinPath = RemapPinPath(LinkedPath.SourcePinPath, false);
		RemappedLinkedPath.TargetPinPath = RemapPinPath(LinkedPath.TargetPinPath, true);

		URigVMPin* SourcePin = RemappedLinkedPath.GetSourcePin();
		URigVMPin* TargetPin = RemappedLinkedPath.GetTargetPin();
		
		if(InSettings.bRelayToOrphanPins && (SourcePin != nullptr) && (TargetPin != nullptr))
		{
			if (!SourcePin->IsLinkedTo(TargetPin))
			{
				if (!URigVMPin::CanLink(SourcePin, TargetPin, nullptr, nullptr, ERigVMPinDirection::IO, true))
				{
					if(SourcePin->GetNode()->HasOrphanedPins() && InSettings.bRelayToOrphanPins)
					{
						SourcePin = nullptr;
					}
					else if(TargetPin->GetNode()->HasOrphanedPins() && InSettings.bRelayToOrphanPins)
					{
						TargetPin = nullptr;
					}
					else
					{
						ReportWarningf(TEXT("Unable to re-create link %s"), *LinkedPath.GetPinPathRepresentation());
						continue;
					}
				}
			}
		}

		if(InSettings.bRelayToOrphanPins)
		{
			for(int32 PinIndex=0; PinIndex<2; PinIndex++)
			{
				URigVMPin*& PinToFind = PinIndex == 0 ? SourcePin : TargetPin;
				
				if(PinToFind == nullptr)
				{
					const FString& PinPathToFind = PinIndex == 0 ? RemappedLinkedPath.SourcePinPath : RemappedLinkedPath.TargetPinPath;
					FString NodeName, RemainingPinPath;
					URigVMPin::SplitPinPathAtStart(PinPathToFind, NodeName, RemainingPinPath);
					check(!NodeName.IsEmpty() && !RemainingPinPath.IsEmpty());

					const URigVMNode* Node = GetGraph()->FindNode(NodeName);
					if(Node == nullptr)
					{
						continue;
					}

					RemainingPinPath = FString::Printf(TEXT("%s%s"), URigVMPin::OrphanPinPrefix, *RemainingPinPath);
					PinToFind = Node->FindPin(RemainingPinPath);

					if(PinToFind != nullptr)
					{
						if(PinIndex == 0)
						{
							RemappedLinkedPath.SourcePinPath = PinToFind->GetPinPath();
							SourcePin = PinToFind;
						}
						else
						{
							RemappedLinkedPath.TargetPinPath = PinToFind->GetPinPath();
							TargetPin = PinToFind;
						}
					}
				}
			}
		}
	}

	return RemappedLinkedPaths; 
}

bool URigVMController::RestoreLinkedPaths(
	const TArray<FLinkedPath>& InLinkedPaths,
	const FRestoreLinkedPathSettings& InSettings,
	bool bSetupUndoRedo)
{
	const TArray<FLinkedPath> RemappedLinkedPaths = RemapLinkedPaths(InLinkedPaths, InSettings, bSetupUndoRedo);
	TArray<uint32> ProcessedLinks;

	bool bSuccess = true;
	bool bAffectedAnyTemplateNode = false;
	URigVMGraph* Graph = GetGraph();
	
	for(const FLinkedPath& LinkedPath : RemappedLinkedPaths)
	{
		const FString& OriginalPinPathRepresentation = LinkedPath.OriginalPinPathRepresentation;

		const uint32 Hash = GetTypeHash(OriginalPinPathRepresentation);
		if(ProcessedLinks.Contains(Hash))
		{
			continue;
		}
		ProcessedLinks.Add(Hash);
		
		const FString SourcePath = LinkedPath.SourcePinPath;
		const FString TargetPath = LinkedPath.TargetPinPath;
		URigVMPin* SourcePin = Graph->FindPin(SourcePath);
		URigVMPin* TargetPin = Graph->FindPin(TargetPath);

		URigVMLink* ExistingLink = FindLinkFromPinPathRepresentation(OriginalPinPathRepresentation, true);
		auto OnFailedToRestoreLink = [this, Graph, ExistingLink, LinkedPath, bSetupUndoRedo, &bSuccess](const FString& InFailureReason)
		{
			URigVMPin* SourcePin = LinkedPath.GetSourcePin(Graph);
			URigVMPin* TargetPin = LinkedPath.GetTargetPin(Graph);

			bool bNotify = true;

			// treat links on injected nodes special. if a pin is within another pin
			// it means it is on an injected node
			if(SourcePin && TargetPin)
			{
				if(SourcePin->IsLinkedTo(TargetPin))
				{
					if(SourcePin->IsInOuter(TargetPin) ||
						TargetPin->IsInOuter(SourcePin))
					{
						bNotify = false;
					}
				}
			}

			if(bNotify)
			{
				ReportRemovedLink(LinkedPath.SourcePinPath, LinkedPath.TargetPinPath, InFailureReason);
			}

			// Capture the link's source/target node pointers before destroying it, so
			// we can match any injection whose node was connected through this link.
			URigVMNode* LinkSourceNode = ExistingLink ? ExistingLink->GetSourceNode() : nullptr;
			URigVMNode* LinkTargetNode = ExistingLink ? ExistingLink->GetTargetNode() : nullptr;
				
			// If the failed link connected an injected node to its host pin,
			// the injection is now orphaned - remove it (and the injected node
			// along with it) so the graph doesn't claim a broken binding.
			// Match by node identity: if ExistingLink was linked to Injection->Node,
			// that injection owns the failed link and must go.
			if(LinkedPath.bSourceNodeIsInjected && TargetPin && LinkSourceNode)
			{
				for(URigVMInjectionInfo* Injection : TargetPin->GetInjectedNodes())
				{
					if(Injection && Injection->Node == LinkSourceNode)
					{
						RemoveInjectedNode(TargetPin->GetPinPath(), Injection->bInjectedAsInput, bSetupUndoRedo);
						break;
					}
				}
			}
			else if(LinkedPath.bTargetNodeIsInjected && SourcePin && LinkTargetNode)
			{
				for(URigVMInjectionInfo* Injection : SourcePin->GetInjectedNodes())
				{
					if(Injection && Injection->Node == LinkTargetNode)
					{
						RemoveInjectedNode(SourcePin->GetPinPath(), Injection->bInjectedAsInput, bSetupUndoRedo);
						break;
					}
				}
			}

			if(ExistingLink)
			{
				if(bNotify)
				{
					Notify(ERigVMGraphNotifType::LinkRemoved, ExistingLink);
				}
				Graph->DetachedLinks.Remove(ExistingLink);
				DestroyObject(ExistingLink);
			}

			bSuccess = false;
		};

		if(SourcePin == nullptr || TargetPin == nullptr)
		{
			static const FString BothPinsMissing = TEXT("Source and target pin cannot be found.");
			static const FString SourcePinMissing = TEXT("Source pin cannot be found.");
			static const FString TargetPinMissing = TEXT("Target pin cannot be found.");

			const FString* Reason = nullptr;
			if(SourcePin == nullptr && TargetPin == nullptr)
			{
				Reason = &BothPinsMissing;
			}
			else if(SourcePin == nullptr)
			{
				Reason = &SourcePinMissing;
			}
			else
			{
				Reason = &TargetPinMissing;
			}

			OnFailedToRestoreLink(*Reason);
			continue;
		}

		if(InSettings.CompatibilityDelegate.IsBound())
		{
			if(!InSettings.CompatibilityDelegate.Execute(SourcePin, TargetPin))
			{
				static const FString IncompatibleReason = TEXT("Pins are not compatible.");
				OnFailedToRestoreLink(IncompatibleReason);
				continue;
			}
		}

		bool bSingleLinkSuccess = true;

		if(!SourcePin->IsLinkedTo(TargetPin))
		{
			// make sure the existing link has the right pin path representation
			// for AddLink to find it and reuse it
			if(ExistingLink)
			{
				ExistingLink->SetSourceAndTargetPinPaths(SourcePin->GetPinPath(), TargetPin->GetPinPath());
			}
			
			// it's ok if this fails - we want to maintain the minimum set of links
			FString FailureReason;
			if(!AddLink(SourcePin, TargetPin, bSetupUndoRedo, InSettings.UserDirection, false, true, &FailureReason))
			{
				OnFailedToRestoreLink(FailureReason);
				bSingleLinkSuccess = false;
			}
		}
		else
		{
			if (ExistingLink)
			{
				Graph->DetachedLinks.Remove(ExistingLink);
			}
		}

		if(bSingleLinkSuccess)
		{
			if(SourcePin->GetNode() && SourcePin->GetNode()->IsA<URigVMTemplateNode>())
			{
				bAffectedAnyTemplateNode = true;
			}
			if(TargetPin->GetNode() && TargetPin->GetNode()->IsA<URigVMTemplateNode>())
			{
				bAffectedAnyTemplateNode = true;
			}
		}
	}

	for (int32 PathIndex=0; PathIndex<InLinkedPaths.Num(); ++PathIndex)
	{
		if ((InLinkedPaths[PathIndex].SourcePinPath != RemappedLinkedPaths[PathIndex].SourcePinPath) ||
			(InLinkedPaths[PathIndex].TargetPinPath != RemappedLinkedPaths[PathIndex].TargetPinPath))
		{
			const FString& SourcePath = InLinkedPaths[PathIndex].SourcePinPath;
			const FString& TargetPath = InLinkedPaths[PathIndex].TargetPinPath;
			Graph->DetachedLinks = Graph->DetachedLinks.FilterByPredicate(
				[SourcePath, TargetPath](const URigVMLink* Link) -> bool
				{
					return !Link->GetSourcePinPath().Equals(SourcePath, ESearchCase::CaseSensitive) ||
						!Link->GetTargetPinPath().Equals(TargetPath, ESearchCase::CaseSensitive);
				});
		}
	}
	return bSuccess;
}

void URigVMController::ProcessDetachedLinks(const FRestoreLinkedPathSettings& InSettings)
{
	if(!IsValidGraph())
	{
		return;
	}

	// try to restore the links
	if(!GetGraph()->DetachedLinks.IsEmpty())
	{
		TGuardValue<bool> SuspendNotifications(bSuspendNotifications, true);
		TGuardValue<bool> ReportWarningsAndErrors(bReportWarningsAndErrors, true);
		const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(GetGraph()->DetachedLinks);
		RestoreLinkedPaths(LinkedPaths, InSettings);
	}

	// remove the links from the graph first
	TArray<TObjectPtr<URigVMLink>> DetachedLinks;
	Swap(GetGraph()->DetachedLinks, DetachedLinks);

	// destroy the links finally
	for(URigVMLink* DetachedLink : DetachedLinks)
	{
		ReportRemovedLink(DetachedLink->GetSourcePinPath(), DetachedLink->GetTargetPinPath());
		Notify(ERigVMGraphNotifType::LinkRemoved, DetachedLink);
		DestroyObject(DetachedLink);
	}
}


#if WITH_EDITOR

void URigVMController::RegisterUseOfTemplate(const URigVMTemplateNode* InNode)
{
	check(InNode);

	if(!bRegisterTemplateNodeUsage)
	{
		return;
	}

	const FRigVMTemplate* Template = InNode->GetTemplate();
	if(Template == nullptr)
	{
		return;
	}

	if(!InNode->IsResolved())
	{
		return;
	}

	const int32& ResolvedPermutation = InNode->ResolvedPermutation;
	if(!ensure(ResolvedPermutation != INDEX_NONE))
	{
		return;
	}

	URigVMControllerSettings* Settings = GetMutableDefault<URigVMControllerSettings>();
	Settings->Modify();

	const FName& Notation = Template->GetNotation();
	FRigVMController_CommonTypePerTemplate& TypesForTemplate = Settings->TemplateDefaultTypes.FindOrAdd(Notation);

	FRigVMRegistryReadLock ReadLock;
	const FString TypesString = FRigVMTemplate::GetStringFromArgumentTypes(Template->GetTypesForPermutation(ResolvedPermutation), ReadLock);
	int32& Count = TypesForTemplate.Counts.FindOrAdd(TypesString);
	Count++;
}

FRigVMTemplate::FTypeMap URigVMController::GetCommonlyUsedTypesForTemplate(
	const URigVMTemplateNode* InNode) const
{
	static FRigVMTemplate::FTypeMap EmptyTypes;

	const URigVMControllerSettings* Settings = GetDefault<URigVMControllerSettings>();
	if(!Settings->bAutoResolveTemplateNodesWhenLinkingExecute)
	{
		return EmptyTypes;
	}
	
	const FRigVMTemplate* Template = InNode->GetTemplate();
	if(Template == nullptr)
	{
		return EmptyTypes;
	}

	const FName& Notation = Template->GetNotation();

	const FRigVMController_CommonTypePerTemplate* TypesForTemplate = Settings->TemplateDefaultTypes.Find(Notation);
	if(TypesForTemplate == nullptr)
	{
		return EmptyTypes;
	}

	if(TypesForTemplate->Counts.IsEmpty())
	{
		return EmptyTypes;
	}

	TPair<FString,int32> MaxPair;
	for(const TPair<FString,int32>& Pair : TypesForTemplate->Counts)
	{
		if(Pair.Value > MaxPair.Value)
		{
			MaxPair = Pair;
		}
	}

	const FString& TypesString = MaxPair.Key;
	return Template->GetArgumentTypesFromString(TypesString);
}

URigVMNode* URigVMController::ConvertRerouteNodeToDispatch(URigVMRerouteNode* InRerouteNode,
	const FName& InTemplateNotation, bool bSetupUndoRedo, bool bPrintPythonCommand)
{
	static const TMap<FString, FString> EmptyRedirects = {};
	
	static const TMap<FString, FString> StructRedirects = {
		{
			URigVMRerouteNode::ValueName,
			FRigVMDispatch_MakeStruct::StructName.ToString(),
		}};

	static const TMap<FString, FString> ElementsRedirects = {
		{
			URigVMRerouteNode::ValueName,
			FRigVMDispatch_MakeStruct::ElementsName.ToString(),
		}};

	static const TMap<FString, FString> ValuesRedirects = {
		{
			URigVMRerouteNode::ValueName,
			FRigVMDispatch_ArrayMake::ValuesName.ToString(),
		}};

	static const TMap<FString, FString> ArrayRedirects = {
		{
			URigVMRerouteNode::ValueName,
			FRigVMDispatch_ArrayMake::ArrayName.ToString(),
		}};

	const TMap<FString, FString>* InputRedirects = &EmptyRedirects;
	const TMap<FString, FString>* OutputRedirects = &EmptyRedirects;
	FString PinToResolveName;

	// constant is empty
	FString NewNodeNameSuffix;
	if(InTemplateNotation == FRigVMDispatch_Constant().GetTemplateNotation())
	{
		PinToResolveName = FRigVMDispatch_Constant::ValueName.ToString();
		NewNodeNameSuffix = TEXT("Constant");
	}
	else if(InTemplateNotation == FRigVMDispatch_MakeStruct().GetTemplateNotation())
	{
		InputRedirects = &ElementsRedirects;
		OutputRedirects = &StructRedirects;
		PinToResolveName = FRigVMDispatch_MakeStruct::StructName.ToString();
		NewNodeNameSuffix = TEXT("MakeStruct");
	}
	else if(InTemplateNotation == FRigVMDispatch_BreakStruct().GetTemplateNotation())
	{
		InputRedirects = &StructRedirects;
		OutputRedirects = &ElementsRedirects;
		PinToResolveName = FRigVMDispatch_MakeStruct::StructName.ToString();
		NewNodeNameSuffix = TEXT("BreakStruct");
	}
	else if(InTemplateNotation == FRigVMDispatch_ArrayMake().GetTemplateNotation())
	{
		InputRedirects = &ValuesRedirects;
		OutputRedirects = &ArrayRedirects;
		PinToResolveName = FRigVMDispatch_ArrayMake::ArrayName.ToString();
		NewNodeNameSuffix = TEXT("ArrayMake");
	}
	else
	{
		static constexpr TCHAR Format[] = TEXT("Template '%s' not supported when converting a reroute node.");
		ReportErrorf(Format, *InTemplateNotation.ToString());
		return nullptr;
	}

	if(bSetupUndoRedo)
	{
		OpenUndoBracket(TEXT("Converting reroute node"));
	}

	TMap<FString, FPinState> PinStates = GetPinStates(InRerouteNode);
	const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(InRerouteNode);
	FastBreakLinkedPaths(LinkedPaths, bSetupUndoRedo);

	const URigVMPin* ValuePin = InRerouteNode->FindPin(URigVMRerouteNode::ValueName);
	if(ValuePin == nullptr)
	{
		if(bSetupUndoRedo)
		{
			CancelUndoBracket();
		}
		return nullptr;
	}

	const FString NodeName = InRerouteNode->GetName();
	const FVector2D NodePosition = InRerouteNode->GetPosition();
	const TRigVMTypeIndex TypeIndex = ValuePin->GetTypeIndex();

	// Before removing the node (which will remove all injected nodes), lets rename it
	// create the new node, and apply pin states to transfer ownership of injected nodes
	// to the new node. Only after everything has been moved, we can safely remove the
	// old node.
	const FString DeletedName = GetSchema()->GetValidNodeName(GetGraph(), FString::Printf(TEXT("%s_Deleted"), *NodeName));
	RenameNode(InRerouteNode, *DeletedName, false);
	const FString NewNodeName = NodeName + TEXT("_") + NewNodeNameSuffix;
	FRestoreLinkedPathSettings RestoreLinkedPathSettings;
	RestoreLinkedPathSettings.NodeNameMap.Add(NodeName, NewNodeName);
	
	URigVMNode* NewNode = AddTemplateNode(InTemplateNotation, NodePosition, NewNodeName, bSetupUndoRedo, bPrintPythonCommand);
	if(NewNode)
	{
		URigVMPin* PinToResolve = NewNode->FindPin(PinToResolveName);
		check(PinToResolve);
		verify(ResolveWildCardPin(PinToResolve, TypeIndex, bSetupUndoRedo, bPrintPythonCommand));
		
		TMap<FString, FPinState> RemappedPinStates;
		for(const TPair<FString, FPinState>& Pair : PinStates)
		{
			FString PinPath = Pair.Key;
			TArray<FString> Parts;
			if(!URigVMPin::SplitPinPath(PinPath, Parts))
			{
				Parts = {PinPath};
			}
			if(const FString* RedirectedPart = InputRedirects->Find(Parts[0]))
			{
				Parts[0] = *RedirectedPart;
				PinPath = URigVMPin::JoinPinPath(Parts);
			}
			RemappedPinStates.Add(PinPath, Pair.Value);
		}
		ApplyPinStates(NewNode, RemappedPinStates, {}, bSetupUndoRedo);

		RestoreLinkedPathSettings.RemapDelegates.Add(NodeName,
			FRigVMController_PinPathRemapDelegate::CreateLambda([NodeName, NewNodeName, InputRedirects, OutputRedirects](const FString& InPinPath, bool bIsInput) -> FString
			{
				TArray<FString> Parts;
				if(URigVMPin::SplitPinPath(InPinPath, Parts))
				{
					if(Parts[0].Equals(NodeName, ESearchCase::CaseSensitive))
					{
						Parts[0] = NewNodeName;
					}
					const TMap<FString, FString>* Redirects = bIsInput ? InputRedirects : OutputRedirects;
					if(const FString* RedirectedPart = Redirects->Find(Parts[1]))
					{
						Parts[1] = *RedirectedPart;
					}
					return URigVMPin::JoinPinPath(Parts);
				}
				return InPinPath;
			})
		);
		RestoreLinkedPaths(LinkedPaths, RestoreLinkedPathSettings, bSetupUndoRedo);

		if(!RemoveNode(InRerouteNode, bSetupUndoRedo, bPrintPythonCommand))
		{
			if(bSetupUndoRedo)
			{
				CancelUndoBracket();
			}
			return nullptr;
		}
	}

	if(bSetupUndoRedo)
	{
		CloseUndoBracket();
	}
	
	return NewNode;
}

FRigVMClientPatchResult URigVMController::PatchRerouteNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoConstant;
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoMakeStruct;
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoBreakStruct;
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoMakeAndBreakStruct;
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoRerouteAndBreakStruct;
		TArray<URigVMRerouteNode*> ReroutesToTurnIntoMakeArray;
		TArray<URigVMRerouteNode*> ReroutesToRemove;

		for(URigVMNode* Node : Graph->GetNodes())
		{
			if(URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(Node))
			{
				const URigVMPin* ValuePin = RerouteNode->FindPin(URigVMRerouteNode::ValueName);
				check(ValuePin);

				// skip reroutes which are execute contexts
				if(ValuePin->IsExecuteContext())
				{
					continue;
				}

				const int32 TopLevelSourceLinks = ValuePin->GetSourceLinks(false).Num();
				const int32 TopLevelTargetLinks = ValuePin->GetTargetLinks(false).Num();
				const int32 SubPinSourceLinks = ValuePin->GetSourceLinks(true).Num() - TopLevelSourceLinks;
				const int32 SubPinTargetLinks = ValuePin->GetTargetLinks(true).Num() - TopLevelTargetLinks;

				const bool bOnlyTopLevelSourceLinks = (TopLevelSourceLinks > 0) && (SubPinSourceLinks == 0);
				const bool bOnlyTopLevelTargetLinks = (TopLevelTargetLinks > 0) && (SubPinTargetLinks == 0);
				const bool bOnlySubPinSourceLinks = (TopLevelSourceLinks == 0) && (SubPinSourceLinks > 0);
				const bool bOnlySubPinTargetLinks = (TopLevelTargetLinks == 0) && (SubPinTargetLinks > 0);
				const bool bHasSourceLinks = (TopLevelSourceLinks + SubPinSourceLinks > 0);
				const bool bHasTargetLinks = (TopLevelTargetLinks + SubPinTargetLinks > 0);

				if(bHasSourceLinks && bHasTargetLinks)
				{
					if(bOnlyTopLevelSourceLinks && bOnlyTopLevelTargetLinks)
					{
						// this is a normal reroute and we can keep it
					}
					else if(bOnlyTopLevelSourceLinks && bOnlySubPinTargetLinks)
					{
						check(ValuePin->IsStruct());
						ReroutesToTurnIntoBreakStruct.Add(RerouteNode);
					}
					else if(bOnlyTopLevelTargetLinks && bOnlySubPinSourceLinks)
					{
						if(ValuePin->IsArray())
						{
							ReroutesToTurnIntoMakeArray.Add(RerouteNode);
						}
						else
						{
							check(ValuePin->IsStruct());
							ReroutesToTurnIntoMakeStruct.Add(RerouteNode);
						}
					}
					else if(bOnlyTopLevelSourceLinks && !bOnlyTopLevelTargetLinks && !bOnlySubPinTargetLinks)
					{
						check(ValuePin->IsStruct());
						ReroutesToTurnIntoRerouteAndBreakStruct.Add(RerouteNode);
					}
					else // bOnlySubPinSourceLinks && bOnlySubPinTargetLinks
					{
						check(ValuePin->IsStruct());
						ReroutesToTurnIntoMakeAndBreakStruct.Add(RerouteNode);
					}
				}
				else if(bHasSourceLinks) // && !bHasTargetLinks
				{
					if(bOnlyTopLevelSourceLinks)
					{
						// don't do anything - keep this node
					}
					else // bOnlySubPinSourceLinks
					{
						if(ValuePin->IsArray())
						{
							ReroutesToTurnIntoMakeArray.Add(RerouteNode);
						}
						else
						{
							check(ValuePin->IsStruct());
							ReroutesToTurnIntoMakeStruct.Add(RerouteNode);
						}
					}
				}
				else if(bHasTargetLinks) // && !bHasSourceLinks
				{
					if(bOnlyTopLevelTargetLinks)
					{
						if(ValuePin->IsStruct())
						{
							ReroutesToTurnIntoMakeStruct.Add(RerouteNode);
						}
						else if(ValuePin->IsArray())
						{
							ReroutesToTurnIntoMakeArray.Add(RerouteNode);
						}
						else
						{
							ReroutesToTurnIntoConstant.Add(RerouteNode);
						}
					}
					else if(!bOnlyTopLevelTargetLinks && !bOnlySubPinTargetLinks)
					{
						check(ValuePin->IsStruct());
						ReroutesToTurnIntoRerouteAndBreakStruct.Add(RerouteNode);
					}
					else // bOnlySubPinTargetLinks
					{
						check(ValuePin->IsStruct());
						ReroutesToTurnIntoBreakStruct.Add(RerouteNode);
					}
				}
				else
				{
					if (ValuePin->IsArray())
					{
						ReroutesToTurnIntoMakeArray.Add(RerouteNode);
					}
					else
					{
						ReroutesToRemove.Add(RerouteNode);
					}
				}
			}
		}
		
		// remove obsolete reroutes - reroutes with no connections at all
		if(ReroutesToRemove.Num() > 0)
		{
			for(URigVMRerouteNode* RerouteNode : ReroutesToRemove)
			{
				const FString PathName = RerouteNode->GetPathName();
				if(RemoveNode(RerouteNode, false, false))
				{
					Result.RemovedNodes.Add(PathName);
					Result.bChangedContent = true;
				}
			}
			ReroutesToRemove.Reset();
		}
		
		// convert some reroutes to make constant nodes
		if(ReroutesToTurnIntoConstant.Num() > 0)
		{
			const FName TemplateNotation = FRigVMDispatch_Constant().GetTemplateNotation();
			const TMap<FString, FString> PinRedirects;
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoConstant)
			{
				ConvertRerouteNodeToDispatch(RerouteNode, TemplateNotation, false, false);
			}
			ReroutesToTurnIntoConstant.Reset();
		}

		// convert some reroutes to make struct nodes
		if(ReroutesToTurnIntoMakeStruct.Num() > 0)
		{
			const FName TemplateNotation = FRigVMDispatch_MakeStruct().GetTemplateNotation();
			const TMap<FString, FString> PinRedirects;
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoMakeStruct)
			{
				ConvertRerouteNodeToDispatch(RerouteNode, TemplateNotation, false, false);
			}
			ReroutesToTurnIntoMakeStruct.Reset();
		}

		// convert some reroutes to break struct nodes
		if(ReroutesToTurnIntoBreakStruct.Num() > 0)
		{
			const FName TemplateNotation = FRigVMDispatch_BreakStruct().GetTemplateNotation();
			const TMap<FString, FString> PinRedirects;
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoBreakStruct)
			{
				ConvertRerouteNodeToDispatch(RerouteNode, TemplateNotation, false, false);
			}
			ReroutesToTurnIntoBreakStruct.Reset();
		}

		// convert some reroutes to a pair of a a make struct and a break struct node
		if(ReroutesToTurnIntoMakeAndBreakStruct.Num() > 0)
		{
			const FName MakeTemplateNotation = FRigVMDispatch_MakeStruct().GetTemplateNotation();
			const FName BreakTemplateNotation = FRigVMDispatch_BreakStruct().GetTemplateNotation();
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoMakeAndBreakStruct)
			{
				const URigVMPin* ValuePin = RerouteNode->FindPin(URigVMRerouteNode::ValueName);
				check(ValuePin);
				const TRigVMTypeIndex& TypeIndex = ValuePin->GetTypeIndex();
				const FString NodeName = RerouteNode->GetName();
				const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(RerouteNode);
				FastBreakLinkedPaths(LinkedPaths);

				if(const URigVMNode* MakeStructNode = ConvertRerouteNodeToDispatch(RerouteNode, MakeTemplateNotation, false, false))
				{
					if(const URigVMNode* BreakStructNode = AddTemplateNode(BreakTemplateNotation, MakeStructNode->GetPosition() + FVector2D(150, 0), FString(), false, false))
					{
						URigVMPin* PinToResolve = BreakStructNode->FindPin(FRigVMDispatch_BreakStruct::StructName.ToString());
						check(PinToResolve);
						verify(ResolveWildCardPin(PinToResolve, TypeIndex, false, false));

						AddLink(
							MakeStructNode->FindPin(FRigVMDispatch_MakeStruct::StructName.ToString()),
							BreakStructNode->FindPin(FRigVMDispatch_BreakStruct::StructName.ToString()),
							false);

						const FString NodeNamePrefix = URigVMPin::JoinPinPath({NodeName, FString()});
						FRestoreLinkedPathSettings RestoreSettings;
						RestoreSettings.RemapDelegates.Add(NodeName,
							FRigVMController_PinPathRemapDelegate::CreateLambda(
								[NodeNamePrefix, MakeStructNode, BreakStructNode](const FString& InPinPath, bool bIsInput) -> FString
							{
								if(InPinPath.StartsWith(NodeNamePrefix, ESearchCase::CaseSensitive))
								{
									const URigVMNode* NewNode = bIsInput ? MakeStructNode : BreakStructNode;
									const FString RemainingPinPath = InPinPath.Mid(NodeNamePrefix.Len());
									FString Left, Right;
									if(!URigVMPin::SplitPinPathAtStart(RemainingPinPath, Left, Right))
									{
										Left = RemainingPinPath;
									}
									else
									{
										Right = URigVMPin::JoinPinPath({FString(), Right});
									}

									static const FString& ValueName = URigVMRerouteNode::ValueName;
									static const FString ElementsName = FRigVMDispatch_MakeStruct::ElementsName.ToString();

									if(Left.Equals(ValueName, ESearchCase::CaseSensitive))
									{
										const FString NewNodeNamePrefix = URigVMPin::JoinPinPath({NewNode->GetName(), FString()});
										return NewNodeNamePrefix + ElementsName + Right;
									}

								}
								return InPinPath;
							})
						);
						RestoreLinkedPaths(LinkedPaths, RestoreSettings);
					}
				}
				else
				{
					// clean up detached links
					ProcessDetachedLinks();
				}
			}
			ReroutesToTurnIntoMakeAndBreakStruct.Reset();
		}

		// convert some reroutes to a pair of a reroute and a break struct node
		if(ReroutesToTurnIntoRerouteAndBreakStruct.Num() > 0)
		{
			const FName BreakTemplateNotation = FRigVMDispatch_BreakStruct().GetTemplateNotation();
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoRerouteAndBreakStruct)
			{
				const URigVMPin* ValuePin = RerouteNode->FindPin(URigVMRerouteNode::ValueName);
				check(ValuePin);
				const TRigVMTypeIndex& TypeIndex = ValuePin->GetTypeIndex();
				const FString NodeName = RerouteNode->GetName();
				const TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(RerouteNode);
				FastBreakLinkedPaths(LinkedPaths);

				if(const URigVMNode* BreakStructNode = AddTemplateNode(BreakTemplateNotation, RerouteNode->GetPosition() + FVector2D(150, 0), FString(), false, false))
				{
					URigVMPin* PinToResolve = BreakStructNode->FindPin(FRigVMDispatch_BreakStruct::StructName.ToString());
					check(PinToResolve);
					verify(ResolveWildCardPin(PinToResolve, TypeIndex, false, false));

					AddLink(
						RerouteNode->FindPin(URigVMRerouteNode::ValueName),
						BreakStructNode->FindPin(FRigVMDispatch_BreakStruct::StructName.ToString()),
						false);

					const FString NodeNamePrefix = URigVMPin::JoinPinPath({NodeName, FString()});
					FRestoreLinkedPathSettings RestoreSettings;
					RestoreSettings.RemapDelegates.Add(NodeName,
						FRigVMController_PinPathRemapDelegate::CreateLambda(
							[NodeNamePrefix, RerouteNode, BreakStructNode](const FString& InPinPath, bool bIsInput) -> FString
						{
							if(InPinPath.StartsWith(NodeNamePrefix, ESearchCase::CaseSensitive))
							{
								const URigVMNode* NewNode = bIsInput ? RerouteNode : BreakStructNode;
								const FString RemainingPinPath = InPinPath.Mid(NodeNamePrefix.Len());
								FString Left, Right;
								if(!URigVMPin::SplitPinPathAtStart(RemainingPinPath, Left, Right))
								{
									Left = RemainingPinPath;
								}
								else
								{
									Right = URigVMPin::JoinPinPath({FString(), Right});
								}

								static const FString& ValueName = URigVMRerouteNode::ValueName;
								static const FString ElementsName = FRigVMDispatch_BreakStruct::ElementsName.ToString();

								if(Left.Equals(ValueName, ESearchCase::CaseSensitive) && !Right.IsEmpty())
								{
									const FString NewNodeNamePrefix = URigVMPin::JoinPinPath({NewNode->GetName(), FString()});
									return NewNodeNamePrefix + ElementsName + Right;
								}
							}
							return InPinPath;
						})
					);
					RestoreLinkedPaths(LinkedPaths, RestoreSettings);
				}
			}
			ReroutesToTurnIntoRerouteAndBreakStruct.Reset();
		}

		// convert some reroutes to make array nodes
		if(ReroutesToTurnIntoMakeArray.Num() > 0)
		{
			const FName TemplateNotation = FRigVMDispatch_ArrayMake().GetTemplateNotation();
			const TMap<FString, FString> PinRedirects;
			for(URigVMRerouteNode* RerouteNode : ReroutesToTurnIntoMakeArray)
			{
				ConvertRerouteNodeToDispatch(RerouteNode, TemplateNotation, false, false);
			}
			ReroutesToTurnIntoMakeArray.Reset();
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchUnitNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		TArray<URigVMUnitNode*> UnitNodesToTurnToDispatches;
		
		// check for unit nodes that should be dispatches
		for(URigVMNode* Node : Graph->GetNodes())
		{
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(const FRigVMTemplate* Template = UnitNode->GetTemplate())
				{
					if(Template->GetDispatchFactory() != nullptr)
					{
						UnitNodesToTurnToDispatches.Add(UnitNode);
					}
				}
			}
		}

		// convert unit nodes to dispatches
		for(URigVMUnitNode* UnitNode : UnitNodesToTurnToDispatches)
		{
			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(UnitNode, true);
			const FVector2D NodePosition = UnitNode->GetPosition();
			const FString NodeName = UnitNode->GetName();
			TMap<FString, FPinState> PinStates = GetPinStates(UnitNode, false);
		
			FRigVMTemplate::FTypeMap TypeMap = UnitNode->GetTemplatePinTypeMap();
			const FRigVMTemplate* Template = UnitNode->GetTemplate();

			Result.RemovedNodes.Add(UnitNode->GetPathName());
			Result.bChangedContent = true;

			FastBreakLinkedPaths(LinkedPaths);

			// Before removing the node (which will remove all injected nodes), lets rename it
			// create the new node, and apply pin states to transfer ownership of injected nodes
			// to the new node. Only after everything has been moved, we can safely remove the
			// old node.
			const FString DeletedName = GetSchema()->GetValidNodeName(Graph, FString::Printf(TEXT("%s_Deleted"), *NodeName));
			RenameNode(UnitNode, *DeletedName, false, false);
			
			const FString NewNodeName = NodeName + TEXT("_") + URigVMUnitNode::StaticClass()->GetName();
			FRestoreLinkedPathSettings RestoreLinkedPathSettings;
			RestoreLinkedPathSettings.NodeNameMap.Add(NodeName, NewNodeName);
			
			URigVMTemplateNode* NewNode = AddTemplateNode(
				Template->GetNotation(),
				NodePosition, 
				NewNodeName, 
				false, 
				false);

			Result.AddedNodes.Add(NewNode);

			TArray<int32> Permutations;
			Template->Resolve(TypeMap, Permutations, false);

			for(URigVMPin* Pin : NewNode->GetPins())
			{
				if(Pin->IsWildCard())
				{
					if(const TRigVMTypeIndex* ResolvedTypeIndex = TypeMap.Find(Pin->GetFName()))
					{
						if(!FRigVMRegistry::Get().IsWildCardType(*ResolvedTypeIndex))
						{
							ChangePinType(Pin, *ResolvedTypeIndex, false, false);
						}
					}
				}
			}

			ApplyPinStates(NewNode, PinStates, {}, false);

			RemoveNode(UnitNode, false, false);

			RestoreLinkedPaths(LinkedPaths, RestoreLinkedPathSettings);
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchDispatchNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		for(URigVMNode* Node : Graph->GetNodes())
		{
			if(URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
			{
				// find template performs backwards lookup
				if(const FRigVMTemplate* Template = DispatchNode->GetTemplate())
				{
					if(Template->GetNotation() != DispatchNode->TemplateNotation)
					{
						Result.bChangedContent = Result.bChangedContent || DispatchNode->TemplateNotation != Template->GetNotation(); 
						DispatchNode->TemplateNotation = Template->GetNotation();
						
						if(!DispatchNode->ResolvedFunctionName.IsEmpty())
						{
							FString FactoryName, ArgumentsString;
							if(DispatchNode->ResolvedFunctionName.Split(TEXT("::"), &FactoryName, &ArgumentsString))
							{
								const FString PreviousResolvedFunctionName = DispatchNode->ResolvedFunctionName;
								const int32 PreviousResolvedPermutation = DispatchNode->ResolvedPermutation;
								
								DispatchNode->ResolvedFunctionName.Reset();
								DispatchNode->ResolvedPermutation = INDEX_NONE;
								
								const FRigVMTemplateTypeMap ArgumentTypes = Template->GetArgumentTypesFromString(ArgumentsString);
								if(ArgumentTypes.Num() == Template->NumArguments())
								{
									if(const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory())
									{
										const FString ResolvedPermutationName = Factory->GetPermutationName(ArgumentTypes);
										if(const FRigVMFunction* Function = FRigVMRegistry::Get().FindFunction(*ResolvedPermutationName))
										{
											DispatchNode->ResolvedFunctionName = Function->GetName();
											DispatchNode->ResolvedPermutation = Template->FindPermutation(Function);
											Result.bChangedContent = true;
										}
									}
								}

								// fall back on the serialized information
								if(DispatchNode->ResolvedFunctionName.IsEmpty())
								{
									DispatchNode->ResolvedFunctionName = PreviousResolvedFunctionName;
									DispatchNode->ResolvedPermutation = PreviousResolvedPermutation;
								}
							}
						}
					}
				}
			}
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchBranchNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		TArray<URigVMNode*> BranchNodes = Graph->GetNodes().FilterByPredicate([](URigVMNode* Node)
		{
			return Node->IsA<UDEPRECATED_RigVMBranchNode>();
		});

		for(URigVMNode* BranchNode : BranchNodes)
		{
			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(BranchNode, true);
			const FVector2D NodePosition = BranchNode->GetPosition();
			const FString NodeName = BranchNode->GetName();
			const URigVMPin* OldConditionPin = BranchNode->FindPin(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, Condition).ToString());
			const FString ConditionDefault = GetPinDefaultValue(OldConditionPin->GetPinPath());

			Result.RemovedNodes.Add(BranchNode->GetPathName());
			Result.bChangedContent = true;

			FastBreakLinkedPaths(LinkedPaths);

			// Before removing the node (which will remove all injected nodes), lets rename it
			// create the new node, and apply pin states to transfer ownership of injected nodes
			// to the new node. Only after everything has been moved, we can safely remove the
			// old node.
			const FString DeletedName = GetSchema()->GetValidNodeName(Graph, FString::Printf(TEXT("%s_Deleted"), *NodeName));
			RenameNode(BranchNode, *DeletedName, false, false);

			// Cannot reuse the name of deprecated nodes, otherwise we may get the following error when PIE a second time
			// Failed import: class 'RigVMBranchNode' name 'Branch_1_1' outer 'RigVMModel'. There is another object (of 'RigVMUnitNode' class) at the path.
			const FString NewNodeName = NodeName + TEXT("_") + URigVMUnitNode::StaticClass()->GetName();
			FRestoreLinkedPathSettings RestoreLinkedPathSettings;
			RestoreLinkedPathSettings.NodeNameMap.Add(NodeName, NewNodeName);
			const URigVMNode* NewNode = AddUnitNode(FRigVMFunction_ControlFlowBranch::StaticStruct(), FRigVMStruct::ExecuteName, NodePosition, NewNodeName, false, false);

			Result.AddedNodes.Add(NewNode);

			if(!ConditionDefault.IsEmpty())
			{
				const URigVMPin* ConditionPin = NewNode->FindPin(GET_MEMBER_NAME_CHECKED(FRigVMFunction_ControlFlowBranch, Condition).ToString());
				SetPinDefaultValue(ConditionPin->GetPinPath(), ConditionDefault, false, false, false, false);
			}
			RestoreLinkedPaths(LinkedPaths, RestoreLinkedPathSettings);

			RemoveNode(BranchNode, false, false);
		}
	}
	
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchIfSelectNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		TArray<URigVMNode*> IfOrSelectNodes = Graph->GetNodes().FilterByPredicate([](URigVMNode* Node)
		{
			return Node->IsA<UDEPRECATED_RigVMIfNode>() ||
				Node->IsA<UDEPRECATED_RigVMSelectNode>();
		});

		for(URigVMNode* IfOrSelectNode : IfOrSelectNodes)
		{
			const bool bIsIfNode = IfOrSelectNode->IsA<UDEPRECATED_RigVMIfNode>();
			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(IfOrSelectNode, true);
			const FVector2D NodePosition = IfOrSelectNode->GetPosition();
			const FString NodeName = IfOrSelectNode->GetName();
			const TRigVMTypeIndex TypeIndex = IfOrSelectNode->GetPins().Last()->GetTypeIndex();
			TMap<FString, FPinState> PinStates = GetPinStates(IfOrSelectNode, true);

			Result.RemovedNodes.Add(IfOrSelectNode->GetPathName());
			Result.bChangedContent = true;
			
			FastBreakLinkedPaths(LinkedPaths);
			
			// Before removing the node (which will remove all injected nodes), lets rename it
			// create the new node, and apply pin states to transfer ownership of injected nodes
			// to the new node. Only after everything has been moved, we can safely remove the
			// old node.
			const FString DeletedName = GetSchema()->GetValidNodeName(Graph, FString::Printf(TEXT("%s_Deleted"), *NodeName));
			RenameNode(IfOrSelectNode, *DeletedName, false, false);

			const FRigVMDispatchFactory* Factory = FRigVMRegistry::Get().FindOrAddDispatchFactory(
				bIsIfNode ? FRigVMDispatch_If::StaticStruct() : FRigVMDispatch_SelectInt32::StaticStruct());

			FRigVMTemplate* Template = const_cast<FRigVMTemplate*>(Factory->GetTemplate());
			const FString NewNodeName = NodeName + TEXT("_") + Factory->GetFactoryName().ToString();
			FRestoreLinkedPathSettings RestoreLinkedPathSettings;
			RestoreLinkedPathSettings.NodeNameMap.Add(NodeName, NewNodeName);
			URigVMTemplateNode* NewNode = AddTemplateNode(
				Template->GetNotation(),
				NodePosition, 
				NewNodeName, 
				false, 
				false);

			Result.AddedNodes.Add(NewNode);

			if(!FRigVMRegistry::Get().IsWildCardType(TypeIndex))
			{
				TArray<int32> Permutations;
				FRigVMTemplateTypeMap Types;
				Types.Add(IfOrSelectNode->GetPins().Last()->GetFName(), TypeIndex);
				Template->Resolve(Types, Permutations, false);
				if(Permutations.Num() == 1)
				{
					Template->GetOrCreatePermutation(Permutations[0]);
				}
				
				for(URigVMPin* Pin : NewNode->GetPins())
				{
					if(Pin->IsWildCard())
					{
						if(const TRigVMTypeIndex* ResolvedTypeIndex = Types.Find(Pin->GetFName()))
						{
							if(!FRigVMRegistry::Get().IsWildCardType(*ResolvedTypeIndex))
							{
								ResolveWildCardPin(Pin, *ResolvedTypeIndex, false);
							}
						}
					}
				}
			}

			ApplyPinStates(NewNode, PinStates, {}, false);

			RestoreLinkedPaths(LinkedPaths, RestoreLinkedPathSettings);

			RemoveNode(IfOrSelectNode, false, false);
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchArrayNodesOnLoad()
{
	FRigVMClientPatchResult Result;

	if (const URigVMGraph* Graph = GetGraph())
	{
		TArray<URigVMNode*> ArrayNodes = Graph->GetNodes().FilterByPredicate([](URigVMNode* Node)
		{
			return Node->IsA<UDEPRECATED_RigVMArrayNode>();
		});

		for(URigVMNode* ModelNode : ArrayNodes)
		{
			UDEPRECATED_RigVMArrayNode* ArrayNode = CastChecked<UDEPRECATED_RigVMArrayNode>(ModelNode);
			TArray<FLinkedPath> LinkedPaths = GetLinkedPaths(ArrayNode, true);
			const FVector2D NodePosition = ArrayNode->GetPosition();
			const FString NodeName = ArrayNode->GetName();
			const FString CPPType = ArrayNode->GetCPPType();
			UObject* CPPTypeObject = ArrayNode->GetCPPTypeObject();
			const ERigVMOpCode OpCode = ArrayNode->GetOpCode();
			TMap<FString, FPinState> PinStates = GetPinStates(ArrayNode, true);

			Result.RemovedNodes.Add(ArrayNode->GetPathName());
			Result.bChangedContent = true;

			FastBreakLinkedPaths(LinkedPaths);

			// Before removing the node (which will remove all injected nodes), lets rename it
			// create the new node, and apply pin states to transfer ownership of injected nodes
			// to the new node. Only after everything has been moved, we can safely remove the
			// old node.
			const FString DeletedName = GetSchema()->GetValidNodeName(Graph, FString::Printf(TEXT("%s_Deleted"), *NodeName));
			RenameNode(ArrayNode, *DeletedName, false);
			
			const FString NewNodeName = NodeName + TEXT("_") + StaticEnum<ERigVMOpCode>()->GetDisplayNameTextByValue((int64)OpCode).ToString();
			FRestoreLinkedPathSettings RestoreLinkedPathSettings;
			RestoreLinkedPathSettings.NodeNameMap.Add(NodeName, NewNodeName);
			URigVMNode* NewNode = AddArrayNode(OpCode, CPPType, CPPTypeObject, NodePosition, NewNodeName, false, false, true);
			ApplyPinStates(NewNode, PinStates, {}, false);
			Result.AddedNodes.Add(NewNode);
			
			RemoveNode(ArrayNode, false, false);

			RestoreLinkedPaths(LinkedPaths, RestoreLinkedPathSettings);
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchReduceArrayFloatDoubleConvertsionsOnLoad()
{
	FRigVMClientPatchResult Result;

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	if (const URigVMGraph* Graph = GetGraph())
	{
		bool bChangedType = false;
		TArray<FString> NodesModified;
		do
		{
			bChangedType = false;
			for (URigVMLink* Link : Graph->GetLinks())
			{
				URigVMPin* SourcePin = Link->GetSourcePin();
				URigVMPin* TargetPin = Link->GetTargetPin();
				if (!SourcePin || !TargetPin)
				{
					continue;
				}

				URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(TargetPin->GetNode());
				if (!TemplateNode)
				{
					continue;
				}
				
				const TRigVMTypeIndex SourceType = SourcePin->GetTypeIndex();
				const TRigVMTypeIndex TargetType = TargetPin->GetTypeIndex();

				// Only patch when target is array
				if (!Registry.IsArrayType(TargetType))
				{
					continue;
				}

				TRigVMTypeIndex SourceBaseType = SourceType;
				TRigVMTypeIndex TargetBaseType = TargetType;
				while (Registry.IsArrayType(SourceBaseType) || Registry.IsArrayType(TargetBaseType))
				{
					SourceBaseType = Registry.GetBaseTypeFromArrayTypeIndex(SourceBaseType);
					TargetBaseType = Registry.GetBaseTypeFromArrayTypeIndex(TargetBaseType);
				}

				if ((SourceBaseType != RigVMTypeUtils::TypeIndex::Float && SourceBaseType != RigVMTypeUtils::TypeIndex::Double) ||
				   (TargetBaseType != RigVMTypeUtils::TypeIndex::Float && TargetBaseType != RigVMTypeUtils::TypeIndex::Double))
				{
					continue;
				}

				if (SourceBaseType == TargetBaseType)
				{
					continue;
				}
		
				// If we have already changed the type of this node, we might be inside an infinite loop
				// Lets break the loop and return failure
				if (NodesModified.Contains(TemplateNode->GetName()))
				{
					Result.bSucceeded = false;
					break;
				}
		
				if (TemplateNode->SupportsType(TargetPin, SourceType))
				{
					FRigVMTemplate::FTypeMap TypeMap;
					TypeMap.Add(TargetPin->GetFName(), SourceType);
					TArray<int32> Permutations;
					TemplateNode->GetTemplate()->Resolve(TypeMap, Permutations, false);
					if (Permutations.Num() == 1)
					{
						FRigVMTemplateTypeMap NewTypes = TemplateNode->GetTemplate()->GetTypesForPermutation(Permutations[0]);
						for (auto Pair : NewTypes)
						{
							const FRigVMTemplateArgumentType Type = Registry.GetType(Pair.Value);
							ChangePinType(TemplateNode->FindPin(Pair.Key.ToString()), Type.CPPType.ToString(), Type.GetCPPTypeObjectPath(), false, false, false, false, false);
							bChangedType = true;
							NodesModified.Add(TemplateNode->GetName());
							Result.bChangedContent = true;
						}
					}
				}
			}
		} while(bChangedType && Result.bSucceeded);
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchInvalidLinksOnWildcards()
{
	FRigVMClientPatchResult Result;
	
	if (const URigVMGraph* Graph = GetGraph())
	{
		// Remove links between wildcard pins
		TArray<URigVMLink*> LinksToRemove;
		for (URigVMLink* Link : Graph->GetLinks())
		{
			bool bRemove = false;
			if (URigVMPin* SourcePin = Link->GetSourcePin())
			{
				if (SourcePin->IsWildCard())
				{
					bRemove = true;
				}
			}
			if (URigVMPin* TargetPin = Link->GetTargetPin())
			{
				if (TargetPin->IsWildCard())
				{
					bRemove = true;
				}
			}
			if (bRemove)
			{
				LinksToRemove.Add(Link);
			}
		}
		if (!LinksToRemove.IsEmpty())
		{
			Result.bChangedContent = true;
		}
		for (URigVMLink* Link : LinksToRemove)
		{
			if (!BreakLink(Link->GetSourcePin(), Link->GetTargetPin()))
			{
				Result.ErrorMessages.Add(FString::Printf(TEXT("Error breaking link %s in PatchInvalidLinksOnWildcards"), *Link->GetPinPathRepresentation()));
				Result.bSucceeded = false;
			}
		}

		// Remove exposed pins of type wildcard
		if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Graph->GetOuter()))
		{
			TArray<URigVMPin*> ExposedPins = CollapseNode->GetPins();
			for (const URigVMPin* ExposedPin : ExposedPins)
			{
				if (ExposedPin->IsWildCard())
				{
					if(URigVMController* CollapseController = GetControllerForGraph(CollapseNode->GetContainedGraph()))
					{
						if (!CollapseController->RemoveExposedPin(ExposedPin->GetFName(), false))
						{
							Result.ErrorMessages.Add(FString::Printf(TEXT("Error removing exposed pin %s PatchInvalidLinksOnWildcards"), *ExposedPin->GetPinPath(true)));
							Result.bSucceeded = false;
						}
					}
				}
			}
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchFunctionsWithInvalidReturnPaths()
{
	FRigVMClientPatchResult Result;
	TGuardValue<bool> GuardReportWarningsAndErrors(bSuspendNotifications, true);

	auto FindExecutePin = [](URigVMNode* Node, ERigVMPinDirection Direction = ERigVMPinDirection::Invalid) -> URigVMPin*
	{
		URigVMPin* const* Result = Node->GetPins().FindByPredicate([Direction](const URigVMPin* Pin)
		{
			if (Direction == ERigVMPinDirection::Invalid)
			{
				return Pin->IsExecuteContext();
			}
			else
			{
				return Pin->IsExecuteContext() && Pin->GetDirection() == Direction;
			}
		});
		if (Result)
		{
			return *Result;
		}
		return nullptr;
	};

	if (URigVMGraph* Graph = GetGraph())
	{
		if (!Graph->GetOuter()->IsA<URigVMCollapseNode>())
		{
			Result.bSucceeded = false;
			return Result;
		}

		URigVMFunctionEntryNode* EntryNode = Graph->GetEntryNode();
		URigVMFunctionReturnNode* ReturnNode = Graph->GetReturnNode();
		bool bEntryIsMutable = false;
		bool bReturnIsMutable = false;
		if (EntryNode)
		{
			RefreshFunctionPins(EntryNode);
			bEntryIsMutable = EntryNode->IsMutable();
		}
		if (ReturnNode)
		{
			RefreshFunctionPins(ReturnNode);
			bReturnIsMutable = ReturnNode->IsMutable();
		}

		if (!bEntryIsMutable && !bReturnIsMutable)
		{
			return Result;
		}

		// Make sure at this point that both interface nodes exist
		if (!EntryNode)
		{
			EntryNode = NewObject<URigVMFunctionEntryNode>(Graph, TEXT("Entry"));
			if(AddGraphNode(EntryNode, false))
			{
				RefreshFunctionPins(EntryNode);
				bEntryIsMutable = EntryNode->IsMutable();
			}
		}
		if (!ReturnNode)
		{
			ReturnNode = NewObject<URigVMFunctionReturnNode>(Graph, TEXT("Return"));
			if(AddGraphNode(ReturnNode, false))
			{
				RefreshFunctionPins(ReturnNode);
				bReturnIsMutable = ReturnNode->IsMutable();
			}
		}
		
		bool bReturnExecuteIsLinked = false;
		if (bReturnIsMutable)
		{
			if(URigVMPin* ReturnExecutePin = FindExecutePin(ReturnNode))
			{
				bReturnExecuteIsLinked = ReturnExecutePin->IsLinked();
			}
		}

		if (bEntryIsMutable && (!bReturnIsMutable || !bReturnExecuteIsLinked))
		{
			URigVMPin* EntryExecutePin = FindExecutePin(EntryNode);
			check(EntryExecutePin);
			
			URigVMPin* OtherExecutePin = nullptr;
			if (EntryExecutePin->IsLinked())
			{
				OtherExecutePin = EntryExecutePin->GetLinkedTargetPins()[0];
			}

			// Is the execute pin actually missing somewhere
			if (!bEntryIsMutable || !bReturnIsMutable)
			{
				const FName ExecutePinName = EntryExecutePin->GetFName();
				const FString ExecuteCPPType = EntryExecutePin->GetCPPType();
				const FName ExecuteCPPTypeObjectPath = *EntryExecutePin->GetCPPTypeObject()->GetPathName();
				RemoveExposedPin_Internal(ExecutePinName, false, /*bPrintPythonCommand=*/ false, /*bAllowExecutePinManagement=*/ true);
				AddExposedPin_Internal(ExecutePinName, ERigVMPinDirection::IO, ExecuteCPPType, ExecuteCPPTypeObjectPath, FString(), false, /*bPrintPythonCommand=*/ false, /*bIsInputVariable=*/ false, /*bAllowExecutePinManagement=*/ true);
			}
			
			EntryExecutePin = FindExecutePin(EntryNode);
			check(EntryExecutePin);
			URigVMPin* ReturnExecutePin = FindExecutePin(ReturnNode);
			check(ReturnExecutePin);
			
			if (OtherExecutePin)
			{
				URigVMNode* SequenceNode = AddUnitNode(FRigVMFunction_Sequence::StaticStruct(),
				   FRigVMStruct::ExecuteName,
				   EntryNode->GetPosition() + FVector2D(200.f, 0),
				   FString(), false);
				AddLink(SequenceNode->Pins[1], OtherExecutePin, false);
				AddLink(EntryExecutePin, SequenceNode->Pins[0], false);
				AddLink(SequenceNode->Pins[2], ReturnExecutePin, false);
			}
			else
			{
				AddLink(EntryExecutePin, ReturnExecutePin, false);
			}

			Result.bChangedContent = true;
			return Result;
		}

		if (bReturnIsMutable && !bEntryIsMutable)
		{
			URigVMPin* ReturnExecutePin = FindExecutePin(ReturnNode);
			const FName ExecutePinName = ReturnExecutePin->GetFName();
			const FString ExecuteCPPType = ReturnExecutePin->GetCPPType();
			const FName ExecuteCPPTypeObjectPath = *ReturnExecutePin->GetCPPTypeObject()->GetPathName();
			RemoveExposedPin_Internal(ExecutePinName, false, /*bPrintPythonCommand=*/ false, /*bAllowExecutePinManagement=*/ true);
			AddExposedPin_Internal(ExecutePinName, ERigVMPinDirection::IO, ExecuteCPPType, ExecuteCPPTypeObjectPath, FString(), false, /*bPrintPythonCommand=*/ false, /*bIsInputVariable=*/ false, /*bAllowExecutePinManagement=*/ true);
		}

		URigVMPin* EntryExecutePin = FindExecutePin(EntryNode);
		URigVMPin* ReturnExecutePin = FindExecutePin(ReturnNode);

		const TArray<URigVMPin*> LinkedSourcePins = ReturnExecutePin->GetLinkedSourcePins();
		URigVMPin* ExecuteSourcePin = LinkedSourcePins.IsEmpty() ? nullptr : LinkedSourcePins[0];
		while(ExecuteSourcePin && !ExecuteSourcePin->GetNode()->IsA<URigVMFunctionEntryNode>())
		{
			URigVMPin* ExecuteTargetPin = ExecuteSourcePin;
			if (ExecuteSourcePin->GetNode()->IsControlFlowNode())
			{
				if (ExecuteSourcePin->GetFName() != FRigVMStruct::ControlFlowCompletedName)
				{
					URigVMPin* CompletedPin = ExecuteSourcePin->GetNode()->FindPin(FRigVMStruct::ControlFlowCompletedName.ToString());
					AddLink(CompletedPin, ReturnExecutePin);
					ExecuteSourcePin = CompletedPin;
					Result.bChangedContent = true;
				}
			}

			if (ExecuteSourcePin->GetDirection() == ERigVMPinDirection::Output)
			{
				ExecuteTargetPin = *ExecuteSourcePin->GetNode()->GetPins().FindByPredicate([](URigVMPin* Pin)
				{
					return Pin->IsExecuteContext() && (Pin->GetDirection() == ERigVMPinDirection::Input || Pin->GetDirection() == ERigVMPinDirection::IO);
				});
			}

			const TArray<URigVMPin*>& SourcePins = ExecuteTargetPin->GetLinkedSourcePins();
			if (SourcePins.IsEmpty())
			{
				AddLink(EntryExecutePin, ExecuteTargetPin);
				break;
			}
			ExecuteSourcePin = SourcePins[0];
		}
	}
	
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchExecutePins()
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		for (URigVMNode* Node : Graph->GetNodes())
		{
			Result.bChangedContent |= CorrectExecutePinsOnNode(Node);
		}
	}
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchLazyPins()
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		for (const URigVMNode* Node : Graph->GetNodes())
		{
			for(URigVMPin* Pin : Node->GetPins())
			{
				const bool bShouldBeLazy = Node->ShouldInputPinComputeLazily(Pin);
				if(Pin->bIsLazy != bShouldBeLazy)
				{
					TArray<URigVMPin*> AllPins = {Pin};
					for(int32 Index = 0; Index < AllPins.Num(); Index++)
					{
						AllPins[Index]->bIsLazy = true;
						AllPins.Append(AllPins[Index]->GetSubPins());
					}
					Result.bChangedContent = true;
				}
			}
		}
	}
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchConstantPins()
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		for (const URigVMNode* Node : Graph->GetNodes())
		{
			for(URigVMPin* Pin : Node->GetPins())
			{
				const bool bShouldBeConstant = Node->IsPinConstant(Pin);
				if(Pin->bIsConstant != bShouldBeConstant)
				{
					Pin->bIsConstant = bShouldBeConstant;
					Result.bChangedContent = true;
				}
			}
		}
	}
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchPinDefaultValues()
{
	FRigVMClientPatchResult Result;
	if(!CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		return Result;
	}

	FRigVMDefaultValueTypeGuard AutoDetectGuard(this, ERigVMPinDefaultValueType::AutoDetect, true);
	if (const URigVMGraph* Graph = GetGraph())
	{
		for (const URigVMNode* Node : Graph->GetNodes())
		{
			const TArray<URigVMPin*> Pins = Node->GetAllPinsRecursively();
			for(URigVMPin* Pin : Pins)
			{
				if(Pin->CanProvideDefaultValue() && Pin->GetSubPins().IsEmpty())
				{
					if(Pin->GetDefaultValueType() == ERigVMPinDefaultValueType::AutoDetect)
					{
						const FString DefaultValue = Pin->GetDefaultValue();
						if(!DefaultValue.IsEmpty())
						{
							// this will determine based on the delta if the default value
							// is unchanged or overridden by the user
							const ERigVMPinDefaultValueType DefaultValueType = GetDefaultValueType(Pin, DefaultValue);
							if(Pin->DefaultValueType != DefaultValueType)
							{
								Pin->DefaultValueType = DefaultValueType;
								Result.bChangedContent = true;
							}
						}
					}
					else if(!Pin->GetRootPin()->HasDefaultValueOverride())
					{
						if(Pin->GetDefaultValueType() == ERigVMPinDefaultValueType::Unset)
						{
							// if the pin is supposed to be the original value of the providing
							// node (like unit struct, dispatch factory or function ref).
							// reset the value back to the current / updated default.
							FString OriginalDefaultValue = Pin->GetOriginalDefaultValue();
							PostProcessDefaultValue(Pin, OriginalDefaultValue);
							if(!OriginalDefaultValue.IsEmpty() && (Pin->GetCPPType() != RigVMTypeUtils::FStringType))
							{
								if(Pin->IsValidDefaultValue(OriginalDefaultValue))
								{
									const FString DefaultValue = Pin->GetDefaultValue();
									if(!DefaultValue.Equals(OriginalDefaultValue, ESearchCase::CaseSensitive))
									{
										FRigVMDefaultValueTypeGuard OverrideGuard(this, ERigVMPinDefaultValueType::Unset, true);
										if(SetPinDefaultValue(Pin, OriginalDefaultValue, true, false, false))
										{
											Result.bChangedContent = true;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
	return Result;

}

FRigVMClientPatchResult URigVMController::PatchUserDefinedStructPinNames()
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		for (const URigVMNode* Node : Graph->GetNodes())
		{
			for (URigVMPin* Pin : Node->GetAllPinsRecursively())
			{
				if (UUserDefinedStruct* UDStruct = Cast<UUserDefinedStruct>(Pin->GetCPPTypeObject()))
				{
					for (URigVMPin* SubPin : Pin->GetSubPins())
					{
						FString SubPinPath = SubPin->GetSubPinPath(Pin);
						const FProperty* StructSubProperty = UDStruct->FindPropertyByName(*SubPinPath);
						if (!StructSubProperty)
						{
							// Patch UserDefinedStruct subpins that are named after their display name
							StructSubProperty = UDStruct->CustomFindProperty(*SubPinPath);
						}
						if (StructSubProperty)
						{
							FString NewNameSubPin = *StructSubProperty->GetName();
							if (!NewNameSubPin.Equals(SubPin->GetName(), ESearchCase::IgnoreCase))
							{
								Result.bChangedContent = true;

								// We are going to rename the pin, so we need to modify the link paths
								// First we will find the link pins, then rename, then udpate the link paths from the link pins
								for (URigVMLink* Link : SubPin->GetSourceLinks(true))
								{
									Link->UpdatePinPointers();
								}
								for (URigVMLink* Link : SubPin->GetTargetLinks(true))
								{
									Link->UpdatePinPointers();
								}
								RenameObject(SubPin, *NewNameSubPin);
								for (URigVMLink* Link : SubPin->GetSourceLinks(true))
								{
									Link->UpdatePinPaths();
								}
								for (URigVMLink* Link : SubPin->GetTargetLinks(true))
								{
									Link->UpdatePinPaths();
								}
							}

							FName DisplayName = *StructSubProperty->GetAuthoredName();
							if (SubPin->GetDisplayName() != DisplayName)
							{
								Result.bChangedContent = true;
								SubPin->SetDisplayName(DisplayName);
							}
						}
					}
				}
			}
		}
	}
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchLocalVariableTypes()
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		if (EnsureLocalVariableValidity())
		{
			Result.bChangedContent = true;
		}
	}
	return Result;
}

FRigVMClientPatchResult URigVMController::PatchPinGuidInCollapseNodes()
{
	FRigVMClientPatchResult Result;
	if (URigVMGraph* Graph = GetGraph())
	{
		for (URigVMNode* Node : Graph->GetNodes())
		{
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
			{
				for (URigVMPin* Pin : CollapseNode->GetPins())
				{
					if (FGuid* PinGuid = CollapseNode->PinNameToGuid.Find(Pin->GetFName()))
					{
						if (Pin->IsDefinedAsInputVariable())
						{
							FRigVMExternalVariable ExternalVariable = GetVariableByName(Pin->GetFName(), true);

							if (ExternalVariable.GetGuid() != *PinGuid)
							{
								*PinGuid = ExternalVariable.GetGuid();
								Result.bChangedContent = true;
							}
						}
						else
						{
							// Path based GUID should always be correct. Do nothing
						}
					}
					else
					{
						if (Pin->IsDefinedAsInputVariable())
						{
							FRigVMExternalVariable ExternalVariable = GetVariableByName(Pin->GetFName(), true);
							CollapseNode->PinNameToGuid.FindOrAdd(Pin->GetFName()) = ExternalVariable.GetGuid();
						}
						else
						{
							CollapseNode->PinNameToGuid.FindOrAdd(Pin->GetFName()) = FGuid::NewDeterministicGuid(Pin->GetPathName());
						}
						Result.bChangedContent = true;
					}
				}
			}
		}
		
		for (FRigVMGraphVariableDescription& LocalVariable : Graph->LocalVariables)
		{
			const FString VariablePath = FString::Printf(TEXT("%s.%s:%s"), *Graph->GetPathName(), *LocalVariable.Name.ToString(), *LocalVariable.CPPType);
			LocalVariable.Guid = FGuid::NewDeterministicGuid(VariablePath);
			Result.bChangedContent = true;
		}
	}

	return Result;
}

FRigVMClientPatchResult URigVMController::PatchExternalVariablesToArguments(bool bInSetupUndoRedo)
{
	FRigVMClientPatchResult Result;
	if (const URigVMGraph* Graph = GetGraph())
	{
		if (const URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(Graph->GetParentGraph()))
		{
			if (const URigVMLibraryNode* CollapseNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
			{
				// if this function isn't public - don't bother
				if (!FunctionLibrary->PublicFunctionNames.Contains(CollapseNode->GetFName()))
				{
					return Result;
				}
				
				const TArray<FRigVMExternalVariable> ExternalVariables = Graph->GetExternalVariables();
				const TArray<FRigVMGraphVariableDescription> LocalVariables = Graph->GetLocalVariables(false);
				for (const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
				{
					if (CollapseNode->FindRootPinByName(ExternalVariable.GetName()))
					{
						continue;
					}
					if (LocalVariables.ContainsByPredicate([&ExternalVariable](const FRigVMGraphVariableDescription& LocalVariable) -> bool
						{
							return ExternalVariable.GetName() == LocalVariable.Name;
						}))
					{
						continue;
					}

					const FString TypeName = ExternalVariable.GetExtendedCPPType().ToString();
					const FName PinName = AddExposedPin(ExternalVariable.GetName(), ERigVMPinDirection::Input, TypeName, *ExternalVariable.GetCPPTypeObjectPathName(), FString(), bInSetupUndoRedo, false, true);
					Result.bSucceeded = !PinName.IsNone();
					if (Result.bSucceeded)
					{
						Result.bChangedContent = true;
					}
				}
			}
		}
		else
		{
			for (const URigVMNode* Node : Graph->GetNodes())
			{
				const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node);
				if (!FunctionReferenceNode)
				{
					continue;
				}
				for (URigVMPin* Pin : Node->GetPins())
				{
					if (Pin->IsDefinedAsInputVariable())
					{
						const FString MappedVariablePath = FunctionReferenceNode->GetOuterVariablePath(Pin->GetFName());
						if (MappedVariablePath.IsEmpty())
						{
							continue;
						}
						if (BindPinToVariable(Pin, MappedVariablePath, bInSetupUndoRedo))
						{
							Result.bChangedContent = true;
						}
					}
				}
			}
		}
	}
	return Result;
}

ERigVMPinDefaultValueType URigVMController::GetDefaultValueType(const URigVMPin* InPin, const FString& InDefaultValue) const
{
	if(!CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		return ERigVMPinDefaultValueType::AutoDetect;
	}
	
	// default to user provided values
	ERigVMPinDefaultValueType Type = OptionalDefaultValueType.Get(InPin->GetDefaultValueType());
	
	if(Type == ERigVMPinDefaultValueType::AutoDetect)
	{
		if(InDefaultValue.IsEmpty())
		{
			return ERigVMPinDefaultValueType::Unset;
		}
		
		FString OriginalDefaultValue = InPin->GetOriginalDefaultValue();
		PostProcessDefaultValue(InPin, OriginalDefaultValue);
		if(OriginalDefaultValue.Equals(InDefaultValue, ESearchCase::CaseSensitive))
		{
			return ERigVMPinDefaultValueType::Unset;
		}

		// if the node didn't provide a valid default value let's keep it at autodetect
		if(OriginalDefaultValue.IsEmpty() && (InPin->GetCPPType() != RigVMTypeUtils::FStringType))
		{
			return ERigVMPinDefaultValueType::AutoDetect;
		}
		
		return ERigVMPinDefaultValueType::Override;
	}

	if(Type == ERigVMPinDefaultValueType::KeepValueType)
	{
		return InPin->GetDefaultValueType();
	}

	return Type;
}

#endif

URigVMActionStack* URigVMController::GetActionStack() const
{
	UE::TScopeLock Lock(ActionStackLock);
	return GetActionStack_NoLock();
}

URigVMActionStack* URigVMController::GetActionStack_NoLock() const
{
	if(WeakActionStack.IsValid())
	{
		return WeakActionStack.Get();
	}
	if (bIsRunningUnitTest)
	{
		URigVMActionStack* ActionStack = NewObject<URigVMActionStack>(GetTransientPackage(), TEXT("ActionStack"));
		WeakActionStack = ActionStack;
		ActionStackHandle = ActionStack->OnModified().AddLambda([&](ERigVMGraphNotifType NotifType, URigVMGraph* InGraph, UObject* InSubject) -> void {
			if(InGraph == GetGraph())
			{
				Notify(NotifType, InSubject);
			}
		});
		return WeakActionStack.Get();
	}
	checkNoEntry();
	return nullptr;
}

void URigVMController::SetActionStack(URigVMActionStack* InActionStack)
{
	UE::TScopeLock Lock(ActionStackLock);
	SetActionStack_NoLock(InActionStack);
}

void URigVMController::SetActionStack_NoLock(URigVMActionStack* InActionStack)
{
	if(URigVMActionStack* PreviousActionStack = WeakActionStack.Get())
	{
		PreviousActionStack->OnModified().Remove(ActionStackHandle);
		ActionStackHandle.Reset();
	}

	WeakActionStack = InActionStack;

	if(URigVMActionStack* NewActionStack = WeakActionStack.Get())
	{
		ActionStackHandle = NewActionStack->OnModified().AddLambda([&](ERigVMGraphNotifType NotifType, URigVMGraph* InGraph, UObject* InSubject) -> void {
			if(InGraph == GetGraph())
			{
				Notify(NotifType, InSubject);
			}
		});
	}
}

URigVMControllerSettings::URigVMControllerSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	bAutoResolveTemplateNodesWhenLinkingExecute = true;
}

