// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMControllerActions.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Algo/Transform.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMControllerActions)

#if WITH_EDITOR
#include "Misc/TransactionObjectEvent.h"
#endif

UScriptStruct* FRigVMActionKey::GetScriptStruct() const
{
	return FindObjectChecked<UScriptStruct>(nullptr, *ScriptStructPath);		
}

TSharedPtr<FStructOnScope> FRigVMActionKey::GetAction() const
{
	UScriptStruct* ScriptStruct = GetScriptStruct();
	TSharedPtr<FStructOnScope> StructOnScope = MakeShareable(new FStructOnScope(ScriptStruct));
	ScriptStruct->ImportText(*ExportedText, StructOnScope->GetStructMemory(), nullptr, PPF_None, nullptr, ScriptStruct->GetName());
	return StructOnScope;
}

FRigVMActionWrapper::FRigVMActionWrapper(const FRigVMActionKey& Key)
{
	StructOnScope = Key.GetAction();
}
FRigVMActionWrapper::~FRigVMActionWrapper()
{
}

const UScriptStruct* FRigVMActionWrapper::GetScriptStruct() const
{
	return CastChecked<UScriptStruct>(StructOnScope->GetStruct());
}

FRigVMBaseAction* FRigVMActionWrapper::GetAction(URigVMController* InLastController) const
{
	FRigVMBaseAction* Action = (FRigVMBaseAction*)StructOnScope->GetStructMemory();
	Action->EnsureControllerValidity();

	// reuse the last controller so that we don't have to rely on resolving it
	// from the soft object path each time
	if(Action->WeakController.IsValid() && IsValid(InLastController) && InLastController->IsValidGraph())
	{
		if(Action->ControllerPath.ToString() == InLastController->GetPathName())
		{
			Action->WeakController = InLastController;
		}
	}
	return Action;
}

FString FRigVMActionWrapper::ExportText() const
{
	FString ExportedText;
	if (StructOnScope.IsValid() && StructOnScope->IsValid())
	{
		const UScriptStruct* ScriptStruct = GetScriptStruct();
		FStructOnScope DefaultScope(ScriptStruct);
		ScriptStruct->ExportText(ExportedText, GetAction(), DefaultScope.GetStructMemory(), nullptr, PPF_None, nullptr);
	}
	return ExportedText;
}

URigVMActionStack* URigVMActionStack::GetDisabledActionStack()
{
	return StaticClass()->GetDefaultObject<URigVMActionStack>();
}

bool URigVMActionStack::OpenUndoBracket(URigVMController* InController, const FString& InTitle)
{
	if(IsDisabled())
	{
		return true;
	}
	FRigVMBaseAction* Action = new FRigVMBaseAction(InController);
	Action->Title = InTitle;
	BracketActions.Add(Action);
	BeginAction(*Action);
	return true;
}

bool URigVMActionStack::CloseUndoBracket(URigVMController* InController)
{
	if(IsDisabled())
	{
		return true;
	}
	if (ensure(BracketActions.Num() > 0))
	{
		if(BracketActions.Last()->IsEmpty())
		{
			return CancelUndoBracket(InController);
		}
		FRigVMBaseAction* Action = BracketActions.Pop();
		EndAction(*Action);
		delete(Action);
	}
	return true;
}

bool URigVMActionStack::CancelUndoBracket(URigVMController* InController)
{
	if(IsDisabled())
	{
		return true;
	}
	ensure(BracketActions.Num() > 0);
	FRigVMBaseAction* Action = BracketActions.Pop();
	CancelAction(*Action);
	delete(Action);
	return true;
}

bool URigVMActionStack::Undo(URigVMController* InController)
{
	if (UndoActions.Num() == 0 && InController)
	{
		InController->ReportWarning(TEXT("Nothing to undo."));
		return false;
	}

	FRigVMActionKey KeyToUndo = UndoActions.Pop();
	ActionIndex = UndoActions.Num();
	
	FRigVMActionWrapper Wrapper(KeyToUndo);

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	TGuardValue<int32> TabDepthGuard(LogActionDepth, 0);
	LogAction(Wrapper.GetScriptStruct(), *Wrapper.GetAction(), FRigVMBaseAction::UndoPrefix);
#endif
	
	if (Wrapper.GetAction(InController)->Undo())
	{
		RedoActions.Add(KeyToUndo);
		return true;
	}

	if(InController)
	{
		InController->ReportAndNotifyErrorf(TEXT("Error while undoing action %s."), *Wrapper.GetAction()->Title);
	}
	return false;
}

bool URigVMActionStack::Redo(URigVMController* InController)
{
	if (RedoActions.Num() == 0 && InController)
	{
		InController->ReportWarning(TEXT("Nothing to redo."));
		return false;
	}

	FRigVMActionKey KeyToRedo = RedoActions.Pop();

	FRigVMActionWrapper Wrapper(KeyToRedo);

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	TGuardValue<int32> TabDepthGuard(LogActionDepth, 0);
	LogAction(Wrapper.GetScriptStruct(), *Wrapper.GetAction(), FRigVMBaseAction::RedoPrefix);
#endif
	
	if (Wrapper.GetAction(InController)->Redo())
	{
		UndoActions.Add(KeyToRedo);
		ActionIndex = UndoActions.Num();
		return true;
	}

	if(InController)
	{
		InController->ReportAndNotifyErrorf(TEXT("Error while undoing action %s."), *Wrapper.GetAction()->Title);
	}
	return false;
}

#if WITH_EDITOR

void URigVMActionStack::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	if (TransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		int32 DesiredActionIndex = ActionIndex;
		ActionIndex = UndoActions.Num();

		if (DesiredActionIndex == ActionIndex)
		{
			return;
		}

		ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketOpened, nullptr, nullptr);

		while (DesiredActionIndex < ActionIndex)
		{
			if (UndoActions.Num() == 0)
			{
				break;
			}
			if (!Undo(nullptr))
			{
				ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketCanceled, nullptr, nullptr);
				return;
			}
		}
		while (DesiredActionIndex > ActionIndex)
		{
			if (RedoActions.Num() == 0)
			{
				break;
			}
			if (!Redo(nullptr))
			{
				ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketCanceled, nullptr, nullptr);
				return;
			}
		}

		ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketClosed, nullptr, nullptr);
	}
}

bool URigVMActionStack::IsDisabled() const
{
	return HasAnyFlags(RF_ClassDefaultObject);
}

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		

void URigVMActionStack::LogAction(const UScriptStruct* InActionStruct, const FRigVMBaseAction& InAction, const FString& InPrefix)
{
	if(bSuspendLogActions)
	{
		return;
	}
	
	if(URigVMController* Controller = InAction.GetController())
	{
		static TArray<FString> TabPrefix = {TEXT(""), TEXT("  ")};

		while(TabPrefix.Num() <= LogActionDepth)
		{
			TabPrefix.Add(TabPrefix.Last() + TabPrefix[1]);
		}
		
		const FString ActionContent = FRigVMStruct::ExportToFullyQualifiedText(InActionStruct, (const uint8*)&InAction);
		Controller->ReportInfof(TEXT("%s%s: %s (%s) '%s"), *TabPrefix[LogActionDepth], *InPrefix, *InAction.GetTitle(), *InActionStruct->GetStructCPPName(), *ActionContent);
	}

	if(InPrefix == FRigVMBaseAction::AddActionPrefix)
	{
		URigVMController* Controller = InAction.GetController();
		TGuardValue<int32> ActionDepthGuard(LogActionDepth, LogActionDepth + 1);
		for(const FRigVMActionKey& SubAction : InAction.SubActions)
		{
			const FRigVMActionWrapper Wrapper(SubAction);
			LogAction(Wrapper.GetScriptStruct(), *Wrapper.GetAction(Controller), InPrefix);
		}
	}
}

#endif

#endif


FRigVMBaseAction::FRigVMBaseAction(URigVMController* InController)
: WeakController(InController)
, Title(TEXT("Action"))
{
	if(InController)
	{
		ControllerPath = InController;

		if (IRigVMClientHost* ClientHost = InController->GetClientHost())
		{
			if (FRigVMClient* Client = ClientHost->GetRigVMClient())
			{
				if (UObject* ClientOuter = Client->GetOuter())
				{
					ClientHostPath = ClientOuter;
					ControllerModelPath = InController->GetGraph()->GetNodePath(); 
				}
			}
		}
	}
}

URigVMController* FRigVMBaseAction::GetController() const
{
	EnsureControllerValidity();
	if (!WeakController.IsValid())
	{
		if(URigVMController* Controller = Cast<URigVMController>(ControllerPath.ResolveObject()))
		{
			WeakController = Controller;
		}
	}
	if (!WeakController.IsValid())
	{
		if(UObject* ClientOuter = ClientHostPath.ResolveObject())
		{
			if (IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(ClientOuter))
			{
				if (FRigVMClient* Client = ClientHost->GetRigVMClient())
				{
					if (const URigVMGraph* Graph = Client->GetModel(ControllerModelPath))
					{
						WeakController = Client->GetOrCreateController(Graph);
					}
				}
			}
		}
	}
	return WeakController.Get();
}

void FRigVMBaseAction::EnsureControllerValidity() const
{
	if(WeakController.IsValid())
	{
		if(const URigVMController* Controller = WeakController.Get())
		{
			if(!IsValid(Controller) || !Controller->IsValidGraph())
			{
				WeakController.Reset();
			}
		}
	}
}

bool FRigVMBaseAction::Merge(const FRigVMBaseAction* Other)
{
	return SubActions.Num() == 0 && Other->SubActions.Num() == 0;
}

bool FRigVMBaseAction::MakesObsolete(const FRigVMBaseAction* Other) const
{
	return false;
}

bool FRigVMBaseAction::Undo()
{
	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	URigVMActionStack* Stack = Controller->GetActionStack();
	check(Stack);
	TGuardValue<int32> TabDepthGuard(Stack->LogActionDepth, Stack->LogActionDepth + 1);
#endif
	TGuardValue<bool> TransactionGuard(Controller->bIsTransacting, true);

	bool Result = true;
	for (int32 KeyIndex = SubActions.Num() - 1; KeyIndex >= 0; KeyIndex--)
	{
		FRigVMActionWrapper Wrapper(SubActions[KeyIndex]);
		FRigVMBaseAction* SubAction = Wrapper.GetAction(Controller);
#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		SubAction->LogAction(UndoPrefix);
#endif
		if(!SubAction->Undo())
		{
			Controller->ReportAndNotifyErrorf(TEXT("Error while undoing action '%s'."), *SubAction->Title);
			Result = false;
		}
	}
	return Result;
}

bool FRigVMBaseAction::Redo()
{
	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	URigVMActionStack* Stack = Controller->GetActionStack();
	check(Stack);
	TGuardValue<int32> TabDepthGuard(Stack->LogActionDepth, Stack->LogActionDepth + 1);
#endif
	TGuardValue<bool> TransactionGuard(Controller->bIsTransacting, true);

	bool Result = true;
	for (int32 KeyIndex = 0; KeyIndex < SubActions.Num(); KeyIndex++)
	{
		FRigVMActionWrapper Wrapper(SubActions[KeyIndex]);
#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		Wrapper.GetAction()->LogAction(RedoPrefix);
#endif
		if (!Wrapper.GetAction()->Redo())
		{
			Controller->ReportAndNotifyErrorf(TEXT("Error while redoing action '%s'."), *Wrapper.GetAction()->Title);
			Result = false;
		}
	}
	return Result;
}

bool FRigVMBaseAction::StoreNode(const URigVMNode* InNode, bool bIsPriorChange)
{
	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

	check(InNode);

	const FString& Content = Controller->ExportNodesToText({InNode->GetFName()}, true);
	if(!Content.IsEmpty())
	{
		FRigVMActionNodeContent& StoredContent = ExportedNodes.FindOrAdd(InNode->GetFName());
		if(bIsPriorChange)
		{
			StoredContent.Old = Content;
		}
		else
		{
			StoredContent.New = Content;
		}
	}
	return !Content.IsEmpty();
}

bool FRigVMBaseAction::RestoreNode(const FName& InNodeName, bool bIsUndoing)
{
	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

	URigVMGraph* Graph = Controller->GetGraph();
	check(Graph);

	FRigVMActionNodeContent& Content = ExportedNodes.FindChecked(InNodeName);

	if(URigVMNode* ExistingNode = Graph->FindNodeByName(InNodeName))
	{
		if(!Controller->RemoveNode(ExistingNode, false, false))
		{
			return false;
		}
	}

	const FString& ContentToImport = bIsUndoing ? Content.Old : Content.New;
	const TArray<FName> ImportedNodeNames = Controller->ImportNodesFromText(ContentToImport, false, false);
	if(ImportedNodeNames.Num() == 1)
	{
		if(!ImportedNodeNames[0].IsEqual(InNodeName, ENameCase::CaseSensitive))
		{
			return false;
		}
	}

	return true;
}

#if RIGVM_ACTIONSTACK_VERBOSE_LOG

void FRigVMBaseAction::LogAction(const FString& InPrefix) const
{
	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return;
	}
	
	URigVMActionStack* Stack = Controller->GetActionStack();
	check(Stack);
	TGuardValue<int32> TabDepthGuard(Stack->LogActionDepth, Stack->LogActionDepth + 1);

	Stack->LogAction(GetScriptStruct(), *this, InPrefix);
}

#endif

FRigVMInjectNodeIntoPinAction::FRigVMInjectNodeIntoPinAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, bAsInput(false)
, InputPinName(NAME_None)
, OutputPinName(NAME_None)
, NodePath()
{
}

FRigVMInjectNodeIntoPinAction::FRigVMInjectNodeIntoPinAction(URigVMController* InController, URigVMInjectionInfo* InInjectionInfo)
: FRigVMBaseAction(InController)
, PinPath(InInjectionInfo->GetPin()->GetPinPath())
, bAsInput(InInjectionInfo->bInjectedAsInput)
, NodePath(InInjectionInfo->Node->GetName())
{
	if (InInjectionInfo->InputPin)
	{
		InputPinName = InInjectionInfo->InputPin->GetFName();
	}
	if (InInjectionInfo->OutputPin)
	{
		OutputPinName = InInjectionInfo->OutputPin->GetFName();
	}
}

bool FRigVMInjectNodeIntoPinAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->EjectNodeFromPin(*PinPath, false) != nullptr;
}

bool FRigVMInjectNodeIntoPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	if (URigVMInjectionInfo* InjectionInfo = GetController()->InjectNodeIntoPin(PinPath, bAsInput, InputPinName, OutputPinName, false))
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMEjectNodeFromPinAction::FRigVMEjectNodeFromPinAction()
: FRigVMInjectNodeIntoPinAction()
{
}

FRigVMEjectNodeFromPinAction::FRigVMEjectNodeFromPinAction(URigVMController* InController, URigVMInjectionInfo* InInjectionInfo)
: FRigVMInjectNodeIntoPinAction(InController, InInjectionInfo)
{
}

bool FRigVMEjectNodeFromPinAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
#if WITH_EDITOR
	const URigVMInjectionInfo* InjectionInfo = GetController()->InjectNodeIntoPin(PinPath, bAsInput, InputPinName, OutputPinName, false);
	return InjectionInfo != nullptr;
#else
	return false;
#endif
}

bool FRigVMEjectNodeFromPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (GetController()->EjectNodeFromPin(*PinPath, false) != nullptr)
	{
		return FRigVMBaseAction::Redo();
	}
	return false;
}

FRigVMRemoveNodesAction::FRigVMRemoveNodesAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMRemoveNodesAction::FRigVMRemoveNodesAction(URigVMController* InController, TArray<URigVMNode*> InNodes)
: FRigVMBaseAction(InController)
{
	Algo::Transform(InNodes, NodeNames, [](const URigVMNode* Node) -> FName
	{
		return Node->GetFName();
	});
	ExportedContent = InController->ExportNodesToText(NodeNames, true);
}

bool FRigVMRemoveNodesAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	const TArray<FName> NewNodeNames = GetController()->ImportNodesFromText(ExportedContent, false, false);
	return NewNodeNames.Num() == NodeNames.Num();
}

bool FRigVMRemoveNodesAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(GetController()->RemoveNodesByName(NodeNames, false, false))
	{
		return FRigVMBaseAction::Redo();
	}
	return false;
}

FRigVMSetNodeSelectionAction::FRigVMSetNodeSelectionAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMSetNodeSelectionAction::FRigVMSetNodeSelectionAction(URigVMController* InController, URigVMGraph* InGraph, TArray<FName> InNewSelection)
: FRigVMBaseAction(InController)
{
	OldSelection = InGraph->GetSelectNodes();
	NewSelection = InNewSelection;
}

bool FRigVMSetNodeSelectionAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeSelection(OldSelection, false);
}

bool FRigVMSetNodeSelectionAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetNodeSelection(NewSelection, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodePositionAction::FRigVMSetNodePositionAction()
: FRigVMBaseAction(nullptr)
, OldPosition(FVector2D::ZeroVector)
, NewPosition(FVector2D::ZeroVector)
{
}

FRigVMSetNodePositionAction::FRigVMSetNodePositionAction(URigVMController* InController, URigVMNode* InNode, const FVector2D& InNewPosition)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldPosition(InNode->GetPosition())
, NewPosition(InNewPosition)
{
}

bool FRigVMSetNodePositionAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodePositionAction* Action = (const FRigVMSetNodePositionAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewPosition = Action->NewPosition;
	return true;
}

bool FRigVMSetNodePositionAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodePositionByName(*NodePath, OldPosition, false);
}

bool FRigVMSetNodePositionAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetNodePositionByName(*NodePath, NewPosition, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeSizeAction::FRigVMSetNodeSizeAction()
: FRigVMBaseAction(nullptr)
, OldSize(FVector2D::ZeroVector)
, NewSize(FVector2D::ZeroVector)
{
}

FRigVMSetNodeSizeAction::FRigVMSetNodeSizeAction(URigVMController* InController, URigVMNode* InNode, const FVector2D& InNewSize)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldSize(InNode->GetSize())
, NewSize(InNewSize)
{
}

bool FRigVMSetNodeSizeAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeSizeAction* Action = (const FRigVMSetNodeSizeAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewSize = Action->NewSize;
	return true;
}

bool FRigVMSetNodeSizeAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeSizeByName(*NodePath, OldSize, false);
}

bool FRigVMSetNodeSizeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetNodeSizeByName(*NodePath, NewSize, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeTitleAction::FRigVMSetNodeTitleAction()
	: FRigVMBaseAction(nullptr)
	, OldTitle()
	, NewTitle()
{
}

FRigVMSetNodeTitleAction::FRigVMSetNodeTitleAction(URigVMController* InController, URigVMNode* InNode, const FString& InNewTitle)
	: FRigVMBaseAction(InController)
	, NodePath(InNode->GetNodePath())
	, OldTitle(InNode->GetNodeTitleRaw())
	, NewTitle(InNewTitle)
{
}

bool FRigVMSetNodeTitleAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeTitleAction* Action = (const FRigVMSetNodeTitleAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewTitle = Action->NewTitle;
	return true;
}

bool FRigVMSetNodeTitleAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeTitleByName(*NodePath, OldTitle, false);
}

bool FRigVMSetNodeTitleAction::Redo()
{
	if (!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetNodeTitleByName(*NodePath, NewTitle, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeColorAction::FRigVMSetNodeColorAction()
: FRigVMBaseAction(nullptr)
, OldColor(FLinearColor::Black)
, NewColor(FLinearColor::Black)
{
}

FRigVMSetNodeColorAction::FRigVMSetNodeColorAction(URigVMController* InController, URigVMNode* InNode, const FLinearColor& InNewColor)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldColor(InNode->GetNodeColor())
, NewColor(InNewColor)
{
}

bool FRigVMSetNodeColorAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeColorAction* Action = (const FRigVMSetNodeColorAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewColor = Action->NewColor;
	return true;
}

bool FRigVMSetNodeColorAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeColorByName(*NodePath, OldColor, false);
}

bool FRigVMSetNodeColorAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetNodeColorByName(*NodePath, NewColor, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeCategoryAction::FRigVMSetNodeCategoryAction()
: FRigVMBaseAction(nullptr)
, OldCategory()
, NewCategory()
{
}

FRigVMSetNodeCategoryAction::FRigVMSetNodeCategoryAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewCategory)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldCategory(InNode->GetNodeCategory())
, NewCategory(InNewCategory)
{
}

bool FRigVMSetNodeCategoryAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeCategoryAction* Action = (const FRigVMSetNodeCategoryAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewCategory = Action->NewCategory;
	return true;
}

bool FRigVMSetNodeCategoryAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeCategoryByName(*NodePath, OldCategory, false);
}

bool FRigVMSetNodeCategoryAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetNodeCategoryByName(*NodePath, NewCategory, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeKeywordsAction::FRigVMSetNodeKeywordsAction()
: FRigVMBaseAction(nullptr)
, OldKeywords()
, NewKeywords()
{
}

FRigVMSetNodeKeywordsAction::FRigVMSetNodeKeywordsAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewKeywords)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldKeywords(InNode->GetNodeKeywords())
, NewKeywords(InNewKeywords)
{
}

bool FRigVMSetNodeKeywordsAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeKeywordsAction* Action = (const FRigVMSetNodeKeywordsAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewKeywords = Action->NewKeywords;
	return true;
}

bool FRigVMSetNodeKeywordsAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeKeywordsByName(*NodePath, OldKeywords, false);
}

bool FRigVMSetNodeKeywordsAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetNodeKeywordsByName(*NodePath, NewKeywords, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetNodeDescriptionAction::FRigVMSetNodeDescriptionAction()
: FRigVMBaseAction(nullptr)
, OldDescription()
, NewDescription()
{
}

FRigVMSetNodeDescriptionAction::FRigVMSetNodeDescriptionAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewDescription)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldDescription(InNode->GetNodeDescription())
, NewDescription(InNewDescription)
{
}

bool FRigVMSetNodeDescriptionAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetNodeDescriptionAction* Action = (const FRigVMSetNodeDescriptionAction*)Other;
	if (NodePath != Action->NodePath)
	{
		return false;
	}

	NewDescription = Action->NewDescription;
	return true;
}

bool FRigVMSetNodeDescriptionAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetNodeDescriptionByName(*NodePath, OldDescription, false);
}

bool FRigVMSetNodeDescriptionAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetNodeDescriptionByName(*NodePath, NewDescription, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetCommentTextAction::FRigVMSetCommentTextAction()
: FRigVMBaseAction(nullptr)
, NodePath()
, OldText()
, NewText()
, OldFontSize(18)
, NewFontSize(18)
, bOldBubbleVisible(false)
, bNewBubbleVisible(false)
, bOldColorBubble(false)
, bNewColorBubble(false)
{
	
}


FRigVMSetCommentTextAction::FRigVMSetCommentTextAction(URigVMController* InController, URigVMCommentNode* InNode, const FString& InNewText, const int32& InNewFontSize, const bool& bInNewBubbleVisible, const bool& bInNewColorBubble)
: FRigVMBaseAction(InController)
, NodePath(InNode->GetNodePath())
, OldText(InNode->GetCommentText())
, NewText(InNewText)
, OldFontSize(InNode->GetCommentFontSize())
, NewFontSize(InNewFontSize)
, bOldBubbleVisible(InNode->GetCommentBubbleVisible())
, bNewBubbleVisible(bInNewBubbleVisible)
, bOldColorBubble(InNode->GetCommentColorBubble())
, bNewColorBubble(bInNewColorBubble)
{
}

bool FRigVMSetCommentTextAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetCommentTextByName(*NodePath, OldText, OldFontSize, bOldBubbleVisible, bOldColorBubble, false);
}

bool FRigVMSetCommentTextAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetCommentTextByName(*NodePath, NewText, NewFontSize, bNewBubbleVisible, bNewColorBubble, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMRenameVariableAction::FRigVMRenameVariableAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMRenameVariableAction::FRigVMRenameVariableAction(URigVMController* InController, const FGuid& InVariableGuid, const FName& InOldVariableName, const FName& InNewVariableName)
: FRigVMBaseAction(InController)
, VariableGuid(InVariableGuid)
, OldVariableName(InOldVariableName.ToString())
, NewVariableName(InNewVariableName.ToString())
{
}

bool FRigVMRenameVariableAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->OnExternalVariableRenamed(VariableGuid, *OldVariableName, false);
}

bool FRigVMRenameVariableAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->OnExternalVariableRenamed(VariableGuid, *NewVariableName, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinExpansionAction::FRigVMSetPinExpansionAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldIsExpanded(false)
, NewIsExpanded(false)
{
}

FRigVMSetPinExpansionAction::FRigVMSetPinExpansionAction(URigVMController* InController, URigVMPin* InPin, bool bNewIsExpanded)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldIsExpanded(InPin->IsExpanded())
, NewIsExpanded(bNewIsExpanded)
{
}

bool FRigVMSetPinExpansionAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinExpansion(PinPath, OldIsExpanded, false);
}

bool FRigVMSetPinExpansionAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinExpansion(PinPath, NewIsExpanded, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinDisplayNameAction::FRigVMSetPinDisplayNameAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldDisplayName()
, NewDisplayName()
{
}

FRigVMSetPinDisplayNameAction::FRigVMSetPinDisplayNameAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewDisplayName)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldDisplayName(InPin->DisplayName.IsNone() ? FString() : InPin->DisplayName.ToString())
, NewDisplayName(InNewDisplayName)
{
}

bool FRigVMSetPinDisplayNameAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinDisplayName(PinPath, OldDisplayName, false);
}

bool FRigVMSetPinDisplayNameAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinDisplayName(PinPath, NewDisplayName, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinCategoryAction::FRigVMSetPinCategoryAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldCategory()
, OldPinIndex(INDEX_NONE)
, NewCategory()
{
}

FRigVMSetPinCategoryAction::FRigVMSetPinCategoryAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewCategory)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldCategory(InPin->UserDefinedCategory)
, OldPinIndex(InPin->GetIndexInCategory())
, NewCategory(InNewCategory)
{
}

bool FRigVMSetPinCategoryAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	if(const URigVMGraph* Graph = GetController()->GetGraph())
	{
		if(URigVMPin* Pin = Graph->FindPin(PinPath))
		{
			return GetController()->SetPinCategory(Pin, OldCategory, OldPinIndex, false);
		}
	}
	return false;
}

bool FRigVMSetPinCategoryAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinCategory(PinPath, NewCategory, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMChangeNodePinCategoriesAction::FRigVMChangeNodePinCategoriesAction()
: FRigVMBaseAction(nullptr)
, NodeName()
, OldCategories()
, NewCategories()
{
}

FRigVMChangeNodePinCategoriesAction::FRigVMChangeNodePinCategoriesAction(URigVMController* InController, const URigVMNode* InNode)
: FRigVMBaseAction(InController)
, NodeName(InNode->GetName())
, OldCategories(InNode->GetPinCategories())
, NewCategories()
{
}

bool FRigVMChangeNodePinCategoriesAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinCategories(*NodeName, OldCategories, false);
}

bool FRigVMChangeNodePinCategoriesAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinCategories(*NodeName, NewCategories, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

void FRigVMChangeNodePinCategoriesAction::UpdateAfterModification(const URigVMNode* InNode)
{
	NewCategories = InNode->GetPinCategories();
}

FRigVMSetPinCategoryExpansionAction::FRigVMSetPinCategoryExpansionAction()
: FRigVMBaseAction(nullptr)
, NodeName()
, PinCategory()
, bOldExpansionState(false)
{
}

FRigVMSetPinCategoryExpansionAction::FRigVMSetPinCategoryExpansionAction(URigVMController* InController, const URigVMNode* InNode, const FString& InPinCategory)
: FRigVMBaseAction(InController)
, NodeName(InNode->GetName())
, PinCategory(InPinCategory)
, bOldExpansionState(InNode->IsPinCategoryExpanded(InPinCategory))
{
}

bool FRigVMSetPinCategoryExpansionAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinCategoryExpansion(*NodeName, PinCategory, bOldExpansionState, false);
}

bool FRigVMSetPinCategoryExpansionAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinCategoryExpansion(*NodeName, PinCategory, !bOldExpansionState, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinIndexInCategoryAction::FRigVMSetPinIndexInCategoryAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldIndexInCategory(INDEX_NONE)
, NewIndexInCategory(INDEX_NONE)
{
}

FRigVMSetPinIndexInCategoryAction::FRigVMSetPinIndexInCategoryAction(URigVMController* InController, URigVMPin* InPin, int32 InNewIndexInCategory)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldIndexInCategory(InPin->GetIndexInCategory())
, NewIndexInCategory(InNewIndexInCategory)
{
}

bool FRigVMSetPinIndexInCategoryAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinIndexInCategory(PinPath, OldIndexInCategory, false);
}

bool FRigVMSetPinIndexInCategoryAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinIndexInCategory(PinPath, NewIndexInCategory, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinWatchAction::FRigVMSetPinWatchAction()
: FRigVMBaseAction(nullptr)
, OldIsWatched(false)
, NewIsWatched(false)
{
}

FRigVMSetPinWatchAction::FRigVMSetPinWatchAction(URigVMController* InController, URigVMPin* InPin, bool bNewIsWatched)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldIsWatched(InPin->RequiresWatch())
, NewIsWatched(bNewIsWatched)
{
}

bool FRigVMSetPinWatchAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetPinIsWatched(PinPath, OldIsWatched, false);
}

bool FRigVMSetPinWatchAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->SetPinIsWatched(PinPath, NewIsWatched, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinDefaultValueAction::FRigVMSetPinDefaultValueAction()
: FRigVMBaseAction(nullptr)
, OldDefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
, NewDefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
{
}

FRigVMSetPinDefaultValueAction::FRigVMSetPinDefaultValueAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewDefaultValue)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldDefaultValue(InPin->GetDefaultValueStoredByUserInterface())
, NewDefaultValue(InNewDefaultValue)
, OldDefaultValueType(InPin->GetDefaultValueType())
, NewDefaultValueType(InController->GetDefaultValueType(InPin, InNewDefaultValue))
{
	/* Since for template we are chaning types - it is possible that the
	 * pin is no longer compliant with the old value
	if(!OldDefaultValue.IsEmpty())
	{
		check(InPin->IsValidDefaultValue(OldDefaultValue));
	}
	*/
	if(!NewDefaultValue.IsEmpty())
	{
		check(InPin->IsValidDefaultValue(NewDefaultValue));
	}
}

bool FRigVMSetPinDefaultValueAction::Merge(const FRigVMBaseAction* Other)
{
	if (!FRigVMBaseAction::Merge(Other))
	{
		return false;
	}

	const FRigVMSetPinDefaultValueAction* Action = (const FRigVMSetPinDefaultValueAction*)Other;
	if (PinPath != Action->PinPath)
	{
		return false;
	}

	NewDefaultValue = Action->NewDefaultValue;
	NewDefaultValueType = Action->NewDefaultValueType;
	return true;
}

bool FRigVMSetPinDefaultValueAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	if (OldDefaultValue.IsEmpty())
	{
		// strings and wildcards allow to set an empty default
		if(const URigVMPin* Pin = GetController()->GetGraph()->FindPin(PinPath))
		{
			if((Pin->GetCPPType() != RigVMTypeUtils::FStringType) &&
				!Pin->IsWildCard())
			{
				return true;
			}
		}
	}

	FRigVMDefaultValueTypeGuard _(GetController(), OldDefaultValueType, true);
	return GetController()->SetPinDefaultValue(PinPath, OldDefaultValue, true, false);
}

bool FRigVMSetPinDefaultValueAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	bool bIsValidDefaultValue = !NewDefaultValue.IsEmpty();
	if(!bIsValidDefaultValue)
	{
		// strings and wildcards allow to set an empty default
		if(const URigVMPin* Pin = GetController()->GetGraph()->FindPin(PinPath))
		{
			if(Pin->GetCPPType() == RigVMTypeUtils::FStringType)
			{
				bIsValidDefaultValue = true;
			}
			else if(Pin->IsWildCard())
			{
				bIsValidDefaultValue = true;
			}
		}
	}
	if (bIsValidDefaultValue)
	{
		FRigVMDefaultValueTypeGuard _(GetController(), NewDefaultValueType, true);
		if (!GetController()->SetPinDefaultValue(PinPath, NewDefaultValue, true, false))
		{
			return false;
		}
	}
	return FRigVMBaseAction::Redo();
}

FRigVMInsertArrayPinAction::FRigVMInsertArrayPinAction()
: FRigVMBaseAction(nullptr)
, Index(0)
, NewDefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
{
}

FRigVMInsertArrayPinAction::FRigVMInsertArrayPinAction(URigVMController* InController, URigVMPin* InArrayPin, int32 InIndex, const FString& InNewDefaultValue, const ERigVMPinDefaultValueType& InNewDefaultValueType)
: FRigVMBaseAction(InController)
, ArrayPinPath(InArrayPin->GetPinPath())
, Index(InIndex)
, NewDefaultValue(InNewDefaultValue)
, NewDefaultValueType(InNewDefaultValueType)
{
}

bool FRigVMInsertArrayPinAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveArrayPin(FString::Printf(TEXT("%s.%d"), *ArrayPinPath, Index), false);
}

bool FRigVMInsertArrayPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}

	FRigVMDefaultValueTypeGuard _(GetController(), NewDefaultValueType, true);
	if(GetController()->InsertArrayPin(ArrayPinPath, Index, NewDefaultValue, false).IsEmpty())
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMRemoveArrayPinAction::FRigVMRemoveArrayPinAction()
: FRigVMBaseAction(nullptr)
, Index(0)
, DefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
{
}

FRigVMRemoveArrayPinAction::FRigVMRemoveArrayPinAction(URigVMController* InController, URigVMPin* InArrayElementPin)
: FRigVMBaseAction(InController)
, ArrayPinPath(InArrayElementPin->GetParentPin()->GetPinPath())
, Index(InArrayElementPin->GetPinIndex())
, DefaultValue(InArrayElementPin->GetDefaultValue())
, DefaultValueType(InArrayElementPin->GetDefaultValueType())
{
}

bool FRigVMRemoveArrayPinAction::Undo()
{
	if(!CanUndoRedo())
	{
		return false;
	}

	FRigVMDefaultValueTypeGuard _(GetController(), DefaultValueType, true);
	if(GetController()->InsertArrayPin(*ArrayPinPath, Index, DefaultValue, false).IsEmpty())
	{
		return false;
	}
	
	return FRigVMBaseAction::Undo();
}

bool FRigVMRemoveArrayPinAction::Redo()
{
	if (!FRigVMBaseAction::Redo())
	{
		return false;
	}

	return GetController()->RemoveArrayPin(FString::Printf(TEXT("%s.%d"), *ArrayPinPath, Index), false);
}

FRigVMAddLinkAction::FRigVMAddLinkAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMAddLinkAction::FRigVMAddLinkAction(URigVMController* InController, URigVMPin* InOutputPin, URigVMPin* InInputPin)
: FRigVMBaseAction(InController)
, OutputPinPath(InOutputPin->GetPinPath())
, InputPinPath(InInputPin->GetPinPath())
{
}

bool FRigVMAddLinkAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->BreakLink(OutputPinPath, InputPinPath, false);
}

bool FRigVMAddLinkAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->AddLink(OutputPinPath, InputPinPath, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMBreakLinkAction::FRigVMBreakLinkAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMBreakLinkAction::FRigVMBreakLinkAction(URigVMController* InController, URigVMPin* InOutputPin, URigVMPin* InInputPin)
: FRigVMBaseAction(InController)
, OutputPinPath(InOutputPin->GetPinPath())
, InputPinPath(InInputPin->GetPinPath())
{
	GraphPath = TSoftObjectPtr<URigVMGraph>(Cast<URigVMGraph>(InOutputPin->GetGraph())).GetUniqueID();
}

bool FRigVMBreakLinkAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->AddLink(OutputPinPath, InputPinPath, false);
}

bool FRigVMBreakLinkAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->BreakLink(OutputPinPath, InputPinPath, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMChangePinTypeAction::FRigVMChangePinTypeAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldTypeIndex(INDEX_NONE)
, NewTypeIndex(INDEX_NONE)
, bSetupOrphanPins(true)
, bBreakLinks(true)
, bRemoveSubPins(true)
{}

FRigVMChangePinTypeAction::FRigVMChangePinTypeAction(URigVMController* InController, URigVMPin* InPin, int32 InTypeIndex, bool InSetupOrphanPins, bool InBreakLinks, bool InRemoveSubPins)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetPinPath())
, OldTypeIndex(InPin->GetTypeIndex())
, NewTypeIndex(InTypeIndex)
, bSetupOrphanPins(InSetupOrphanPins)
, bBreakLinks(InBreakLinks)
, bRemoveSubPins(InRemoveSubPins)
{
}

bool FRigVMChangePinTypeAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}

	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

	const URigVMGraph* Graph = Controller->GetGraph();
	if(Graph == nullptr)
	{
		return false;
	}

	URigVMPin* Pin = Graph->FindPin(PinPath);
	if(Pin == nullptr)
	{
		return false;
	}

	if(!Controller->ChangePinType(Pin, OldTypeIndex, false, bSetupOrphanPins, bBreakLinks, bRemoveSubPins))
	{
		return false;
	}

	// If the pin belongs to a function definition's exposed interface, mirror the
	// cross-graph propagation that ChangeExposedPinType_Internal does on the forward edit
	// (interface entry/return repopulate + function reference refresh). That side effect is
	// not on the action stack on its own, so without this call the function reference and
	// interface pin layouts would remain at the new type after undo.
	if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Pin->GetNode()))
	{
		if(CollapseNode->IsGraphFunctionDefinition())
		{
			Controller->PropagateExposedPinTypeChangeToReferencesAndInterface(CollapseNode, bSetupOrphanPins);
		}
	}
	return true;
}

bool FRigVMChangePinTypeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}

	URigVMController* Controller = GetController();
	if(Controller == nullptr)
	{
		return false;
	}

	const URigVMGraph* Graph = Controller->GetGraph();
	if(Graph == nullptr)
	{
		return false;
	}

	URigVMPin* Pin = Graph->FindPin(PinPath);
	if(Pin == nullptr)
	{
		return false;
	}

	if(!Controller->ChangePinType(Pin, NewTypeIndex, false, bSetupOrphanPins, bBreakLinks, bRemoveSubPins))
	{
		return false;
	}

	// See FRigVMChangePinTypeAction::Undo for rationale.
	if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Pin->GetNode()))
	{
		if(CollapseNode->IsGraphFunctionDefinition())
		{
			Controller->PropagateExposedPinTypeChangeToReferencesAndInterface(CollapseNode, bSetupOrphanPins);
		}
	}

	return FRigVMBaseAction::Redo();
}

FRigVMCollapseNodesAction::FRigVMCollapseNodesAction()
: FRigVMBaseAction(nullptr)
, LibraryNodePath()
, bIsAggregate(false)
{
}

FRigVMCollapseNodesAction::FRigVMCollapseNodesAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes, const FString& InNodePath, const bool bIsAggregate)
: FRigVMBaseAction(InController)
, LibraryNodePath(InNodePath)
, bIsAggregate(bIsAggregate)
{
	TArray<FName> NodesToExport;
	for (const URigVMNode* InNode : InNodes)
	{
		CollapsedNodesPaths.Add(InNode->GetName());
		NodesToExport.Add(InNode->GetFName());
	}

	CollapsedNodesContent = InController->ExportNodesToText(NodesToExport, true);
}

bool FRigVMCollapseNodesAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}

	// remove the library node
	{
		if(!GetController()->RemoveNodeByName(*LibraryNodePath, false, false))
		{
			return false;
		}
	}

	const TArray<FName> RecoveredNodes = GetController()->ImportNodesFromText(CollapsedNodesContent, false, false);
	if(RecoveredNodes.Num() != CollapsedNodesPaths.Num())
	{
		return false;
	}

	return true;
}

bool FRigVMCollapseNodesAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	TArray<FName> NodeNames;
	for (const FString& NodePath : CollapsedNodesPaths)
	{
		NodeNames.Add(*NodePath);
	}

	URigVMLibraryNode* LibraryNode = GetController()->CollapseNodes(NodeNames, LibraryNodePath, false, false, bIsAggregate);
	if (LibraryNode)
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMExpandNodeAction::FRigVMExpandNodeAction()
: FRigVMBaseAction(nullptr)
, LibraryNodePath()
{
}

FRigVMExpandNodeAction::FRigVMExpandNodeAction(URigVMController* InController, URigVMLibraryNode* InLibraryNode)
: FRigVMBaseAction(InController)
, LibraryNodePath(InLibraryNode->GetName())
{
	TArray<FName> NodesToExport;
	NodesToExport.Add(InLibraryNode->GetFName());
	LibraryNodeContent = InController->ExportNodesToText(NodesToExport, true);
}

bool FRigVMExpandNodeAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}

	// remove the expanded nodes
	TArray<FName> NodeNames;
	Algo::Transform(ExpandedNodePaths, NodeNames, [](const FString& NodePath)
	{
		return FName(*NodePath); 
	});
	if(!GetController()->RemoveNodesByName(NodeNames, false, false))
	{
		return false;
	}

	const TArray<FName> RecoveredNodes = GetController()->ImportNodesFromText(LibraryNodeContent, false, false);
	if(RecoveredNodes.Num() != 1)
	{
		return false;
	}

	return true;
}

bool FRigVMExpandNodeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	const TArray<URigVMNode*> ExpandedNodes = GetController()->ExpandLibraryNode(*LibraryNodePath, false);
	if (ExpandedNodes.Num() == ExpandedNodePaths.Num())
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMRenameNodeAction::FRigVMRenameNodeAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMRenameNodeAction::FRigVMRenameNodeAction(URigVMController* InController, const FName& InOldNodeName, const FName& InNewNodeName)
: FRigVMBaseAction(InController)
, OldNodeName(InOldNodeName.ToString())
, NewNodeName(InNewNodeName.ToString())
{
}

bool FRigVMRenameNodeAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	if (URigVMNode* Node = GetController()->GetGraph()->FindNode(NewNodeName))
	{
		return GetController()->RenameNode(Node, *OldNodeName, false);
	}
	return false;
}

bool FRigVMRenameNodeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (URigVMNode* Node = GetController()->GetGraph()->FindNode(OldNodeName))
	{
		return GetController()->RenameNode(Node, *NewNodeName, false);
	}
	return false;
}

FRigVMAddExposedPinAction::FRigVMAddExposedPinAction()
: FRigVMBaseAction(nullptr)
, Direction(ERigVMPinDirection::Input)
{
}

FRigVMAddExposedPinAction::FRigVMAddExposedPinAction(URigVMController* InController, URigVMPin* InPin)
: FRigVMBaseAction(InController)
, PinName(InPin->GetName())
, Direction(InPin->GetDirection())
, CPPType(InPin->GetCPPType())
, CPPTypeObjectPath()
, DefaultValue(InPin->GetDefaultValue())
{
	if (UObject* CPPTypeObject = InPin->GetCPPTypeObject())
	{
		CPPTypeObjectPath = CPPTypeObject->GetPathName();
	}
}

bool FRigVMAddExposedPinAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveExposedPin(*PinName, false);
}

bool FRigVMAddExposedPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->AddExposedPin(*PinName, Direction, CPPType, *CPPTypeObjectPath, DefaultValue, false).IsNone())
	{
		return FRigVMBaseAction::Redo();
	}
	return false;
}

FRigVMRemoveExposedPinAction::FRigVMRemoveExposedPinAction()
: FRigVMBaseAction(nullptr)
, Direction(ERigVMPinDirection::Input)
, PinIndex(INDEX_NONE)
{
}

FRigVMRemoveExposedPinAction::FRigVMRemoveExposedPinAction(URigVMController* InController, URigVMPin* InPin)
: FRigVMBaseAction(InController)
, PinName(InPin->GetName())
, Direction(InPin->GetDirection())
, CPPType(InPin->GetCPPType())
, CPPTypeObjectPath()
, DefaultValue(InPin->GetDefaultValue())
, PinIndex(InPin->GetPinIndex())
{
	if (UObject* CPPTypeObject = InPin->GetCPPTypeObject())
	{
		CPPTypeObjectPath = CPPTypeObject->GetPathName();
	}
}

bool FRigVMRemoveExposedPinAction::Undo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->AddExposedPin(*PinName, Direction, CPPType, *CPPTypeObjectPath, DefaultValue, false).IsNone())
	{
		if (GetController()->SetExposedPinIndex(*PinName, PinIndex, false))
		{
			return FRigVMBaseAction::Undo();
		}
	}
	return false;
}

bool FRigVMRemoveExposedPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(FRigVMBaseAction::Redo())
	{
		return GetController()->RemoveExposedPin(*PinName, false);
	}
	return false;
}

FRigVMRenameExposedPinAction::FRigVMRenameExposedPinAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMRenameExposedPinAction::FRigVMRenameExposedPinAction(URigVMController* InController, const FName& InOldPinName, const FName& InNewPinName)
: FRigVMBaseAction(InController)
, OldPinName(InOldPinName.ToString())
, NewPinName(InNewPinName.ToString())
{
}

bool FRigVMRenameExposedPinAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RenameExposedPin(*NewPinName, *OldPinName, false);
}

bool FRigVMRenameExposedPinAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(!GetController()->RenameExposedPin(*OldPinName, *NewPinName, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetPinIndexAction::FRigVMSetPinIndexAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldIndex(INDEX_NONE)
, NewIndex(INDEX_NONE)
{
}

FRigVMSetPinIndexAction::FRigVMSetPinIndexAction(URigVMController* InController, URigVMPin* InPin, int32 InNewIndex)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetName())
, OldIndex(InPin->GetPinIndex())
, NewIndex(InNewIndex)
{
}

bool FRigVMSetPinIndexAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetExposedPinIndex(*PinPath, OldIndex, false);
}

bool FRigVMSetPinIndexAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetExposedPinIndex(*PinPath, NewIndex, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMMarkExposedPinAsInputVariableAction::FRigVMMarkExposedPinAsInputVariableAction()
: FRigVMBaseAction(nullptr)
, PinPath()
, OldState(false)
, NewState(false)
{
}

FRigVMMarkExposedPinAsInputVariableAction::FRigVMMarkExposedPinAsInputVariableAction(URigVMController* InController, URigVMPin* InPin, bool bNewDefinedAsExternalVariableState)
: FRigVMBaseAction(InController)
, PinPath(InPin->GetName())
, OldState(InPin->IsDefinedAsInputVariable())
, NewState(bNewDefinedAsExternalVariableState)
{
}

bool FRigVMMarkExposedPinAsInputVariableAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->MarkExposedPinAsInputVariable(*PinPath, OldState, false);
}

bool FRigVMMarkExposedPinAsInputVariableAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->MarkExposedPinAsInputVariable(*PinPath, NewState, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMAddLocalVariableAction::FRigVMAddLocalVariableAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMAddLocalVariableAction::FRigVMAddLocalVariableAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable)
: FRigVMBaseAction(InController)
, LocalVariable(InLocalVariable)
{
	
}

bool FRigVMAddLocalVariableAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveLocalVariable(LocalVariable.Name, false);
}

bool FRigVMAddLocalVariableAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!LocalVariable.Name.IsNone())
	{
		return GetController()->AddLocalVariable(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, LocalVariable.DefaultValue, false).Name.IsNone() == false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMRemoveLocalVariableAction::FRigVMRemoveLocalVariableAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMRemoveLocalVariableAction::FRigVMRemoveLocalVariableAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable)
: FRigVMBaseAction(InController)
, LocalVariable(InLocalVariable)
{
}

bool FRigVMRemoveLocalVariableAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return !GetController()->AddLocalVariable(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, LocalVariable.DefaultValue, false).Name.IsNone();
}

bool FRigVMRemoveLocalVariableAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!LocalVariable.Name.IsNone())
	{
		return GetController()->RemoveLocalVariable(LocalVariable.Name, false);
	}
	return FRigVMBaseAction::Redo();
}

FRigVMRenameLocalVariableAction::FRigVMRenameLocalVariableAction()
: FRigVMBaseAction(nullptr)
, OldVariableName(NAME_None)
, NewVariableName(NAME_None)
{
}

FRigVMRenameLocalVariableAction::FRigVMRenameLocalVariableAction(URigVMController* InController, const FName& InOldName, const FName& InNewName)
: FRigVMBaseAction(InController)
, OldVariableName(InOldName)
, NewVariableName(InNewName)
{
	
}

bool FRigVMRenameLocalVariableAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RenameLocalVariable(NewVariableName, OldVariableName, false);
}

bool FRigVMRenameLocalVariableAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->RenameLocalVariable(OldVariableName, NewVariableName, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMChangeLocalVariableTypeAction::FRigVMChangeLocalVariableTypeAction()
: FRigVMBaseAction(nullptr)
, LocalVariable()
, CPPType()
, CPPTypeObject(nullptr)
{
}

FRigVMChangeLocalVariableTypeAction::FRigVMChangeLocalVariableTypeAction(
	URigVMController* InController, 
	const FRigVMGraphVariableDescription& InLocalVariable,
	const FString& InCPPType,
	UObject* InCPPTypeObject)
: FRigVMBaseAction(InController)
, LocalVariable(InLocalVariable)
, CPPType(InCPPType)
, CPPTypeObject(InCPPTypeObject)
{
}

bool FRigVMChangeLocalVariableTypeAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetLocalVariableType(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, false);
}

bool FRigVMChangeLocalVariableTypeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetLocalVariableType(LocalVariable.Name, CPPType, CPPTypeObject, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMChangeLocalVariableDefaultValueAction::FRigVMChangeLocalVariableDefaultValueAction()
: FRigVMBaseAction(nullptr)
, LocalVariable()
, DefaultValue()
{
}

FRigVMChangeLocalVariableDefaultValueAction::FRigVMChangeLocalVariableDefaultValueAction(
	URigVMController* InController, 
	const FRigVMGraphVariableDescription& InLocalVariable,
	const FString& InDefaultValue)
: FRigVMBaseAction(InController)
, LocalVariable(InLocalVariable)
, DefaultValue(InDefaultValue)
{
}

bool FRigVMChangeLocalVariableDefaultValueAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetLocalVariableDefaultValue(LocalVariable.Name, LocalVariable.DefaultValue, false);
}

bool FRigVMChangeLocalVariableDefaultValueAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetLocalVariableDefaultValue(LocalVariable.Name, DefaultValue, false))
	{
		return false;		
	}
	return FRigVMBaseAction::Redo();
}

FRigVMSetLocalVariableIndexAction::FRigVMSetLocalVariableIndexAction()
: FRigVMBaseAction(nullptr)
, VariableGuid()
, OldIndex(INDEX_NONE)
, NewIndex(INDEX_NONE)
{
}

FRigVMSetLocalVariableIndexAction::FRigVMSetLocalVariableIndexAction(
	URigVMController* InController,
	const FGuid& InVariableGuid,
	int32 InOldIndex,
	int32 InNewIndex)
: FRigVMBaseAction(InController)
, VariableGuid(InVariableGuid)
, OldIndex(InOldIndex)
, NewIndex(InNewIndex)
{
}

bool FRigVMSetLocalVariableIndexAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->SetLocalVariableIndex(VariableGuid, OldIndex, false);
}

bool FRigVMSetLocalVariableIndexAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (!GetController()->SetLocalVariableIndex(VariableGuid, NewIndex, false))
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMPromoteNodeAction::FRigVMPromoteNodeAction()
: FRigVMBaseAction(nullptr)
, LibraryNodePath()
, FunctionDefinitionPath()
, bFromFunctionToCollapseNode(false)
{
}

FRigVMPromoteNodeAction::FRigVMPromoteNodeAction(URigVMController* InController, const URigVMNode* InNodeToPromote, const FString& InNodePath, const FString& InFunctionDefinitionPath)
: FRigVMBaseAction(InController)
, LibraryNodePath(InNodePath)
, FunctionDefinitionPath(InFunctionDefinitionPath)
, bFromFunctionToCollapseNode(InNodeToPromote->IsA<URigVMFunctionReferenceNode>())
{
}

bool FRigVMPromoteNodeAction::Undo()
{
	if(bFromFunctionToCollapseNode)
	{
		const FName FunctionRefNodeName = GetController()->PromoteCollapseNodeToFunctionReferenceNode(*LibraryNodePath, false, false, FunctionDefinitionPath);
		return FunctionRefNodeName.ToString() == LibraryNodePath;
	}

	const FName CollapseNodeName = GetController()->PromoteFunctionReferenceNodeToCollapseNode(*LibraryNodePath, false, false, true);
	return CollapseNodeName.ToString() == LibraryNodePath;
}

bool FRigVMPromoteNodeAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if(bFromFunctionToCollapseNode)
	{
		const FName CollapseNodeName = GetController()->PromoteFunctionReferenceNodeToCollapseNode(*LibraryNodePath, false, false, false);
		return CollapseNodeName.ToString() == LibraryNodePath;
	}
	const FName FunctionRefNodeName = GetController()->PromoteCollapseNodeToFunctionReferenceNode(*LibraryNodePath, false, false);
	return FunctionRefNodeName.ToString() == LibraryNodePath;
}

FRigVMMarkFunctionPublicAction::FRigVMMarkFunctionPublicAction()
: FRigVMBaseAction(nullptr)
, FunctionName(NAME_None)
, bIsPublic(false)
{
}

FRigVMMarkFunctionPublicAction::FRigVMMarkFunctionPublicAction(URigVMController* InController, const FName& InFunctionName, bool bInIsPublic)
: FRigVMBaseAction(InController)
, FunctionName(InFunctionName)
, bIsPublic(bInIsPublic)
{
}

bool FRigVMMarkFunctionPublicAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->MarkFunctionAsPublic(FunctionName, !bIsPublic, false);
}

bool FRigVMMarkFunctionPublicAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	if (GetController()->MarkFunctionAsPublic(FunctionName, bIsPublic, false))
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMCreateFunctionVariantAction::FRigVMCreateFunctionVariantAction()
: FRigVMBaseAction(nullptr)
, FunctionName(NAME_None)
{
}

FRigVMCreateFunctionVariantAction::FRigVMCreateFunctionVariantAction(URigVMController* InController, const FName& InFunctionName, const FName& InNewFunctionName)
: FRigVMBaseAction(InController)
, FunctionName(InFunctionName)
, NewFunctionName(InNewFunctionName)
{
}

bool FRigVMCreateFunctionVariantAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveFunctionFromLibrary(NewFunctionName);
}

bool FRigVMCreateFunctionVariantAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	if (GetController()->CreateFunctionVariant(FunctionName, NewFunctionName, false))
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMAddFunctionVariantTagAction::FRigVMAddFunctionVariantTagAction()
: FRigVMBaseAction(nullptr)
, FunctionName(NAME_None)
{
	
}

FRigVMAddFunctionVariantTagAction::FRigVMAddFunctionVariantTagAction(URigVMController* InController, const FName& InFunctionName, const FRigVMTag& InTag)
: FRigVMBaseAction(InController)
, FunctionName(InFunctionName)
, FunctionTag(InTag)
{
}

bool FRigVMAddFunctionVariantTagAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveTagFromFunctionVariant(FunctionName, FunctionTag.Name, false);
}

bool FRigVMAddFunctionVariantTagAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	if (GetController()->AddTagToFunctionVariant(FunctionName, FunctionTag, false))
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMRemoveFunctionVariantTagAction::FRigVMRemoveFunctionVariantTagAction()
: FRigVMBaseAction(nullptr)
, FunctionName(NAME_None)
{
	
}

FRigVMRemoveFunctionVariantTagAction::FRigVMRemoveFunctionVariantTagAction(URigVMController* InController, const FName& InFunctionName, const FName& InTagName)
: FRigVMBaseAction(InController)
, FunctionName(InFunctionName)
{
	if (URigVMFunctionLibrary* Library = Cast<URigVMFunctionLibrary>(InController->GetGraph()))
	{
		if (const FRigVMVariant* Variant = Library->GetFunctionVariant(InFunctionName))
		{
			if (const FRigVMTag* Tag = Variant->Tags.FindByPredicate([InTagName](const FRigVMTag& InTag)
			{
				return InTag.Name == InTagName;
			}))
			{
				FunctionTag = *Tag;
			}
		}
	}
}

bool FRigVMRemoveFunctionVariantTagAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->AddTagToFunctionVariant(FunctionName, FunctionTag, false);
}

bool FRigVMRemoveFunctionVariantTagAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
#if WITH_EDITOR
	if (GetController()->RemoveTagFromFunctionVariant(FunctionName, FunctionTag.Name, false))
	{
		return FRigVMBaseAction::Redo();
	}
#endif
	return false;
}

FRigVMImportFromTextAction::FRigVMImportFromTextAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMImportFromTextAction::FRigVMImportFromTextAction(URigVMController* InController, const FString& InContent, const TArray<FName>& InTopLevelNodeNames)
: FRigVMBaseAction(InController)
, Content(InContent)
, TopLevelNodeNames(InTopLevelNodeNames)
{
}

FRigVMImportFromTextAction::FRigVMImportFromTextAction(URigVMController* InController, URigVMNode* InNode, bool bIncludeExteriorLinks)
: FRigVMBaseAction(InController)
{
	SetContent({InNode}, bIncludeExteriorLinks);
}

FRigVMImportFromTextAction::FRigVMImportFromTextAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes, bool bIncludeExteriorLinks)
: FRigVMBaseAction(InController)
{
	SetContent(InNodes, bIncludeExteriorLinks);
}

void FRigVMImportFromTextAction::SetContent(const TArray<URigVMNode*>& InNodes, bool bIncludeExteriorLinks)
{
	TopLevelNodeNames.Reset();
	Algo::Transform(InNodes, TopLevelNodeNames, [](const URigVMNode* Node)
	{
		return Node->GetFName();
	});
	Content = GetController()->ExportNodesToText(TopLevelNodeNames, bIncludeExteriorLinks);
}

bool FRigVMImportFromTextAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveNodesByName(TopLevelNodeNames, false, false);
}

bool FRigVMImportFromTextAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	const TArray<FName> ImportedNames = GetController()->ImportNodesFromText(Content, false, false);
	if(ImportedNames.Num() != TopLevelNodeNames.Num())
	{
		return false;
	}
	return FRigVMBaseAction::Redo();
}

FRigVMReplaceNodesAction::FRigVMReplaceNodesAction()
: FRigVMBaseAction(nullptr)
{
}

FRigVMReplaceNodesAction::FRigVMReplaceNodesAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes)
: FRigVMBaseAction(InController)
{
	for(const URigVMNode* Node : InNodes)
	{
		StoreNode(Node, true);
	}
}

bool FRigVMReplaceNodesAction::Undo()
{
	if(!FRigVMBaseAction::Undo())
	{
		return false;
	}

	for(const TPair<FName, FRigVMActionNodeContent>& Pair : ExportedNodes)
	{
		if(!RestoreNode(Pair.Key, true))
		{
			return false;
		}
	}

	return true;
}

bool FRigVMReplaceNodesAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	for(const TPair<FName, FRigVMActionNodeContent>& Pair : ExportedNodes)
	{
		if(!RestoreNode(Pair.Key, false))
		{
			return false;
		}
	}
	
	return FRigVMBaseAction::Redo();
}

FRigVMAddTraitAction::FRigVMAddTraitAction()
: FRigVMBaseAction()
, NodeName(NAME_None)
, TraitName(NAME_None)
, ScriptStructPath()
, TraitDefault()
, PinIndex(INDEX_NONE)
{
}

FRigVMAddTraitAction::FRigVMAddTraitAction(URigVMController* InController, const URigVMNode* InNode, const FName& InTraitName,
	const UScriptStruct* InTraitScriptStruct, const FString& InTraitDefault, int32 InPinIndex)
: FRigVMBaseAction(InController)
, NodeName(InNode->GetFName())
, TraitName(InTraitName)
, ScriptStructPath(InTraitScriptStruct->GetPathName())
, TraitDefault(InTraitDefault)
, PinIndex(InPinIndex)
{
}

bool FRigVMAddTraitAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->RemoveTrait(NodeName, TraitName, false);
}

bool FRigVMAddTraitAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (GetController()->AddTrait(NodeName, *ScriptStructPath, TraitName, TraitDefault, PinIndex, false, false) == TraitName)
	{
		return FRigVMBaseAction::Redo();
	}
	return false;
}

FRigVMRemoveTraitAction::FRigVMRemoveTraitAction()
: FRigVMAddTraitAction()
{
}

FRigVMRemoveTraitAction::FRigVMRemoveTraitAction(URigVMController* InController, const URigVMNode* InNode,
	const FName& InTraitName, const UScriptStruct* InTraitScriptStruct, const FString& InTraitDefault, int32 InPinIndex)
: FRigVMAddTraitAction(InController, InNode, InTraitName, InTraitScriptStruct, InTraitDefault, InPinIndex)
{
}

bool FRigVMRemoveTraitAction::Undo()
{
	if (!FRigVMBaseAction::Undo())
	{
		return false;
	}
	return GetController()->AddTrait(NodeName, *ScriptStructPath, TraitName, TraitDefault, PinIndex, false, false) == TraitName;
}

bool FRigVMRemoveTraitAction::Redo()
{
	if(!CanUndoRedo())
	{
		return false;
	}
	if (GetController()->RemoveTrait(NodeName, TraitName, false))
	{
		return FRigVMBaseAction::Redo();
	}
	return false;
}
