// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMExternalDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMExternalDependency)

TArray<FName> IRigVMExternalDependencyManager::DependencyCategories = {
	UserDefinedEnumCategory,
	UserDefinedStructCategory,
	RigVMGraphFunctionCategory
};

const TArray<FName>& IRigVMExternalDependencyManager::GetExternalDependencyCategories() const
{
	return IRigVMExternalDependencyManager::DependencyCategories;
}

TArray<FRigVMExternalDependency> IRigVMExternalDependencyManager::GetAllExternalDependencies() const
{
	const TArray<FName>& Categories = GetExternalDependencyCategories();
	TArray<FRigVMExternalDependency> Dependencies;
	for(const FName& Category : Categories)
	{
		Dependencies.Append(GetExternalDependenciesForCategory(Category));
	}
	return Dependencies;
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMClient* InClient) const
{
	check(InClient);

	const TArray<URigVMGraph*> Graphs = InClient->GetAllModels(true, true);
	for(const URigVMGraph* Graph : Graphs)
	{
		CollectExternalDependencies(OutDependencies, InCategory, Graph);
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionStore* InFunctionStore) const
{
	check(InFunctionStore);
	for(const FRigVMGraphFunctionData& Function : InFunctionStore->PublicFunctions)
	{
		CollectExternalDependencies(OutDependencies, InCategory, &Function);
	}
	for(const FRigVMGraphFunctionData& Function : InFunctionStore->PrivateFunctions)
	{
		CollectExternalDependencies(OutDependencies, InCategory, &Function);
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionData* InFunction) const
{
	check(InFunction);
	CollectExternalDependencies(OutDependencies, InCategory, &InFunction->Header);
	CollectExternalDependencies(OutDependencies, InCategory, &InFunction->CompilationData);
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionHeader* InHeader) const
{
	check(InHeader);
	if(InCategory == RigVMGraphFunctionCategory)
	{
		OutDependencies.AddUnique({InHeader->LibraryPointer.GetLibraryNodePath(), InCategory});
	}
	for(const FRigVMGraphFunctionArgument& Argument : InHeader->Arguments)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Argument.CPPTypeObject.Get());
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMFunctionCompilationData* InFunction) const
{
	check(InFunction);

	for(const FRigVMFunctionCompilationPropertyDescription& Property : InFunction->LiteralPropertyDescriptions)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Property.CPPTypeObject.Get());
	}
	for(const FRigVMFunctionCompilationPropertyDescription& Property : InFunction->WorkPropertyDescriptions)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, Property.CPPTypeObject.Get());
	}

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();
	for(const FName& FunctionName : InFunction->FunctionNames)
	{
		if(const FRigVMFunction* Function = Registry.FindFunction(*FunctionName.ToString()))
		{
			for(const FRigVMFunctionArgument& Argument :  Function->Arguments)
			{
				const FRigVMTemplateArgumentType& ArgumentType = Registry.FindTypeFromCPPType(Argument.Type);
				if(ArgumentType.IsValid())
				{
					CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, ArgumentType.CPPTypeObject.Get());
				}
			}
		}
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMGraph* InGraph) const
{
	for(const URigVMNode* Node : InGraph->GetNodes())
	{
		CollectExternalDependencies(OutDependencies, InCategory, Node);
	}

	const TArray<FRigVMGraphVariableDescription> LocalVariables = InGraph->GetLocalVariables();
	for(const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
	{
		CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, LocalVariable.CPPTypeObject.Get());
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMNode* InNode) const
{
	for(const URigVMPin* Pin : InNode->GetPins())
	{
		CollectExternalDependencies(OutDependencies, InCategory, Pin);
	}
	if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		if(const FRigVMGraphFunctionData* Function = FunctionReferenceNode->GetReferencedFunctionData(true))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Function);
		}
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMPin* InPin) const
{
	CollectExternalDependenciesForCPPTypeObject(OutDependencies, InCategory, InPin->GetCPPTypeObject());
	for(const URigVMPin* SubPin : InPin->GetSubPins())
	{
		CollectExternalDependencies(OutDependencies, InCategory, SubPin);
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UStruct* InStruct) const
{
	check(InStruct);
	if(InCategory == UserDefinedStructCategory)
	{
		if(const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InStruct))
		{
			OutDependencies.AddUnique({UserDefinedStruct->GetPathName(), InCategory});
		}
	}
	for (TFieldIterator<FProperty> PropertyIt(InStruct); PropertyIt; ++PropertyIt)
	{
		const FProperty* Property = *PropertyIt;
		while(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			Property = ArrayProperty->Inner;
		}
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			CollectExternalDependencies(OutDependencies, InCategory, StructProperty->Struct);
		}
		else if(const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			if(const UEnum* Enum = EnumProperty->GetEnum())
			{
				CollectExternalDependencies(OutDependencies, InCategory, Enum);
			}
		}
		else if(const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if(const UEnum* Enum = ByteProperty->Enum)
			{
				CollectExternalDependencies(OutDependencies, InCategory, Enum);
			}
		}
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UEnum* InEnum) const
{
	check(InEnum);
	if(InCategory == UserDefinedEnumCategory)
	{
		if(const UUserDefinedEnum* UserDefinedEnum = Cast<UUserDefinedEnum>(InEnum))
		{
			OutDependencies.AddUnique({UserDefinedEnum->GetPathName(), InCategory});
		}
	}
}

void IRigVMExternalDependencyManager::CollectExternalDependenciesForCPPTypeObject(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UObject* InObject) const
{
	if(InObject)
	{
		if(const UEnum* Enum = Cast<UEnum>(InObject))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Enum);
		}
		else if(const UStruct* Struct = Cast<UStruct>(InObject))
		{
			CollectExternalDependencies(OutDependencies, InCategory, Struct);
		}
	}
}
