// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMFunctionLibrary.h"

#include "RigVMModel/RigVMController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMFunctionLibrary)

URigVMFunctionLibrary::URigVMFunctionLibrary()
: URigVMGraph()
{
	
}

FString URigVMFunctionLibrary::GetNodePath() const
{
	static constexpr TCHAR FunctionLibraryNodePath[] = TEXT("FunctionLibrary::");
	return FunctionLibraryNodePath;
}

URigVMFunctionLibrary* URigVMFunctionLibrary::GetDefaultFunctionLibrary() const
{
	if(URigVMFunctionLibrary* DefaultFunctionLibrary = Super::GetDefaultFunctionLibrary())
	{
		return DefaultFunctionLibrary;
	}
	return (URigVMFunctionLibrary*)this;
}

TArray<URigVMLibraryNode*> URigVMFunctionLibrary::GetFunctions() const
{
	TArray<URigVMLibraryNode*> Functions;

	for (URigVMNode* Node : GetNodes())
	{
		// we only allow library nodes under a function library graph
		URigVMLibraryNode* LibraryNode = CastChecked<URigVMLibraryNode>(Node);
		Functions.Add(LibraryNode);
	}

	return Functions;
}

URigVMLibraryNode* URigVMFunctionLibrary::FindFunction(const FName& InFunctionName) const
{
	FString FunctionNameStr = InFunctionName.ToString();
	if (FunctionNameStr.StartsWith(TEXT("FunctionLibrary::|")))
	{
		FunctionNameStr.RightChopInline(18);
	}
	return Cast<URigVMLibraryNode>(FindNodeByName(*FunctionNameStr));
}

URigVMLibraryNode* URigVMFunctionLibrary::FindFunctionForNode(URigVMNode* InNode) const
{
	if(InNode == nullptr)
	{
		return nullptr;
	}
	
	UObject* Subject = InNode;
	while (Subject->GetOuter() != this)
	{
		Subject = Subject->GetOuter();
		if(Subject == nullptr)
		{
			return nullptr;
		}
	}

	return Cast<URigVMLibraryNode>(Subject);
}

TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > URigVMFunctionLibrary::GetReferencesForFunction(const FName& InFunctionName)
{
	TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > Result;

	ForEachReferenceSoftPtr(InFunctionName, [&Result](TSoftObjectPtr<URigVMFunctionReferenceNode> Reference)
	{
		Result.Add(TSoftObjectPtr<URigVMFunctionReferenceNode>(Reference.GetUniqueID()));
	});

	return Result;
}

TArray< FString > URigVMFunctionLibrary::GetReferencePathsForFunction(const FName& InFunctionName)
{
	TArray< FString > Result;

	ForEachReferenceSoftPtr(InFunctionName, [&Result](TSoftObjectPtr<URigVMFunctionReferenceNode> Reference)
	{
		Result.Add(Reference.ToString());
	});

	return Result;
}

void URigVMFunctionLibrary::ForEachReference(const FName& InFunctionName,
	TFunction<void(URigVMFunctionReferenceNode*)> PerReferenceFunction,
	bool bLoadIfNecessary) const
{
	if (URigVMLibraryNode* Function = FindFunction(InFunctionName))
	{
		if(const URigVMBuildData* BuildData = URigVMBuildData::Get())
		{
			BuildData->ForEachFunctionReference(Function->GetFunctionIdentifier(),PerReferenceFunction, bLoadIfNecessary);
		}
	}
}

void URigVMFunctionLibrary::ForEachReferenceSoftPtr(const FName& InFunctionName,
	TFunction<void(TSoftObjectPtr<URigVMFunctionReferenceNode>)> PerReferenceFunction) const
{
	if (URigVMLibraryNode* Function = FindFunction(InFunctionName))
	{
		if(const URigVMBuildData* BuildData = URigVMBuildData::Get())
		{
			BuildData->ForEachFunctionReferenceSoftPtr(Function->GetFunctionIdentifier(), PerReferenceFunction);
		}
	}
}

URigVMLibraryNode* URigVMFunctionLibrary::FindPreviouslyLocalizedFunction(FRigVMGraphFunctionIdentifier InFunctionToLocalize)
{
	const FString& PathName = InFunctionToLocalize.GetLibraryNodePath();
	if(!LocalizedFunctions.Contains((PathName)))
	{
		return nullptr;
	}

	FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(InFunctionToLocalize);
	if (!FunctionData)
	{
		return nullptr;
	}
	FRigVMGraphFunctionHeader& Header = FunctionData->Header;
	
	URigVMLibraryNode* LocalizedFunction = LocalizedFunctions.FindChecked(PathName);

	// once we found the function - let's make sure it's notation is right
	if(LocalizedFunction->GetPins().Num() != Header.Arguments.Num())
	{
		return nullptr;
	}
	for(int32 PinIndex=0; PinIndex < Header.Arguments.Num(); PinIndex++)
	{
		FRigVMGraphFunctionArgument& Argument = Header.Arguments[PinIndex];
		URigVMPin* Pin = LocalizedFunction->GetPins()[PinIndex];

		if((Argument.Name != Pin->GetFName()) ||
			(Argument.CPPType.ToString() != Pin->GetCPPType()) ||
			(Argument.CPPTypeObject != Pin->GetCPPTypeObject()) ||
			(Argument.bIsArray != Pin->IsArray()))
		{
			return nullptr;
		}
	}
	
	return LocalizedFunction;
}

const FSoftObjectPath URigVMFunctionLibrary::GetFunctionHostObjectPath() const
{
	if (GetFunctionHostObjectPathDelegate.IsBound())
	{
		return GetFunctionHostObjectPathDelegate.Execute();
	}
	return FSoftObjectPath();
}

const FRigVMVariant* URigVMFunctionLibrary::GetFunctionVariant(const FName& InFunctionName) const
{
	return const_cast<URigVMFunctionLibrary*>(this)->GetFunctionVariant(InFunctionName);
}

FRigVMVariant* URigVMFunctionLibrary::GetFunctionVariant(const FName& InFunctionName)
{
	return FunctionToVariant.Find(InFunctionName);
}


