// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMFunctions/Execution/RigVMFunction_BreakLoop.h"
#include "RigVMFunctions/Execution/RigVMFunction_ContinueLoop.h"
#include "UObject/Package.h"
#include "UObject/ObjectSaveContext.h"
#include "RigVMTypeUtils.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMDeveloperModule.h"

#if WITH_EDITOR
#include "RigVMSettings.h"
#endif

#include "ScribbleGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMGraph)

URigVMGraph::URigVMGraph()
: DiagnosticsAST(nullptr)
, RuntimeAST(nullptr)
, bEditable(true)
, SchemaClass(nullptr)
, bSectionsEnabled(false)
{
	SetExecuteContextStruct(FRigVMExecuteContext::StaticStruct());
}

void URigVMGraph::PostLoad()
{
	Super::PostLoad();

	Nodes.Remove(nullptr);
	Links.Remove(nullptr);

	// update the bookkeeping on the links
	for(URigVMLink* Link : Links)
	{
		URigVMPin* SourcePin = Link->GetSourcePin();
		URigVMPin* TargetPin = Link->GetTargetPin();
		if ((SourcePin == nullptr) != (TargetPin == nullptr))
		{
			static constexpr TCHAR Format[] = TEXT("Cannot add link %s in package %s.");
			UE_LOG(LogRigVMDeveloper, Warning, Format, *Link->GetPinPathRepresentation(), *GetPackage()->GetPathName());
		}
		
		if(SourcePin)
		{
			SourcePin->Links.AddUnique(Link);
		}
		if(TargetPin)
		{
			TargetPin->Links.AddUnique(Link);
		}
	}

	SetupScribbleGraph();
}

const TArray<URigVMNode*>& URigVMGraph::GetNodes() const
{
	return Nodes;
}

const TArray<URigVMLink*>& URigVMGraph::GetLinks() const
{
	return Links;
}

bool URigVMGraph::ContainsLink(const FString& InPinPathRepresentation) const
{
	return FindLink(InPinPathRepresentation) != nullptr;
}

TArray<URigVMGraph*> URigVMGraph::GetContainedGraphs(bool bRecursive) const
{
	TArray<URigVMGraph*> Graphs;
	for (URigVMNode* Node : GetNodes())
	{
		if (const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if (ensure(CollapseNode->GetContainedGraph()))
			{
				Graphs.AddUnique(CollapseNode->GetContainedGraph());
				if (bRecursive)
				{
					Graphs.Append(CollapseNode->GetContainedGraph()->GetContainedGraphs(true));
				}
			}
		}
	}
	return Graphs;
}

URigVMGraph* URigVMGraph::GetParentGraph() const
{
	if(URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetOuter()))
	{
		return CollapseNode->GetGraph();
	}
	return nullptr;
}

URigVMGraph* URigVMGraph::GetRootGraph() const
{
	if(URigVMGraph* ParentGraph = GetParentGraph())
	{
		return ParentGraph->GetRootGraph();
	}
	return (URigVMGraph*)this;
}

int32 URigVMGraph::GetGraphDepth() const
{
	if(const URigVMGraph* ParentGraph = GetParentGraph())
	{
		return ParentGraph->GetGraphDepth() + 1;
	}
	return 0;
}

bool URigVMGraph::IsRootGraph() const
{
	return GetRootGraph() == this;
}

URigVMFunctionEntryNode* URigVMGraph::GetEntryNode() const
{
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(Node))
		{
			return EntryNode;
		}
	}
	return nullptr;
}

URigVMFunctionReturnNode* URigVMGraph::GetReturnNode() const
{
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(Node))
		{
			return ReturnNode;
		}
	}
	return nullptr;
}

TArray<FName> URigVMGraph::GetEventNames() const
{
	TArray<FName> EventNames;
	
	if (!IsTopLevelGraph())
	{
		return EventNames;
	}
	
	if (IsA<URigVMFunctionLibrary>())
	{
		return EventNames;
	}

	for (const URigVMNode* Node : Nodes)
	{
		if (Node->IsEvent())
		{
			EventNames.Add(Node->GetEventName());
		}
	}

	return EventNames;
}

TArray<FRigVMGraphVariableDescription> URigVMGraph::GetVariableDescriptions() const
{
	TArray<FRigVMGraphVariableDescription> Variables;
	for (URigVMNode* Node : Nodes)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			Variables.AddUnique(VariableNode->GetVariableDescription());
		}
	}
	return Variables;
}

FString URigVMGraph::GetNodePath() const
{
	if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetOuter()))
	{
		return CollapseNode->GetNodePath(true /* recursive */);
	}

	static constexpr TCHAR NodePathFormat[] = TEXT("%s::");
	return FString::Printf(NodePathFormat, *GetName());
}

FString URigVMGraph::GetGraphName() const
{
	if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetOuter()))
	{
		return CollapseNode->GetNodePath(false /* recursive */);
	}
	return GetName();
}

URigVMNode* URigVMGraph::FindNodeByName(const FName& InNodeName) const
{
	for (URigVMNode* Node : Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (Node->GetFName() == InNodeName)
		{
			return Node;
		}
	}
	return nullptr;
}

URigVMNode* URigVMGraph::FindNode(const FString& InNodePath) const
{
	if (InNodePath.IsEmpty())
	{
		return nullptr;
	}

	FString Path = InNodePath;

	if(IsRootGraph())
	{
		const FString MyNodePath = GetNodePath();
		if(Path.StartsWith(MyNodePath))
		{
			Path.RightChopInline(MyNodePath.Len() + 1);
		}
	}

	FString Left = Path, Right;
	URigVMNode::SplitNodePathAtStart(Path, Left, Right);

	if (Right.IsEmpty())
	{
		return FindNodeByName(*Left);
	}

	if (URigVMCollapseNode* CollapseNode = Cast< URigVMCollapseNode>(FindNodeByName(*Left)))
	{
		if(const URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph())
		{
			return ContainedGraph->FindNode(Right);
		}
		else
		{
			static constexpr TCHAR Format[] = TEXT("Collapse Node '%s' does not contain a subgraph.");
			UE_LOG(LogRigVMDeveloper, Warning, Format, *CollapseNode->GetPathName());
		}
	}

	return nullptr;
}

URigVMPin* URigVMGraph::FindPin(const FString& InPinPath) const
{
	FString Left, Right;
	if (!URigVMPin::SplitPinPathAtStart(InPinPath, Left, Right))
	{
		Left = InPinPath;
	}

	URigVMNode* Node = FindNode(Left);
	if (Node)
	{
		if (URigVMPin* Pin = Node->FindPin(Right))
		{
			return Pin;
		}
	}

	return nullptr;
}

URigVMLink* URigVMGraph::FindLink(const FString& InLinkPinPathRepresentation) const
{
	for(URigVMLink* Link : Links)
	{
		if(Link->GetPinPathRepresentation() == InLinkPinPathRepresentation)
		{
			return Link;
		}
	}
	return nullptr;
}

bool URigVMGraph::IsNodeSelected(const FName& InNodeName) const
{
	return SelectedNodes.Contains(InNodeName);
}

const TArray<FName>& URigVMGraph::GetSelectNodes() const
{
	return SelectedNodes;
}

bool URigVMGraph::IsNodeHighlighted(const FName& InNodeName) const
{
	for (const FRigVMGraphSection& MatchingSection : SectionsMatchingTheSelection)
	{
		if (MatchingSection.ContainsNode(InNodeName))
		{
			return true;
		}
	}
	return false;
}

bool URigVMGraph::IsTopLevelGraph() const
{
	if (GetOuter()->IsA<URigVMLibraryNode>())
	{
		return false;
	}
	return true;
}

URigVMFunctionLibrary* URigVMGraph::GetDefaultFunctionLibrary() const
{
	if (DefaultFunctionLibraryPtr.IsValid())
	{
		return CastChecked<URigVMFunctionLibrary>(DefaultFunctionLibraryPtr.Get());
	}

	if (URigVMLibraryNode* OuterLibraryNode = Cast<URigVMLibraryNode>(GetOuter()))
	{
		if (URigVMGraph* OuterGraph = OuterLibraryNode->GetGraph())
		{
			return OuterGraph->GetDefaultFunctionLibrary();
		}
	}
	return nullptr;
}

void URigVMGraph::SetDefaultFunctionLibrary(URigVMFunctionLibrary* InFunctionLibrary)
{
	DefaultFunctionLibraryPtr = InFunctionLibrary;
}

TArray<FRigVMExternalVariable> URigVMGraph::GetExternalVariables() const
{
	TArray<FRigVMExternalVariable> Variables;
	
	for(const URigVMNode* Node : GetNodes())
	{
		if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
		{
			const TArray<FRigVMExternalVariable> FuncRefVariables = FunctionReferenceNode->GetExternalVariables();
			for(const FRigVMExternalVariable& FuncRefVariable : FuncRefVariables)
			{
				FRigVMExternalVariable::MergeExternalVariable(Variables, FuncRefVariable);
			}
		}
		else if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			const TArray<FRigVMExternalVariable> ContainedVariables = CollapseNode->GetExternalVariables();
			for(const FRigVMExternalVariable& ContainedVariable : ContainedVariables)
			{
				FRigVMExternalVariable::MergeExternalVariable(Variables, ContainedVariable);
			}
		}
		else if(const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Node))
		{
			// Make sure it is not a local variable or input argument
			if (VariableNode->IsExternalVariable())
			{
				FRigVMExternalVariable::MergeExternalVariable(Variables, VariableNode->GetVariableDescription().ToExternalVariable());
			}
		}
	}
	
	return Variables;
}

TArray<FRigVMGraphVariableDescription> URigVMGraph::GetLocalVariables(bool bIncludeInputArguments) const
{
	if (bIncludeInputArguments)
	{
		TArray<FRigVMGraphVariableDescription> Variables;
		Variables.Append(GetInputArguments());
		Variables.Append(LocalVariables);
		return Variables;
	}
	
	return LocalVariables;
}

TArray<FRigVMGraphVariableDescription> URigVMGraph::GetInputArguments() const
{
	TArray<FRigVMGraphVariableDescription> Inputs;
	if (URigVMFunctionEntryNode* EntryNode = GetEntryNode())
	{
		for (URigVMPin* Pin : EntryNode->GetPins())
		{
			if(Pin->IsExecuteContext())
			{
				continue;
			}
			if (Pin->IsDefinedAsInputVariable())
			{
				continue;
			}
			FRigVMGraphVariableDescription Description;
			if (const URigVMPin* ReferencedPin = EntryNode->FindReferencedPin(Pin))
			{
				if (const URigVMCollapseNode* CollapseNode = EntryNode->FindReferencedNode())
				{
					Description.Guid = CollapseNode->FindPinGuid(ReferencedPin);
				}
			}
			Description.Name = Pin->GetFName();
			Description.CPPType = Pin->GetCPPType();
			Description.CPPTypeObject = Pin->GetCPPTypeObject();
			if (Description.CPPTypeObject)
			{
				Description.CPPTypeObjectPath = *Description.CPPTypeObject->GetPathName();
			}
			Inputs.Add(Description);
		}
	}
	return Inputs;
}

TArray<FRigVMGraphVariableDescription> URigVMGraph::GetOutputArguments() const
{
	TArray<FRigVMGraphVariableDescription> Outputs;
	if (URigVMFunctionReturnNode* ReturnNode = GetReturnNode())
	{
		for (URigVMPin* Pin : ReturnNode->GetPins())
		{
			if(Pin->IsExecuteContext())
			{
				continue;
			}
			FRigVMGraphVariableDescription Description;
			Description.Name = Pin->GetFName();
			Description.CPPType = Pin->GetCPPType();
			Description.CPPTypeObject = Pin->GetCPPTypeObject();
			if (Description.CPPTypeObject)
			{
				Description.CPPTypeObjectPath = *Description.CPPTypeObject->GetPathName();
			}
			Outputs.Add(Description);
		}
	}
	return Outputs;
}

TArray<FRigVMGraphVariableDescription> URigVMGraph::GetInputVariables() const
{
	TArray<FRigVMGraphVariableDescription> InputVariables;
	if (URigVMFunctionEntryNode* EntryNode = GetEntryNode())
	{
		for (URigVMPin* Pin : EntryNode->GetPins())
		{
			if (!Pin->IsDefinedAsInputVariable())
			{
				continue;
			}
			InputVariables.Add(Pin->GetInputVariableDescription());
		}
	}
	return InputVariables;
}

URigVMSchema* URigVMGraph::GetSchema() const
{
	return SchemaClass ? SchemaClass->GetDefaultObject<URigVMSchema>() : nullptr;
}

void URigVMGraph::SetSchemaClass(TSubclassOf<URigVMSchema> InSchemaClass)
{
	SchemaClass = InSchemaClass;
}

FRigVMGraphModifiedEvent& URigVMGraph::OnModified()
{
	return ModifiedEvent;
}

void URigVMGraph::SetExecuteContextStruct(UScriptStruct* InExecuteContextStruct)
{
	check(InExecuteContextStruct);
	ensure(InExecuteContextStruct->IsChildOf(FRigVMExecuteContext::StaticStruct()));
	ExecuteContextStruct = InExecuteContextStruct;
}

UScriptStruct* URigVMGraph::GetExecuteContextStruct() const
{
	if (URigVMGraph* RootGraph = GetRootGraph())
	{
		return RootGraph->ExecuteContextStruct.Get();
	}
	return nullptr;
}

TArray<FRigVMGraphSection> URigVMGraph::GetMatchingSections(const FRigVMGraphSection& InSection) const
{
	TArray<FRigVMGraphSection> Matches;
	for (const FRigVMGraphSection& Match : SectionsMatchingTheSelection)
	{
		if (Match.Hash == InSection.Hash)
		{
			Matches.Add(Match);
		}
	}
	return Matches;
}

void URigVMGraph::UpdateSections(bool bForce)
{
	SelectedSections.Reset();
	SectionsMatchingTheSelection.Reset();

	if (SelectedNodes.IsEmpty())
	{
		return;
	}

	if (!AreSectionsEnabled() && !bForce)
	{
		return;
	}

	SelectedSections = FRigVMGraphSection::GetSectionsFromSelection(this);
	
	TArray<URigVMNode*> AvailableNodes = Nodes;
	AvailableNodes.RemoveAll([this]( const URigVMNode* Node) -> bool
	{
		return SelectedNodes.Contains(Node->GetFName()) || !FRigVMGraphSection::IsValidNode(Node);
	});

	for (const FRigVMGraphSection& SelectedSection : SelectedSections)
	{
		SectionsMatchingTheSelection.Append(SelectedSection.FindMatches(AvailableNodes));
	}
}

void URigVMGraph::Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject)
{
	ModifiedEvent.Broadcast(InNotifType, this, InSubject);
}

TSharedPtr<FRigVMParserAST> URigVMGraph::GetDiagnosticsAST(bool bForceRefresh, TArray<URigVMLink*> InLinksToSkip)
{
	// only refresh the diagnostics AST if have a different set of links to skip
	if (!bForceRefresh && DiagnosticsAST.IsValid())
	{
		const TArray<URigVMLink*> PreviousLinksToSkip = DiagnosticsAST->GetSettings().LinksToSkip;
		if (PreviousLinksToSkip.Num() < InLinksToSkip.Num())
		{
			bForceRefresh = true;
		}
		else
		{
			for (int32 LinkIndex = 0; LinkIndex < InLinksToSkip.Num(); LinkIndex++)
			{
				if (PreviousLinksToSkip[LinkIndex] != InLinksToSkip[LinkIndex])
				{
					bForceRefresh = true;
					break;
				}
			}
		}
	}

	if (DiagnosticsAST == nullptr || bForceRefresh)
	{
		FRigVMParserASTSettings Settings = FRigVMParserASTSettings::Fast();
		Settings.LinksToSkip = InLinksToSkip;
		DiagnosticsAST = MakeShareable(new FRigVMParserAST({this}, nullptr, Settings));
	}
	return DiagnosticsAST;
}

TSharedPtr<FRigVMParserAST> URigVMGraph::GetRuntimeAST(const FRigVMParserASTSettings& InSettings, bool bForceRefresh)
{
	if (RuntimeAST == nullptr || bForceRefresh)
	{
		RuntimeAST = MakeShareable(new FRigVMParserAST({this}, nullptr, InSettings));
	}
	return RuntimeAST;
}

void URigVMGraph::ClearAST(bool bClearDiagnostics, bool bClearRuntime)
{
	if (bClearDiagnostics)
	{
		DiagnosticsAST.Reset();
	}
	if (bClearRuntime)
	{
		RuntimeAST.Reset();
	}
}

uint32 URigVMGraph::GetStructureHash() const
{
	uint32 Hash = GetTypeHash(GetName());
	for(const URigVMNode* Node : Nodes)
	{
		const uint32 NodeHash = Node->GetStructureHash();
		Hash = HashCombine(Hash, NodeHash);
	}
	return Hash;
}

void URigVMGraph::PreSave(FObjectPreSaveContext SaveContext)
{
	UObject::PreSave(SaveContext);


	// save the structure hash along side this graph
	LastStructureHash = GetStructureHash();
}

bool URigVMGraph::AreSectionsEnabled() const
{
#if WITH_EDITOR
	if (const IRigVMClientHost* ClientHost = GetImplementingOuter<IRigVMClientHost>())
	{
		const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>(ClientHost->GetRigVMEditorSettingsClass());
		if (Settings && Settings->bHighlightSimilarNodes)
		{
			return true;
		}
	}
#endif
	return bSectionsEnabled;
}

bool URigVMGraph::IsNameAvailable(const FString& InName) const
{
	for (const URigVMNode* Node : Nodes)
	{
		if (Node->GetName() == InName)
		{
			return false;
		}
	}
	return true;
}

bool URigVMGraph::IsNameAvailable(FName InName) const
{
	for (const URigVMNode* Node : Nodes)
	{
		if (Node->GetFName() == InName)
		{
			return false;
		}
	}
	return true;
}

TSharedPtr<FScribbleGraphData> URigVMGraph::SetupScribbleGraph(bool bCreateIfMissing)
{
	if (!ScribbleGraph.IsValid())
	{
		if (bCreateIfMissing)
		{
			Modify();
			(void)ScribbleGraph.GetOrCreateDataPtr();
		}
		else
		{
			return nullptr;
		}
	}

	TSharedPtr<FScribbleGraphData> Graph = ScribbleGraph.GetDataPtr();
	if (!Graph.IsValid())
	{
		return nullptr;
	}

	Graph->OnModify().Clear();

	TWeakObjectPtr<URigVMGraph> WeakThis = this;
	Graph->OnModify().AddLambda([WeakThis]()
	{
		if (!WeakThis.IsValid())
		{
			return;
		}
		WeakThis->Modify();
	});

	Graph->OnGetCurrentAnchor().BindLambda([WeakThis]() -> FName
	{
		if (!WeakThis.IsValid())
		{
			return NAME_None;
		}
		const TArray<FName> SelectedNodes = WeakThis->GetSelectNodes();
		if (SelectedNodes.IsEmpty())
		{
			return NAME_None;
		}
		return SelectedNodes[0];
	});

	Graph->OnResolveAnchor().BindLambda([WeakThis](const FName& InNodeName) -> TOptional<FVector2f>
	{
		if (!WeakThis.IsValid())
		{
			return {};
		}
		const URigVMNode* Node = WeakThis->FindNodeByName(InNodeName);
		if (!Node)
		{
			return {};
		}
		return FVector2f(Node->GetPosition());
	});

	return Graph;
}

void URigVMGraph::PrepareCycleChecking(URigVMPin* InPin, bool bAsInput)
{
	TArray<URigVMLink*> LinksToSkip;
	if (InPin)
	{
		LinksToSkip = InPin->GetLinks();

		// since execute output pins also only allow one link we need to ignore links
		// on other output links. this is so that cycle checking doesn't kick in irregularly.
		if(InPin->IsExecuteContext() && InPin->GetDirection() == ERigVMPinDirection::Output)
		{
			for(const URigVMPin* OtherPin : InPin->GetNode()->GetPins())
			{
				if(OtherPin != InPin)
				{
					if(OtherPin->IsExecuteContext() && OtherPin->GetDirection() == ERigVMPinDirection::Output)
					{
						LinksToSkip.Append(OtherPin->GetLinks());
					}
				}
			}
		}
	}

	GetDiagnosticsAST(false, LinksToSkip)->PrepareCycleChecking(InPin);
}

namespace
{
	bool AreNodesLinked(const URigVMNode* InNodeToGetLinksFrom, const URigVMNode* InNodeToCheck, TSet<URigVMNode*>& OutNodes, const bool bSource)
	{
		const TArray<URigVMLink*> RigVMLinks = InNodeToGetLinksFrom->GetLinks();
		for (const URigVMLink* Link: RigVMLinks)
		{
			const URigVMPin* Pin = bSource ? Link->GetSourcePin() : Link->GetTargetPin();
			if (URigVMNode* Node = Pin ? Pin->GetNode() : nullptr)
			{
				if (Node != InNodeToGetLinksFrom && !OutNodes.Contains(Node))
				{
					if (Node == InNodeToCheck)
					{
						return true;
					}
					OutNodes.Add(Node);
					
					if (AreNodesLinked(Node, InNodeToCheck, OutNodes, bSource))
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	bool AreNodesGoingToCycle(const URigVMNode* InSourceNode, const URigVMNode* InTargetNode)
	{
		TSet<URigVMNode*> CheckedNodes;

		const bool bIsTargetAlreadyASource = AreNodesLinked(InSourceNode, InTargetNode, CheckedNodes, true);
		if (bIsTargetAlreadyASource)
		{
			return true;
		}

		CheckedNodes.Reset();
		const bool bIsSourceAlreadyATarget = AreNodesLinked(InTargetNode, InSourceNode, CheckedNodes, false);
		if(bIsSourceAlreadyATarget)
		{
			return true;
		}

		return false;
	}

	bool AreNodesAlreadyConnected(const URigVMNode* InSourceNode, const URigVMNode* InTargetNode)
	{
		TSet<URigVMNode*> CheckedNodes;

		const bool bIsTargetAlreadyATarget = AreNodesLinked(InSourceNode, InTargetNode, CheckedNodes, false);
		if (bIsTargetAlreadyATarget)
		{
			return true;
		}

		CheckedNodes.Reset();
		const bool bIsSourceAlreadyASource = AreNodesLinked(InTargetNode, InSourceNode, CheckedNodes, true);
		if (bIsSourceAlreadyASource)
		{
			return true;
		}

		return false;
	}
}

bool URigVMGraph::CanLink(URigVMPin* InSourcePin, URigVMPin* InTargetPin, FString* OutFailureReason, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection, bool bEnableTypeCasting)
{
	if (!URigVMPin::CanLink(InSourcePin, InTargetPin, OutFailureReason, InByteCode, InUserLinkDirection, false, bEnableTypeCasting))
	{
		return false;
	}

	const URigVMNode* SourceNode = InSourcePin->GetNode();
	const URigVMNode* TargetNode = InTargetPin->GetNode();


	// check for cycling nodes dependencies before going into the AST
	if (AreNodesGoingToCycle(SourceNode, TargetNode))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Cycles are not allowed.");
		}
		return false;
	}

	// Break/Continue Loop nodes are only meaningful inside a loop body. Reject any
	// link into such a node unless the source pin would place the target inside a
	// loop. Lives on the graph-level check (rather than URigVMPin::CanLink) so that
	// paste/restore — which reattaches links one at a time and bypasses graph-level
	// validation via bIsRestoringLinks/bSuspendASTLinkChecks in URigVMController::AddLink
	// — is not affected by transient ordering. The compiler's FindEnclosingSlicedLoopExpr
	// is the authoritative backstop on the fully-resolved AST.
	if (const URigVMUnitNode* TargetUnitNode = Cast<URigVMUnitNode>(TargetNode))
	{
		const UScriptStruct* TargetStruct = TargetUnitNode->GetScriptStruct();
		const bool bTargetRequiresLoopScope = TargetStruct &&
			(TargetStruct->IsChildOf(FRigVMFunction_BreakFromLoop::StaticStruct()) ||
			 TargetStruct->IsChildOf(FRigVMFunction_ContinueLoop::StaticStruct()));

		if (bTargetRequiresLoopScope)
		{
			const bool bSourceIsCompletedExec =
				InSourcePin->IsExecuteContext() &&
				InSourcePin->GetFName() == FRigVMStruct::ForLoopCompletedPinName;

			const bool bSourceIsLoopNonCompletedPin =
				SourceNode->IsLoopNode() && !bSourceIsCompletedExec;

			const bool bSourceIsInsideLoop =
				bSourceIsLoopNonCompletedPin || SourceNode->IsWithinLoop();

			if (!bSourceIsInsideLoop)
			{
				if (OutFailureReason)
				{
					const TCHAR* TargetTypeName =
						TargetStruct->IsChildOf(FRigVMFunction_BreakFromLoop::StaticStruct())
							? TEXT("Break From Loop")
							: TEXT("Continue Loop");
					*OutFailureReason = FString::Printf(
						TEXT("%s can only be connected from inside a loop body."),
						TargetTypeName);
				}
				return false;
			}
		}
	}

	// full check needed
	return GetDiagnosticsAST()->CanLink(InSourcePin, InTargetPin, OutFailureReason);
}


