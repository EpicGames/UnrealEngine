// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMGraphSection.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMCommentNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "RigVMModule.h"
#include "BoneControllers/BoneControllerTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMGraphSection)

//UE_DISABLE_OPTIMIZATION

URigVMPin* FRigVMGraphSectionLink::FindSourcePinSkippingReroutes(const URigVMLink* InLink)
{
	check(InLink);

	URigVMPin* SourcePin = InLink->GetSourcePin();
	if (SourcePin == nullptr)
	{
		return nullptr;
	}

	// traverse the reroute and find the source node
	while (SourcePin->GetNode()->IsA<URigVMRerouteNode>())
	{
		TArray<URigVMPin*> SourcePins = SourcePin->GetLinkedSourcePins();
		if (SourcePins.Num() == 0)
		{
			SourcePin = nullptr;
			break;
		}
		SourcePin = SourcePins[0];
	}

	return SourcePin; // can be nullptr
}

TArray<URigVMPin*> FRigVMGraphSectionLink::FindTargetPinsSkippingReroutes(const URigVMLink* InLink)
{
	TArray<URigVMPin*> AllTargetPins = {InLink->GetTargetPin()};
	TArray<URigVMPin*> FilteredTargetPins;

	for (int32 TargetPinIndex = 0; TargetPinIndex < AllTargetPins.Num(); ++TargetPinIndex)
	{
		URigVMPin* TargetPin = AllTargetPins[TargetPinIndex];
		if (TargetPin == nullptr)
		{
			continue;
		}
		
		if (TargetPin->GetNode()->IsA<URigVMRerouteNode>())
		{
			AllTargetPins.Append(TargetPin->GetLinkedTargetPins());
			continue;
		}

		FilteredTargetPins.Add(TargetPin);
	}

	return FilteredTargetPins;
}

TArray<FRigVMGraphSectionLink::FPinTuple> FRigVMGraphSectionLink::FindLinksSkippingReroutes(const URigVMNode* InNode)
{
	check(InNode);
	const TArray<URigVMLink*> AllLinks = InNode->GetLinks();
	TArray<FPinTuple> FilteredLinks;
	FilteredLinks.Reserve(AllLinks.Num());

	for (const URigVMLink* Link : AllLinks)
	{
		const URigVMPin* SourcePin = FindSourcePinSkippingReroutes(Link);
		if (SourcePin == nullptr)
		{
			continue;
		}
				
		const TArray<URigVMPin*> TargetPins = FindTargetPinsSkippingReroutes(Link);
		for (const URigVMPin* TargetPin : TargetPins)
		{
			if (TargetPin == nullptr)
			{
				continue;
			}
			FilteredLinks.Emplace(SourcePin, TargetPin);
		}
	}

	return FilteredLinks;
}

const URigVMNode* FRigVMGraphSectionLink::FindSourceNode(const URigVMNode* InTargetNode) const
{
	check(InTargetNode);

	const URigVMPin* TargetPin = InTargetNode->FindPin(TargetPinPath);
	if (TargetPin == nullptr)
	{
		return nullptr;
	}

	const TArray<URigVMLink*>& Links = TargetPin->GetLinks();
	for (const URigVMLink* Link : Links)
	{
		const URigVMPin* SourcePin = FindSourcePinSkippingReroutes(Link);
		if (SourcePin == nullptr)
		{
			continue;
		}
		
		if (SourcePin->GetSegmentPath(true) != SourcePinPath)
		{
			continue;
		}

		const URigVMNode* SourceNode = SourcePin->GetNode();
		if (SourceNode == nullptr)
		{
			continue;
		}

		if (SourceNodeHash != FRigVMGraphSection::GetNodeHash(SourceNode))
		{
			continue;
		}

		return SourceNode;
	}

	return nullptr;
}

TArray<const URigVMNode*> FRigVMGraphSectionLink::FindTargetNodes(const URigVMNode* InSourceNode) const
{
	check(InSourceNode);

	TArray<const URigVMNode*> TargetNodes;

	const URigVMPin* SourcePin = InSourceNode->FindPin(SourcePinPath);
	if (SourcePin == nullptr)
	{
		return TargetNodes;
	}

	const TArray<URigVMLink*>& Links = SourcePin->GetLinks();
	for (const URigVMLink* Link : Links)
	{
		const TArray<URigVMPin*> TargetPins = FindTargetPinsSkippingReroutes(Link);
		for (const URigVMPin* TargetPin : TargetPins)
		{
			if (TargetPin->GetSegmentPath(true) != TargetPinPath)
			{
				continue;
			}

			const URigVMNode* TargetNode = TargetPin->GetNode();
			if (TargetNode == nullptr)
			{
				continue;
			}

			if (TargetNodeHash != FRigVMGraphSection::GetNodeHash(TargetNode))
			{
				continue;
			}

			TargetNodes.Add(TargetNode);
		}
	}

	return TargetNodes;
}

FRigVMGraphSection::FRigVMGraphSection()
: Hash(UINT32_MAX)
{
}

FRigVMGraphSection::FRigVMGraphSection(const TArray<URigVMNode*>& InNodes)
: Hash(UINT32_MAX)
{
	if (InNodes.Contains(nullptr))
	{
		return;
	}

	TArray<URigVMNode*> SortedNodes;
	SortedNodes.Append(InNodes);

	SortedNodes.RemoveAll([](const URigVMNode* Node) -> bool
	{
		return !IsValidNode(Node);
	});

	if(SortedNodes.IsEmpty())
	{
		return;
	}

	const URigVMGraph* Graph = SortedNodes[0]->GetGraph();

	for(const URigVMNode* Node : SortedNodes)
	{
		if(Node->GetGraph() != Graph)
		{
			UE_LOGF(LogRigVM, Error, "Provided nodes are not on the same graph.");
			Reset();
			return;
		}
	}

	// find all links within the section.
	TArray<bool> bNodeHasOutputLink;
	bNodeHasOutputLink.AddZeroed(SortedNodes.Num());

	TArray<TArray<FRigVMGraphSectionLink::FPinTuple>> PerNodeInputLinks;
	PerNodeInputLinks.SetNum(SortedNodes.Num());

	for(int32 TargetNodeIndex = 0; TargetNodeIndex < SortedNodes.Num(); TargetNodeIndex++)
	{
		const URigVMNode* TargetNode = SortedNodes[TargetNodeIndex];

		// links are sorted from first pin to last pin
		const TArray<FRigVMGraphSectionLink::FPinTuple> AllLinks = FRigVMGraphSectionLink::FindLinksSkippingReroutes(TargetNode);
		PerNodeInputLinks[TargetNodeIndex].Reserve(AllLinks.Num());
		
		for (const FRigVMGraphSectionLink::FPinTuple& Link : AllLinks)
		{
			const URigVMPin* SourcePin = Link.Get<0>();
			check(SourcePin);

			const int32 SourceNodeIndex = SortedNodes.Find(SourcePin->GetNode());
			if (SourceNodeIndex == INDEX_NONE)
			{
				continue;
			}

			const URigVMPin* TargetPin = Link.Get<1>();
			check(TargetPin);

			if (TargetPin->GetNode() == TargetNode)
			{
				PerNodeInputLinks[TargetNodeIndex].Emplace(SourcePin, TargetPin);
			}
			else
			{
				URigVMNode* OtherTargetNode = TargetPin->GetNode();
				const int32 OtherTargetNodeIndex = SortedNodes.Find(OtherTargetNode);
				if  (OtherTargetNodeIndex != INDEX_NONE)
				{
					bNodeHasOutputLink[SourceNodeIndex] = true;
				}
			}
		}
	}

	// determine the leaf nodes
	LeafNodes.Reserve(SortedNodes.Num());
	for(int32 NodeIndex = 0; NodeIndex < SortedNodes.Num(); NodeIndex++)
	{
		LeafNodes.Add(NodeIndex);
	}
	LeafNodes.RemoveAll([bNodeHasOutputLink](const int32& NodeIndex) -> bool
	{
		return bNodeHasOutputLink[NodeIndex];
	});
	
	if (LeafNodes.IsEmpty())
	{
		Reset();
		return;
	}

	TMap<int32, int32> NodeTraversalByInt32;
	struct Local
	{
		static void VisitNode(
			const TArray<URigVMNode*>& InNodes,
			int32 NodeIndex,
			int32& TraversalIndex,
			const TArray<TArray<FRigVMGraphSectionLink::FPinTuple>>& InPerNodeInputLinks,
			TMap<int32, int32>& NodeTraversalIndex)
		{
			if(NodeTraversalIndex.Contains(NodeIndex))
			{
				return;
			}
			NodeTraversalIndex.Add(NodeIndex, TraversalIndex++);

			const TArray<FRigVMGraphSectionLink::FPinTuple>& LinksForNode = InPerNodeInputLinks[NodeIndex];
			for(const FRigVMGraphSectionLink::FPinTuple& LinkForNode : LinksForNode)
			{
				const URigVMPin* SourcePin = LinkForNode.Get<0>();
				if (SourcePin == nullptr)
				{
					continue;
				}
				
				URigVMNode* SourceNode = SourcePin->GetNode();
				const int32 SourceNodeIndex = InNodes.Find(SourceNode);
				if (SourceNodeIndex != INDEX_NONE)
				{
					VisitNode(InNodes, SourceNodeIndex, TraversalIndex, InPerNodeInputLinks, NodeTraversalIndex);
				}
			}
		}
	};

	// visit all leaf nodes
	int32 TraversalIndex = 0;
	TArray<FName> LeafNodeNames;
	for(const int32& LeafNodeIndex : LeafNodes)
	{
		LeafNodeNames.Add(SortedNodes[LeafNodeIndex]->GetFName());
		Local::VisitNode(SortedNodes, LeafNodeIndex, TraversalIndex, PerNodeInputLinks, NodeTraversalByInt32);
	}

	// convert the map from int32 to object ptr	
	TMap<const URigVMNode*, int32> NodeTraversalByPtr;
	for(int32 NodeIndex = 0; NodeIndex < SortedNodes.Num(); NodeIndex++)
	{
		NodeTraversalByPtr.Add(SortedNodes[NodeIndex], NodeTraversalByInt32.FindChecked(NodeIndex)); 
	}

	// sort the nodes
	Algo::Sort(SortedNodes, [NodeTraversalByPtr](const URigVMNode* InNodeA, const URigVMNode* InNodeB) -> bool
	{
		return NodeTraversalByPtr.FindChecked(InNodeA) < NodeTraversalByPtr.FindChecked(InNodeB);  
	});

	Nodes.Reserve(SortedNodes.Num());
	NodeHashes.Reserve(SortedNodes.Num());
	
	for(URigVMNode* Node : SortedNodes)
	{
		Nodes.Add(Node->GetFName());
		NodeHashes.Add(GetNodeHash(Node));
	}

	for(URigVMNode* Node : SortedNodes)
	{
		const TArray<FRigVMGraphSectionLink::FPinTuple> PerNodeLinks = FRigVMGraphSectionLink::FindLinksSkippingReroutes(Node);
		for (const FRigVMGraphSectionLink::FPinTuple& Link : PerNodeLinks)
		{
			const URigVMPin* SourcePin = Link.Get<0>();
			const URigVMPin* TargetPin = Link.Get<1>();
			if(SourcePin == nullptr || TargetPin == nullptr)
			{
				continue;
			}

			URigVMNode* SourceNode = SourcePin->GetNode();
			URigVMNode* TargetNode = TargetPin->GetNode();

			if(TargetNode == Node)
			{
				if(SortedNodes.Contains(SourceNode))
				{
					FRigVMGraphSectionLink LinkInfo;
					LinkInfo.SourceNodeIndex = SortedNodes.Find(SourceNode);
					LinkInfo.TargetNodeIndex = SortedNodes.Find(TargetNode);
					LinkInfo.SourceNodeHash = NodeHashes[LinkInfo.SourceNodeIndex];
					LinkInfo.TargetNodeHash = NodeHashes[LinkInfo.TargetNodeIndex];
					LinkInfo.SourcePinPath = SourcePin->GetSegmentPath(true);
					LinkInfo.TargetPinPath = TargetPin->GetSegmentPath(true);

					Links.Add(LinkInfo);
				}
			}
		}
	}

	LeafNodes.Reset();
	Algo::Transform(LeafNodeNames, LeafNodes, [this](const FName& LeafNodeName) -> int32
	{
		return Nodes.Find(LeafNodeName);
	});

	Hash = 0;
	for(uint32 NodeHash : NodeHashes)
	{
		Hash = HashCombine(Hash, NodeHash);
	}
	for (const FRigVMGraphSectionLink& Link : Links)
	{
		Hash = HashCombine(Hash, GetLinkHash(Link));
	}
}

TArray<FRigVMGraphSection> FRigVMGraphSection::GetSectionsFromSelection(const URigVMGraph* InGraph)
{
	TArray<FRigVMGraphSection> Sections;
	TArray<URigVMNode*> AllSelectedNodes;
	Algo::Transform(InGraph->GetSelectNodes(), AllSelectedNodes, [InGraph](const FName& NodeName) -> URigVMNode*
	{
		return InGraph->FindNodeByName(NodeName);
	});

	AllSelectedNodes.Remove(nullptr);
	if(AllSelectedNodes.Num() == 0)
	{
		return Sections;
	}

	const TArray<TArray<URigVMNode*>> Islands = FRigVMGraphSection::GetNodeIslands(AllSelectedNodes);
	for (const TArray<URigVMNode*>& Island : Islands)
	{
		Sections.Emplace(Island);
	}
	
	return Sections;
}

bool FRigVMGraphSection::IsValid() const
{
	return Hash != UINT32_MAX && !Nodes.IsEmpty();
}

bool FRigVMGraphSection::IsValidNode(const URigVMNode* InNode)
{
	if (InNode)
	{
		return !InNode->IsA<URigVMCommentNode>() &&
			!InNode->IsA<URigVMRerouteNode>();
	}
	return true;
}

void FRigVMGraphSection::Reset()
{
	Hash = UINT32_MAX;
	Nodes.Reset();
	NodeHashes.Reset();
	LeafNodes.Reset();
	Links.Reset();
}

uint32 FRigVMGraphSection::GetNodeHash(const URigVMNode* InNode)
{
	check(InNode);

	uint32 Hash = GetTypeHash(InNode->GetClass());
	
	if(const URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		const FRigVMGraphFunctionIdentifier Identifier = FunctionRefNode->GetFunctionIdentifier();
		if(Identifier.IsValid())
		{
			Hash = HashCombine(Hash, GetTypeHash(Identifier));
		}
	}
	else if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InNode))
	{
		if(const FRigVMFunction* Function = TemplateNode->GetResolvedFunction())
		{
			Hash = HashCombine(Hash, GetTypeHash(Function->Name));
		}
		else if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
		{
			Hash = HashCombine(Hash, GetTypeHash(Template->GetNotation()));
			
			const FRigVMTemplateTypeMap Types = TemplateNode->GetTemplatePinTypeMap();
			for(const TPair<FName, TRigVMTypeIndex>& Pair : Types)
			{
				Hash = HashCombine(Hash, GetTypeHash(Pair.Key));
				Hash = HashCombine(Hash, GetTypeHash((int32)Pair.Value));
			}
		}
	}
	else if(const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode))
	{
		Hash = HashCombine(Hash, GetTypeHash(VariableNode->GetVariableName()));
		Hash = HashCombine(Hash, GetTypeHash(VariableNode->IsGetter()));
	}

	return Hash;
}

uint32 FRigVMGraphSection::GetLinkHash(const FRigVMGraphSectionLink& InLink)
{
	uint32 Hash = HashCombine(GetTypeHash(InLink.SourceNodeIndex), GetTypeHash(InLink.TargetNodeIndex));
	Hash = HashCombine(Hash, GetTypeHash(InLink.SourcePinPath));
	Hash = HashCombine(Hash, GetTypeHash(InLink.TargetPinPath));
	return Hash;
}

TArray<FRigVMGraphSection> FRigVMGraphSection::FindMatches(const URigVMGraph* InGraph) const
{
	return FindMatches(InGraph->GetNodes());
}

TArray<FRigVMGraphSection> FRigVMGraphSection::FindMatches(const TArray<URigVMNode*>& InAvailableNodes) const
{
	TArray<URigVMNode*> AvailableNodes = InAvailableNodes;
	TArray<FRigVMGraphSection> Matches;
	TArray<URigVMNode*> MatchedNodes;

	bool bFoundMatch = true;
	while (bFoundMatch)
	{
		bFoundMatch = false;
		
		for (const URigVMNode* AvailableNode : AvailableNodes)
		{
			MatchedNodes.Reset();
			if (MatchesNode(AvailableNode, AvailableNodes, &MatchedNodes))
			{
				FRigVMGraphSection Match(MatchedNodes);
				if (Match.IsValid() && Match.Hash == Hash)
				{
					Matches.Add(Match);
					AvailableNodes.RemoveAll([&MatchedNodes]( const URigVMNode* Node) -> bool
					{
						return MatchedNodes.Contains(Node);
					});
					bFoundMatch = true;
					break;
				}
			}
		}
	}
	return Matches;
}

TArray<TArray<URigVMNode*>> FRigVMGraphSection::GetNodeIslands(const TArray<URigVMNode*>& InNodes)
{
	TArray<TArray<URigVMNode*>> Islands;
	
	// compute islands of connected nodes - each subset is going
	// to represent one island
	TArray<int32> IslandIndexPerNode;
	IslandIndexPerNode.AddZeroed(InNodes.Num());

	while (IslandIndexPerNode.Contains(0))
	{
		TArray<int32> NodeIndices = {IslandIndexPerNode.Find(0)};
		Islands.Add({});

		for (int32 Index = 0; Index < NodeIndices.Num(); Index++)
		{
			const int32& NodeIndex = NodeIndices[Index];
			URigVMNode* Node = InNodes[NodeIndex];

			Islands.Last().Add(Node);
			IslandIndexPerNode[NodeIndex] = Islands.Num();

			TArray<URigVMNode*> LinkedNodes = Node->GetLinkedSourceNodes();
			LinkedNodes.Append(Node->GetLinkedTargetNodes());

			for (URigVMNode* LinkedNode : LinkedNodes)
			{
				const int32 LinkedNodeIndex = InNodes.Find(LinkedNode);
				if (LinkedNodeIndex == INDEX_NONE)
				{
					continue;
				}

				if (IslandIndexPerNode[LinkedNodeIndex] == 0)
				{
					NodeIndices.AddUnique(LinkedNodeIndex);
				}
			}
		}
	}

	return Islands;
}

bool FRigVMGraphSection::ContainsNode(const FName& InNodeName) const
{
	return Nodes.Contains(InNodeName);
}

bool FRigVMGraphSection::MatchesNode(const URigVMNode* InNode, const TArray<URigVMNode*>& InAvailableNodes, TArray<URigVMNode*>* OutMatchingNodesForSet) const
{
	if (!IsValidNode(InNode))
	{
		return false;
	}
	
	const uint32 NodeHash = GetNodeHash(InNode);
	for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); NodeIndex++)
	{
		if (NodeHash != NodeHashes[NodeIndex])
		{
			continue;
		}
		
		TArray<URigVMNode*> VisitedNodes;
		VisitedNodes.AddZeroed(Nodes.Num());
		if (MatchesNode_Impl(InNode, InAvailableNodes, NodeIndex, VisitedNodes))
		{
			if (OutMatchingNodesForSet)
			{
				OutMatchingNodesForSet->Append(VisitedNodes);
			}
			return true;
		}
	}

	return false;
}

bool FRigVMGraphSection::MatchesNode_Impl(const URigVMNode* InNode, const TArray<URigVMNode*>& InAvailableNodes, int32 InNodeIndex, TArray<URigVMNode*>& VisitedNodes) const
{
	if (!InAvailableNodes.Contains(InNode))
	{
		return false;
	}
	if (VisitedNodes[InNodeIndex])
	{
		return true;
	}
	if (VisitedNodes.Contains(InNode))
	{
		return false;
	}
	VisitedNodes[InNodeIndex] = const_cast<URigVMNode*>(InNode);

	// traverse all right to left
	for (const FRigVMGraphSectionLink& Link : Links)
	{
		if (Link.TargetNodeIndex == InNodeIndex)
		{
			if (VisitedNodes[Link.SourceNodeIndex])
			{
				continue;
			}

			if (const URigVMNode* LinkedNode = Link.FindSourceNode(InNode))
			{
				if (!MatchesNode_Impl(LinkedNode, InAvailableNodes, Link.SourceNodeIndex, VisitedNodes))
				{
					VisitedNodes[InNodeIndex] = nullptr;
					return false;
				}
			}
			else
			{
				VisitedNodes[InNodeIndex] = nullptr;
				return false;
			}
		}
	}

	// also traverse all links from left to right
	for (const FRigVMGraphSectionLink& Link : Links)
	{
		if (Link.SourceNodeIndex == InNodeIndex)
		{
			if (VisitedNodes[Link.TargetNodeIndex])
			{
				continue;
			}
			
			TArray<const URigVMNode*> LinkedNodes = Link.FindTargetNodes(InNode);
			if (LinkedNodes.Num() == 0)
			{
				VisitedNodes[InNodeIndex] = nullptr;
				return false;
			}

			if (LinkedNodes.Num() == 1)
			{
				if (!MatchesNode_Impl(LinkedNodes[0], InAvailableNodes, Link.TargetNodeIndex, VisitedNodes))
				{
					VisitedNodes[InNodeIndex] = nullptr;
					return false;
				}
			}

			// if we have more than one match for the linked nodes - we'll need to see which
			// one may be a potential match. for that we'll need to create a copy of the visited
			// nodes array since it may be filled up to a point - just to realize that we hit
			// a mismatching node later.
			else
			{
				const TArray<URigVMNode*> VisitedNodesCopy = VisitedNodes;

				bool bFoundMatch = false;
				for (const URigVMNode* LinkedNode : LinkedNodes)
				{
					if (!MatchesNode_Impl(LinkedNode, InAvailableNodes, Link.TargetNodeIndex, VisitedNodes))
					{
						// roll back the visited nodes array
						VisitedNodes = VisitedNodesCopy;
					}
					else
					{
						bFoundMatch = true;
						break;
					}
				}

				if (!bFoundMatch)
				{
					VisitedNodes[InNodeIndex] = nullptr;
					return false;
				}
			}
		}
	}

	return true;
}

//UE_ENABLE_OPTIMIZATION
