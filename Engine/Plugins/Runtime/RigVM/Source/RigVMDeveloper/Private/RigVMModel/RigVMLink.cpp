// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMLink.h"
#include "RigVMModel/RigVMGraph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMLink)

URigVMGraph* URigVMLink::GetGraph() const
{
	return Cast<URigVMGraph>(GetOuter());
}

int32 URigVMLink::GetGraphDepth() const
{
	return GetGraph()->GetGraphDepth();
}

void URigVMLink::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		Ar << SourcePinPath;
		Ar << TargetPinPath;
	}
	else
	{
		FString SourcePinPathTemp;
		if (SourcePin)
		{
			if (SourcePin->GetNode())
			{
				SourcePinPathTemp = SourcePin->GetPinPath();
			}
		}
		FString TargetPinPathTemp;
		if (TargetPin)
		{
			if (TargetPin->GetNode())
			{
				TargetPinPathTemp = TargetPin->GetPinPath();
			}
		}
		Ar << SourcePinPathTemp;
		Ar << TargetPinPathTemp;
	}
}

int32 URigVMLink::GetLinkIndex() const
{
	if (const URigVMGraph* Graph = GetGraph())
	{
		return Graph->GetLinks().Find((URigVMLink*)this);
	}
	return INDEX_NONE;
}

URigVMPin* URigVMLink::GetSourcePin() const
{
	if(SourcePin)
	{
		if(SourcePin->GetOutermost() == GetTransientPackage())
		{
			if(GetOutermost() != GetTransientPackage())
			{
				SourcePin = nullptr;
			}
		}
	}
	if (SourcePin == nullptr)
	{
		UpdatePinPointers();
	}
	return SourcePin;
}

URigVMPin* URigVMLink::GetTargetPin() const
{
	if(TargetPin)
	{
		if(TargetPin->GetOutermost() == GetTransientPackage())
		{
			if(GetOutermost() != GetTransientPackage())
			{
				TargetPin = nullptr;
			}
		}
	}
	if (TargetPin == nullptr)
	{
		UpdatePinPointers();
	}
	return TargetPin;
}

URigVMNode* URigVMLink::GetSourceNode() const
{
	if(const URigVMPin* CurrentSourcePin = GetSourcePin())
	{
		return CurrentSourcePin->GetNode();
	}
	return nullptr;
}

URigVMNode* URigVMLink::GetTargetNode() const
{
	if(const URigVMPin* CurrentTargetPin = GetTargetPin())
	{
		return CurrentTargetPin->GetNode();
	}
	return nullptr;
}

FString URigVMLink::GetSourcePinPath() const
{
	if(SourcePin)
	{
		return SourcePin->GetPinPath();
	}
	return SourcePinPath;
}

FString URigVMLink::GetTargetPinPath() const
{
	if(TargetPin)
	{
		return TargetPin->GetPinPath();
	}
	return TargetPinPath;
}

bool URigVMLink::SetSourcePinPath(const FString& InPinPath)
{
	URigVMLink* MutableThis = (URigVMLink*)this;
	MutableThis->Detach();
	SourcePinPath = InPinPath;
	MutableThis->Attach();
	return true;
}

bool URigVMLink::SetTargetPinPath(const FString& InPinPath)
{
	URigVMLink* MutableThis = (URigVMLink*)this;
	MutableThis->Detach();
	TargetPinPath = InPinPath;
	MutableThis->Attach();
	return true;
}

bool URigVMLink::SetSourceAndTargetPinPaths(const FString& InSourcePinPath, const FString& InTargetPinPath)
{
	URigVMLink* MutableThis = (URigVMLink*)this;
	MutableThis->Detach();
	SourcePinPath = InSourcePinPath;
	TargetPinPath = InTargetPinPath;
	return MutableThis->Attach();
}

bool URigVMLink::IsAttached() const
{
	return SourcePin != nullptr && TargetPin != nullptr;
}

bool URigVMLink::Detach()
{
	UpdatePinPaths();
	SourcePin = TargetPin = nullptr;
	return !SourcePinPath.IsEmpty() && !TargetPinPath.IsEmpty();
}

bool URigVMLink::Attach(FString* OutFailureReason)
{
	if(IsAttached())
	{
		return false;
	}

	UpdatePinPointers();

	if(SourcePin == nullptr)
	{
		if(OutFailureReason)
		{
			static constexpr TCHAR Format[] = TEXT("Source pin '%s' cannot be found.");
			*OutFailureReason = FString::Printf(Format, *SourcePinPath);
		}

		SourcePin = TargetPin = nullptr;
		return false;
	}

	if(TargetPin == nullptr)
	{
		if(OutFailureReason)
		{
			static constexpr TCHAR Format[] = TEXT("Target pin '%s' cannot be found.");
			*OutFailureReason = FString::Printf(Format, *TargetPinPath);
		}

		SourcePin = TargetPin = nullptr;
		return false;
	}

	return true;
}

void URigVMLink::UpdatePinPaths()
{
	if(IsAttached())
	{
		if (const URigVMPin* CurrentSourcePin = GetSourcePin())
		{
			SourcePinPath = CurrentSourcePin->GetPinPath();
		}
		if (const URigVMPin* CurrenTargetPin = GetTargetPin())
		{
			TargetPinPath = CurrenTargetPin->GetPinPath();
		}
	}
}

void URigVMLink::UpdatePinPointers() const
{
	if (URigVMGraph* Graph = GetGraph())
	{
		SourcePin = Graph->FindPin(SourcePinPath);
		TargetPin = Graph->FindPin(TargetPinPath);
	}
}

URigVMPin* URigVMLink::GetOppositePin(const URigVMPin* InPin) const
{
	if (InPin == GetSourcePin())
	{
		return GetTargetPin();
	}
	else if (InPin == GetTargetPin())
	{
		return GetSourcePin();
	}
	return nullptr;
}

FString URigVMLink::GetPinPathRepresentation() const
{
	return GetPinPathRepresentation(GetSourcePinPath(), GetTargetPinPath());
}

FString URigVMLink::GetPinPathRepresentation(const FString& InSourcePinPath, const FString& InTargetPinPath)
{
	static constexpr TCHAR Format[] = TEXT("%s -> %s"); 
	return FString::Printf(Format, *InSourcePinPath, *InTargetPinPath);
}

bool URigVMLink::SplitPinPathRepresentation(const FString& InString, FString& OutSource, FString& OutTarget)
{
	static constexpr TCHAR Format[] = TEXT(" -> "); 
	return InString.Split(Format, &OutSource, &OutTarget);
}
