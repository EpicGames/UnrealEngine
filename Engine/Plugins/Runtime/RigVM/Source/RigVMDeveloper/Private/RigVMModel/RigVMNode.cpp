// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMNode.h"
#include "RigVMStringUtils.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMModel/RigVMTraitDefaultValueStruct.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMUserWorkflowRegistry.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Algo/Count.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMNode)

#if WITH_EDITOR
TArray<int32> URigVMNode::EmptyInstructionArray;
#endif

URigVMNode::URigVMNode()
: UObject()
, Position(FVector2D::ZeroVector)
, Size(FVector2D::ZeroVector)
, NodeColor(FLinearColor::White)
, NodeColorType(ERigVMNodeColorType::FromMetadata)
, bHasEarlyExitMarker(false)
, bIsExcludedByEarlyExit(false)
, NodeVersion(0)
#if WITH_EDITOR
, ProfilingHash(0)
#endif
{
}

URigVMNode::~URigVMNode()
{
}

void URigVMNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	for (const FString& TraitRootPinName : TraitRootPinNames)
	{
		if (!TraitDefaultValues.Contains(TraitRootPinName))
		{
			if (URigVMPin* TraitPin = FindPin(TraitRootPinName))
			{
				UScriptStruct* TraitScriptStruct = TraitPin->GetScriptStruct();
				FRigVMTraitDefaultValueStruct& TraitDefaultValueStruct = TraitDefaultValues.Add(TraitRootPinName);
				TraitDefaultValueStruct.Init(TraitScriptStruct);
				TraitDefaultValueStruct.SetValue(TraitPin->DefaultValue);
			}
		}
	}
}

FString URigVMNode::GetNodePath(bool bRecursive) const
{
	if (bRecursive)
	{
		if(URigVMGraph* Graph = GetGraph())
		{
			const FString ParentNodePath = Graph->GetNodePath();
			if (!ParentNodePath.IsEmpty())
			{
				return JoinNodePath(ParentNodePath, GetName());
			}
		}
	}
	return GetName();
}

bool URigVMNode::SplitNodePathAtStart(const FString& InNodePath, FString& LeftMost, FString& Right)
{
	return RigVMStringUtils::SplitNodePathAtStart(InNodePath, LeftMost, Right);
}

bool URigVMNode::SplitNodePathAtEnd(const FString& InNodePath, FString& Left, FString& RightMost)
{
	return RigVMStringUtils::SplitNodePathAtEnd(InNodePath, Left, RightMost);
}

bool URigVMNode::SplitNodePath(const FString& InNodePath, TArray<FString>& Parts)
{
	return RigVMStringUtils::SplitNodePath(InNodePath, Parts);
}

FString URigVMNode::JoinNodePath(const FString& Left, const FString& Right)
{
	return RigVMStringUtils::JoinNodePath(Left, Right);
}

FString URigVMNode::JoinNodePath(const TArray<FString>& InParts)
{
	return RigVMStringUtils::JoinNodePath(InParts);
}

int32 URigVMNode::GetNodeIndex() const
{
	int32 Index = INDEX_NONE;
	URigVMGraph* Graph = GetGraph();
	if (Graph != nullptr)
	{
		Graph->GetNodes().Find((URigVMNode*)this, Index);
	}
	return Index;
}

const TArray<URigVMPin*>& URigVMNode::GetPins() const
{
	return Pins;
}

TArray<URigVMPin*> URigVMNode::GetAllPinsRecursively() const
{
	struct Local
	{
		static void VisitPinRecursively(URigVMPin* InPin, TArray<URigVMPin*>& OutPins)
		{
			OutPins.Add(InPin);
			for (URigVMPin* SubPin : InPin->GetSubPins())
			{
				VisitPinRecursively(SubPin, OutPins);
			}
		}
	};

	TArray<URigVMPin*> Result;
	for (URigVMPin* Pin : GetPins())
	{
		Local::VisitPinRecursively(Pin, Result);
	}
	return Result;
}

TArray<FString> URigVMNode::GetPinCategories() const
{
	return PinCategories;
}

TArray<FString> URigVMNode::GetSubPinCategories(const FString InCategory, bool bOnlyExisting, bool bRecursive) const
{
	if(InCategory.IsEmpty())
	{
		return {};
	}
	
	const TArray<FString> ExistingCategories = GetPinCategories();
	const FString Prefix = InCategory + TEXT("|");

	const TArray<FString> IncompleteSubCategories = ExistingCategories.FilterByPredicate([Prefix](const FString& ExistingCategory)
	{
		return ExistingCategory.StartsWith(Prefix, ESearchCase::CaseSensitive);
	});

	TArray<FString> SubCategories;
	for(const FString& SubCategory : IncompleteSubCategories)
	{
		TArray<FString> Parts;
		verify(RigVMStringUtils::SplitNodePath(SubCategory, Parts));

		TArray<FString> ParentsOfSubCategory;
		while(!Parts.IsEmpty())
		{
			const FString ParentCategory = RigVMStringUtils::JoinNodePath(Parts);
			if(!ParentCategory.StartsWith(Prefix))
			{
				break;
			}
			ParentsOfSubCategory.Add(ParentCategory);
			Parts.Pop();
		}

		for(int32 Index = ParentsOfSubCategory.Num() - 1; Index >= 0; Index--)
		{
			SubCategories.AddUnique(ParentsOfSubCategory[Index]);
		}
	}

	if(!bRecursive)
	{
		// remove any category that is not a direct child of the input category
		SubCategories.RemoveAll([Prefix](const FString& InCategory) -> bool
		{
			return InCategory.Mid(Prefix.Len()).Contains(TEXT("|"));
		});
	}

	if(bOnlyExisting)
	{
		SubCategories.RemoveAll([&ExistingCategories](const FString& InCategory) -> bool
		{
			return !ExistingCategories.Contains(InCategory);
		});
	}

	return SubCategories;
}

FString URigVMNode::GetPinCategoryName(const FString InCategory) const
{
	if(InCategory.IsEmpty())
	{
		return FString();
	}
	FString ParentCategory, CategoryName;
	if(RigVMStringUtils::SplitNodePathAtEnd(InCategory, ParentCategory, CategoryName))
	{
		return CategoryName;
	}
	return FString();
}

FString URigVMNode::GetParentPinCategory(const FString InCategory, bool bOnlyExisting) const
{
	if(InCategory.IsEmpty())
	{
		return FString();
	}
	FString ParentCategory, CategoryName;
	if(RigVMStringUtils::SplitNodePathAtEnd(InCategory, ParentCategory, CategoryName))
	{
		return ParentCategory;
	}
	return FString();
}

TArray<FString> URigVMNode::GetParentPinCategories(const FString InCategory, bool bOnlyExisting, bool bIncludeSelf) const
{
	if(InCategory.IsEmpty())
	{
		return {};
	}
	
	const TArray<FString> ExistingCategories = GetPinCategories();

	TArray<FString> Parts;
	verify(RigVMStringUtils::SplitNodePath(InCategory, Parts));

	TArray<FString> ParentCategories;
	while(!Parts.IsEmpty())
	{
		ParentCategories.Add(RigVMStringUtils::JoinNodePath(Parts));
		Parts.Pop();
	}

	if(!bIncludeSelf)
	{
		ParentCategories.Remove(InCategory);
	}

	if(bOnlyExisting)
	{
		ParentCategories.RemoveAll([&ExistingCategories](const FString& InCategory) -> bool
		{
			return !ExistingCategories.Contains(InCategory);
		});
	}

	return ParentCategories;
}

int32 URigVMNode::GetPinCategoryDepth(const FString& InCategory)
{
	TArray<FString> Parts;
	if(RigVMStringUtils::SplitNodePath(InCategory, Parts))
	{
		return Parts.Num() - 1;
	}
	return 0;
}

TArray<URigVMPin*> URigVMNode::GetPinsForCategory(FString InCategory) const
{
	InCategory.TrimStartAndEndInline();
	if(InCategory.IsEmpty())
	{
		return {};
	}
	
	const TArray<URigVMPin*> AllPins = GetAllPinsRecursively();

	TArray<URigVMPin*> PinsInCategory;
	for(URigVMPin* Pin : AllPins)
	{
		if(Pin->GetCategory().Equals(InCategory))
		{
			PinsInCategory.Add(Pin);
		}
	}

	Algo::SortBy(PinsInCategory, [](const URigVMPin* Pin) -> int32
	{
		return Pin->GetIndexInCategory();
	});
	
	return PinsInCategory;
}

bool URigVMNode::IsPinCategoryExpanded(FString InCategory) const
{
	if(InCategory.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase))
	{
		return true;
	}
	if(const bool* ExpansionState = PinCategoryExpansion.Find(InCategory))
	{
		return *ExpansionState;
	}
	return false;
}

FRigVMNodeLayout URigVMNode::GetNodeLayout(bool bIncludeEmptyCategories) const
{
	FRigVMNodeLayout Layout;
	
	// fill in the pin categories based on the data stored on the pins themselves
	const TArray<URigVMPin*> AllPins = GetAllPinsRecursively();
	TMap<FString, FRigVMPinCategory> CategoryMap;
	for(const URigVMPin* Pin : AllPins)
	{
		if(!Pin->UserDefinedCategory.IsEmpty())
		{
			FRigVMPinCategory& Category = CategoryMap.FindOrAdd(Pin->UserDefinedCategory);
			Category.Path = Pin->UserDefinedCategory;
			Category.Elements.Add(Pin->GetSegmentPath(true));
		}
	}

	// add the categories in the order they have been added
	for(const FString& PinCategory : PinCategories)
	{
		if(const FRigVMPinCategory* Category = CategoryMap.Find(PinCategory))
		{
			FRigVMPinCategory CategoryCopy = *Category;

			// sort the elements based on pin index
			// we start by assuming indices above the user defined range,
			// so say for 4 pins we'll use (4,5,6,7) and then inline the
			// user provided pin indices within the range of 0 to 3.
			TMap<FString,int32> PinPathToIndex;
			for(const FString& PinPath : CategoryCopy.Elements)
			{
				const int32 Index = CategoryCopy.Elements.Num() + PinPathToIndex.Num();
				PinPathToIndex.Add(PinPath, Index);
			}
			for(const FString& PinPath : CategoryCopy.Elements)
			{
				if(const URigVMPin* Pin = FindPin(PinPath))
				{
					const int32 Index = Pin->GetIndexInCategory();
					if(CategoryCopy.Elements.IsValidIndex(Index))
					{
						PinPathToIndex.FindChecked(PinPath) = Index;
					}
				}
			}

			Algo::SortBy(CategoryCopy.Elements, [PinPathToIndex](const FString& PinPath) -> int32
			{
 				return PinPathToIndex.FindChecked(PinPath);
			});
			
			Layout.Categories.Add(CategoryCopy);
		}
		else if(bIncludeEmptyCategories)
		{
			FRigVMPinCategory EmptyCategory;
			EmptyCategory.Path = PinCategory;
			Layout.Categories.Add(EmptyCategory);
		}
	}

	for(FRigVMPinCategory& Category : Layout.Categories)
	{
		Category.bExpandedByDefault = IsPinCategoryExpanded(Category.Path);
	}

	// fill in all user provided display names and pin category indices
	for(const URigVMPin* Pin : AllPins)
	{
		if(!Pin->GetCategory().IsEmpty() && Pin->GetIndexInCategory() != INDEX_NONE)
		{
			const FString SegmentPath = Pin->GetSegmentPath(true);
			Layout.PinIndexInCategory.Add(SegmentPath, Pin->GetIndexInCategory());
		}

		if(!Pin->DisplayName.IsNone())
		{
			if(!Pin->DisplayName.IsEqual(GetDisplayNameForPin(Pin), ENameCase::CaseSensitive))
			{
				const FString SegmentPath = Pin->GetSegmentPath(true);
				Layout.DisplayNames.Add(SegmentPath, Pin->DisplayName.ToString());
			}
		}
	}

	return Layout;
}

FString URigVMNode::GetOriginalPinDefaultValue(const URigVMPin* InPin) const
{
	const FString CompleteSegmentPath = InPin->GetSegmentPath(true);
	if(const FString* CachedOriginalPinDefaultValue = CachedOriginalPinDefaultValues.Find(CompleteSegmentPath))
	{
		return *CachedOriginalPinDefaultValue;
	}
	
	const URigVMPin* RootPin = InPin->GetRootPin();
	const FString OriginalDefaultValue = GetOriginalDefaultValueForRootPin(RootPin);
	if((RootPin != InPin) && !OriginalDefaultValue.IsEmpty())
	{
		struct Local
		{
			static FString TraverseArrayElement(TMap<FString, FString>& Cache, const URigVMPin* InPin, const FString& InSegmentPath, const FString& InRemainingSegmentPath, const FString& InDefaultValue)
			{
				FString Left = InRemainingSegmentPath, Right;
				(void)URigVMPin::SplitPinPathAtStart(InRemainingSegmentPath, Left, Right);

				if(const URigVMPin* SubPin = InPin->FindSubPin(Left))
				{
					const TArray<FString> DefaultValues = URigVMPin::SplitDefaultValue(InDefaultValue);
					if(DefaultValues.IsValidIndex(SubPin->GetPinIndex()))
					{
						const FString SubPinDefaultValue = DefaultValues[SubPin->GetPinIndex()];
						return Traverse(Cache, SubPin, URigVMPin::JoinPinPath(InSegmentPath, Left), Right, SubPinDefaultValue);
					}
				}
				return FString();
			}

			static FString TraverseStructMember(TMap<FString, FString>& Cache, const URigVMPin* InPin, const FString& InSegmentPath, const FString& InRemainingSegmentPath, const FString& InDefaultValue)
			{
				FString Left = InRemainingSegmentPath, Right;
				(void)URigVMPin::SplitPinPathAtStart(InRemainingSegmentPath, Left, Right);

				const TArray<FString> DefaultValues = URigVMPin::SplitDefaultValue(InDefaultValue);
				for(const FString& DefaultValue : DefaultValues)
				{
					FString Name, Value;
					if (DefaultValue.Split(TEXT("="), &Name, &Value))
					{
						if(Left.Equals(Name, ESearchCase::CaseSensitive))
						{
							if(const URigVMPin* SubPin = InPin->FindSubPin(Left))
							{
								return Traverse(Cache, SubPin, URigVMPin::JoinPinPath(InSegmentPath, Left), Right, Value);
							}
						}
					}
				}

				return FString();
			}

			static FString Traverse(TMap<FString, FString>& Cache, const URigVMPin* InPin, const FString& InSegmentPath, const FString& InRemainingSegmentPath, const FString& InDefaultValue)
			{
				FString DefaultValue = InDefaultValue;
				if(!InRemainingSegmentPath.IsEmpty())
				{
					if(InPin->IsArray())
					{
						DefaultValue = TraverseArrayElement(Cache, InPin, InSegmentPath, InRemainingSegmentPath, InDefaultValue);
					}
					else if(InPin->IsStruct())
					{
						DefaultValue = TraverseStructMember(Cache, InPin, InSegmentPath, InRemainingSegmentPath, InDefaultValue);
					}
				}

				if(!InDefaultValue.IsEmpty())
				{
					Cache.FindOrAdd(InSegmentPath, InDefaultValue);
				}
				return DefaultValue;
			}
		};

		const FString SegmentPath = InPin->GetSegmentPath(false);
		return Local::Traverse(CachedOriginalPinDefaultValues, RootPin, RootPin->GetName(), SegmentPath, OriginalDefaultValue);
	}

	if(!OriginalDefaultValue.IsEmpty())
	{
		CachedOriginalPinDefaultValues.FindOrAdd(CompleteSegmentPath, OriginalDefaultValue);
	}
	return OriginalDefaultValue;
}

ERigVMNodeDefaultValueOverrideState::Type URigVMNode::GetPinDefaultValueOverrideState() const
{
	int32 NumPinsWithDefaultValue = 0;
	int32 NumPinsWithDefaultValueOverride = 0;
	if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		for(const URigVMPin* Pin : Pins)
		{
			if(Pin->CanProvideDefaultValue())
			{
				NumPinsWithDefaultValue++;
				if(Pin->HasDefaultValueOverride())
				{
					NumPinsWithDefaultValueOverride++;
				}
			}
		}
	}
	if(NumPinsWithDefaultValueOverride == 0)
	{
		return ERigVMNodeDefaultValueOverrideState::None;
	}
	if(NumPinsWithDefaultValueOverride < NumPinsWithDefaultValue)
	{
		return ERigVMNodeDefaultValueOverrideState::SomePins;
	}
	return ERigVMNodeDefaultValueOverrideState::AllPins;
}

URigVMPin* URigVMNode::FindPin(const FString& InPinPath) const
{
	FStringView Left, Right;
	if (!URigVMPin::SplitPinPathAtStart(InPinPath, Left, Right))
	{
		Left = InPinPath;
	}

	for (URigVMPin* Pin : GetPins())
	{
		if (Pin == nullptr)
		{
			continue;
		}
		if (Pin->NameEquals(Left, true))
		{
			if (Right.IsEmpty())
			{
				return Pin;
			}
			return Pin->FindSubPinView(Right);
		}
	}

	if(Left.StartsWith(URigVMPin::OrphanPinPrefix))
	{
		for (URigVMPin* Pin : OrphanedPins)
		{
			if (Pin == nullptr)
			{
				continue;
			}
			if (Pin->GetName() == InPinPath)
			{
				return Pin;
			}
			if (Pin->GetName() == Left)
			{
				if (Right.IsEmpty())
				{
					return Pin;
				}
				return Pin->FindSubPinView(Right);
			}
		}
	}

	if(Right.IsEmpty())
	{
		static const FString ExecuteContextNameString = FRigVMStruct::ExecuteContextName.ToString();
		static const FString ExecutePinNameString = FRigVMStruct::ExecutePinName.ToString();
		if(Left.Equals(ExecuteContextNameString, ESearchCase::IgnoreCase) || 
			Left.Equals(ExecutePinNameString, ESearchCase::IgnoreCase))
		{
			return FindExecutePin();
		}
	}
	
	return nullptr;
}

URigVMPin* URigVMNode::FindRootPinByName(const FName& InPinName) const
{
	for(TObjectPtr<URigVMPin> Pin : Pins)
	{
		if(Pin->GetFName().IsEqual(InPinName, ENameCase::CaseSensitive))
		{
			return Pin;
		}
	}
	for(TObjectPtr<URigVMPin> OrphanedPin : OrphanedPins)
	{
		if(OrphanedPin->GetFName().IsEqual(InPinName, ENameCase::CaseSensitive))
		{
			return OrphanedPin;
		}
	}

	if(InPinName == FRigVMStruct::ExecuteContextName || 
		InPinName == FRigVMStruct::ExecutePinName)
	{
		return FindExecutePin();
	}
	return nullptr;
}

URigVMPin* URigVMNode::FindExecutePin() const
{
	for(TObjectPtr<URigVMPin> Pin : Pins)
	{
		if(Pin->IsExecuteContext())
		{
			return Pin;
		}
	}
	return nullptr;
}

const TArray<URigVMPin*>& URigVMNode::GetOrphanedPins() const
{
	return OrphanedPins;
}

URigVMGraph* URigVMNode::GetGraph() const
{
	if (URigVMGraph* Graph = Cast<URigVMGraph>(GetOuter()))
	{
		return Graph;
	}
	if (URigVMInjectionInfo* InjectionInfo = GetInjectionInfo())
	{
		return InjectionInfo->GetGraph();
	}
	return nullptr;
}

URigVMGraph* URigVMNode::GetRootGraph() const
{
	if (URigVMGraph* Graph = GetGraph())
	{
		return Graph->GetRootGraph();
	}
	return nullptr;
}

int32 URigVMNode::GetGraphDepth() const
{
	return GetGraph()->GetGraphDepth();
}

URigVMInjectionInfo* URigVMNode::GetInjectionInfo() const
{
	return Cast<URigVMInjectionInfo>(GetOuter());
}

FString URigVMNode::GetNodeTitle() const
{
	if (!NodeTitle.IsEmpty())
	{
		return NodeTitle;
	}
	return GetName();
}

const FString& URigVMNode::GetNodeTitleRaw() const
{
	return NodeTitle;
}

FString URigVMNode::GetNodeSubTitle() const
{
	return FString();
}

bool URigVMNode::SupportsRenaming() const
{
	return false;
}

FVector2D URigVMNode::GetPosition() const
{
	return Position;
}

FVector2D URigVMNode::GetSize() const
{
	return Size;
}

FLinearColor URigVMNode::GetNodeColor() const
{
	return NodeColor;
}

FSlateIcon URigVMNode::GetNodeIcon() const
{
	static FSlateIcon FunctionIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return FunctionIcon;
}

FText URigVMNode::GetToolTipText() const
{
	return FText::FromName(GetFName());
}

FText URigVMNode::GetToolTipTextForPin(const URigVMPin* InPin) const
{
	const FText NodePinToolTipText = FText::FromName(InPin->GetFName());
	return GetTypedToolTipText(InPin, NodePinToolTipText);
}

FString URigVMNode::GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const
{
	ensure(InRootPin->IsRootPin());
	return FString();
}

void URigVMNode::UpdateTraitRootPinNames()
{
	TArray<FString> NewTraitRootPinNames;
	for(URigVMPin* Pin : GetPins())
	{
		if(Pin->IsTraitPin())
		{
			if(URigVMPin* NamePin = Pin->FindSubPin(TEXT("Name")))
			{
				NamePin->DefaultValue = Pin->GetName();
			}
			NewTraitRootPinNames.Add(Pin->GetName());
		}
	}
	TraitRootPinNames = NewTraitRootPinNames;
}

void URigVMNode::IncrementVersion()
{
	NodeVersion++;
}

bool URigVMNode::IsSelected() const
{
	if (const URigVMGraph* Graph = GetGraph())
	{
		return Graph->IsNodeSelected(GetFName());
	}
	return false;
}

bool URigVMNode::IsHighlighted() const
{
	if (const URigVMGraph* Graph = GetGraph())
	{
		return Graph->IsNodeHighlighted(GetFName());
	}
	return false;
}

bool URigVMNode::IsInjected() const
{
	return Cast<URigVMInjectionInfo>(GetOuter()) != nullptr;
}

bool URigVMNode::IsVisibleInUI() const
{
	return !IsInjected();
}

bool URigVMNode::IsPure() const
{
	if(IsMutable())
	{
		return false;
	}

	for (URigVMPin* Pin : GetPins())
	{
		if(Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			return false;
		}
	}

	return true;
}

bool URigVMNode::IsMutable() const
{
	for (const URigVMPin* Pin : GetPins())
	{
		if(Pin->IsExecuteContext())
		{
			return true;
		}
	}
	return false;
}

bool URigVMNode::HasWildCardPin() const
{
	for (const URigVMPin* Pin : GetPins())
	{
		if (Pin->IsWildCard())
		{
			return true;
		}
	}
	return false;
}

bool URigVMNode::IsEvent() const
{
	return IsMutable() && !GetEventName().IsNone();
}

FName URigVMNode::GetEventName() const
{
	return NAME_None;
}

bool URigVMNode::CanOnlyExistOnce() const
{
	return false;
}

bool URigVMNode::HasInputPin(bool bIncludeIO) const
{
	if (HasPinOfDirection(ERigVMPinDirection::Input))
	{
		return true;
	}
	if (bIncludeIO)
	{
		return HasPinOfDirection(ERigVMPinDirection::IO);
	}
	return false;

}

bool URigVMNode::HasIOPin() const
{
	return HasPinOfDirection(ERigVMPinDirection::IO);
}

bool URigVMNode::HasLazyPin(bool bOnlyConsiderPinsWithLinks) const
{
	return Pins.ContainsByPredicate([bOnlyConsiderPinsWithLinks](const URigVMPin* Pin) -> bool
	{
		if(Pin->IsLazy())
		{
			if(bOnlyConsiderPinsWithLinks)
			{
				return Pin->GetLinkedSourcePins(true).Num() > 0;
			}
			return true;
		}
		return false;
	});
}

bool URigVMNode::HasOutputPin(bool bIncludeIO) const
{
	if (HasPinOfDirection(ERigVMPinDirection::Output))
	{
		return true;
	}
	if (bIncludeIO)
	{
		return HasPinOfDirection(ERigVMPinDirection::IO);
	}
	return false;
}

bool URigVMNode::HasPinOfDirection(ERigVMPinDirection InDirection) const
{
	for (URigVMPin* Pin : GetPins())
	{
		if (Pin->GetDirection() == InDirection)
		{
			return true;
		}
	}
	return false;
}

bool URigVMNode::IsLinkedTo(URigVMNode* InNode) const
{
	if (InNode == nullptr)
	{
		return false;
	}
	if (InNode == this)
	{
		return false;
	}
	if (GetGraph() != InNode->GetGraph())
	{
		return false;
	}
	for (URigVMPin* Pin : GetPins())
	{
		if (IsLinkedToRecursive(Pin, InNode))
		{
			return true;
		}
	}
	return false;
}

uint32 URigVMNode::GetStructureHash() const
{
	uint32 Hash = GetTypeHash(GetName());
	for(const URigVMPin* Pin : Pins)
	{
		const uint32 PinHash = Pin->GetStructureHash();
		Hash = HashCombine(Hash, PinHash);
	}
	return Hash;
}

TArray<URigVMPin*> URigVMNode::GetTraitPins() const
{
	TArray<URigVMPin*> TraitPins;
	TraitPins.Reserve(TraitRootPinNames.Num());
	
	for(const FString& TraitRootPinName : TraitRootPinNames)
	{
		URigVMPin* TraitPin = FindPin(TraitRootPinName);
		check(TraitPin);

		TraitPins.Add(TraitPin);
	}

	return TraitPins;
}

bool URigVMNode::IsTraitPin(FName InName) const
{
	if(const URigVMPin* Pin = FindPin(InName.ToString()))
	{
		return IsTraitPin(Pin);
	}
	return false;
}

bool URigVMNode::IsTraitPin(const URigVMPin* InTraitPin) const
{
	return FindTrait(InTraitPin) != nullptr;
}

URigVMPin* URigVMNode::FindTrait(const FName& InName, const FString& InSubPinPath) const
{
	const FString NameString = InName.ToString();
	for(const FString& TraitRootPinName : TraitRootPinNames)
	{
		if(TraitRootPinName.Equals(NameString, ESearchCase::CaseSensitive))
		{
			if(InSubPinPath.IsEmpty())
			{
				return FindPin(TraitRootPinName);
			}
			return FindPin(URigVMPin::JoinPinPath(TraitRootPinName, InSubPinPath));
		}
	}
	return nullptr;
}

URigVMPin* URigVMNode::FindTrait(const URigVMPin* InTraitPin) const
{
	if(InTraitPin)
	{
		const URigVMPin* RootPin = InTraitPin->GetRootPin();
		if(RootPin->GetNode() == this)
		{
			return FindTrait(RootPin->GetFName());
		}
	}
	return nullptr;
}

TSharedPtr<FStructOnScope> URigVMNode::GetTraitInstance(const FName& InName, bool bUseDefaultValueFromPin) const
{
	return GetTraitInstance(FindPin(InName.ToString()), bUseDefaultValueFromPin);
}

TSharedPtr<FStructOnScope> URigVMNode::GetTraitInstance(const URigVMPin* InTraitPin, bool bUseDefaultValueFromPin) const
{
	if(const URigVMPin* RootPin = FindTrait(InTraitPin))
	{
		check(RootPin->IsStruct());

		UScriptStruct* ScriptStruct = RootPin->GetScriptStruct();
		check(ScriptStruct->IsChildOf(FRigVMTrait::StaticStruct()));

		TSharedPtr<FStructOnScope> Scope(new FStructOnScope(ScriptStruct));
		FRigVMTrait* Trait = (FRigVMTrait*)Scope->GetStructMemory();

		if(bUseDefaultValueFromPin)
		{
			const FString DefaultValue = RootPin->GetDefaultValue();
			if(!DefaultValue.IsEmpty())
			{
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
				{
					// force logging to the error pipe for error detection
					LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity());
					ScriptStruct->ImportText(*DefaultValue, Trait, nullptr, PPF_SerializedAsImportText, &ErrorPipe, ScriptStruct->GetName());
				}
			}
		}

		Trait->Name = RootPin->GetName();
		
		return Scope;
	}

	static const TSharedPtr<FStructOnScope> EmptyScope;
	return EmptyScope;
}

UScriptStruct* URigVMNode::GetTraitScriptStruct(const FName& InName) const
{
	return GetTraitScriptStruct(FindPin(InName.ToString()));
}

UScriptStruct* URigVMNode::GetTraitScriptStruct(const URigVMPin* InTraitPin) const
{
	if(const URigVMPin* RootPin = FindTrait(InTraitPin))
	{
		check(RootPin->IsStruct());

		UScriptStruct* ScriptStruct = RootPin->GetScriptStruct();
		check(ScriptStruct->IsChildOf(FRigVMTrait::StaticStruct()));
		return ScriptStruct;
	}

	return nullptr;
}

FName URigVMNode::GetDisplayNameForPin(const FString& InPinPath) const
{
	if(const URigVMPin* Pin = FindPin(InPinPath))
	{
		return GetDisplayNameForPin(Pin);
	}
	return NAME_None;
}

FName URigVMNode::GetDisplayNameForPin(const URigVMPin* InPin) const
{
	check(InPin);
	if(InPin->IsArrayElement())
	{
		return *FString::FromInt(InPin->GetPinIndex());
	}
	if(InPin->IsExecuteContext())
	{
		if(InPin->GetDirection() == ERigVMPinDirection::IO)
		{
			return FRigVMStruct::ExecuteName;
		}
		const int32 NumExecutePins = static_cast<int32>(Algo::CountIf(GetPins(), [](const URigVMPin* Pin) -> bool
		{
			return Pin->IsExecuteContext();
		}));
		if(NumExecutePins == 1)
		{
			return FRigVMStruct::ExecuteName;
		}
	}
	return GetDisplayNameForStructMember(InPin);
}

FName URigVMNode::GetDisplayNameForStructMember(const URigVMPin* InPin)
{
#if WITH_EDITORONLY_DATA
	if(InPin)
	{
		if(const URigVMPin* ParentPin = InPin->GetParentPin())
		{
			if(const UStruct* Struct = ParentPin->GetScriptStruct())
			{
				if(const FProperty* Property = Struct->FindPropertyByName(InPin->GetFName()))
				{
					const FText DisplayNameText = Property->GetDisplayNameText();
					if(!DisplayNameText.IsEmpty())
					{
						return *DisplayNameText.ToString();
					}
				}
				else
				{
					// Possible the pin was programmatically generated from the trait's shared struct
					TSharedPtr<FStructOnScope> TraitScope = InPin->GetTraitInstance();
					if(TraitScope.IsValid())
					{
						const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
						Struct = VMTrait->GetTraitSharedDataStruct();
						Property = Struct != nullptr ? Struct->FindPropertyByName(InPin->GetFName()) : nullptr;
						if(Property)
						{
							const FText DisplayNameText = Property->GetDisplayNameText();
							if(!DisplayNameText.IsEmpty())
							{
								return *DisplayNameText.ToString();
							}
						}
					}
				}
			}
		}
	}
#endif
	return NAME_None;
}

FName URigVMNode::GetDisplayNameForStructMember(const UStruct* InStruct, const FString& InPath)
{
	check(InStruct);
	if(!InPath.IsEmpty())
	{
		FString Left, Right;
		if(!RigVMStringUtils::SplitPinPathAtStart(InPath, Left, Right))
		{
			Left = InPath;
		}

		if(const FProperty* Property = InStruct->FindPropertyByName(*Left))
		{
			return GetDisplayNameForProperty(Property, Right);
		}
	}
	return NAME_None;
}

FName URigVMNode::GetDisplayNameForProperty(const FProperty* InProperty, const FString& InRemainingPath)
{
	check(InProperty);

	FText DisplayNameText = InProperty->GetDisplayNameText();

	if(!InRemainingPath.IsEmpty())
	{
		const FRigVMPropertyPath PropertyPath(InProperty, InRemainingPath);
		if(PropertyPath.IsValid())
		{
			if(const FProperty* TailProperty = PropertyPath.GetTailProperty())
			{
				DisplayNameText = TailProperty->GetDisplayNameText();
			}
		}
	}

	if(DisplayNameText.IsEmpty())
	{
		return NAME_None;
	}
	return *DisplayNameText.ToString();
}

FString URigVMNode::GetCategoryForPin(const FString& InPinPath) const
{
	return FString();
}

int32 URigVMNode::GetIndexInCategoryForPin(const FString& InPinPath) const
{
	return INDEX_NONE;
}

FText URigVMNode::GetTypedToolTipText(const URigVMPin* InPin, const FText& ToolTipBody) const
{
	FText PinTypeText = FText::GetEmpty();

	if (UObject* PinTypeObject = InPin->GetCPPTypeObject())
	{
		if (const UField* Field = Cast<const UField>(PinTypeObject))
		{
			PinTypeText = Field->GetDisplayNameText();
		}
		else
		{
			PinTypeText = FText::FromName(PinTypeObject->GetFName());
		}
	}
	else
	{
		PinTypeText = FText::FromString(InPin->GetCPPType());
	}

	if (!ToolTipBody.IsEmptyOrWhitespace())
	{
		return FText::Format(INVTEXT("{0}\n{1}"), ToolTipBody, PinTypeText);
	}
	else
	{
		return PinTypeText;
	}
}

URigVMLibraryNode* URigVMNode::FindFunctionForNode() const  
{
	const UObject* Subject = this;
	while (Subject->GetOuter() && !Subject->GetOuter()->IsA<URigVMFunctionLibrary>())
	{
		Subject = Subject->GetOuter();
		if(Subject == nullptr)
		{
			return nullptr;
		}
	}

	return const_cast<URigVMLibraryNode*>(Cast<URigVMLibraryNode>(Subject));
}

bool URigVMNode::IsLinkedToRecursive(URigVMPin* InPin, URigVMNode* InNode) const
{
	for (URigVMPin* LinkedPin : InPin->GetLinkedSourcePins())
	{
		if (LinkedPin->GetNode() == InNode)
		{
			return true;
		}
	}
	for (URigVMPin* LinkedPin : InPin->GetLinkedTargetPins())
	{
		if (LinkedPin->GetNode() == InNode)
		{
			return true;
		}
	}
	for (URigVMPin* SubPin : InPin->GetSubPins())
	{
		if (IsLinkedToRecursive(SubPin, InNode))
		{
			return true;
		}
	}
	return false;
}

TArray<URigVMLink*> URigVMNode::GetLinks() const
{
	TArray<URigVMLink*> Links;

	struct Local
	{
		static void Traverse(URigVMPin* InPin, TArray<URigVMLink*>& Links)
		{
			Links.Append(InPin->GetLinks());
			for (URigVMPin* SubPin : InPin->GetSubPins())
			{
				Local::Traverse(SubPin, Links);
			}
		}
	};

	for (URigVMPin* Pin : GetPins())
	{
		Local::Traverse(Pin, Links);
	}

	return Links;
}

TArray<URigVMNode*> URigVMNode::GetLinkedSourceNodes() const
{
	TArray<URigVMNode*> Nodes;
	for (URigVMPin* Pin : GetPins())
	{
		GetLinkedNodesRecursive(Pin, true, Nodes);
	}
	return Nodes;
}

TArray<URigVMNode*> URigVMNode::GetLinkedTargetNodes() const
{
	TArray<URigVMNode*> Nodes;
	for (URigVMPin* Pin : GetPins())
	{
		GetLinkedNodesRecursive(Pin, false, Nodes);
	}
	return Nodes;
}

void URigVMNode::GetLinkedNodesRecursive(URigVMPin* InPin, bool bLookForSources, TArray<URigVMNode*>& OutNodes) const
{
	TArray<URigVMPin*> LinkedPins = bLookForSources ? InPin->GetLinkedSourcePins() : InPin->GetLinkedTargetPins();
	for (URigVMPin* LinkedPin : LinkedPins)
	{
		OutNodes.AddUnique(LinkedPin->GetNode());
	}
	for (URigVMPin* SubPin : InPin->GetSubPins())
	{
		GetLinkedNodesRecursive(SubPin, bLookForSources, OutNodes);
	}
}

void URigVMNode::InvalidateCache()
{
	CachedOriginalPinDefaultValues.Reset();
	IncrementVersion();
}

const TArray<int32>& URigVMNode::GetInstructionsForVM(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
	if(const FProfilingCache* Cache = UpdateProfilingCacheIfNeeded(Context, InVM, InProxy))
	{
		return Cache->Instructions;
	}
	return EmptyInstructionArray;
}

TArray<int32> URigVMNode::GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
	TArray<int32> Instructions;

#if WITH_EDITOR

	if(InVM == nullptr)
	{
		return Instructions;
	}
	
	if(InProxy.IsValid())
	{
		const FRigVMASTProxy Proxy = InProxy.GetChild((UObject*)this);
		return InVM->GetByteCode().GetAllInstructionIndicesForCallstack(Proxy.GetCallstack().GetStack());
	}
	else
	{
		return InVM->GetByteCode().GetAllInstructionIndicesForSubject((URigVMNode*)this);
	}
#else
	return Instructions;
#endif
}

int32 URigVMNode::GetInstructionVisitedCount(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
#if WITH_EDITOR
	if(InVM)
	{
		if(const FProfilingCache* Cache = UpdateProfilingCacheIfNeeded(Context, InVM, InProxy))
		{
			return Cache->VisitedCount;
		}
	}
#endif
	return 0;
}

double URigVMNode::GetInstructionMicroSeconds(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
#if WITH_EDITOR
	if(InVM)
	{
		if(const FProfilingCache* Cache = UpdateProfilingCacheIfNeeded(Context, InVM, InProxy))
		{
			return Cache->MicroSeconds;
		}
	}
#endif
	return -1.0;
}

bool URigVMNode::IsLoopNode() const
{
	if(IsControlFlowNode())
	{
		const TArray<FName>& Blocks = GetControlFlowBlocks();
		if(Blocks.Num() == 2 && Blocks[1] == FRigVMStruct::ForLoopCompletedPinName)
		{
			// The body block is named "ExecuteContext" for FRigVMStruct-based loops
			// and "ExecutePin" for FRigVMStructMutable-based loops (e.g. ForLoopCount).
			return Blocks[0] == FRigVMStruct::ExecuteContextName || Blocks[0] == FRigVMStruct::ExecutePinName;
		}
	}

	return false;
}

bool URigVMNode::IsControlFlowNode() const
{
	return !GetControlFlowBlocks().IsEmpty();
}

const TArray<FName>& URigVMNode::GetControlFlowBlocks() const
{
	static const TArray<FName> EmptyArray;
	return EmptyArray;
}

const bool URigVMNode::IsControlFlowBlockSliced(const FName& InBlockName) const
{
	return false;
}

bool URigVMNode::IsWithinLoop() const
{
	TArray<URigVMNode*> SourceNodes;
	for(const URigVMPin* Pin : Pins)
	{
		const TArray<URigVMPin*> SourcePins = Pin->GetLinkedSourcePins(true);
		for(const URigVMPin* SourcePin : SourcePins)
		{
			if(SourcePin->GetNode()->IsLoopNode())
			{
				if(!SourcePin->IsExecuteContext() || SourcePin->GetFName() != FRigVMStruct::ForLoopCompletedPinName)
				{
					return true;
				}
			}
		}

		for (const URigVMPin* SourcePin : SourcePins)
		{
			SourceNodes.AddUnique(SourcePin->GetNode());
		}
	}

	for(const URigVMNode* SourceNode : SourceNodes)
	{
		if(SourceNode->IsWithinLoop())
		{
			return true;
		}
	}
	
	return false;
}

TArray<FRigVMUserWorkflow> URigVMNode::GetSupportedWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const
{
	if(InSubject == nullptr)
	{
		InSubject = this;
	}

	const UScriptStruct* Struct = nullptr;
	if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(this))
	{
		Struct = UnitNode->GetScriptStruct();
	}
	return URigVMUserWorkflowRegistry::Get()->GetWorkflows(InType, Struct, InSubject);
}

bool URigVMNode::HasEarlyExitMarker() const
{
	return bHasEarlyExitMarker;
}

void URigVMNode::SetHasEarlyExitMarker(const bool bValue)
{
	if (bHasEarlyExitMarker == bValue)
	{
		return;
	}
	bHasEarlyExitMarker = bValue;

	URigVMGraph* Graph = GetGraph();
	if (Graph == nullptr)
	{
		return;
	}

	Graph->Notify(ERigVMGraphNotifType::NodeEarlyExitChanged, this);
}

bool URigVMNode::IsExcludedByEarlyExit() const
{
	return bIsExcludedByEarlyExit;
}

void URigVMNode::SetIsExcludedByEarlyExit(bool bIsExcluded)
{
	bIsExcludedByEarlyExit = bIsExcluded;
}

bool URigVMNode::IsAggregate() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	const TArray<URigVMPin*> AggregateInputs = GetAggregateInputs();
	const TArray<URigVMPin*> AggregateOutputs = GetAggregateOutputs();

	if ((AggregateInputs.Num() == 2 && AggregateOutputs.Num() == 1) ||
		(AggregateInputs.Num() == 1 && AggregateOutputs.Num() == 2))
	{
		TArray<URigVMPin*> AggregateAll = AggregateInputs;
		AggregateAll.Append(AggregateOutputs);
		for (int32 i = 1; i < 3; ++i)
		{
			if (AggregateAll[0]->GetCPPType() != AggregateAll[i]->GetCPPType() ||
				AggregateAll[0]->GetCPPTypeObject() != AggregateAll[i]->GetCPPTypeObject())
			{
				return false;
			}
		}
		
		return true;
	}
#endif
	
	return false;
}

URigVMPin* URigVMNode::GetFirstAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	const TArray<URigVMPin*> Inputs = GetAggregateInputs();
	const TArray<URigVMPin*> Outputs = GetAggregateOutputs();
	if (Inputs.Num() == 2 && Outputs.Num() == 1)
	{
		return Inputs[0];
	}
	if (Inputs.Num() == 1 && Outputs.Num() == 2)
	{
		return Outputs[0];
	}
#endif
	return nullptr;
}

URigVMPin* URigVMNode::GetSecondAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	const TArray<URigVMPin*> Inputs = GetAggregateInputs();
	const TArray<URigVMPin*> Outputs = GetAggregateOutputs();
	if (Inputs.Num() == 2 && Outputs.Num() == 1)
	{
		return Inputs[1];
	}
	if (Inputs.Num() == 1 && Outputs.Num() == 2)
	{
		return Outputs[1];
	}
#endif
	return nullptr;
}

URigVMPin* URigVMNode::GetOppositeAggregatePin() const
{
#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	const TArray<URigVMPin*> Inputs = GetAggregateInputs();
	const TArray<URigVMPin*> Outputs = GetAggregateOutputs();
	if (Inputs.Num() == 2 && Outputs.Num() == 1)
	{
		return Outputs[0];
	}
	if (Inputs.Num() == 1 && Outputs.Num() == 2)
	{
		return Inputs[0];
	}
#endif
	return nullptr;
}

bool URigVMNode::IsInputAggregate() const
{
	return GetAggregateInputs().Num() == 2;
}

#if WITH_EDITOR

const URigVMNode::FProfilingCache* URigVMNode::UpdateProfilingCacheIfNeeded(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const
{
	if(InVM == nullptr)
	{
		return nullptr;
	}
	
	const uint32 VMHash = HashCombine(GetTypeHash(InVM), GetTypeHash(Context.GetNumExecutions()));
	if(VMHash != ProfilingHash)
	{
		ProfilingCache.Reset();
	}
	ProfilingHash = VMHash;

	const uint32 ProxyHash = InProxy.IsValid() ? GetTypeHash(InProxy) : GetTypeHash(this);

	const TSharedPtr<FProfilingCache>* ExistingCache = ProfilingCache.Find(ProxyHash);
	if(ExistingCache)
	{
		return ExistingCache->Get();
	}

	TSharedPtr<FProfilingCache> Cache(new FProfilingCache);

	Cache->Instructions = GetInstructionsForVMImpl(Context, InVM, InProxy);
	Cache->VisitedCount = 0;
	Cache->MicroSeconds = -1.0;

	if(Cache->Instructions.Num() > 0)
	{
		for(const int32 Instruction : Cache->Instructions)
		{
			const int32 CountPerInstruction = InVM->GetInstructionVisitedCount(Context, Instruction);
			Cache->VisitedCount += CountPerInstruction;

			const double MicroSecondsPerInstruction = InVM->GetInstructionMicroSeconds(Context, Instruction);
			if(MicroSecondsPerInstruction >= 0.0)
			{
				if(Cache->MicroSeconds < 0.0)
				{
					Cache->MicroSeconds = MicroSecondsPerInstruction;
				}
				else
				{
					Cache->MicroSeconds += MicroSecondsPerInstruction;
				}
			}
		}
	}

	ProfilingCache.Add(ProxyHash, Cache);
	return Cache.Get();;
}

#endif
