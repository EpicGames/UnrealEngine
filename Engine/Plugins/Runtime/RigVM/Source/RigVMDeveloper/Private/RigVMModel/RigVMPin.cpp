// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMPin.h"

#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMLink.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMUnknownType.h"
#include "UObject/Package.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/PackageName.h"
#include "Misc/OutputDevice.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMFunctionEntryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReturnNode.h"
#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"
#include "RigVMModel/Nodes/RigVMSelectNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMStringUtils.h"
#include "RigVMDeveloperTypeUtils.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMPin)

TAutoConsoleVariable<bool> CVarRigVMEnablePinOverrides(TEXT("RigVM.EnablePinOverrides"), false, TEXT("enables the use of overrides for pin default values"));

#if WITH_EDITOR
#include "UObject/CoreRedirects.h"
#endif

URigVMGraph* URigVMInjectionInfo::GetGraph() const
{
	if(URigVMPin* Pin = GetPin())
	{
		return Pin->GetGraph();
	}
	return nullptr;
}

URigVMPin* URigVMInjectionInfo::GetPin() const
{
	return Cast<URigVMPin>(GetOuter());
}

URigVMInjectionInfo::FWeakInfo URigVMInjectionInfo::GetWeakInfo() const
{
	FWeakInfo Info;
	Info.bInjectedAsInput = bInjectedAsInput;
	Info.Node = Node;
#if WITH_EDITOR
	if (!Node)
	{
		Info.Node = UnitNode_DEPRECATED;
	}
#endif
	Info.InputPinName = InputPin != nullptr ? InputPin->GetFName() : NAME_None;
	Info.OutputPinName = OutputPin != nullptr ? OutputPin->GetFName() : NAME_None;
	return Info;
}

const URigVMPin::FPinOverrideMap URigVMPin::EmptyPinOverrideMap;
const URigVMPin::FPinOverride URigVMPin::EmptyPinOverride = URigVMPin::FPinOverride(FRigVMASTProxy(), EmptyPinOverrideMap);

void URigVMPin::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);
}

void URigVMPin::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::RigVMPinDefaultValueObjectPtr)
	{
		if (IsUObject() && DefaultValueObject == nullptr)
		{
			// Patch object default value - value should already be hashed (if not fully loaded) as it is also referenced via the edgraph node
			FSoftObjectPath SoftObjectPath(DefaultValue.TrimQuotes());
			DefaultValueObject = SoftObjectPath.ResolveObject();
			if (!SoftObjectPath.IsNull())
			{
				// Log if we couldnt resolve the object reference - this is usually fine as it means the reference didnt exist anyways
				UE_CLOGF(DefaultValueObject == nullptr, LogRigVMDeveloper, Display, "Could not resolve legacy text pin reference to '%ls'", *SoftObjectPath.ToString());
			}
			DefaultValue.Empty();
		}
	}
}

bool URigVMPin::SplitPinPathAtStart(const FString& InPinPath, FString& LeftMost, FString& Right)
{
	return RigVMStringUtils::SplitPinPathAtStart(InPinPath, LeftMost, Right);
}

bool URigVMPin::SplitPinPathAtStart(FStringView InPinPath, FStringView& LeftMost, FStringView& Right)
{
	return RigVMStringUtils::SplitPinPathAtStart(InPinPath, LeftMost, Right);
}

bool URigVMPin::SplitPinPathAtEnd(const FString& InPinPath, FString& Left, FString& RightMost)
{
	return RigVMStringUtils::SplitPinPathAtEnd(InPinPath, Left, RightMost);
}

bool URigVMPin::SplitPinPath(const FString& InPinPath, TArray<FString>& Parts)
{
	return RigVMStringUtils::SplitPinPath(InPinPath, Parts);
}

FString URigVMPin::JoinPinPath(const FString& Left, const FString& Right)
{
	return RigVMStringUtils::JoinPinPath(Left, Right);
}

FString URigVMPin::JoinPinPath(const TArray<FString>& InParts)
{
	return RigVMStringUtils::JoinPinPath(InParts);
}

TArray<FString> URigVMPin::SplitDefaultValue(const FString& InDefaultValue)
{
	return RigVMStringUtils::SplitDefaultValue(InDefaultValue);
}

FString URigVMPin::GetDefaultValueForArray(TConstArrayView<FString> DefaultValues)
{
	TStringBuilder<256> Builder;
	Builder << TCHAR('(');
	if (DefaultValues.Num())
	{
		Builder << DefaultValues[0];
		for (const FString& CurrentDefaultValue : DefaultValues.Slice(1, DefaultValues.Num() - 1))
		{
			Builder << TCHAR(',') << CurrentDefaultValue;
		}
	}
	Builder << TCHAR(')');
	return FString(Builder);
}

URigVMPin::URigVMPin()
	: Direction(ERigVMPinDirection::Invalid)
	, bIsExpanded(false)
	, bIsConstant(false)
	, bIsInputVariable(false)
	, bRequiresWatch(false)
	, bIsDynamicArray(false)
	, bIsLazy(false)
	, CPPType(FString())
	, CPPTypeObject(nullptr)
	, CPPTypeObjectPath(NAME_None)
	, DefaultValue(FString())
	, DefaultValueType(ERigVMPinDefaultValueType::AutoDetect)
	, CustomWidgetName(NAME_None)
	, IndexInCategory(INDEX_NONE)
	, BoundVariablePath_DEPRECATED()
	, PinVersion(0)
	, CombinedPinVersion(0)
	, LastKnownTypeIndex(INDEX_NONE)
	, CachedIsStringType(this)
	, CachedDefaultValue(this)
	, CachedAdaptedDefaultValue(this)
	, CachedCPPTypeObjectHash(this)
	, CachedShowInDetailsPanelOnly(this)
	, CachedPinPath(this)
	, CachedPinPathWithNodePath(this)
	, CachedPinCategory(this)
	, CachedDisplayName(this)
	, CachedDefaultValueOverride(this)
	, CachedHasOriginalDefaultValue(this)
{
}

bool URigVMPin::NameEquals(FStringView InName, bool bFollowCoreRedirectors) const
{
	if(InName.Equals(GetName(), ESearchCase::IgnoreCase))
	{
		return true;
	}
#if WITH_EDITOR
	if(bFollowCoreRedirectors)
	{
		UScriptStruct* Struct = nullptr;
		if(const URigVMPin* ParentPin = GetParentPin())
		{
			Struct = ParentPin->GetScriptStruct();
		}
		else if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(GetNode()))
		{
			Struct = UnitNode->GetScriptStruct();
		}

		if(Struct)
		{
			typedef TPair<FName, FString> FRedirectPinPair;
			const FRedirectPinPair Key(Struct->GetFName(), InName);
			static FRWLock RedirectedPinNamesLock;
			static TMap<FRedirectPinPair, FName> RedirectedPinNames;

			FWriteScopeLock ScopeLock(RedirectedPinNamesLock);
			if(const FName* RedirectedNamePtr = RedirectedPinNames.Find(Key))
			{
				if(RedirectedNamePtr->IsNone())
				{
					return false;
				}
				return NameEquals(*RedirectedNamePtr->ToString(), false);
			}

			const FCoreRedirectObjectName OldObjectName(FName(InName), Struct->GetFName(), *Struct->GetOutermost()->GetPathName());
			const FCoreRedirectObjectName NewObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Property, OldObjectName);
			if (OldObjectName != NewObjectName)
			{
				RedirectedPinNames.Add(Key, NewObjectName.ObjectName);
				
				const FString RedirectedName = NewObjectName.ObjectName.ToString();
				return NameEquals(RedirectedName, false);
			}

			RedirectedPinNames.Add(Key, NAME_None);
		}
	}
#endif
	return false;
}

FString URigVMPin::GetPinPath(bool bUseNodePath) const
{
	TRigVMModelCachedValue<URigVMPin, FString>& Cache = bUseNodePath ? CachedPinPathWithNodePath : CachedPinPath;
	if(Cache.IsValid())
	{
		return Cache.GetValue();
	}
	
	Cache.ResetCachedValue();
	
	if (const URigVMPin* ParentPin = GetParentPin())
	{
		Cache = JoinPinPath(ParentPin->GetPinPath(bUseNodePath), GetName());
	}
	else if (const URigVMNode* Node = GetNode())
	{
		Cache = JoinPinPath(Node->GetNodePath(bUseNodePath), GetName());
	}

	static const FString EmptyPinPath;
	return Cache.Get(EmptyPinPath);
}

FString URigVMPin::GetSubPinPath(const URigVMPin* InParentPin, bool bIncludeParentPinName) const
{
	if (const URigVMPin* ParentPin = GetParentPin())
	{
		if(ParentPin == InParentPin)
		{
			if(bIncludeParentPinName)
			{
				return JoinPinPath(ParentPin->GetName(),GetName());
			}
		}
		else
		{
			return JoinPinPath(ParentPin->GetSubPinPath(InParentPin, bIncludeParentPinName), GetName());
		}
	}
	return GetName();
}

FString URigVMPin::GetCategory() const
{
	if (UserDefinedCategory.IsEmpty())
	{
		if(CachedPinCategory.IsValid())
		{
			return CachedPinCategory.GetValue();
		}
		
		CachedPinCategory.ResetCachedValue();
		
		if(const URigVMNode* Node = GetNode())
		{
			const FString CategoryFromNode = Node->GetCategoryForPin(this->GetSegmentPath(true));
			if(!CategoryFromNode.IsEmpty())
			{
				CachedPinCategory = CategoryFromNode;
			}
		}

		static const FString EmptyCategory;
		return CachedPinCategory.Get(EmptyCategory);
	}
	return UserDefinedCategory;
}

int32 URigVMPin::GetIndexInCategory() const
{
	if(IndexInCategory == INDEX_NONE)
	{
		if(const URigVMNode* Node = GetNode())
		{
			int32 IndexFromNode = Node->GetIndexInCategoryForPin(this->GetSegmentPath(true));
			if(IndexFromNode != INDEX_NONE)
			{
				return IndexFromNode;
			}
		}
	}
	return IndexInCategory;
}

FString URigVMPin::GetSegmentPath(bool bIncludeRootPin) const
{
	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin)
	{
		FString ParentSegmentPath = ParentPin->GetSegmentPath(bIncludeRootPin);
		if (ParentSegmentPath.IsEmpty())
		{
			return GetName();
		}
		return JoinPinPath(ParentSegmentPath, GetName());
	}

	if(bIncludeRootPin)
	{
		return GetName();
	}
	
	return FString();
}

void URigVMPin::GetExposedPinChain(TArray<const URigVMPin*>& OutExposedPins) const
{
	TArray<const URigVMPin*> VisitedPins = {this};
	GetExposedPinChainImpl(OutExposedPins, VisitedPins);
}

void URigVMPin::GetExposedPinChainImpl(TArray<const URigVMPin*>& OutExposedPins, TArray<const URigVMPin*>& VisitedPins) const
{
	// Variable nodes do not share the operand with their source link
	if (GetNode()->IsA<URigVMVariableNode>() && GetDirection() == ERigVMPinDirection::Input)
	{
		OutExposedPins.Add(this);
		return;
	}
	
	// Find the first pin in the chain (source)
	for (URigVMLink* Link : GetSourceLinks())
	{
		URigVMPin* SourcePin = Link->GetSourcePin();
		check(SourcePin != nullptr);

		// Stop recursion when cycles are present
		if (VisitedPins.Contains(SourcePin))
		{
			return;
		}
		VisitedPins.Add(SourcePin);
		
		// If the source is on an entry node, add the pin and make a recursive call on the collapse node pin
		if (URigVMFunctionEntryNode* EntryNode = Cast<URigVMFunctionEntryNode>(SourcePin->GetNode()))
		{
			URigVMGraph* Graph = EntryNode->GetGraph();
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Graph->GetOuter()))
			{
				if(URigVMPin* CollapseNodePin = CollapseNode->FindPin(SourcePin->GetName()))
				{
					CollapseNodePin->GetExposedPinChainImpl(OutExposedPins, VisitedPins);
				}
			}
		}
		else if (URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(SourcePin->GetNode()))
		{
			URigVMGraph* Graph = ReturnNode->GetGraph();
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Graph->GetOuter()))
			{
				if(URigVMPin* CollapseNodePin = CollapseNode->FindPin(SourcePin->GetName()))
				{
					CollapseNodePin->GetExposedPinChainImpl(OutExposedPins, VisitedPins);
				}
			}
		}
		// Variable nodes do not share the operand with their source link
		else if (SourcePin->GetNode()->IsA<URigVMVariableNode>())
		{
			continue;
		}
		else
		{
			SourcePin->GetExposedPinChainImpl(OutExposedPins, VisitedPins);
		}

		return;
	}

	// Add the pins in the OutExposedPins array in depth-first order
	TSet<const URigVMPin*> FoundPins;
	TArray<const URigVMPin*> ToProcess;
	ToProcess.Push(this);
	while (!ToProcess.IsEmpty())
	{
		const URigVMPin* Current = ToProcess.Pop();
		if (FoundPins.Contains(Current))
		{
			continue;
		}
		FoundPins.Add(Current);
		OutExposedPins.Add(Current);

		// Add target pins connected to the current pin
		for (URigVMLink* Link : Current->GetTargetLinks())
		{
			if (URigVMPin* TargetPin = Link->GetTargetPin())
			{
				// Variable nodes do not share the operand with their source link
				if (TargetPin->GetNode()->IsA<URigVMVariableNode>())
				{
					continue;
				}
				ToProcess.Push(TargetPin);
			}
		}

		// If pin is on a collapse node, add entry pin
		if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Current->GetNode()))
		{
			URigVMFunctionEntryNode* EntryNode = CollapseNode->GetEntryNode();
			URigVMPin* EntryPin = EntryNode->FindPin(Current->GetName());
			if (EntryPin)
			{
				ToProcess.Push(EntryPin);
			}
		}
		// If pin is on a return node, add parent pin on collapse node
		else if (URigVMFunctionReturnNode* ReturnNode = Cast<URigVMFunctionReturnNode>(Current->GetNode()))
		{
			URigVMGraph* Graph = ReturnNode->GetGraph();
			if (URigVMCollapseNode* ParentNode = Cast<URigVMCollapseNode>(Graph->GetOuter()))
			{
				URigVMPin* CollapseNodePin = ParentNode->FindPin(Current->GetName());
				if(CollapseNodePin)
				{
					ToProcess.Push(CollapseNodePin);
				}
			}
		}
	}		
}

void URigVMPin::ClearPinPathCache() const
{
	for (URigVMPin* SubPin : GetSubPins())
	{
		SubPin->ClearPinPathCache();
	}
	CachedPinPath.ResetCachedValue();
	CachedPinPathWithNodePath.ResetCachedValue();
}

FName URigVMPin::GetDisplayName() const
{
	if(CachedDisplayName.IsValid())
	{
		return CachedDisplayName.GetValue();
	}

	if (DisplayName == NAME_None)
	{
		if(const URigVMNode* Node = GetNode())
		{
			const FName DisplayNameFromNode = Node->GetDisplayNameForPin(this);
			if(!DisplayNameFromNode.IsNone())
			{
				CachedDisplayName = DisplayNameFromNode;
				return CachedDisplayName.GetValue();
			}
		}

		const FName StructMemberDisplayName = URigVMNode::GetDisplayNameForStructMember(this);
		if(!StructMemberDisplayName.IsNone())
		{
			CachedDisplayName = StructMemberDisplayName;
			return CachedDisplayName.GetValue();
		}
		
		CachedDisplayName = GetFName();
		return CachedDisplayName.GetValue();
	}

	if (InjectionInfos.Num() > 0)
	{
		FString ProcessedDisplayName = DisplayName.ToString();

		for (URigVMInjectionInfo* Injection : InjectionInfos)
		{
			if (URigVMUnitNode* InjectedUnitNode = Cast<URigVMUnitNode>(Injection->Node))
			{
				if (TSharedPtr<FStructOnScope> DefaultStructScope = InjectedUnitNode->ConstructStructInstance())
				{
					FRigVMStruct* DefaultStruct = (FRigVMStruct*)DefaultStructScope->GetStructMemory();
					ProcessedDisplayName = DefaultStruct->ProcessPinLabelForInjection(ProcessedDisplayName);
				}
			}
		}

		CachedDisplayName = *ProcessedDisplayName;
		return CachedDisplayName.GetValue();
	}

	return DisplayName;
}

ERigVMPinDirection URigVMPin::GetDirection() const
{
	return Direction;
}

bool URigVMPin::IsExpanded() const
{
	if(!bIsExpanded)
	{
		if(ShouldOnlyShowSubPins())
		{
			return true;
		}
	}
	return bIsExpanded;
}

bool URigVMPin::IsDefinedAsConstant() const
{
	if (IsArrayElement())
	{
		return GetParentPin()->IsDefinedAsConstant();
	}
	return bIsConstant;
}

bool URigVMPin::IsDefinedAsInputVariable() const
{
	if (!IsRootPin())
	{
		return false;
	}
	if (IsDefinedAsConstant())
	{
		return false;
	}
	return bIsInputVariable;
}

FRigVMGraphVariableDescription URigVMPin::GetInputVariableDescription() const
{
	FRigVMGraphVariableDescription Description;
	if (IsDefinedAsInputVariable())
	{
		const URigVMPin* PinToQueryGuid = this;
		const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(GetNode());
		if (!CollapseNode)
		{
			if (URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(GetNode()))
			{
				CollapseNode = InterfaceNode->FindReferencedNode();
				PinToQueryGuid = InterfaceNode->FindReferencedPin(this);
			}
		}
		
		if (CollapseNode && PinToQueryGuid)
		{
			Description.Guid = CollapseNode->FindPinGuid(PinToQueryGuid);
		}
		
		Description.Name = GetFName();
		Description.CPPType = GetCPPType();
		Description.CPPTypeObject = GetCPPTypeObject();
		if (Description.CPPTypeObject)
		{
			Description.CPPTypeObjectPath = *Description.CPPTypeObject->GetPathName();
		}
	}
	return Description;
}

bool URigVMPin::RequiresWatch(const bool bCheckExposedPinChain) const
{
	if (!bRequiresWatch && bCheckExposedPinChain)
	{
		TArray<const URigVMPin*> VirtualPins;
		GetExposedPinChain(VirtualPins);
		
		for (const URigVMPin* VirtualPin : VirtualPins)
		{
			if (VirtualPin->bRequiresWatch)
			{
				return true;
			}
		}		
	}
	
	return bRequiresWatch;
}

bool URigVMPin::IsEnum() const
{
	if (IsArray())
	{
		return false;
	}
	return GetEnum() != nullptr;
}

bool URigVMPin::IsStruct() const
{
	if (IsArray())
	{
		return false;
	}
	return GetScriptStruct() != nullptr;
}

bool URigVMPin::IsStructMember() const
{
	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin == nullptr)
	{
		return false;
	}
	return ParentPin->IsStruct();
}

bool URigVMPin::IsUObject() const
{
	return RigVMTypeUtils::IsUObjectType(CPPType);
}

bool URigVMPin::IsInterface() const
{
	return RigVMTypeUtils::IsInterfaceType(CPPType);
}

bool URigVMPin::IsArray() const
{
	return RigVMTypeUtils::IsArrayType(CPPType);
}

bool URigVMPin::IsArrayElement() const
{
	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin == nullptr)
	{
		return false;
	}
	return ParentPin->IsArray();
}

bool URigVMPin::IsDynamicArray() const
{
	return bIsDynamicArray;
}

bool URigVMPin::IsLazy() const
{
	return bIsLazy;
}

int32 URigVMPin::GetPinIndex() const
{
	int32 Index = INDEX_NONE;

	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin != nullptr)
	{
		ParentPin->GetSubPins().Find((URigVMPin*)this, Index);
	}
	else
	{
		URigVMNode* Node = GetNode();
		if (Node != nullptr)
		{
			Node->GetPins().Find((URigVMPin*)this, Index);
		}
	}
	return Index;
}

int32 URigVMPin::GetAbsolutePinIndex() const
{
	return GetNode()->GetAllPinsRecursively().Find((URigVMPin*)this);
}

void URigVMPin::SetNameFromIndex()
{
	LowLevelRename(*FString::FormatAsNumber(GetPinIndex()));
}

void URigVMPin::SetDisplayName(const FName& InDisplayName)
{
	if(InDisplayName == GetFName())
	{
		DisplayName = NAME_None;
	}
	else
	{
		DisplayName = InDisplayName;
	}
}

void URigVMPin::IncrementVersion(bool bAffectParentPin, bool bAffectSubPins)
{
	PinVersion++;

	if(bAffectParentPin)
	{
		if(URigVMPin* ParentPin = GetParentPin())
		{
			ParentPin->IncrementVersion(true, false);
		}
	}
	if(bAffectSubPins)
	{
		for(URigVMPin* SubPin : SubPins)
		{
			SubPin->IncrementVersion(false, true);
		}
	}

	if(IsRootPin())
	{
		// pin changes may affect the event name on a node
		// or the highlight state
		if(URigVMNode* Node = GetNode())
		{
			Node->IncrementVersion();
		}
	}
}

int32 URigVMPin::GetArraySize() const
{
	return SubPins.Num();
}

FString URigVMPin::GetCPPType() const
{
	return RigVMTypeUtils::PostProcessCPPType(CPPType, GetCPPTypeObject());
}

FString URigVMPin::GetArrayElementCppType() const
{
	if (!IsArray())
	{
		return FString();
	}

	const FString ResolvedType = GetCPPType();
	return RigVMTypeUtils::BaseTypeFromArrayType(ResolvedType);
}

FRigVMTemplateArgumentType URigVMPin::GetTemplateArgumentType() const
{
	return FRigVMRegistry::Get().GetType(GetTypeIndex());
}

TRigVMTypeIndex URigVMPin::GetTypeIndex() const
{
	if(LastKnownCPPType != GetCPPType())
	{
		LastKnownTypeIndex = INDEX_NONE;
	}
	if(LastKnownTypeIndex == INDEX_NONE)
	{
		LastKnownCPPType = GetCPPType();
		// cpp type can be empty if it is an unsupported type such as a UObject type
		if (!LastKnownCPPType.IsEmpty())
		{
			const FRigVMTemplateArgumentType Type(*LastKnownCPPType, GetCPPTypeObject());
			LastKnownTypeIndex = FRigVMRegistry::Get().FindOrAddType(Type);
			// in rare cases LastKnowTypeIndex can still be NONE here because
			// we have nodes that has constant pin that references struct type like FRuntimeFloatCurve
			// which contains a object ptr member and is thus not registered in the registry
		}
	}
	return LastKnownTypeIndex;
}

bool URigVMPin::IsStringType() const
{
	if(!CachedIsStringType.IsValid())
	{
		const FString ResolvedType = GetCPPType();
		CachedIsStringType = ResolvedType.Equals(TEXT("FString")) || ResolvedType.Equals(TEXT("FName"));
	}
	return CachedIsStringType.GetValue();
}

bool URigVMPin::IsExecuteContext() const
{
	if (const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		if (ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
		{
			return true;
		}
	}
	return false;
}

bool URigVMPin::IsWildCard() const
{
	if (const UScriptStruct* ScriptStruct = GetScriptStruct())
	{
		if (ScriptStruct->IsChildOf(FRigVMUnknownType::StaticStruct()))
		{
			return true;
		}
	}
	if (CPPType.IsEmpty())
	{
		// Unknown type
		return true;
	}
	return false;
}

bool URigVMPin::ContainsWildCardSubPin() const
{
	for(const URigVMPin* SubPin : SubPins)
	{
		if(SubPin->IsWildCard() || SubPin->ContainsWildCardSubPin())
		{
			return true;
		}
	}
	return false;
}

bool URigVMPin::IsFixedSizeArray() const
{
#if WITH_EDITOR

	if(IsArray() && IsRootPin())
	{
		if(const URigVMNode* Node = GetNode())
		{
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(const UScriptStruct* Struct = UnitNode->GetScriptStruct())
				{
					if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
					{
						return Property->HasMetaData(FRigVMStruct::FixedSizeArrayMetaName);
					}
				}
			}
			else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					return Factory->HasArgumentMetaData(GetFName(), FRigVMStruct::FixedSizeArrayMetaName);
				}
			}
			else if(Node->IsA<UDEPRECATED_RigVMSelectNode>())
			{
				return GetFName().ToString() == UDEPRECATED_RigVMSelectNode::ValueName;
			}
		}
	}
#endif
	return false;
}

bool URigVMPin::ShouldOnlyShowSubPins() const
{
#if WITH_EDITOR

	if(IsRootPin())
	{
		if(const URigVMNode* Node = GetNode())
		{
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(const UScriptStruct* Struct = UnitNode->GetScriptStruct())
				{
					if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
					{
						return Property->HasMetaData(FRigVMStruct::ShowOnlySubPinsMetaName);
					}
				}
			}
			else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					return Factory->HasArgumentMetaData(GetFName(), FRigVMStruct::ShowOnlySubPinsMetaName);
				}
			}
		}
	}
#endif

	return false;
}

bool URigVMPin::ShouldHideSubPins() const
{
#if WITH_EDITOR
	if(ShouldOnlyShowSubPins())
	{
		return false;
	}
	
	if(IsRootPin())
	{
		if(const URigVMNode* Node = GetNode())
		{
			if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
			{
				if(const UScriptStruct* Struct = UnitNode->GetScriptStruct())
				{
					if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
					{
						return Property->HasMetaData(FRigVMStruct::HideSubPinsMetaName);
					}
				}
			}
			else if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
			{
				if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
				{
					return Factory->HasArgumentMetaData(GetFName(), FRigVMStruct::HideSubPinsMetaName);
				}
			}
		}
		if (IsDefinedAsInputVariable() && GetDirection() == ERigVMPinDirection::Input)
		{
			return true;
		}
	}
#endif

	return false;
}

FString URigVMPin::GetOriginalDefaultValue() const
{
	if(const URigVMNode* Node = GetNode())
	{
		return Node->GetOriginalPinDefaultValue(this);
	}
	return FString();
}

bool URigVMPin::HasOriginalDefaultValue() const
{
	if(CachedHasOriginalDefaultValue.IsValid())
	{
		return CachedHasOriginalDefaultValue.GetValue();
	}

	if(!CanProvideDefaultValue())
	{
		CachedHasOriginalDefaultValue = false;
		return false;
	}

	CachedHasOriginalDefaultValue = true;

	if(SubPins.IsEmpty())
	{
		const FString CurrentDefaultValue = GetDefaultValue();
		FString OriginalDefaultValue = GetOriginalDefaultValue();
		URigVMController::PostProcessDefaultValue(this, OriginalDefaultValue);
		if(CurrentDefaultValue != OriginalDefaultValue)
		{
			CachedHasOriginalDefaultValue = false;
		}
	}
	else
	{
		for(const URigVMPin* SubPin : SubPins)
		{
			if(!SubPin->HasOriginalDefaultValue())
			{
				CachedHasOriginalDefaultValue = false;
				break;
			}
		}
	}

	return CachedHasOriginalDefaultValue.GetValue();
}

FString URigVMPin::GetDefaultValue() const
{
	return GetDefaultValue(EmptyPinOverride, true);
}

FString URigVMPin::GetDefaultValue(const URigVMPin::FPinOverride& InOverride, bool bAdaptValueForPinType) const
{
	if (FPinOverrideValue const* OverrideValuePtr = InOverride.Value.Find(InOverride.Key.GetSibling((URigVMPin*)this)))
	{
		return OverrideValuePtr->DefaultValue;
	}

	TRigVMModelCachedValue<URigVMPin, FString>& Cache = bAdaptValueForPinType ? CachedAdaptedDefaultValue : CachedDefaultValue;

	if(Cache.IsValid())
	{
		return Cache.GetValue();
	}

	if (IsArray())
	{
		FRigVMRegistry& Registry = FRigVMRegistry::Get();
		const TRigVMTypeIndex ArrayType = GetTypeIndex();
		if (!Registry.IsArrayType(ArrayType))
		{
			Cache = TEXT("()"); 
			return Cache.GetValue();
		}
		if (SubPins.Num() > 0)
		{
			const TRigVMTypeIndex ElementType = Registry.GetBaseTypeFromArrayTypeIndex(ArrayType);
			TArray<FString> ElementDefaultValues;
			for (URigVMPin* SubPin : SubPins)
			{
				if (SubPin->GetTypeIndex() != ElementType)
				{
					Cache = TEXT("()"); 
					return Cache.GetValue();
				}
				FString ElementDefaultValue = SubPin->GetDefaultValue(InOverride, bAdaptValueForPinType);
				if (SubPin->IsStringType())
				{
					ElementDefaultValue = TEXT("\"") + ElementDefaultValue + TEXT("\"");
				}

				ElementDefaultValues.Add(ElementDefaultValue);
			}
			if (ElementDefaultValues.Num() == 0)
			{
				Cache = TEXT("()"); 
				return Cache.GetValue();
			}
			Cache = FString::Printf(TEXT("(%s)"), *FString::Join(ElementDefaultValues, TEXT(",")));
			return Cache.GetValue();
		}

		return DefaultValue.IsEmpty() ? TEXT("()") : DefaultValue;
	}
	else if (IsStruct())
	{
		static const FString EmptyStructDefaultValue = TEXT("()");
		
		if((GetScriptStruct()->StructFlags & (STRUCT_ImportTextItemNative | STRUCT_ExportTextItemNative)) == (STRUCT_ImportTextItemNative | STRUCT_ExportTextItemNative))
		{
			// If the struct has a native import/export, then its default value must be used verbatim (subpins are not displayed)
			return DefaultValue;
		}
		else
		{
			// for trait pins, there are cases where a pin is not created for a property (see ShouldCreatePinForProperty())
			// so we store the value of that property in the default value of the struct pin containing that property
			// as a result, to retrieve the default value we need to combine the default value on the struct pin, with additional overrides in the available sub pins
			if (SubPins.Num() > 0 || IsTraitPin())
			{
				FString FinalDefaultValue = DefaultValue;
				
				// root trait pin store their default value in a separate property bag so that
				// things like soft object ptr can be used and tracked in a uproperty
				if (IsTraitPin() && IsRootPin())
				{
					FRigVMTraitDefaultValueStruct* DefaultValueStructPtr = GetNode()->TraitDefaultValues.Find(GetName());
					if (ensure(DefaultValueStructPtr))
					{
						FinalDefaultValue = DefaultValueStructPtr->GetValue();
					}
				}

				for (const URigVMPin* SubPin : SubPins)
				{
					FString MemberDefaultValue = SubPin->GetDefaultValue(InOverride, bAdaptValueForPinType);
					if (SubPin->IsStringType() && !MemberDefaultValue.IsEmpty())
					{
						MemberDefaultValue = TEXT("\"") + MemberDefaultValue + TEXT("\"");
					}
					else if (MemberDefaultValue.IsEmpty() || MemberDefaultValue == TEXT("()"))
					{
						continue;
					}

					URigVMController::OverrideDefaultValueMember(SubPin->GetName(), MemberDefaultValue, FinalDefaultValue);
				}

				Cache = !FinalDefaultValue.IsEmpty() ? FinalDefaultValue : EmptyStructDefaultValue;
				return Cache.GetValue();
			}
		}

		// special case certain pin types to adapt their values from
		// alternative representations.
		
		if(bAdaptValueForPinType && !DefaultValue.IsEmpty() && DefaultValue != EmptyStructDefaultValue)
		{
			if(GetScriptStruct() == TBaseStructure<FQuat>::Get())
			{
				// quaternions also allow default values stored as rotators
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
				FRotator Rotator = FRotator::ZeroRotator;
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity()); 
				TBaseStructure<FRotator>::Get()->ImportText(*DefaultValue, &Rotator, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName());
				if(ErrorPipe.NumErrors == 0)
				{
					const FQuat Quat = FQuat::MakeFromRotator(Rotator);
					FString AdaptedDefaultValue;	
					TBaseStructure<FQuat>::Get()->ExportText(AdaptedDefaultValue, &Quat, &Quat, nullptr, PPF_None, nullptr);
					Cache = AdaptedDefaultValue;
					return Cache.GetValue();
				}
			}
		}
		
		return DefaultValue.IsEmpty() ? EmptyStructDefaultValue : DefaultValue;
	}
	else if (IsArrayElement() && ((!IsUObject() && DefaultValue.IsEmpty()) || (IsUObject() && DefaultValueObject == nullptr)))
	{
		// array element cannot have an empty default value because when an array pin is
		// added as a property to a memory storage class, its default value needs to reflect 
		// the number of array elements in that array pin.
		// for example:
		// for an array pin of 1 float, the final default value should be "(0.0)" instead of "()".
		// This default value is used during URigVMCompiler::FindOrAddRegister(...)
		// Thus in this block, we have to return something like 0.0 instead of empty string

		Cache = URigVMController::GetPinInitialDefaultValue(this);
		return Cache.GetValue();
	}
	else if (IsUObject())
	{
		FSoftObjectPath Object(DefaultValueObject);
		Cache = Object.ToString();
		return Cache.GetValue();
	}

	return DefaultValue;
}

UObject* URigVMPin::GetDefaultValueObject() const
{
	return DefaultValueObject;
}

FString URigVMPin::GetDefaultValueStoredByUserInterface() const
{
	return GetDefaultValue(EmptyPinOverride, false);
}

template< typename Type>
static FString ClampValue(const FString& InValueString, const FString& InMinValueString, const FString& InMaxValueString)
{
	FString RetValString = InValueString;
	Type RetVal;
	TTypeFromString<Type>::FromString(RetVal, *RetValString);

	// Enforce min
	if(!InMinValueString.IsEmpty())
	{
		checkSlow(InMinValueString.IsNumeric());
		Type MinValue;
		TTypeFromString<Type>::FromString(MinValue, *InMinValueString);
		RetVal = FMath::Max<Type>(MinValue, RetVal);
	}
	//Enforce max 
	if(!InMaxValueString.IsEmpty())
	{
		checkSlow(InMaxValueString.IsNumeric());
		Type MaxValue;
		TTypeFromString<Type>::FromString(MaxValue, *InMaxValueString);
		RetVal = FMath::Min<Type>(MaxValue, RetVal);
	}

	RetValString = TTypeToString<Type>::ToString(RetVal);
	return RetValString;
}

bool URigVMPin::IsValidDefaultValue(const FString& InDefaultValue) const
{
	TArray<FString> DefaultValues; 

	if (IsArray())
	{
		if(InDefaultValue.IsEmpty())
		{
			return false;
		}
		
		if(InDefaultValue[0] != TCHAR('('))
		{
			return false;
		}

		if(InDefaultValue[InDefaultValue.Len() - 1] != TCHAR(')'))
		{
			return false;
		}

		DefaultValues = URigVMPin::SplitDefaultValue(InDefaultValue);
	}
	else
	{
		DefaultValues.Add(InDefaultValue);
	}

	FString BaseCPPType = GetCPPType()
		.Replace(RigVMTypeUtils::TArrayPrefix, TEXT(""))
		.Replace(RigVMTypeUtils::TObjectPtrPrefix, TEXT(""))
		.Replace(RigVMTypeUtils::TScriptInterfacePrefix, TEXT(""))
		.Replace(TEXT(">"), TEXT(""));

	for (const FString& Value : DefaultValues)
	{
		// perform single value validation
		if (UClass* Class = Cast<UClass>(GetCPPTypeObject()))
		{
			// "None" is the serialized form of a null object reference.
			static const FString NoneString = TEXT("None");
			if(Value.IsEmpty() || Value.Equals(NoneString, ESearchCase::CaseSensitive))
			{
				return true;
			}

			UObject* Object = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(Value);
			if(Object == nullptr)
			{
				return false;
			}

			const bool bIsClass = RigVMTypeUtils::IsUClassType(GetCPPType());
			if(bIsClass)
			{
				if(!CastChecked<UClass>(Object)->IsChildOf(Class))
				{
					return false;
				}
			}
			else
			{
				if(!Object->GetClass()->IsChildOf(Class))
				{
					return false;
				}
			}
		} 
		else if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(GetCPPTypeObject()))
		{
			// special case alternative representations 
			if(ScriptStruct == TBaseStructure<FQuat>::Get())
			{
				// quaternions also allow default values stored as rotators
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
				FRotator Rotator = FRotator::ZeroRotator;
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity()); 
				TBaseStructure<FRotator>::Get()->ImportText(*Value, &Rotator, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName());
				if(ErrorPipe.NumErrors == 0)
				{
					return true;
				}
			}
			
			TArray<uint8> TempStructBuffer;
			TempStructBuffer.AddUninitialized(ScriptStruct->GetStructureSize());
			ScriptStruct->InitializeDefaultValue(TempStructBuffer.GetData());

			FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
			{
				// force logging to the error pipe for error detection
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity());
				// Note we use PPF_UseDeprecatedProperties as if a deprecated property is encountered this call will otherwise fail via the ErrorPipe
				ScriptStruct->ImportText(*Value, TempStructBuffer.GetData(), nullptr, PPF_UseDeprecatedProperties, &ErrorPipe, ScriptStruct->GetName()); 
			}

			ScriptStruct->DestroyStruct(TempStructBuffer.GetData());

			if (ErrorPipe.NumErrors > 0)
			{
				return false;
			}
		} 
		else if (UEnum* EnumType = Cast<UEnum>(GetCPPTypeObject()))
		{
			FName EnumName(EnumType->GenerateFullEnumName(*Value));
			if (!EnumType->IsValidEnumName(EnumName))
			{
				return false;
			}
			else
			{
				if (EnumType->HasMetaData(TEXT("Hidden"), EnumType->GetIndexByName(EnumName)))
				{
					return false;
				}
			}
		} 
		else if (BaseCPPType == TEXT("float"))
		{ 
			if (!FDefaultValueHelper::IsStringValidFloat(Value))
			{
				return false;
			}
		}
		else if (BaseCPPType == TEXT("double"))
		{ 
			if (!FDefaultValueHelper::IsStringValidFloat(Value))
			{
				return false;
			}
		}
		else if (BaseCPPType == TEXT("int32"))
		{ 
			if (!FDefaultValueHelper::IsStringValidInteger(Value))
			{
				return false;
			}
		}
		else if (BaseCPPType == TEXT("bool"))
		{
			if (Value != TEXT("True") && Value != TEXT("False"))
			{
				return false;
			}
		}
		else if (BaseCPPType == TEXT("FString"))
		{ 
			// anything is allowed
		}
		else if (BaseCPPType == TEXT("FName"))
		{
			// anything is allowed
		}
	}

	return true;
}

bool URigVMPin::HasUserProvidedDefaultValue() const
{
	return HasDefaultValueOverride();
}

bool URigVMPin::HasDefaultValueOverride() const
{
	if(!CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		return false;
	}

	if(!CanProvideDefaultValue())
	{
		return false;
	}

	if(CachedDefaultValueOverride.IsValid())
	{
		return CachedDefaultValueOverride.GetValue();
	}

	if(DefaultValueType == ERigVMPinDefaultValueType::Override)
	{
		CachedDefaultValueOverride = true;
		return CachedDefaultValueOverride.GetValue();
	}

	for(const TObjectPtr<URigVMPin>& SubPin : SubPins)
	{
		if(SubPin->HasDefaultValueOverride())
		{
			CachedDefaultValueOverride = true;
			return CachedDefaultValueOverride.GetValue();
		}
	}

	CachedDefaultValueOverride = false;
	if(!HasOriginalDefaultValue())
	{
		CachedDefaultValueOverride = true;
	}
	return CachedDefaultValueOverride.GetValue();
}

bool URigVMPin::CanProvideDefaultValue() const
{
	if((GetDirection() != ERigVMPinDirection::Input) &&
		(GetDirection() != ERigVMPinDirection::IO) &&
		(GetDirection() != ERigVMPinDirection::Visible))
	{
		return false;
	}
	if(IsWildCard() && !IsArray())
	{
		return false;
	}
	if(IsExecuteContext())
	{
		return false;
	}
	return true;
}

FString URigVMPin::ClampDefaultValueFromMetaData(const FString& InDefaultValue) const
{
	FString RetVal = InDefaultValue;
	if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(GetNode()))
	{
		TArray<FString> RetVals;
		TArray<FString> DefaultValues; 

		if (IsArray())
		{
			DefaultValues = URigVMPin::SplitDefaultValue(InDefaultValue);
		}
		else
		{
			DefaultValues.Add(InDefaultValue);
		}

		FString MinValue, MaxValue;	
		if (UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
		{
			if (FProperty* Property = ScriptStruct->FindPropertyByName(*GetName()))
			{
				MinValue = Property->GetMetaData(TEXT("ClampMin"));
				MaxValue = Property->GetMetaData(TEXT("ClampMax"));
			}
		}
		

		FString BaseCPPType = GetCPPType()
			.Replace(RigVMTypeUtils::TArrayPrefix, TEXT(""))
			.Replace(RigVMTypeUtils::TObjectPtrPrefix, TEXT(""))
			.Replace(RigVMTypeUtils::TScriptInterfacePrefix, TEXT(""))
			.Replace(TEXT(">"), TEXT(""));

		RetVals.SetNumZeroed(DefaultValues.Num());
		for (int32 Index = 0; Index < DefaultValues.Num(); ++Index)
		{
			const FString& Value = DefaultValues[Index]; 

			if (!MinValue.IsEmpty() || !MaxValue.IsEmpty())
			{
				// perform single value validation
				if (BaseCPPType == TEXT("float"))
				{ 
					RetVals[Index] = ClampValue<float>(Value, MinValue, MaxValue);
				}
				else if (BaseCPPType == TEXT("double"))
				{ 
					RetVals[Index] = ClampValue<double>(Value, MinValue, MaxValue);
				}
				else if (BaseCPPType == TEXT("int32"))
				{ 
					RetVals[Index] = ClampValue<int32>(Value, MinValue, MaxValue);
				}
				else
				{
					RetVals[Index] = Value;
				}
			}
			else
			{
				RetVals[Index] = Value;
			}
		}

		if (IsArray())
		{
			RetVal = GetDefaultValueForArray(RetVals);
		}
		else
		{
			RetVal = RetVals[0];
		}
	}

	return RetVal;
}

FName URigVMPin::GetCustomWidgetName() const
{
	if (IsArrayElement())
	{
		return GetParentPin()->GetCustomWidgetName();
	}

#if WITH_EDITOR
	if(CustomWidgetName.IsNone())
	{
		return FName(GetMetaData(FRigVMStruct::CustomWidgetMetaName));
	}
#endif
	return CustomWidgetName;
}

FString URigVMPin::GetMetaData(FName InKey) const
{
	if (IsArrayElement())
	{
		return GetParentPin()->GetMetaData(InKey);
	}

#if WITH_EDITOR
	if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(GetNode()))
	{
		if(IsTraitPin())
		{
			if(const UScriptStruct* Struct = GetTraitScriptStruct())
			{
				if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
				{
					return Property->GetMetaData(InKey);
				}
				else
				{
					// Possible the pin was programmatically generated from the trait's shared struct
					TSharedPtr<FStructOnScope> TraitScope = GetTraitInstance();
					if(TraitScope.IsValid())
					{
						const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
						Struct = VMTrait->GetTraitSharedDataStruct();
						Property = Struct != nullptr ? Struct->FindPropertyByName(GetFName()) : nullptr;
						if(Property)
						{
							return Property->GetMetaData(InKey);
						}
					}

				}
			}
		}

		if(const UScriptStruct* Struct = GetParentScriptStruct(UnitNode))
		{
			if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
			{
				return Property->GetMetaData(InKey);
			}
		}
	}
	if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(GetNode()))
	{
		if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
		{
			return Template->GetArgumentMetaData(GetFName(), InKey);
		}
	}
#endif

	return FString();
}

bool URigVMPin::HasMetaData(FName InKey) const
{
	if (IsArrayElement())
	{
		return GetParentPin()->HasMetaData(InKey);
	}

#if WITH_EDITOR
	if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(GetNode()))
	{
		if(IsTraitPin())
		{
			if(const UScriptStruct* Struct = GetTraitScriptStruct())
			{
				if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
				{
					return Property->HasMetaData(InKey);
				}
				else
				{
					// Possible the pin was programmatically generated from the trait's shared struct
					TSharedPtr<FStructOnScope> TraitScope = GetTraitInstance();
					if(TraitScope.IsValid())
					{
						const FRigVMTrait* VMTrait = (FRigVMTrait*)TraitScope->GetStructMemory();
						Struct = VMTrait->GetTraitSharedDataStruct();
						Property = Struct != nullptr ? Struct->FindPropertyByName(GetFName()) : nullptr;
						if(Property)
						{
							return Property->HasMetaData(InKey);
						}
					}
				}
			}
		}
		else
		{
			if(const UScriptStruct* Struct = GetParentScriptStruct(UnitNode))
			{
				if(const FProperty* Property = Struct->FindPropertyByName(GetFName()))
				{
					return Property->HasMetaData(InKey);
				}
			}
		}
	}
	if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(GetNode()))
	{
		if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
		{
			return !Template->GetArgumentMetaData(GetFName(), InKey).IsEmpty();
		}
	}
#endif

	return false;
}

FText URigVMPin::GetToolTipText() const
{
	if(URigVMNode* Node = GetNode())
	{
		return Node->GetToolTipTextForPin(this);
	}
	return FText();
}

URigVMVariableNode* URigVMPin::GetBoundVariableNode() const
{
	for (const TObjectPtr<URigVMInjectionInfo>& InjectionInfo : InjectionInfos)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InjectionInfo->Node))
		{
			return VariableNode;
		}
	}
	if (IsDefinedAsInputVariable() && Links.Num() == 1)
	{
		if (Links[0]->GetTargetPin() == this)
		{
			if (const URigVMPin* SourcePin = Links[0]->GetSourcePin())
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
				{
					if (VariableNode->GetValuePin() == SourcePin->GetRootPin())
					{
						return VariableNode;
					}
				}
			}
		}
	}
	
	return nullptr;
}

// Returns the variable bound to this pin (or NAME_None)
const FString URigVMPin::GetBoundVariablePath() const
{
	return GetBoundVariablePath(EmptyPinOverride);
}

// Returns the variable bound to this pin (or NAME_None)
const FString URigVMPin::GetBoundVariablePath(const URigVMPin::FPinOverride& InOverride) const
{
	if (FPinOverrideValue const* OverrideValuePtr = InOverride.Value.Find(InOverride.Key.GetSibling((URigVMPin*)this)))
	{
		return OverrideValuePtr->BoundVariablePath;
	}

	for (const TObjectPtr<URigVMInjectionInfo>& InjectionInfo : InjectionInfos)
	{
		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InjectionInfo->Node))
		{
			if (URigVMPin* Pin = InjectionInfo->GetPin())
			{
				FString SegmentPath = Pin->GetSegmentPath(false);
				if (SegmentPath.IsEmpty())
				{
					return VariableNode->GetVariableName().ToString();
				}
				return VariableNode->GetVariableName().ToString() + TEXT(".") + SegmentPath;
			}
		}
	}
	if (IsDefinedAsInputVariable() && Links.Num() == 1)
	{
		if (Links[0]->GetTargetPin() == this)
		{
			if (const URigVMPin* SourcePin = Links[0]->GetSourcePin())
			{
				if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
				{
					if (VariableNode->GetValuePin() == SourcePin->GetRootPin())
					{
						const FString SegmentPath = SourcePin->GetSegmentPath(false);
						if (SegmentPath.IsEmpty())
						{
							return VariableNode->GetVariableName().ToString();
						}
						return VariableNode->GetVariableName().ToString() + TEXT(".") + SegmentPath;
					}
				}
				if (SourcePin->IsDefinedAsInputVariable() && Cast<URigVMFunctionEntryNode>(SourcePin->GetNode()))
				{
					const FString SegmentPath = SourcePin->GetSegmentPath(false);
					if (SegmentPath.IsEmpty())
					{
						return SourcePin->GetName();
					}
					return SourcePin->GetName() + TEXT(".") + SegmentPath;
				}
			}
		}
	}
	
	return FString();
}

// Returns the variable bound to this pin (or NAME_None)
FString URigVMPin::GetBoundVariableName() const
{
	if (URigVMVariableNode* VariableNode = GetBoundVariableNode())
	{
		return VariableNode->GetVariableName().ToString();
	}
	if (IsDefinedAsInputVariable() && Links.Num() == 1)
	{
		if (Links[0]->GetTargetPin() == this)
		{
			if (const URigVMPin* SourcePin = Links[0]->GetSourcePin())
			{
				if (SourcePin->IsDefinedAsInputVariable() && Cast<URigVMFunctionEntryNode>(SourcePin->GetNode()))
				{
					return SourcePin->GetName();
				}
			}
		}
	}

	return FString();
}

// Returns true if this pin is bound to a variable
bool URigVMPin::IsBoundToVariable() const
{
	return IsBoundToVariable(EmptyPinOverride);
}

// Returns true if this pin is bound to a variable
bool URigVMPin::IsBoundToVariable(const URigVMPin::FPinOverride& InOverride) const
{
	return !GetBoundVariablePath(InOverride).IsEmpty();
}

bool URigVMPin::IsBoundToExternalVariable() const
{
	FString VariableName = GetBoundVariableName();
	if (VariableName.IsEmpty())
	{
		return false;
	}

	TArray<FRigVMGraphVariableDescription> LocalVariables = GetGraph()->GetLocalVariables(true);
	for (FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
	{
		if (LocalVariable.Name == *VariableName)
		{
			return false;
		}
	}

	return true;
}

bool URigVMPin::IsBoundToLocalVariable() const
{
	FString VariableName = GetBoundVariableName();
	if (VariableName.IsEmpty())
	{
		return false;
	}

	TArray<FRigVMGraphVariableDescription> LocalVariables = GetGraph()->GetLocalVariables(false);
	for (FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
	{
		if (LocalVariable.Name == *VariableName)
		{
			return true;
		}
	}

	return false;
}

bool URigVMPin::IsBoundToInputArgument() const
{
	FString VariableName = GetBoundVariableName();
	if (VariableName.IsEmpty())
	{
		return false;
	}

	if (URigVMFunctionEntryNode* EntryNode = GetGraph()->GetEntryNode())
	{
		if (const URigVMPin* InterfacePin = EntryNode->FindPin(VariableName))
		{
			return !InterfacePin->IsDefinedAsInputVariable();
		}
	}
	
	return false;
}

bool URigVMPin::CanBeBoundToVariable(const FRigVMExternalVariable& InExternalVariable, const FString& InSegmentPath) const
{
	if (!InExternalVariable.IsValid(true))
	{
		return false;
	}

	if (bIsConstant)
	{
		return false;
	}

	if (bIsInputVariable && InExternalVariable.IsReadOnly())
	{
		return false;
	}

	// only allow to bind variables to input pins for now
	if (Direction == ERigVMPinDirection::Output)
	{
		return false;
	}

	// check type validity
	// in the future we need to allow arrays as well
	if (IsArray() && !InSegmentPath.IsEmpty())
	{
		return false;
	}
	if (IsArray() != InExternalVariable.IsArray())
	{
		return false;
	}

	FName ExternalCPPType = InExternalVariable.GetExtendedCPPType();
	UObject* ExternalCPPTypeObject = const_cast<UObject*>(InExternalVariable.GetCPPTypeObject());

	if(!InSegmentPath.IsEmpty())
	{
		check(InExternalVariable.GetProperty());

		const FProperty* Property = InExternalVariable.GetProperty();
		const FRigVMPropertyPath PropertyPath(Property, InSegmentPath);
		Property = PropertyPath.GetTailProperty();

		RigVMPropertyUtils::GetTypeFromProperty(Property, ExternalCPPType, ExternalCPPTypeObject);
	}

	const FString CPPBaseType = IsArray() ? GetArrayElementCppType() : GetCPPType();
	return RigVMTypeUtils::AreCompatible(ExternalCPPType, ExternalCPPTypeObject, *CPPBaseType, GetCPPTypeObject());
}

bool URigVMPin::ShowInDetailsPanelOnly() const
{
	if(CachedShowInDetailsPanelOnly.IsValid())
	{
		return CachedShowInDetailsPanelOnly.GetValue();
	}
	CachedShowInDetailsPanelOnly.ResetCachedValue();

#if WITH_EDITOR
	if (GetParentPin() == nullptr)
	{
		if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(GetNode()))
		{
			if (UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
			{
				if (FProperty* Property = ScriptStruct->FindPropertyByName(GetFName()))
				{
					if (Property->HasMetaData(FRigVMStruct::DetailsOnlyMetaName))
					{
						CachedShowInDetailsPanelOnly = true;
					}
				}
			}
		}
		else if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(GetNode()))
		{
			if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
			{
				CachedShowInDetailsPanelOnly = !Template->GetArgumentMetaData(GetFName(), FRigVMStruct::DetailsOnlyMetaName).IsEmpty();
			}
		}
	}
#endif

	static constexpr bool bShowOnlyInDetailsPanel = false;
	return CachedShowInDetailsPanelOnly.Get(bShowOnlyInDetailsPanel); 
}

// Returns nullptr external variable matching this description
FRigVMExternalVariable URigVMPin::ToExternalVariable() const
{
	FRigVMExternalVariable ExternalVariable;

	FString VariableName = GetBoundVariableName();
	if (VariableName.IsEmpty())
	{
		FString NodeName, PinPath;
		if (!SplitPinPathAtStart(GetPinPath(), NodeName, VariableName))
		{
			return ExternalVariable;
		}

		VariableName = VariableName.Replace(TEXT("."), TEXT("_"));
	}

	ExternalVariable = RigVMTypeUtils::ExternalVariableFromCPPType(FGuid(), *VariableName, CPPType, GetCPPTypeObject(), false, false);
	
	return ExternalVariable;
}

bool URigVMPin::IsOrphanPin() const
{
	if(URigVMPin* RootPin = GetRootPin())
	{
		if(RootPin != this)
		{
			return RootPin->IsOrphanPin();
		}
	}
	if(URigVMNode* Node = GetNode())
	{
		return Node->OrphanedPins.Contains(this);
	}
	return false;
}

uint32 URigVMPin::GetStructureHash() const
{
	uint32 Hash = GetTypeHash(GetName());
	Hash = HashCombine(Hash, GetTypeHash(GetCPPType()));
	Hash = HashCombine(Hash, GetTypeHash((int32)GetDirection()));
	Hash = HashCombine(Hash, FRigVMRegistry::Get().GetHashForType(GetTypeIndex()));
	return Hash;
}

bool URigVMPin::IsTraitPin() const
{
	if(const URigVMNode* Node = GetNode())
	{
		return Node->IsTraitPin(GetRootPin());
	}
	return false;
}

bool URigVMPin::IsProgrammaticPin() const
{
	// Traits can generate their own programmatic pins via FRigVMTrait::GetProgrammaticPins. We account for these as additional expressions if the
	// pin is not part of the set of sub-pins exposed on the struct
	if(const URigVMPin* ParentPin = GetParentPin())
	{
		UScriptStruct* ScriptStruct = ParentPin->GetScriptStruct();
		if(ScriptStruct && ScriptStruct->IsChildOf(FRigVMTrait::StaticStruct()))
		{
			if(ScriptStruct->FindPropertyByName(GetFName()) == nullptr)
			{
				return true;
			}
		}
	}

	return false;
}

TArray<URigVMPin*> URigVMPin::GetProgrammaticSubPins() const
{
	TArray<URigVMPin*> ProgrammaticPins;
	for(URigVMPin* SubPin : SubPins)
	{
		if(SubPin->IsProgrammaticPin())
		{
			ProgrammaticPins.Add(SubPin);
		}
	}

	return ProgrammaticPins;
}

TSharedPtr<FStructOnScope> URigVMPin::GetTraitInstance(bool bUseDefaultValueFromPin) const
{
	if(const URigVMNode* Node = GetNode())
	{
		return Node->GetTraitInstance(GetRootPin(), bUseDefaultValueFromPin);
	}

	static const TSharedPtr<FStructOnScope> EmptyTrait;
	return EmptyTrait;
}

UScriptStruct* URigVMPin::GetTraitScriptStruct() const
{
	if(const URigVMNode* Node = GetNode())
	{
		return Node->GetTraitScriptStruct(GetRootPin());
	}

	return nullptr;
}

const uint32& URigVMPin::GetNodeCachedValueVersion() const
{
	if(const URigVMNode* Node = GetNode())
	{
		return Node->GetCachedValueVersion();
	}
	static constexpr uint32 InvalidVersion = 0;
	return InvalidVersion;
}

const uint32& URigVMPin::GetCachedValueVersion() const
{
	CombinedPinVersion = HashCombine(GetNodeCachedValueVersion(), PinVersion);
	return CombinedPinVersion;
}

void URigVMPin::UpdateTypeInformationIfRequired() const
{
	const uint32 CPPTypeObjectHash = GetTypeHash(CPPTypeObject);
	if(CachedCPPTypeObjectHash.IsValid())
	{
		if(CachedCPPTypeObjectHash == CPPTypeObjectHash)
		{
			return;
		}
	}
	CachedCPPTypeObjectHash = CPPTypeObjectHash;
	
	if (CPPTypeObject == nullptr)
	{
		if (CPPTypeObjectPath != NAME_None)
		{
			URigVMPin* MutableThis = (URigVMPin*)this;
			MutableThis->CPPTypeObject = RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(CPPTypeObjectPath.ToString());
			RigVMTypeUtils::FixCPPTypeAndObject(MutableThis->CPPType, MutableThis->CPPTypeObject);
			if (!MutableThis->CPPType.IsEmpty())
			{
				MutableThis->LastKnownTypeIndex = FRigVMRegistry::Get().FindOrAddType(FRigVMTemplateArgumentType(*CPPType, CPPTypeObject));
				MutableThis->LastKnownCPPType = MutableThis->CPPType;
			} 
		}
	}

	if (CPPTypeObject)
	{
		// refresh the type string 
		URigVMPin* MutableThis = (URigVMPin*)this;
		MutableThis->CPPType = RigVMTypeUtils::PostProcessCPPType(CPPType, CPPTypeObject);
	}
}

UObject* URigVMPin::GetCPPTypeObject() const
{
	UpdateTypeInformationIfRequired();
	return CPPTypeObject;
}

UScriptStruct* URigVMPin::GetScriptStruct() const
{
	return Cast<UScriptStruct>(GetCPPTypeObject());
}

UScriptStruct* URigVMPin::GetParentScriptStruct(const URigVMUnitNode* FallbackNode) const
{
	if (const URigVMPin* ParentPin = GetParentPin())
	{
		if (ParentPin->GetScriptStruct())
		{
			return ParentPin->GetScriptStruct();
		}
		else
		{
			return nullptr;
		}
	}

	return FallbackNode ? FallbackNode->GetScriptStruct() : nullptr;
}

UEnum* URigVMPin::GetEnum() const
{
	return Cast<UEnum>(GetCPPTypeObject());
}

URigVMPin* URigVMPin::GetParentPin() const
{
	return Cast<URigVMPin>(GetOuter());
}

URigVMPin* URigVMPin::GetRootPin() const
{
	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin == nullptr)
	{
		return const_cast<URigVMPin*>(this);
	}
	return ParentPin->GetRootPin();
}

bool URigVMPin::IsRootPin() const
{
	return GetParentPin() == nullptr;
}

URigVMPin* URigVMPin::GetPinForLink() const
{
	URigVMPin* RootPin = GetRootPin();

	if (!RootPin->HasInjectedUnitNodes())
	{
		return const_cast<URigVMPin*>(this);
	}

	URigVMPin* PinForLink =
		((Direction == ERigVMPinDirection::Input) || (Direction == ERigVMPinDirection::IO)) ?
		RootPin->InjectionInfos.Last()->InputPin :
		RootPin->InjectionInfos.Last()->OutputPin;

	if (RootPin != this)
	{
		FString SegmentPath = GetSegmentPath();
		return PinForLink->FindSubPin(SegmentPath);
	}

	return PinForLink;
}

URigVMLink* URigVMPin::FindLinkForPin(const URigVMPin* InOtherPin) const
{
	for (URigVMLink* Link : Links)
	{
		if ((Link->GetSourcePin() == this && Link->GetTargetPin() == InOtherPin) ||
			(Link->GetSourcePin() == InOtherPin && Link->GetTargetPin() == this))
		{
			return Link;
		}
	}
	return nullptr;
}

URigVMPin* URigVMPin::GetOriginalPinFromInjectedNode() const
{
	if(GetNode() == nullptr)
	{
		return nullptr;
	}
	
	if (URigVMInjectionInfo* Injection = GetNode()->GetInjectionInfo())
	{
		URigVMPin* RootPin = GetRootPin();
		URigVMPin* OriginalPin = nullptr;
		if (Injection->bInjectedAsInput && Injection->InputPin == RootPin && Injection->OutputPin)
		{
			TArray<URigVMPin*> LinkedPins = Injection->OutputPin->GetLinkedTargetPins();
			if (LinkedPins.Num() == 1)
			{
				OriginalPin = LinkedPins[0]->GetOriginalPinFromInjectedNode();
			}
		}
		else if (!Injection->bInjectedAsInput && Injection->OutputPin == RootPin && Injection->InputPin)
		{
			TArray<URigVMPin*> LinkedPins = Injection->InputPin->GetLinkedSourcePins();
			if (LinkedPins.Num() == 1)
			{
				OriginalPin = LinkedPins[0]->GetOriginalPinFromInjectedNode();
			}
		}

		if (OriginalPin)
		{
			if (this != RootPin)
			{
				OriginalPin = OriginalPin->FindSubPin(GetSegmentPath());
			}
			return OriginalPin;
		}
	}

	return const_cast<URigVMPin*>(this);
}

const TArray<URigVMPin*>& URigVMPin::GetSubPins() const
{
	return SubPins;
}

TArray<URigVMPin*> URigVMPin::GetAllSubPinsRecursively() const
{
	TArray<URigVMPin*> AllSubPins;
	AllSubPins.Append(SubPins);
	for(const TObjectPtr<URigVMPin>& SubPin : SubPins)
	{
		AllSubPins.Append(SubPin->GetAllSubPinsRecursively());
	}
	return AllSubPins;
}

URigVMPin* URigVMPin::FindSubPinView(FStringView InPinPath) const
{
	FStringView Left, Right;
	if (!URigVMPin::SplitPinPathAtStart(InPinPath, Left, Right))
	{
		Left = InPinPath;
	}

	for (URigVMPin* Pin : SubPins)
	{
		if (Pin->NameEquals(Left, true))
		{
			if (Right.IsEmpty())
			{
				return Pin;
			}
			return Pin->FindSubPinView(Right);
		}
	}
	return nullptr;
}

URigVMPin* URigVMPin::FindSubPin(const FString& InPinPath) const
{
	return FindSubPinView(InPinPath);
}

bool URigVMPin::IsLinkedTo(const URigVMPin* InPin) const
{
	for (const URigVMLink* Link : Links)
	{
		if (Link->GetSourcePin() == InPin || Link->GetTargetPin() == InPin)
		{
			return true;
		}
	}
	return false;
}

bool URigVMPin::IsLinked(bool bRecursive) const
{
	if(!GetLinks().IsEmpty())
	{
		return true;
	}

	if(bRecursive)
	{
		for (const URigVMPin* SubPin : SubPins)
		{
			if(SubPin->IsLinked(true))
			{
				return true;
			}
		}
	}

	return false;
}

const TArray<URigVMLink*>& URigVMPin::GetLinks() const
{
	return Links;
}

TArray<URigVMPin*> URigVMPin::GetLinkedSourcePins(bool bRecursive) const
{
	TArray<URigVMPin*> Pins;
	for (URigVMLink* Link : Links)
	{
		if (Link->GetTargetPin() == this)
		{
			Pins.AddUnique(Link->GetSourcePin());
		}
	}

	if (bRecursive)
	{
		for (URigVMPin* SubPin : SubPins)
		{
			Pins.Append(SubPin->GetLinkedSourcePins(bRecursive));
		}
	}

	return Pins;
}

TArray<URigVMPin*> URigVMPin::GetLinkedTargetPins(bool bRecursive) const
{
	TArray<URigVMPin*> Pins;
	for (URigVMLink* Link : Links)
	{
		if (Link->GetSourcePin() == this)
		{
			Pins.AddUnique(Link->GetTargetPin());
		}
	}

	if (bRecursive)
	{
		for (URigVMPin* SubPin : SubPins)
		{
			Pins.Append(SubPin->GetLinkedTargetPins(bRecursive));
		}
	}

	return Pins;
}

TArray<URigVMLink*> URigVMPin::GetSourceLinks(bool bRecursive) const
{
	TArray<URigVMLink*> Results;
	if(GetDirection() == ERigVMPinDirection::IO ||
		GetDirection() == ERigVMPinDirection::Input)
	{
		for (URigVMLink* Link : Links)
		{
			if (Link->GetTargetPin() == this)
			{
				Results.Add(Link);
			}
		}

		if (bRecursive)
		{
			for (URigVMPin* SubPin : SubPins)
			{
				Results.Append(SubPin->GetSourceLinks(bRecursive));
			}
		}
	}
	return Results;
}

TArray<URigVMLink*> URigVMPin::GetTargetLinks(bool bRecursive) const
{
	TArray<URigVMLink*> Results;
	if(GetDirection() == ERigVMPinDirection::IO ||
		GetDirection() == ERigVMPinDirection::Output)
	{
		for (URigVMLink* Link : Links)
		{
			if (Link->GetSourcePin() == this)
			{
				Results.Add(Link);
			}
		}

		if (bRecursive)
		{
			for (URigVMPin* SubPin : SubPins)
			{
				Results.Append(SubPin->GetTargetLinks(bRecursive));
			}
		}
	}
	return Results;
}

URigVMNode* URigVMPin::GetNode() const
{
	URigVMPin* ParentPin = GetParentPin();
	if (ParentPin)
	{
		return ParentPin->GetNode();
	}

	URigVMNode* Node = Cast<URigVMNode>(GetOuter());
	if(IsValid(Node))
	{
		return Node;
	}

	return nullptr;
}

URigVMGraph* URigVMPin::GetGraph() const
{
	URigVMNode* Node = GetNode();
	if(IsValid(Node))
	{
		return Node->GetGraph();
	}

	return nullptr;
}

bool URigVMPin::CanLink(const URigVMPin* InSourcePin, const URigVMPin* InTargetPin, FString* OutFailureReason, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection, bool bInAllowNonArgumentPins, bool bEnableTypeCasting)
{
	if (InSourcePin == nullptr || InTargetPin == nullptr)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("One of the pins is nullptr.");
		}
		return false;
	}

	if (InSourcePin == InTargetPin)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source and target pins are the same.");
		}
		return false;
	}

	if(InSourcePin->ShouldOnlyShowSubPins() || InSourcePin->IsFixedSizeArray())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source pin only allows links to sub-pins.");
		}
		return false;
	}

	if(InTargetPin->ShouldOnlyShowSubPins() || InTargetPin->IsFixedSizeArray())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Target pin only allows links to sub-pins.");
		}
		return false;
	}

	if((InSourcePin->IsTraitPin() && InSourcePin->IsRootPin()) ||
		(InTargetPin->IsTraitPin() && InTargetPin->IsRootPin()))
	{
		if(OutFailureReason)
		{
			*OutFailureReason = TEXT("Cannot add link to root trait pins.");
		}
		return false;
	}

	URigVMNode* SourceNode = InSourcePin->GetNode();
	URigVMNode* TargetNode = InTargetPin->GetNode();
	if (SourceNode == TargetNode)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source and target pins are on the same node.");
		}
		return false;
	}

	if (InSourcePin->GetGraph() != InTargetPin->GetGraph())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source and target pins are in different graphs.");
		}
		return false;
	}

	if (InSourcePin->Direction != ERigVMPinDirection::Output &&
		InSourcePin->Direction != ERigVMPinDirection::IO)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source pin is not an output.");
		}
		return false;
	}

	if (InTargetPin->Direction != ERigVMPinDirection::Input &&
		InTargetPin->Direction != ERigVMPinDirection::IO)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Target pin is not an input.");
		}
		return false;
	}

	if (InSourcePin->CPPType != InTargetPin->CPPType)
	{
		bool bCPPTypesDiffer = true;
		static const FString Float = TEXT("float");
		static const FString Double = TEXT("double");

		if (FRigVMRegistry::Get().CanMatchTypes(InSourcePin->GetTypeIndex(), InTargetPin->GetTypeIndex(), true))
		{
			bCPPTypesDiffer = false;
		}

		if (bCPPTypesDiffer)
		{
			if(bEnableTypeCasting && RigVMTypeUtils::CanCastTypes(InSourcePin->GetTypeIndex(), InTargetPin->GetTypeIndex()))
			{
				bCPPTypesDiffer = false;
			}

			if(bCPPTypesDiffer)
			{
				auto TemplateNodeSupportsType = [](const URigVMPin* InPin, const int32& InTypeIndex, FString* OutFailureReason) -> bool
				{
					if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InPin->GetNode()))
					{
						if (TemplateNode->SupportsType(InPin, InTypeIndex))
						{
							if (OutFailureReason)
							{
								*OutFailureReason = FString();
							}
						}
						else
						{
							return false;
						}
					}
					return true;
				};

				if(InSourcePin->IsWildCard() && !InTargetPin->IsWildCard())
				{
					bCPPTypesDiffer = !TemplateNodeSupportsType(InSourcePin, InTargetPin->GetTypeIndex(), OutFailureReason);
				}
				else if(InTargetPin->IsWildCard() && !InSourcePin->IsWildCard())
				{
					bCPPTypesDiffer = !TemplateNodeSupportsType(InTargetPin, InSourcePin->GetTypeIndex(), OutFailureReason);
				}
				else if(InSourcePin->IsWildCard() && InTargetPin->IsWildCard())
				{
					// Find out if these pins have any type in common
					uint8 SourceLevels = 0;
					uint8 TargetLevels = 0;
					const URigVMPin* RootSourcePin = InSourcePin;
					const URigVMPin* RootTargetPin = InTargetPin;
					while (RootSourcePin->IsArrayElement())
					{
						SourceLevels++;
						RootSourcePin = RootSourcePin->GetParentPin();
					}
					while (RootTargetPin->IsArrayElement())
					{
						TargetLevels++;
						RootTargetPin = RootTargetPin->GetParentPin();
					}

					URigVMTemplateNode* SourceTemplateNode = Cast<URigVMTemplateNode>(RootSourcePin->GetNode());
					URigVMTemplateNode* TargetTemplateNode = Cast<URigVMTemplateNode>(RootTargetPin->GetNode());
					TArray<int32> SourcePermutations = SourceTemplateNode->GetResolvedPermutationIndices(true);
					TArray<int32> TargetPermutations = TargetTemplateNode->GetResolvedPermutationIndices(true);
					const FRigVMTemplate* SourceTemplate = SourceTemplateNode->GetTemplate();
					const FRigVMTemplate* TargetTemplate = TargetTemplateNode->GetTemplate();

					ensureMsgf(SourceTemplate != nullptr, TEXT("Source Template can not be resolved. Might have a pin with a type not registered in the RigVM Registry."));
					ensureMsgf(TargetTemplate != nullptr, TEXT("Target Template can not be resolved. Might have a pin with a type not registered in the RigVM Registry."));
					if (SourceTemplate == nullptr || TargetTemplate == nullptr)
					{
						if (OutFailureReason)
						{
							*OutFailureReason = TEXT("One of the templates can not be resolved. Might have a pin with a type not registered in the RigVM Registry.");
						}

						return false;
					}

					const FRigVMTemplateArgument* SourceRootArgument = SourceTemplate->FindArgument(RootSourcePin->GetFName());
					const FRigVMTemplateArgument* TargetRootArgument = TargetTemplate->FindArgument(RootTargetPin->GetFName());

					TArray<TRigVMTypeIndex> SourceTypes;
					FRigVMRegistry& Registry = FRigVMRegistry::Get();
					for (int32 Permutation : SourcePermutations)
					{
						TRigVMTypeIndex Type = SourceRootArgument->GetTypeIndex(Permutation);
						for (int32 i=0; i<SourceLevels; ++i)
						{
							check(Registry.IsArrayType(Type));
							Type = Registry.GetBaseTypeFromArrayTypeIndex(Type);
						}
						SourceTypes.Add(Type);
					}
					for (int32 Permutation : TargetPermutations)
					{
						TRigVMTypeIndex Type = TargetRootArgument->GetTypeIndex(Permutation);
						for (int32 i=0; i<TargetLevels; ++i)
						{
							check(Registry.IsArrayType(Type));
							Type = Registry.GetBaseTypeFromArrayTypeIndex(Type);
						}
						if (SourceTypes.Contains(Type))
						{
							bCPPTypesDiffer = false;
							break;
						}
					}
				}
			}
		}

		if (bCPPTypesDiffer)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = TEXT("Source and target pin types are not compatible.");

				const URigVMPin* TemplatePinToCheck = nullptr;
				switch(InUserLinkDirection)
				{
					case ERigVMPinDirection::Input:
					{
						TemplatePinToCheck = InSourcePin;
						break;
					}
					case ERigVMPinDirection::Output:
					{
						TemplatePinToCheck = InTargetPin;
						break;
					}
					default:
					{
						break;
					}
				}

				if(TemplatePinToCheck)
				{
					if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(TemplatePinToCheck->GetNode()))
					{
						if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
						{
							if(const FRigVMTemplateArgument* Argument = Template->FindArgument(TemplatePinToCheck->GetFName()))
							{
								const URigVMPin* OtherPin = TemplatePinToCheck == InSourcePin ? InTargetPin : InSourcePin;
								if(Argument->SupportsTypeIndex(OtherPin->GetTypeIndex()))
								{
									*OutFailureReason = TEXT("Link supported - please unresolve template node.");
								}
							}
						}
					}
				}
			}
			return false;
		}
	}

	if(!SourceNode->AllowsLinksOn(InSourcePin))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Node doesn't allow links on this pin.");
		}
		return false;
	}

	if(!TargetNode->AllowsLinksOn(InTargetPin))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Node doesn't allow links on this pin.");
		}
		return false;
	}

	if (InTargetPin->IsDefinedAsInputVariable())
	{
		if (!InSourcePin->IsDefinedAsInputVariable())
		{
			if (!InSourcePin->IsRootPin())
			{
				if (OutFailureReason)
				{
					*OutFailureReason = TEXT("Variable / Ref pins can only be connected to other root variable pins.");
				}
				return false;
			}

			const URigVMVariableNode* SourceVariableNode = Cast<URigVMVariableNode>(SourceNode);
			if (!SourceVariableNode)
			{
				if (OutFailureReason)
				{
					*OutFailureReason = TEXT("Variable / Ref pins can only be connected to other root variable pins.");
				}
				return false;
			}

			if (SourceVariableNode->GetValuePin() != InSourcePin)
			{
				if (OutFailureReason)
				{
					*OutFailureReason = TEXT("Variable / Ref pins can only be connected to other root variable pins.");
				}
				return false;
			}

			if (SourceVariableNode->IsInputArgument())
			{
				if (OutFailureReason)
				{
					*OutFailureReason = TEXT("Variable / Ref pins cannot be connected to input arguments.");
				}
				return false;
			}
		}
	}

	if (!bInAllowNonArgumentPins)
	{
		if (const URigVMTemplateNode* SourceTemplateNode = Cast<URigVMTemplateNode>(SourceNode))
		{
			if (!SourceNode->IsA<URigVMFunctionEntryNode>() && !SourceNode->IsA<URigVMFunctionReturnNode>())
			{
				if (const FRigVMTemplate* Template = SourceTemplateNode->GetTemplate())
				{
					const URigVMPin* RootPin = InSourcePin->GetRootPin();
					if(!RootPin->IsOrphanPin())
					{
						if (!Template->FindArgument(RootPin->GetFName()))
						{
							if(!RootPin->IsExecuteContext())
							{
								if (OutFailureReason)
								{
									*OutFailureReason = FString::Printf(TEXT("Library pin %s supported types need to be reduced."), *RootPin->GetPinPath(true));
								}
								return false;
							}
						}
					}
				}
			}
		}
		if (const URigVMTemplateNode* TargetTemplateNode = Cast<URigVMTemplateNode>(TargetNode))
		{
			if (!TargetNode->IsA<URigVMFunctionEntryNode>() && !TargetNode->IsA<URigVMFunctionReturnNode>())
			{
				if (const FRigVMTemplate* Template = TargetTemplateNode->GetTemplate())
				{
					const URigVMPin* RootPin = InTargetPin->GetRootPin();
					if(!RootPin->IsOrphanPin())
					{
						if (!Template->FindArgument(RootPin->GetFName()))
						{
							if(!RootPin->IsExecuteContext())
							{
								if (OutFailureReason)
								{
									*OutFailureReason = FString::Printf(TEXT("Library pin %s supported types need to be reduced."), *RootPin->GetPinPath(true));
								}
								return false;
							}
						}
					}
				}
			}
		}
	}

	// only allow to link to specified input / output pins on an injected node
	if (const URigVMInjectionInfo* SourceInjectionInfo = SourceNode->GetInjectionInfo())
	{
		if (SourceInjectionInfo->OutputPin != InSourcePin->GetRootPin())
		{
			if (OutFailureReason)
			{
				*OutFailureReason = TEXT("Cannot link to a non-exposed pin on an injected node.");
			}
			return false;
		}
	}

	// only allow to link to specified input / output pins on an injected node
	if (const URigVMInjectionInfo* TargetInjectionInfo = TargetNode->GetInjectionInfo())
	{
		if (TargetInjectionInfo->InputPin != InTargetPin->GetRootPin())
		{
			if (OutFailureReason)
			{
				*OutFailureReason = TEXT("Cannot link to a non-exposed pin on an injected node.");
			}
			return false;
		}
	}

	if (InSourcePin->IsLinkedTo(InTargetPin))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Source and target pins are already connected.");
		}
		return false;
	}

	TArray<URigVMNode*> SourceNodes;
	SourceNodes.Add(SourceNode);

	if (InByteCode)
	{
		int32 TargetNodeInstructionIndex = InByteCode->GetFirstInstructionIndexForSubject(TargetNode);
		if (TargetNodeInstructionIndex != INDEX_NONE)
		{
			for (int32 SourceNodeIndex = 0; SourceNodeIndex < SourceNodes.Num(); SourceNodeIndex++)
			{
				bool bNodeCanLinkAnywhere =
					SourceNodes[SourceNodeIndex]->IsA<URigVMRerouteNode>() ||
					SourceNodes[SourceNodeIndex]->IsA<URigVMVariableNode>();
				if (!bNodeCanLinkAnywhere)
				{
					// pure / immutable nodes can be connected to any input in any order.
					// since a new link is going to change the abstract syntax tree
					if (!SourceNodes[SourceNodeIndex]->IsMutable())
					{
						bNodeCanLinkAnywhere = true;
					}
				}

				if (!bNodeCanLinkAnywhere)
				{
					const int32 SourceNodeInstructionIndex = InByteCode->GetFirstInstructionIndexForSubject(SourceNodes[SourceNodeIndex]);
					if (SourceNodeInstructionIndex != INDEX_NONE &&
						SourceNodeInstructionIndex > TargetNodeInstructionIndex)
					{
						if (OutFailureReason)
						{
							static constexpr TCHAR IncorrectNodeOrderMessage[] = TEXT("Source node %s (%s) and target node %s (%s) are in the incorrect order.");
							*OutFailureReason = FString::Printf(
								IncorrectNodeOrderMessage,
								*SourceNodes[SourceNodeIndex]->GetName(),
								*SourceNodes[SourceNodeIndex]->GetNodeTitle(),
								*TargetNode->GetName(),
								*TargetNode->GetNodeTitle());
						}

						return false;
					}
					const TArray<URigVMNode*> LinkedSourceNodes = SourceNodes[SourceNodeIndex]->GetLinkedSourceNodes();
					for(URigVMNode* LinkedSourceNode : LinkedSourceNodes)
					{
						SourceNodes.AddUnique(LinkedSourceNode);
					}
				}
			}
		}
	}

	// In pure function graphs, certain sources cannot connect to IO pins.
	// Specifically: non-local variables and input parameters are rejected, as they would
	// allow the pure function to mutate external state. Local variables, constants, and
	// other function outputs are allowed since they are either function-scoped or immutable.
	if (const URigVMCollapseNode* OwningFunction = Cast<URigVMCollapseNode>(SourceNode->FindFunctionForNode()))
	{
		if (OwningFunction->IsPure() && !InSourcePin->IsExecuteContext() && !InTargetPin->IsExecuteContext())
		{
			const URigVMPin* SourcePin = InSourcePin;
			while (SourcePin->GetNode()->IsA<URigVMRerouteNode>())
			{
				if (URigVMRerouteNode* RereouteNode = Cast<URigVMRerouteNode>(SourcePin->GetNode()))
				{
					TArray<URigVMPin*> SourcePins = RereouteNode->FindPin(URigVMRerouteNode::ValueName)->GetLinkedSourcePins();
					if (SourcePins.Num() == 1)
					{
						SourcePin = SourcePins[0];
					}
					else
					{
						break;
					}
				}
			}
			SourceNode = SourcePin->GetNode();
			
			TArray<const URigVMPin*> PinsToVisit = {InTargetPin};
			TArray<const URigVMPin*> TargetPins;
			while (!PinsToVisit.IsEmpty())
			{
				const URigVMPin* PinToVisit = PinsToVisit.Pop();
				if (URigVMRerouteNode* RereouteNode = Cast<URigVMRerouteNode>(PinToVisit->GetNode()))
				{
					PinsToVisit.Append(RereouteNode->FindPin(URigVMRerouteNode::ValueName)->GetLinkedTargetPins());
				}
				else
				{
					TargetPins.Add(PinToVisit);
				}
			}
			
			for (const URigVMPin* TargetPin : TargetPins)
			{
				TargetNode = TargetPin->GetNode();

				// If the target pin is const, the connection is fine.
				// However, whether this target is an IO pin or an input variable pin — both represent
				// mutable references that a pure function must not be allowed to write through.
				const bool bIsIOPin = (TargetPin->GetDirection() == ERigVMPinDirection::IO);
				const bool bIsInputVariablePin = TargetPin->IsDefinedAsInputVariable();
				const bool bIsConstPin = TargetPin->IsDefinedAsConstant();
				
				if (bIsConstPin)
				{
					continue;
				}

				if (bIsIOPin || bIsInputVariablePin)
				{
					// Build a human-readable label for the target pin kind used in failure messages
					const TCHAR* TargetPinKind = bIsInputVariablePin ? TEXT("input variable") : TEXT("IO");

					// Local variable getters are allowed — they're function-scoped and
					// do not expose external state, so mutation is safe.
					if (const URigVMVariableNode* VarNode = Cast<URigVMVariableNode>(SourcePin->GetNode()))
					{
						if (VarNode->IsGetter() && !VarNode->IsLocalVariable())
						{
							if (OutFailureReason)
							{
								*OutFailureReason = FString::Printf(
									TEXT("Only local variables can connect to %s pin '%s' in a pure function %s. Non-local data cannot be modified."),
									TargetPinKind, *TargetPin->GetPinPath(), *OwningFunction->GetName());
							}

							return false;
						}
					}

					// Entry node pins (input parameters) connected to IO / input variable pins
					// would allow the pure function to mutate caller-owned data.
					if (SourceNode->IsA<URigVMFunctionEntryNode>())
					{
						if (OutFailureReason)
						{
							*OutFailureReason = FString::Printf(
								TEXT("Input parameter '%s' cannot connect to %s pin '%s' in a pure function %s. This would allow modification of the input."),
								*SourcePin->GetName(), TargetPinKind, *TargetPin->GetPinPath(), *OwningFunction->GetName());
						}
						return false;
					}
				}
			}
		}
	}

	return true;
}

bool URigVMPin::HasInjectedUnitNodes() const
{
	for (URigVMInjectionInfo* Info : InjectionInfos)
	{
		if (Info->Node.IsA<URigVMUnitNode>())
		{
			return true;
		}
	}
	
	return false;
}

