// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMSchema.h"
#include "RigVMStringUtils.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMSchema)

TAutoConsoleVariable<bool> CVarRigVMEnableNodeLayouts(TEXT("RigVM.EnableNodeLayouts"), false, TEXT("Set to true to turn on support for node layouts"));

URigVMSchema::URigVMSchema()
	: ExecuteContextStruct(nullptr)
	, Registry(&FRigVMRegistry::Get())
{
	SetExecuteContextStruct(FRigVMExecuteContext::StaticStruct());
}

URigVMSchema::URigVMSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ExecuteContextStruct(nullptr)
	, Registry(&FRigVMRegistry::Get())
{
	SetExecuteContextStruct(FRigVMExecuteContext::StaticStruct());
}

UClass* URigVMSchema::GetEdGraphSchemaClass() const
{
	return URigVMEdGraphSchema::StaticClass();
}

UClass* URigVMSchema::GetRuntimeAssetClass() const
{
	return URigVMRuntimeAsset::StaticClass();
}

UClass* URigVMSchema::GetEditorAssetClass() const
{
	return URigVMEditorAsset::StaticClass();
}

bool URigVMSchema::SupportsType(URigVMController* InController, TRigVMTypeIndex InTypeIndex) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsType_NoLock(InController, InTypeIndex, Lock);
}

bool URigVMSchema::SupportsType_NoLock(URigVMController* InController, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const
{
	// filter out incompatible execute types
	if(InRegistry->IsExecuteType_NoLock(InTypeIndex))
	{
		const UStruct* Struct = CastChecked<UStruct>(InRegistry->GetType_NoLock(InTypeIndex).CPPTypeObject);
		if(!ValidExecuteContextStructs.Contains(Struct))
		{
			if (InController)
			{
				static constexpr TCHAR Format[] = TEXT("ExecuteContext struct '%s' is not supported.");
				InController->ReportErrorf(Format, *Struct->GetName());
			}
			return false;
		}
	}
	return true;
}

bool URigVMSchema::SupportsUnitFunction(URigVMController* InController, const FRigVMFunction* InUnitFunction) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsUnitFunction_NoLock(InController, InUnitFunction, Lock);
}

bool URigVMSchema::SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const
{
	if(InUnitFunction)
	{
		if(const UScriptStruct* FunctionExecuteContextStruct = InUnitFunction->GetExecuteContextStruct_NoLock(InRegistry))
		{
			const TRigVMTypeIndex TypeIndex = InRegistry->GetTypeIndexFromCPPType_NoLock(FunctionExecuteContextStruct->GetStructCPPName());
			if(!SupportsType_NoLock(InController, TypeIndex, InRegistry))
			{
				return false;
			}
		}
		
		const TArray<TRigVMTypeIndex>& ArgumentTypes = InUnitFunction->GetArgumentTypeIndices_NoLock(InRegistry);
		for(const TRigVMTypeIndex& ArgumentType : ArgumentTypes)
		{
			if(!SupportsType_NoLock(InController, ArgumentType, InRegistry))
			{
				if (InController)
				{
					const FString CPPTypeString = InRegistry->GetType_NoLock(ArgumentType).CPPType.ToString();
					static constexpr TCHAR Format[] = TEXT("Unit function '%s' is not supported since type '%s' is not supported.");
					InController->ReportErrorf(Format, *InUnitFunction->GetName(), *CPPTypeString);
				}
				return false;
			}
		}
		return true;
	}

	return false;
}

bool URigVMSchema::SupportsDispatchFactory(URigVMController* InController, const FRigVMDispatchFactory* InDispatchFactory) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsDispatchFactory_NoLock(InController, InDispatchFactory, Lock);
}

bool URigVMSchema::SupportsDispatchFactory_NoLock(URigVMController* InController, const FRigVMDispatchFactory* InDispatchFactory, FRigVMRegistryHandle& InRegistry) const
{
	if(InDispatchFactory)
	{
		if(const UScriptStruct* DispatchExecuteContextStruct = InDispatchFactory->GetExecuteContextStruct())
		{
			const TRigVMTypeIndex TypeIndex = InRegistry->GetTypeIndexFromCPPType_NoLock(DispatchExecuteContextStruct->GetStructCPPName());
			if(!SupportsType_NoLock(InController, TypeIndex, InRegistry))
			{
				return false;
			}
		}

		if(const FRigVMTemplate* Template = InDispatchFactory->GetTemplate())
		{
			if(!SupportsTemplate_NoLock(InController, Template, InRegistry))
			{
				return false;
			}
		}
		return true;
	}

	return false;
}

bool URigVMSchema::SupportsTemplate(URigVMController* InController, const FRigVMTemplate* InTemplate) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsTemplate_NoLock(InController, InTemplate, Lock);
}

bool URigVMSchema::SupportsTemplate_NoLock(URigVMController* InController, const FRigVMTemplate* InTemplate, FRigVMRegistryHandle& InRegistry) const
{
	if(InTemplate)
	{
		// we are only checking for execute arguments here to make sure that we don't
		// support access to templates that cannot be compiled for the given graph.
		// normal template arguments (non-execute) will be allowed - but during resolval
		// we'll check if the target types are supported by the graph.
		static const FRigVMDispatchContext DispatchContext;
		for(int32 Index = 0; Index < InTemplate->NumExecuteArguments(DispatchContext); Index++)
		{
			const TRigVMTypeIndex& TypeIndex = InTemplate->GetExecuteArgument(Index, DispatchContext)->TypeIndex;
			if(!SupportsType_NoLock(InController, TypeIndex, InRegistry))
			{
				if (InController)
				{
					const FString CPPTypeString = InRegistry->GetType_NoLock(TypeIndex).CPPType.ToString();
					static constexpr TCHAR Format[] = TEXT("Template '%s' is not supported since type '%s' is not supported.");
					InController->ReportErrorf(Format, *InTemplate->GetNotation().ToString(), *CPPTypeString);
				}
				return false;
			}
		}
		return true;
	}

	return false;
}

bool URigVMSchema::SupportsGraphFunction(URigVMController* InController, const FRigVMGraphFunctionHeader* InGraphFunction) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsGraphFunction_NoLock(InController, InGraphFunction, Lock);
}

bool URigVMSchema::SupportsGraphFunction_NoLock(URigVMController* InController, const FRigVMGraphFunctionHeader* InGraphFunction, FRigVMRegistryHandle& InRegistry) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	check(InGraphFunction);

	if(Graph->IsA<URigVMFunctionLibrary>())
	{
		if (InController)
		{
			static const FString Message = TEXT("You cannot place functions inside of the top level function library graph.");
			InController->ReportError(Message);
		}
		return false;
	}

	// Make sure all the argument types are supported before creating the node
	for (const FRigVMGraphFunctionArgument& Argument : InGraphFunction->Arguments)
	{
		TRigVMTypeIndex Type = InRegistry->GetTypeIndexFromCPPType_NoLock(Argument.CPPType.ToString());
		if (Type == INDEX_NONE)
		{
			if (Argument.IsCPPTypeObjectValid())
			{
				FRigVMTemplateArgumentType ArgumentType(Argument.CPPType, Argument.CPPTypeObject.Get());
				Type = InRegistry->FindOrAddType_NoLock(ArgumentType);
			}
		}
		
		if (Type == INDEX_NONE)
		{
			if (InController)
			{
				InController->ReportErrorf(TEXT("Cannot add function reference to %s because argument %s has invalid type %s."), *InGraphFunction->Name.ToString(), *Argument.Name.ToString(), *Argument.CPPType.ToString());
			}
			return false;
		}
	}

	bool bAllowPrivateFunctions = true;
	if (InController)
	{
		bAllowPrivateFunctions = InController->bAllowPrivateFunctions;
	}

	if (!bAllowPrivateFunctions)
	{
		if(IRigVMClientHost* ClientHost = Graph->GetImplementingOuter<IRigVMClientHost>())
		{
			bool bIsAvailable = ClientHost->GetRigVMClient()->GetFunctionLibrary()->GetFunctionHostObjectPath() ==
				InGraphFunction->LibraryPointer.HostObject;
			if (!bIsAvailable)
			{
				if (IRigVMGraphFunctionHost* Host = Cast<IRigVMGraphFunctionHost>(InGraphFunction->LibraryPointer.HostObject.TryLoad()))
				{
					bIsAvailable = Host->GetRigVMGraphFunctionStore()->IsFunctionPublic(InGraphFunction->LibraryPointer);		
				}
			}
			if (!bIsAvailable)
			{
				if (InController)
				{
					InController->ReportError(TEXT("Function is not available for placement in another graph host."));
				}
				return false;
			}
		}
	}

	if (const URigVMNode* Node = Cast<URigVMNode>(Graph->GetOuter()))
	{
		if (const URigVMLibraryNode* LibraryNode = Node->FindFunctionForNode())
		{
			if (InGraphFunction->Dependencies.Contains(LibraryNode->GetFunctionIdentifier()))
			{
				if (InController)
				{
					static const FString Message = TEXT("Function is not available for placement in this graph host due to dependency cycles."); 
					InController->ReportError(Message);
				}
				return false;
			}
		}
	}
	
	URigVMLibraryNode* ParentLibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	while (ParentLibraryNode)
	{
		if (TSoftObjectPtr<UObject>(ParentLibraryNode).ToSoftObjectPath() == InGraphFunction->LibraryPointer.GetNodeSoftPath())
		{
			if (InController)
			{
				static const FString Message = TEXT("You cannot place functions inside of itself or an indirect recursion.");
				InController->ReportError(Message);
			}
			return false;
		}
		ParentLibraryNode = Cast<URigVMLibraryNode>(ParentLibraryNode->GetGraph()->GetOuter());
	}

	return true;
}

bool URigVMSchema::SupportsExternalVariable(URigVMController* InController, const FRigVMExternalVariable* InExternalVariable) const
{
	check(Registry);
	FRigVMRegistryWriteLock Lock(*Registry);
	return SupportsExternalVariable_NoLock(InController, InExternalVariable, Lock);
}

bool URigVMSchema::SupportsExternalVariable_NoLock(URigVMController* InController, const FRigVMExternalVariable* InExternalVariable, FRigVMRegistryHandle& InRegistry) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if(InExternalVariable)
	{
		TRigVMTypeIndex Type = InRegistry->GetTypeIndex_NoLock(InExternalVariable->GetExtendedCPPType(), const_cast<UObject*>(InExternalVariable->GetCPPTypeObject()));
		if(Type == INDEX_NONE)
		{
			return false;
		}
		return SupportsType_NoLock(InController, Type, InRegistry);
	}

	return false;
}

bool URigVMSchema::SupportsNodeLayouts(const URigVMGraph* InGraph) const
{
	return CVarRigVMEnableNodeLayouts.GetValueOnAnyThread();
}

bool URigVMSchema::ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (InStruct == nullptr)
	{
		return false;
	}
	if (InStruct->IsChildOf(UClass::StaticClass()))
	{
		return false;
	}
	if(InStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
	{
		return false;
	}
	if(InStruct->IsChildOf(RigVMTypeUtils::GetWildCardCPPTypeObject()))
	{
		return false;
	}

	return true;
}

bool URigVMSchema::IsValidNodeName(const URigVMGraph* InGraph, const FName& InNodeName) const
{
	return InGraph->IsNameAvailable(InNodeName.ToString());
}

bool URigVMSchema::CanAddNode(URigVMController* InController, const URigVMNode* InNode) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if(InNode == nullptr)
	{
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		if(!CanAddFunction(InController, InNode))
		{
			return false;
		}
	}

	if (const URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InNode))
	{
		FRigVMGraphFunctionHeader FunctionDefinition = FunctionRefNode->GetReferencedFunctionHeader();

		bool bSupportsGraphFunction;
		{
			TGuardValue<bool> GuardErrorReporting(InController->bReportWarningsAndErrors, false);
			bSupportsGraphFunction = SupportsGraphFunction(InController, &FunctionDefinition);
		}
		if(!bSupportsGraphFunction)
		{
			URigVMFunctionLibrary* TargetLibrary = Graph->GetDefaultFunctionLibrary();
			URigVMLibraryNode* LocalizedFunctionDefinition = TargetLibrary->FindPreviouslyLocalizedFunction(FunctionDefinition.LibraryPointer);

			if((LocalizedFunctionDefinition == nullptr) && InController->RequestLocalizeFunctionDelegate.IsBound())
			{
				if(InController->RequestLocalizeFunctionDelegate.Execute(FunctionDefinition.LibraryPointer))
				{
					LocalizedFunctionDefinition = TargetLibrary->FindPreviouslyLocalizedFunction(FunctionDefinition.LibraryPointer);
				}
			}

			if(LocalizedFunctionDefinition == nullptr)
			{
				return false;
			}
		
			InController->SetReferencedFunction(const_cast<URigVMFunctionReferenceNode*>(FunctionRefNode), LocalizedFunctionDefinition, false);
			FunctionDefinition = FunctionRefNode->GetReferencedFunctionHeader();

			if(!SupportsGraphFunction(InController, &FunctionDefinition))
			{
				return false;
			}
		}
	}
	else if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		// only allow entry / return nodes on sub graphs
		if(Graph->IsRootGraph())
		{
			static const FString Message("Entry and Return nodes can only be added to sub graphs.");
			InController->ReportError(Message);
			return false;
		}

		// only allow one function entry node
		if(InNode->IsA<URigVMFunctionEntryNode>())
		{
			if(const URigVMFunctionEntryNode* ExistingEntryNode = Graph->GetEntryNode())
			{
				if(ExistingEntryNode != InNode)
				{
					static const FString Message("Graphs can only contain on Entry node.");
					InController->ReportError(Message);
					return false;
				}
			}
		}
		// only allow one function return node
		else if(InNode->IsA<URigVMFunctionReturnNode>())
		{
			if(const URigVMFunctionReturnNode* ExistingReturnNode = Graph->GetReturnNode())
			{
				if(ExistingReturnNode != InNode)
				{
					static const FString Message("Graphs can only contain on Return node.");
					InController->ReportError(Message);
					return false;
				}
			}
		}
	}
	else if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InNode))
	{
		URigVMGraph* ContainedGraph = CollapseNode->GetContainedGraph();
		const TArray<URigVMNode*> ContainedNodes = CollapseNode->GetContainedNodes();
		URigVMController* ContainedController = InController->GetControllerForGraph(ContainedGraph);
		TGuardValue<bool> GuardEditGraph(ContainedGraph->bEditable, true);
		for(const URigVMNode* ContainedNode : ContainedNodes)
		{
			if(!CanAddNode(ContainedController, ContainedNode))
			{
				return false;
			}
		}
	}
	else if(const URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(InNode))
	{
		if (const URigVMPin* NamePin = VariableNode->FindPin(URigVMVariableNode::VariableName))
		{
			const FString VarNameString = NamePin->GetDefaultValue();
			if (!VarNameString.IsEmpty())
			{
				const FName VarName = *VarNameString;

				TArray<FRigVMExternalVariable> AllVariables = InController->GetAllVariables(true);
				for(const FRigVMExternalVariable& Variable : AllVariables)
				{
					if(Variable.GetName().IsEqual(VarName, ENameCase::CaseSensitive))
					{
						return true;
					}
				}
				return false;
			}
		}
	}
	else if (InNode->IsEvent())
	{
		if (const URigVMUnitNode* InUnitNode = Cast<URigVMUnitNode>(InNode))
		{
			if(const UScriptStruct* EventStruct = InUnitNode->GetScriptStruct())
			{
				// check if we're trying to add a node within a graph which is not the top level one
				if (!Graph->IsTopLevelGraph())
				{
					static const FString Message = TEXT("Event nodes can only be added to top level graphs."); 
					InController->ReportError(Message);
					return false;
				}

				if (InController->GetAllEventNames().Contains(InUnitNode->GetEventName()))
				{
					static const FString Message = FString::Printf(TEXT("An event named %s already exists."), *InUnitNode->GetEventName().ToString()); 
					InController->ReportError(Message);
					return false;
				}

				const TObjectPtr<URigVMNode> EventNode = FindEventNode(InController, EventStruct);
				const bool bHasEventNode = (EventNode != nullptr) && EventNode->CanOnlyExistOnce();
				if (bHasEventNode)
				{
					const FString ErrorMessage = FString::Printf(TEXT("Rig Graph can only contain one single %s node."),
																 *EventStruct->GetDisplayNameText().ToString());
					InController->ReportError(ErrorMessage);
					return false;
				}
			}
		}
	}
	
	return true;
}

bool URigVMSchema::CanRemoveNode(URigVMController* InController, const URigVMNode* InNode) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if(InNode->IsA<URigVMFunctionInterfaceNode>())
	{
		return false;
	}

	return true;
}

bool URigVMSchema::CanRenameNode(URigVMController* InController, const URigVMNode* InNode, const FName& InNewNodeName) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if(Graph)
	{
		if(InNode)
		{
			if(InNode->GetName() != InNewNodeName)
			{
				return IsValidNodeName(Graph, InNewNodeName);
			}
		}
	}

	return false;
}

bool URigVMSchema::CanMoveNode(URigVMController* InController, const URigVMNode* InNode, const FVector2D& InNewPosition) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if ((InNode->Position - InNewPosition).IsNearlyZero())
	{
		return false;
	}

	return true;
}

bool URigVMSchema::CanResizeNode(URigVMController* InController, const URigVMNode* InNode, const FVector2D& InNewSize) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if ((InNode->Size - InNewSize).IsNearlyZero())
	{
		return false;
	}

	return InNode->IsA<URigVMCommentNode>();
}

bool URigVMSchema::CanSetNodeTitle(URigVMController* InController, const URigVMNode* InNode) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InNode))
	{
		if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
		{
			if(const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory())
			{
				return Factory->SupportsRenaming();
			}
		}
	}
	
	return false;
}

bool URigVMSchema::CanRecolorNode(URigVMController* InController, const URigVMNode* InNode, const FLinearColor& InNewColor) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (InNode->NodeColor.Equals(InNewColor, 0.001f))
	{
		return false;
	}

	return InNode->IsA<URigVMCommentNode>() || InNode->IsA<URigVMLibraryNode>();
}

bool URigVMSchema::CanAddLink(URigVMController* InController, const URigVMPin* InSourcePin, const URigVMPin* InTargetPin, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection, bool bInAllowWildcard, bool bEnableTypeCasting, FString* OutFailureReason) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		static const FString FailureReason = TEXT("Cannot add links in function library graphs."); 
		if(OutFailureReason)
		{
			*OutFailureReason = FailureReason;
		}		
		return false;
	}

	if(InSourcePin == nullptr)
	{
		static const FString FailureReason = TEXT("SourcePin is nullptr."); 
		if(OutFailureReason)
		{
			*OutFailureReason = FailureReason;
		}		
		return false;
	}

	if(InTargetPin == nullptr)
	{
		static const FString FailureReason = TEXT("TargetPin is nullptr."); 
		if(OutFailureReason)
		{
			*OutFailureReason = FailureReason;
		}		
		return false;
	}

	if((InSourcePin->GetGraph() != Graph) || (InTargetPin->GetGraph() != Graph))
	{
		static const FString FailureReason = TEXT("Pin is not valid for graph."); 
		if(OutFailureReason)
		{
			*OutFailureReason = FailureReason;
		}		
		return false;
	}

	if(!URigVMPin::CanLink(InSourcePin, InTargetPin, OutFailureReason, InByteCode, InUserLinkDirection, bInAllowWildcard))
	{
		return false;
	}

	return true;
}

bool URigVMSchema::CanBreakLink(URigVMController* InController, const URigVMPin* InSourcePin, const URigVMPin* InTargetPin) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		static const FString FailureReason = TEXT("Cannot break links in function library graphs."); 
		InController->ReportError(FailureReason);
		return false;
	}

	if(InSourcePin == nullptr)
	{
		static const FString FailureReason = TEXT("SourcePin is nullptr."); 
		InController->ReportError(FailureReason);
		return false;
	}

	if(InTargetPin == nullptr)
	{
		static const FString FailureReason = TEXT("TargetPin is nullptr."); 
		InController->ReportError(FailureReason);
		return false;
	}

	if((InSourcePin->GetGraph() != Graph) || (InTargetPin->GetGraph() != Graph))
	{
		static const FString FailureReason = TEXT("Pin is not valid for graph."); 
		InController->ReportError(FailureReason);
		return false;
	}

	if (!InSourcePin->IsLinkedTo(InTargetPin))
	{
		return false;
	}
	ensure(InTargetPin->IsLinkedTo(InSourcePin));
	
	return true;
}

bool URigVMSchema::CanCollapseNodes(URigVMController* InController, const TArrayView<URigVMNode* const>& InNodesToCollapse) const
{
	if(InNodesToCollapse.IsEmpty())
	{
		return false;
	}

	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	for(int32 Index = 0; Index < InNodesToCollapse.Num(); Index++)
	{
		if(InNodesToCollapse[Index]->GetGraph() != Graph)
		{
			static const FString Message = TEXT("You can only collapse nodes within the same graph.");
			InController->ReportError(Message);
			return false;
		}
	}

	return true;
}

bool URigVMSchema::CanExpandNode(URigVMController* InController, const URigVMNode* InNodeToExpand) const
{
	check(InNodeToExpand);

	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if(Graph->IsA<URigVMFunctionLibrary>())
	{
		static const FString Message = TEXT("You cannot expand subgraphs within a function library graph.");
		InController->ReportError(Message);
		return false;
	}

	return true;
}

bool URigVMSchema::CanUnfoldPin(URigVMController* InController, const URigVMPin* InPinToUnfold) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (InPinToUnfold->ShouldHideSubPins())
	{
		return false;
	}
	if (InPinToUnfold->IsStruct())
	{
		return ShouldUnfoldStruct(InController, InPinToUnfold->GetScriptStruct());
	}
	if (InPinToUnfold->IsArray())
	{
		return InPinToUnfold->GetDirection() == ERigVMPinDirection::Input ||
			InPinToUnfold->GetDirection() == ERigVMPinDirection::IO ||
			InPinToUnfold->IsFixedSizeArray();
	}
	return false;
}

bool URigVMSchema::CanBindVariable(URigVMController* InController, const URigVMPin* InPinToBind, const FRigVMExternalVariable* InVariableToBind, const FString& InNewBoundVariablePath) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		InController->ReportError(TEXT("Cannot bind pins to variables in function library graphs."));
		return false;
	}

	if (InPinToBind->GetBoundVariablePath() == InNewBoundVariablePath)
	{
		return false;
	}

	if (InPinToBind->GetDirection() != ERigVMPinDirection::Input)
	{
		InController->ReportError(TEXT("Variables can only be bound to input pins."));
		return false;
	}

	if (!InPinToBind->IsRootPin())
	{
		InController->ReportError(TEXT("Variables can only be bound to root pins."));
		return false;
	}

	if(!SupportsType(InController, InVariableToBind->GetTypeIndex()))
	{
		return false;
	}

	return true;
}

bool URigVMSchema::CanUnbindVariable(URigVMController* InController, const URigVMPin* InBoundPin) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	check(InBoundPin);

	if (Graph->IsA<URigVMFunctionLibrary>())
	{
		InController->ReportError(TEXT("Cannot unbind pins from variables in function library graphs."));
		return false;
	}
	if (!InBoundPin->IsBoundToVariable())
	{
		InController->ReportError(TEXT("Pin is not bound to any variable."));
		return false;
	}

	return true;
}

bool URigVMSchema::CanAddFunction(URigVMController* InController, const URigVMNode* InFunctionNode) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		InController->ReportError(TEXT("Can only add function definitions to function library graphs."));
		return false;
	}

	if(InFunctionNode)
	{
		if (!InFunctionNode->IsA<URigVMCollapseNode>())
		{
			return false;
		}
	}

	return true;
}

bool URigVMSchema::CanRemoveFunction(URigVMController* InController, const URigVMNode* InFunctionNode) const
{
	RIGVMSCHEMA_DEFAULT_FUNCTION_BODY

	if (!Graph)
	{
		return false;
	}

	check(InFunctionNode);

	if (!Graph->IsA<URigVMFunctionLibrary>())
	{
		InController->ReportError(TEXT("Can only remove function definitions from function library graphs."));
		return false;
	}

	return true;
}

FString URigVMSchema::GetSanitizedName(const FString& InName, bool bAllowPeriod, bool bAllowSpace) const
{
	FString CopiedName = InName;
	SanitizeName(CopiedName, bAllowPeriod, bAllowSpace);
	return CopiedName;
}

FString URigVMSchema::GetSanitizedGraphName(const FString& InName) const
{
	return GetSanitizedName(InName, true, true);
}

FString URigVMSchema::GetSanitizedNodeName(const FString& InName) const
{
	return GetSanitizedName(InName, false, true);
}

FString URigVMSchema::GetSanitizedVariableName(const FString& InName) const
{
	return GetSanitizedName(InName, false, true);
}

FString URigVMSchema::GetSanitizedPinName(const FString& InName) const
{
	return GetSanitizedName(InName, false, true);
}

FString URigVMSchema::GetSanitizedPinPath(const FString& InName) const
{
	return GetSanitizedName(InName, true, true);
}

FString URigVMSchema::GetGraphOuterName(const URigVMGraph* InGraph) const
{
	check(InGraph);
	return GetSanitizedName(InGraph->GetRootGraph()->GetOuter()->GetFName().ToString(), true, false);
}

FString URigVMSchema::GetValidNodeName(const URigVMGraph* InGraph, const FString& InPrefix) const
{
	check(InGraph);

	return GetUniqueName(*InPrefix, [&](const FName& InName) {
		return InGraph->IsNameAvailable(InName);
	}, false, true).ToString();
}

void URigVMSchema::SanitizeName(FString& InOutName, bool bAllowPeriod, bool bAllowSpace)
{
	RigVMStringUtils::SanitizeName(InOutName, bAllowPeriod, bAllowSpace, GetMaxNameLength());
}

FName URigVMSchema::GetUniqueName(const FName& InName, TFunction<bool(const FName&)> IsNameAvailablePredicate,
	bool bAllowPeriod, bool bAllowSpace)
{
	FString SanitizedPrefix = InName.ToString();
	SanitizeName(SanitizedPrefix, bAllowPeriod, bAllowSpace);

	static constexpr int32 InitialSuffix = 2; // offset since FName uses 0 as the indicator that there's no number suffix, plus we want to start with index 1. 
	int32 NameSuffix = InitialSuffix; 
	FName Name = *SanitizedPrefix;

	while (!IsNameAvailablePredicate(Name))
	{
		if(NameSuffix == InitialSuffix)
		{
			Name = FName(Name, NameSuffix);
		}
		else
		{
			Name.SetNumber(NameSuffix);
		}
		NameSuffix++;
	}
	return Name;
}

void URigVMSchema::SetExecuteContextStruct(UScriptStruct* InExecuteContextStruct)
{
	if(InExecuteContextStruct != ExecuteContextStruct)
	{
		if(GetClass() == URigVMSchema::StaticClass() && HasAnyFlags(RF_ClassDefaultObject))
		{
			// only allow the default execute context on the base class.
			// please create a child class of URigVMSchema for your use case.
			verify(InExecuteContextStruct == FRigVMExecuteContext::StaticStruct());
		}
		
		ExecuteContextStruct = InExecuteContextStruct;
		ValidExecuteContextStructs.Reset();
		
		if(ExecuteContextStruct)
		{
			ValidExecuteContextStructs = FRigVMTemplate::GetSuperStructs(ExecuteContextStruct, true);
		}
	}
}

bool URigVMSchema::IsGraphEditable(const URigVMGraph* InGraph) const
{
	if(InGraph)
	{
		return InGraph->bEditable;
	}
	return false;
}

TObjectPtr<URigVMNode> URigVMSchema::FindEventNode(URigVMController* InController, const UScriptStruct* InScriptStruct) const
{
	const URigVMGraph* Graph = InController->GetGraph();
	check(Graph);
	check(InScriptStruct);

	if (Graph)
	{
		// construct equivalent default struct
		FStructOnScope InDefaultStructScope(InScriptStruct);

		const TObjectPtr<URigVMNode>* FoundNode = 
		Graph->Nodes.FindByPredicate( [&InDefaultStructScope](const TObjectPtr<URigVMNode>& Node) {
			if (Node->IsEvent())
			{
				if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
				{
					// compare default structures
					TSharedPtr<FStructOnScope> DefaultStructScope = UnitNode->ConstructStructInstance(true);
					if (DefaultStructScope.IsValid() && InDefaultStructScope.GetStruct() == DefaultStructScope->GetStruct())
					{
						return true;
					}
				}
			}
			return false;
		});

		if (FoundNode)
		{
			return *FoundNode;
		}
	}
	
	return TObjectPtr<URigVMNode>();
}
