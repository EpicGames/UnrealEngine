// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMModel/RigVMTraitDefaultValueStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMTraitDefaultValueStruct)

const TCHAR* FRigVMTraitDefaultValueStruct::DefaultValuePropertyName = TEXT("DefaultValue");

void FRigVMTraitDefaultValueStruct::Init(UScriptStruct* InTraitScriptStruct)
{
	PropertyBag.AddProperty(DefaultValuePropertyName, EPropertyBagPropertyType::Struct, InTraitScriptStruct);	
}

void FRigVMTraitDefaultValueStruct::SetValue(const FString& InDefaultValue)
{
	PropertyBag.SetValueSerializedString(DefaultValuePropertyName, InDefaultValue);	
}

FString FRigVMTraitDefaultValueStruct::GetValue() const
{
	TValueOrError<FString, EPropertyBagResult> TraitDefaultValue = PropertyBag.GetValueSerializedString(DefaultValuePropertyName);
	
	check(TraitDefaultValue.HasValue())
	return TraitDefaultValue.GetValue();
}
