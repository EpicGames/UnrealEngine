// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMPythonUtils.h"
#include "Internationalization/BreakIterator.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"
#include "IMessageLogListing.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMDeveloperModule"


FString RigVMPythonUtils::PythonizeName(FStringView InName, const RigVMPythonUtils::EPythonizeNameCase InNameCase)
{
	// Wish we could use PyGenUtil::PythonizeName, but unfortunately it's private
	
	static const TSet<FString> ReservedKeywords = {
		TEXT("and"),
		TEXT("as"),
		TEXT("assert"),
		TEXT("async"),
		TEXT("break"),
		TEXT("class"),
		TEXT("continue"),
		TEXT("def"),
		TEXT("del"),
		TEXT("elif"),
		TEXT("else"),
		TEXT("except"),
		TEXT("finally"),
		TEXT("for"),
		TEXT("from"),
		TEXT("global"),
		TEXT("if"),
		TEXT("import"),
		TEXT("in"),
		TEXT("is"),
		TEXT("lambda"),
		TEXT("nonlocal"),
		TEXT("not"),
		TEXT("or"),
		TEXT("pass"),
		TEXT("raise"),
		TEXT("return"),
		TEXT("try"),
		TEXT("while"),
		TEXT("with"),
		TEXT("yield"),
		TEXT("property"),
	};

	// Remove spaces
	FString Name = InName.GetData();
	Name.ReplaceCharInline(' ', '_');

	FString PythonizedName;
	PythonizedName.Reserve(Name.Len() + 10);

	static TSharedPtr<IBreakIterator> NameBreakIterator;
	if (!NameBreakIterator.IsValid())
	{
		NameBreakIterator = FBreakIterator::CreateCamelCaseBreakIterator();
	}

	NameBreakIterator->SetStringRef(Name);
	for (int32 PrevBreak = 0, NameBreak = NameBreakIterator->MoveToNext(); NameBreak != INDEX_NONE; NameBreak = NameBreakIterator->MoveToNext())
	{
		const int32 OrigPythonizedNameLen = PythonizedName.Len();

		// Append an underscore if this was a break between two parts of the identifier, *and* the previous character isn't already an underscore
		if (OrigPythonizedNameLen > 0 && PythonizedName[OrigPythonizedNameLen - 1] != TEXT('_'))
		{
			PythonizedName += TEXT('_');
		}

		// Append this part of the identifier
		PythonizedName.AppendChars(&Name[PrevBreak], NameBreak - PrevBreak);

		// Remove any trailing underscores in the last part of the identifier
		while (PythonizedName.Len() > OrigPythonizedNameLen)
		{
			const int32 CharIndex = PythonizedName.Len() - 1;
			if (PythonizedName[CharIndex] != TEXT('_'))
			{
				break;
			}
			PythonizedName.RemoveAt(CharIndex, EAllowShrinking::No);
		}

		PrevBreak = NameBreak;
	}
	NameBreakIterator->ClearString();

	if (InNameCase == EPythonizeNameCase::Lower)
	{
		PythonizedName.ToLowerInline();
	}
	else if (InNameCase == EPythonizeNameCase::Upper)
	{
		PythonizedName.ToUpperInline();
	}

	// Don't allow the name to conflict with a keyword
	if (ReservedKeywords.Contains(PythonizedName))
	{
		PythonizedName += TEXT('_');
	}

	return PythonizedName;
}

FString RigVMPythonUtils::TransformToPythonString(const FTransform& Transform)
{
	static constexpr TCHAR TransformFormat[] = TEXT("unreal.Transform(location=[%f,%f,%f],rotation=[%f,%f,%f],scale=[%f,%f,%f])");
	return FString::Printf(TransformFormat,
	                       Transform.GetLocation().X,
	                       Transform.GetLocation().Y,
	                       Transform.GetLocation().Z,
	                       Transform.Rotator().Pitch,
	                       Transform.Rotator().Yaw,
	                       Transform.Rotator().Roll,
	                       Transform.GetScale3D().X,
	                       Transform.GetScale3D().Y,
	                       Transform.GetScale3D().Z);
}

FString RigVMPythonUtils::Vector2DToPythonString(const FVector2D& Vector)
{
	static constexpr TCHAR Vector2DFormat[] = TEXT("unreal.Vector2D(%f, %f)");
	return FString::Printf(Vector2DFormat,
	                       Vector.X,
	                       Vector.Y);
}

FString RigVMPythonUtils::LinearColorToPythonString(const FLinearColor& Color)
{
	static constexpr TCHAR LinearColorFormat[] = TEXT("unreal.LinearColor(%f, %f, %f, %f)");
	return FString::Printf(LinearColorFormat,
	                       Color.R, Color.G, Color.B, Color.A);
}

FString RigVMPythonUtils::EnumValueToPythonString(UEnum* Enum, int64 Value)
{
	static constexpr TCHAR EnumPrefix[] = TEXT("E");
	static constexpr TCHAR EnumValueFormat[] = TEXT("unreal.%s.%s");

	FString EnumName = Enum->GetName();
	EnumName.RemoveFromStart(EnumPrefix, ESearchCase::CaseSensitive);
	
	return FString::Printf(
		EnumValueFormat,
		*EnumName,
		*PythonizeName(Enum->GetNameStringByValue((int64)Value), EPythonizeNameCase::Upper)
	);
}

#if WITH_EDITOR
void RigVMPythonUtils::Print(const FString& BlueprintTitle, const FString& InMessage)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	if (!MessageLogModule.IsRegisteredLogListing("ControlRigPythonLog"))
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = true;
		InitOptions.bShowPages = true;
		InitOptions.bAllowClear = true;
		InitOptions.bScrollToBottom = true;
		MessageLogModule.RegisterLogListing("ControlRigPythonLog", LOCTEXT("ControlRigPythonLog", "Control Rig Python Log"), InitOptions);
	}
	TSharedRef<IMessageLogListing> PythonLog = MessageLogModule.GetLogListing( TEXT("ControlRigPythonLog") );
	PythonLog->SetCurrentPage(FText::FromString(BlueprintTitle));

	TSharedRef<FTokenizedMessage> Token = FTokenizedMessage::Create(EMessageSeverity::Info, FText::FromString(InMessage));
	PythonLog->AddMessage(Token, false);
}

void RigVMPythonUtils::PrintPythonContext(const FString& InBlueprintPath)
{
	FString BlueprintName = InBlueprintPath;
	int32 DotIndex = BlueprintName.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (DotIndex != INDEX_NONE)
	{
		BlueprintName = BlueprintName.Right(BlueprintName.Len() - DotIndex - 1);
	}

	static constexpr TCHAR LoadObjectFormat[] = TEXT("blueprint = unreal.load_object(name = '%s', outer = None)");
	static constexpr TCHAR DefineFunctionLibraryFormat[] = TEXT("library = blueprint.get_local_function_library()");
	static constexpr TCHAR DefineLibraryControllerFormat[] = TEXT("library_controller = blueprint.get_controller(library)");
	static constexpr TCHAR DefineHierarchyFormat[] = TEXT("hierarchy = blueprint.hierarchy");
	static constexpr TCHAR DefineHierarchyControllerFormat[] = TEXT("hierarchy_controller = hierarchy.get_controller()");
		
	TArray<FString> PyCommands = {
		TEXT("import unreal"),
		FString::Printf(LoadObjectFormat, *InBlueprintPath),
		DefineFunctionLibraryFormat,
		DefineLibraryControllerFormat,
		DefineHierarchyFormat,
		DefineHierarchyControllerFormat
	};

	for (FString& Command : PyCommands)
	{
		RigVMPythonUtils::Print(BlueprintName, Command);
	}
}

#undef LOCTEXT_NAMESPACE

#endif
