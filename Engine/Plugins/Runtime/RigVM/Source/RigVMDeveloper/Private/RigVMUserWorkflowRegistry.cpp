// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMUserWorkflowRegistry.h"
#include "RigVMModel/Nodes/RigVMTemplateNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMUserWorkflowRegistry)

URigVMUserWorkflowRegistry* URigVMUserWorkflowRegistry::Get()
{
	return StaticClass()->GetDefaultObject<URigVMUserWorkflowRegistry>();
}

int32 URigVMUserWorkflowRegistry::RegisterProvider(const UScriptStruct* InStruct, FRigVMUserWorkflowProvider InProvider)
{
	const int32 Handle = ++MaxHandle;
	ProvidersForUnits.Emplace(Handle, InStruct, InProvider);
	return Handle;
}

int32 URigVMUserWorkflowRegistry::RegisterProviderForFunction(const FString InFunctionNotation, FRigVMUserWorkflowProvider InProvider)
{
	const int32 Handle = ++MaxHandle;
	ProvidersForDispatchFunctions.Emplace(Handle, InFunctionNotation, InProvider);
	return Handle;
}

void URigVMUserWorkflowRegistry::UnregisterProvider(int32 InHandle)
{
	int32 NumRemoved = ProvidersForUnits.RemoveAll([InHandle](const TTuple<int32, const UScriptStruct*, FRigVMUserWorkflowProvider>& Provider) -> bool
		{
			return Provider.Get<0>() == InHandle;
		});
	if (NumRemoved == 0)
	{
		ProvidersForDispatchFunctions.RemoveAll([InHandle](const TTuple<int32, const FString, FRigVMUserWorkflowProvider>& Provider) -> bool
		{
			return Provider.Get<0>() == InHandle;
		});
	}
}

TArray<FRigVMUserWorkflow> URigVMUserWorkflowRegistry::GetWorkflows(ERigVMUserWorkflowType InType, const UScriptStruct* InStruct, const UObject* InSubject) const
{
	TArray<FRigVMUserWorkflow> Workflows;

	// remove stale delegates
	ProvidersForUnits.RemoveAll([](const TTuple<int32, const UScriptStruct*, FRigVMUserWorkflowProvider>& Provider) -> bool
	{
		return !Provider.Get<2>().IsBound();
	});
	ProvidersForDispatchFunctions.RemoveAll([](const TTuple<int32, const FString, FRigVMUserWorkflowProvider>& Provider) -> bool
	{
		return !Provider.Get<2>().IsBound();
	});
	

	if (InStruct)
	{
		for(const TTuple<int32, const UScriptStruct*, FRigVMUserWorkflowProvider>& Provider : ProvidersForUnits)
		{
			if(Provider.Get<1>() == InStruct || Provider.Get<1>() == nullptr)
			{
				Workflows.Append(Provider.Get<2>().Execute(InSubject));
			}
		}
	}
	else if (const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(InSubject))
	{
		if (const FRigVMFunction* Function = TemplateNode->GetResolvedFunction())
		{
			const FString FunctionName = Function->GetName();
			for(const TTuple<int32, const FString, FRigVMUserWorkflowProvider>& Provider : ProvidersForDispatchFunctions)
			{
				if(Provider.Get<1>() == FunctionName)
				{
					Workflows.Append(Provider.Get<2>().Execute(InSubject));
				}
			}
		}
	}

	Workflows = Workflows.FilterByPredicate([InType](const FRigVMUserWorkflow& InWorkflow) -> bool
	{
		return uint32(InWorkflow.GetType()) & uint32(InType) &&
			InWorkflow.IsValid();
	});

	return Workflows;
}

