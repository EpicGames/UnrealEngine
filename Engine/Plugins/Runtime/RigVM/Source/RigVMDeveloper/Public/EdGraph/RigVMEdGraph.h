// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMEditorAsset.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCore/RigVM.h"
#include "RigVMModel/RigVMClient.h"
#include "Input/Events.h"
#include "RigVMEdGraph.generated.h"

#define UE_API RIGVMDEVELOPER_API

class IRigVMEditorAssetInterface;
class URigVMEdGraphSchema;
class URigVMController;
class URigVMBlueprint;

struct FRigVMStringTag
{
public:
	
	FRigVMStringTag()
		: Name(NAME_None)
		, Color(FLinearColor::Red)
	{}

	explicit FRigVMStringTag(const FName& InName, const FLinearColor& InColor)
		: Name(InName)
		, Color(InColor)
	{
	}

	const FName& GetName() const
	{
		return Name;
	}
	
	const FLinearColor& GetColor() const
	{
		return Color;
	}

	bool IsValid() const
	{
		return !Name.IsNone();
	}

	bool Equals(const FName& InOther) const
	{
		return GetName().IsEqual(InOther, ENameCase::CaseSensitive);
	}

	bool Equals(const FRigVMStringTag& InOther) const
	{
		return Equals(InOther.GetName());
	}

private:
	FName Name;
	FLinearColor Color;
};

struct FRigVMStringWithTag
{
public:
	
	FRigVMStringWithTag() = default;

	FRigVMStringWithTag(const FString& InString, const FRigVMStringTag& InTag = FRigVMStringTag())
		: String(InString)
		, Tag(InTag)
	{
	}

	const FString& GetString() const
	{
		return String;
	}

	FString GetStringWithTag() const
	{
		if(HasTag())
		{
			static constexpr TCHAR Format[] = TEXT("%s (%s)");
			return FString::Printf(Format, *GetString(), *GetTag().GetName().ToString());
		}
		return GetString();
	}

	bool HasTag() const
	{
		return Tag.IsValid();
	}

	const FRigVMStringTag& GetTag() const
	{
		return Tag;
	}

	bool operator ==(const FRigVMStringWithTag& InOther) const
	{
		return Equals(InOther);
	}

	bool operator >(const FRigVMStringWithTag& InOther) const
	{
		return GetString() > InOther.GetString();
	}

	bool operator <(const FRigVMStringWithTag& InOther) const
	{
		return GetString() < InOther.GetString();
	}

	bool Equals(const FString& InOther) const
	{
		return GetString().Equals(InOther, ESearchCase::CaseSensitive);
	}

	bool Equals(const FRigVMStringWithTag& InOther) const
	{
		return Equals(InOther.GetString());
	}

private:
	FString String;
	FRigVMStringTag Tag;
};

DECLARE_MULTICAST_DELEGATE_ThreeParams(FRigVMEdGraphNodeClicked, URigVMEdGraphNode*, const FGeometry&, const FPointerEvent&);

UCLASS(MinimalAPI)
class URigVMEdGraph : public UEdGraph, public IRigVMEditorSideObject
{
	GENERATED_BODY()

public:
	UE_API URigVMEdGraph();

	/** IRigVMEditorSideObject interface */
	UE_API virtual FRigVMClient* GetRigVMClient() const override;
	UE_API virtual FString GetRigVMNodePath() const override;
	UE_API virtual void HandleRigVMGraphRenamed(const FString& InOldNodePath, const FString& InNewNodePath) override;

	/** Set up this graph */
	UE_DEPRECATED(5.7, "Please use const FRigVMEditorAssetInterfacePtr GetAssetDefaultObject() const")
	UE_API const URigVMBlueprint* GetBlueprintDefaultObject() const;
	UE_API const FRigVMEditorAssetInterfacePtr GetAssetDefaultObject() const;
	UE_API void SetBlueprintClass(const UClass* InClass);
	UE_DEPRECATED(5.7, "Please use void InitializeFromAsset(FRigVMEditorAssetInterfacePtr InAsset);")
	virtual void InitializeFromBlueprint(URigVMBlueprint* InBlueprint) {}
	UE_API virtual void InitializeFromAsset(FRigVMEditorAssetInterfacePtr InAsset);
	UE_API bool IsPreviewGraph() const;

	/** Get the ed graph schema */
	UE_API const URigVMEdGraphSchema* GetRigVMEdGraphSchema();

#if WITH_EDITORONLY_DATA
	/** Customize blueprint changes based on backwards compatibility */
	UE_API virtual void Serialize(FArchive& Ar) override;
#endif

#if WITH_EDITOR

	bool bSuspendModelNotifications;
	bool bIsTemporaryGraphForCopyPaste;
	bool bIsSelecting;

	UE_API UEdGraphNode* FindNodeForModelNodeName(const FName& InModelNodeName, const bool bCacheIfRequired = true);

	UE_DEPRECATED(5.7, "Please use FRigVMEditorAssetInterfacePtr GetAsset() const")
	UE_API URigVMBlueprint* GetBlueprint() const;
	UE_API FRigVMEditorAssetInterfacePtr GetAsset() const;
	UE_API URigVMGraph* GetModel() const;
	UE_API URigVMController* GetController() const;
	bool IsRootGraph() const { return GetRootGraph() == this; }
	UE_API const URigVMEdGraph* GetRootGraph() const;

	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);
	UE_API virtual bool HandleModifiedEvent_Internal(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	UE_API int32 GetInstructionIndex(const URigVMEdGraphNode* InNode, bool bAsInput);

	UE_API void CacheEntryNameList();
	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetEntryNameList(URigVMPin* InPin = nullptr) const;

	virtual const TArray<TSharedPtr<FRigVMStringWithTag>>* GetNameListForWidget(const FString& InWidgetName) const { return nullptr; }

	UE_API void AddLocalVariableSearchMetaDataInfo(const FName InVariableName, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const;

	UPROPERTY()
	FString ModelNodePath;

	UPROPERTY()
	bool bIsFunctionDefinition;

protected:
	using Super::AddNode;
	UE_API virtual void AddNode(UEdGraphNode* NodeToAdd, bool bUserAction = false, bool bSelectNewNode = true) override;

private:

	FRigVMEdGraphNodeClicked OnGraphNodeClicked;

	TMap<URigVMNode*, TPair<int32, int32>> CachedInstructionIndices;

	UE_API void RemoveNode(UEdGraphNode* InNode);

protected:
	UE_API void HandleVMCompiledEvent(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext);

private:
	TMap<FName, UEdGraphNode*> ModelNodePathToEdNode;
	mutable TWeakObjectPtr<URigVMGraph> CachedModelGraph;
	TArray<TSharedPtr<FRigVMStringWithTag>> EntryNameList;
	const UClass* RigVMBlueprintClass;

#endif
	friend class URigVMEdGraphNode;
	friend class IRigVMEditorAssetInterface;
	friend class FRigVMEditorBase;
	friend class SRigVMGraphNode;

	friend class URigVMEdGraphNodeSpawner;
	friend class URigVMEdGraphUnitNodeSpawner;
	friend class URigVMEdGraphVariableNodeSpawner;
	friend class URigVMEdGraphParameterNodeSpawner;
	friend class URigVMEdGraphBranchNodeSpawner;
	friend class URigVMEdGraphIfNodeSpawner;
	friend class URigVMEdGraphSelectNodeSpawner;
	friend class URigVMEdGraphTemplateNodeSpawner;
	friend class URigVMEdGraphEnumNodeSpawner;
	friend class URigVMEdGraphFunctionRefNodeSpawner;
	friend class URigVMEdGraphArrayNodeSpawner;
	friend class URigVMEdGraphInvokeEntryNodeSpawner;
};

#undef UE_API
