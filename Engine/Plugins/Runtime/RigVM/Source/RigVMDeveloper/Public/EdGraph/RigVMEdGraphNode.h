// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "RigVMEditorAsset.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Blueprint/BlueprintExtension.h"
#include "RigVMEdGraphNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class IRigVMEditorAssetInterface;
class URigVMBlueprint;
class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
struct FSlateIcon;

/** Base class for RigVM editor side nodes */
UCLASS(MinimalAPI)
class URigVMEdGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

	friend class FRigVMEdGraphNodeDetailsCustomization;
	friend class FRigVMBlueprintCompilerContext;
	friend class URigVMEdGraph;
	friend class URigVMEdGraphSchema;
	friend class URigVMBlueprint;
	friend class FRigVMEdGraphTraverser;
	friend class FRigVMEdGraphPanelPinFactory;
	friend class FRigVMEditorBase;
	friend struct FRigVMBlueprintUtils;
	friend class SRigVMEdGraphPinCurveFloat;

private:

	UPROPERTY()
	FString ModelNodePath;

	UPROPERTY(transient)
	TWeakObjectPtr<URigVMNode> CachedModelNode;

	UPROPERTY(transient)
	TMap<FString, TWeakObjectPtr<URigVMPin>> PinPathToModelPin;

#if WITH_EDITORONLY_DATA
	/** The property we represent. For template nodes this represents the struct/property type name. */
	UPROPERTY()
	FName PropertyName_DEPRECATED;

	UPROPERTY()
	FString StructPath_DEPRECATED;

	/** Pin Type for property */
	UPROPERTY()
	FEdGraphPinType PinType_DEPRECATED;

	/** The type of parameter */
	UPROPERTY()
	int32 ParameterType_DEPRECATED;

	/** Expanded pins */
	UPROPERTY()
	TArray<FString> ExpandedPins_DEPRECATED;
#endif

	/** Cached dimensions of this node (used for auto-layout) */
	FVector2D Dimensions;

	/** The cached node titles */
	mutable FText NodeTitle;

	/** The cached fulol node title */
	mutable FText FullNodeTitle;

	/** Set this to true to enable the sub title */
	bool bSubTitleEnabled;

public:

	void SetSubTitleEnabled(bool bEnabled = true)
	{
		bSubTitleEnabled = bEnabled;
	}

	DECLARE_MULTICAST_DELEGATE(FNodeTitleDirtied);
	DECLARE_MULTICAST_DELEGATE(FNodePinsChanged);
	DECLARE_MULTICAST_DELEGATE(FNodePinExpansionChanged);
	DECLARE_MULTICAST_DELEGATE(FNodeBeginRemoval);

	struct FPinPair
	{
		FPinPair()
			: InputPin(nullptr)
			, OutputPin(nullptr)
		{}
		UEdGraphPin* InputPin;
		UEdGraphPin* OutputPin;

		bool IsValid() const { return InputPin != nullptr || OutputPin != nullptr; }
	};

	UE_API URigVMEdGraphNode();

	// UObject Interface.
#if WITH_EDITOR
	virtual bool Modify( bool bAlwaysMarkDirty=true ) { return false; }
#endif
	
	// UEdGraphNode Interface.
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FLinearColor GetNodeBodyTintColor() const;
	UE_API virtual bool ShowPaletteIconOnNode() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void ReconstructNode() override;
	UE_API virtual void ReconstructNode_Internal(bool bForce = false);
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* InSchema) const override;
	UE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;	
	virtual bool SupportsCommentBubble() const override { return false; }
	UE_API virtual bool IsSelectedInEditor() const;
	UE_API virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override;
	UE_API virtual void BeginDestroy() override;
#if WITH_EDITOR
#if WITH_RIGVMLEGACYEDITOR
	UE_API virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const override;
#else
	UE_API void AddRigVMSearchMetaDataInfo(TArray<struct UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const;
	UE_API void AddRigVMPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const;
#endif
#endif
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	UE_API void RequestRename(float InDelay = 0.f);

	UE_API virtual bool IsDeprecated() const override;
	UE_API bool IsOutDated() const;
	UE_API virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;

	/** Set the cached dimensions of this node */
	void SetDimensions(const FVector2D& InDimensions) { Dimensions = InDimensions; }

	/** Get the cached dimensions of this node */
	const FVector2D& GetDimensions() const { return Dimensions; }

	/** Check a pin's expansion state */
	UE_API bool IsPinExpanded(const FString& InPinPath);

	/** Propagate pin defaults to underlying properties if they have changed */
	UE_API void CopyPinDefaultsToModel(UEdGraphPin* Pin, bool bUndo = false, bool bPrintPythonCommand = false);

	/** Get the blueprint that this node is contained within */
	UE_DEPRECATED(5.7, "Please use FRigVMEditorAssetInterfacePtr GetAsset() const")
	UE_API URigVMBlueprint* GetBlueprint() const;
	UE_API FRigVMEditorAssetInterfacePtr GetAsset() const;

	/** Get the VM model this node lives within */
	UE_API URigVMGraph* GetModel() const;

	/** Get the blueprint that this node is contained within */
	UE_API URigVMController* GetController() const;

	/** Get the VM node this is node is wrapping */
	UE_API URigVMNode* GetModelNode() const;

	/** Get the VM node name this node is wrapping */
	UE_API FName GetModelNodeName() const;

	/** Get the VM node path this node is wrapping */
	UE_API const FString& GetModelNodePath() const;

	UE_API URigVMPin* GetModelPinFromPinPath(const FString& InPinPath) const;

	/** Add a new element to the aggregate node referred to by the property path */
	UE_API void HandleAddAggregateElement(const FString& InNodePath);

	/** Add a new array element to the array referred to by the property path */
	UE_API void HandleAddArrayElement(FString InPinPath);
	
	/** Clear the array referred to by the property path */
	UE_API void HandleClearArray(FString InPinPath);

	/** Remove the array element referred to by the property path */
	UE_API void HandleRemoveArrayElement(FString InPinPath);

	/** Insert a new array element after the element referred to by the property path */
	UE_API void HandleInsertArrayElement(FString InPinPath);

	UE_API int32 GetInstructionIndex(bool bAsInput) const;
	UE_API bool IsExcludedFromPreview() const;

	UE_API const FRigVMTemplate* GetTemplate() const;

	UE_API void ClearErrorInfo();

	UE_API void AddErrorInfo(const EMessageSeverity::Type& InSeverity, const FString& InMessage);

	UE_API void SetErrorInfo(const EMessageSeverity::Type& InSeverity, const FString& InMessage);

	UE_API URigVMPin* FindModelPinFromGraphPin(const UEdGraphPin* InGraphPin) const;
	UE_API UEdGraphPin* FindGraphPinFromModelPin(const URigVMPin* InModelPin, bool bAsInput) const;
	UE_API UEdGraphPin* FindGraphPinFromCategory(const FString& InCategory, bool bAsInput) const;

	/// Synchronize the stored name/value/type on the graph pin with the value stored on the node. 
	/// If the pin has sub-pins, the value update is done recursively.
	UE_API void SynchronizeGraphPinNameWithModelPin(const URigVMPin* InModelPin, bool bNotify = true);
	UE_API void SynchronizeGraphPinValueWithModelPin(const URigVMPin* InModelPin);
	UE_API void SynchronizeGraphPinTypeWithModelPin(const URigVMPin* InModelPin);
	UE_API void SynchronizeGraphPinExpansionWithModelPin(const URigVMPin* InModelPin);
	
	UE_API void SyncGraphNodeTitleWithModelNodeTitle();
	UE_API void SyncGraphNodeNameWithModelNodeName(const URigVMNode* InModelNode);

	FNodeTitleDirtied& OnNodeTitleDirtied() { return NodeTitleDirtied; }
	FNodePinsChanged& OnNodePinsChanged() { return NodePinsChanged; }
	FNodePinExpansionChanged& OnNodePinExpansionChanged() { return NodePinExpansionChanged; }
	FNodeBeginRemoval& OnNodeBeginRemoval() { return NodeBeginRemoval; }

	/** Called when there's a drastic change in the pins */
	UE_API bool ModelPinsChanged(bool bForce = false);

	/** Called when a model pin is added after the node creation */
	UE_API bool ModelPinAdded(const URigVMPin* InModelPin);

	/** Called when a model pin is being removed */
	UE_API bool ModelPinRemoved(const URigVMPin* InModelPin);

	/** Returns true if this node is relying on the cast template */
	UE_API bool DrawAsCompactNode() const;

	/** Sets the node's content from an external client, used for preview nodes without a graph */
	UE_API void SetModelNode(URigVMNode* InModelNode);

	/** Returns all input pins */
	const TArray<URigVMPin*>& GetInputPins() const { return InputPins; }

	/** Returns all execute pins */
	const TArray<URigVMPin*>& GetExecutePins() const { return ExecutePins; };

	// Returns true if this Node should be faded out
	UE_API bool IsFadedOut() const;

	// Returns 1.0 if the node is 100% opaque, and 0.0 if the node isn't visible.
	UE_API float GetFadedOutState() const;

	// Overrides the fade out state of this node
	UE_API void OverrideFadeOutState(TOptional<float> InFadedOutState);

	// Removes any user based fade out state
	void ResetFadedOutState() { OverrideFadeOutState({}); }

protected:

	/** Helper function for AllocateDefaultPins */
	UE_API void UpdatePinLists();
	UE_API bool CreateGraphPinFromCategory(const FString& InCategory, EEdGraphPinDirection InDirection);
	UE_API bool CreateGraphPinFromModelPin(const URigVMPin* InModelPin, EEdGraphPinDirection InDirection,  UEdGraphPin* InParentPin = nullptr);
	UE_API void RemoveGraphSubPins(UEdGraphPin *InParentPin, const TArray<UEdGraphPin*>& InPinsToKeep = TArray<UEdGraphPin*>());
	UE_API bool ModelPinAdded_Internal(const URigVMPin* InModelPin);
	UE_API bool ModelPinRemoved_Internal(const URigVMPin* InModelPin);
	UE_API bool CategoryPinAdded_Internal(const FString& InCategory, EEdGraphPinDirection InDirection);
	UE_API bool CategoryPinRemoved_Internal(const FString& InCategory);

	/** Copies default values from underlying properties into pin defaults, for editing */
	UE_API void SetupPinDefaultsFromModel(UEdGraphPin* Pin, const URigVMPin* InModelPin = nullptr);

	/** Recreate pins when we reconstruct this node */
	UE_API virtual void ReallocatePinsDuringReconstruction(const TArray<UEdGraphPin*>& OldPins);

	/** Wire-up new pins given old pin wiring */
	UE_API virtual void RewireOldPinsToNewPins(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins);

	/** Handle anything post-reconstruction */
	UE_API virtual void PostReconstructNode();

	/** Something that could change our title has changed */
	UE_API void InvalidateNodeTitle() const;

	/** Something that could change our color has changed */
	UE_API void InvalidateNodeColor();

	/** Destroy all pins in an array */
	UE_API void DestroyPinList(TArray<UEdGraphPin*>& InPins);

	/** Sets the body + title color from a color provided by the model */
	UE_API void SetColorFromModel(const FLinearColor& InColor);

	static UE_API FEdGraphPinType GetPinTypeForModelPin(const URigVMPin* InModelPin);
	static UE_API FEdGraphPinType GetPinTypeForCategoryPin();

	UE_API virtual void ConfigurePin(UEdGraphPin* EdGraphPin, const URigVMPin* ModelPin) const;
private:

	int32 GetNodeTopologyVersion() const { return NodeTopologyVersion; }
	int32 NodeTopologyVersion;

	UE_API TArray<URigVMPin*>& PinListForPin(const URigVMPin* InModelPin);

	FLinearColor CachedTitleColor;
	FLinearColor CachedNodeColor;

#if WITH_EDITOR
	UE_API void UpdateVisualSettings();
	
	TOptional<bool> bIsHighlighted;
	TOptional<FLinearColor> ProfilingColor;
	TOptional<FLinearColor> HighlightingColor;
	TOptional<bool> bEnableProfiling;
	mutable TOptional<double> MicroSeconds;
	mutable TArray<double> MicroSecondsFrames;
#endif

	TArray<URigVMPin*> ExecutePins;
	TArray<URigVMPin*> InputOutputPins;
	TArray<URigVMPin*> InputPins;
	TArray<URigVMPin*> OutputPins;
	TArray<TSharedPtr<FRigVMExternalVariable>> ExternalVariables;
	TArray<UEdGraphPin*> LastEdGraphPins;
	
	mutable TMap<TWeakObjectPtr<URigVMPin>, FPinPair> CachedPins;
	TMap<FString, FPinPair> CachedCategoryPins;

	FNodeTitleDirtied NodeTitleDirtied;
	FNodePinsChanged NodePinsChanged;
	FNodePinExpansionChanged NodePinExpansionChanged;
	FNodeBeginRemoval NodeBeginRemoval;

	TSet<uint32> ErrorMessageHashes;

	mutable const FRigVMTemplate* CachedTemplate;
	mutable TOptional<bool> DrawAsCompactNodeCache;
	FSimpleDelegate RequestRenameDelegate;
	mutable bool bRenameIsPending;

	mutable TOptional<bool> bIsFadedOut;
	mutable TOptional<float> FadedOutOverride;

	friend class SRigVMGraphNode;
	friend class FRigVMFunctionArgumentLayout;
	friend class FRigVMEdGraphDetailCustomization;
	friend class URigVMEdGraphTemplateNodeSpawner;
	friend class URigVMEdGraphArrayNodeSpawner;
	friend class URigVMEdGraphIfNodeSpawner;
	friend class URigVMEdGraphSelectNodeSpawner;
};

class FRigVMGraphNodeNameValidator : public INameValidatorInterface
{
public:
	FRigVMGraphNodeNameValidator(const URigVMController* InController);

	// Begin FNameValidatorInterface
	virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override;
	virtual EValidatorResult IsValid(const FName& Name, bool bOriginal) override;
	// End FNameValidatorInterface

private:

	const URigVMController* Controller;
};

#undef UE_API
