// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMEditorAsset.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditorDragDropAction.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCore/RigVMVariant.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Kismet2/Kismet2NameValidators.h"

#include "RigVMEdGraphSchema.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMEdGraph;
class URigVMEdGraphNode;

/**
 *Const values for Find-in-Blueprints to tag searchable data
 */
struct FRigVMSearchTags
{
	// /** Begin const values for Find-in-Blueprint */
	
	/** Name tag */
	static UE_API const FText FiB_Name;
	/** Class Name tag */
	static UE_API const FText FiB_ClassName;
	/** NodeGuid tag */
	static UE_API const FText FiB_NodeGuid;
	
	// /** Pin type tags */
	/** Pin Category tag */
	static UE_API const FText FiB_PinCategory;
	/** Pin Sub-Category tag */
	static UE_API const FText FiB_PinSubCategory;
	/** Pin Binding tag */
	static UE_API const FText FiB_PinBinding;
	/** Pin object class tag */
	static UE_API const FText FiB_ObjectClass;
	/** Pin IsArray tag */
	static UE_API const FText FiB_IsArray;
	/** Glyph icon tag */
	static UE_API const FText FiB_Glyph;
	/** Style set the glyph belongs to */
	static UE_API const FText FiB_GlyphStyleSet;
	/** Glyph icon color tag */
	static UE_API const FText FiB_GlyphColor;
	
	// /** End const values for Find-in-Blueprint */
};

/** Extra operations that can be performed on pin connection */
enum class ECanCreateConnectionResponse_Extended
{
	None,

	BreakChildren,

	BreakParent,
};

/** Struct used to extend the response to a connection request to include breaking parents/children */
struct FRigVMPinConnectionResponse
{
	FRigVMPinConnectionResponse(const ECanCreateConnectionResponse InResponse, FText InMessage, ECanCreateConnectionResponse_Extended InExtendedResponse = ECanCreateConnectionResponse_Extended::None)
		: Response(InResponse, MoveTemp(InMessage))
		, ExtendedResponse(InExtendedResponse)
	{
	}

	friend bool operator==(const FRigVMPinConnectionResponse& A, const FRigVMPinConnectionResponse& B)
	{
		return (A.Response == B.Response) && (A.ExtendedResponse == B.ExtendedResponse);
	}	

	FPinConnectionResponse Response;
	ECanCreateConnectionResponse_Extended ExtendedResponse;
};

/////////////////////////////////////////////////////
// FRigVMLocalVariableNameValidator
class FRigVMLocalVariableNameValidator : public FStringSetNameValidator
{

public:
	UE_API FRigVMLocalVariableNameValidator(const FRigVMEditorAssetInterfacePtr Blueprint, const URigVMGraph* Graph, FName InExistingName = NAME_None);

	// Begin FNameValidatorInterface
	UE_API virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override;
	UE_API virtual EValidatorResult IsValid(const FName& Name, bool bOriginal) override;
	// End FNameValidatorInterface
};

/////////////////////////////////////////////////////
// FRigVMNameValidator
class FRigVMNameValidator : public FStringSetNameValidator
{

public:
	UE_API FRigVMNameValidator(const class UBlueprint* Blueprint, const UStruct* ValidationScope, FName InExistingName = NAME_None);
	UE_API FRigVMNameValidator(const FRigVMEditorAssetInterfacePtr Blueprint, const UStruct* ValidationScope, FName InExistingName = NAME_None);

	// Begin FNameValidatorInterface
	UE_API virtual EValidatorResult IsValid(const FString& Name, bool bOriginal) override;
	UE_API virtual EValidatorResult IsValid(const FName& Name, bool bOriginal) override;
	// End FNameValidatorInterface
};

USTRUCT()
struct FRigVMEdGraphSchemaAction_LocalVar : public FEdGraphSchemaAction_BlueprintVariableBase
{
	GENERATED_BODY()

	
public:

	// Simple type info
	static FName StaticGetTypeId()
	{
		static const FLazyName Type("FRigVMEdGraphSchemaAction_LocalVar");
		return Type;
	}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FRigVMEdGraphSchemaAction_LocalVar()
		: FEdGraphSchemaAction_BlueprintVariableBase()
	{}

	FRigVMEdGraphSchemaAction_LocalVar(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID)
		: FEdGraphSchemaAction_BlueprintVariableBase(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InSectionID)
	{}

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId() || InType == FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId();
	}

	UE_API virtual FEdGraphPinType GetPinType() const override;

	UE_API virtual void ChangeVariableType(const FEdGraphPinType& NewPinType) override;

	UE_API virtual void RenameVariable(const FName& NewName) override;

	UE_API virtual bool IsValidName(const FName& NewName, FText& OutErrorMessage) const override;

	UE_API virtual void DeleteVariable() override;

	UE_API virtual bool IsVariableUsed() override;
};

USTRUCT()
struct FRigVMEdGraphSchemaAction_PromoteToVariable : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	// Simple type info
	static FName StaticGetTypeId()
	{
		static const FLazyName Type("FRigVMEdGraphSchemaAction_PromoteToVariable");
		return Type;
	}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FRigVMEdGraphSchemaAction_PromoteToVariable()
		: FEdGraphSchemaAction()
		, EdGraphPin(nullptr)
		, bLocalVariable(false)
	{}

	UE_API FRigVMEdGraphSchemaAction_PromoteToVariable(UEdGraphPin* InEdGraphPin, bool InLocalVariable);

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId();
	}

	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode) override;

private:

	UEdGraphPin* EdGraphPin;
	bool bLocalVariable;
};

USTRUCT()
struct FRigVMEdGraphSchemaAction_PromoteToExposedPin : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	// Simple type info
	static FName StaticGetTypeId()
	{
		static const FLazyName Type("FRigVMEdGraphSchemaAction_PromoteToExposedPin");
		return Type;
	}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FRigVMEdGraphSchemaAction_PromoteToExposedPin()
		: FEdGraphSchemaAction()
		, EdGraphPin(nullptr)
	{}

	UE_API FRigVMEdGraphSchemaAction_PromoteToExposedPin(UEdGraphPin* InEdGraphPin);

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId();
	}

	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode) override;

private:

	UEdGraphPin* EdGraphPin;
};

USTRUCT()
struct FRigVMEdGraphSchemaAction_Event : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:

	// Simple type info
	static FName StaticGetTypeId()
	{
		static const FLazyName Type("FRigVMEdGraphSchemaAction_Event");
		return Type;
	}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FRigVMEdGraphSchemaAction_Event()
		: FEdGraphSchemaAction()
	{}

	UE_API FRigVMEdGraphSchemaAction_Event(const FName& InEventName, const FString& InNodePath, const FText& InNodeCategory);

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId() || Super::IsA(InType);
	}
	virtual bool IsParentable() const override { return true; }
	UE_API virtual FReply OnDoubleClick(UBlueprint* InBlueprint) override;
	virtual bool CanBeRenamed() const override { return false; }
	UE_API virtual FSlateBrush const* GetPaletteIcon() const override;

private:

	FString NodePath;
};

/** DragDropAction class for drag and dropping an item from the My Blueprints tree (e.g., variable or function) */
class FRigVMFunctionDragDropAction : public FGraphSchemaActionDragDropAction
{
public:

	DRAG_DROP_OPERATOR_TYPE(FRigVMFunctionDragDropAction, FGraphSchemaActionDragDropAction)

	// FGraphEditorDragDropAction interface
	UE_API virtual FReply DroppedOnPanel(const TSharedRef< class SWidget >& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	UE_API virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;
	UE_API virtual FReply DroppedOnAction(TSharedRef<FEdGraphSchemaAction> Action) override;
	UE_API virtual FReply DroppedOnCategory(FText Category) override;
	UE_API virtual void HoverTargetChanged() override;
	// End of FGraphEditorDragDropAction

	/** Set if operation is modified by alt */
	void SetAltDrag(bool InIsAltDrag) { bAltDrag = InIsAltDrag; }

	/** Set if operation is modified by the ctrl key */
	void SetCtrlDrag(bool InIsCtrlDrag) { bControlDrag = InIsCtrlDrag; }

protected:

	/** Constructor */
	UE_API FRigVMFunctionDragDropAction();

	static UE_API TSharedRef<FRigVMFunctionDragDropAction> New(TSharedPtr<FEdGraphSchemaAction> InAction, FRigVMEditorAssetInterfacePtr InRigBlueprint, URigVMEdGraph* InRigGraph);

protected:

	TScriptInterface<IRigVMEditorAssetInterface> SourceRigBlueprint;
	URigVMEdGraph* SourceRigGraph;
	bool bControlDrag;
	bool bAltDrag;

	friend class URigVMEdGraphSchema;
};

UCLASS(MinimalAPI)
class URigVMEdGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	/** Name constants */
	static inline const FLazyName GraphName_RigVM = FLazyName(TEXT("RigVM"));

public:
	UE_API URigVMEdGraphSchema();

	virtual const FLazyName& GetRootGraphName() const { return GraphName_RigVM; }

	// UEdGraphSchema interface
	UE_API virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual bool SupportsPinType(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType) const override;
	UE_API virtual bool SupportsPinTypeContainer(TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const FEdGraphPinType& PinType, const EPinContainerType& ContainerType) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	UE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	UE_API virtual bool CanGraphBeDropped(TSharedPtr<FEdGraphSchemaAction> InAction) const override;
	UE_API virtual FString GetFindReferenceSearchTerm(const FEdGraphSchemaAction* InGraphAction) const override;
	UE_API virtual FReply BeginGraphDragAction(TSharedPtr<FEdGraphSchemaAction> InAction, const FPointerEvent& MouseEvent = FPointerEvent() ) const override;
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
	UE_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	UE_API virtual void TrySetDefaultValue(UEdGraphPin& InPin, const FString& InNewDefaultValue, bool bMarkAsModified = true) const override;
	UE_API virtual void TrySetDefaultObject(UEdGraphPin& InPin, UObject* InNewDefaultObject, bool bMarkAsModified) const override;
	UE_API virtual void TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	UE_API virtual bool ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext, bool bIgnoreArray /*= false*/) const override;
	virtual bool DoesSupportPinWatching() const	override { return true; }
	UE_API virtual bool IsPinBeingWatched(UEdGraphPin const* Pin) const override;
	UE_API virtual void ClearPinWatch(UEdGraphPin const* Pin) const override;
	UE_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	UE_API virtual bool MarkBlueprintDirtyFromNewNode(UBlueprint* InBlueprint, UEdGraphNode* InEdGraphNode) const override;
	UE_API virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* Node) const override;
	UE_API virtual bool CanVariableBeDropped(UEdGraph* InGraph, FProperty* InVariableToDrop) const override;
	UE_API virtual bool RequestFunctionDropOnPanel(UEdGraph* InGraph, const FRigVMGraphFunctionIdentifier& InFunction, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const;
	UE_API virtual bool RequestVariableDropOnPanel(UEdGraph* InGraph, FProperty* InVariableToDrop, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const override;
	UE_API virtual bool RequestVariableDropOnPin(UEdGraph* InGraph, FProperty* InVariableToDrop, UEdGraphPin* InPin, const FVector2f& InDropPosition, const FVector2f& InScreenPosition) const override;
	using UEdGraphSchema::RequestVariableDropOnPin;
	UE_API virtual bool RequestVariableDropOnPin(UEdGraph* InGraph, const FRigVMExternalVariable& InVariableToDrop, UEdGraphPin* InPin, const FVector2D& InDropPosition, const FVector2D& InScreenPosition) const;
	UE_API virtual bool IsStructEditable(UStruct* InStruct) const;
	UE_API virtual void SetNodePosition(UEdGraphNode* Node, const FVector2f& Position) const override;
	using UEdGraphSchema::SetNodePosition;
	UE_API void SetNodePosition(UEdGraphNode* Node, const FVector2D& Position, bool bSetupUndo) const;
	UE_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	UE_API virtual bool GetLocalVariables(const UEdGraph* InGraph, TArray<FBPVariableDescription>& OutLocalVariables) const override;
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> MakeActionFromVariableDescription(const UEdGraph* InEdGraph, const FBPVariableDescription& Variable) const override;
	UE_API virtual FText GetGraphCategory(const UEdGraph* InGraph) const override;
	UE_API virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const override;
	UE_API virtual FReply TrySetGraphCategory(const UEdGraph* InGraph, const FText& InCategory) const override;
	UE_API virtual bool TryDeleteGraph(UEdGraph* GraphToDelete) const override;
	UE_API virtual bool TryRenameGraph(UEdGraph* GraphToRename, const FName& InNewName) const override;
	UE_API virtual bool TryToGetChildEvents(const UEdGraph* Graph, const int32 SectionId, TArray<TSharedPtr<FEdGraphSchemaAction>>& Actions, const FText& ParentCategory) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const { return false; }
	virtual bool AllowsFunctionVariants() const override { return CVarRigVMEnableVariants.GetValueOnAnyThread(); }
	UE_API virtual UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const override;
	UE_API virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	virtual void SetPinBeingDroppedOnNode(UEdGraphPin* InSourcePin) const override { PinBeingDropped = InSourcePin; }
	UE_API virtual void InsertAdditionalActions(TArray<UBlueprint*> InBlueprints, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins, FGraphActionListBuilderBase& OutAllActions) const override;
	UE_API virtual void InsertAdditionalActions(TArray<FRigVMEditorAssetInterfacePtr> InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins, FGraphActionListBuilderBase& OutAllActions) const;
	UE_API virtual TSharedPtr<INameValidatorInterface> GetNameValidator(const UBlueprint* BlueprintObj, const FName& OriginalName, const UStruct* ValidationScope, const FName& ActionTypeId) const override;
	UE_API virtual TSharedPtr<INameValidatorInterface> GetNameValidator(const FRigVMEditorAssetInterfacePtr Asset, const FName& OriginalName, const UStruct* ValidationScope) const;
	UE_API bool CanPromotePinToVariable(const UEdGraphPin& Pin, bool bCond) const;
	UE_API virtual bool SupportsPreviewFromHereOnNode(const UEdGraphNode* InNode) const;
	UE_API virtual bool AllowsFuzzyActionFiltering(const FEdGraphSchemaAction& InCurrentAction) const override;
	UE_API virtual float GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;

	/** Returns true if the schema supports the script type */
	UE_API bool SupportsPinType(const UScriptStruct* ScriptStruct) const;

	/** Create a graph node for a rig */
	UE_API URigVMEdGraphNode* CreateGraphNode(URigVMEdGraph* InGraph, const FName& InPropertyName) const;

	/** Helper function to rename a node */
	UE_API void RenameNode(URigVMEdGraphNode* Node, const FName& InNewNodeName) const;

	/** Helper function to recursively reset the pin defaults */
	UE_API virtual void ResetPinDefaultsRecursive(UEdGraphPin* InPin) const;

	/** Returns all of the applicable pin types for variables within a rigvm host */
	UE_API virtual void GetVariablePinTypes(TArray<FEdGraphPinType>& PinTypes) const;

	UE_API void StartGraphNodeInteraction(UEdGraphNode* InNode) const;
	UE_API void EndGraphNodeInteraction(UEdGraphNode* InNode) const;
	static UE_API TArray<UEdGraphNode*> GetNodesToMoveForNode(UEdGraphNode* InNode);
	UE_API FVector2D GetNodePositionAtStartOfInteraction(const UEdGraphNode* InNode) const;

	UE_API bool AutowireNewNode(URigVMEdGraphNode* NewNode, UEdGraphPin* FromPin) const;

	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	// Allow derived classes to spawn derived node classes
	UE_API virtual TSubclassOf<URigVMEdGraphNode> GetGraphNodeClass(const URigVMEdGraph* InGraph) const;

	UE_API virtual bool IsRigVMDefaultEvent(const FName& InEventName) const;

	// Allows to hide an input variable pin from node node
	UE_API virtual bool ShouldHidePinOnNode(const URigVMPin* InModelPin) const;

protected:
#if WITH_EDITOR
	mutable UClass* EditorSettingsClass;
#endif

private:

	const UEdGraphPin* LastPinForCompatibleCheck = nullptr;
	bool bLastPinWasInput;
	mutable UEdGraphPin* PinBeingDropped = nullptr;

#if WITH_EDITOR
	mutable TArray<UEdGraphNode*> NodesBeingInteracted;
	mutable TMap<FName, FVector2D> NodePositionsDuringStart;
#endif

	friend class URigVMEdGraphIfNodeSpawner;
	friend class URigVMEdGraphSelectNodeSpawner;
	friend class URigVMEdGraphUnitNodeSpawner;
	friend class URigVMEdGraphArrayNodeSpawner;
};

#undef UE_API
