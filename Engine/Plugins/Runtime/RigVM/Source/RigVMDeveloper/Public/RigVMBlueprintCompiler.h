// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetCompilerModule.h"
#include "KismetCompiler.h"

#define UE_API RIGVMDEVELOPER_API

class FRigVMBlueprintCompilerContext;
class URigVMBlueprintGeneratedClass;

class FRigVMBlueprintCompiler : public IBlueprintCompiler
{
public:
	/** IBlueprintCompiler interface */
	virtual bool CanCompile(const UBlueprint* Blueprint) override { return false; } // to be implemented by the specializations
	UE_API virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
};

class FRigVMBlueprintCompilerContext : public FKismetCompilerContext
{
public:
	FRigVMBlueprintCompilerContext(UBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
		: FKismetCompilerContext(SourceSketch, InMessageLog, InCompilerOptions)
		, NewRigVMBlueprintGeneratedClass(nullptr)
	{
	}

	// FKismetCompilerContext interface
	virtual void ValidateLink(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override {}
	virtual void ValidatePin(const UEdGraphPin* Pin) const override {}
	virtual void ValidateNode(const UEdGraphNode* Node) const override {}
	virtual bool CanIgnoreNode(const UEdGraphNode* Node) const override { return true; }
	virtual bool ShouldForceKeepNode(const UEdGraphNode* Node) const override { return false; }
	virtual void PrecompileFunction(FKismetFunctionContext& Context, EInternalCompilerFlags InternalFlags) override {}
	UE_API virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& Context) override;
	UE_API virtual void CopyTermDefaultsToDefaultObject(UObject* DefaultObject) override;
	UE_API virtual void EnsureProperGeneratedClass(UClass*& TargetUClass) override;
	UE_API virtual void SpawnNewClass(const FString& NewClassName) override;
	UE_API virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) override;
	virtual void PruneIsolatedNodes(const TArray<UEdGraphNode*>& RootSet, TArray<UEdGraphNode*>& GraphNodes) override {}
	UE_API virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO) override;
	virtual void MergeUbergraphPagesIn(UEdGraph* Ubergraph) override {};
	UE_API virtual void PreCompileUpdateBlueprintOnLoad(UBlueprint* BP) override;
	virtual void CreateFunctionList() override {}
	virtual void ProcessOneFunctionGraph(UEdGraph* SourceGraph, bool bInternalFunction = false) override {}

private:

	// used to fail a compilation and mark the blueprint in error
	UE_API void MarkCompilationFailed(const FString& Message);

protected:
	/** the new class we are generating */
	URigVMBlueprintGeneratedClass* NewRigVMBlueprintGeneratedClass;
};

#undef UE_API
