// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Engine/Blueprint.h"
#include "RigVMCore/RigVM.h"
#include "RigVMHost.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#if WITH_EDITOR
#include "HAL/CriticalSection.h"
#endif

#include "RigVMEditorAsset.h"
#include "RigVMModel/RigVMExternalDependency.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"

#include "RigVMBlueprintLegacy.generated.h"

#define UE_API RIGVMDEVELOPER_API


class URigVMBlueprintGeneratedClass;

USTRUCT(meta = (Deprecated = "5.2"))
struct FRigVMOldPublicFunctionArg
{
	GENERATED_BODY();
	
	FRigVMOldPublicFunctionArg()
	: Name(NAME_None)
	, CPPType(NAME_None)
	, CPPTypeObjectPath(NAME_None)
	, bIsArray(false)
	, Direction(ERigVMPinDirection::Input)
	{}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName CPPType;

	UPROPERTY()
	FName CPPTypeObjectPath;

	UPROPERTY()
	bool bIsArray;

	UPROPERTY()
	ERigVMPinDirection Direction;

	UE_API FEdGraphPinType GetPinType() const;
};

USTRUCT(meta = (Deprecated = "5.2"))
struct FRigVMOldPublicFunctionData
{
	GENERATED_BODY();

	FRigVMOldPublicFunctionData()
		:Name(NAME_None)
	{}

	UE_API virtual ~FRigVMOldPublicFunctionData();

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FString DisplayName;

	UPROPERTY()
	FString Category;

	UPROPERTY()
	FString Keywords;

	UPROPERTY()
	FRigVMOldPublicFunctionArg ReturnValue;

	UPROPERTY()
	TArray<FRigVMOldPublicFunctionArg> Arguments;

	UE_API bool IsMutable() const;
};

UCLASS(MinimalAPI, BlueprintType, meta=(IgnoreClassThumbnail))
class URigVMBlueprint : public UBlueprint, public IRigVMEditorAssetInterface, public IRigVMClientHost, public IRigVMExternalDependencyManager
{
	GENERATED_UCLASS_BODY()

public:
	virtual TScriptInterface<IRigVMRuntimeAssetInterface> GetRuntimeAssetInterface() const override { return TScriptInterface<IRigVMRuntimeAssetInterface>(GetRigVMGeneratedClass()); }
	virtual UClass* GetRuntimeAssetClass() const override { return URigVMBlueprintGeneratedClass::StaticClass(); }
	virtual UObject* GetAssetObjectForEditor() override { return this; }
	UE_API virtual void BeginDestroy() override;
	virtual UObject* GetObject() override { return this; }
	virtual const TArray<FRigVMGraphFunctionHeader>& GetPublicGraphFunctions() const override { return PublicGraphFunctions; }
	virtual void SetPublicGraphFunctions(const TArray<FRigVMGraphFunctionHeader>& InHeaders) override { PublicGraphFunctions = InHeaders; }
	virtual FRigVMClient* GetRigVMClient() override { return &RigVMClient; }
	virtual const FRigVMClient* GetRigVMClient() const override { return &RigVMClient; }
	virtual FRigVMVariant& GetAssetVariant() override { return AssetVariant; }
	virtual const FRigVMVariant& GetAssetVariant() const override { return AssetVariant; }
	virtual FRigVMCompileSettings& GetVMCompileSettings() override { return VMCompileSettings; }
	virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() override { return VMRuntimeSettings; }
	virtual bool IsRegeneratingOnLoad() const override { return bIsRegeneratingOnLoad;}
	UE_API virtual TArray<FRigVMGraphVariableDescription> GetAssetVariables() const override;
	virtual TArray<struct FEditedDocumentInfo>& GetLastEditedDocuments() override { return LastEditedDocuments;}
	UE_API virtual const TArray<FString>& GetRequiredPlugins(bool bRefresh = true) const override;
	virtual FRigVMEdGraphDisplaySettings& GetRigGraphDisplaySettings() override { return RigGraphDisplaySettings; }

#if WITH_EDITORONLY_DATA
	UE_API static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
#endif
	
#if WITH_EDITOR
	UE_API static bool CopyBlueprintToAsset(URigVMBlueprint* InBlueprint, URigVMRuntimeAsset* InAsset);
#endif

protected:
	virtual UClass* GetRigVMGeneratedClass() const { return GeneratedClass; }
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables(bool bFallbackToBlueprint) const override;
	virtual const UStruct* GetVariablesStruct() override { return GetRigVMBlueprintGeneratedClass(); }
	virtual uint8* GetVariablesMemory() override { return GetRigVMBlueprintGeneratedClass()->GetVariablesMemory(); }
	UE_API virtual FString GetVariableDefaultValue(const FName& InName, bool bFromDebuggedObject) const override;
	UE_API virtual URigVM* GetVM(bool bCreateIfNeeded = true) const override;
	UE_API virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() override;
	virtual TScriptInterface<IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() override { return GetRigVMBlueprintGeneratedClass(); }
	virtual TScriptInterface<const IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() const override { return GetRigVMBlueprintGeneratedClass(); }
	virtual FCompilerResultsLog* GetCurrentMessageLog() const override { return CurrentMessageLog; }
	UE_API virtual void SetAssetStatus(const ERigVMAssetStatus& InStatus) override;
	UE_API virtual ERigVMAssetStatus GetAssetStatus() const override;
	virtual bool IsUpToDate() const override { return UBlueprint::IsUpToDate(); }
	UE_API virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeCDO, bool bIncludeDerivedClasses) const override;
	virtual const FString& GetObjectBeingDebuggedPath() const override { return UBlueprint::GetObjectPathToDebug(); }
	virtual UWorld* GetWorldBeingDebugged() const override { return UBlueprint::GetWorldBeingDebugged(); }
	virtual void SetWorldBeingDebugged(UWorld* NewWorld) override { return UBlueprint::SetWorldBeingDebugged(NewWorld); }
	UE_API virtual FRigVMDebugInfo& GetDebugInfo() override;
	virtual TObjectPtr<URigVMEdGraph>& GetFunctionLibraryEdGraph() override { return FunctionLibraryEdGraph; }
	UE_API virtual URigVMHost* CreateRigVMHostSuper(UObject* InOuter, FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags) override;
	UE_API virtual void MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus = false) override;
	UE_API virtual void MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(nullptr)) override;
	UE_API virtual void AddUbergraphPage(URigVMEdGraph* RigVMEdGraph) override;
	UE_API virtual void AddLastEditedDocument(URigVMEdGraph* RigVMEdGraph) override;
	UE_API virtual void Compile() override;
	UE_API virtual FCompilerResultsLog CompileBlueprint() override;
	UE_API virtual void PatchVariableNodesOnLoad() override;
	UE_API virtual void AddPinWatch(UEdGraphPin* InPin) override;
	UE_API virtual void RemovePinWatch(UEdGraphPin* InPin) override;
	UE_API virtual void ClearPinWatches() override;
	UE_API virtual bool IsPinBeingWatched(const UEdGraphPin* InPin) const override;
	UE_API virtual bool NodeContainsWatchedPins(const UEdGraphNode* InNode) const override;
	UE_API virtual void ForeachPinWatch(TFunctionRef<void(UEdGraphPin*)> Task) override;
	UE_API virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName(bool bFromCDO) override;
	UE_API virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName(bool bFromCDO) override;
	UE_API virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse(bool bFromCDO) override;
	virtual TArray<TObjectPtr<UEdGraph>>& GetFunctionGraphs() override { return FunctionGraphs; }
	virtual bool& IsReferencedObjectPathsStored() override { return ReferencedObjectPathsStored; }
	virtual TArray<FSoftObjectPath>& GetReferencedObjectPaths() override { return ReferencedObjectPaths; }
	virtual TArray<FName>& GetSupportedEventNames() override { return SupportedEventNames; }
	UE_API virtual void UpdateSupportedEventNames() override;
	virtual TArray<FRigVMReferenceNodeData>& GetFunctionReferenceNodeData() override { return FunctionReferenceNodeData; }
	virtual void NotifyGraphRenamedSuper(class UEdGraph* Graph, FName OldName, FName NewName) override { UBlueprint::NotifyGraphRenamed(Graph, OldName, NewName); }
	UE_API virtual UObject* GetDefaultsObject() override;
	UE_API virtual void PostEditChangeBlueprintActors() override;

public:
	UE_API URigVMBlueprint();

	/** Get the (full) generated class for this rigvm blueprint */
	UE_API URigVMBlueprintGeneratedClass* GetRigVMBlueprintGeneratedClass() const;
	/** Returns the class used as the super class for all generated classes */
	virtual UClass* GetRigVMGeneratedClassPrototype() const { return URigVMBlueprintGeneratedClass::StaticClass(); }

	virtual void Serialize(FArchive& Ar) override { IRigVMEditorAssetInterface ::SerializeImpl(Ar); }
	virtual void SerializeSuper(FArchive& Ar) override { UBlueprint ::Serialize(Ar); }
	UE_API virtual void PostSerialize(FArchive& Ar) override;

#if WITH_EDITOR

	// UBlueprint interface
	UE_API virtual UClass* GetBlueprintClass() const override;
	UE_API virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) override;
	virtual bool SupportedByDefaultBlueprintFactory() const override { return false; }
	virtual bool IsValidForBytecodeOnlyRecompile() const override { return false; }
	virtual void LoadModulesRequiredForCompilation() override {}
	UE_API virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual void GetInstanceActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void SetObjectBeingDebugged(UObject* NewObject) override { IRigVMEditorAssetInterface::SetObjectBeingDebugged(NewObject); }
	virtual void SetObjectBeingDebuggedSuper(UObject* NewObject) override { UBlueprint::SetObjectBeingDebugged(NewObject); }
	UE_API virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) override;
	virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) const override { return const_cast<URigVMBlueprint*>(this)->GetObjectBeingDebugged(bEvenIfPendingKill); }
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;  
	UE_API virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	UE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif
	virtual bool IsPostLoadThreadSafe() const override { return IRigVMEditorAssetInterface::IsPostLoadThreadSafe(); }
	virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override { IRigVMEditorAssetInterface::PostTransacted(TransactionEvent); }
	virtual void PostTransactedSuper(const FTransactionObjectEvent& TransactionEvent) override { UBlueprint::PostTransacted(TransactionEvent); };
	virtual void ReplaceDeprecatedNodes() override { IRigVMEditorAssetInterface::ReplaceDeprecatedNodes(); }
	virtual void ReplaceDeprecatedNodesSuper() override { UBlueprint::ReplaceDeprecatedNodes(); };
	virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override { IRigVMEditorAssetInterface::PreDuplicate(DupParams); }
	virtual void PreDuplicateSuper(FObjectDuplicationParameters& DupParams) override { UBlueprint::PreDuplicate(DupParams); }
	virtual void PostDuplicate(bool bDuplicateForPIE) override { IRigVMEditorAssetInterface::PostDuplicate(bDuplicateForPIE); }
	virtual void PostDuplicateSuper(bool bDuplicateForPIE) override { UBlueprint::PostDuplicate(bDuplicateForPIE); };
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override { IRigVMEditorAssetInterface::GetAssetRegistryTags(Context); }
	virtual void GetAssetRegistryTagsSuper(FAssetRegistryTagsContext Context) const override { UBlueprint::GetAssetRegistryTags(Context); }
	virtual bool SupportsGlobalVariables() const override { return true; }
	virtual bool SupportsLocalVariables() const override { return true; }
	virtual bool SupportsFunctions() const override { return true; }
	virtual bool SupportsMacros() const override { return false; }
	virtual bool SupportsDelegates() const override { return false; }
	virtual bool SupportsEventGraphs() const override { return true; }
	virtual bool SupportsAnimLayers() const override { return false; }
	virtual bool ExportGraphToText(UEdGraph* InEdGraph, FString& OutText) override { IRigVMEditorAssetInterface::ExportGraphToText(InEdGraph, OutText); return true; }
	virtual bool TryImportGraphFromText(const FString& InClipboardText, UEdGraph** OutGraphPtr = nullptr) override { return IRigVMEditorAssetInterface::TryImportGraphFromText(InClipboardText, OutGraphPtr); }
	virtual bool CanImportGraphFromText(const FString& InClipboardText) override { return IRigVMEditorAssetInterface::CanImportGraphFromText(InClipboardText); }
	virtual bool RequiresForceLoadMembers(UObject* InObject) const override { return IRigVMEditorAssetInterface::RequiresForceLoadMembers(InObject); }
	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const override { return UBlueprint::RequiresForceLoadMembers(InObject); }
	

	// UObject interface
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	/** Called during cooking. Must return all objects that will be Preload()ed when this is serialized at load time. */
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	virtual TArray<TObjectPtr<UEdGraph>>& GetUberGraphs() override { return UbergraphPages; }
	virtual void GetAllEdGraphs(TArray<UEdGraph*>& Graphs) const override { UBlueprint::GetAllGraphs(Graphs); }
	virtual void SetUberGraphs(const TArray<TObjectPtr<UEdGraph>>& InGraphs) override { UbergraphPages = InGraphs; }
	UE_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;

	//  --- IRigVMClientHost interface Start---
	virtual FString GetAssetName() const override { return GetName(); }
	virtual UClass* GetRigVMSchemaClass() const override { return URigVMSchema::StaticClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const override { return FRigVMExecuteContext::StaticStruct(); }
	virtual UClass* GetRigVMEdGraphClass() const override { return URigVMEdGraph::StaticClass(); }
	virtual UClass* GetRigVMEdGraphNodeClass() const override { return URigVMEdGraphNode::StaticClass(); }
	virtual UClass* GetRigVMEdGraphSchemaClass() const override { return URigVMEdGraphSchema::StaticClass(); }
	virtual UClass* GetRigVMEditorSettingsClass() const override { return URigVMEditorSettings::StaticClass(); }
	virtual UObject* GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const override { return IRigVMEditorAssetInterface::GetEditorObjectForRigVMGraph(InVMGraph); }
	virtual URigVMGraph* GetRigVMGraphForEditorObject(UObject* InObject) const override { return IRigVMEditorAssetInterface::GetRigVMGraphForEditorObject(InObject); }
	virtual void HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphAdded(InClient, InNodePath); }
	virtual void HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphRemoved(InClient, InNodePath); }
	virtual void HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphRenamed(InClient, InOldNodePath, InNewNodePath); }
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) override { return IRigVMEditorAssetInterface::HandleConfigureRigVMController(InClient, InControllerToConfigure); }
	virtual void IncrementVMRecompileBracket() override { return IRigVMEditorAssetInterface::IncrementVMRecompileBracket(); }
	virtual void DecrementVMRecompileBracket() override { return IRigVMEditorAssetInterface::DecrementVMRecompileBracket(); }
	virtual void RefreshAllModels(ERigVMLoadType InLoadType = ERigVMLoadType::PostLoad) override { return IRigVMEditorAssetInterface::RefreshAllModels(InLoadType); }
	virtual void OnRigVMRegistryChanged() override { return IRigVMEditorAssetInterface::OnRigVMRegistryChanged(); }
	virtual URigVMGraph* GetModel(const FString& InNodePath) const override { return IRigVMEditorAssetInterface::GetModel(InNodePath); }
	virtual FRigVMGetFocusedGraph& OnGetFocusedGraph() override { return IRigVMEditorAssetInterface::OnGetFocusedGraph(); }
	virtual const FRigVMGetFocusedGraph& OnGetFocusedGraph() const override { return IRigVMEditorAssetInterface::OnGetFocusedGraph(); }
	virtual void SetupPinRedirectorsForBackwardsCompatibility() override { return IRigVMEditorAssetInterface::SetupPinRedirectorsForBackwardsCompatibility(); }
	virtual FRigVMGraphModifiedEvent& OnModified() override { return IRigVMEditorAssetInterface::OnModified(); }
	virtual bool IsFunctionPublic(const FName& InFunctionName) const override { return IRigVMEditorAssetInterface::IsFunctionPublic(InFunctionName); }
	virtual void MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic = true) override { return IRigVMEditorAssetInterface::MarkFunctionPublic(InFunctionName, bIsPublic); }
	virtual void RenameGraph(const FString& InNodePath, const FName& InNewName) override { return IRigVMEditorAssetInterface::RenameGraph(InNodePath, InNewName); }
	//  --- IRigVMClientHost interface End---
	
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void RecompileVM() override { IRigVMEditorAssetInterface::RecompileVM(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void RecompileVMIfRequired() override { return IRigVMEditorAssetInterface::RecompileVMIfRequired(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void RequestAutoVMRecompilation() override { return IRigVMEditorAssetInterface::RequestAutoVMRecompilation(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void SetAutoVMRecompile(bool bAutoRecompile) override { return IRigVMEditorAssetInterface::SetAutoVMRecompile(bAutoRecompile); }
	UFUNCTION(BlueprintPure, Category = "RigVM Blueprint")
	virtual bool GetAutoVMRecompile() const override { return IRigVMEditorAssetInterface::GetAutoVMRecompile(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void RequestRigVMInit() override { IRigVMEditorAssetInterface::RequestRigVMInit(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const override { return IRigVMEditorAssetInterface::GetModel(InEdGraph); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMGraph* GetDefaultModel() const override { return IRigVMEditorAssetInterface::GetDefaultModel(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual TArray<URigVMGraph*> GetAllModels() const override { return IRigVMEditorAssetInterface::GetAllModels(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMFunctionLibrary* GetLocalFunctionLibrary() const override { return IRigVMEditorAssetInterface::GetLocalFunctionLibrary(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo = true) override { return IRigVMEditorAssetInterface::GetOrCreateLocalFunctionLibrary(bSetupUndoRedo); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMGraph* AddModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override { return IRigVMEditorAssetInterface::AddModel(InName, bSetupUndoRedo, bPrintPythonCommand); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual bool RemoveModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override { return IRigVMEditorAssetInterface::RemoveModel(InName, bSetupUndoRedo, bPrintPythonCommand); }


	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMGraph* GetFocusedModel() const override { return IRigVMEditorAssetInterface::GetFocusedModel(); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMController* GetController(const URigVMGraph* InGraph = nullptr) const override { return IRigVMEditorAssetInterface::GetController(InGraph); }
	virtual URigVMController* GetController(const UEdGraph* InGraph) const override { return IRigVMEditorAssetInterface::GetController(InGraph); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMController* GetControllerByName(const FString InGraphName = TEXT("")) const override { return IRigVMEditorAssetInterface::GetControllerByName(InGraphName); }

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual URigVMController* GetOrCreateController(URigVMGraph* InGraph = nullptr) override { return IRigVMEditorAssetInterface::GetOrCreateController(InGraph); }
	virtual URigVMController* GetOrCreateController(const UEdGraph* InGraph) override { return IRigVMEditorAssetInterface::GetOrCreateController(InGraph); }
	
	UE_DEPRECATED(5.8, "Use TArray<FString> GeneratePythonCommands() instead.")
	virtual TArray<FString> GeneratePythonCommands(const FString InNewBlueprintName) override { return TArray<FString>(); }
	
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual TArray<FString> GeneratePythonCommands() override { return IRigVMEditorAssetInterface::GeneratePythonCommands(); }
	UE_API virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) override;
#endif

	virtual TArray<FRigVMExternalDependency> GetExternalDependenciesForCategory(const FName& InCategory) const override { return IRigVMEditorAssetInterface::GetExternalDependenciesForCategory(InCategory); }


	virtual bool ShouldBeMarkedDirtyUponTransaction() const override { return false; }
	virtual FRigVMPropertyBag* GetVariablesPropertyBag() override { return nullptr; }
	
	virtual FRigVMDrawContainer& GetDrawContainer() override { return DrawContainer; }

	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<URigVMEdGraph> FunctionLibraryEdGraph;
#endif


	UPROPERTY(EditAnywhere, Category = "User Interface")
	FRigVMEdGraphDisplaySettings RigGraphDisplaySettings;

	UPROPERTY(EditAnywhere, Category = "VM")
	FRigVMRuntimeSettings VMRuntimeSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "VM")
	FRigVMCompileSettings VMCompileSettings;

	UPROPERTY(EditAnywhere, Category = "Python Log Settings")
	FRigVMPythonSettings PythonLogSettings;

	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedStructGuidToPathName;

	UPROPERTY()
	TMap<FString, FSoftObjectPath> UserDefinedEnumToPathName;

	UPROPERTY(transient)
	TSet<TObjectPtr<UObject>> UserDefinedTypesInUse;

protected:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<URigVMGraph> Model_DEPRECATED;

	UPROPERTY()
	TObjectPtr<URigVMFunctionLibrary> FunctionLibrary_DEPRECATED;
#endif

	UPROPERTY()
	FRigVMClient RigVMClient;

#if WITH_EDITORONLY_DATA

	UPROPERTY()
	bool ReferencedObjectPathsStored;

	UPROPERTY()
	TArray<FSoftObjectPath> ReferencedObjectPaths;

#endif

	/** Asset searchable information about exposed public functions on this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FRigVMGraphFunctionHeader> PublicGraphFunctions;

	/** Asset searchable information function references in this rig */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FRigVMReferenceNodeData> FunctionReferenceNodeData;

#if WITH_EDITORONLY_DATA

	/** Variant information about this asset */
	UPROPERTY(EditAnywhere, AssetRegistrySearchable, Category = "Variant")
	FRigVMVariant AssetVariant;

#endif
	
	UPROPERTY(VisibleAnywhere, AssetRegistrySearchable, Category = "Dependencies")
	TArray<FString> RequiredPlugins;
	
	UPROPERTY(EditAnywhere, Category = "Drawing")
	FRigVMDrawContainer DrawContainer;

public:

	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual UClass* GetRigVMHostClass() const { return GeneratedClass; }

	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual URigVMHost* CreateRigVMHost() { return IRigVMEditorAssetInterface::CreateRigVMHostImpl(); }

	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual URigVMHost* GetDebuggedRigVMHost() { return IRigVMEditorAssetInterface::GetDebuggedRigVMHost(); }

	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual TArray<UStruct*> GetAvailableRigVMStructs() const override { return IRigVMEditorAssetInterface::GetAvailableRigVMStructs(); }

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = "Variables")
	virtual TArray<FRigVMGraphVariableDescription> GetMemberVariables() const { return GetAssetVariables(); }
	
	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual FName AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) override;

	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual bool RemoveMemberVariable(const FName& InName) override;

	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual bool BulkRemoveMemberVariables(const TArray<FName>& InNames) override;

	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual bool RenameMemberVariable(const FName& InOldName, const FName& InNewName) override;

	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual bool ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) override;
	UE_API virtual bool ChangeMemberVariableType(const FName& InName, const FEdGraphPinType& InType) override;

	UFUNCTION(BlueprintCallable, Category = "Variables")
	UE_API virtual bool SetVariableIndex(const FName& InName, int32 NewIndex) override;
	UE_API virtual bool SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex);

	UFUNCTION(BlueprintPure, Category = "Variants", meta = (DisplayName = "GetAssetVariant", ScriptName = "GetAssetVariant"))
	virtual FRigVMVariant GetAssetVariantBP() const { return AssetVariant; }

	UFUNCTION(BlueprintPure, Category = "Variants")
	virtual FRigVMVariantRef GetAssetVariantRef() const override { return IRigVMEditorAssetInterface::GetAssetVariantRefImpl(); }

	/** Resets the asset's guid to a new one and splits it from the former variant set */
	UFUNCTION(BlueprintCallable, Category = "Variants")
	virtual bool SplitAssetVariant() override { return IRigVMEditorAssetInterface::SplitAssetVariantImpl(); }

	/** Merges the asset's guid with a provided one to join the variant set */
	UFUNCTION(BlueprintCallable, Category = "Variants")
	virtual bool JoinAssetVariant(const FGuid& InGuid) override { return IRigVMEditorAssetInterface::JoinAssetVariantImpl(InGuid); }

	UFUNCTION(BlueprintPure, Category = "Variants")
	virtual TArray<FRigVMVariantRef> GetMatchingVariants() const override { return IRigVMEditorAssetInterface::GetMatchingVariantsImpl(); }
#endif	// #if WITH_EDITOR

	
private:

	/** The event names this rigvm blueprint contains */
	UPROPERTY(AssetRegistrySearchable)
	TArray<FName> SupportedEventNames;

#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	virtual void SuspendNotifications(bool bSuspendNotifs) { return IRigVMEditorAssetInterface::SuspendNotifications(bSuspendNotifs); }
#endif

protected:

#if WITH_EDITOR
	/** Our currently running rig vm instance */
	// Declaring these transient properties here instead of the IRigVMEditorAssetInterface class in order to have them as UPROPERTY, to avoid having stale pointers when GC happens
	UPROPERTY(transient)
	TObjectPtr<URigVMHost> EditorHost = nullptr;
	virtual TObjectPtr<URigVMHost>& GetEditorHost() override { return EditorHost; }

	static FName FindHostMemberVariableUniqueName(TSharedPtr<FKismetNameValidator> InNameValidator, const FString& InBaseName);
	static int32 AddHostMemberVariable(URigVMBlueprint* InBlueprint, const FGuid InVarGuid, const FName& InVarName, FEdGraphPinType InVarType, bool bIsPublic, bool bIsReadOnly, FString InDefaultValue);
	UE_API virtual FName AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue = FString()) override;
public:

	FOnRigVMPreVariablesChanged OnPreVariablesChangedDelegate;
	FOnRigVMPostVariablesChanged OnPostVariablesChangedDelegate;
	virtual FOnRigVMPreVariablesChanged& OnPreVariablesChanged() override { return OnPreVariablesChangedDelegate; }
	virtual FOnRigVMPostVariablesChanged& OnPostVariablesChanged() override { return OnPostVariablesChangedDelegate; }
	UE_API virtual void HandlePreVariableChange(UObject* InObject);
	UE_API virtual void HandlePostVariableChange(UBlueprint* InBlueprint);
	bool bUpdatingExternalVariables;

	UE_API void OnBlueprintChanged(UBlueprint* InBlueprint);
	UE_API void OnSetObjectBeingDebuggedReceived(UObject* InObject);
#endif

protected:

	virtual void SetupDefaultObjectDuringCompilation(URigVMHost* InCDO) override { IRigVMEditorAssetInterface::SetupDefaultObjectDuringCompilation(InCDO); }
	UE_API virtual void PreCompile() override;

	UE_API virtual FProperty* FindGeneratedPropertyByName(const FName& InName) override;
	UE_API virtual bool SetVariableTooltip(const FName& InName, const FText& InTooltip) override;
	UE_API virtual FText GetVariableTooltip(const FName& InName) const override;
	UE_API virtual bool SetVariableCategory(const FName& InName, const FString& InCategory) override;
	UE_API virtual FString GetVariableCategory(const FName& InName) override;
	UE_API virtual FString GetVariableMetadataValue(const FName& InName, const FName& InKey) override;
	UE_API virtual bool SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn) override;
	UE_API virtual bool SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics) override;
	UE_API virtual bool SetVariablePrivate(const FName& InName, const bool bInPrivate) override;
	UE_API virtual bool SetVariablePublic(const FName& InName, const bool bIsPublic) override;
	//virtual bool ChangeAssetVariableType(const FName& InName, const FEdGraphPinType& InType) override;
	UE_API virtual FString OnCopyVariable(const FName& InName) const override;
	UE_API virtual bool OnPasteVariable(const FString& InText) override;
	
#if WITH_EDITOR

private:

	TArray<FBPVariableDescription> LastNewVariables;
#endif

	// friend class FRigVMTreePackageNode;
	// friend class UE::RigVM::Editor::Tools::FFilterByAssetTag;
	// friend class FRigVMEditorModule;
};

#undef UE_API
