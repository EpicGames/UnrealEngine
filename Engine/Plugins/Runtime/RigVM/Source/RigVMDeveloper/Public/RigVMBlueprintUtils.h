// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMEditorAsset.h"
#include "Templates/SubclassOf.h"
#include "EdGraph/EdGraphPin.h"

#define UE_API RIGVMDEVELOPER_API

class UStruct;
class UBlueprint;
class URigVMEdGraphNode;
class UEdGraph;
class UEdGraphPin;

struct FRigVMBlueprintUtils
{
/** Call a function for each valid rig unit struct */
static UE_API void ForAllRigVMStructs(TFunction<void(UScriptStruct*)> InFunction);

/** Handle blueprint node reconstruction */
static UE_API void HandleReconstructAllBlueprintNodes(UBlueprint* InBlueprint);
static UE_API void HandleReconstructAllBlueprintNodes(FRigVMEditorAssetInterfacePtr InBlueprint);

/** Handle blueprint node refresh */
static UE_API void HandleRefreshAllNodes(FRigVMEditorAssetInterfacePtr InBlueprint);
static UE_API void HandleRefreshAllBlueprintNodes(UBlueprint* InBlueprint);

/** Handle blueprint deleted */
static UE_API void HandleAssetDeleted(const FAssetData& InAssetData);

/** remove the variable if not used by anybody else but ToBeDeleted*/
static UE_API void RemoveMemberVariableIfNotUsed(FRigVMEditorAssetInterfacePtr Blueprint, const FName VarName);

/** Create a new EdGraph */
static UE_API URigVMEdGraph* CreateNewGraph(UObject* ParentScope, const FName& GraphName, TSubclassOf<class URigVMEdGraph> GraphClass, TSubclassOf<class URigVMEdGraphSchema> SchemaClass);

static UE_API FName ValidateName(FRigVMEditorAssetInterfacePtr InBlueprint, const FString& InName);

/** Searches all nodes in a Blueprint and checks for a matching Guid */
static UE_API UEdGraphNode* GetNodeByGUID(const FRigVMEditorAssetInterfacePtr InBlueprint, const FGuid& InNodeGuid);

// Helper function to get the asset that ultimately owns a graph.
static UE_API FRigVMEditorAssetInterfacePtr FindAssetForGraph(const UEdGraph* Graph);

// Helper function to get the asset that ultimately owns a graph.
static UE_API FRigVMEditorAssetInterfacePtr FindAssetForNode(const UEdGraphNode* Node);

// Helper function to find a unique name for an asset variable
static UE_API FName FindUniqueVariableName(const IRigVMEditorAssetInterface* InBlueprint, const FString& InBaseName);
};

#undef UE_API
