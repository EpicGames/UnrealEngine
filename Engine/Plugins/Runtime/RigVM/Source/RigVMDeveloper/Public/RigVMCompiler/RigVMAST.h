// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMModel/RigVMLink.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMASTProxy.h"
#include "Logging/TokenizedMessage.h"
#include "RigVMAST.generated.h"

#define UE_API RIGVMDEVELOPER_API

class FRigVMParserAST;
class FRigVMBlockExprAST;
class FRigVMNodeExprAST;
class FRigVMEntryExprAST;
class FRigVMInvokeEntryExprAST;
class FRigVMCallExternExprAST;
class FRigVMInlineFunctionExprAST;
class FRigVMNoOpExprAST;
class FRigVMVarExprAST;
class FRigVMLiteralExprAST;
class FRigVMAssignExprAST;
class FRigVMCopyExprAST;
class FRigVMCachedValueExprAST;
class FRigVMExitExprAST;

class URigVMPin;
class URigVMLink;
class URigVMNode;
class URigVMGraph;
class URigVMController;
class URigVMLibraryNode;

/*
 * A structure to describe a link between two proxies
 */
struct FRigVMASTLinkDescription
{
	FRigVMASTLinkDescription() {}
	
	FRigVMASTLinkDescription(const FRigVMASTProxy& InSourceProxy, const FRigVMASTProxy& InTargetProxy, const FString& InSegmentPath)
	: SourceProxy(InSourceProxy)
	, TargetProxy(InTargetProxy)
	, SegmentPath(InSegmentPath)
	, LinkIndex(INDEX_NONE)
	{}

	FRigVMASTLinkDescription(const FRigVMASTLinkDescription& InOther)
	: SourceProxy(InOther.SourceProxy)
	, TargetProxy(InOther.TargetProxy)
	, SegmentPath(InOther.SegmentPath)
	, LinkIndex(INDEX_NONE)
	{}

	// Note that unlike the copy constructor, LinkIndex is copied by the assignment operator.
	FRigVMASTLinkDescription& operator=(const FRigVMASTLinkDescription&) = default;

	FRigVMASTProxy SourceProxy;
	FRigVMASTProxy TargetProxy;
	FString SegmentPath;
	int32 LinkIndex;
};

/*
 * Base class for an expression within an abstract syntax tree.
 * The base implements parent / child relationships as well as a
 * simple typing system.
 * An expression is a multi child / multi parent element of a
 * directed tree (there can be no cycles).
 * Expressions can only be constructed by an AST parser, and
 * are also memory-owned by the parser.
 */
class FRigVMExprAST
{
public:

	// Simple enum for differentiating expression types.
	enum EType
	{
		Block,
		Entry,
		InvokeEntry,
		CallExtern,
		InlineFunction,
		NoOp,
		Var,
		Literal,
		Assign,
		Copy,
		CachedValue,
		Exit,
		Invalid
	};

	// virtual destructor
	virtual ~FRigVMExprAST() = default;

	// disable copy constructor
	FRigVMExprAST(const FRigVMExprAST&) = delete;

	/// returns the parser this expression is owned by
	/// @return the parser this expression is owned by 
	const FRigVMParserAST* GetParser() const { return ParserPtr; }

	/// returns the name of the expression (can be NAME_None)
	/// @return the name of the expression
	FName GetName() const { return Name; }

	/// returns the exact type of the expression
	/// @return the exact type of the expression
	EType GetType() const { return Type; }

	/// returns the name of the expression's type
	/// @return the name of the expression's type
	UE_API FName GetTypeName() const;

	/// provides type checking for inherited types
	/// @param InType the type to check against
	/// @return true if this expression is of a given type
	virtual bool IsA(EType InType) const = 0;

	/// returns true if the expression is a node expression
	virtual bool IsNode() const
	{
		return false;
	}

	/// returns true if the expression is a var expression
	virtual bool IsVar() const
	{
		return false;
	}

	/// returns the index of this expression within the parser's storage
	/// @return the index of this expression within the parser's storage
	int32 GetIndex() const { return Index; }

	/// returns true if the expressoin is valid
	bool IsValid() const { return GetIndex() != INDEX_NONE; }

	/// returns the model proxy this expression relates to
	virtual const FRigVMASTProxy& GetProxy() const
	{
		static const FRigVMASTProxy EmptyProxy;
		return EmptyProxy;
	}

	/// returns the parent of this expression
	/// @return the parent of this expression
	UE_API const FRigVMExprAST* GetParent() const;

	/// returns the first parent in the tree of a given type
	/// @param InExprType The type of expression to look for in the parent tree
	/// @return the first parent in the tree of a given type
	UE_API virtual const FRigVMExprAST* GetFirstParentOfType(EType InExprType) const;

	/// returns true if a given expression is parented to another expression
	/// @param InParentExpr The potential parent to check
	/// @return True if this expression is parented to the InParentExpr
	UE_API bool IsParentedTo(const FRigVMExprAST* InParentExpr) const;

	/// returns true if a given expression is a parent of another expression
	/// @param InChildExpr The potential child to check
	/// @return True if this expression is a parent of the InChildExpr
	UE_API bool IsParentOf(const FRigVMExprAST* InChildExpr) const;

	/// returns the block of this expression
	/// @return the block of this expression
	UE_API const FRigVMBlockExprAST* GetBlock() const;

	/// returns the root / top level block of this expression
	/// @return the root / top level block of this expression
	UE_API const FRigVMBlockExprAST* GetRootBlock() const;

	/// returns all of the block this expression is in
	/// @param bSortByDepth If true the blocks will be returned sorted (head to tail)
	/// @return all of the block this expression is in
	typedef TArray<const FRigVMBlockExprAST*, TInlineAllocator<5>> FRigVMBlockArray;
	UE_API FRigVMBlockArray GetBlocks(bool bSortByDepth) const;

	/// returns the unique block combination hash for this expression (or INDEX_NONE)
	/// @return the unique block combination name for this expression (or INDEX_NONE)
	UE_API TOptional<uint32> GetBlockCombinationHash() const;

	/// returns the unique block combination name for this expression (or an empty string)
	/// @return the unique block combination name for this expression (or an empty string)
	UE_API const FString& GetBlockCombinationName() const;

	/// returns the lowest child index found for this expression within a parent (or INDEX_NONE)
	/// @return the lowest child index found for this expression within a parent (or INDEX_NONE)
	UE_API int32 GetMinChildIndexWithinParent(const FRigVMExprAST* InParentExpr) const;

	/// returns the number of children of this expression
	/// @return the number of children of this expression
	int32 NumChildren() const { return Children.Num(); }

	/// returns true if this expressions is constant (non varying)
	/// @return true if this expressions is constant (non varying)
	UE_API virtual bool IsConstant() const;

	/// returns true if this expressions is varying (non constant)
	/// @return true if this expressions is varying (non constant)
	bool IsVarying() const { return !IsConstant(); }

	/// returns the maximum depth of the expression in tree (number of recursive parents)
	/// @return the maximum depth of the expression in tree (number of recursive parents)
	UE_API int32 GetMaximumDepth() const;

	/// accessor operator for a given child
	/// @param InIndex the index of the child to retrieve (bound = NumChildren() - 1)
	/// @return the child at the given index
	const FRigVMExprAST* operator[](int32 InIndex) const { return Children[InIndex]; }

	// begin iterator accessor for the children
	TArray<FRigVMExprAST*>::RangedForConstIteratorType begin() const { return Children.begin(); }
	// end iterator accessor for the children
	TArray<FRigVMExprAST*>::RangedForConstIteratorType end() const { return Children.end(); }

	/// templated getter to retrieve a child with a given index
	/// type checking will occur within the ::To method and raise
	/// @param InIndex the index of the child to retrieve
	/// @return the child at a given index cast to the provided class
	template<class ObjectType>
	const ObjectType* ChildAt(int32 InIndex) const
	{
		return Children[InIndex]->To<ObjectType>();
	}

	/// getter to retrieve a child with a given index
	/// @param InIndex the index of the child to retrieve
	/// @return the child at a given index
	const FRigVMExprAST* ChildAt(int32 InIndex) const
	{
		return Children[InIndex];
	}

	/// returns the first parent in the tree of a given type
	/// @param InExprType The type of expression to look for in the parent tree
	/// @return the first parent in the tree of a given type
	UE_API virtual const FRigVMExprAST* GetFirstChildOfType(EType InExprType) const;

	/// returns the number of parents of this expression
	/// @return the number of parents of this expression
	int32 NumParents() const { return Parents.Num(); }

	/// templated getter to retrieve a parent with a given index
	/// type checking will occur within the ::To method and raise
	/// @param InIndex the index of the parent to retrieve
	/// @return the parent at a given index cast to the provided class
	template<class ObjectType>
	const ObjectType* ParentAt(int32 InIndex) const
	{
		return Parents[InIndex]->To<ObjectType>();
	}

	/// getter to retrieve a parent with a given index
	/// @param InIndex the index of the parent to retrieve
	/// @return the parent at a given index
	const FRigVMExprAST* ParentAt(int32 InIndex) const
	{
		return Parents[InIndex];
	}

	/// const templated cast for casting between
	/// different expression types.
	/// specializations below are used for type checking
	/// @return this object cast to the provided class
	template<class ObjectType>
	const ObjectType* To() const
	{
		checkNoEntry();
		return nullptr;
	}

	/// templated cast for casting between
	/// different expression types.
	/// specializations below are used for type checking
	/// @return this object cast to the provided class
	template<class ObjectType>
	ObjectType* To()
	{
		return (ObjectType*)this;
	}

protected:

	// default constructor (protected so that only parser can access it)
	UE_API FRigVMExprAST(EType InType = EType::Invalid, const FRigVMASTProxy& InProxy = FRigVMASTProxy());

	// adds a parent to this expression
	// this in consequence also adds this as a child to the parent
	// @param InParent the parent to add
	UE_API void AddParent(FRigVMExprAST* InParent);

	// removes a parent from this expression
	// this in consequence also removes this as a child from the parent
	// @param InParent the parent to remove
	UE_API void RemoveParent(FRigVMExprAST* InParent);

	// removes a child from this expression
	// this in consequence also removes this as a parent from the child
	// @param InChild the child to remove
	UE_API void RemoveChild(FRigVMExprAST* InChild);

	// replaces a parent of this expression with a new one
	// @param InCurrentParent the current parent to replace
	// @param InNewParent the new parent to replace it with
	// @param OutParentIndices The list of parent indices affected
	UE_API void ReplaceParent(FRigVMExprAST* InCurrentParent, FRigVMExprAST* InNewParent, TArray<int32>* OutParentIndices = nullptr);

	// replaces a child of this expression with a new one
	// @param InCurrentChild the current child to replace
	// @param InNewChild the new child to replace it with
	// @param OutChildIndices The list of child indices affected
	UE_API void ReplaceChild(FRigVMExprAST* InCurrentChild, FRigVMExprAST* InNewChild, TArray<int32>* OutChildIndices = nullptr);

	// replaces this expression with another one
	// @param InReplacement the expression to replace this one
	UE_API void ReplaceBy(FRigVMExprAST* InReplacement);

	// computes all of the block this expression is in
	UE_API void GetBlocksImpl(FRigVMBlockArray& InOutBlocks) const;

	// returns a string containing an indented tree structure
	// for debugging purposes. this is only used by the parser
	// @param InPrefix the prefix to use for indentation
	// @return the text representation of this part of the tree
	UE_API virtual FString DumpText(const FString& InPrefix = FString()) const;

	// Resets the internal caches of the expression
	UE_API void InvalidateCaches();
	UE_API void InvalidateCachesImpl(TArray<bool>& OutProcessed);

	FName Name;
	EType Type;
	int32 Index;
	const FRigVMParserAST* ParserPtr;
	TArray<FRigVMExprAST*> Parents;
	TArray<FRigVMExprAST*> Children;
	TMap<FName, int32> PinNameToChildIndex;
	mutable TOptional<TOptional<uint32>> BlockCombinationHash;
	mutable TOptional<int32> MaximumDepth;
	mutable TOptional<int32> BlocksCacheVersion;
	mutable FRigVMBlockArray CachedBlocks;
	mutable TOptional<int32> FirstChildOfTypeCacheVersion;
	mutable TMap<int32, const FRigVMExprAST*> CachedFirstChildOfType;
	mutable TOptional<int32> FirstParentOfTypeCacheVersion;
	mutable TMap<int32, const FRigVMExprAST*> CachedFirstParentOfType;

	friend class FRigVMParserAST;
	friend class URigVMCompiler;
};

// specialized cast for type checking
// for a Block / FRigVMBlockExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMBlockExprAST
template<>
inline const FRigVMBlockExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Block));
	return (const FRigVMBlockExprAST*)this;
}

// specialized cast for type checking
// for a Node / FRigVMNodeExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMNodeExprAST
template<>
inline const FRigVMNodeExprAST* FRigVMExprAST::To() const
{
	ensure(
		IsA(EType::Entry) ||
		IsA(EType::InvokeEntry) ||
		IsA(EType::CallExtern) ||
		IsA(EType::InlineFunction) ||
		IsA(EType::NoOp)
	);
	return (const FRigVMNodeExprAST*)this;
}

// specialized cast for type checking
// for a Entry / FRigVMEntryExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMEntryExprAST
template<>
inline const FRigVMEntryExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Entry));
	return (const FRigVMEntryExprAST*)this;
}

// specialized cast for type checking
// for a InvokeEntry / FRigVMInvokeEntryExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMInvokeEntryExprAST
template<>
inline const FRigVMInvokeEntryExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::InvokeEntry));
	return (const FRigVMInvokeEntryExprAST*)this;
}

// specialized cast for type checking
// for a CallExtern / FRigVMCallExternExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMCallExternExprAST
template<>
inline const FRigVMCallExternExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::CallExtern));
	return (const FRigVMCallExternExprAST*)this;
}

// specialized cast for type checking
// for a InlineFunction / FRigVMInlineFunctionExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMInlineFunctionExprAST
template<>
inline const FRigVMInlineFunctionExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::InlineFunction));
	return (const FRigVMInlineFunctionExprAST*)this;
}

// specialized cast for type checking
// for a NoOp / FRigVMNoOpExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMNoOpExprASTo 
template<>
inline const FRigVMNoOpExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::NoOp));
	return (const FRigVMNoOpExprAST*)this;
}

// specialized cast for type checking
// for a Var / FRigVMVarExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMVarExprAST
template<>
inline const FRigVMVarExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Var));
	return (const FRigVMVarExprAST*)this;
}

// specialized cast for type checking
// for a Literal / FRigVMLiteralExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMLiteralExprAST
template<>
inline const FRigVMLiteralExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Literal));
	return (const FRigVMLiteralExprAST*)this;
}

// specialized cast for type checking
// for a Assign / FRigVMAssignExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMAssignExprAST
template<>
inline const FRigVMAssignExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Assign));
	return (const FRigVMAssignExprAST*)this;
}

// specialized cast for type checking
// for a Copy / FRigVMCopyExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMCopyExprASTo 
template<>
inline const FRigVMCopyExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Copy));
	return (const FRigVMCopyExprAST*)this;
}

// specialized cast for type checking
// for a CachedValue / FRigVMCachedValueExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMCachedValueExprAST
template<>
inline const FRigVMCachedValueExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::CachedValue));
	return (const FRigVMCachedValueExprAST*)this;
}

// specialized cast for type checking
// for a Exit / FRigVMExitExprAST expression
// will raise if types are not compatible
// @return this expression cast to FRigVMExitExprAST 
template<>
inline const FRigVMExitExprAST* FRigVMExprAST::To() const
{
	ensure(IsA(EType::Exit));
	return (const FRigVMExitExprAST*)this;
}

/*
 * An abstract syntax tree block expression represents a sequence
 * of child expressions to be executed in order.
 * In C++ a block is represented by the curly braces { expr1, expr2, ...}.
 */ 
class FRigVMBlockExprAST : public FRigVMExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMBlockExprAST() = default;

	// disable copy constructor
	FRigVMBlockExprAST(const FRigVMBlockExprAST&) = delete;

	// returns true if this block needs to execute
	// this is determined by the block containing an entry expression
	// @return true if this block needs to execute
	UE_API bool ShouldExecute() const;

	// returns true if this block contains an entry expression
	// @return true if this block contains an entry expression
	UE_API bool ContainsEntry() const;

	// returns true if this block contains a given expression
	// @param InExpression the expression to check
	// @return true if this block contains a given expression
	UE_API bool Contains(const FRigVMExprAST* InExpression, TMap<const FRigVMExprAST*, bool>* ContainedExpressionsCache = nullptr) const;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::Block;
	};

	// returns true in case this block should not be executed
	UE_API bool IsObsolete() const;

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMBlockExprAST(EType InType = EType::Block, const FRigVMASTProxy& InProxy = FRigVMASTProxy())
		: FRigVMExprAST(InType, InProxy)
		, bIsObsolete(false)
	{}

private:

	bool bIsObsolete;

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree node expression represents any expression
 * which references a node from the RigVM model.
 */
class FRigVMNodeExprAST : public FRigVMBlockExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMNodeExprAST() = default;

	// disable copy constructor
	FRigVMNodeExprAST(const FRigVMNodeExprAST&) = delete;

	// returns the proxy this expression is using
	virtual const FRigVMASTProxy& GetProxy() const override { return Proxy; }

	virtual bool IsNode() const
	{
		return true;
	}

	// returns the node from the model this expression is referencing
	// @return the node from the model this expression is referencing
	URigVMNode* GetNode() const { return GetProxy().GetSubjectChecked<URigVMNode>(); }

	UE_API virtual bool IsConstant() const override;
	UE_API const FRigVMExprAST* FindExprWithPinName(const FName& InPinName) const;
	UE_API const FRigVMVarExprAST* FindVarWithPinName(const FName& InPinName) const;
	UE_API const FName& GetPinNameForChildIndex(int32 InChildIndex) const;

	void SetSegmentPathForChild(int32 InChildIndex, const FString& InSegmentPath);
	const FString& GetSegmentPathForChild(int32 InChildIndex) const;

protected:

	// default constructor (protected so that only parser can access it)
	UE_API FRigVMNodeExprAST(EType InType, const FRigVMASTProxy& InNodeProxy);

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return false;
	};

private:

	TMap<int32,FString> SegmentPathForChild;
	
	FRigVMASTProxy Proxy;
	friend class FRigVMParserAST;
	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
};

/*
 * An abstract syntax tree entry expression represents an entry point
 * for a function or an event in an event graph.
 * In C++ the entry point is the declaration: void main(...);
 */
class FRigVMEntryExprAST : public FRigVMNodeExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMEntryExprAST() = default;

	// disable copy constructor
	FRigVMEntryExprAST(const FRigVMEntryExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		if(FRigVMBlockExprAST::IsA(InType))
		{
			return true;
		}
		return InType == EType::Entry;
	};

	// returns the name of the entry / event
	// @return the name of the entry / event
	UE_API FName GetEventName() const;

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMEntryExprAST(const FRigVMASTProxy& InNodeProxy)
		: FRigVMNodeExprAST(EType::Entry, InNodeProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree entry expression represents an invocation
 * of an entry point.
 */
class FRigVMInvokeEntryExprAST : public FRigVMNodeExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMInvokeEntryExprAST() = default;

	// disable copy constructor
	FRigVMInvokeEntryExprAST(const FRigVMInvokeEntryExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		if(FRigVMNodeExprAST::IsA(InType))
		{
			return true;
		}
		return InType == EType::InvokeEntry;
	};

	// returns the name of the entry / event
	// @return the name of the entry / event
	UE_API FName GetEventName() const;

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMInvokeEntryExprAST(const FRigVMASTProxy& InNodeProxy)
		: FRigVMNodeExprAST(EType::InvokeEntry, InNodeProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree call extern expression represents the invocation
 * of an extern function.
 * In C++ the call extern is an invocation: FMath::Clamp(1.4, 0.0, 1.0);
 * The call extern expression references a node (through parent class)
 * from the model providing all of the relevant information for the invocation.
 */
class FRigVMCallExternExprAST: public FRigVMNodeExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMCallExternExprAST() = default;

	// disable copy constructor
	FRigVMCallExternExprAST(const FRigVMCallExternExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::CallExtern;
	};

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMCallExternExprAST(const FRigVMASTProxy& InNodeProxy)
		: FRigVMNodeExprAST(EType::CallExtern, InNodeProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 *
 */
class FRigVMInlineFunctionExprAST: public FRigVMNodeExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMInlineFunctionExprAST() = default;

	// disable copy constructor
	FRigVMInlineFunctionExprAST(const FRigVMInlineFunctionExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::InlineFunction;
	};

	// todo: For now, all function references are considered varying (UE-170129)
	virtual bool IsConstant() const override { return false; }

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMInlineFunctionExprAST(const FRigVMASTProxy& InNodeProxy)
		: FRigVMNodeExprAST(EType::InlineFunction, InNodeProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree no-op expression represents an expression which is
 * relevant for the structure of the tree (for grouping for example) but which 
 * itself has no operation connected to it.
 * For the RigVM AST we use the no-op expression for representing reroute nodes
 * in the model as well as parameter and variable getter nodes.
 */
class FRigVMNoOpExprAST : public FRigVMNodeExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMNoOpExprAST() = default;

	// disable copy constructor
	FRigVMNoOpExprAST(const FRigVMNoOpExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::NoOp;
	};

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMNoOpExprAST(const FRigVMASTProxy& InNodeProxy)
		: FRigVMNodeExprAST(EType::NoOp, InNodeProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree var expression represents the definition of 
 * mutable memory for a single variable.
 * In C++ the var expression is a variable declaration: int A;
 * The var expression references a pin from the model.
 */
class FRigVMVarExprAST: public FRigVMExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMVarExprAST() = default;

	// disable copy constructor
	FRigVMVarExprAST(const FRigVMVarExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::Var;
	};

	virtual bool IsVar() const
	{
		return true;
	}

	UE_API virtual bool IsConstant() const override;

	// returns the proxy this expression is using
	virtual const FRigVMASTProxy& GetProxy() const override { return Proxy; }

	// returns the pin in the model this variable is representing
	// @return the pin in the model this variable is representing
	URigVMPin* GetPin() const { if (ensure(GetProxy().GetSubject()))  { return GetProxy().GetSubjectChecked<URigVMPin>(); } return nullptr; }

	// returns the C++ data type of this variable
	// @return the C++ data type of this variable
	UE_API FString GetCPPType() const;

	// returns the C++ data type object (ustruct / uenum)
	// @return the C++ data type object (ustruct / uenum)
	UE_API UObject* GetCPPTypeObject() const;

	// returns the pin direction of this variable (input, output, hidden etc)
	// @return the pin direction of this variable (input, output, hidden etc)
	UE_API ERigVMPinDirection GetPinDirection() const;

	// returns the default value on the pin for this variable
	// @return the default value on the pin for this variable
	UE_API FString GetDefaultValue() const;

	// returns true if this variable is an execute context
	// @return true if this variable is an execute context
	UE_API bool IsExecuteContext() const;

	// returns true if this variable is a graph variable
	// @return true if this variable is a graph variable
	UE_API bool IsGraphVariable() const;

	// returns true if this is a constant enum index
	UE_API bool IsEnumValue() const;

	// returns true if this variable allows links to be "soft", so without a cache / computed value.
	UE_API bool SupportsSoftLinks() const;

	// Returns the clone this variable expression is using.
	const FRigVMVarExprAST* GetVariableCloneExpr() const
	{
		return ClonedVarExpr;
	}

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMVarExprAST(EType InType, const FRigVMASTProxy& InPinProxy)
		: FRigVMExprAST(InType)
		, Proxy(InPinProxy)
	{
	}

private:

	FRigVMASTProxy Proxy;
	const FRigVMVarExprAST* ClonedVarExpr = nullptr;

	friend class FRigVMParserAST;
	friend class URigVMCompiler;
};

/*
 * An abstract syntax tree literal expression represents the definition of 
 * const memory for a single variable - vs a var expression which is mutable.
 * In C++ the literal expression is a literal declaration, for ex: const float PI = 3.14f;
 * The literal expression references a pin from the model.
 */
class FRigVMLiteralExprAST : public FRigVMVarExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMLiteralExprAST() = default;

	// disable copy constructor
	FRigVMLiteralExprAST(const FRigVMLiteralExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		if(FRigVMVarExprAST::IsA(InType))
		{
			return true;
		}
		return InType == EType::Literal;
	};

	virtual bool IsConstant() const override
	{
		return true;
	}

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMLiteralExprAST(const FRigVMASTProxy& InPinProxy)
		: FRigVMVarExprAST(EType::Literal, InPinProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree assign expression represents the assignment of one
 * expression to another. This can result in referencing memory from a to b or
 * copying memory from a to b (thus the copy expression inherits the assign).
 * In C++ the assign expression used for construction and copy, for ex: int32 A = B + 1;
 * The assign expression references two pins / a link from the model.
 */
class FRigVMAssignExprAST : public FRigVMExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMAssignExprAST() = default;

	// disable copy constructor
	FRigVMAssignExprAST(const FRigVMAssignExprAST&) = delete;

	// returns the source proxy this expression is using
	const FRigVMASTLinkDescription& GetLink() const { return Link; }

	// returns the source proxy this expression is using
	const FRigVMASTProxy& GetSourceProxy() const { return Link.SourceProxy; }

	// returns the source pin for this assignment
	// @return the source pin for this assignment
	URigVMPin* GetSourcePin() const { return Link.SourceProxy.GetSubjectChecked<URigVMPin>(); }

	// returns the target proxy this expression is using
	const FRigVMASTProxy& GetTargetProxy() const { return Link.TargetProxy; }

	// returns the target pin for this assignment
	// @return the target pin for this assignment
	URigVMPin* GetTargetPin() const { return Link.TargetProxy.GetSubjectChecked<URigVMPin>(); }

	virtual const FRigVMASTProxy& GetProxy() const override { return GetTargetProxy(); }

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::Assign;
	};

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMAssignExprAST(EType InType, const FRigVMASTLinkDescription& InLink)
		: FRigVMExprAST(InType)
		, Link(InLink)
	{
	}

private:
	FRigVMASTLinkDescription Link;
	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree copy expression represents the an assignment of one
 * expression to another which causes / requires a copy operation. 
 * Within the RigVM AST this is only used for copying work state out of / into parameters
 * or when composing / decomposing a structure (for ex: assigning a float to a vector.x).
 * In C++ the copy expression is used for structures, for ex: FVector A = B;
 * The copy expression references two pins / a link from the model.
 */
class FRigVMCopyExprAST : public FRigVMAssignExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMCopyExprAST() = default;

	// disable copy constructor
	FRigVMCopyExprAST(const FRigVMCopyExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		if(FRigVMAssignExprAST::IsA(InType))
		{
			return true;
		}
		return InType == EType::Copy;
	};

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMCopyExprAST(const FRigVMASTLinkDescription& InLink)
		: FRigVMAssignExprAST(EType::Copy, InLink)
	{}


private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree cached value expression represents the reference to 
 * a variable which needs to be calculated by a call extern expression.
 * The first child of the cached value expression is the var expression to be
 * computed / cached, the second child is the call extern expression to use.
 */
class FRigVMCachedValueExprAST : public FRigVMExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMCachedValueExprAST() = default;

	// disable copy constructor
	FRigVMCachedValueExprAST(const FRigVMCachedValueExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::CachedValue;
	};

	// returns the var expression of this cached value (const)
	// @return the var expression of this cached value (const)
	const FRigVMVarExprAST* GetVarExpr() const { return ChildAt<FRigVMVarExprAST>(0); }

	// returns the call extern expression of this cached value (const)
	// @return the call extern expression of this cached value (const)
	const FRigVMCallExternExprAST* GetCallExternExpr() const { return ChildAt<FRigVMCallExternExprAST>(1); }

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMCachedValueExprAST(const FRigVMASTProxy& InProxy)
		: FRigVMExprAST(EType::CachedValue, InProxy)
	{}


private:

	friend class FRigVMParserAST;
};

/*
 * An abstract syntax tree exit expression represents the exit out of an entry expression.
 * In C++ the exit expression is a return from a main function.
 */
class FRigVMExitExprAST : public FRigVMExprAST
{
public:

	// virtual destructor
	virtual ~FRigVMExitExprAST() = default;

	// disable copy constructor
	FRigVMExitExprAST(const FRigVMExitExprAST&) = delete;

	// overload of the type checking mechanism
	virtual bool IsA(EType InType) const override
	{
		return InType == EType::Exit;
	};

	virtual bool IsConstant() const override
	{
		return true;
	}

protected:

	// default constructor (protected so that only parser can access it)
	FRigVMExitExprAST(const FRigVMASTProxy& InProxy)
		: FRigVMExprAST(EType::Exit, InProxy)
	{}

private:

	friend class FRigVMParserAST;
};

/*
 * The settings to apply during the parse of the abstract syntax tree.
 * The folding settings can affect the performance of the parse dramatically.
 */
USTRUCT(BlueprintType)
struct FRigVMParserASTSettings
{
	GENERATED_BODY()

	// fold root pin assignments / copies
	UPROPERTY(EditAnywhere, Category = "AST")
	bool bFoldAssignments = false;
	
   // fold sub pin assignments / copies
   UPROPERTY(EditAnywhere, Category = "AST")
   bool bFoldSubPinCopies = false;
   
	// fold literals and share memory
	UPROPERTY(EditAnywhere, Category = "AST")
	bool bFoldLiterals = false;

	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category = "AST")
	bool bSetupTraits = true;

	// links to be ignored during the parse
	UPROPERTY()
	TArray<TObjectPtr<URigVMLink>> LinksToSkip;

	// nodes whose input literals must not be shared with other nodes' literals.
	// used by Direct Manipulation to keep per-node registers during interactive editing
	// so that writing to one node's literal register does not affect other nodes.
	// stored as FObjectKey rather than FName to correctly handle duplicate node names
	// inside collapsed graphs. not a UPROPERTY — session-only, intentionally not serialized.
	TSet<FObjectKey> InteractiveNodes;

	FRigVMReportDelegate ReportDelegate;

	UPROPERTY(VisibleAnywhere, Category = "AST", transient)
	TObjectPtr<UScriptStruct> ExecuteContextStruct;

	// static method to provide fast AST parse settings
	static FRigVMParserASTSettings Fast(UScriptStruct* InExecuteContextStruct = nullptr)
	{
		FRigVMParserASTSettings Settings;
		Settings.bFoldAssignments = false;
		Settings.bFoldSubPinCopies = false;
		Settings.bFoldLiterals = false;
		Settings.ExecuteContextStruct = InExecuteContextStruct ? InExecuteContextStruct : FRigVMExecuteContext::StaticStruct();
		return Settings;
	}

	// static method to provide AST parse settings
	// tuned for a fast executing runtime, but slow parse
	static FRigVMParserASTSettings Optimized(UScriptStruct* InExecuteContextStruct = nullptr)
	{
		FRigVMParserASTSettings Settings;
		Settings.bFoldAssignments = true;
		Settings.bFoldSubPinCopies = false;
		Settings.bFoldLiterals = true;
		Settings.ExecuteContextStruct = InExecuteContextStruct ? InExecuteContextStruct : FRigVMExecuteContext::StaticStruct();
		return Settings;
	}

	UE_API void Report(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage) const;

	template <typename... Types>
		void Reportf(EMessageSeverity::Type InSeverity,
								 UObject* InSubject,
								 UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt,
								 Types... Args) const
	{
		Report(InSeverity, InSubject, FString::Printf(Fmt, Args...));
	}
};

/*
 * The abstract syntax tree parser is the main object to parse a
 * RigVM model graph. It's the memory owner for all expressions and
 * provides functionality for introspection of the tree.
 * The abstract syntax tree is then fed into the RigVMCompiler to 
 * generate the byte code for the virtual machine.
 */
class FRigVMParserAST : public TSharedFromThis<FRigVMParserAST>
{
public:

	// default constructor
	// @param InGraph The graph / model to parse
	// @param InSettings The parse settings to use
	UE_API FRigVMParserAST(TArray<URigVMGraph*> InGraphs, URigVMController* InController = nullptr, const FRigVMParserASTSettings& InSettings = FRigVMParserASTSettings::Fast(), const TArray<FRigVMExternalVariable>& InExternalVariables = TArray<FRigVMExternalVariable>());

	// default destructor
	UE_API ~FRigVMParserAST();

	// returns the number of root expressions
	int32 Num() const { return RootExpressions.Num(); }

	// operator accessor for a given root expression
	// @param InIndex the index of the expression to retrieve (bound = Num() - 1)
	// @return the root expression with the given index
	const FRigVMExprAST* operator[](int32 InIndex) const { return RootExpressions[InIndex]; }

	// begin iterator accessor for the root expressions
	TArray<FRigVMExprAST*>::RangedForConstIteratorType begin() const { return RootExpressions.begin(); }

	// end iterator accessor for the root expressions
	TArray<FRigVMExprAST*>::RangedForConstIteratorType end() const { return RootExpressions.end(); }

	// accessor method for a given root expression
	// @param InIndex the index of the expression to retrieve (bound = Num() - 1)
	// @return the root expression with the given index
	const FRigVMExprAST* At(int32 InIndex) const { return RootExpressions[InIndex]; }

	// returns the expression for a given subject. subjects include nodes and pins.
	// @param InSubject the subject to retrieve the expression for (node or pin)
	// @return the expressoin for the given subject (or nullptr)
	UE_API const FRigVMExprAST* GetExprForSubject(const FRigVMASTProxy& InProxy) const;

	// returns all expressions for a given subject. subjects include nodes and pins.
	// @param InSubject the subject to retrieve the expression for (node or pin)
	// @return all expressions for the given subject
	UE_API TArray<const FRigVMExprAST*> GetExpressionsForSubject(UObject* InSubject) const;

	// Prepares the parser for cycle checking on a given pin.
	// This marks up the parents and childen of the corresponding expression in the graph,
	// to allow the client to determine if a new parent / child relationship could cause a cycle.
	// @param InPin the pin to initialize the cycle checking for
	UE_API void PrepareCycleChecking(URigVMPin* InPin);

	// Performs a cycle check for a new potential link (assign or copy) between two pins.
	// @param InSourcePin the source (left) pin of the potential assign / copy
	// @param InTargetPin the target (right) pin of the potential assign / copy
	// @param OutFailureReason an optional storage for the possible failure reason description
	// @return true if the potential link (assign / copy) can be established
	UE_API bool CanLink(URigVMPin* InSourcePin, URigVMPin* InTargetPin, FString* OutFailureReason);

	// returns a string containing an indented tree structure
	// for debugging purposes.
	// @return the simple text representing this abstract syntax tree
	UE_API FString DumpText() const;

	// returns a string containing a dot file notation
	// for debugging purposes.
	// @return the dot file text representing this abstract syntax tree
	UE_API FString DumpDot() const;

	// returns an obsolete block for unmanaged expression
	UE_API FRigVMBlockExprAST* GetObsoleteBlock(bool bCreateIfMissing = true);
	UE_API const FRigVMBlockExprAST* GetObsoleteBlock(bool bCreateIfMissing = true) const;

	// returns the AST's override table for pin defaults
	const URigVMPin::FPinOverrideMap& GetPinOverrides() const { return PinOverrides; }

	// returns the settings used for this parser
	const FRigVMParserASTSettings& GetSettings() const { return Settings; }

private:

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(FRigVMExprAST::EType InType, const FRigVMASTProxy& InProxy)
	{
		ExprType* Expr = new ExprType(InType, InProxy);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(FRigVMExprAST::EType InType, const FRigVMASTLinkDescription& InLink)
	{
		ExprType* Expr = new ExprType(InType, InLink);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(FRigVMExprAST::EType InType, const FRigVMASTProxy& InProxyA, const FRigVMASTProxy& InProxyB)
	{
		ExprType* Expr = new ExprType(InType, InProxyA, InProxyB);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(const FRigVMASTProxy& InProxyA, const FRigVMASTProxy& InProxyBB)
	{
		ExprType* Expr = new ExprType(InProxyA, InProxyBB);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(const FRigVMASTProxy& InProxy)
	{
		ExprType* Expr = new ExprType(InProxy);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr(const FRigVMASTLinkDescription& InLink)
	{
		ExprType* Expr = new ExprType(InLink);
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// make function to create an expression
	template<class ExprType>
	ExprType* MakeExpr()
	{
		ExprType* Expr = new ExprType();
		Expr->ParserPtr = this;
		Expr->Index = Expressions.Add(Expr);
		return Expr;
	}

	// removes an array of expressions from the parser
	// @param InExprs the expressions to remove
	UE_API void RemoveExpressions(TArray<FRigVMExprAST*> InExprs);

	// a static helper function to traverse along all parents of an expression,
	// provided a predicate to return true if the traverse should continue,
	// and false if the traverse should stop.
	// @param InExpr the expression to start the traverse at
	// @param InContinuePredicate the predicate to use to break out of the traverse (return false)
	static UE_API void TraverseParents(const FRigVMExprAST* InExpr, TFunctionRef<bool(const FRigVMExprAST*)> InContinuePredicate);

	// a static helper function to traverse along all children of an expression,
	// provided a predicate to return true if the traverse should continue,
	// and false if the traverse should stop.
	// @param InExpr the expression to start the traverse at
	// @param InContinuePredicate the predicate to use to break out of the traverse (return false)
	static UE_API void TraverseChildren(const FRigVMExprAST* InExpr, TFunctionRef<bool(const FRigVMExprAST*)> InContinuePredicate);

	// helper function to fold all entries with the same event name into one block
	UE_API void FoldEntries();

	// helper function to inject an exit expression at the end of every entry expressoin
	UE_API void InjectExitsToEntries();

	// helper function to refresh the expression indices (used after deleting an expression)
	UE_API void RefreshExprIndices();

	// helper function to fold / remove the no op expressions (parameters mostly)
	UE_API void FoldNoOps();

	// helper function to fold / merge redundant literals with the same value
	UE_API void FoldLiterals();

	// helper function to fold / remove obsolete assignments and reduce assignment chains
	UE_API void FoldAssignments();

	// helper function to fold / remove obsolete copies from a sub pin to a root pin
   	UE_API void FoldSubPinCopies();

	// helper function to inline all contributing nodes of the graph
	UE_API void Inline(const TArray<URigVMGraph*>& InGraphs);
	UE_API void Inline(const TArray<URigVMGraph*>& InGraphs, const TArray<FRigVMASTProxy>& InNodeProxies);

	// helper functions to retrieve links for a given pin
	UE_API TArray<int32> GetSourceLinkIndices(const FRigVMASTProxy& InPinProxy, bool bRecursive = false) const;
	UE_API TArray<int32> GetTargetLinkIndices(const FRigVMASTProxy& InPinProxy, bool bRecursive = false) const;
	UE_API TArray<int32> GetLinkIndices(const FRigVMASTProxy& InPinProxy, bool bGetSource, bool bRecursive) const;

	// function to retrieve a link given its index
	UE_API const FRigVMASTLinkDescription& GetLink(int32 InLinkIndex) const;

	// traverse a single mutable node (constructs entry, call extern and other expressions)
	UE_API FRigVMExprAST* TraverseMutableNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr);

	// traverse a single pure node (constructs call extern expressions)
	UE_API FRigVMExprAST* TraverseNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr);

	// returns the expression for a given node
	UE_API FRigVMExprAST* CreateExpressionForNode(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr);

	// traverse an array of pins for a given node
	UE_API TArray<FRigVMExprAST*> TraversePins(const FRigVMASTProxy& InNodeProxy, FRigVMExprAST* InParentExpr);

	// traverse a single pin (constructs var + literal expressions)
	UE_API FRigVMExprAST* TraversePin(const FRigVMASTProxy& InPinProxy, FRigVMExprAST* InParentExpr);

	// traverse a single link (constructs assign + copy expressions)
	UE_API FRigVMExprAST* TraverseLink(int32 InLinkIndex, FRigVMExprAST* InParentExpr);

	UE_API bool TraverseNodeExpressionForRequiredVariableClone(FRigVMNodeExprAST* InNodeExpr);

	UE_API bool ShouldLinkBeSkipped(const FRigVMASTLinkDescription& InLink) const;

	static UE_API FString GetLinkAsString(const FRigVMASTLinkDescription& InLink);

	UE_API void IncrementCacheVersion() const;

	static bool IsNoOpUnitNode(const URigVMNode* InNode);
	static bool IsConstantDispatchNode(const URigVMNode* InNode);
	static bool IsConstantDispatchValuePin(const URigVMPin* InPin);
 	static bool IsMakeStructDispatchNode(const URigVMNode* InNode);
	static bool IsMakeStructDispatchElementsPin(const URigVMPin* InPin);
	static bool IsMakeStructDispatchStructPin(const URigVMPin* InPin);
	static bool IsBreakStructDispatchNode(const URigVMNode* InNode);
	static bool IsBreakStructDispatchStructPin(const URigVMPin* InPin);
	static bool IsBreakStructDispatchElementsPin(const URigVMPin* InPin);

	static bool IsFloatOrFloatArrayPinOnVariable(const URigVMPin* InPin);

	TMap<FRigVMASTProxy, FRigVMExprAST*> SubjectToExpression;
	TMap<FRigVMASTProxy, int32> NodeExpressionIndex;
	TArray<FRigVMExprAST*> Expressions;
	TArray<FRigVMExprAST*> RootExpressions;
	TArray<FRigVMExprAST*> DeletedExpressions;
	FRigVMBlockExprAST* ObsoleteBlock;
	mutable TMap<uint32, FString> BlockCombinationHashToName;
	mutable int32 CacheVersion;

	TArray<FRigVMASTProxy> NodeProxies;
	TMap<FRigVMASTProxy, FRigVMASTProxy> SharedOperandPins;
	TMap<FRigVMASTProxy, TArray<int32>> TargetLinkIndices;
	TMap<FRigVMASTProxy, TArray<int32>> SourceLinkIndices;
	TArray<FRigVMASTLinkDescription> Links;
	static UE_API const TArray<FRigVMASTProxy> EmptyProxyArray;
	URigVMPin::FPinOverrideMap PinOverrides;

	enum ETraverseRelationShip
	{
		ETraverseRelationShip_Unknown,
		ETraverseRelationShip_Parent,
		ETraverseRelationShip_Child,
		ETraverseRelationShip_Self
	};

	const FRigVMExprAST* LastCycleCheckExpr;
	TArray<ETraverseRelationShip> CycleCheckFlags;
	TArray<URigVMLink*> LinksToSkip;

	FRigVMParserASTSettings Settings;
	mutable TMap<TTuple<const FRigVMExprAST*,const FRigVMExprAST*>, int32> MinIndexOfChildWithinParent;

	URigVMLibraryNode* LibraryNodeBeingCompiled;

	friend class FRigVMExprAST;
	friend class FRigVMAssignExprAST;
	friend class URigVMCompiler;
};

#undef UE_API
