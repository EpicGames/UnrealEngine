// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "UObject/UObjectBaseUtility.h"

#define UE_API RIGVMDEVELOPER_API

/*
 * A stack describing the model object which led to this invocation.
 * Objects on the stack can be URigVMGraph or URigVMNode.
 */
class FRigVMCallstack
{
public:

	// Returns the path
	UE_API FString GetCallPath(bool bIncludeLast = true) const;

	// Returns the number of entries on this callstack
	UE_API int32 Num() const;

	// Returns the last object in the callstack
	UE_API const UObject* Last() const;

	// Returns the element at a given index
	UE_API const UObject* operator[](int32 InIndex) const;

	// Returns true if the stack contains a given entry
	UE_API bool Contains(const UObject* InEntry) const;

	friend uint32 GetTypeHash(const FRigVMCallstack& Callstack)
	{
		return Callstack.GetEntryTypeHash(Callstack.Num() - 1);
	}

	bool operator ==(const FRigVMCallstack& Other) const
	{
		if (Num() == Other.Num())
		{
			for (int32 Index = 0; Index < Num(); Index++)
			{
				if (operator[](Index) != Other[Index])
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

	bool operator !=(const FRigVMCallstack& Other) const
	{
		return !(*this == Other);
	}

	bool operator <(const FRigVMCallstack& Other) const
	{
		if (Num() < Other.Num())
		{
			return true;
		}

		if (Num() > Other.Num())
		{
			return false;
		}

		if (Num() == 0)
		{
			return false;
		}

		return operator[](0) < Other[0];
	}

	bool operator >(const FRigVMCallstack& Other) const
	{
		if (Num() > Other.Num())
		{
			return true;
		}

		if (Num() < Other.Num())
		{
			return false;
		}

		if (Num() == 0)
		{
			return true;
		}

		return operator[](0) > Other[0];
	}

	TArray<TWeakObjectPtr<UObject>> GetStack() const { return Stack; }

	UE_API FRigVMCallstack GetCallStackUpTo(int32 InIndex) const;

private:

	uint32 GetEntryTypeHash(int32 Index) const
	{
		if (Index < 0)
		{
			return 0;
		}

		const uint32 StackHash = Stack[Index].IsValid() ? GetTypeHash(Stack[Index]->GetPathName()) : 0;
		if (Index == 0)
		{
			return StackHash;
		}

		return HashCombine(GetEntryTypeHash(Index - 1), StackHash);
	}

	TArray<TWeakObjectPtr<UObject>> Stack;

	friend class FRigVMParserAST;
	friend class FRigVMASTProxy;
	friend class URigVMCompiler;
	friend struct FRigVMCompilerWorkData;
};

/*
 * A proxy which describes an occurence of a subject within
 * a graph. The subject can be a pin, a node, a link, etc.
 * The context is the callstack under which is occured.
 */
class FRigVMASTProxy
{
public:

	FRigVMASTProxy()
	{
#if UE_BUILD_DEBUG
		DebugName = FString();
#endif
		Callstack.Stack.Push(nullptr);
	}

	FRigVMASTProxy(const FRigVMASTProxy& InOther)
	{
		Callstack.Stack = InOther.Callstack.Stack;
#if UE_BUILD_DEBUG
		DebugName = Callstack.GetCallPath();
#endif
	}

	FRigVMASTProxy& operator=(const FRigVMASTProxy& InOther)
	{
		if (this != &InOther)
		{
			Callstack.Stack = InOther.Callstack.Stack;
#if UE_BUILD_DEBUG
			DebugName = Callstack.GetCallPath();
#endif
		}
		return *this;
	}

	static UE_API FRigVMASTProxy MakeFromUObject(UObject* InSubject);
	static UE_API FRigVMASTProxy MakeFromCallPath(const FString& InCallPath, UObject* InRootObject);
	static UE_API FRigVMASTProxy MakeFromCallstack(const FRigVMCallstack& InCallstack);
	static UE_API FRigVMASTProxy MakeFromCallstack(const TArray<TWeakObjectPtr<UObject>>* InCallstack);

	FRigVMASTProxy GetSibling(UObject* InSubject) const
	{
		FRigVMASTProxy Sibling = *this;
		Sibling.Callstack.Stack.Last() = InSubject;
#if UE_BUILD_DEBUG
		Sibling.DebugName = Sibling.Callstack.GetCallPath();
#endif
		return Sibling;
	}

	FRigVMASTProxy GetParent() const
	{
		ensure(Callstack.Num() > 1);
		
		FRigVMASTProxy Parent = *this;
		Parent.Callstack.Stack.Pop();
#if UE_BUILD_DEBUG
		Parent.DebugName = Parent.Callstack.GetCallPath();
#endif
		return Parent;
	}

	FRigVMASTProxy GetChild(UObject* InSubject) const
	{
		FRigVMASTProxy Child = *this;

		if(!Child.IsValid())
		{
			Child.Callstack.Stack.Reset();
		}
		
		Child.Callstack.Stack.Push(InSubject);
#if UE_BUILD_DEBUG
		Child.DebugName = Child.Callstack.GetCallPath();
#endif
		return Child;
	}

	const FRigVMCallstack& GetCallstack() const { return Callstack; }

	UObject* GetSubject() const { return Callstack.Stack.Last().IsValid() ? Callstack.Stack.Last().Get() : nullptr; }

	template<class T>
	T* GetSubject() const { return Cast<T>(GetSubject()); }

	template<class T>
	T* GetSubjectChecked() const { return CastChecked<T>(GetSubject()); }

	bool IsValid() const { return Callstack.Num() > 0 && GetSubject() != nullptr; }

	template<class T>
	bool IsA() const { return GetSubject() ? GetSubject()->IsA<T>() : false; }

	friend uint32 GetTypeHash(const FRigVMASTProxy& Proxy)
	{
		return GetTypeHash(Proxy.GetCallstack());
	}

	bool operator ==(const FRigVMASTProxy& Other) const
	{
		return GetCallstack() == Other.GetCallstack();
	}

	bool operator !=(const FRigVMASTProxy& Other) const
	{
		return GetCallstack() != Other.GetCallstack();
	}

	bool operator <(const FRigVMASTProxy& Other) const
	{
		return GetCallstack() < Other.GetCallstack();
	}

	bool operator >(const FRigVMASTProxy& Other) const
	{
		return GetCallstack() > Other.GetCallstack();
	}

private:

#if UE_BUILD_DEBUG
	FString DebugName;
#endif

	FRigVMCallstack Callstack;
};

typedef TPair<FRigVMASTProxy, FRigVMASTProxy> FRigVMPinProxyPair;
typedef TMap<FRigVMASTProxy, FString> FRigVMPinDefaultValueOverride;

#undef UE_API
