// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCodeTemplate.h"
#include "RigVMEditorAsset.h"
#include <inja/inja.hpp>

#define UE_API RIGVMDEVELOPER_API

/**
 * Settings for RigVM code conversion/nativization.
 */
struct FRigVMCodeConversionSettings
{
	FRigVMCodeConversionSettings()
	: TargetModule(TEXT("Test"))
	, bWriteFiles(false)
	{
	}

	/** Name of the UE module where generated code will be placed. */
	FString TargetModule;

	/** Directory where generated files should be written. */
	FString OutputFolder;

	/** Optional path to write the intermediate JSON representation (for debugging). */
	FString JsonFilePath;

	/** If true, generated files are written to disk automatically. */
	bool bWriteFiles;
};

/**
 * Converts RigVM bytecode into a JSON representation for template rendering.
 *
 * FRigVMCodeConverter is the core of the nativization pipeline. It parses a compiled
 * RigVM's bytecode and produces a comprehensive JSON structure that Inja templates
 * can use to generate C++ source code.
 *
 * ## Conversion Pipeline
 *
 * 1. **Construction**: Parses the VM's bytecode in a 14-step process:
 *    - External variables, memory layouts (literal/work)
 *    - Property paths, callables, blocks, entries
 *    - Instructions with operand resolution
 *    - Dependency tracking (includes, libraries)
 *
 * 2. **Render**: Combines the JSON with an Inja template to produce C++ code.
 *
 * ## JSON Structure Overview
 *
 * The generated JSON contains:
 * - `AssetName`: Name of the source asset
 * - `VMHash`: Hash for validation
 * - `Variables`: External variable definitions
 * - `Memory.Literal`: Constant/default values
 * - `Memory.Work`: Runtime temporaries
 * - `Functions`: Unit functions and dispatch factories
 * - `Blocks`: Control flow blocks (branches)
 * - `Callables`: User-defined callable functions
 * - `Entries`: Entry points into the VM
 * - `Instructions`: Bytecode instructions with resolved operands
 * - `Includes`: Required C++ header files
 * - `Libraries`: Required UE module dependencies
 *
 * ## Usage
 *
 * @code
 *   FRigVMCodeConversionSettings Settings;
 *   Settings.TargetModule = TEXT("MyGame");
 *   Settings.OutputFolder = TEXT("Source/MyGame/Generated");
 *
 *   auto Converter = MakeShared<FRigVMCodeConverter>(EditorAsset, Settings);
 *   if (Converter->HasError())
 *   {
 *       // Handle conversion failure
 *       return;
 *   }
 *
 *   // Use with FRigVMCodeGenerator to produce output files
 *   FRigVMCodeGenerator Generator(ERigVMCodeLanguage::CPlusPlus);
 *   auto Outputs = Generator.GenerateAll(Converter);
 * @endcode
 *
 * @see FRigVMCodeGenerator for template management and rendering orchestration
 * @see FRigVMCodeEnvironment for custom template functions
 */
class FRigVMCodeConverter
{
public:
	/**
	 * Constructs a converter and parses the VM's bytecode into JSON.
	 *
	 * This is a potentially expensive operation that analyzes the entire VM.
	 * Check HasError() after construction to verify success.
	 *
	 * @param InEditorAsset The editor asset containing the RigVM to nativize.
	 * @param InSettings Conversion settings (target module, output paths, etc.).
	 */
	UE_API FRigVMCodeConverter(IRigVMEditorAssetInterface* InEditorAsset, const FRigVMCodeConversionSettings& InSettings);

	/** Returns the sanitized asset name used for generated class names. */
	UE_API FString GetAssetName() const;

	/**
	 * Renders a template using the parsed JSON data.
	 *
	 * @param InTemplate The template to render.
	 * @return The rendered output with content and metadata.
	 */
	UE_API TSharedPtr<FRigVMCodeOutput> Render(const TSharedPtr<FRigVMCodeTemplate>& InTemplate);

	/** Returns true if an error occurred during construction or rendering. */
	bool HasError() const { return bHasError; }

	/** Returns the parsed JSON representation of the VM (for debugging/inspection). */
	const inja::json& GetJson() const { return Json; }

private:

	// ============================================================================
	// JSON Conversion Methods (Bytecode -> JSON)
	// ============================================================================

	/** Converts an external variable description to JSON. */
	UE_API inja::json ToJson(const FRigVMGraphVariableDescription& InVariable);

	/** Converts a function (unit or dispatch) to JSON. */
	UE_API inja::json ToJson(const FRigVMFunction* InFunction);

	/** Converts a branch/block to JSON. */
	UE_API inja::json ToJson(const FRigVMBranchInfo& InBlock);

	/** Converts a predicate branch to JSON. */
	UE_API inja::json ToJson(const FRigVMPredicateBranch& InPredicateBranch);

	/** Converts a callable (user function) to JSON. */
	UE_API inja::json ToJson(const FRigVMCallableInfo* InCallable);

	/** Converts a callable argument to JSON. */
	UE_API inja::json ToJson(const FRigVMCallableArgument& InCallableArgument);

	/** Converts a range of instructions to JSON. */
	UE_API inja::json ToJson(const FRigVMInstructionArray& InInstructions, int32 InFirstIndex, int32 InLastIndex);

	/** Converts a single instruction to JSON. */
	UE_API inja::json ToJson(const FRigVMInstruction& InInstruction);

	/** Converts a memory storage struct (literal or work memory) to JSON. */
	UE_API inja::json ToJson(const FRigVMMemoryStorageStruct& InMemoryStorageStruct);

	/** Converts a property path to JSON. */
	UE_API inja::json ToJson(const FRigVMPropertyPath& InPropertyPaths);

	/** Converts a property with direction and default value to JSON. */
	UE_API inja::json ToJson(ERigVMPinDirection InDirection, const FProperty* InProperty, const FString& InDefaultValue, int32 InMemoryType);

	/** Converts an operand reference to JSON with access expression. */
	UE_API inja::json ToJson(const FRigVMOperand& InOperand, const ERigVMPinDirection& InDirection, int32 InInstructionIndex);

	// ============================================================================
	// Enum/Primitive JSON Conversion
	// ============================================================================

	/** Converts an enum value to its display name as JSON. */
	template<typename EnumType>
	static inja::json ToJson(int64 InEnumValue, bool bTrimWhiteSpace = false)
	{
		const UEnum* Enum = StaticEnum<EnumType>();
		return ToJson(Enum, InEnumValue, bTrimWhiteSpace);
	}

	/** Converts an enum value to its display name as JSON. */
	UE_API static inja::json ToJson(const UEnum* InEnum, int64 InEnumValue, bool bTrimWhiteSpace = false);

	/** Converts FString to inja::json (UTF-8 encoded). */
	UE_API static inja::json ToJson(const FString& InString);

	/** Converts FName to inja::json. */
	UE_API static inja::json ToJson(const FName& InName);

	/** Converts FText to inja::json. */
	UE_API static inja::json ToJson(const FText& InText);

	/** Converts inja::json to FString. */
	UE_API static FString FromJson(const inja::json& InJson);

	/** Converts std::string to FString. */
	UE_API static FString FromJson(const std::string& InJson);

	// ============================================================================
	// Lazy Evaluation Support
	// ============================================================================

	/**
	 * Extracts lazy block information for a range of instructions.
	 *
	 * Lazy values are evaluated on-demand rather than eagerly. This method
	 * identifies which blocks contain lazy values and how to access them.
	 */
	UE_API inja::json GetLazyBlocks(int32 InFirstInstruction, int32 InLastInstruction, int32 CallableIndex, int32 BlockIndex);

	// ============================================================================
	// Dependency Tracking
	// ============================================================================

	/**
	 * Processes a type dependency and extracts include/library requirements.
	 *
	 * For native UE types, extracts the header path and module name.
	 * Returns false for non-native types (which cannot be nativized).
	 *
	 * @param InObject The UObject (typically UStruct, UClass, or UEnum) to process.
	 * @return True if the dependency is valid for nativization.
	 */
	UE_API bool ProcessDependency(const UObject* InObject);

	/** Adds an include path to the includes list. */
	UE_API void ProcessInclude(const FString& InInclude);

	// ============================================================================
	// Name Sanitization
	// ============================================================================

	/** Converts a string to a valid C++ identifier (replaces invalid chars with '_'). */
	UE_API static FString SanitizeName(const FString& InString);

	/** Sanitizes a native type name, handling 2D array remapping. */
	UE_API FString SanitizeNativeType(const FString& InString) const;

	/** Sanitizes a property name, ensuring uniqueness within a memory type. */
	UE_API FString SanitizePropertyName(const FString& InString, int32 InMemoryType);

	// ============================================================================
	// Type/Path Resolution
	// ============================================================================

	/** Computes a hash for a property path (used for deduplication). */
	UE_API static uint32 GetPropertyPathHash(const FRigVMPropertyPath& InPropertyPath);

	/** Populates JSON object with native path information for a property. */
	UE_API void GetNativePathForProperty(const FProperty* InProperty, inja::json& OutJsonObject);

	/** Gets the native path (UObject path name) as JSON. */
	UE_API inja::json GetNativePath(const UObject* InObject);

	/** Resolves the C++ type string for an operand. */
	UE_API FString GetNativeTypeForOperand(const FRigVMOperand& InOperand, int32 InInstructionIndex) const;

	/** Finds a UObject by its native path (cached lookup). */
	template<typename T>
	const T* FindObjectFromNativePath(const FString& InNativePath) const
	{
		return Cast<T>(FindObjectFromNativePath(InNativePath));
	}

	/** Finds a UObject by its native path (cached lookup). */
	UE_API const UObject* FindObjectFromNativePath(const FString& InNativePath) const;

	// ============================================================================
	// State
	// ============================================================================

	/** The editor asset being converted. */
	IRigVMEditorAssetInterface* EditorAsset = nullptr;

	/** The complete JSON representation of the VM. */
	inja::json Json;

	/** Conversion settings. */
	FRigVMCodeConversionSettings Settings;

	/** Accumulated include paths for generated code. */
	TArray<FString> Includes;

	/** Accumulated library/module dependencies. */
	TArray<FString> Libraries;

	/** Set of already-processed dependencies (avoids duplicates). */
	TSet<const UObject*> KnownDependencies;

	/** Sanitized package name (used for class naming). */
	FString PackageName;

	/** The RigVM being converted. */
	URigVM* VM = nullptr;

	/** Direct reference to VM's bytecode. */
	const FRigVMByteCode* ByteCode = nullptr;

	/** All blocks (branches) including synthesized RunInstruction blocks. */
	TArray<FRigVMBranchInfo> Blocks;

	/** All instructions from bytecode. */
	FRigVMInstructionArray Instructions;

	/** Maps instruction index -> containing block index. */
	TArray<int32> BlockIndexPerInstruction;

	/** Maps instruction index -> containing callable index. */
	TArray<int32> CallableIndexPerInstruction;

	/** Maps (first instruction, last instruction) -> branch info index for RunInstruction blocks. */
	TMap<TTuple<int32,int32>, int32> RunInstructionBranches;

	/** Tracks unique property names per memory type for deduplication. */
	TSet<TTuple<int32,FString>> UniquePropertyNames;

	/** Maps (memory type, register index) -> exported property index. */
	TMap<TTuple<int32,int32>, int32> PropertyMap;

	/** Maps (memory type, register index) -> (callable index, argument index, argument name). */
	TMap<TTuple<int32,int32>, TTuple<int32, int32, FName>> CallableArgumentMap;

	/** Maps property path hash -> property path index. */
	TMap<uint32, int32> PropertyPathMap;

	/** Maps function pointer -> function index (for deduplication). */
	TMap<const void*, int32> FunctionIndexMap;

	/** Maps instruction index -> custom function (for dispatch permutations). */
	TMap<int32, TSharedPtr<FRigVMFunction>> InstructionToCustomFunction;

	/** Maps type hash -> custom function (for dispatch permutations). */
	TMap<uint32, TSharedPtr<FRigVMFunction>> HashToCustomFunction;

	/** Set of (function index, argument index) pairs that are lazy values. */
	TSet<TTuple<int32,int32>> LazyFunctionArguments;

	/** Cache: native path string -> UObject. */
	mutable TMap<FString, const UObject*> NativePathMap;

	/** Cache: original type -> sanitized type (for 2D array remapping). */
	mutable TMap<FString, FString> NativeTypeMap;

	/** Accumulated lazy property definitions. */
	mutable TArray<TTuple<FString,FString>> LazyProperties;

	/** True if an error occurred during conversion. */
	bool bHasError = false;

	friend struct FRigVMCodeGenerator;
	friend class FRigVMCodeEnvironment;
};

#undef UE_API
