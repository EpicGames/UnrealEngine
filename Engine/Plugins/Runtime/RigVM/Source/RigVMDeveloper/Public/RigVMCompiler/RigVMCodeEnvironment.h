// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include <inja/inja.hpp>
#include "EulerTransform.h"

#define UE_API RIGVMDEVELOPER_API

class FRigVMCodeConverter;

/**
 * Custom Inja template environment providing C++ code generation helpers.
 *
 * Extends inja::Environment to register custom callback functions that can be
 * invoked from within Inja templates. These functions handle UE-specific types,
 * indentation management, and C++ code generation patterns.
 *
 * ## Template Function Reference
 *
 * ### String Utilities
 * - `sanitize(name)` - Converts a name to a valid C++ identifier
 * - `starts_with(str, prefix)` - Returns true if str starts with prefix
 * - `ends_with(str, suffix)` - Returns true if str ends with suffix
 * - `remove_whitespace(str)` - Removes all spaces and tabs from str
 * - `format_number(format, value)` - Printf-style number formatting (e.g., "%04d")
 *
 * ### Indentation Management
 * - `tabs()` - Returns current indentation as tab characters
 * - `inc_tabs()` - Increases indentation level by 1 (void, no return)
 * - `dec_tabs()` - Decreases indentation level by 1 (void, no return)
 *
 * ### JSON Utilities
 * - `has_key(object, key)` - Returns true if object contains key with truthy value
 * - `count_key(array, key)` - Counts array elements that contain key with truthy value
 *
 * ### C++ Code Generation
 * - `cpp_get_default_for_native_type(type)` - Returns default value for a C++ type
 *   (e.g., "FVector" -> "FVector::ZeroVector", "bool" -> "true")
 * - `cpp_get_initialize_code_for_property(memType, name, origName, nativeType, nativePath, default)`
 *   Returns array of C++ initialization statements for a property
 * - `cpp_get_block_to_run_operand(instruction)` - Resolves the BlockToRun operand for
 *   control flow instructions
 * - `cpp_is_default_value(nativeType, defaultValue)` - Returns true if the value matches
 *   the header inline default, allowing constructor init to be skipped
 *
 * ## Example Template Usage
 * @code
 * {{ tabs() }}{{ sanitize(Property.Name) }} = {{ cpp_get_default_for_native_type(Property.NativeType) }};
 * {% inc_tabs() %}
 * ##for Prop in Properties
 * {{ tabs() }}// Initialize {{ Prop.Name }}
 * ##endfor
 * {% dec_tabs() %}
 * @endcode
 *
 * @see FRigVMCodeConverter for the JSON data structure passed to templates
 */
class FRigVMCodeEnvironment : public inja::Environment
{
public:
	/**
	 * Constructs the environment and registers all custom template functions.
	 *
	 * @param InTemplateFolder Base folder for template includes (passed to inja::Environment).
	 * @param InConverter Back-reference to converter for context-dependent lookups.
	 */
	UE_API FRigVMCodeEnvironment(const FString& InTemplateFolder, FRigVMCodeConverter* InConverter);

private:

	// ============================================================================
	// String Utility Callbacks
	// ============================================================================

	/** Sanitizes a string to be a valid C++ identifier. Replaces invalid chars with '_'. */
	UE_API static inja::json sanitize(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Returns true if Arg0 starts with Arg1. */
	UE_API static inja::json starts_with(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Returns true if Arg0 ends with Arg1. */
	UE_API static inja::json ends_with(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Removes all whitespace (spaces and tabs) from Arg0. */
	UE_API static inja::json remove_whitespace(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Printf-style formatting: format_number("%04d", 42) -> "0042". */
	UE_API static inja::json format_number(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	// ============================================================================
	// Indentation Management Callbacks
	// ============================================================================

	/** Returns the current indentation as a string of tab characters. */
	UE_API inja::json tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Increments the indentation level. */
	UE_API void inc_tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Decrements the indentation level. */
	UE_API void dec_tabs(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	// ============================================================================
	// JSON Utility Callbacks
	// ============================================================================

	/** Returns true if Arg0 (object) contains Arg1 (key) with a truthy value. */
	UE_API static inja::json has_key(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/** Counts elements in Arg0 (array) that contain Arg1 (key) with a truthy value. */
	UE_API static inja::json count_key(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	// ============================================================================
	// C++ Code Generation Callbacks
	// ============================================================================

	/**
	 * Returns the default value expression for a C++ native type.
	 *
	 * Examples:
	 *   - "bool" -> "true"
	 *   - "float" -> "0.f"
	 *   - "FVector" -> "FVector::ZeroVector"
	 *   - "FTransform" -> "FTransform::Identity"
	 *   - Enum types -> "(EnumType)0"
	 *   - Pointers -> "nullptr"
	 */
	UE_API inja::json cpp_get_default_for_native_type(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/**
	 * Generates C++ initialization code for a property.
	 *
	 * Handles complex types like structs, arrays, enums with proper initialization syntax.
	 * Returns an array of code lines that can be iterated in templates.
	 *
	 * @param Arg0 Memory type string ("Literal" or "Work")
	 * @param Arg1 Sanitized property name
	 * @param Arg2 Original property name
	 * @param Arg3 Native C++ type
	 * @param Arg4 Native path for type lookup
	 * @param Arg5 Default value string
	 */
	UE_API inja::json cpp_get_initialize_code_for_property(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/**
	 * Resolves the BlockToRun operand for Execute instructions with control flow.
	 *
	 * Used to generate the operand access expression for instructions that branch
	 * to different blocks based on runtime values.
	 *
	 * @param Arg0 Instruction JSON object (must have Index and Operands keys)
	 */
	UE_API inja::json cpp_get_block_to_run_operand(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	/**
	 * Checks if a property's default value matches the header inline default.
	 *
	 * Used to skip redundant constructor initialization when the value equals
	 * what the header already initializes to. This reduces code size and improves
	 * constructor performance.
	 *
	 * @param Arg0 Native C++ type (e.g., "FVector", "FTransform", "bool")
	 * @param Arg1 Default value string to check
	 * @return true if the value matches the header default and can be skipped
	 */
	UE_API inja::json cpp_is_default_value(inja::Renderer* InRenderer, inja::Arguments& InArguments);

	// ============================================================================
	// JSON Conversion Helpers
	// ============================================================================

	/** Converts FString to inja::json. */
	UE_API static inja::json ToJson(const FString& InString);

	/** Converts FName to inja::json. */
	UE_API static inja::json ToJson(const FName& InName);

	/** Converts FText to inja::json. */
	UE_API static inja::json ToJson(const FText& InText);

	/** Converts inja::json to FString. */
	UE_API static FString FromJson(const inja::json& InJson);

	/** Converts std::string to FString. */
	UE_API static FString FromJson(const std::string& InJson);

	// ============================================================================
	// State
	// ============================================================================

	/** Current indentation level for tabs() output. */
	int32 Tabs = 0;

	/** Back-reference to converter for type lookups and bytecode access. */
	FRigVMCodeConverter* Converter = nullptr;

	/** Local cache for looking up native types by name */
	TMap<FString, UObject*> NativeTypeLookup;

	/** Set of C++ integer type names for default value lookup. */
	inline static const TSet<FString> IntegerTypes = {TEXT("uint8"), TEXT("uint16"), TEXT("uint32"), TEXT("uint64"), TEXT("int16"), TEXT("int32"), TEXT("int64")};

	/** Map of native type names to their default value expressions. O(1) lookup. */
	inline static const TMap<FString, FString> NativeTypeDefaults = {
		{TEXT("bool"), TEXT("true")},
		{TEXT("float"), TEXT("0.f")},
		{TEXT("double"), TEXT("0.0")},
		{TEXT("uint8"), TEXT("0")},
		{TEXT("uint16"), TEXT("0")},
		{TEXT("uint32"), TEXT("0")},
		{TEXT("uint64"), TEXT("0")},
		{TEXT("int16"), TEXT("0")},
		{TEXT("int32"), TEXT("0")},
		{TEXT("int64"), TEXT("0")},
		{TEXT("FName"), TEXT("NAME_None")},
		{TEXT("FVector2D"), TEXT("FVector2D::ZeroVector")},
		{TEXT("FVector"), TEXT("FVector::ZeroVector")},
		{TEXT("FQuat"), TEXT("FQuat::Identity")},
		{TEXT("FRotator"), TEXT("FRotator::ZeroRotator")},
		{TEXT("FTransform"), TEXT("FTransform::Identity")},
		{TEXT("FEulerTransform"), TEXT("FEulerTransform::Identity")},
		{TEXT("FMatrix"), TEXT("FMatrix::Identity")},
		{TEXT("FLinearColor"), TEXT("FLinearColor(ForceInitToZero)")}
	};

	inline static const FLinearColor DefaultLinearColor = FLinearColor(ForceInitToZero);

	/** Map of native type names to their default value expressions. O(1) lookup. */
	inline static const TMap<FString, const void*> NativeStructTypeDefaults = {
		{TEXT("FVector2D"), &FVector2D::ZeroVector},
		{TEXT("FVector"), &FVector::ZeroVector},
		{TEXT("FQuat"), &FQuat::Identity},
		{TEXT("FRotator"), &FRotator::ZeroRotator},
		{TEXT("FTransform"), &FTransform::Identity},
		{TEXT("FEulerTransform"), &FEulerTransform::Identity},
		{TEXT("FMatrix"), &FMatrix::Identity},
		{TEXT("FLinearColor"), &DefaultLinearColor}
	};
};

#undef UE_API
