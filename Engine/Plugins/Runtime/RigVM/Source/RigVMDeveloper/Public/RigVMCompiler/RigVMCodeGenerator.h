// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCodeConverter.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * Supported target languages for code generation.
 */
namespace ERigVMCodeLanguage
{
	enum Type : uint8
	{
		/** Generate C++ source files (.h and .cpp). */
		CPlusPlus,

		/** Generate Verse source files (not yet implemented). */
		Verse
	};
};

/**
 * Discovers and manages code generation templates for a target language.
 *
 * FRigVMCodeGenerator is responsible for:
 * - Discovering template files from the Content/CodeGeneration/{Language}/ directory
 * - Providing access to templates by name
 * - Orchestrating code generation by combining a converter with templates
 *
 * Usage:
 * @code
 *   // Create generator for C++
 *   FRigVMCodeGenerator Generator(ERigVMCodeLanguage::CPlusPlus);
 *
 *   // Create converter from a RigVM asset
 *   auto Converter = MakeShared<FRigVMCodeConverter>(EditorAsset, Settings);
 *
 *   // Generate all output files
 *   TArray<TSharedPtr<FRigVMCodeOutput>> Outputs = Generator.GenerateAll(Converter);
 *
 *   // Or generate a specific template
 *   auto Output = Generator.Generate(Converter, TEXT("AssetNameVM.h.template"));
 * @endcode
 *
 * @see FRigVMCodeConverter for bytecode parsing and JSON generation
 * @see FRigVMCodeTemplate for template file representation
 */
struct FRigVMCodeGenerator
{
public:

	/**
	 * Constructs a generator and discovers all templates for the specified language.
	 *
	 * Templates are loaded from: Content/CodeGeneration/{LanguageName}/{FileName}.template
	 * For C++, this is: Content/CodeGeneration/CPlusPlus/{FileName}.template
	 *
	 * @param InLanguage The target language for code generation.
	 */
	UE_API FRigVMCodeGenerator(ERigVMCodeLanguage::Type InLanguage);

	/** Returns the number of discovered templates. */
	UE_API int32 NumTemplates() const;

	/**
	 * Retrieves a template by name.
	 *
	 * @param InName Template name (e.g., "AssetNameVM.h.template").
	 * @return The template, or an invalid shared pointer if not found.
	 */
	UE_API const TSharedPtr<FRigVMCodeTemplate>& GetTemplate(const FString& InName) const;

	/**
	 * Generates output from a single template.
	 *
	 * @param InConverter The converter containing parsed bytecode as JSON.
	 * @param InTemplateName Name of the template to render.
	 * @return The rendered output, or nullptr if the template was not found.
	 */
	UE_API TSharedPtr<FRigVMCodeOutput> Generate(const TSharedPtr<FRigVMCodeConverter>& InConverter, const FString& InTemplateName) const;

	/**
	 * Generates output from a template (static version).
	 *
	 * @param InConverter The converter containing parsed bytecode as JSON.
	 * @param InCodeTemplate The template to render.
	 * @return The rendered output.
	 */
	UE_API static TSharedPtr<FRigVMCodeOutput> Generate(const TSharedPtr<FRigVMCodeConverter>& InConverter, const TSharedPtr<FRigVMCodeTemplate>& InCodeTemplate);

	/**
	 * Generates output from all discovered templates.
	 *
	 * @param InConverter The converter containing parsed bytecode as JSON.
	 * @return Array of rendered outputs, one per template.
	 */
	UE_API TArray<TSharedPtr<FRigVMCodeOutput>> GenerateAll(const TSharedPtr<FRigVMCodeConverter>& InConverter) const;

private:

	/** The target language for this generator. */
	ERigVMCodeLanguage::Type Language;

	/** All discovered templates for the target language. */
	TArray<TSharedPtr<FRigVMCodeTemplate>> CodeTemplates;
};

#undef UE_API
