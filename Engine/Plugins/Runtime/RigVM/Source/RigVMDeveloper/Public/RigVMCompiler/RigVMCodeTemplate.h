// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * Represents an Inja template file used for code generation.
 *
 * Templates are lazily loaded from disk when first accessed. The template content
 * is cached after loading to avoid repeated file I/O during rendering.
 *
 * Template files use Inja syntax (similar to Jinja2) and are typically stored in:
 *   Content/CodeGeneration/CPlusPlus/{FileName}.template
 *
 * @see FRigVMCodeOutput for rendered output handling
 * @see FRigVMCodeGenerator for template discovery and rendering orchestration
 */
struct FRigVMCodeTemplate
{
	/** Loads the template content from disk. Called automatically by LoadIfRequired(). */
	UE_API void Load();

	/** Loads the template content if not already loaded. Thread-safe lazy initialization. */
	UE_API void LoadIfRequired();

	/** Returns the template content. Triggers lazy load if needed. */
	UE_API const FString& GetContent() const;

	/** Template identifier (e.g., "AssetNameVM.h" or "AssetNameVM.cpp"). */
	FString Name;

	/** Absolute path to the template file on disk. */
	FString FilePath;

	/** Lazily-loaded template content. Mutable to allow const GetContent() to trigger load. */
	mutable TOptional<FString> Content;
};

/**
 * Represents the result of rendering a code template.
 *
 * Extends FRigVMCodeTemplate to hold both the rendered content and metadata about
 * the output file. Can be saved to disk via Save().
 *
 * After rendering:
 * - Content holds the generated C++ code (inherited from FRigVMCodeTemplate)
 * - ErrorMessage is set if rendering failed
 * - bSaved tracks whether Save() was called and succeeded
 */
struct FRigVMCodeOutput : public FRigVMCodeTemplate
{
	/**
	 * Writes the rendered content to disk.
	 *
	 * Output path is: OutputFolder / Name
	 * Sets bSaved to true on success, false on failure.
	 *
	 * @return True if the file was written successfully.
	 */
	UE_API bool Save() const;

	/** Directory where the output file should be written. */
	FString OutputFolder;

	/** Error message if rendering failed (empty on success). */
	FString ErrorMessage;

	/** Tracks save state: unset = not attempted, true = succeeded, false = failed. */
	TOptional<bool> bSaved;
};

#undef UE_API
