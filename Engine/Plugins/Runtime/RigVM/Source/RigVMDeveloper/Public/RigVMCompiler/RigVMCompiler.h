// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMModel/Nodes/RigVMParameterNode.h"
#include "RigVMModel/Nodes/RigVMCommentNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCompiler/RigVMAST.h"
#include "Logging/TokenizedMessage.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"

#include "RigVMCompiler.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMCompiler;
class URigVMFunctionReferenceNode;
struct FRigVMGraphFunctionData;
struct FRigVMGraphFunctionHeader;
struct FRigVMFunctionCompilationData;

USTRUCT(BlueprintType)
struct FRigVMCompileSettings
{
	GENERATED_BODY()

public:

	UE_API FRigVMCompileSettings();

	UE_API FRigVMCompileSettings(UScriptStruct* InExecuteContextScriptStruct);

	UScriptStruct* GetExecuteContextStruct() const { return ASTSettings.ExecuteContextStruct; }
	void SetExecuteContextStruct(UScriptStruct* InExecuteContextScriptStruct) { ASTSettings.ExecuteContextStruct = InExecuteContextScriptStruct; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool SurpressInfoMessages;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool SurpressWarnings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool SurpressErrors;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool EnablePinWatches;

	UPROPERTY(Transient)
	bool IsPreprocessorPhase;

	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category = FRigVMCompileSettings)
	FRigVMParserASTSettings ASTSettings;

	UPROPERTY()
	bool SetupNodeInstructionIndex;

	UPROPERTY()
	bool ASTErrorsAsNotifications;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool bWarnAboutDeprecatedNodes;

	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = FRigVMCompileSettings)
	bool bWarnAboutDuplicateEvents;

	static FRigVMCompileSettings Fast(UScriptStruct* InExecuteContextStruct = nullptr)
	{
		FRigVMCompileSettings Settings(InExecuteContextStruct);
		Settings.EnablePinWatches = true;
		Settings.IsPreprocessorPhase = false;
		Settings.ASTSettings = FRigVMParserASTSettings::Fast(InExecuteContextStruct);
		return Settings;
	}

	static FRigVMCompileSettings Optimized(UScriptStruct* InExecuteContextStruct = nullptr)
	{
		FRigVMCompileSettings Settings(InExecuteContextStruct);
		Settings.EnablePinWatches = false;
		Settings.IsPreprocessorPhase = false;
		Settings.ASTSettings = FRigVMParserASTSettings::Optimized(InExecuteContextStruct);
		return Settings;
	}

	UE_API void ReportInfo(const FString& InMessage) const;
	UE_API void ReportWarning(const FString& InMessage) const;
	UE_API void ReportError(const FString& InMessage) const;

	template <typename... Types>
		void ReportInfof(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportInfo(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
		void ReportWarningf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportWarning(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
		void ReportErrorf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportError(FString::Printf(Fmt, Args...));
	}

	void Report(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage) const
	{
		ASTSettings.Report(InSeverity, InSubject, InMessage);
	}

	template <typename... Types>
		void Reportf(EMessageSeverity::Type InSeverity,
								 UObject* InSubject,
								 UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt,
								 Types... Args) const
	{
		Report(InSeverity, InSubject, FString::Printf(Fmt, Args...));
	}
};

class FRigVMCompileSettingsDuringLoadGuard
{
public:

	FRigVMCompileSettingsDuringLoadGuard(FRigVMCompileSettings& InSettings)
		: ASTErrorsAsNotifications(InSettings.ASTErrorsAsNotifications, true)
	{}

private:

	TGuardValue<bool> ASTErrorsAsNotifications;
};

enum ERigVMCompilerPhase : uint8
{
	ERigVMCompilerPhase_SetupMemory,
	ERigVMCompilerPhase_BuildInstructions,
};

struct FRigVMCompilerWorkData
{
public:
	URigVMCompiler* Compiler = nullptr;
	FRigVMCompileSettings Settings;
	ERigVMCompilerPhase CompilerPhase = ERigVMCompilerPhase_SetupMemory;
	URigVM* VM = nullptr;
	TArray<URigVMGraph*> Graphs;
	const URigVMLibraryNode* CurrentlyCompilingFunctionNode = nullptr;
	FRigVMFunctionCompilationData* FunctionCompilationData = nullptr;
	TMap<uint32,TMap<FRigVMOperand,FRigVMOperand>> OperandMapPerFunction;
	UScriptStruct* ExecuteContextStruct = nullptr;
	TMap<FString, FRigVMOperand>* PinPathToOperand = nullptr;
	FRigVMExtendedExecuteContext* Context = nullptr;
	TMap<const FRigVMVarExprAST*, FRigVMOperand> ExprToOperand;
	TMap<const FRigVMExprAST*, bool> ExprComplete;
	TArray<const FRigVMExprAST*> ExprToSkip;
	TArray<const FRigVMExprAST*> TraversalExpressions;
	TMap<FString, int32> ProcessedLinks;
	TMap<const URigVMNode*, FRigVMOperand> TraitListLiterals;
	FRigVMOperand ComparisonOperand;

	using FRigVMASTProxyArray = TArray<FRigVMASTProxy, TInlineAllocator<3>>; 
	using FRigVMASTProxySourceMap = TMap<FRigVMASTProxy, FRigVMASTProxy>;
	using FRigVMASTProxyTargetsMap =
		TMap<FRigVMASTProxy, FRigVMASTProxyArray>;
	TMap<FRigVMASTProxy, FRigVMASTProxyArray> CachedProxiesWithSharedOperand;
	const FRigVMASTProxySourceMap* ProxySources = nullptr;
	FRigVMASTProxyTargetsMap ProxyTargets;
	TMap<URigVMNode*, TArray<FRigVMBranchInfo>> BranchInfos;

	struct FFunctionRegisterData
	{
		uint32 ObjectId;;
		ERigVMMemoryType MemoryType = ERigVMMemoryType::Invalid;
		int32 RegisterIndex = 0;

		friend inline uint32 GetTypeHash(const FFunctionRegisterData& Data)
		{
			uint32 Result = HashCombine(GetTypeHash(Data.ObjectId), GetTypeHash(Data.MemoryType));
			return HashCombine(Result, GetTypeHash(Data.RegisterIndex));
		}

		bool operator==(const FFunctionRegisterData& Other) const
		{
			return ObjectId == Other.ObjectId && MemoryType == Other.MemoryType && RegisterIndex == Other.RegisterIndex;
		}
	};
	TMap<FFunctionRegisterData, FRigVMOperand> FunctionRegisterToOperand; 

	TArray<URigVMPin*> WatchedPins;
	
	TMap<ERigVMMemoryType, TArray<FRigVMPropertyPathDescription>> PropertyPathDescriptions;
	TMap<ERigVMMemoryType, TArray<FRigVMPropertyDescription>> PropertyDescriptions;

	// literal register indices that belong to interactive nodes (e.g. nodes with DM active).
	// the FindOrAddProperty sharing scan skips these so that non-interactive nodes cannot
	// be mapped to an interactive node's register, which would cause DM writes to bleed
	// into nodes that are not being manipulated.
	TSet<int32> InteractiveLiteralIndices;

	UE_API FRigVMOperand AddProperty(ERigVMMemoryType InMemoryType, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue = FString());
	UE_API FRigVMOperand FindProperty(ERigVMMemoryType InMemoryType, const FName& InName) const;
	UE_API FRigVMOperand FindOrAddProperty(const ERigVMMemoryType InMemoryType, const FRigVMFunctionCompilationPropertyDescription& InProperty, const FString& InNewName, const bool bMakeUniqueProperty);
	UE_API FRigVMOperand FindOrAddProperty(ERigVMMemoryType InMemoryType, const FName& InName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, const bool bMakeUniqueProperty);
	UE_API FRigVMPropertyDescription GetProperty(const FRigVMOperand& InOperand);
	UE_API int32 FindOrAddPropertyPath(const FRigVMOperand& InOperand, const FString& InHeadCPPType, const FString& InSegmentPath);
	UE_API const FProperty* GetPropertyForOperand(const FRigVMOperand& InOperand) const;
	UE_API TRigVMTypeIndex GetTypeIndexForOperand(const FRigVMOperand& InOperand) const;
	UE_API FName GetUniquePropertyName(ERigVMMemoryType InMemoryType, const FName& InDesiredName) const;
	UE_API bool AddPropertiesForFunction(
		bool InIsCallable,
		const uint32& InHash,
		const FRigVMGraphFunctionHeader& InHeader,
		const FRigVMFunctionCompilationData* InCompilationData,
		int32 InFirstInstruction, int32 InLastInstruction,
		const TArray<FRigVMOperand>& InInterfaceOperands,
		const TMap<FRigVMOperand, FRigVMOperand>& InForwardedHandleToInterfaceOperand,
		const URigVMFunctionReferenceNode* InFunctionReferenceNode = nullptr);
	UE_API void SetupPropertyPathsForFoldedCopies(const FRigVMNodeExprAST* InExpr);

	UE_API bool RemapInstructionAt(
		const FRigVMInstruction& InOldInstruction,
		const FString& InFunctionIdentifier,
		const FRigVMFunctionCompilationData* InCompilationData,
		FRigVMByteCode& InNewByteCode,
		const FRigVMInstructionArray& InNewInstructions,
		const URigVMFunctionReferenceNode* InFunctionReferenceNode,
		const TFunction<int32(const int32&)>& InRemapInstructionIndex,
		const TFunction<FRigVMOperand(const FRigVMOperand&)>& InRemapOperand,
		const TFunction<int32(const int32&)>& InRemapBranchIndex,
		const TFunction<int32(const int32&)>& InRemapCallableIndex);

	TSharedPtr<FRigVMParserAST> AST;
	
	struct FCopyOpInfo
	{
		FRigVMCopyOp Op;
		const FRigVMAssignExprAST* AssignExpr = nullptr;
		const FRigVMVarExprAST* SourceExpr = nullptr;
		const FRigVMVarExprAST* TargetExpr = nullptr;
	};

	// operators that have been delayed for injection into the bytecode
	TMap<FRigVMOperand, FCopyOpInfo> DeferredCopyOps;

	struct FLazyBlockInfo
	{
		FLazyBlockInfo()
			: StartInstruction(INDEX_NONE)
			, EndInstruction(INDEX_NONE)
			, bProcessed(false)
		{}

		TOptional<uint32> Hash;
		FString BlockCombinationName;
		FRigVMOperand ExecuteStateOperand;
		int32 StartInstruction;
		int32 EndInstruction;
		TArray<const FRigVMExprAST*> Expressions;
		TArray<int32> RunInstructionsToUpdate;
		bool bProcessed;
	};

	TMap<uint32, TSharedPtr<FLazyBlockInfo>> LazyBlocks;
	TArray<uint32> LazyBlocksToProcess;
	TOptional<uint32> CurrentBlockHash;

	/**
	 * Tracks a JumpForward op emitted for a Break or Continue node that needs
	 * its relative offset patched once the enclosing loop's EndBlock is emitted.
	 */
	struct FPendingLoopJump
	{
		int32 ByteOffset;       // Byte offset of the JumpForward op in the bytecode stream (returned by AddJumpOp)
		int32 InstructionIndex; // Instruction index of the JumpForward op (for relative offset calculation)
	};

	/**
	 * Maps each enclosing loop's CallExtern expression to the list of Break/Continue
	 * JumpForward ops that must be patched to jump to that loop's EndBlock.
	 * Entries are consumed (removed) when the loop's EndBlock is emitted.
	 */
	TMap<const FRigVMCallExternExprAST*, TArray<FPendingLoopJump>> PendingLoopJumps;

	UE_API void ReportInfo(const FString& InMessage) const;
	UE_API void ReportWarning(const FString& InMessage) const;
	UE_API void ReportError(const FString& InMessage) const;

	template <typename... Types>
	void ReportInfof(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportInfo(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportWarningf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportWarning(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportErrorf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportError(FString::Printf(Fmt, Args...));
	}

	FRigVMReportDelegate OriginalReportDelegate;
	UE_API void OverrideReportDelegate(bool& bEncounteredASTError, bool& bSurpressedASTError);
	UE_API void RemoveOverrideReportDelegate();

	void Clear()
	{
		if (VM)
		{
			if (Context)
			{
				VM->ClearMemory(*Context);
				VM->Reset(*Context);
			}
			else
			{
				VM->Reset_Internal();
			}
		}
		Graphs.Reset();
		if (PinPathToOperand)
		{
			PinPathToOperand->Reset();
		}
		ExprToOperand.Reset();
		ExprComplete.Reset();
		ExprToSkip.Reset();
		ProcessedLinks.Reset();
		CachedProxiesWithSharedOperand.Reset();
		ProxyTargets.Reset();
		BranchInfos.Reset();
		FunctionRegisterToOperand.Reset();
		WatchedPins.Reset();
		PropertyDescriptions.Reset();
		PropertyPathDescriptions.Reset();
		DeferredCopyOps.Reset();
		PendingLoopJumps.Reset();
	}
};

DECLARE_DELEGATE_RetVal_OneParam(const FRigVMFunctionCompilationData*, FRigVMCompiler_GetFunctionCompilationData, const FRigVMGraphFunctionHeader& Header);

UCLASS(MinimalAPI, BlueprintType)
class URigVMCompiler : public UObject
{
	GENERATED_BODY()

public:

	UE_API URigVMCompiler();

	UPROPERTY()
	FRigVMCompileSettings Settings_DEPRECATED;

	UFUNCTION(BlueprintCallable, Category = FRigVMCompiler, meta=(DeprecatedFunction, DeprecationMessage="Compile is deprecated, use CompileVM with Context parameter."))
	bool Compile(TArray<URigVMGraph*> InGraphs, URigVMController* InController, URigVM* OutVM) { return false; }

	UFUNCTION(BlueprintCallable, Category = FRigVMCompiler)
	bool CompileVM(TArray<URigVMGraph*> InGraphs, URigVMController* InController, URigVM* OutVM, FRigVMExtendedExecuteContext& Context)
	{
		return Compile(Settings_DEPRECATED, InGraphs, InController, OutVM, Context, TArray<FRigVMExternalVariable>(), nullptr);
	}

	UE_API bool Compile(const FRigVMCompileSettings& InSettings, TArray<URigVMGraph*> InGraphs, URigVMController* InController, URigVM* OutVM, FRigVMExtendedExecuteContext& OutVMContext, const TArray<FRigVMExternalVariable>& InExternalVariables, TMap<FString, FRigVMOperand>* OutOperands, TSharedPtr<FRigVMParserAST> InAST = TSharedPtr<FRigVMParserAST>(), FRigVMFunctionCompilationData* OutFunctionCompilationData = nullptr);

	UE_API bool CompileFunction(const FRigVMCompileSettings& InSettings, const URigVMLibraryNode* InLibraryNode, URigVMController* InController, const TArray<FRigVMExternalVariable>& InExternalVariables, FRigVMFunctionCompilationData* OutFunctionCompilationData, FRigVMExtendedExecuteContext& OutVMContext);

	UE_API bool CompileCallable(FRigVMCompilerWorkData& WorkData, URigVM* OutVM, FRigVMExtendedExecuteContext& OutVMContext, const FRigVMCallableInfo* InCallable, const FRigVMGraphFunctionHeader& InHeader, const FRigVMFunctionCompilationData* InFunctionCompilationData, const URigVMFunctionReferenceNode* InFunctionReferenceNode = nullptr);

	FRigVMCompiler_GetFunctionCompilationData GetFunctionCompilationData;

	struct FCompiledFunctionData
	{
		const URigVMFunctionReferenceNode* FunctionReferenceNode;
		const FRigVMFunctionCompilationData* CompilationData;
	};
	
	TMap<FString, FCompiledFunctionData> CompiledFunctions;

	static UE_API UScriptStruct* GetScriptStructForCPPType(const FString& InCPPType);
	static UE_API FString GetPinHash(const URigVMPin* InPin, const FRigVMVarExprAST* InVarExpr, const URigVMLibraryNode* FunctionCompiling = nullptr, const FRigVMASTProxy& InPinProxy = FRigVMASTProxy());

	// follows assignment expressions to find the source ref counted containers
	// since ref counted containers are not copied for assignments.
	// this is currently only used for arrays.
	static UE_API const FRigVMVarExprAST* GetSourceVarExpr(const FRigVMExprAST* InExpr);

	UE_API void MarkDebugWatch(FRigVMExtendedExecuteContext& Context, bool bRequired, URigVMPin* InPin, URigVM* OutVM, TMap<FString, FRigVMOperand>* OutOperands, TSharedPtr<FRigVMParserAST> InRuntimeAST);

private:

	const URigVMLibraryNode* CurrentlyCompilingFunctionNode = nullptr;
	TSet<const URigVMLibraryNode*> FunctionCompilationStack;

	UE_API TArray<URigVMPin*> GetLinkedPins(URigVMPin* InPin, bool bInputs = true, bool bOutputs = true, bool bRecursive = true);
	UE_API int32 GetElementSizeFromCPPType(const FString& InCPPType, UScriptStruct* InScriptStruct);

	static UE_API FString GetPinHashImpl(const URigVMPin* InPin, const FRigVMVarExprAST* InVarExpr, const URigVMLibraryNode* FunctionCompiling = nullptr, const FRigVMASTProxy& InPinProxy = FRigVMASTProxy());

	UE_API bool TraverseExpression(const FRigVMExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseChildren(const FRigVMExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseBlock(const FRigVMBlockExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseEntry(const FRigVMEntryExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseCallExtern(const FRigVMCallExternExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseInlineFunction(const FRigVMInlineFunctionExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseCallableFunction(const FRigVMInlineFunctionExprAST* InExpr, const uint32 InCallableHash, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseNoOp(const FRigVMNoOpExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseVar(const FRigVMVarExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseLiteral(const FRigVMVarExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseAssign(const FRigVMAssignExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseCopy(const FRigVMCopyExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseCachedValue(const FRigVMCachedValueExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseExit(const FRigVMExitExprAST* InExpr, FRigVMCompilerWorkData& WorkData);
	UE_API bool TraverseInvokeEntry(const FRigVMInvokeEntryExprAST* InExpr, FRigVMCompilerWorkData& WorkData);

	UE_API void AddCopyOperator(
		const FRigVMCopyOp& InOp,
		const FRigVMAssignExprAST* InAssignExpr,
		const FRigVMVarExprAST* InSourceExpr,
		const FRigVMVarExprAST* InTargetExpr,
		FRigVMCompilerWorkData& WorkData,
		bool bDelayCopyOperations = true);

	UE_API void AddCopyOperator(
		const FRigVMCompilerWorkData::FCopyOpInfo& CopyOpInfo,
		FRigVMCompilerWorkData& WorkData,
		bool bDelayCopyOperations = true);

	UE_API FRigVMOperand FindOrAddRegister(const FRigVMVarExprAST* InVarExpr, FRigVMCompilerWorkData& WorkData);
	UE_API const FRigVMCompilerWorkData::FRigVMASTProxyArray& FindProxiesWithSharedOperand(const FRigVMVarExprAST* InVarExpr, FRigVMCompilerWorkData& WorkData);

	static UE_API FString GetPinNameWithDirectionPrefix(const URigVMPin* Pin);
	static UE_API int32 GetOperandFunctionInterfaceParameterIndex(const TArray<FString>& OperandsPinNames, const FRigVMFunctionCompilationData* FunctionCompilationData, const FRigVMOperand& Operand);

	UE_API bool ValidateNode(const FRigVMCompileSettings& InSettings, URigVMNode* InNode, bool bCheck = true);
	
	UE_API void ReportInfo(const FRigVMCompileSettings& InSettings, const FString& InMessage);
	UE_API void ReportWarning(const FRigVMCompileSettings& InSettings, const FString& InMessage);
	UE_API void ReportError(const FRigVMCompileSettings& InSettings, const FString& InMessage);

	template <typename... Types>
	void ReportInfof(const FRigVMCompileSettings& InSettings, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		ReportInfo(InSettings, FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportWarningf(const FRigVMCompileSettings& InSettings, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		ReportWarning(InSettings, FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportErrorf(const FRigVMCompileSettings& InSettings, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		ReportError(InSettings, FString::Printf(Fmt, Args...));
	}
	
	friend class FRigVMCompilerImportErrorContext;
	friend class FFunctionCompilationScope;
	friend class SRigVMGraphNode;
};

class FFunctionCompilationScope
{
public:
	FFunctionCompilationScope(URigVMCompiler* InCompiler, const URigVMLibraryNode* InLibraryNode)
		: Compiler(InCompiler), LibraryNode(InLibraryNode)
	{
		InCompiler->FunctionCompilationStack.Add(InLibraryNode);
	}

	~FFunctionCompilationScope()
	{
		Compiler->FunctionCompilationStack.Remove(LibraryNode);
	}

private:
	URigVMCompiler* Compiler;
	const URigVMLibraryNode* LibraryNode;
};

#undef UE_API
