// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVM.h: Module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

RIGVMDEVELOPER_API DECLARE_LOG_CATEGORY_EXTERN(LogRigVMDeveloper, Log, All);

