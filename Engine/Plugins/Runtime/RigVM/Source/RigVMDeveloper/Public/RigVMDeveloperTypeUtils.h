// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMDeveloperModule.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "EdGraphSchema_K2.h"
#include "RigVMTypeUtils.h"
#include "RigVMCore/RigVMFunction.h"

struct FRigVMGraphVariableDescription;
class URigVMPin;

namespace RigVMTypeUtils
{
#if WITH_EDITOR
	RIGVMDEVELOPER_API FEdGraphPinType PinTypeFromTypeIndex(const TRigVMTypeIndex& InTypeIndex);

	RIGVMDEVELOPER_API FEdGraphPinType PinTypeFromCPPType(const FName& InCPPType, UObject* InCPPTypeObject);

	RIGVMDEVELOPER_API FEdGraphPinType PinTypeFromExternalVariable(const FRigVMExternalVariable& InExternalVariable);

	RIGVMDEVELOPER_API FEdGraphPinType PinTypeFromRigVMVariableDescription(const FRigVMGraphVariableDescription& InVariableDescription);

	RIGVMDEVELOPER_API FEdGraphPinType SubPinType(const FEdGraphPinType& InPinType, const FString& SegmentPath);

	RIGVMDEVELOPER_API bool CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, UObject** OutCPPTypeObject, bool bGetBaseCPPType = false);

	RIGVMDEVELOPER_API bool CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, FName& OutCPPTypeObjectPath, bool bGetBaseCPPType = false);

	RIGVMDEVELOPER_API bool CPPTypeFromPin(URigVMPin* InPin, FString& OutCPPType, FString& OutCPPTypeObjectPath, bool bGetBaseCPPType = false);

	RIGVMDEVELOPER_API bool CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, UObject** OutCPPTypeObject);

	RIGVMDEVELOPER_API bool CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, FName& OutCPPTypeObjectPath);

	RIGVMDEVELOPER_API bool CPPTypeFromPinType(const FEdGraphPinType& InPinType, FString& OutCPPType, FString& OutCPPTypeObjectPath);

	RIGVMDEVELOPER_API TRigVMTypeIndex TypeIndexFromPinType(const FEdGraphPinType& InPinType);

	RIGVMDEVELOPER_API bool AreCompatible(const FName& InCPPTypeA, UObject* InCPPTypeObjectA, const FName& InCPPTypeB, UObject* InCPPTypeObjectB);

	RIGVMDEVELOPER_API bool AreCompatible(const FRigVMExternalVariable& InTypeA, const FRigVMExternalVariable& InTypeB, const FString& InSegmentPathA = FString(), const FString& InSegmentPathB = FString());
	
	RIGVMDEVELOPER_API bool AreCompatible(const FEdGraphPinType& InTypeA, const FEdGraphPinType& InTypeB, const FString& InSegmentPathA = FString(), const FString& InSegmentPathB = FString());
	
	RIGVMDEVELOPER_API const TArray<TRigVMTypeIndex>& GetAvailableCasts(const TRigVMTypeIndex& InTypeIndex, bool bAsInput);

	RIGVMDEVELOPER_API bool CanCastTypes(const TRigVMTypeIndex& InSourceTypeIndex, const TRigVMTypeIndex& InTargetTypeIndex);
	
	RIGVMDEVELOPER_API const FRigVMFunction* GetCastForTypeIndices(const TRigVMTypeIndex& InSourceTypeIndex, const TRigVMTypeIndex& InTargetTypeIndex);

	RIGVMDEVELOPER_API const FLazyName& GetCastTemplateValueName();

	RIGVMDEVELOPER_API const FLazyName& GetCastTemplateResultName();

	RIGVMDEVELOPER_API const FLazyName& GetCastTemplateNotation();

	// Get a user-facing name for an argument type
	RIGVMDEVELOPER_API FText GetDisplayTextForArgumentType(const FRigVMTemplateArgumentType& InType);
#endif
}
