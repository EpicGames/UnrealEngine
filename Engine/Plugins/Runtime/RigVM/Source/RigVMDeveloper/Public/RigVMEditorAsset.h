// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMRuntimeAsset.h"
#include "RigVMCore/RigVM.h"
#include "RigVMHost.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMExternalDependency.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMSettings.h"
#include "Blueprint/BlueprintExtension.h"
#include "UObject/AssetRegistryTagsContext.h"
#if WITH_EDITOR
#include "HAL/CriticalSection.h"
#include "Kismet2/CompilerResultsLog.h"
#endif

#include "RigVMEditorAsset.generated.h"

#define UE_API RIGVMDEVELOPER_API

#if WITH_EDITOR
class URigVMEdGraph;
class UEdGraphPin;
class IRigVMEditorModule;
namespace UE::RigVM::Editor::Tools
{
	class FFilterByAssetTag;
}
#endif
struct FEndLoadPackageContext;
struct FRigVMMemoryStorageStruct;
struct FGuardSkipDirtyBlueprintStatus;

class URigVMEditorAssetInterface;
class IRigVMEditorAssetInterface;
typedef TScriptInterface<IRigVMEditorAssetInterface> FRigVMEditorAssetInterfacePtr;

using IRigVMAssetInterface = IRigVMEditorAssetInterface;
using URigVMAssetInterface = URigVMEditorAssetInterface;
using FRigVMAssetInterfacePtr = FRigVMEditorAssetInterfacePtr;

DECLARE_EVENT_ThreeParams(IRigVMEditorAssetInterface, FOnRigVMCompiledEvent, UObject*, URigVM*, FRigVMExtendedExecuteContext&);
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMRefreshEditorEvent, FRigVMEditorAssetInterfacePtr);
DECLARE_EVENT_FourParams(IRigVMEditorAssetInterface, FOnRigVMVariableDroppedEvent, UObject*, FProperty*, const FVector2D&, const FVector2D&);
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMExternalVariablesChanged, const TArray<FRigVMExternalVariable>&);
DECLARE_EVENT_TwoParams(IRigVMEditorAssetInterface, FOnRigVMNodeDoubleClicked, FRigVMEditorAssetInterfacePtr, URigVMNode*);
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMGraphImported, UEdGraph*);
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMPostEditChangeChainProperty, FPropertyChangedChainEvent&);
DECLARE_EVENT_FourParams(IRigVMEditorAssetInterface, FOnRigVMLocalizeFunctionDialogRequested, FRigVMGraphFunctionIdentifier&, URigVMController*, IRigVMGraphFunctionHost*, bool);
DECLARE_EVENT_ThreeParams(IRigVMEditorAssetInterface, FOnRigVMReportCompilerMessage, EMessageSeverity::Type, UObject*, const FString&);
DECLARE_DELEGATE_RetVal_FourParams(FRigVMController_BulkEditResult, FRigVMOnBulkEditDialogRequestedDelegate, FRigVMEditorAssetInterfacePtr, URigVMController*, URigVMLibraryNode*, ERigVMControllerBulkEditType);
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMOnBreakLinksDialogRequestedDelegate, TArray<URigVMLink*>);
DECLARE_DELEGATE_RetVal_OneParam(TRigVMTypeIndex, FRigVMOnPinTypeSelectionRequestedDelegate, const TArray<TRigVMTypeIndex>&);
DECLARE_EVENT(IRigVMEditorAssetInterface, FOnRigVMBreakpointAdded);
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMRequestInspectObject, const TArray<UObject*>& );
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMRequestInspectMemoryStorage, const TArray<FRigVMMemoryStorageStruct*>&);
DECLARE_EVENT_OneParam( IRigVMEditorAssetInterface, FOnRigVMAssetChangedEvent, class UObject * );
DECLARE_EVENT_OneParam(IRigVMEditorAssetInterface, FOnRigVMSetObjectBeingDebugged, UObject* /*InDebugObj*/);

USTRUCT(MinimalAPI)
struct FRigVMPythonSettings
{
	GENERATED_BODY();

	FRigVMPythonSettings()
	{
	}
};

UENUM(BlueprintType)
enum class ERigVMTagDisplayMode : uint8
{
	None = 0,
	All = 0x001,
	DeprecationOnly = 0x002,
	Last = DeprecationOnly UMETA(Hidden), 
};

USTRUCT(MinimalAPI)
struct FRigVMEdGraphDisplaySettings
{
	GENERATED_BODY();

	UE_API FRigVMEdGraphDisplaySettings();
	UE_API ~FRigVMEdGraphDisplaySettings();

	// When enabled shows the first node instruction index
	// matching the execution stack window.
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	bool bShowNodeInstructionIndex;

	// When enabled shows the node counts both in the graph view as
	// we as in the execution stack window.
	// The number on each node represents how often the node has been run.
	// Keep in mind when looking at nodes in a function the count
	// represents the sum of all counts for each node based on all
	// references of the function currently running.
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	bool bShowNodeRunCounts;

	// A lower limit for counts for nodes used for debugging.
	// Any node lower than this count won't show the run count.
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	int32 NodeRunLowerBound;

	// A upper limit for counts for nodes used for debugging.
	// If a node reaches this count a warning will be issued for the
	// node and displayed both in the execution stack as well as in the
	// graph. Setting this to <= 1 disables the warning.
	// Note: The count limit doesn't apply to functions / collapse nodes.
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	int32 NodeRunLimit;

	// The duration in microseconds of the fastest instruction / node
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings", transient, meta = (EditCondition = "!bAutoDetermineRange"))
	double MinMicroSeconds;

	// The duration in microseconds of the slowest instruction / node
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings", transient, meta = (EditCondition = "!bAutoDetermineRange"))
	double MaxMicroSeconds;

	// The total duration of the last execution of the rig
	UPROPERTY(VisibleAnywhere, Category = "Graph Display Settings", transient)
	double TotalMicroSeconds;

	// If you set this to more than 1 the results will be averaged across multiple frames
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings", meta = (UIMin=1, UIMax=256))
	int32 AverageFrames;

	TArray<double> MinMicroSecondsFrames;
	TArray<double> MaxMicroSecondsFrames;
	TArray<double> TotalMicroSecondsFrames;

	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	bool bAutoDetermineRange;

	UPROPERTY(transient)
	double LastMinMicroSeconds;

	UPROPERTY(transient)
	double LastMaxMicroSeconds;

	// The color of the fastest instruction / node
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	FLinearColor MinDurationColor;

	// The color of the slowest instruction / node
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	FLinearColor MaxDurationColor;

	// The color of the slowest instruction / node
	UPROPERTY(EditAnywhere, Category = "Graph Display Settings")
	ERigVMTagDisplayMode TagDisplayMode;

	UE_API void SetTotalMicroSeconds(double InTotalMicroSeconds);
	UE_API void SetLastMinMicroSeconds(double InMinMicroSeconds);
	UE_API void SetLastMaxMicroSeconds(double InMaxMicroSeconds);
	UE_API double AggregateAverage(TArray<double>& InFrames, double InPrevious, double InNext) const;
};

UINTERFACE(MinimalAPI, BlueprintType)
class URigVMEditorAssetInterface : public UInterface
{
	GENERATED_BODY()
};

class IRigVMEditorAssetInterface 
{
	GENERATED_BODY()

#if WITH_EDITOR
	
public:
	UE_API static FRigVMEditorAssetInterfacePtr GetInterfaceOuter(const UObject* InObject);

	UE_API virtual TScriptInterface<IRigVMRuntimeAssetInterface> GetRuntimeAssetInterface() const;
	virtual UClass* GetRuntimeAssetClass() const = 0;
	virtual UObject* GetAssetObjectForEditor() = 0;
	
	virtual UObject* GetObject() = 0;
	const UObject* GetObject() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetObject(); }
	UE_API bool IsBlueprintAsset() const;
	UE_API virtual void BeginDestroy();

	UE_API TScriptInterface<IRigVMClientHost> GetRigVMClientHost();
	const TScriptInterface<IRigVMClientHost> GetRigVMClientHost() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetRigVMClientHost(); }

	FRigVMClient* GetRigVMClient() { return GetRigVMClientHost()->GetRigVMClient(); }
	const FRigVMClient* GetRigVMClient() const { return GetRigVMClientHost()->GetRigVMClient(); }
	UE_API void RequestClientHostRigVMInit();
	
	static inline const FLazyName RigVMPanelNodeFactoryName = FLazyName(TEXT("FRigVMEdGraphPanelNodeFactory"));
	static inline const FLazyName RigVMPanelPinFactoryName = FLazyName(TEXT("FRigVMEdGraphPanelPinFactory"));

	virtual void GetAllEdGraphs(TArray<UEdGraph*>& Graphs) const = 0;
	virtual TArray<FRigVMGraphVariableDescription> GetAssetVariables() const = 0;
	UE_API virtual FRigVMGraphVariableDescription FindAssetVariableByGuid(const FGuid& InGuid) const;
	virtual bool IsRegeneratingOnLoad() const = 0;

	virtual FOnRigVMPreVariablesChanged& OnPreVariablesChanged() = 0;
	virtual FOnRigVMPostVariablesChanged& OnPostVariablesChanged() = 0;
	virtual FProperty* FindGeneratedPropertyByName(const FName& InName) = 0;
	virtual const FProperty* FindGeneratedPropertyByName(const FName& InName) const { return const_cast<IRigVMEditorAssetInterface*>(this)->FindGeneratedPropertyByName(InName); }
	virtual FName AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue = FString()) = 0;
	virtual FName AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) = 0;
	virtual bool RemoveMemberVariable(const FName& InName) = 0;
	virtual bool BulkRemoveMemberVariables(const TArray<FName>& InNames) = 0;
	virtual bool RenameMemberVariable(const FName& InOldName, const FName& InNewName) = 0;
	virtual bool ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) = 0;
	virtual bool ChangeMemberVariableType(const FName& InName, const FEdGraphPinType& InType) = 0;
	virtual bool SetVariableIndex(const FName& InName, int32 NewIndex) = 0;
	virtual FText GetVariableTooltip(const FName& InName) const = 0;
	virtual FString GetVariableCategory(const FName& InName) = 0;
	virtual FString GetVariableMetadataValue(const FName& InName, const FName& InKey) = 0;
	virtual bool SetVariableCategory(const FName& InName, const FString& InCategory) = 0;
	virtual bool SetVariableTooltip(const FName& InName, const FText& InTooltip) = 0; 
	virtual bool SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn) = 0; 
	virtual bool SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics) = 0; 
	virtual bool SetVariablePrivate(const FName& InName, const bool bInPrivate) = 0; 
	virtual bool SetVariablePublic(const FName& InName, const bool bIsPublic) = 0; 
	virtual FString OnCopyVariable(const FName& InName) const = 0;
	virtual bool OnPasteVariable(const FString& InText) = 0;
	virtual const UStruct* GetVariablesStruct() = 0;
	virtual uint8* GetVariablesMemory() = 0;
	
	virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) = 0;
	virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetObjectBeingDebugged(bEvenIfPendingKill); }
	virtual const FString& GetObjectBeingDebuggedPath() const = 0;
	virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeCDO, bool bIncludeDerivedClass) const = 0;
	virtual UWorld* GetWorldBeingDebugged() const = 0;
	virtual void SetWorldBeingDebugged(UWorld* NewWorld) = 0;
	virtual ERigVMAssetStatus GetAssetStatus() const = 0;
	virtual bool IsUpToDate() const = 0;
	virtual URigVM* GetVM(bool bCreateIfNeeded) const = 0;
	virtual FRigVMDrawContainer& GetDrawContainer() = 0;
	const FRigVMDrawContainer& GetDrawContainer() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetDrawContainer(); }
	
	UE_API virtual bool ExportGraphToText(UEdGraph* InEdGraph, FString& OutText);
	UE_API virtual bool TryImportGraphFromText(const FString& InClipboardText, UEdGraph** OutGraphPtr = nullptr);
	UE_API virtual bool CanImportGraphFromText(const FString& InClipboardText);

	UE_API virtual void RefreshAllNodes() ;

	UE_API virtual void RecompileVM() ;
	UE_API virtual void RecompileVMIfRequired() ;
	UE_API virtual void RequestAutoVMRecompilation() ;
	virtual void SetAutoVMRecompile(bool bAutoRecompile)  { bAutoRecompileVM = bAutoRecompile; }
	virtual bool GetAutoVMRecompile() const  { return bAutoRecompileVM; }
	UE_API virtual void IncrementVMRecompileBracket();
	UE_API virtual void DecrementVMRecompileBracket();

	UE_API virtual void SetEditingLocked(bool InEditingLocked );
	UE_API virtual bool IsEditingLocked() const;
	UE_API virtual bool CanSetEditingLocked() const;
	UE_API virtual bool IsEditingLockedVisible() const;
	UE_API virtual FText GetEditingLockedLabel() const;
	UE_API virtual FText GetEditingLockedTooltip() const;

	UE_API virtual void SetObjectBeingDebugged(UObject* NewObject);
	virtual FOnRigVMSetObjectBeingDebugged& OnSetObjectBeingDebugged()  { return SetObjectBeingDebuggedEvent; }

	/** Returns the editor module to be used for this blueprint */
	UE_API virtual IRigVMEditorModule* GetEditorModule() const;

	/** Returns true if a given panel node factory is compatible this blueprint */
	UE_API virtual const FLazyName& GetPanelNodeFactoryName() const;

	virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() = 0;
	virtual const FRigVMRuntimeSettings& GetVMRuntimeSettings() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetVMRuntimeSettings();};
	
	
	UE_API static void QueueCompilerMessageDelegate(const FOnRigVMReportCompilerMessage::FDelegate& InDelegate);
	UE_API static void ClearQueuedCompilerMessageDelegates();

	UE_API virtual FRigVMGraphModifiedEvent& OnModified();
	UE_API FOnRigVMCompiledEvent& OnVMCompiled();

	virtual FOnRigVMReportCompilerMessage& OnReportCompilerMessage()  { return ReportCompilerMessageEvent; }

	/** Returns true if a given panel pin factory is compatible this blueprint */
	UE_API virtual const FLazyName& GetPanelPinFactoryName() const;

	virtual void AddPinWatch(UEdGraphPin* InPin) = 0;
	virtual bool IsPinBeingWatched(const UEdGraphPin* InPin) const = 0;
	virtual bool NodeContainsWatchedPins(const UEdGraphNode* InNode) const = 0;

	TMap<FString, FRigVMOperand> PinToOperandMap;

	/**
	 * Sets the instruction index to exit early on 
	 */
	UE_API bool SetEarlyExitInstruction(URigVMNode* InNodeToExitEarlyAfter, int32 InInstruction = INDEX_NONE, bool bRequestHyperLink = false);

	/**
	 * Resets / removes the instruction index to exit early on 
	 */
	UE_API bool ResetEarlyExitInstruction(bool bResetCallstack = true);

	/**
	 * Toggles the preview here functionality based on the selection
	 */
	UE_API void TogglePreviewHere(const URigVMGraph* InGraph);

	/**
	 * Steps forward in a preview here session
	 */
	UE_API void PreviewHereStepForward();

	/**
	 * Returns true if we can step forward in a preview here session
	 */
	UE_API bool CanPreviewHereStepForward() const;

	/**
	 * Returns true if we can step forward in a preview here session
	 */
	UE_API bool MarkNodeForPreviewHere(URigVMNode* InNode, int32 InInstructionIndex, bool bRequestHyperLink);

	/**
	 * Returns the instruction index of the currently selected node
	 */
	UE_API int32 GetPreviewNodeInstructionIndexFromSelection(const URigVMGraph* InGraph) const;

	virtual FRigVMController_RequestJumpToHyperlinkDelegate& OnRequestJumpToHyperlink()  { return RequestJumpToHyperlink; };

	/** Returns the settings defaults for this blueprint */
	UE_API URigVMEditorSettings* GetRigVMEditorSettings() const;

	UE_API virtual UEdGraph* GetEdGraph(const URigVMGraph* InModel) const;
	UE_API virtual UEdGraph* GetEdGraph(const FString& InNodePath) const;
	
	UE_API void AddVariableSearchMetaDataInfo(const FName InVariableName, TArray<UBlueprintExtension::FSearchTagDataPair>& OutTaggedMetaData) const;

	virtual FRigVMEdGraphDisplaySettings& GetRigGraphDisplaySettings() = 0;
	virtual const FRigVMEdGraphDisplaySettings& GetRigGraphDisplaySettings() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetRigGraphDisplaySettings(); };
	virtual FRigVMCompileSettings& GetVMCompileSettings() = 0;
	virtual const FRigVMCompileSettings& GetVMCompileSettings() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetVMCompileSettings(); };
	

	UE_API virtual URigVMHost* GetDebuggedRigVMHost();

	UE_API virtual URigVMGraph* GetFocusedModel() const;
	UE_API virtual TArray<URigVMGraph*> GetAllModels() const;
	UE_API virtual URigVMFunctionLibrary* GetLocalFunctionLibrary() const;
	UE_API virtual URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const;
	UE_API virtual URigVMGraph* GetModel(const FString& InNodePath) const;
	UE_API virtual URigVMController* GetControllerByName(const FString InGraphName = TEXT("")) const;
	UE_API virtual URigVMController* GetOrCreateController(URigVMGraph* InGraph = nullptr);
	UE_API virtual URigVMController* GetController(const UEdGraph* InEdGraph) const;
	UE_API virtual URigVMController* GetOrCreateController(const UEdGraph* InGraph);
	UE_API virtual URigVMController* GetController(const URigVMGraph* InGraph = nullptr) const;

	virtual FOnRigVMLocalizeFunctionDialogRequested& OnRequestLocalizeFunctionDialog()  { return RequestLocalizeFunctionDialog; }
	UE_API void BroadcastRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier InFunction, bool bForce = false);

	UE_API const FCompilerResultsLog& GetCompileLog() const;
	UE_API FCompilerResultsLog& GetCompileLog();
	UE_API void HandleReportFromCompiler(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage);

	// Returns a list of dependencies of this blueprint.
	// Dependencies are blueprints that contain functions used in this blueprint
	UE_API TArray<IRigVMEditorAssetInterface*> GetDependencies(bool bRecursive = false) const;

	// Returns a list of dependents as unresolved soft object pointers.
	// A dependent is a blueprint which uses a function defined in this blueprint.
	// This function is not recursive, since it avoids opening the asset.
	// Use GetDependentBlueprints as an alternative.
	UE_API TArray<FAssetData> GetDependentAssets() const;

	// Returns a list of dependents as resolved blueprints.
	// A dependent is a blueprint which uses a function defined in this blueprint.
	// If bOnlyLoaded is false, this function loads the dependent assets and can introduce a large cost
	// depending on the size / count of assets in the project.
	UE_API TArray<IRigVMEditorAssetInterface*> GetDependentResolvedAssets(bool bRecursive = false, bool bOnlyLoaded = false) const;

	UE_API void BroadcastRefreshEditor();
	virtual FOnRigVMRefreshEditorEvent& OnRefreshEditor()  { return RefreshEditorEvent; }

	UE_API FOnRigVMRequestInspectMemoryStorage& OnRequestInspectMemoryStorage();
	UE_API void RequestInspectMemoryStorage(const TArray<FRigVMMemoryStorageStruct*>& InMemoryStorageStructs) const;

	FOnRigVMPostEditChangeChainProperty& OnPostEditChangeChainProperty() { return PostEditChangeChainPropertyEvent; }
	UE_API void BroadcastPostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent);
	virtual void MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus = false) = 0;
	virtual void MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(nullptr)) = 0;

	UE_API virtual URigVMGraph* GetDefaultModel() const;
	virtual TArray<TObjectPtr<UEdGraph>>& GetUberGraphs() = 0;
	virtual const TArray<TObjectPtr<UEdGraph>>& GetUberGraphs() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetUberGraphs(); };

	/** Enables or disables profiling for this asset */
	UE_API void SetProfilingEnabled(const bool bEnabled);

	/** Returns the names of the plugins required to be loaded for this asset to work */
	virtual const TArray<FString>& GetRequiredPlugins(bool bRefresh = true) const = 0;
	virtual TArray<struct FEditedDocumentInfo>& GetLastEditedDocuments() = 0;
	virtual FRigVMPropertyBag* GetVariablesPropertyBag() = 0;

	virtual FRigVMVariant& GetAssetVariant() = 0;
	virtual const FRigVMVariant& GetAssetVariant() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetAssetVariant(); }
	virtual FRigVMVariantRef GetAssetVariantRef() const = 0;
	virtual void SetAssetStatus(const ERigVMAssetStatus& InStatus) = 0;	
	
protected:
	IRigVMEditorAssetInterface() {}
	UE_API IRigVMEditorAssetInterface(const FObjectInitializer& ObjectInitializer);

	virtual const TArray<FRigVMGraphFunctionHeader>& GetPublicGraphFunctions() const = 0;
	virtual void SetPublicGraphFunctions(const TArray<FRigVMGraphFunctionHeader>& InHeaders) = 0;
	
	virtual void RemovePinWatch(UEdGraphPin* InPin) = 0;
	virtual void ClearPinWatches() = 0;
	virtual void ForeachPinWatch(TFunctionRef<void(UEdGraphPin*)> Task) = 0;
	
	
	virtual UClass* GetRigVMGeneratedClassPrototype() const = 0;
	virtual UObject* GetDefaultsObject() = 0;
	virtual void SetObjectBeingDebuggedSuper(UObject* NewObject) = 0;
	virtual void SetUberGraphs(const TArray<TObjectPtr<UEdGraph>>& InGraphs) = 0;
	virtual void PostTransactedSuper(const FTransactionObjectEvent& TransactionEvent) = 0;
	virtual void ReplaceDeprecatedNodesSuper() = 0;
	virtual void PreDuplicateSuper(FObjectDuplicationParameters& DupParams) = 0;
	virtual void PostDuplicateSuper(bool bDuplicateForPIE) = 0;
	virtual void GetAssetRegistryTagsSuper(FAssetRegistryTagsContext Context) const = 0;
	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const = 0;
	virtual TArray<FRigVMExternalVariable> GetExternalVariables(bool bFallbackToBlueprint) const = 0;
	virtual FString GetVariableDefaultValue(const FName& InName, bool bFromDebuggedObject) const = 0;
	virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() = 0;
	virtual void PreCompile() = 0;
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName(bool bFromCDO) = 0;
	virtual const TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName(bool bFromCDO) const {return const_cast<IRigVMEditorAssetInterface*>(this)->GetUserDefinedStructGuidToPathName(bFromCDO);}
	virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName(bool bFromCDO) = 0;
	virtual const TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName(bool bFromCDO) const {return const_cast<IRigVMEditorAssetInterface*>(this)->GetUserDefinedEnumToPathName(bFromCDO);}
	virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse(bool bFromCDO) = 0;
	virtual FCompilerResultsLog* GetCurrentMessageLog() const = 0;
	virtual FRigVMDebugInfo& GetDebugInfo() = 0;
	virtual TObjectPtr<URigVMEdGraph>& GetFunctionLibraryEdGraph() = 0;
	virtual const TObjectPtr<URigVMEdGraph>& GetFunctionLibraryEdGraph() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetFunctionLibraryEdGraph(); }
	virtual TArray<TObjectPtr<UEdGraph>>& GetFunctionGraphs() = 0;
	virtual const TArray<TObjectPtr<UEdGraph>>& GetFunctionGraphs() const { return const_cast<IRigVMEditorAssetInterface*>(this)->GetFunctionGraphs();};
	virtual bool& IsReferencedObjectPathsStored() = 0;
	virtual void SetReferencedObjectPathsStored(bool bValue) { IsReferencedObjectPathsStored() = bValue; }
	virtual TArray<FSoftObjectPath>& GetReferencedObjectPaths() = 0;
	virtual TArray<FName>& GetSupportedEventNames() = 0;
	virtual void UpdateSupportedEventNames() = 0;
	virtual TArray<FRigVMReferenceNodeData>& GetFunctionReferenceNodeData() = 0;
	virtual void NotifyGraphRenamedSuper(class UEdGraph* Graph, FName OldName, FName NewName) = 0;
	virtual URigVMHost* CreateRigVMHostSuper(UObject* InOuter, FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags) = 0;
	virtual void AddUbergraphPage(URigVMEdGraph* RigVMEdGraph) = 0;
	virtual void AddLastEditedDocument(URigVMEdGraph* RigVMEdGraph) = 0;
	virtual void Compile() = 0;
	virtual FCompilerResultsLog CompileBlueprint() = 0;
	virtual void PostEditChangeBlueprintActors() = 0;
	virtual bool SplitAssetVariant() = 0;
	virtual bool JoinAssetVariant(const FGuid& InGuid) = 0;
	virtual TArray<FRigVMVariantRef> GetMatchingVariants() const = 0;
	virtual void SerializeSuper(FArchive& Ar) = 0;
	virtual void PostSerialize(FArchive& Ar) = 0;


	// Interface to URigVMExternalDependencyManager
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMClient* InClient) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionStore* InFunctionStore) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionData* InFunction) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionHeader* InHeader) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMFunctionCompilationData* InCompilationData) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMGraph* InGraph) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMNode* InNode) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMPin* InPin) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UStruct* InStruct) const;
	UE_API void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UEnum* InEnum) const;
	UE_API void CollectExternalDependenciesForCPPTypeObject(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UObject* InObject) const;
	// end of Interface to URigVMExternalDependencyManager

	UE_API void CommonInitialization(const FObjectInitializer& ObjectInitializer);
	
	UE_API void InitializeModelIfRequired(bool bRecompileVM = true);


	UE_API void SerializeImpl(FArchive& Ar);


	
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext);
	UE_API virtual void PostLoad();
	UE_API static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
	virtual bool IsPostLoadThreadSafe() const { return false; }
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent);
	UE_API virtual void ReplaceDeprecatedNodes();
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams);
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE);
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const;
	
	
	UE_API virtual bool RequiresForceLoadMembers(UObject* InObject) const;

	// UObject interface
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent);
	/** Called during cooking. Must return all objects that will be Preload()ed when this is serialized at load time. */
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	//  --- IRigVMClientHost interface Start---
	UE_API virtual UObject* GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const;
	UE_API virtual URigVMGraph* GetRigVMGraphForEditorObject(UObject* InObject) const;
	UE_API virtual void HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath);
	UE_API virtual void HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath);
	UE_API virtual void HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath);
	UE_API virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure);


	UE_API void GenerateUserDefinedDependenciesData(FRigVMExtendedExecuteContext& Context);

	

	// this is needed since even after load
	// model data can change while the RigVM BP is not opened
	// for example, if a user defined struct changed after BP load,
	// any pin that references the struct needs to be regenerated
	UE_API virtual void RefreshAllModels(ERigVMLoadType InLoadType = ERigVMLoadType::PostLoad);

	// RigVMRegistry changes can be triggered when new user defined types(structs/enums) are added/removed
	// in which case we have to refresh the model
	UE_API virtual void OnRigVMRegistryChanged();

	UE_API void RequestRigVMInit() const;
	

	UE_API virtual URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo = true);

	UE_API virtual URigVMGraph* AddModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	UE_API virtual bool RemoveModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true);

	UE_API virtual FRigVMGetFocusedGraph& OnGetFocusedGraph();
	UE_API virtual const FRigVMGetFocusedGraph& OnGetFocusedGraph() const;


	UE_API virtual TArray<FString> GeneratePythonCommands();
	virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) = 0;

	virtual void SetupPinRedirectorsForBackwardsCompatibility() {};

	

	UE_API virtual bool IsFunctionPublic(const FName& InFunctionName) const;
	UE_API virtual void MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic = true);

	UE_API virtual void RenameGraph(const FString& InNodePath, const FName& InNewName);

	//  --- IRigVMClientHost interface End ---

	//  --- IRigVMExternalDependencyManager interface Start ---

	UE_API virtual TArray<FRigVMExternalDependency> GetExternalDependenciesForCategory(const FName& InCategory) const;
	
	//  --- IRigVMExternalDependencyManager interface End ---


	virtual FOnRigVMRequestInspectObject& OnRequestInspectObject() { return OnRequestInspectObjectEvent; }
	void RequestInspectObject(const TArray<UObject*>& InObjects) { OnRequestInspectObjectEvent.Broadcast(InObjects); }

	

	UE_API URigVMGraph* GetTemplateModel(bool bIsFunctionLibrary = false);
	UE_API URigVMController* GetTemplateController(bool bIsFunctionLibrary = false);


	

	

	mutable TArray<UObject::FAssetRegistryTag> CachedAssetTags;
	TArray<TPair<TWeakObjectPtr<URigVMNode>,int32>> LastPreviewHereNodes;
	

	bool bSuspendModelNotificationsForSelf;
	bool bSuspendAllNotifications;

	UE_API void RebuildGraphFromModel();

	UE_API URigVMHost* CreateRigVMHostImpl();
	
	UE_API virtual TArray<UStruct*> GetAvailableRigVMStructs() const;

	UE_API FRigVMVariantRef GetAssetVariantRefImpl() const;

	/** Resets the asset's guid to a new one and splits it from the former variant set */
	UE_API bool SplitAssetVariantImpl();

	/** Merges the asset's guid with a provided one to join the variant set */
	UE_API bool JoinAssetVariantImpl(const FGuid& InGuid);

	UE_API TArray<FRigVMVariantRef> GetMatchingVariantsImpl() const;

	

	bool bAutoRecompileVM;

	bool bVMRecompilationRequired;

	bool bIsCompiling;

	int32 VMRecompilationBracket;
	
	bool bSkipDirtyBlueprintStatus;

	FRigVMGraphModifiedEvent ModifiedEvent;
	UE_API void Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject);
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	FOnRigVMAssetChangedEvent ChangedEvent;
	virtual FOnRigVMAssetChangedEvent& OnChanged()  { return ChangedEvent; }

	FOnRigVMSetObjectBeingDebugged SetObjectBeingDebuggedEvent;
	
	
	UE_API void ReplaceFunctionIdentifiers(const FString& OldFunctionHostPath, const FString& NewFunctionHostPath, const FString& OldFunctionLibraryPath, const FString& NewFunctionLibraryPath);

	FOnRigVMRefreshEditorEvent RefreshEditorEvent;
	FOnRigVMVariableDroppedEvent VariableDroppedEvent;

	UE_API void SuspendNotifications(bool bSuspendNotifs);
	
	virtual FOnRigVMVariableDroppedEvent& OnVariableDropped()  { return VariableDroppedEvent; }


	FOnRigVMCompiledEvent VMCompiledEvent;

	virtual void PathDomainSpecificContentOnLoad() {}
	UE_API virtual void PatchBoundVariables();
	UE_API virtual void PatchVariableNodesWithIncorrectType();
	virtual void PatchParameterNodesOnLoad() {}
	UE_API virtual void PatchLinksWithCast();
	UE_API virtual void GetBackwardsCompatibilityPublicFunctions(TArray<FName> &BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders);

	UE_API virtual void CreateMemberVariablesOnLoad();
	UE_API virtual void PatchVariableNodesOnLoad();
	// [GuidDedupe] One-shot recovery for the corrupted-state where two member variables in the
	// bag ended up with identical FGuids (a real bug: AddHostMemberVariableFromExternal used to
	// accept whatever Guid the caller passed, so duplicate-paste of a variable carried the source's
	// Guid forward). Detect duplicates, regenerate Guids on later occurrences, and rewrite any
	// variable node that was pointing at the now-old Guid for the regenerated variable.
	// The Guid is the ground truth of variable identity; variable Name is used here ONLY as the
	// disambiguator at the point of corruption (i.e. "of the nodes referencing the duplicate Guid,
	// the ones whose Name pin equals the regenerated variable's name belonged to that variable").
	// Idempotent: detection scans (a TSet<FGuid> for the asset bag and one per graph) always run,
	// but no writes occur on a healthy asset and bDirtyDuringLoad is left untouched.
	UE_API virtual void RepairDuplicateVariableGuidsOnLoad();
	TMap<FName, int32> AddedMemberVariableMap;

	UE_API void PropagateRuntimeSettingsFromBPToInstances();
	UE_API void InitializeArchetypeInstances();

	UE_API void HandlePackageDone(const FEndLoadPackageContext& Context);

	UE_API virtual void HandlePackageDone();

	/** Our currently running rig vm instance */
	virtual TObjectPtr<URigVMHost>& GetEditorHost() = 0;

	// RigVMBP, once end-loaded, will inform other RigVM-Dependent systems that Host instances are ready.
	UE_API void BroadcastRigVMPackageDone();

	// Previously some memory classes were parented to the asset object
	// however it is no longer supported since classes are now identified 
	// with only package name + class name, see FTopLevelAssetPath
	// this function removes those deprecated class.
	// new classes should be created by RecompileVM and parented to the Package
	// during PostLoad
	UE_API void RemoveDeprecatedVMMemoryClass();

	// During load, we do not want the GC to destroy the generator classes until all URigVMMemoryStorage objects
	// are loaded, so we need to keep a pointer to the classes. These pointers will be removed on PreSave so that the
	// GC can do its work.
	TArray<TObjectPtr<URigVMMemoryStorageGeneratorClass>> OldMemoryStorageGeneratorClasses;

	FOnRigVMExternalVariablesChanged& OnExternalVariablesChanged() { return ExternalVariablesChangedEvent; }

	UE_API virtual void OnVariableAdded(const FName& InVarName);
	UE_API virtual void OnVariableRemoved(const FGuid& InVarGuid);
	UE_API virtual void OnVariableRenamed(const FGuid& InVarGuid, const FName& InNewVarName);
	UE_API virtual void OnVariableTypeChanged(const FGuid& InVarGuid, FEdGraphPinType InOldPinType, FEdGraphPinType InNewPinType);

	UE_API FName AddAssetVariableFromPinType(const FName& InName, const FEdGraphPinType& InType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = FString());
	
	virtual FOnRigVMNodeDoubleClicked& OnNodeDoubleClicked()  { return NodeDoubleClickedEvent; }
	UE_API void BroadcastNodeDoubleClicked(URigVMNode* InNode);

	virtual FOnRigVMGraphImported& OnGraphImported()  { return GraphImportedEvent; }
	UE_API void BroadcastGraphImported(UEdGraph* InGraph);

	

	
	virtual FRigVMOnBulkEditDialogRequestedDelegate& OnRequestBulkEditDialog()  { return RequestBulkEditDialog; }

	virtual FRigVMOnBreakLinksDialogRequestedDelegate& OnRequestBreakLinksDialog()  { return RequestBreakLinksDialog; }

	virtual FRigVMOnPinTypeSelectionRequestedDelegate& OnRequestPinTypeSelectionDialog()  { return RequestPinTypeSelectionDialog; }

	

	
	UE_API void BroadCastReportCompilerMessage(EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage);

	
	FOnRigVMExternalVariablesChanged ExternalVariablesChangedEvent;
	UE_API void BroadcastExternalVariablesChangedEvent();

	FOnRigVMNodeDoubleClicked NodeDoubleClickedEvent;
	FOnRigVMGraphImported GraphImportedEvent;
	FOnRigVMPostEditChangeChainProperty PostEditChangeChainPropertyEvent;
	FOnRigVMLocalizeFunctionDialogRequested RequestLocalizeFunctionDialog;
	FOnRigVMReportCompilerMessage ReportCompilerMessageEvent;
	FRigVMOnBulkEditDialogRequestedDelegate RequestBulkEditDialog;
	FRigVMOnBreakLinksDialogRequestedDelegate RequestBreakLinksDialog;
	FRigVMOnPinTypeSelectionRequestedDelegate RequestPinTypeSelectionDialog;
	FRigVMController_RequestJumpToHyperlinkDelegate RequestJumpToHyperlink;

	UE_API UEdGraph* CreateEdGraph(URigVMGraph* InModel, bool bForce = false);
	UE_API bool RemoveEdGraph(URigVMGraph* InModel);
	UE_API void DestroyObject(UObject* InObject);
	UE_API void CreateEdGraphForCollapseNodeIfNeeded(URigVMCollapseNode* InNode, bool bForce = false);
	UE_API bool RemoveEdGraphForCollapseNode(URigVMCollapseNode* InNode, bool bNotify = false);

	FCompilerResultsLog CompileLog;
	
	UE_API TArray<TScriptInterface<IRigVMGraphFunctionHost>> GetReferencedFunctionHosts(bool bForceLoad) const;

	FOnRigVMRequestInspectObject OnRequestInspectObjectEvent;
	FOnRigVMRequestInspectMemoryStorage OnRequestInspectMemoryStorageEvent;

	UE_API TArray<FRigVMReferenceNodeData> GetReferenceNodeData() const;
	UE_API virtual void SetupDefaultObjectDuringCompilation(URigVMHost* InCDO);


	static FSoftObjectPath PreDuplicateHostPath;
	static FSoftObjectPath PreDuplicateFunctionLibraryPath;
	UE_API static TArray<FRigVMEditorAssetInterfacePtr> sCurrentlyOpenedRigVMBlueprints;

	UE_API void MarkDirtyDuringLoad();

	UE_API bool IsMarkedDirtyDuringLoad() const;

	bool bDirtyDuringLoad;
	bool bErrorsDuringCompilation;
	bool bSuspendPythonMessagesForRigVMClient;
	bool bMarkBlueprintAsStructurallyModifiedPending;

	UE_API void UpdateDebugMemoryOnHost(URigVMHost* InHost);
	
	TOptional<TArray<TWeakObjectPtr<URigVMPin>>> WeakWatchedPins;

	static FCriticalSection QueuedCompilerMessageDelegatesMutex;
	static TArray<FOnRigVMReportCompilerMessage::FDelegate> QueuedCompilerMessageDelegates;



#endif
public:

protected:
	

	friend class IControlRigEditorAssetInterface;
	friend class FRigVMBlueprintCompilerContext;
	friend class FRigVMLegacyEditor;
	friend class FRigVMNewEditor;
	friend class FRigVMEditorBase;
	friend class FRigVMEditorModule;
	friend class URigVMEdGraphSchema;
	friend struct FRigVMEdGraphSchemaAction_PromoteToVariable;
	friend class URigVMBuildData;
	friend class FRigVMVariantDetailCustomization;
	friend class FRigVMTreeAssetVariantFilter;
	friend class FRigVMTreePackageNode;
	friend class SRigVMGraphNode;
	friend struct FGuardSkipDirtyBlueprintStatus;
	friend class SRigModuleAssetBrowser;
	friend class UE::RigVM::Editor::Tools::FFilterByAssetTag;
	friend class UEngineTestControlRig;
	friend class SRigCurveContainer;
	friend class SRigHierarchy;
	friend class FControlRigBaseEditor;
	friend class URigVMBlueprint;
	friend class SRigVMPythonLogDetails;
};

UCLASS(MinimalAPI, BlueprintType)
class URigVMEditorAsset : public UObject, public IRigVMEditorAssetInterface, public IRigVMClientHost
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FRigVMClient RigVMClient;

	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> RootEdGraphs;

	UPROPERTY()
	FRigVMEdGraphDisplaySettings GraphDisplaySettings;

	UPROPERTY()
	FRigVMCompileSettings CompileSettings;

	/** Set of documents that were being edited in this blueprint, so we can open them right away */
	UPROPERTY()
	TArray<struct FEditedDocumentInfo> LastEditedDocuments;

	/** If this asset is currently being compiled, the CurrentMessageLog will be the log currently being used to send logs to. */
	TUniquePtr<FCompilerResultsLog> CurrentMessageLog;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<URigVMEdGraph> FunctionLibraryEdGraph;

	/** Set of functions implemented for this class graphically */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UEdGraph>> FunctionGraphs;

	TArray<UEdGraphPin*> WatchedPin;

	UPROPERTY()
	bool bReferencedObjectPathsStored;

	UPROPERTY()
	TArray<FSoftObjectPath> ReferencedObjectPaths;

	/** Our currently running rig vm instance */
	// Declaring these transient properties here instead of the IRigVMEditorAssetInterface class in order to have them as UPROPERTY, to avoid having stale pointers when GC happens
	UPROPERTY(transient)
	TObjectPtr<URigVMHost> EditorHost = nullptr;
#endif

	UE_API URigVMEditorAsset();
	
	virtual UClass* GetRuntimeAssetClass() const override { return URigVMRuntimeAsset::StaticClass(); }
	UE_API URigVMRuntimeAsset* GetRuntimeAsset() const;
	virtual UObject* GetAssetObjectForEditor() override { return GetRuntimeAsset(); }

	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	UE_API virtual void PostLoad() override;
	virtual bool IsPostLoadThreadSafe() const override { return IRigVMEditorAssetInterface::IsPostLoadThreadSafe(); }
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
	UE_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	UE_API virtual void BeginDestroy() override;

#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass) { IRigVMEditorAssetInterface::DeclareConstructClasses(OutConstructClasses, SpecificSubclass); }
#endif

	virtual UObject* GetObject() override { return this; }
	UE_API virtual void GetAllEdGraphs(TArray<UEdGraph*>& Graphs) const override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual TArray<FRigVMGraphVariableDescription> GetAssetVariables() const override;
	virtual bool IsRegeneratingOnLoad() const override { return false; }
	UE_API virtual FOnRigVMPreVariablesChanged& OnPreVariablesChanged() override;
	UE_API virtual FOnRigVMPostVariablesChanged& OnPostVariablesChanged() override;
	UE_API virtual FProperty* FindGeneratedPropertyByName(const FName& InName) override;
	UE_API virtual FName AddHostMemberVariableFromExternal(FRigVMExternalVariable InVariableToCreate, FString InDefaultValue = FString()) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual FName AddMemberVariable(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool RemoveMemberVariable(const FName& InName) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool BulkRemoveMemberVariables(const TArray<FName>& InNames) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool RenameMemberVariable(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual bool ChangeMemberVariableType(const FName& InName, const FEdGraphPinType& InType) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool ChangeMemberVariableType(const FName& InName, const FString& InCPPType, bool bIsPublic = false, bool bIsReadOnly = false, FString InDefaultValue = TEXT("")) override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual bool SetVariableIndex(const FName& InName, int32 NewIndex) override;
	UE_API virtual bool SetVariableIndex(const FGuid& InVariableGuid, int32 NewIndex);
	UE_API virtual FText GetVariableTooltip(const FName& InName) const override;
	UE_API virtual FString GetVariableCategory(const FName& InName) override;
	UE_API virtual FString GetVariableMetadataValue(const FName& InName, const FName& InKey) override;
	UE_API virtual bool SetVariableCategory(const FName& InName, const FString& InCategory) override;
	UE_API virtual bool SetVariableTooltip(const FName& InName, const FText& InTooltip) override;
	UE_API virtual bool SetVariableExposeOnSpawn(const FName& InName, const bool bInExposeOnSpawn) override;
	UE_API virtual bool SetVariableExposeToCinematics(const FName& InName, const bool bInExposeToCinematics) override;
	UE_API virtual bool SetVariablePrivate(const FName& InName, const bool bInPrivate) override;
	UE_API virtual bool SetVariablePublic(const FName& InName, const bool bIsPublic) override;
	UE_API virtual FString OnCopyVariable(const FName& InName) const override;
	UE_API virtual bool OnPasteVariable(const FString& InText) override;
	UE_API virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) override;
	virtual UObject* GetObjectBeingDebugged(bool bEvenIfPendingKill = false) const override { return const_cast<URigVMEditorAsset*>(this)->GetObjectBeingDebugged(bEvenIfPendingKill); }
	UE_API virtual const FString& GetObjectBeingDebuggedPath() const override;
	UE_API virtual UWorld* GetWorldBeingDebugged() const override;
	UE_API virtual void SetWorldBeingDebugged(UWorld* NewWorld) override;
	UE_API virtual TArray<UObject*> GetArchetypeInstances(bool bIncludeCDO, bool bIncludeDerivedClass) const override;
	UE_API virtual ERigVMAssetStatus GetAssetStatus() const override;
	UE_API virtual void SetAssetStatus(const ERigVMAssetStatus& InStatus) override;
	UE_API virtual bool IsUpToDate() const override;
	//virtual UClass* GetRigVMGeneratedClass() override { return nullptr; }
	UE_API virtual URigVM* GetVM(bool bCreateIfNeeded) const override;
	UE_API virtual FRigVMRuntimeSettings& GetVMRuntimeSettings() override;
	UE_API virtual void AddPinWatch(UEdGraphPin* InPin) override;
	UE_API virtual void RemovePinWatch(UEdGraphPin* InPin) override;
	UE_API virtual bool IsPinBeingWatched(const UEdGraphPin* InPin) const override;
	UE_API virtual bool NodeContainsWatchedPins(const UEdGraphNode* InNode) const override;
	UE_API virtual void ClearPinWatches() override;
	UE_API virtual void ForeachPinWatch(TFunctionRef<void(UEdGraphPin*)> Task) override;
	virtual FRigVMEdGraphDisplaySettings& GetRigGraphDisplaySettings() override { return GraphDisplaySettings; }
	virtual FRigVMCompileSettings& GetVMCompileSettings() override { return CompileSettings; }
	UE_API virtual void MarkAssetAsStructurallyModified(bool bSkipDirtyAssetStatus = false) override;
	UE_API virtual void MarkAssetAsModified(FPropertyChangedEvent PropertyChangedEvent = FPropertyChangedEvent(nullptr)) override;
	virtual TArray<TObjectPtr<UEdGraph>>& GetUberGraphs() override { return RootEdGraphs; }
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual FRigVMVariant& GetAssetVariant() override;
	UFUNCTION(BlueprintCallable, Category = "VM")
	UE_API virtual FRigVMVariantRef GetAssetVariantRef() const override;
	UE_API virtual const TArray<FRigVMGraphFunctionHeader>& GetPublicGraphFunctions() const override;
	UE_API virtual void SetPublicGraphFunctions(const TArray<FRigVMGraphFunctionHeader>& InHeaders) override;
	virtual UClass* GetRigVMGeneratedClassPrototype() const override { return nullptr; }
	virtual TArray<struct FEditedDocumentInfo>& GetLastEditedDocuments() override { return LastEditedDocuments; }
	virtual UObject* GetDefaultsObject() override { return GetRuntimeAssetInterface().GetObject(); }
	UE_API void DebuggingWorldRegistrationHelper(UObject* ObjectProvidingWorld, UObject* ValueToRegister);
	UE_API virtual void SetObjectBeingDebuggedSuper(UObject* NewObject) override;
	virtual void SetUberGraphs(const TArray<TObjectPtr<UEdGraph>>& InGraphs) override { RootEdGraphs = InGraphs; }
	virtual void PostTransactedSuper(const FTransactionObjectEvent& TransactionEvent) override { UObject::PostTransacted(TransactionEvent); }
	virtual void ReplaceDeprecatedNodesSuper() override { ensure(false); }
	virtual void PreDuplicateSuper(FObjectDuplicationParameters& DupParams) override { UObject::PreDuplicate(DupParams); }
	UE_API virtual void PostDuplicateSuper(bool bDuplicateForPIE) override;
	virtual void GetAssetRegistryTagsSuper(FAssetRegistryTagsContext Context) const override { UObject::GetAssetRegistryTags(Context); }
	virtual bool RequiresForceLoadMembersSuper(UObject* InObject) const override { return true; }
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables(bool bFallbackToBlueprint) const override;
	UE_API virtual const UStruct* GetVariablesStruct() override;
	UE_API virtual uint8* GetVariablesMemory() override;
	UE_API virtual FString GetVariableDefaultValue(const FName& InName, bool bFromDebuggedObject) const override;
	UE_API virtual FRigVMExtendedExecuteContext* GetRigVMExtendedExecuteContext() override;
	virtual void PreCompile() override {  }
	UE_API virtual TMap<FString, FSoftObjectPath>& GetUserDefinedStructGuidToPathName(bool bFromCDO) override;
	UE_API virtual TMap<FString, FSoftObjectPath>& GetUserDefinedEnumToPathName(bool bFromCDO) override;
	UE_API virtual TSet<TObjectPtr<UObject>>& GetUserDefinedTypesInUse(bool bFromCDO) override;
	virtual FCompilerResultsLog* GetCurrentMessageLog() const override { return CurrentMessageLog.Get(); }
	UE_API virtual FRigVMDebugInfo& GetDebugInfo() override;
	virtual TObjectPtr<URigVMEdGraph>& GetFunctionLibraryEdGraph() override { return FunctionLibraryEdGraph; }
	virtual TArray<TObjectPtr<UEdGraph>>& GetFunctionGraphs() override { return FunctionGraphs; }
	virtual bool& IsReferencedObjectPathsStored() override { return bReferencedObjectPathsStored; }
	virtual TArray<FSoftObjectPath>& GetReferencedObjectPaths() override { return ReferencedObjectPaths; }
	UE_API virtual TArray<FName>& GetSupportedEventNames() override;
	UE_API virtual void UpdateSupportedEventNames() override;
	UE_API virtual TArray<FRigVMReferenceNodeData>& GetFunctionReferenceNodeData() override;
	virtual void NotifyGraphRenamedSuper(class UEdGraph* Graph, FName OldName, FName NewName) override {  }
	UFUNCTION(BlueprintCallable, Category = "VM")
	virtual URigVMHost* CreateRigVMHost() { return IRigVMEditorAssetInterface::CreateRigVMHostImpl(); }
	UE_API virtual URigVMHost* CreateRigVMHostSuper(UObject* InOuter, FName InName = NAME_None, EObjectFlags InFlags = RF_NoFlags) override;
	UE_API virtual void AddUbergraphPage(URigVMEdGraph* RigVMEdGraph) override;
	UE_API virtual void AddLastEditedDocument(URigVMEdGraph* RigVMEdGraph) override;
	virtual void Compile() override { ensure(false); }
	UE_API virtual FCompilerResultsLog CompileBlueprint() override;
	virtual void PostEditChangeBlueprintActors() override { ensure(false); }
	UFUNCTION(BlueprintCallable, Category = "Variants")
	virtual bool SplitAssetVariant() override { return IRigVMEditorAssetInterface::SplitAssetVariantImpl(); }
	UFUNCTION(BlueprintCallable, Category = "Variants")
	virtual bool JoinAssetVariant(const FGuid& InGuid) override { return IRigVMEditorAssetInterface::JoinAssetVariantImpl(InGuid); }
	UFUNCTION(BlueprintCallable, Category = "Variants")
	virtual TArray<FRigVMVariantRef> GetMatchingVariants() const override { return IRigVMEditorAssetInterface::GetMatchingVariantsImpl(); }
	virtual void Serialize(FArchive& Ar) override { IRigVMEditorAssetInterface::SerializeImpl(Ar); }
	virtual void SerializeSuper(FArchive& Ar) override { UObject::Serialize(Ar); }
	UE_API virtual void PostSerialize(FArchive& Ar) override;
	virtual TObjectPtr<URigVMHost>& GetEditorHost() override { return EditorHost; }
	virtual FString GetAssetName() const override { return GetName(); }
	virtual UClass* GetRigVMSchemaClass() const override  { return URigVMSchema::StaticClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const override { return FRigVMExecuteContext::StaticStruct(); }
	UE_API virtual UClass* GetRigVMEdGraphClass() const override;
	UE_API virtual UClass* GetRigVMEdGraphNodeClass() const override;
	UE_API virtual UClass* GetRigVMEdGraphSchemaClass() const override;
	UE_API virtual UClass* GetRigVMEditorSettingsClass() const override;
	virtual FRigVMClient* GetRigVMClient() override { return &RigVMClient; }
	virtual const FRigVMClient* GetRigVMClient() const override { return &RigVMClient; }
	virtual TScriptInterface<IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() override { return GetRuntimeAssetInterface().GetObject(); }
	virtual TScriptInterface<const IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() const override { return GetRuntimeAssetInterface().GetObject(); }
	virtual UObject* GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const override { return IRigVMEditorAssetInterface::GetEditorObjectForRigVMGraph(InVMGraph); }
	virtual URigVMGraph* GetRigVMGraphForEditorObject(UObject* InObject) const override { return IRigVMEditorAssetInterface::GetRigVMGraphForEditorObject(InObject); }
	virtual void HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphAdded(InClient, InNodePath); }
	virtual void HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphRemoved(InClient, InNodePath); }
	virtual void HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath) override { return IRigVMEditorAssetInterface::HandleRigVMGraphRenamed(InClient, InOldNodePath, InNewNodePath); }
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) override { return IRigVMEditorAssetInterface::HandleConfigureRigVMController(InClient, InControllerToConfigure); }
	virtual void IncrementVMRecompileBracket() override { return IRigVMEditorAssetInterface::IncrementVMRecompileBracket(); }
	virtual void DecrementVMRecompileBracket() override { return IRigVMEditorAssetInterface::DecrementVMRecompileBracket(); }
	virtual void RefreshAllModels(ERigVMLoadType InLoadType = ERigVMLoadType::PostLoad) override { return IRigVMEditorAssetInterface::RefreshAllModels(InLoadType); }
	virtual void OnRigVMRegistryChanged() override { return IRigVMEditorAssetInterface::OnRigVMRegistryChanged(); }
	virtual URigVMGraph* GetModel(const FString& InNodePath) const override { return IRigVMEditorAssetInterface::GetModel(InNodePath); }
	virtual FRigVMGetFocusedGraph& OnGetFocusedGraph() override { return IRigVMEditorAssetInterface::OnGetFocusedGraph(); }
	virtual const FRigVMGetFocusedGraph& OnGetFocusedGraph() const override { return IRigVMEditorAssetInterface::OnGetFocusedGraph(); }
	virtual void SetupPinRedirectorsForBackwardsCompatibility() override { return IRigVMEditorAssetInterface::SetupPinRedirectorsForBackwardsCompatibility(); }
	virtual FRigVMGraphModifiedEvent& OnModified() override { return IRigVMEditorAssetInterface::OnModified(); }
	virtual bool IsFunctionPublic(const FName& InFunctionName) const override { return IRigVMEditorAssetInterface::IsFunctionPublic(InFunctionName); }
	virtual void MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic = true) override { return IRigVMEditorAssetInterface::MarkFunctionPublic(InFunctionName, bIsPublic); }
	virtual void RenameGraph(const FString& InNodePath, const FName& InNewName) override { return IRigVMEditorAssetInterface::RenameGraph(InNodePath, InNewName); }
	
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	UE_API virtual void RecompileVM() override;
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual void RecompileVMIfRequired() override { IRigVMEditorAssetInterface::RecompileVMIfRequired(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual void RequestAutoVMRecompilation() override { IRigVMEditorAssetInterface::RequestAutoVMRecompilation(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual void SetAutoVMRecompile(bool bAutoRecompile) override { IRigVMEditorAssetInterface::SetAutoVMRecompile(bAutoRecompile); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual bool GetAutoVMRecompile() const override { return IRigVMEditorAssetInterface::GetAutoVMRecompile(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual void RequestRigVMInit() override { IRigVMEditorAssetInterface::RequestRigVMInit(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const override { return IRigVMEditorAssetInterface::GetModel(InEdGraph); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMGraph* GetDefaultModel() const override { return IRigVMEditorAssetInterface::GetDefaultModel(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual TArray<URigVMGraph*> GetAllModels() const override { return IRigVMEditorAssetInterface::GetAllModels(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMFunctionLibrary* GetLocalFunctionLibrary() const override { return IRigVMEditorAssetInterface::GetLocalFunctionLibrary(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo = true) override { return IRigVMEditorAssetInterface::GetOrCreateLocalFunctionLibrary(bSetupUndoRedo); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMGraph* AddModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override { return IRigVMEditorAssetInterface::AddModel(InName, bSetupUndoRedo, bPrintPythonCommand); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual bool RemoveModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) override { return IRigVMEditorAssetInterface::RemoveModel(InName, bSetupUndoRedo, bPrintPythonCommand); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMGraph* GetFocusedModel() const override { return IRigVMEditorAssetInterface::GetFocusedModel(); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMController* GetController(const URigVMGraph* InGraph = nullptr) const override { return IRigVMEditorAssetInterface::GetController(InGraph); }
	virtual URigVMController* GetController(const UEdGraph* InGraph) const override { return IRigVMEditorAssetInterface::GetController(InGraph); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMController* GetControllerByName(const FString InGraphName = TEXT("")) const override { return IRigVMEditorAssetInterface::GetControllerByName(InGraphName); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual URigVMController* GetOrCreateController(URigVMGraph* InGraph = nullptr) override { return IRigVMEditorAssetInterface::GetOrCreateController(InGraph); }
	virtual URigVMController* GetOrCreateController(const UEdGraph* InGraph) override { return IRigVMEditorAssetInterface::GetOrCreateController(InGraph); }
	UFUNCTION(BlueprintCallable, Category = "RigVM Asset")
	virtual TArray<FString> GeneratePythonCommands() override { return IRigVMEditorAssetInterface::GeneratePythonCommands(); }
	UE_DEPRECATED(5.8, "Use TArray<FString> GeneratePythonCommands() instead.")
	virtual TArray<FString> GeneratePythonCommands(const FString InNewBlueprintName) override { return TArray<FString>(); }
	UE_API virtual const TArray<FString>& GetRequiredPlugins(bool bRefresh = true) const override;
	UE_API virtual FRigVMPropertyBag* GetVariablesPropertyBag() override;
	UE_API virtual FRigVMDrawContainer& GetDrawContainer() override;
	virtual const FLazyName& GetPanelNodeFactoryName() const override { return IRigVMEditorAssetInterface::GetPanelNodeFactoryName(); }
	virtual const FLazyName& GetPanelPinFactoryName() const override { return IRigVMEditorAssetInterface::GetPanelPinFactoryName(); }
	
#if WITH_EDITOR
	UE_API virtual TArray<FString> GeneratePythonContextCommands(const FString InNewBlueprintName, bool bCreateAsset) override;
#endif
	
	FOnRigVMPreVariablesChanged DummyOnPreVariableChanged;
	FOnRigVMPostVariablesChanged DummyOnPostVariableChanged;
};

class FRigVMBlueprintCompileScope
{
public:
   
	UE_API FRigVMBlueprintCompileScope(FRigVMEditorAssetInterfacePtr InBlueprint);

	UE_API ~FRigVMBlueprintCompileScope();

private:

	FRigVMEditorAssetInterfacePtr Blueprint;
};

struct FGuardSkipDirtyBlueprintStatus : private FNoncopyable
{
	[[nodiscard]] FGuardSkipDirtyBlueprintStatus(FRigVMEditorAssetInterfacePtr InBlueprint, bool bNewValue)
	{
		if (InBlueprint)
		{
			WeakBlueprint = InBlueprint;
			bOldValue = InBlueprint->bSkipDirtyBlueprintStatus;
			InBlueprint->bSkipDirtyBlueprintStatus = bNewValue;
		}
	}
	~FGuardSkipDirtyBlueprintStatus()
	{
		if (UObject* Asset = WeakBlueprint.GetWeakObjectPtr().Get())
		{
			if (FRigVMEditorAssetInterfacePtr Interface = Asset)
			{
				Interface->bSkipDirtyBlueprintStatus = bOldValue;
			}
		}
	}

private:
	TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakBlueprint;
	bool bOldValue = false;
};

#undef UE_API