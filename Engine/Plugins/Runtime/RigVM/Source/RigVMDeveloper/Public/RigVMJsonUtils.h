// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMDispatchFactory.h"

class IRigVMEditorAssetInterface;
class URigVMGraph;
class URigVMNode;
class URigVMPin;
class URigVMLink;
class URigVMSchema;
class URigVMEdGraphSchema;
struct FRigVMGraphVariableDescription;

namespace RigVMJsonUtils
{
	template <class CharType>
	struct TRigVMJsonPrintPolicy
	: public TPrettyJsonPrintPolicy<CharType>
	{
	};

	typedef TJsonWriter< TCHAR, TRigVMJsonPrintPolicy< TCHAR > > FJsonWriter; 
	typedef TSharedRef< FJsonWriter > FJsonWriterRef;
	typedef TJsonWriterFactory< TCHAR, TRigVMJsonPrintPolicy< TCHAR > > FJsonFactory; 

	RIGVMDEVELOPER_API FJsonWriterRef MakeJsonWriter(FString& OutJsonText);

	struct FIntrospectionSettings
	{
		FIntrospectionSettings(FRigVMRegistryHandle& InRegistry);

		void AddDiscoveredType(const TRigVMTypeIndex& InTypeIndex);

		FRigVMRegistryHandle& Registry;
		TObjectPtr<URigVMSchema> Schema = nullptr;
		TObjectPtr<URigVMEdGraphSchema> EdGraphSchema = nullptr;
		bool bExcludeFunctions = false;
		bool bExcludeDispatches = false;
		bool bExcludeDeprecated = false;
		bool bMergeTemplates = false;
		bool bCreateImages = false;
		FString FilePath;
		TSet<TRigVMTypeIndex> DiscoveredTypes;
		TArray<FString> CommandsToRun;
	};

	// introspection for the registry
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> AvailableNodesToJson(FIntrospectionSettings& InSettings, TFunction<bool(const UScriptStruct*)> InFilterCallback);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const FRigVMFunction* InFunction);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const FRigVMDispatchFactory* InFactory);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const TRigVMTypeIndex& InTypeIndex);
	RIGVMDEVELOPER_API bool GenericStructAttributesToJson(FIntrospectionSettings& InSettings, const UScriptStruct* InStruct, FJsonObject* OutJsonObject);

	// introspection for existing graphs
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const IRigVMEditorAssetInterface* InAsset);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const URigVMGraph* InGraph, bool bOnlySelection = false);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const URigVMNode* InNode);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const URigVMPin* InPin);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const URigVMLink* InLink);
	RIGVMDEVELOPER_API TSharedPtr<FJsonObject> ToJson(FIntrospectionSettings& InSettings, const FRigVMGraphVariableDescription& InVariable);

	// string conversion
	RIGVMDEVELOPER_API FString StructValueToJson(const UScriptStruct* InScriptStruct, const void* InMemory);
}
