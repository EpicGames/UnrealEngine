// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMUnitNode.h"
#include "RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMAggregateNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Aggregate Node contains a subgraph of nodes with aggregate pins (2in+1out or 1out+2in) connected
 * to each other. For example, a unit node IntAdd which adds 2 integers and provides Result=A+B could have
 * A, B and Result as aggregates in order to add additional input pins to construct an Aggregate Node that computes
 * Result=A+B+C.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMAggregateNode : public URigVMCollapseNode
{
	GENERATED_BODY()

public:

	// default constructor
	UE_API URigVMAggregateNode();

	// URigVMUnitNode interface
	UE_API virtual FString GetNodeTitle() const override;
	UE_API virtual FLinearColor GetNodeColor() const override;
	UE_API virtual FName GetMethodName() const;
	UE_API virtual  FText GetToolTipText() const override;
	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	virtual bool IsAggregate() const override { return false; };
#endif
	UE_API virtual bool IsInputAggregate() const override;

	UE_API URigVMNode* GetFirstInnerNode() const;
	UE_API URigVMNode* GetLastInnerNode() const;
	
	UE_API virtual URigVMPin* GetFirstAggregatePin() const override;
	UE_API virtual URigVMPin* GetSecondAggregatePin() const override;
	UE_API virtual URigVMPin* GetOppositeAggregatePin() const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateInputs() const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateOutputs() const override;

protected:

	UE_API virtual FString GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const override;

private:

	UE_API virtual void InvalidateCache() override;

	mutable URigVMNode* FirstInnerNodeCache;
	mutable URigVMNode* LastInnerNodeCache;

	friend class URigVMController;
};

#undef UE_API
