// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMTemplateNode.h"
#include "RigVMArrayNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Array Node represents one of a series available
 * array operations such as SetNum, GetAtIndex etc.
 */
UCLASS(MinimalAPI, BlueprintType, Deprecated)
class UDEPRECATED_RigVMArrayNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API UDEPRECATED_RigVMArrayNode();

	// Returns the op code of this node
	UFUNCTION(BlueprintCallable, Category = RigVMArrayNode)
	UE_API ERigVMOpCode GetOpCode() const;

	// Returns the C++ data type of the element
	UFUNCTION(BlueprintCallable, Category = RigVMArrayNode)
	UE_API FString GetCPPType() const;

	// Returns the C++ data type struct of the array (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMArrayNode)
	UE_API UObject* GetCPPTypeObject() const;

private:

	UPROPERTY()
	ERigVMOpCode OpCode;

	friend class URigVMController;
	friend class URigVMCompiler;
	friend class FRigVMVarExprAST;
	friend class FRigVMParserAST;
};

#undef UE_API
