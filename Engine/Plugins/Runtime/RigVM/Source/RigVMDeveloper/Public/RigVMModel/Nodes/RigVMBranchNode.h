// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/RigVMNode.h"
#include "RigVMBranchNode.generated.h"

class UObject;

/**
 * A branch node is used to branch between two blocks of execution
 */
UCLASS(MinimalAPI, BlueprintType, Deprecated)
class UDEPRECATED_RigVMBranchNode : public URigVMNode
{
	GENERATED_BODY()
};

