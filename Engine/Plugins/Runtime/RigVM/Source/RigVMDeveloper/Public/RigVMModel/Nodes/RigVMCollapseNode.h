// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMCollapseNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Collapse Node is a library node which stores the 
 * function and its nodes directly within the node itself.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMCollapseNode : public URigVMLibraryNode
{
	GENERATED_BODY()

public:

	UE_API URigVMCollapseNode();

	// RigVM node interface
	UE_API virtual FText GetToolTipText() const override;

	// Library node interface
	virtual FString GetNodeCategory() const override { return NodeCategory; }
	virtual FString GetNodeKeywords() const override { return NodeKeywords; }
	virtual FString GetNodeDescription() const override { return NodeDescription; }
	UE_API virtual URigVMFunctionLibrary* GetLibrary() const override;
	bool IsGraphFunctionDefinition() const { return GetLibrary() != nullptr;}
	virtual URigVMGraph* GetContainedGraph() const override { return ContainedGraph; }
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables() const override;
	UE_API virtual bool IsPure() const override;

	UE_API FString GetEditorSubGraphName() const;

	UE_API FGuid FindPinGuid(const URigVMPin* InPin) const;
private:

	UPROPERTY()
	TObjectPtr<URigVMGraph> ContainedGraph;

	UPROPERTY()
	FString NodeCategory;

	UPROPERTY()
	FString NodeKeywords;

	UPROPERTY()
	FString NodeDescription;
	
	UPROPERTY()
	TMap<FName, FGuid> PinNameToGuid;

	friend class URigVMController;
	friend class FRigVMMinimalEnvironment;
};

#undef UE_API
