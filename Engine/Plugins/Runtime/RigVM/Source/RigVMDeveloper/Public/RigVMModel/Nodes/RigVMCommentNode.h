// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "RigVMModel/RigVMNode.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMCommentNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UObject;
struct FFrame;

/**
 * Comment Nodes can be used to annotate a Graph by adding
 * colored grouping as well as user provided text.
 * Comment Nodes are purely cosmetic and don't contribute
 * to the runtime result of the Graph / Function.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMCommentNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMCommentNode();

	// Override of node title
	virtual FString GetNodeTitle() const override { return GetCommentText(); }

	// Returns the current user provided text of this comment.
	UFUNCTION(BlueprintCallable, Category = RigVMCommentNode)
	UE_API FString GetCommentText() const;

	// Returns the current user provided font size of this comment.
	UFUNCTION(BlueprintCallable, Category = RigVMCommentNode)
	UE_API int32 GetCommentFontSize() const;

	// Returns the current user provided bubble visibility of this comment.
	UFUNCTION(BlueprintCallable, Category = RigVMCommentNode)
	UE_API bool GetCommentBubbleVisible() const;

	// Returns the current user provided bubble color inheritance of this comment.
	UFUNCTION(BlueprintCallable, Category = RigVMCommentNode)
	UE_API bool GetCommentColorBubble() const;

private:

	UPROPERTY()
	FString CommentText;

	UPROPERTY()
	int32 FontSize;

	UPROPERTY()
	bool bBubbleVisible;

	UPROPERTY()
	bool bColorBubble;

	friend class URigVMController;
};

#undef UE_API
