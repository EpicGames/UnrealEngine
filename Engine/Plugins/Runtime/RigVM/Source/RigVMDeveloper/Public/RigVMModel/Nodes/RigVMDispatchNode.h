// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMCore/RigVMStructUpgradeInfo.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMDispatchNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UObject;
class URigVMPin;
class UScriptStruct;
struct FRigVMDispatchFactory;

/**
 * The Struct Node represents a Function Invocation of a RIGVM_METHOD
 * declared on a USTRUCT. Struct Nodes have input / output pins for all
 * struct UPROPERTY members.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMDispatchNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Override node functions
	UE_API virtual FString GetNodeTitle() const override;
	UE_API virtual bool SupportsRenaming() const override;
	UE_API virtual FText GetToolTipText() const override;
	UE_API virtual FLinearColor GetNodeColor() const override;
	UE_API virtual bool IsDefinedAsConstant() const override;
	UE_API virtual bool IsDefinedAsVarying() const override;
	UE_API virtual bool IsPure() const override;
	UE_API virtual const TArray<FName>& GetControlFlowBlocks() const override;
	UE_API virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateInputs() const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateOutputs() const override;
	UE_API virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const override;
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
	UE_API virtual FName GetDisplayNameForPin(const URigVMPin* InPin) const override;

	UE_API const FRigVMDispatchFactory* GetFactory() const;
	UE_API virtual bool IsOutDated() const override;
	UE_API virtual FString GetDeprecatedMetadata() const override;

	UE_API FRigVMDispatchContext GetDispatchContext() const;

	// Returns an instance of the factory with the current values.
	// @param bUseDefault If set to true the default struct will be created - otherwise the struct will contains the values from the node
	UE_API TSharedPtr<FStructOnScope> ConstructFactoryInstance(bool bUseDefault = false, FString* OutFactoryDefault = nullptr) const;

	// Returns a copy of the struct with the current values
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
	>
	T ConstructFactoryInstance() const
	{
		if(!ensure(T::StaticStruct() == GetFactoryStruct()))
		{
			return T();
		}

		TSharedPtr<FStructOnScope> Instance = ConstructFactoryInstance(false);
		const T& InstanceRef = *(const T*)Instance->GetStructMemory();
		return InstanceRef;
	}
	
protected:

	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;
	UE_API const UScriptStruct* GetFactoryStruct() const;
	UE_API virtual void InvalidateCache() override;
	UE_API virtual bool ShouldInputPinComputeLazily(const URigVMPin* InPin) const override;
	UE_API virtual bool IsPinConstant(const URigVMPin* InPin) const override;
	UE_API virtual FString GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const override;
	UE_API FString GetFactoryDefaultValue() const;

private:

	mutable FRigVMTemplateTypeMap TypesFromPins;
	mutable const FRigVMDispatchFactory* CachedFactory = nullptr;

	friend class URigVMController;
};

#undef UE_API
