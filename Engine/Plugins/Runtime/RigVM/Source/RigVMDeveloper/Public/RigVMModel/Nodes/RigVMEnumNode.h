// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/RigVMPin.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMEnumNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UEnum;
class UObject;
struct FFrame;

/**
 * The Enum Node represents a constant enum value for use within the graph.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMEnumNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMEnumNode();

	// Override of node title
	UE_API virtual FString GetNodeTitle() const;

	// Returns the enum itself
	UFUNCTION(BlueprintCallable, Category = RigVMEnumNode)
	UE_API UEnum* GetEnum() const;

	// Returns the C++ data type of the parameter
	UFUNCTION(BlueprintCallable, Category = RigVMEnumNode)
	UE_API FString GetCPPType() const;

	// Returns the C++ data type struct of the parameter (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMEnumNode)
	UE_API UObject* GetCPPTypeObject() const;

	// Returns the default value of the parameter as a string
	UE_API FString GetDefaultValue(const URigVMPin::FPinOverride& InOverride = URigVMPin::EmptyPinOverride) const;

	// Override of node title
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Blue; }

	virtual bool IsDefinedAsVarying() const override { return true; }

private:

	static UE_API const TCHAR* EnumName;
	static const inline TCHAR* EnumValueName = TEXT("EnumValue");
	static const inline TCHAR* EnumIndexName = TEXT("EnumIndex");
	
	friend class URigVMController;
	friend class URigVMCompiler;
	friend struct FRigVMAddEnumNodeAction;
	friend class URigVMEdGraphEnumNodeSpawner;
	friend class FRigVMParserAST;
};

#undef UE_API
