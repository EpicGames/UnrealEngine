// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMFunctionInterfaceNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMCollapseNode;

/**
 * The Function Interface node is is used as the base class for
 * both the entry and return nodes.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMFunctionInterfaceNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Override template node functions
	UE_API virtual uint32 GetStructureHash() const override;

	// Override node functions
	UE_API virtual FLinearColor GetNodeColor() const override;
	UE_API virtual bool IsDefinedAsVarying() const override;
	
	// URigVMNode interface
	UE_API virtual  FText GetToolTipText() const override;
	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;

	// returns true if the interface pin is used for anything within the function / collapse graph
	UE_API virtual bool IsInterfacePinUsed(const FName& InInterfacePinName) const;
	
	// find outer node which contains this interface node in its graph
	UE_API const URigVMCollapseNode* FindReferencedNode() const;

	// find pin in referenced node that represents this inner pin
	UE_API const URigVMPin* FindReferencedPin(const URigVMPin* InPin) const;
	UE_API const URigVMPin* FindReferencedPin(const FString& InPinPath) const;

private:

	friend class URigVMController;
};

#undef UE_API
