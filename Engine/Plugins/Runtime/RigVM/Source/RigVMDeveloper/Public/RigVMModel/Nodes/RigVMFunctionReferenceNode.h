// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMFunctionReferenceNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMFunctionLibrary;

/**
 * The Function Reference Node is a library node which references
 * a library node from a separate function library graph.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMFunctionReferenceNode : public URigVMLibraryNode
{
	GENERATED_BODY()

public:

	// URigVMNode interface
	UE_API virtual FString GetNodeTitle() const override;
	UE_API virtual FLinearColor GetNodeColor() const override;
	UE_API virtual FText GetToolTipText() const override;
	UE_API virtual FName GetDisplayNameForPin(const URigVMPin* InPin) const override;
	UE_API virtual FString GetCategoryForPin(const FString& InPinPath) const override;
	UE_API virtual int32 GetIndexInCategoryForPin(const FString& InPinPath) const override;
	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;
	UE_API virtual TArray<FString> GetPinCategories() const override;
	UE_API virtual FRigVMNodeLayout GetNodeLayout(bool bIncludeEmptyCategories) const override;
	// end URigVMNode interface

	// URigVMLibraryNode interface
	UE_API virtual FString GetNodeCategory() const override;
	UE_API virtual FString GetNodeKeywords() const override;
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables() const override;
	virtual const FRigVMTemplate* GetTemplate() const override { return nullptr; }
	UE_API virtual FRigVMGraphFunctionIdentifier GetFunctionIdentifier() const override;
	// end URigVMLibraryNode interface

	UE_API bool IsReferencedFunctionHostLoaded() const;
	UE_API bool IsReferencedNodeLoaded() const;
	UE_API URigVMLibraryNode* LoadReferencedNode() const;

	// Variable remapping
	UE_API bool ReferencesFunctionInSamePackage() const;
	UE_API bool RequiresVariableRemapping() const;
	UE_API bool IsFullyRemapped() const;
	UE_API const TMap<FName, FName> GetVariableMap() const;
	UE_API FName GetOuterVariableName(const FName& InInnerVariableName) const;
	UE_API FString GetOuterVariablePath(const FName& InInnerVariableName) const;
	// end Variable remapping

	UE_API virtual uint32 GetStructureHash() const override;

	UFUNCTION(BlueprintCallable, Category = RigVMLibraryNode, meta = (DisplayName = "GetReferencedFunctionHeader", ScriptName = "GetReferencedFunctionHeader"))
	FRigVMGraphFunctionHeader GetReferencedFunctionHeader_ForBlueprint() const { return GetReferencedFunctionHeader(); }

	const FRigVMGraphFunctionHeader& GetReferencedFunctionHeader() const { return ReferencedFunctionHeader; }

	UE_API void UpdateFunctionHeaderFromHost();

	UE_API const FRigVMGraphFunctionData* GetReferencedFunctionData(bool bLoadIfNecessary = true) const;
	UE_API bool SupportsCallable() const;

	UE_API TArray<FRigVMTag> GetVariantTags() const;
	
	UE_API FRigVMReferenceNodeData GetReferenceNodeData() const;

protected:

	UE_API virtual FString GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const override;
	UE_API TArray<FRigVMExternalVariable> GetAssetExternalVariables() const;
	
private:

	UE_API virtual TArray<int32> GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const override;
	UE_API const URigVMPin* FindReferencedPin(const URigVMPin* InPin) const;
	UE_API const URigVMPin* FindReferencedPin(const FString& InPinPath) const;

	//void SetReferencedFunctionData(FRigVMGraphFunctionData* Data);

	UPROPERTY(AssetRegistrySearchable)
	FRigVMGraphFunctionHeader ReferencedFunctionHeader;
	
	UPROPERTY(AssetRegistrySearchable, meta=(DeprecatedProperty=5.2))
	TSoftObjectPtr<URigVMLibraryNode> ReferencedNodePtr_DEPRECATED;

	UPROPERTY()
	TMap<FName, FName> VariableMap_DEPRECATED;

	friend class URigVMController;
	friend class FRigVMParserAST;
	friend class URigVMBlueprint;
	friend class IRigVMEditorAssetInterface;
	friend struct FRigVMClient;
	friend struct EngineTestRigVMFramework;
	friend class FRigVMExampleAssetActions;
	friend class FControlRigAssetActions;
};

#undef UE_API
