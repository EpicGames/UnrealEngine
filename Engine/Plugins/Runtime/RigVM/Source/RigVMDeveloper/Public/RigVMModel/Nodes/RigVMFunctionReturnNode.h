// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMFunctionInterfaceNode.h"
#include "RigVMFunctionReturnNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Function Return node is used to provide access to the 
 * output pins of the library node for links within.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMFunctionReturnNode : public URigVMFunctionInterfaceNode
{
	GENERATED_BODY()

public:

	// Override template node functions
	virtual UScriptStruct* GetScriptStruct() const override { return nullptr; }
	virtual const FRigVMTemplate* GetTemplate() const override { return nullptr; }
	virtual FName GetNotation() const override { return NAME_None; }

	// URigVMNode interface
	UE_API virtual FString GetNodeTitle() const override;

private:

	friend class URigVMController;
};

#undef UE_API
