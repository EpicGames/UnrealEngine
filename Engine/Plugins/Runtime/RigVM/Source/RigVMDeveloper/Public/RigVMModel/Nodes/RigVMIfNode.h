// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "RigVMTemplateNode.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMIfNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UObject;
struct FRigVMTemplate;

/**
 * A if node is used to pick between two values
 */
UCLASS(MinimalAPI, BlueprintType, Deprecated)
class UDEPRECATED_RigVMIfNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Override from URigVMTemplateNode
	UE_API virtual FName GetNotation() const override;
	UE_API virtual const FRigVMTemplate* GetTemplate() const override;
	virtual bool IsSingleton() const override { return false; }

	// Override from URigVMNode
	virtual FString GetNodeTitle() const override { return IfName; }
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Black; }
private:

	static const inline TCHAR* IfName = TEXT("If");
	static const inline TCHAR* ConditionName = TEXT("Condition");
	static const inline TCHAR* TrueName = TEXT("True");
	static const inline TCHAR* FalseName = TEXT("False");
	static const inline TCHAR* ResultName = TEXT("Result");

	friend class URigVMController;
	friend class URigVMCompiler;
	friend struct FRigVMAddIfNodeAction;
	friend class URigVMEdGraphIfNodeSpawner;
	friend class FRigVMParserAST;
	friend struct FRigVMRemoveNodeAction;
};

#undef UE_API
