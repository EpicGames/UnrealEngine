// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/RigVMNode.h"
#include "RigVMInvokeEntryNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Invoke Entry Node is used to invoke another entry from the 
 * the currently running entry.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMInvokeEntryNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMInvokeEntryNode();

	// Override of node title
	UE_API virtual FString GetNodeTitle() const;

	// Returns the name of the entry to run
	UFUNCTION(BlueprintCallable, Category = RigVMInvokeEntryNode)
	UE_API FName GetEntryName() const;

	// Override of node title
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Red; }
	virtual bool IsDefinedAsVarying() const override { return true; }

private:

	static const inline TCHAR* EntryName = TEXT("Entry");

	UE_API URigVMPin* GetEntryNamePin() const;
	

	friend class URigVMController;
	friend struct FRigVMRemoveNodeAction;
	friend class URigVMCompiler;
	friend class FRigVMParserAST;
	friend class URigVMPin;
};

#undef UE_API
