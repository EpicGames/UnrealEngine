// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMLibraryNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMGraph;
class URigVMFunctionEntryNode;
class URigVMFunctionReturnNode;
class URigVMFunctionLibrary;

/**
 * The Library Node represents a function invocation of a
 * function specified somewhere else. The function can be 
 * expressed as a sub-graph (RigVMGroupNode) or as a 
 * referenced function (RigVMFunctionNode).
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMLibraryNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Override node functions
	UE_API virtual bool IsDefinedAsConstant() const override;
	UE_API virtual bool IsDefinedAsVarying() const override;
	//virtual int32 GetInstructionVisitedCount(URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy(), bool bConsolidatePerNode = false) const override;
	
	// Override template node functions
	virtual UScriptStruct* GetScriptStruct() const override { return nullptr; }
	virtual const FRigVMTemplate* GetTemplate() const override { return nullptr; }
	virtual FName GetNotation() const override { return NAME_None; }
	UE_API virtual uint32 GetStructureHash() const override;

	// URigVMNode interface
	UE_API virtual  FText GetToolTipText() const override;
	
	// Library node interface
	virtual FString GetNodeCategory() const { return FString(); }
	virtual FString GetNodeKeywords() const { return FString(); }
	virtual FString GetNodeDescription() const { return FString(); }
	
	UFUNCTION(BlueprintCallable, Category = RigVMLibraryNode)
	virtual URigVMFunctionLibrary* GetLibrary() const { return nullptr; }

	UFUNCTION(BlueprintCallable, Category = RigVMLibraryNode)
	virtual URigVMGraph* GetContainedGraph() const { return nullptr; }
	
	UE_API virtual const TArray<URigVMNode*>& GetContainedNodes() const;
	UE_API virtual const TArray<URigVMLink*>& GetContainedLinks() const;
	UE_API virtual URigVMFunctionEntryNode* GetEntryNode() const;
	UE_API virtual URigVMFunctionReturnNode* GetReturnNode() const;
	UE_API virtual bool Contains(URigVMLibraryNode* InContainedNode, bool bRecursive = true) const;
	UE_API virtual TArray<FRigVMExternalVariable> GetExternalVariables() const;
	UE_API virtual TMap<FRigVMGraphFunctionIdentifier, uint32> GetDependencies() const;

	UE_API virtual FRigVMGraphFunctionIdentifier GetFunctionIdentifier() const;
	UE_API FRigVMGraphFunctionHeader GetFunctionHeader(IRigVMGraphFunctionHost* InHostObject = nullptr) const;
	
	UFUNCTION(BlueprintPure, Category = RigVMLibraryNode)
	UE_API FRigVMVariant GetFunctionVariant() const;

	UFUNCTION(BlueprintPure, Category = RigVMLibraryNode)
	UE_API FRigVMVariantRef GetFunctionVariantRef() const;

	UFUNCTION(BlueprintPure, Category = RigVMLibraryNode)
	UE_API TArray<FRigVMVariantRef> GetMatchingVariants() const;

	UFUNCTION(BlueprintPure, Category = RigVMLibraryNode)
	bool ContainsLocalVariable(const FName& InVariableName) const;

protected:

	UE_API virtual TArray<int32> GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const override; 
	UE_API const static TArray<URigVMNode*> EmptyNodes;
	UE_API const static TArray<URigVMLink*> EmptyLinks;

private:

	friend class URigVMController;
	friend struct FRigVMSetLibraryTemplateAction;
};

#undef UE_API
