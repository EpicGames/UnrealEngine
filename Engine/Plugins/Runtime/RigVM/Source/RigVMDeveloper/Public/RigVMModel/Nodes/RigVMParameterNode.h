// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMTypeUtils.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMParameterNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UObject;
struct FFrame;

/**
 * The parameter description is used to convey information
 * about unique parameters within a Graph. Multiple Parameter
 * Nodes can share the same parameter description.
 */
USTRUCT(BlueprintType, meta = (Deprecated = "5.1"))
struct FRigVMGraphParameterDescription
{
	GENERATED_BODY()

public:

	// comparison operator
	bool operator ==(const FRigVMGraphParameterDescription& Other) const
	{
		return Name == Other.Name;
	}

	// The name of the parameter
	UPROPERTY(BlueprintReadOnly, Category = RigVMGraphParameterDescription)
	FName Name;

	// True if the parameter is an input
	UPROPERTY(BlueprintReadOnly, Category = RigVMGraphParameterDescription)
	bool bIsInput = false;

	// The C++ data type of the parameter
	UPROPERTY(BlueprintReadOnly, Category = RigVMGraphParameterDescription)
	FString CPPType;

	// The Struct of the C++ data type of the parameter (or nullptr)
	UPROPERTY(BlueprintReadOnly, Category = RigVMGraphParameterDescription)
	TObjectPtr<UObject> CPPTypeObject = nullptr;

	// The default value of the parameter
	UPROPERTY(BlueprintReadOnly, Category = RigVMGraphParameterDescription)
	FString DefaultValue;

	// Returns nullptr external variable matching this description
	FRigVMExternalVariable ToExternalVariable() const
	{
		return FRigVMExternalVariable::Make(FGuid(), Name, CPPType, CPPTypeObject, false, false);
	}
};

/**
 * The Parameter Node represents an input or output argument / parameter
 * of the Function / Graph. Parameter Node have only a single value pin.
 */
UCLASS(MinimalAPI, BlueprintType, meta = (Deprecated = "5.1"))
class URigVMParameterNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMParameterNode();

	// Override of node title
	UE_API virtual FString GetNodeTitle() const;

	// Returns the name of the parameter
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API FName GetParameterName() const;

	// Returns true if this node is an input
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API bool IsInput() const;

	// Returns the C++ data type of the parameter
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API FString GetCPPType() const;

	// Returns the C++ data type struct of the parameter (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API UObject* GetCPPTypeObject() const;

	// Returns the default value of the parameter as a string
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API FString GetDefaultValue() const;

	// Returns this parameter node's parameter description
	UFUNCTION(BlueprintCallable, Category = RigVMParameterNode)
	UE_API FRigVMGraphParameterDescription GetParameterDescription() const;

	// Override of node title
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Green; }

	virtual bool IsDefinedAsVarying() const override { return true; }

private:

	virtual bool ContributesToResult() const override { return !IsInput(); }

	static const inline TCHAR* ParameterName = TEXT("Parameter");
	static const inline TCHAR* DefaultName = TEXT("Default");
	static const inline TCHAR* ValueName = TEXT("Value");

	friend class URigVMController;
	friend class URigVMCompiler;
};

#undef UE_API
