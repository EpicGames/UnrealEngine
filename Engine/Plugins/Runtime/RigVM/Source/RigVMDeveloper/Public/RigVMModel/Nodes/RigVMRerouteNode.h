// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/RigVMNode.h"
#include "RigVMRerouteNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * A reroute node is used to visually improve the 
 * data flow within a Graph. Reroutes are purely 
 * cosmetic and have no impact on the resulting
 * VM whatsoever. Reroutes can furthermore be
 * displayed as full nodes or as small circles.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMRerouteNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMRerouteNode();

	// Override of node title
	UE_API virtual FString GetNodeTitle() const override;

	UE_API virtual FLinearColor GetNodeColor() const override;

	UE_API virtual FText GetToolTipText() const override;

	// Has no source connections
	UE_API bool IsLiteral() const;
	
	virtual bool IsPure() const override { return true; }

private:

	static const inline TCHAR* RerouteName = TEXT("Reroute");
	static const inline TCHAR* ValueName = TEXT("Value");

	friend class URigVMController;
	friend class URigVMCompiler;
	friend class FRigVMParserAST;
	friend class FRigVMDeveloperModule;
	friend class URigVMPin;
};

#undef UE_API
