// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "RigVMTemplateNode.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMSelectNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UObject;
class URigVMPin;
struct FRigVMTemplate;

/**
 * A select node is used to select between multiple values
 */
UCLASS(MinimalAPI, BlueprintType, Deprecated)
class UDEPRECATED_RigVMSelectNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// Override from URigVMTemplateNode
	UE_API virtual FName GetNotation() const override;
	UE_API virtual const FRigVMTemplate* GetTemplate() const override;
	virtual bool IsSingleton() const override { return false; }

	// Override from URigVMNode
	virtual FString GetNodeTitle() const override { return SelectName; }
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Black; }
	
protected:

	UE_API virtual bool AllowsLinksOn(const URigVMPin* InPin) const override;

private:

	static const inline TCHAR* SelectName = TEXT("Select");
	static const inline TCHAR* IndexName = TEXT("Index");
	static const inline TCHAR* ValueName = TEXT("Values");
	static const inline TCHAR* ResultName = TEXT("Result");

	friend class URigVMController;
	friend class URigVMCompiler;
	friend struct FRigVMAddSelectNodeAction;
	friend class URigVMEdGraphSelectNodeSpawner;
	friend class FRigVMParserAST;
	friend class FRigVMSelectExprAST;
	friend struct FRigVMRemoveNodeAction;
	friend class URigVMPin;
};

#undef UE_API
