// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/RigVMNode.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMTemplateNode.generated.h"

#define UE_API RIGVMDEVELOPER_API
#define UE_RIGVM_MANYTYPES_LIMIT 100

USTRUCT()
struct FRigVMTemplatePreferredType
{
	GENERATED_BODY()

public:
	
	FRigVMTemplatePreferredType()
		: Argument(NAME_None)
		, TypeIndex(INDEX_NONE)
	{}
	
	FRigVMTemplatePreferredType(const FName& InArgument, TRigVMTypeIndex InTypeIndex)
		: Argument(InArgument)
		, TypeIndex(InTypeIndex)
	{
#if UE_RIGVM_DEBUG_TYPEINDEX
		UpdateStringFromIndex();
#endif
	}

	const FName& GetArgument() const { return Argument; }
	TRigVMTypeIndex GetTypeIndex() const
	{
#if UE_RIGVM_DEBUG_TYPEINDEX
		const FRigVMTemplateArgumentType& Type = FRigVMRegistry::Get().GetType(TypeIndex);
		return FRigVMRegistry::Get().GetTypeIndex(Type);
#else
		return TypeIndex;
#endif
	}

	bool operator==(const FRigVMTemplatePreferredType& Other) const
	{
		return Argument == Other.Argument && TypeIndex == Other.TypeIndex;
	}
	
	UE_API void UpdateStringFromIndex();
	UE_API void UpdateIndexFromString();

protected:
	
	UPROPERTY()
	FName Argument;

	UPROPERTY()
	int32 TypeIndex;

	UPROPERTY()
	FString TypeString;

	friend class URigVMTemplateNode;
	friend class URigVMController;
};

/**
 * The Template Node represents an unresolved function.
 * Template nodes can morph into all functions implementing
 * the template's template.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMTemplateNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// default constructor
	UE_API URigVMTemplateNode();

	// UObject interface
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	UE_API virtual void PostLoad() override;

	// URigVMNode interface
	UE_API virtual FString GetNodeTitle() const override;
	UE_API virtual FName GetMethodName() const;
	UE_API virtual  FText GetToolTipText() const override;
	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateInputs() const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateOutputs() const override;
	UE_API TArray<URigVMPin*> GetAggregatePins(const ERigVMPinDirection& InDirection) const;
	UE_API virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const override;

	// Returns the UStruct for this unit node
	// (the struct declaring the RIGVM_METHOD)
	UFUNCTION(BlueprintCallable, Category = RigVMUnitNode)
	UE_API virtual UScriptStruct* GetScriptStruct() const;

	// Returns the notation of the node
	UFUNCTION(BlueprintPure, Category = Template)
	UE_API virtual FName GetNotation() const;

	UFUNCTION(BlueprintCallable, Category = Template)
	UE_API virtual bool IsSingleton() const;

	// returns true if a pin supports a given type
	UE_API bool SupportsType(const URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, TRigVMTypeIndex* OutTypeIndex = nullptr);

	// returns the resolved functions for the template
	UE_API TArray<const FRigVMFunction*> GetResolvedPermutations() const;

	// returns the resolved permutation indices for the template
	UE_API TArray<int32> GetResolvedPermutationIndices(bool bAllowFloatingPointCasts) const;

	// Returns a map of the resolved pin types
	UE_API FRigVMTemplateTypeMap GetTemplatePinTypeMap(bool bIncludeHiddenPins = false, bool bIncludeExecutePins = false) const;

	// returns the template used for this node
	UE_API virtual const FRigVMTemplate* GetTemplate() const;

	// returns the resolved function or nullptr if there are still unresolved pins left
	UE_API const FRigVMFunction* GetResolvedFunction() const;

	// returns true if the template node is resolved
	UFUNCTION(BlueprintPure, Category = Template)
	UE_API bool IsResolved() const;

	// returns true if the template is fully unresolved
	UFUNCTION(BlueprintPure, Category = Template)
	UE_API bool IsFullyUnresolved() const;

	// returns a default value for pin if it is known
	UE_API FString GetInitialDefaultValueForPin(const FName& InRootPinName, const TArray<int32>& InPermutationIndices = TArray<int32>()) const;

	// returns the display name for a pin
	UE_API virtual FName GetDisplayNameForPin(const URigVMPin* InPin) const override;
	UE_API FName GetDisplayNameForPin(const URigVMPin* InPin, const TArray<int32>& InPermutationIndices) const;

	// Tries to reduce the input types to a single type, if all are compatible
	// Will prioritize the InPreferredType if available
	UE_API TRigVMTypeIndex TryReduceTypesToSingle(const TArray<TRigVMTypeIndex>& InTypes, const TRigVMTypeIndex PreferredType = TRigVMTypeIndex()) const;
	UE_API TRigVMTypeIndex TryReduceTypesToSingle_NoLock(const TArray<TRigVMTypeIndex>& InTypes, const FRigVMRegistryHandle& InRegistry, const TRigVMTypeIndex PreferredType = TRigVMTypeIndex()) const;

	UE_API virtual uint32 GetStructureHash() const override;

protected:

	UE_API virtual void InvalidateCache() override;
	
	UE_API TArray<int32> FindPermutationsForTypes(const TArray<FRigVMTemplatePreferredType>& ArgumentTypes, bool bAllowCasting = false) const;
	UE_API FRigVMTemplateTypeMap GetTypesForPermutation(const int32 InPermutationIndex) const;

	UPROPERTY()
	FName TemplateNotation;

	UPROPERTY()
	FString ResolvedFunctionName;

#if WITH_EDITORONLY_DATA
	// Indicates a preferred permutation using the types of the arguments
	// Each element is in the format "ArgumentName:CPPType"
	UPROPERTY()
	TArray<FString> PreferredPermutationTypes_DEPRECATED;
#endif

	UPROPERTY()
	TArray<FRigVMTemplatePreferredType> PreferredPermutationPairs_DEPRECATED;

	mutable const FRigVMTemplate* CachedTemplate;
	mutable const FRigVMFunction* CachedFunction;
	mutable int32 ResolvedPermutation;

	friend class URigVMController;
	friend class URigVMBlueprint;
	friend struct FRigVMClient;
	friend struct FRigVMSetTemplateFilteredPermutationsAction;
	friend struct FRigVMSetPreferredTemplatePermutationsAction;
	friend struct FRigVMRemoveNodeAction;
	friend class FRigVMTemplatesLibraryNode3Test;
	friend class URigVMCompiler;
};

#undef UE_API
