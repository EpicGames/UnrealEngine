// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMCore/RigVMStruct.h"
#include "UObject/StructOnScope.h"
#include "RigVMModel/RigVMModelCachedValue.h"
#include "RigVMUnitNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMHost;

/**
 * The Struct Node represents a Function Invocation of a RIGVM_METHOD
 * declared on a USTRUCT. Struct Nodes have input / output pins for all
 * struct UPROPERTY members.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMUnitNode : public URigVMTemplateNode
{
	GENERATED_BODY()

public:

	// UObject interface
	UE_API virtual void PostLoad() override;

	// Override node functions
	UE_API virtual FString GetNodeTitle() const override;
	UE_API virtual FString GetNodeSubTitle() const override;
	UE_API virtual FText GetToolTipText() const override;
	UE_API virtual bool IsDefinedAsConstant() const override;
	UE_API virtual bool IsDefinedAsVarying() const override;
	UE_API virtual bool IsPure() const override;
	UE_API virtual FName GetEventName() const override;
	UE_API virtual bool CanOnlyExistOnce() const override;
	UE_API virtual const TArray<FName>& GetControlFlowBlocks() const override;
	UE_API virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const override;
	UE_API virtual TArray<FRigVMUserWorkflow> GetSupportedWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateInputs() const override;
	UE_API virtual TArray<URigVMPin*> GetAggregateOutputs() const override;
	UE_API virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const override;
	UE_API virtual FName GetDisplayNameForPin(const URigVMPin* InPin) const override;

	UE_API virtual bool IsOutDated() const override;
	UE_API virtual FString GetDeprecatedMetadata() const override;

	// URigVMTemplateNode interface
	UE_API virtual UScriptStruct* GetScriptStruct() const override;

	// Returns the name of the declared RIGVM_METHOD
	UFUNCTION(BlueprintCallable, Category = RigVMUnitNode)
	UE_API virtual FName GetMethodName() const;

	// Returns the default value for the struct as text
	UFUNCTION(BlueprintCallable, Category = RigVMUnitNode)
	UE_API FString GetStructDefaultValue() const;

	// Returns an instance of the struct with the current values.
	// @param bUseDefault If set to true the default struct will be created - otherwise the struct will contains the values from the node
	UE_API TSharedPtr<FStructOnScope> ConstructStructInstance(bool bUseDefault = false) const;

	// Returns an instance of the struct with values backed by the memory of a currently running host
	UE_API TSharedPtr<FStructOnScope> ConstructLiveStructInstance(URigVMHost* InHost, int32 InSliceIndex = 0) const;
	
	// Updates the memory of a host to match a specific struct instance
	UE_API bool UpdateHostFromStructInstance(URigVMHost* InHost, TSharedPtr<FStructOnScope> InInstance, int32 InSliceIndex = 0) const;

	// Compares two struct instances and returns differences in pin values
	UE_API void ComputePinValueDifferences(TSharedPtr<FStructOnScope> InCurrentInstance, TSharedPtr<FStructOnScope> InDesiredInstance, TMap<FString, FString>& OutNewPinDefaultValues) const;

	// Compares a desired struct instance with the node's default and returns differences in pin values
	UE_API void ComputePinValueDifferences(TSharedPtr<FStructOnScope> InDesiredInstance, TMap<FString, FString>& OutNewPinDefaultValues) const;

	// Returns true if the node is part of the debugged runtime rig
	UE_API bool IsPartOfRuntime() const;

	// Returns true if the node is part of the debugged runtime rig
	UE_API bool IsPartOfRuntime(URigVMHost* InHost) const;

	// Returns a copy of the struct with the current values
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type* = nullptr
	>
	T ConstructStructInstance() const
	{
		if(!ensure(T::StaticStruct() == GetScriptStruct()))
		{
			return T();
		}

		TSharedPtr<FStructOnScope> Instance = ConstructStructInstance(false);
		const T& InstanceRef = *(const T*)Instance->GetStructMemory();
		return InstanceRef;
	}

	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	UE_API virtual uint32 GetStructureHash() const override;

protected:

	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const override;
	UE_API virtual bool ShouldInputPinComputeLazily(const URigVMPin* InPin) const override;
	UE_API virtual bool IsPinConstant(const URigVMPin* InPin) const override;
	UE_API virtual FString GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const override;
	UE_API void EnumeratePropertiesOnHostAndStructInstance(
		URigVMHost* InHost,
		TSharedPtr<FStructOnScope> InInstance, 
		bool bPreferLiterals,
		TFunction<void(const URigVMPin*,const FProperty*,uint8*,const FProperty*,uint8*)> InEnumerationFunction,
		int32 InSliceIndex = 0) const;

private:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UScriptStruct> ScriptStruct_DEPRECATED;

	UPROPERTY()
	FName MethodName_DEPRECATED;
#endif

	mutable TRigVMModelCachedValue<URigVMUnitNode, FName> CachedEventName;

	friend class URigVMController;
};

#undef UE_API
