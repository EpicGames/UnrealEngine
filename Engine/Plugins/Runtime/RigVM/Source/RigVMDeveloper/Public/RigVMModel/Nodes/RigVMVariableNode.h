// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMCore/RigVMVariableDescription.h"
#include "RigVMVariableNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

/**
 * The Variable Node represents a mutable value / local state within the
 * the Function / Graph. Variable Node's can be a getter or a setter.
 * Getters are pure nodes with just an output value pin, while setters
 * are mutable nodes with an execute and input value pin.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMVariableNode : public URigVMNode
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMVariableNode();

	// Override of node title
	UE_API virtual FString GetNodeTitle() const;

	// Returns the name of the variable
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API FName GetVariableName() const;

	// Returns true if this node is a variable getter
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API bool IsGetter() const;

	// Returns true if this variable is an external variable
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API bool IsExternalVariable() const;

	// Returns true if this variable is a local variable
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API bool IsLocalVariable() const;

	// Returns true if this variable is an input argument
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API bool IsInputArgument() const;

	// Returns true if this variable is related to an input variable
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API bool IsInputVariable() const;

	// Returns the C++ data type of the variable
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API FString GetCPPType() const;

	// Returns the C++ data type struct of the variable (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API UObject* GetCPPTypeObject() const;

	// Returns the default value of the variable as a string
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API FString GetDefaultValue() const;

	// Returns this variable node's variable description
	UFUNCTION(BlueprintCallable, Category = RigVMVariableNode)
	UE_API FRigVMGraphVariableDescription GetVariableDescription() const;

	// Override of node title
	virtual FLinearColor GetNodeColor() const override { return FLinearColor::Blue; }
	
	virtual bool IsDefinedAsVarying() const override { return true; }
	UE_API virtual bool IsPure() const override;

	// Get the variable name pin
	UE_API URigVMPin* GetVariableNamePin() const;

	// Get the value pin
	UE_API URigVMPin* GetValuePin() const;

	// Get the subtitle of the node
	virtual FString GetNodeSubTitle() const override;
	
	// Get variable Guid
	FGuid GetVariableGuid() const { return VariableGuid; }

private:
	
	UPROPERTY()
	FGuid VariableGuid;

	static const inline TCHAR* VariableName = TEXT("Variable");
	static const inline TCHAR* ValueName = TEXT("Value");

	friend class URigVMController;
	friend class URigVMBlueprint;
	friend struct FRigVMRemoveNodeAction;
	friend class URigVMPin;
	friend class URigVMCompiler;
	friend class FRigVMVarExprAST;
	friend class FRigVMExprAST;
	friend class FRigVMParserAST;
	friend class URigVMSchema;
	friend class URigVMEdGraphVariableNodeSpawner;
};

#undef UE_API
