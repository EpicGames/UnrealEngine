// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMBuildData.generated.h"

#define UE_API RIGVMDEVELOPER_API

USTRUCT(BlueprintType)
struct FRigVMFunctionReferenceArray
{
	GENERATED_BODY()

	// Resets the data structure and maintains all storage.
	void Reset() { FunctionReferences.Reset();  }

	// Returns true if a given function reference index is valid.
	bool IsValidIndex(int32 InIndex) const { return FunctionReferences.IsValidIndex(InIndex); }

	// Returns the number of reference functions
	int32 Num() const { return FunctionReferences.Num(); }

	// const accessor for an function reference given its index
	const TSoftObjectPtr<URigVMFunctionReferenceNode>& operator[](int32 InIndex) const { return FunctionReferences[InIndex]; }

	UPROPERTY(VisibleAnywhere, Category = "BuildData")
	TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > FunctionReferences;
};

/**
 * The Build Data is used to store transient / intermediate build information
 * for the RigVM graph to improve the user experience.
 * This object is never serialized.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMBuildData : public UObject
{
	GENERATED_BODY()

public:

	// Returns the singleton build data
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	static UE_API URigVMBuildData* Get();
	
	// Looks for all function references and public functions (in RigVMClientHost metadata) and initializes the URigVMBuildData
	UE_API void InitializeIfNeeded();

	static UE_API void RegisterReferencesFromAsset(const FAssetData& AssetData);
	static UE_API void RegisterPublicFunctionsFromAsset(const FAssetData& AssetData);

	// Returns the list of references for a given function definition
	UE_API const FRigVMFunctionReferenceArray* FindFunctionReferences(const FRigVMGraphFunctionIdentifier& InFunction) const;

	/**
	 * Iterator function to invoke a lambda / TFunction for each reference of a function
	 * @param InFunction The function to iterate all references for
	 * @param PerReferenceFunction The function to invoke for each reference
	 * @param bLoadIfNecessary If true, will load packages if necessary
	 */
	UE_API void ForEachFunctionReference(const FRigVMGraphFunctionIdentifier& InFunction, TFunction<void(URigVMFunctionReferenceNode*)> PerReferenceFunction, bool bLoadIfNecessary = true) const;

	/**
	* Iterator function to invoke a lambda / TFunction for each reference of a function
	* @param InFunction The function to iterate all references for
	* @param PerReferenceFunction The function to invoke for each reference
	*/
	UE_API void ForEachFunctionReferenceSoftPtr(const FRigVMGraphFunctionIdentifier& InFunction, TFunction<void(TSoftObjectPtr<URigVMFunctionReferenceNode>)>
	                                     PerReferenceFunction) const;

	// registers a new reference node for a given function
	UE_API void RegisterFunctionReference(const FRigVMGraphFunctionIdentifier& InFunction, URigVMFunctionReferenceNode* InReference);

	// registers a new reference node for a given function
	UE_API void RegisterFunctionReference(const FRigVMGraphFunctionIdentifier& InFunction, TSoftObjectPtr<URigVMFunctionReferenceNode> InReference);

	// registers a new reference node for a given function
	UE_API void RegisterFunctionReference(FRigVMReferenceNodeData InReferenceNodeData);

	// unregisters a new reference node for a given function
	UE_API void UnregisterFunctionReference(const FRigVMGraphFunctionIdentifier& InFunction, URigVMFunctionReferenceNode* InReference);

	// unregisters a new reference node for a given function
	UE_API void UnregisterFunctionReference(const FRigVMGraphFunctionIdentifier& InFunction, TSoftObjectPtr<URigVMFunctionReferenceNode> InReference);

	// Clear references to temp assets
	UE_API void ClearInvalidReferences();
	
	// Register a new public function with its header
	UE_API void RegisterPublicFunction(const FRigVMGraphFunctionIdentifier& InFunction, const FRigVMGraphFunctionHeader& InHeader);
	
	// Unregisters a public function
	UE_API void UnregisterPublicFunction(const FRigVMGraphFunctionIdentifier& InFunction);
	
	// Updates the header of a public function
	UE_API void UpdatePublicFunctionHeader(const FRigVMGraphFunctionIdentifier& InFunction, const FRigVMGraphFunctionHeader& InHeader);
	
	// Get header from function identifier
	UE_API const FRigVMGraphFunctionHeader& FindFunctionHeader(const FRigVMGraphFunctionIdentifier& InFunction);

	// Helper function to disable clearing transient package references
	void SetIsRunningUnitTest(bool bIsRunning) { bIsRunningUnitTest = bIsRunning; }

	// Will find all public function variant refs, and private function variant refs from loaded assets
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMVariantRef> GatherAllFunctionVariantRefs() const;

	// Will find the public function variant refs inside this asset, and private function variant refs if the asset is loaded
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMVariantRef> GatherFunctionVariantRefsForAsset(const FAssetData& InAssetData) const;

	// Will find all the function variants matching the given variant guid
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMVariantRef> FindFunctionVariantRefs(const FGuid& InGuid) const;

	// Returns the function identifier given a variant reference
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FRigVMGraphFunctionIdentifier GetFunctionIdentifierForVariant(const FRigVMVariantRef& InVariantRef) const;

	// Creates a new variant of a graph function within the same asset
	UFUNCTION(BlueprintCallable, Category=RigVMBuildData)
	UE_API FRigVMVariantRef CreateFunctionVariant(const FRigVMGraphFunctionIdentifier& InFunctionIdentifier, FName InName = NAME_None);

	// Will find all asset variant refs
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMVariantRef> GatherAllAssetVariantRefs() const;

	// Will find all the asset variants matching the given variant guid
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMVariantRef> FindAssetVariantRefs(const FGuid& InGuid) const;

	// Creates a new variant of an asset
	UFUNCTION(BlueprintCallable, Category=RigVMBuildData)
	UE_API FRigVMVariantRef CreateAssetVariant(const FAssetData& InAssetData, FName InName = NAME_None);

	// Returns the asset data given an object path
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FAssetData GetAssetDataForPath(const FSoftObjectPath& InObjectPath) const;

	// Returns the asset data given a variant reference
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FRigVMVariantRef GetVariantRefForAsset(const FAssetData& InAssetData) const;
	
	// Returns the asset data given a variant reference
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FAssetData GetAssetDataForVariant(const FRigVMVariantRef& InVariantRef) const;

	// Splits a variant from its variant set (by assigning a new, unique guid
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FRigVMVariantRef SplitVariantFromSet(const FRigVMVariantRef& InVariantRef);

	// Joins a variant with another variant's set
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API FRigVMVariantRef JoinVariantSet(const FRigVMVariantRef& InVariantRef, const FGuid& InGuid);

#if WITH_EDITOR
	// Returns all known public function identifiers used in the project
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMGraphFunctionIdentifier> GetAllFunctionIdentifiers(bool bOnlyPublic = true) const;
#endif

	// Returns all known public function identifiers used in the project
	UFUNCTION(BlueprintPure, Category=RigVMBuildData)
	UE_API TArray<FRigVMGraphFunctionIdentifier> GetUsedFunctionIdentifiers(bool bOnlyPublic = true) const;

	// Returns all known function references
	UE_API FRigVMFunctionReferenceArray GetAllFunctionReferences() const;
	
	// Remove all data, and mark that initialization is needed
	UE_API void Reset();

private:

	// disable default constructor
	UE_API URigVMBuildData();

	static UE_API TArray<UClass*> FindAllRigVMAssetClasses();

	UE_API void SetupRigVMGraphFunctionPointers();
	UE_API static void TearDownRigVMGraphFunctionPointers();
	static UE_API TArray<FRigVMGraphFunctionHeader> GetFunctionHeadersForAsset(const FAssetData& InAssetData);
	
	static UE_API bool bInitialized;

	UPROPERTY(meta=(DeprecatedProperty))
	TMap< TSoftObjectPtr<URigVMLibraryNode>, FRigVMFunctionReferenceArray > FunctionReferences_DEPRECATED;

	UPROPERTY(VisibleAnywhere, Category = "BuildData")
	TMap< FRigVMGraphFunctionIdentifier, FRigVMFunctionReferenceArray > GraphFunctionReferences;
	
	UPROPERTY(VisibleAnywhere, Category = "BuildData")
	TMap<FRigVMGraphFunctionIdentifier, FRigVMGraphFunctionHeader> PublicGraphFunctions;

	bool bIsRunningUnitTest;

	friend class URigVMController;
	friend struct FRigVMClient;
	friend class URigVMCompiler;
	friend class FRigVMDeveloperModule;
};

#undef UE_API
