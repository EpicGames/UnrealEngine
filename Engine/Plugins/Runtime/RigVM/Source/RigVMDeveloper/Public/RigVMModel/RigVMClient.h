// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMGraph.h"
#include "RigVMSchema.h"
#include "RigVMFunctionLibrary.h"
#include "RigVMController.h"
#include "UObject/WeakInterfacePtr.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RigVMClient.generated.h"

#define UE_API RIGVMDEVELOPER_API

struct FRigVMClient;

UINTERFACE(MinimalAPI)
class URigVMClientHost : public UInterface
{
	GENERATED_BODY()
};

enum class ERigVMLoadType : uint8
{
	PostLoad,
	CheckUserDefinedStructs
};

DECLARE_DELEGATE_RetVal(URigVMGraph*, FRigVMGetFocusedGraph);

// Interface that allows an object to host a rig VM client. Used by graph edting code to interact with the controller.
class IRigVMClientHost
{
	GENERATED_BODY()

public:

	/** Returns the expected schema class to use for this blueprint */
	virtual FString GetAssetName() const = 0;

	/** Returns the expected schema class to use for this blueprint */
	virtual UClass* GetRigVMSchemaClass() const = 0;

	/** Returns the expected execute context struct to use for this blueprint */
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const = 0;

	/** Returns the expected ed graph class to use for this blueprint */
	virtual UClass* GetRigVMEdGraphClass() const = 0;

	/** Returns the expected ed graph node class to use for this blueprint */
	virtual UClass* GetRigVMEdGraphNodeClass() const = 0;

	/** Returns the expected ed graph schema class to use for this blueprint */
	virtual UClass* GetRigVMEdGraphSchemaClass() const = 0;

	/** Returns the class of the settings to use */
	virtual UClass* GetRigVMEditorSettingsClass() const = 0;

	// Returns the rigvm client for this host
	virtual FRigVMClient* GetRigVMClient() = 0;

	// Returns the rigvm client for this host
	virtual const FRigVMClient* GetRigVMClient() const = 0;

	// Returns the rigvm function host
	virtual TScriptInterface<IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() = 0;

	// Returns the rigvm function host
	virtual TScriptInterface<const IRigVMGraphFunctionHost> GetRigVMGraphFunctionHost() const = 0;

	// Returns the editor object corresponding with the supplied RigVM graph
	virtual UObject* GetEditorObjectForRigVMGraph(const URigVMGraph* InVMGraph) const = 0;

	// Returns the RigVM graph corresponding with the supplied editor object
	virtual URigVMGraph* GetRigVMGraphForEditorObject(UObject* InObject) const = 0;

	// Reacts to adding a graph
	virtual void HandleRigVMGraphAdded(const FRigVMClient* InClient, const FString& InNodePathOrName) = 0;

	// Reacts to removing a graph
	virtual void HandleRigVMGraphRemoved(const FRigVMClient* InClient, const FString& InNodePathOrName) = 0;

	// Reacts to renaming a graph
	virtual void HandleRigVMGraphRenamed(const FRigVMClient* InClient, const FString& InOldNodePath, const FString& InNewNodePath) = 0;

	// Reacts to a request to configure a controller
	virtual void HandleConfigureRigVMController(const FRigVMClient* InClient, URigVMController* InControllerToConfigure) = 0;

	// Given a type name for a user-defined type, either struct or an enum, returns a pointer to the object definition
	// or nullptr if the client host has no knowledge of such a type.
	UE_API virtual UObject* ResolveUserDefinedTypeById(const FString& InTypeName) const;

	// Recompiles the VM if not already being compiled
	virtual void RecompileVM() = 0;

	// Recompiles VM if flagged for recompiling is set
	virtual void RecompileVMIfRequired() = 0;

	// Flags VM requires recompile, and if auto recompile is enabled and no compile bracket is active, requests a recompilation
	virtual void RequestAutoVMRecompilation() = 0;

	// Sets flag for automatic recompile on model changes
	virtual void SetAutoVMRecompile(bool bAutoRecompile) = 0;

	// Returns current state of automatic recompile flag
	virtual bool GetAutoVMRecompile() const = 0;

	// Helper to increase recompile bracket on nested requests
	virtual void IncrementVMRecompileBracket() = 0;

	// Helper to decrease recompile bracket on nested requests. When value == 1, if autorecompile is enabled, it triggers a VM recompilation
	virtual void DecrementVMRecompileBracket() = 0;

	// Regenerates model pins if data has changed while the RigVM Graph is not opened (i.e. user defined struct is changed)
	virtual void RefreshAllModels(ERigVMLoadType InLoadType) = 0;

	virtual void OnRigVMRegistryChanged() = 0;

	virtual void RequestRigVMInit() = 0;

	virtual URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const = 0;
	virtual URigVMGraph* GetModel(const FString& InNodePath) const = 0;

	virtual URigVMGraph* GetDefaultModel() const = 0;

	virtual TArray<URigVMGraph*> GetAllModels() const = 0;

	virtual URigVMFunctionLibrary* GetLocalFunctionLibrary() const = 0;

	virtual URigVMFunctionLibrary* GetOrCreateLocalFunctionLibrary(bool bSetupUndoRedo = true) = 0;

	virtual URigVMGraph* AddModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) = 0;

	virtual bool RemoveModel(FString InName = TEXT("Rig Graph"), bool bSetupUndoRedo = true, bool bPrintPythonCommand = true) = 0;

	virtual FRigVMGetFocusedGraph& OnGetFocusedGraph() = 0;
	virtual const FRigVMGetFocusedGraph& OnGetFocusedGraph() const = 0;

	virtual URigVMGraph* GetFocusedModel() const = 0;

	virtual URigVMController* GetController(const URigVMGraph* InGraph = nullptr) const = 0;

	virtual URigVMController* GetControllerByName(const FString InGraphName = TEXT("")) const = 0;

	virtual URigVMController* GetOrCreateController(URigVMGraph* InGraph = nullptr) = 0;

	virtual URigVMController* GetController(const UEdGraph* InEdGraph) const = 0;
	virtual URigVMController* GetOrCreateController(const UEdGraph* InGraph) = 0;

	UE_DEPRECATED(5.8, "Use TArray<FString> GeneratePythonCommands() instead.")
	virtual TArray<FString> GeneratePythonCommands(const FString InNewBlueprintName)  = 0;
	virtual TArray<FString> GeneratePythonCommands()  = 0;

	virtual void SetupPinRedirectorsForBackwardsCompatibility() = 0;

	virtual FRigVMGraphModifiedEvent& OnModified() = 0;

	virtual bool IsFunctionPublic(const FName& InFunctionName) const = 0;
	virtual void MarkFunctionPublic(const FName& InFunctionName, bool bIsPublic = true) = 0;

	virtual void RenameGraph(const FString& InNodePath, const FName& InNewName) = 0;
};

UINTERFACE(MinimalAPI)
class URigVMEditorSideObject : public UInterface
{
	GENERATED_BODY()
};

// Interface that allows a UI graph to identify itself against a model graph
class IRigVMEditorSideObject
{
	GENERATED_BODY()

public:

	// Returns the corresponding VM graph
	virtual FRigVMClient* GetRigVMClient() const = 0;

	// Returns the nodepath for this UI graph
	virtual FString GetRigVMNodePath() const = 0;

	// Reacts to renaming the model
	virtual void HandleRigVMGraphRenamed(const FString& InOldNodePath, const FString& InNewNodePath) = 0;
};

UINTERFACE(MinimalAPI)
class URigVMClientExternalModelHost : public UInterface
{
	GENERATED_BODY()
};

// Interface that allows access to externally-hosted models
class IRigVMClientExternalModelHost
{
	GENERATED_BODY()

public:

	// Returns the externally-held models for a client
	virtual const TArray<TObjectPtr<URigVMGraph>>& GetExternalModels() const = 0;

	// Sets a new created model as 
	virtual TObjectPtr<URigVMGraph> CreateContainedGraphModel(URigVMCollapseNode* CollapseNode, const FName& Name) = 0;
};

#if WITH_EDITOR
DECLARE_MULTICAST_DELEGATE_ThreeParams(FRigVMClientPostGraphModified, ERigVMGraphNotifType /** InNotifType */, URigVMGraph* /** InGraph */, UObject* /** InSubject */);
#endif // WITH_EDITOR

// A management struct containing graphs and controllers.
USTRUCT()
struct FRigVMClient
{
public:

	GENERATED_BODY()

	FRigVMClient()
		: DefaultSchemaClass(nullptr)
		, ControllerClass(URigVMController::StaticClass())
		, FunctionLibrary(nullptr)
		, ActionStack(nullptr)
		, bSuspendNotifications(false)
		, bIgnoreModelNotifications(false)
		, bDefaultModelCanBeRemoved(false)
		, bSuspendModelNotificationsForOthers(false)
		, OuterClientHost(nullptr)
		, OuterClientPropertyName(NAME_None)	
		, ExternalModelHost(nullptr)
	{
	}

	UE_API void SetDefaultSchemaClass(TSubclassOf<URigVMSchema> InSchemaClass);
	UE_API void SetControllerClass(TSubclassOf<URigVMController> InControllerClass);
	UE_API void SetOuterClientHost(UObject* InOuterClientHost, const FName& InOuterClientHostPropertyName);
	UE_API void SetFromDeprecatedData(URigVMGraph* InDefaultGraph, URigVMFunctionLibrary* InFunctionLibrary);
	UE_API void SetExternalModelHost(IRigVMClientExternalModelHost* InExternalModelHost);

	UE_API FRigVMClient& operator =(const FRigVMClient& InOther);

	UE_API void Reset();
	int32 Num() const { return GetModels().Num(); }
	UE_API URigVMSchema* GetDefaultSchema() const;
	TSubclassOf<URigVMSchema> GetDefaultSchemaClass() const { return DefaultSchemaClass; }
	UE_API URigVMGraph* GetDefaultModel() const;
	UE_API URigVMGraph* GetModel(int32 InIndex) const;
	UE_API URigVMGraph* GetModel(const UEdGraph* InEdGraph = nullptr) const;
	UE_API URigVMGraph* GetModel(const FString& InNodePathOrName) const;
	UE_API URigVMGraph* GetModel(const UObject* InEditorSideObject) const;
	UE_API void RefreshAllModels(ERigVMLoadType InLoadType, bool bEnablePostLoadHashing, bool& bIsCompiling);
	UE_API void RefreshVariableNodes(const TArray<FRigVMExternalVariable> InPublicVariables);
	UE_API const TArray<TObjectPtr<URigVMGraph>>& GetModels() const;
	UE_API TArray<URigVMGraph*> GetAllModels(bool bIncludeFunctionLibrary, bool bRecursive) const;
	UE_API TArray<URigVMGraph*> GetAllModelsLeavesFirst(bool bIncludeFunctionLibrary) const;
	UE_API URigVMController* GetController(int32 InIndex) const;
	UE_API URigVMController* GetController(const FString& InNodePathOrName) const;
	UE_API URigVMController* GetController(const URigVMGraph* InModel) const;
	UE_API URigVMController* GetController(const UObject* InEditorSideObject) const;
	UE_API URigVMController* GetOrCreateController(int32 InIndex);
	UE_API URigVMController* GetOrCreateController(const FString& InNodePathOrName);
	UE_API URigVMController* GetOrCreateController(const URigVMGraph* InModel);
	UE_API URigVMController* GetOrCreateController(const UObject* InEditorSideObject);
	UE_API URigVMController* GetControllerByName(const FString InGraphName) const;
	UE_API bool RemoveController(const URigVMGraph* InModel);
	URigVMFunctionLibrary* GetFunctionLibrary() const { return FunctionLibrary; }
	UE_API void SetGetFunctionHostObjectPathDelegate(URigVMFunctionLibrary* FunctionLibrary);
	UE_API URigVMFunctionLibrary* GetOrCreateFunctionLibrary(bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer = nullptr, bool bCreateController = true);
	UE_API URigVMFunctionLibrary* GetOrCreateFunctionLibrary(TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer = nullptr, bool bCreateController = true);
	UE_API TArray<FName> GetEntryNames(UScriptStruct* InUnitScriptStructFilter = nullptr) const;
	UE_API UScriptStruct* GetDefaultExecuteContextStruct() const;
	UE_API void SetDefaultExecuteContextStruct(UScriptStruct* InExecuteContextStruct);

	FRigVMGetFocusedGraph& OnGetFocusedGraph() { return OnGetFocusedGraphDelegate;}
	const FRigVMGetFocusedGraph& OnGetFocusedGraph() const { return OnGetFocusedGraphDelegate; }
	UE_API URigVMGraph* GetFocusedModel() const;
	UE_API URigVMGraph* AddModel(const FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand);
	UE_API URigVMGraph* AddModel(const FName& InName, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer = nullptr, bool bCreateController = true);
	UE_API URigVMGraph* AddModel(const FName& InName, TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, const FObjectInitializer* ObjectInitializer = nullptr, bool bCreateController = true);

	UE_API URigVMGraph* CreateModel(const FName& InName, TSubclassOf<URigVMSchema> InSchemaClass, bool bSetupUndoRedo, UObject* InOuter, const FObjectInitializer* ObjectInitializer = nullptr, bool bCreateController = true);
	UE_API TObjectPtr<URigVMGraph> CreateContainedGraphModel(URigVMCollapseNode* CollapseNode, const FName& Name);
	
	UE_API void AddModel(URigVMGraph* InModel, bool bCreateController);
	UE_API bool RemoveModel(FString InName, bool bSetupUndoRedo, bool bPrintPythonCommand);
	UE_API bool RemoveModel(const FString& InNodePathOrName, bool bSetupUndoRedo);
	UE_API FName RenameModel(const FString& InNodePathOrName, const FName& InNewName, bool bSetupUndoRedo);
	UE_API void PostTransacted(const FTransactionObjectEvent& TransactionEvent);
	UE_API void OnCollapseNodeRenamed(const URigVMCollapseNode* InCollapseNode);
	UE_API void OnCollapseNodeRemoved(const URigVMCollapseNode* InCollapseNode);

	UE_API URigVMNode* FindNode(const FString& InNodePathOrName) const;
	UE_API URigVMPin* FindPin(const FString& InPinPath) const;
	
	TArray<TObjectPtr<URigVMGraph>>::RangedForIteratorType      begin() { return const_cast<TArray<TObjectPtr<URigVMGraph>>&>(GetModels()).begin(); }
	TArray<TObjectPtr<URigVMGraph>>::RangedForConstIteratorType begin() const { return GetModels().begin(); }
	TArray<TObjectPtr<URigVMGraph>>::RangedForIteratorType      end() { return const_cast<TArray<TObjectPtr<URigVMGraph>>&>(GetModels()).end(); }
	TArray<TObjectPtr<URigVMGraph>>::RangedForConstIteratorType end() const { return GetModels().end(); }

	UE_API UObject* GetOuter() const;
	UE_API FProperty* GetOuterClientProperty() const;
	UE_API void NotifyOuterOfPropertyChange(EPropertyChangeType::Type ChangeType = EPropertyChangeType::Interactive) const;
	UE_API FName GetUniqueName(const FName& InDesiredName) const;
	static UE_API FName GetUniqueName(UObject* InOuter, const FName& InDesiredName);
	static UE_API void DestroyObject(UObject* InObject);

	UE_API uint32 GetStructureHash() const;
	UE_API uint32 GetSerializedStructureHash() const;

	// backwards compatibility
	UE_API FRigVMClientPatchResult PatchModelsOnLoad();
	UE_API void PatchFunctionReferencesOnLoad();
	UE_API void PatchFunctionsOnLoad(IRigVMGraphFunctionHost* FunctionHost, TArray<FName>& BackwardsCompatiblePublicFunctions, TMap<URigVMLibraryNode*, FRigVMGraphFunctionHeader>& OldHeaders);
	UE_API FRigVMClientPatchResult PatchPinDefaultValues();

	// try to reattach detached links and delete remaining ones
	UE_API void ProcessDetachedLinks();

	// work to be done before saving
	UE_API void PreSave(FObjectPreSaveContext ObjectSaveContext);

	UE_API void HandleGraphModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	UE_API FRigVMGraphFunctionStore* FindFunctionStore(const URigVMLibraryNode* InLibraryNode);
	UE_API bool UpdateGraphFunctionData(const URigVMLibraryNode* InLibraryNode);
	UE_API bool UpdateExternalVariablesForFunction(const URigVMLibraryNode* InLibraryNode);
	UE_API bool UpdateDependenciesForFunction(const URigVMLibraryNode* InLibraryNode);
	UE_API bool UpdateFunctionReferences(const FRigVMGraphFunctionHeader& InHeader, bool bUpdateDependencies, bool bUpdateExternalVariables);
	UE_API bool DirtyGraphFunctionCompilationData(URigVMLibraryNode* InLibraryNode);
	UE_API bool UpdateGraphFunctionSerializedGraph(URigVMLibraryNode* InLibraryNode);
	UE_API bool IsFunctionPublic(URigVMLibraryNode* InLibraryNode);

#if WITH_EDITOR
	UE_API bool UpgradeAllOccurencesOfNodes(const TArray<URigVMNode*>& InNodesToUpgrade,  UObject* InOuter, bool bSetupUndoRedo = true);

	/** Returns a delegate raised after a graph in this client was modified. Useful to track changes without keeping track of graphs. */
	FRigVMClientPostGraphModified& GetPostGraphModified() { return PostGraphModifiedDelegate; }
#endif

	static constexpr TCHAR RigVMModelPrefix[] = TEXT("RigVMModel");

private:

	enum ERigVMClientAction
	{
		ERigVMClientAction_AddModel,
		ERigVMClientAction_RemoveModel,
		ERigVMClientAction_RenameModel
	};

	struct FRigVMClientAction
	{
		ERigVMClientAction Type;
		FString NodePath;
		FString OtherNodePath;
		// For ERigVMClientAction_RenameModel only: the index in Models[] the renamed graph
		// occupied at action time. Used by PostTransacted to rebind the renamed graph back
		// into the exact slot that property restoration nulled out, which avoids order
		// corruption when more than one Models[] slot is nullptr simultaneously
		// (e.g. two root-graph renames inside the same outer transaction during redo).
		// INDEX_NONE for non-rename actions or when the graph wasn't tracked in Models.
		int32 ModelIndex = INDEX_NONE;
	};

	UE_API URigVMController* CreateController(const URigVMGraph* InModel);
	UE_API URigVMActionStack* GetOrCreateActionStack();
	UE_API void ResetActionStack();

	FRigVMGetFocusedGraph OnGetFocusedGraphDelegate;

	UPROPERTY(transient)
	TSubclassOf<URigVMSchema> DefaultSchemaClass;

	UPROPERTY(transient)
	TSubclassOf<URigVMController> ControllerClass;

	UPROPERTY()
	TArray<TObjectPtr<URigVMGraph>> Models;

	UPROPERTY()
	TObjectPtr<URigVMFunctionLibrary> FunctionLibrary;

	// Self-healing cache: GetController() can evict stale entries (orphaned controllers whose graph was
	// replaced by an undo/redo transaction); declared mutable so the const observer path can do that.
	UPROPERTY(transient)
	mutable TMap<FSoftObjectPath, TObjectPtr<URigVMController>> Controllers;

	// Evicts the slot at Key from Controllers and tears down the supplied controller if it is alive.
	// Caller MUST hold ControllersLock. The slot is removed first so any reentrant lookup during teardown
	// misses cleanly. StaleController may be nullptr or already-garbage — both are handled here so callers
	// can pass any controller pointer without pre-checking. When non-null and not garbage, the controller's
	// OnModified delegate is cleared, its action stack is detached, and it is renamed into the transient
	// package, removed from root, and marked garbage (matching the teardown done by RemoveController).
	void PurgeStaleController(const FSoftObjectPath& Key, URigVMController* StaleController) const;

	UPROPERTY(transient)
	TObjectPtr<URigVMActionStack> ActionStack;

	UPROPERTY(transient)
	int32 UndoRedoIndex = 0;
	
	TArray<FRigVMClientAction> UndoStack;
	TArray<FRigVMClientAction> RedoStack;

public:
	bool bSuspendNotifications;
	bool bIgnoreModelNotifications;
	bool bDefaultModelCanBeRemoved;
	bool bSuspendModelNotificationsForOthers;
private:
	TWeakObjectPtr<UObject> OuterClientHost;
	FName OuterClientPropertyName;

	TWeakInterfacePtr<IRigVMClientExternalModelHost> ExternalModelHost;

#if WITH_EDITOR
	/** Delegate raised after any graph in this client was modified and after the client adopted these changes */
	FRigVMClientPostGraphModified PostGraphModifiedDelegate;
#endif // WITH_EDITOR

	mutable FTransactionallySafeCriticalSection OuterClientHostLock;
	mutable FTransactionallySafeCriticalSection ControllersLock;

	friend class UEngineTestClientHost;
	friend class URigVMBlueprint;
	friend class IRigVMEditorAssetInterface;
	friend class FRigVMExampleAssetActions;
	friend class FControlRigAssetActions;
};

template<>
struct TStructOpsTypeTraits<FRigVMClient> : public TStructOpsTypeTraitsBase2<FRigVMClient>
{
	enum 
	{
		WithCopy = true,
	};
};

#undef UE_API
