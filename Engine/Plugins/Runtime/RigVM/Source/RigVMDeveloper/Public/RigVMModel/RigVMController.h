// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMGraph.h"
#include "RigVMFunctionLibrary.h"
#include "RigVMSchema.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMModel/Nodes/RigVMParameterNode.h"
#include "RigVMModel/Nodes/RigVMCommentNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"
#include "RigVMModel/Nodes/RigVMIfNode.h"
#include "RigVMModel/Nodes/RigVMSelectNode.h"
#include "RigVMModel/Nodes/RigVMTemplateNode.h"
#include "RigVMModel/Nodes/RigVMEnumNode.h"
#include "RigVMModel/Nodes/RigVMCollapseNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"
#include "RigVMModel/RigVMBuildData.h"
#include "RigVMCore/RigVMUserWorkflow.h"
#include "RigVMCore/RigVMObjectArchive.h"
#include "UObject/Interface.h"
#include "Algo/Sort.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "RigVMController.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMActionStack;
class IRigVMClientHost;
struct FRigVMGraphFunctionArgument;
struct FRigVMGraphFunctionHeader;

UENUM()
enum class ERigVMControllerBulkEditType : uint8
{
	AddExposedPin,
    RemoveExposedPin,
    RenameExposedPin,
    ChangeExposedPinType,
    AddVariable,
    RemoveVariable,
	RenameVariable,
    ChangeVariableType,
	RemoveFunction,
    Max UMETA(Hidden),
};

UENUM()
enum class ERigVMControllerBulkEditProgress : uint8
{
	BeginLoad,
    FinishedLoad,
	BeginEdit,
    FinishedEdit,
    Max UMETA(Hidden),
};

struct FRigVMController_BulkEditResult
{
	bool bCanceled;
	bool bSetupUndoRedo;

	FRigVMController_BulkEditResult()
		: bCanceled(false)
		, bSetupUndoRedo(true)
	{}
};

class FRigVMControllerCompileBracketScope
{
public:
   
	UE_API FRigVMControllerCompileBracketScope(URigVMController *InController);

	UE_API ~FRigVMControllerCompileBracketScope();

private:

	URigVMGraph* Graph;
	bool bSuspendNotifications;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMController_ShouldStructUnfoldDelegate, const UStruct*)
DECLARE_DELEGATE_RetVal_OneParam(TArray<FRigVMExternalVariable>, FRigVMController_GetExternalVariablesDelegate, URigVMGraph*)
DECLARE_DELEGATE_RetVal(const FRigVMByteCode*, FRigVMController_GetByteCodeDelegate)
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMController_RequestLocalizeFunctionDelegate, FRigVMGraphFunctionIdentifier&)
DECLARE_DELEGATE_RetVal_ThreeParams(FName, FRigVMController_RequestNewExternalVariableDelegate, FRigVMGraphVariableDescription, bool, bool);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMController_IsDependencyCyclicDelegate, const FRigVMGraphFunctionHeader& Dependent, const FRigVMGraphFunctionHeader& Dependency)
DECLARE_DELEGATE_RetVal_TwoParams(FRigVMController_BulkEditResult, FRigVMController_RequestBulkEditDialogDelegate, URigVMLibraryNode*, ERigVMControllerBulkEditType)
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMController_RequestBreakLinksDialogDelegate, TArray<URigVMLink*>)
DECLARE_DELEGATE_RetVal_OneParam(TRigVMTypeIndex, FRigVMController_RequestPinTypeSelectionDelegate, const TArray<TRigVMTypeIndex>& Types)
DECLARE_DELEGATE_FiveParams(FRigVMController_OnBulkEditProgressDelegate, TSoftObjectPtr<URigVMFunctionReferenceNode>, ERigVMControllerBulkEditType, ERigVMControllerBulkEditProgress, int32, int32)
DECLARE_DELEGATE_RetVal_TwoParams(FString, FRigVMController_PinPathRemapDelegate, const FString& /* InPinPath */, bool /* bIsInput */);
DECLARE_DELEGATE_OneParam(FRigVMController_RequestJumpToHyperlinkDelegate, const UObject* InSubject);
DECLARE_DELEGATE_OneParam(FRigVMController_ConfigureWorkflowOptionsDelegate, URigVMUserWorkflowOptions* InOutOptions);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMController_CheckPinCompatibilityDelegate, URigVMPin*, URigVMPin*);

USTRUCT(BlueprintType)
struct FRigStructScope
{
	GENERATED_BODY()

public:
	
	FRigStructScope()
		: ScriptStruct(nullptr)
		, Memory(nullptr)
	{}

	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	FRigStructScope(const T& InInstance)
		: ScriptStruct(T::StaticStruct())
		, Memory((const uint8*)&InInstance)
	{}

	FRigStructScope(const FStructOnScope& InScope)
		: ScriptStruct(Cast<UScriptStruct>(InScope.GetStruct()))
		, Memory(InScope.GetStructMemory()) 
	{}

	const UScriptStruct* GetScriptStruct() const { return ScriptStruct; }
	const uint8* GetMemory() const { return Memory; }
	bool IsValid() const { return ScriptStruct != nullptr && Memory != nullptr; }

protected:

	const UScriptStruct* ScriptStruct;
	const uint8* Memory;
};

// A struct describing the result of a backwards compatibility patch
USTRUCT()
struct FRigVMClientPatchResult
{
public:

	GENERATED_BODY()
	
	FRigVMClientPatchResult()
		: bSucceeded(true)
		, bChangedContent(false)
		, bRequiresToMarkPackageDirty(false)
	{}

	bool Succeeded() const { return bSucceeded; }
	bool ChangedContent() const { return bChangedContent; }
	bool RequiresToMarkPackageDirty() const { return bRequiresToMarkPackageDirty; }

	const TArray<FString>& GetErrorMessages() const { return ErrorMessages; }
	const TArray<FString>& GetRemovedNodes() const { return RemovedNodes; }
	const TArray<TWeakObjectPtr<const URigVMNode>>& GetAddedNodes() const { return AddedNodes; }

private:

	UE_API void Merge(const FRigVMClientPatchResult& InOther);
	
	bool bSucceeded;
	bool bChangedContent;
	bool bRequiresToMarkPackageDirty;

	TArray<FString> ErrorMessages;
	TArray<FString> RemovedNodes;
	TArray<TWeakObjectPtr<const URigVMNode>> AddedNodes;

	friend struct FRigVMClient;
	friend class URigVMController;
};

struct FRigVMPinInfo
{
	UE_API FRigVMPinInfo();
	UE_API FRigVMPinInfo(const URigVMPin* InPin, int32 InParentIndex, ERigVMPinDirection InDirection, ERigVMPinDefaultValueType InDefaultValueType);
	UE_API FRigVMPinInfo(FProperty* InProperty, ERigVMPinDirection InDirection, int32 InParentIndex, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory);

	UE_API void CorrectExecuteTypeIndex();

	int32 ParentIndex;
	FGuid Guid;
	FName Name;
	ERigVMPinDirection Direction;
	TRigVMTypeIndex TypeIndex;
	bool bIsArray;
	FProperty* Property;
	FString PinPath;
	FString DefaultValue;
	ERigVMPinDefaultValueType DefaultValueType;
	FString DisplayName;
	FString CustomWidgetName;
	bool bIsExpanded;
	bool bIsConstant;
	bool bIsInputVariable;
	bool bIsDynamicArray;
	bool bIsTrait;
	bool bIsLazy;
	TArray<int32> SubPins;

	friend uint32 GetTypeHash(const FRigVMPinInfo& InPin);

	bool ShouldHideSubPins() const
	{
		return bIsInputVariable && Direction == ERigVMPinDirection::Input;
	}
};

struct FRigVMPinInfoArray
{
	FRigVMPinInfoArray() {}
	UE_API explicit FRigVMPinInfoArray(const URigVMNode* InNode, URigVMController* InController);
	UE_API FRigVMPinInfoArray(const URigVMNode* InNode, URigVMController* InController, const FRigVMPinInfoArray* InPreviousPinInfos);
	UE_API FRigVMPinInfoArray(const FRigVMGraphFunctionHeader& FunctionHeader, URigVMController* InController, const FRigVMPinInfoArray* InPreviousPinInfos = nullptr);

	int32 Num() const { return Pins.Num(); }
	const FRigVMPinInfo& operator[](int32 InIndex) const { return Pins[InIndex]; }
	FRigVMPinInfo& operator[](int32 InIndex) { return Pins[InIndex]; }
	TArray<FRigVMPinInfo>::RangedForIteratorType begin() const { return Pins.begin(); }
	TArray<FRigVMPinInfo>::RangedForIteratorType end() const { return Pins.end(); }

	UE_API int32 AddPin(const URigVMPin* InPin, int32 InParentIndex, ERigVMPinDirection InDirection, ERigVMPinDefaultValueType InDefaultValueType);
	UE_API int32 AddPin(FProperty* InProperty, URigVMController* InController, ERigVMPinDirection InDirection, int32 InParentIndex, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory, bool bAddSubPins);
	UE_API int32 AddPin(URigVMController* InController, int32 InParentIndex, const FName& InName, ERigVMPinDirection InDirection, TRigVMTypeIndex InTypeIndex, const FString& InDefaultValue, ERigVMPinDefaultValueType InDefaultValueType, const uint8* InDefaultValueMemory, const FRigVMPinInfoArray* InPreviousPinInfos, bool bAddSubPins);
	UE_API void AddPins(UScriptStruct* InScriptStruct, URigVMController* InController, ERigVMPinDirection InDirection, int32 InParentIndex, TFunction<ERigVMPinDefaultValueType(const FName&)> InDefaultValueTypeGetter, const uint8* InDefaultValueMemory, bool bAddSubPins);

	UE_API const FString& GetPinPath(const int32 InIndex) const;
	UE_API int32 GetIndexFromPinPath(const FString& InPinPath) const;
	UE_API const FRigVMPinInfo* GetPinFromPinPath(const FString& InPinPath) const;
	UE_API int32 GetRootIndex(const int32 InIndex) const;

	friend RIGVMDEVELOPER_API uint32 GetTypeHash(const FRigVMPinInfoArray& InPins);

	mutable TArray<FRigVMPinInfo> Pins;
	mutable TMap<FString, int32> PinPathLookup;;
};

RIGVMDEVELOPER_API uint32 GetTypeHash(const FRigVMPinInfoArray& InPins);

/**
 * The Controller is the sole authority to perform changes
 * on the Graph. The Controller itself is stateless.
 * The Controller offers a Modified event to subscribe to 
 * for user interface views - so they can be informed about
 * any change that's happening within the Graph.
 * The Controller routes all changes through the Graph itself,
 * so you can have N Controllers performing edits on 1 Graph,
 * and N Views subscribing to 1 Controller.
 * In Python you can also subscribe to this event to be 
 * able to react to topological changes of the Graph there.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMController : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// Default constructor
	UE_API URigVMController();

	// Default destructor
	UE_API ~URigVMController();

#if WITH_EDITORONLY_DATA
	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

	// Returns the currently edited Graph of this controller.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMGraph* GetGraph() const;

	// Sets the currently edited Graph of this controller.
	// This causes a GraphChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetControllerForGraph instead."))
	UE_API void SetGraph(URigVMGraph* InGraph);

	// Returns the schema used by this controller
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMSchema* GetSchema() const;

	UE_DEPRECATED(5.5, "Please use SetSchemaClass instead.")
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please use SetSchemaClass instead."))
	void SetSchema(URigVMSchema* InSchema) { check(InSchema); SetSchemaClass(InSchema->GetClass()); }

	// Sets the schema class on the controller
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	void SetSchemaClass(TSubclassOf<URigVMSchema> InSchemaClass) { SchemaClass = InSchemaClass; }

	// Pushes a new graph to the stack
	// This causes a GraphChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetControllerForGraph instead."))
	UE_API bool PushGraph(URigVMGraph* InGraph, bool bSetupUndoRedo = true);

	// Pops the last graph off the stack
	// This causes a GraphChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please rely on GetControllerForGraph instead."))
	UE_API URigVMGraph* PopGraph(bool bSetupUndoRedo = true);
	
	// Returns the top level graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMGraph* GetTopLevelGraph() const;

	// Returns another controller for a given graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMController* GetControllerForGraph(const URigVMGraph* InGraph) const;

	// Returns the client host this controller belongs to
	UE_API IRigVMClientHost* GetClientHost() const;

	// Returns all events present on the client host
	UE_API TArray<FName> GetAllEventNames() const; 

	// The Modified event used to subscribe to changes
	// happening within the Graph. This is broadcasted to 
	// for any change happening - not only the changes 
	// performed by this Controller - so it can be used
	// for UI Views to react accordingly.
	UE_API FRigVMGraphModifiedEvent& OnModified();

	// Submits an event to the graph for broadcasting.
	UE_API void Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject) const;

	// Resends all notifications
	UE_API void ResendAllNotifications();

	// Enables or disables the error reporting of this Controller.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	void EnableReporting(bool bEnabled = true) { bReportWarningsAndErrors = bEnabled; }

	// Returns true if reporting is enabled
	UFUNCTION(BlueprintPure, Category = RigVMController)
	bool IsReportingEnabled() const { return bReportWarningsAndErrors; }

	// Returns true if the controller is currently transacting
	UFUNCTION(BlueprintPure, Category = RigVMController)
	bool IsTransacting() const { return bIsTransacting; }

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TArray<FString> GeneratePythonCommands();

	UE_API TArray<FString> GetAddNodePythonCommands(URigVMNode* Node) const;
	UE_API TArray<FString> GetAddTraitPythonCommands(URigVMNode* Node, const FName& TraitName) const;

	UE_API FRigVMGraphFunctionStore* GetGraphFunctionStore() const;
	UE_API FRigVMGraphFunctionData* FindFunctionData(const FName& InFunctionName) const;

#if WITH_EDITOR
	// Note: The functions below are scoped with WITH_EDITOR since we are considering
	// to move this code into the runtime in the future. Right now there's a dependency
	// on the metadata of the USTRUCT - which is only available in the editor.

	// Adds a Function / Struct Node to the edited Graph.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMUnitNode* AddUnitNode(UScriptStruct* InScriptStruct, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Function / Struct Node to the edited Graph given its struct object path name.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMUnitNode* AddUnitNodeFromStructPath(const FString& InScriptStructPath, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a unit node using a template
	template <
		typename T,
		typename TEnableIf<TModels_V<CRigVMUStruct, T>>::Type * = nullptr
	>
	URigVMUnitNode* AddUnitNode(const T& InDefaults, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false)
	{
		return AddUnitNodeWithDefaults(T::StaticStruct(), InDefaults, InMethodName, InPosition, InNodeName, bSetupUndoRedo, bPrintPythonCommand); 
	}

	// Adds a Function / Struct Node to the edited Graph.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMUnitNode* AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, const FString& InDefaults, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Function / Struct Node to the edited Graph.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UE_API URigVMUnitNode* AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, const FRigStructScope& InDefaults, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API URigVMUnitNode* AddUnitNodeWithDefaults(UScriptStruct* InScriptStruct, TSubclassOf<URigVMUnitNode> InUnitNodeClass, const FRigStructScope& InDefaults, const FName& InMethodName = TEXT("Execute"), const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Function / Struct Node to the edited Graph.
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetUnitNodeDefaults(URigVMUnitNode* InNode, const FString& InDefaults, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool SetUnitNodeDefaults(URigVMUnitNode* InNode, const FRigStructScope& InDefaults, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Variable Node to the edited Graph.
	// Variables represent local work state for the function and
	// can be read from and written to. 
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMVariableNode* AddVariableNode(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API URigVMVariableNode* AddVariableNode(const FName& InVariableName, TSubclassOf<URigVMVariableNode> InVariableNodeClass, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Variable Node to the edited Graph given a struct object path name.
	// Variables represent local work state for the function and
	// can be read from (bIsGetter == true) or written to (bIsGetter == false). 
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMVariableNode* AddVariableNodeFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsGetter, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Refreshes the variable node with the new data
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta = (DeprecatedFunction, DeprecationMessage = "Use RefreshVariableNode that takes a FGuid"))
	UE_API void RefreshVariableNode(const FName& InNodeName, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins = true);
	
	// Refreshes the variable node with the new data
	UE_API void RefreshVariableNode(const FName& InNodeName, const FGuid& InVariableGuid, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins = true);

	// Removes all nodes related to a given variable
	UE_API void OnExternalVariableRemoved(const FGuid& InVarGuid, bool bSetupUndoRedo);

	// Renames the variable name in all relevant nodes
	UE_API bool OnExternalVariableRenamed(const FGuid& InVarGuid, const FName& InNewVarName, bool bSetupUndoRedo);

	// Changes the data type of all nodes matching a given variable name
	UE_DEPRECATED(5.8, "Use OnExternalVariableTypeChanged that receives the variable Guid")
	UE_API void OnExternalVariableTypeChanged(const FName& InVarName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo);
	UE_API void OnExternalVariableTypeChanged(const FGuid& InVarGuid, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo);
	UE_DEPRECATED(5.8, "Use OnExternalVariableTypeChangedFromObjectPath that receives the variable Guid")
	UE_API void OnExternalVariableTypeChangedFromObjectPath(const FName& InVarName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo);
	UE_API void OnExternalVariableTypeChangedFromObjectPath(const FGuid& InVarGuid, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo);
	

	// Refreshes the variable node with the new data
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMVariableNode* ReplaceParameterNodeWithVariable(const FName& InNodeName, const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo);

	// Turns a resolved templated node(s) back into its template.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool UnresolveTemplateNodes(const TArray<FName>& InNodeNames, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool UnresolveTemplateNodes(const TArray<URigVMNode*>& InNodes, bool bSetupUndoRedo);

	// Upgrades a set of nodes with each corresponding next known version
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TArray<URigVMNode*> UpgradeNodes(const TArray<FName>& InNodeNames, bool bRecursive = true, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Parameter Node to the edited Graph.
	// Parameters represent input or output arguments to the Graph / Function.
	// Input Parameters are constant values / literals.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction))
	UE_API URigVMParameterNode* AddParameterNode(const FName& InParameterName, const FString& InCPPType, UObject* InCPPTypeObject, bool bIsInput, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Parameter Node to the edited Graph given a struct object path name.
	// Parameters represent input or output arguments to the Graph / Function.
	// Input Parameters are constant values / literals.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction))
	UE_API URigVMParameterNode* AddParameterNodeFromObjectPath(const FName& InParameterName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bIsInput, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Comment Node to the edited Graph.
	// Comments can be used to annotate the Graph.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMCommentNode* AddCommentNode(const FString& InCommentText, const FVector2D& InPosition = FVector2D::ZeroVector, const FVector2D& InSize = FVector2D(400.f, 300.f), const FLinearColor& InColor = FLinearColor::Black, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Reroute Node on an existing Link to the edited Graph.
	// Reroute Nodes can be used to visually improve the data flow,
	// they don't require any additional memory though and are purely
	// cosmetic. This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMRerouteNode* AddRerouteNodeOnLink(URigVMLink* InLink, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Reroute Node on an existing Link to the edited Graph given the Link's string representation.
	// Reroute Nodes can be used to visually improve the data flow,
	// they don't require any additional memory though and are purely
	// cosmetic. This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMRerouteNode* AddRerouteNodeOnLinkPath(const FString& InLinkPinPathRepresentation, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Reroute Node on an existing Pin to the editor Graph.
	// Reroute Nodes can be used to visually improve the data flow,
	// they don't require any additional memory though and are purely
	// cosmetic. This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMRerouteNode* AddRerouteNodeOnPin(const FString& InPinPath, bool bAsInput, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a free Reroute Node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMRerouteNode* AddFreeRerouteNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bIsConstant, const FName& InCustomWidgetName, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a constant node
	UE_API URigVMTemplateNode* AddConstantNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a make struct node
	UE_API URigVMTemplateNode* AddMakeStructNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a break struct node
	UE_API URigVMTemplateNode* AddBreakStructNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a constant node on a pin
	UE_API URigVMTemplateNode* AddConstantNodeOnPin(const FString& InPinPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a make struct node on a pin
	UE_API URigVMTemplateNode* AddMakeStructNodeOnPin(const FString& InPinPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// helper function to set up a break struct node on a pin
	UE_API URigVMTemplateNode* AddBreakStructNodeOnPin(const FString& InPinPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// Adds a branch node to the graph.
	// Branch nodes can be used to split the execution of into multiple branches,
	// allowing to drive behavior by logic.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddBranchNode(const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds an if node to the graph.
	// If nodes can be used to pick between two values based on a condition.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddIfNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddIfNodeFromStruct(UScriptStruct* InScriptStruct, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// Adds a select node to the graph.
	// Select nodes can be used to pick between multiple values based on an index.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddSelectNode(const FString& InCPPType, const FName& InCPPTypeObjectPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddSelectNodeFromStruct(UScriptStruct* InScriptStruct, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// Adds a template node to the graph.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMTemplateNode* AddTemplateNode(const FName& InNotation, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Returns all registered unit structs
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	static UE_API TArray<UScriptStruct*> GetRegisteredUnitStructs();

	// Returns all registered template notations
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	static UE_API TArray<FString> GetRegisteredTemplates();

	// Returns all supported unit structs for a given template notation
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	static UE_API TArray<UScriptStruct*> GetUnitStructsForTemplate(const FName& InNotation);

	// Returns the template for a given function (or an empty string)
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	static UE_API FString GetTemplateForUnitStruct(UScriptStruct* InFunction, const FString& InMethodName = TEXT("Execute"));

	// Resolves a wildcard pin on any node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ResolveWildCardPin(const FString& InPinPath, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool ResolveWildCardPin(URigVMPin* InPin, const FRigVMTemplateArgumentType& InType, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool ResolveWildCardPin(const FString& InPinPath, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool ResolveWildCardPin(URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Function / Struct Node to the edited Graph as an injected node
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMInjectionInfo* AddInjectedNode(const FString& InPinPath, bool bAsInput, UScriptStruct* InScriptStruct, const FName& InMethodName, const FName& InInputPinName, const FName& InOutputPinName, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Function / Struct Node to the edited Graph as an injected node
	// UnitNode represent a RIGVM_METHOD declaration on a USTRUCT.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMInjectionInfo* AddInjectedNodeFromStructPath(const FString& InPinPath, bool bAsInput, const FString& InScriptStructPath, const FName& InMethodName, const FName& InInputPinName, const FName& InOutputPinName, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true);

	// Removes an injected node
	// This causes a NodeRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveInjectedNode(const FString& InPinPath, bool bAsInput, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Ejects the last injected node on a pin
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* EjectNodeFromPin(const FString& InPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds an enum node to the graph
	// Enum nodes can be used to represent constant enum values within the graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMEnumNode* AddEnumNode(const FName& InCPPTypeObjectPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a Array Node to the edited Graph.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddArrayNode(ERigVMOpCode InOpCode, const FString& InCPPType, UObject* InCPPTypeObject, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, bool bIsPatching = false);

	// Adds a Array Node to the edited Graph given a struct object path name.
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMNode* AddArrayNodeFromObjectPath(ERigVMOpCode InOpCode, const FString& InCPPType, const FString& InCPPTypeObjectPath, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, bool bIsPatching = false);

	// Adds an entry invocation node
	// This causes a NodeAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMInvokeEntryNode* AddInvokeEntryNode(const FName& InEntryName, const FVector2D& InPosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a trait to a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FName AddTrait(const FName& InNodeName, const FName& InTraitTypeObjectPath, const FName& InTraitName = NAME_None, const FString& InDefaultValue = TEXT(""), int32 InPinIndex = -1, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API FName AddTrait(URigVMNode* InNode, UScriptStruct* InTraitScriptStruct, const FName& InTraitName, const FString& InDefaultValue, int32 InPinIndex = -1, bool bSetupUndoRedo = true);

	// Removes a trait from a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API virtual bool RemoveTrait(const FName& InNodeName, const FName& InTraitName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool RemoveTrait(URigVMNode* InNode, const FName& InTraitName, bool bSetupUndoRedo = true);

	// Un-does the last action on the stack.
	// Note: This should really only be used for unit tests,
	// use the GEditor's main Undo method instead.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool Undo();

	// Re-does the last action on the stack.
	// Note: This should really only be used for unit tests,
	// use the GEditor's main Undo method instead.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool Redo();

	// Opens an undo bracket / scoped transaction for
	// a series of actions to be performed as one step on the 
	// Undo stack. This is primarily useful for Python.
	// This causes a UndoBracketOpened modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool OpenUndoBracket(const FString& InTitle);

	// Closes an undo bracket / scoped transaction.
	// This is primarily useful for Python.
	// This causes a UndoBracketClosed modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool CloseUndoBracket();

	// Cancels an undo bracket / scoped transaction.
	// This is primarily useful for Python.
	// This causes a UndoBracketCanceled modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool CancelUndoBracket();

	// Exports the given nodes as text
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString ExportNodesToText(const TArray<FName>& InNodeNames, bool bIncludeExteriorLinks = false);

	// Exports the given node as text
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString ExportNodeToText(const URigVMNode* InNode, bool bIncludeExteriorLinks = false);

	// Exports the selected nodes as text
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString ExportSelectedNodesToText(bool bIncludeExteriorLinks = false);

	// Exports the given nodes as text
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool CanImportNodesFromText(const FString& InText);

	// Exports the given nodes as text
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TArray<FName> ImportNodesFromText(const FString& InText, bool bSetupUndoRedo = true, bool bPrintPythonCommands = false);

	// Exports the given function to a binary archive
	UE_API bool ExportFunctionToArchive(const FName& InFunctionName, FRigVMObjectArchive& OutArchive);

	// Imports a function from a binary archive
	UE_API URigVMLibraryNode* ImportFunctionFromArchive(const FRigVMObjectArchive& InArchive, const FName& InFunctionName = FName(NAME_None));

	// Copies a function declaration into this graph's local function library
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMLibraryNode* LocalizeFunctionFromPath(const FString& InHostPath, const FName& InFunctionName, bool bLocalizeDependentPrivateFunctions = true, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
    UE_API URigVMLibraryNode* LocalizeFunction(
		const FRigVMGraphFunctionIdentifier& InFunctionDefinition,
		bool bLocalizeDependentPrivateFunctions = true,
		bool bSetupUndoRedo = true,
		bool bPrintPythonCommand = false);

	// Copies a series of function declaratioms into this graph's local function library
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TMap<FRigVMGraphFunctionIdentifier, URigVMLibraryNode*> LocalizeFunctions(
		TArray<FRigVMGraphFunctionIdentifier> InFunctionDefinitions,
		bool bLocalizeDependentPrivateFunctions = true,
		bool bSetupUndoRedo = true,
		bool bPrintPythonCommand = false);


	// Turns a series of nodes into a Collapse node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMCollapseNode* CollapseNodes(const TArray<FName>& InNodeNames, const FString& InCollapseNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, bool bIsAggregate = false);

	// Turns a library node into its contained nodes
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TArray<URigVMNode*> ExpandLibraryNode(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Turns a collapse node into a function node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FName PromoteCollapseNodeToFunctionReferenceNode(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, const FString& InExistingFunctionDefinitionPath = TEXT(""));

	// Turns a collapse node into a function node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FName PromoteFunctionReferenceNodeToCollapseNode(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, bool bRemoveFunctionDefinition = false);

#endif

	// Removes a node from the graph
	// This causes a NodeRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveNode(URigVMNode* InNode, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a node from the graph given the node's name.
	// This causes a NodeRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveNodeByName(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a list of nodes from the graph
	// This causes a NodeRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveNodes(TArray<URigVMNode*> InNodes, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a list of nodes from the graph given the names
	// This causes a NodeRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveNodesByName(const TArray<FName>& InNodeNames, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Renames a node in the graph
	// This causes a NodeRenamed modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RenameNode(URigVMNode* InNode, const FName& InNewName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Selects a single node in the graph.
	// This causes a NodeSelected / NodeDeselected modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SelectNode(URigVMNode* InNode, bool bSelect = true, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Selects a single node in the graph by name.
	// This causes a NodeSelected / NodeDeselected modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SelectNodeByName(const FName& InNodeName, bool bSelect = true, bool bSetupUndoRedo = true);

	// Deselects all currently selected nodes in the graph.
	// This might cause several NodeDeselected modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearNodeSelection(bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Selects the nodes given the selection
	// This might cause several NodeDeselected modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeSelection(const TArray<FName>& InNodeNames, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Selects the linked nodes given the input node names
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SelectLinkedNodes(const TArray<FName>& InNodeNames, bool bSelectSourceNodes, bool bClearSelection, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Selects the node islands given the input node names
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SelectNodeIslands(const TArray<FName>& InNodeNames, bool bClearSelection, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the position of a node in the graph.
	// This causes a NodePositionChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodePosition(URigVMNode* InNode, const FVector2D& InPosition, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the position of a node in the graph by name.
	// This causes a NodePositionChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodePositionByName(const FName& InNodeName, const FVector2D& InPosition, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the size of a node in the graph.
	// This causes a NodeSizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeSize(URigVMNode* InNode, const FVector2D& InSize, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the size of a node in the graph by name.
	// This causes a NodeSizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeSizeByName(const FName& InNodeName, const FVector2D& InSize, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the raw node title of a node in the graph.
	// Some nodes generate customs node titles that override this setting.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeTitle(URigVMNode* InNode, const FString InNodeTitle, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the raw node title of a node in the graph.
	// Some nodes generate customs node titles that override this setting.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeTitleByName(const FName& InNodeName, const FString InNodeTitle, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);

	// Sets the color of a node in the graph.
	// This causes a NodeColorChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeColor(URigVMNode* InNode, const FLinearColor& InColor, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the color of a node in the graph by name.
	// This causes a NodeColorChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeColorByName(const FName& InNodeName, const FLinearColor& InColor, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);

	// Sets the category of a node in the graph.
	// This causes a NodeCategoryChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeCategory(URigVMCollapseNode* InNode, const FString& InCategory, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the category of a node in the graph.
	// This causes a NodeCategoryChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeCategoryByName(const FName& InNodeName, const FString& InCategory, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);

	// Sets the keywords of a node in the graph.
	// This causes a NodeKeywordsChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeKeywords(URigVMCollapseNode* InNode, const FString& InKeywords, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the keywords of a node in the graph.
	// This causes a NodeKeywordsChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeKeywordsByName(const FName& InNodeName, const FString& InKeywords, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);

	// Sets the function description of a node in the graph.
	// This causes a NodeDescriptionChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeDescription(URigVMCollapseNode* InNode, const FString& InDescription, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false);

	// Sets the keywords of a node in the graph.
	// This causes a NodeDescriptionChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeDescriptionByName(const FName& InNodeName, const FString& InDescription, bool bSetupUndoRedo = true, bool bMergeUndoAction = false);

	// Sets the comment text and properties of a comment node in the graph.
	// This causes a CommentTextChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetCommentText(URigVMNode* InNode, const FString& InCommentText, const int32& InCommentFontSize, const bool& bInCommentBubbleVisible, const bool& bInCommentColorBubble, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the comment text and properties of a comment node in the graph by name.
	// This causes a CommentTextChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetCommentTextByName(const FName& InNodeName, const FString& InCommentText, const int32& InCommentFontSize, const bool& bInCommentBubbleVisible, const bool& bInCommentColorBubble, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Renames a variable in the graph.
	// This causes a VariableRenamed modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction))
	UE_API bool RenameVariable(const FName& InOldName, const FName& InNewName, bool bSetupUndoRedo = true);
	UE_API bool RenameVariable(const FGuid& InGuid, const FName& InNewName, bool bSetupUndoRedo = true);

	// Renames a parameter in the graph.
	// This causes a ParameterRenamed modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction))
	UE_API bool RenameParameter(const FName& InOldName, const FName& InNewName, bool bSetupUndoRedo = true);\

	// Upgrades a set of nodes with each corresponding next known version
	UE_API TArray<URigVMNode*> UpgradeNodes(const TArray<URigVMNode*>& InNodes, bool bRecursive = true, bool bSetupUndoRedo = true);

	// Upgrades a single node with its next known version
	UE_API URigVMNode* UpgradeNode(URigVMNode* InNode, bool bSetupUndoRedo = true, FRigVMController_PinPathRemapDelegate* OutRemapPinDelegate = nullptr);

	// Sets the pin to be expanded or not
	// This causes a PinExpansionChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinExpansion(const FString& InPinPath, bool bIsExpanded, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the pin to be watched (or not)
	// This causes a PinWatchedChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinIsWatched(const FString& InPinPath, bool bIsWatched, bool bSetupUndoRedo = true);

	// Sets the pin display name. The display name is UI relevant only.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinDisplayName(const FString& InPinPath, const FString& InDisplayName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a new pin category. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddEmptyPinCategory(const FName& InNodeName, const FString& InCategory, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the pin category. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinCategory(const FString& InPinPath, const FString& InCategory, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Clears the pin category. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearPinCategory(const FString& InPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a pin category. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemovePinCategory(const FName& InNodeName, const FString& InPinCategory, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Renames a pin category. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RenamePinCategory(const FName& InNodeName, const FString& InOldPinCategory, const FString& InNewPinCategory, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Changes a pin category's index. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinCategoryIndex(const FName& InNodeName, const FString& InPinCategory, int32 InNewIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Changes a pin category's expansion state. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinCategoryExpansion(const FName& InNodeName, const FString& InPinCategory, bool bIsExpanded, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Changes a pin category's expansion state. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinIndexInCategory(const FString& InPinPath, int32 InIndexInCategory, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Applies a complete node layout to a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetNodeLayout(const FName& InNodeName, FRigVMNodeLayout InLayout, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes any layout information from a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearNodeLayout(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Returns the default value of a pin given its pinpath.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString GetPinDefaultValue(const FString& InPinPath);

	// Sets the default value of a pin given its pinpath.
	// This causes a PinDefaultValueChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetPinDefaultValue(const FString& InPinPath, const FString& InDefaultValue, bool bResizeArrays = true, bool bSetupUndoRedo = true, bool bMergeUndoAction = false, bool bPrintPythonCommand = false, bool bSetValueOnLinkedPins = true);
	UE_API bool SetPinDefaultValue(URigVMPin* InPin, const FString& InDefaultValue, bool bResizeArrays, bool bSetupUndoRedo, bool bMergeUndoAction, bool bSetValueOnLinkedPins = true);

	// Resets the default value of a pin given its pinpath.
	// This causes a PinDefaultValueChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ResetPinDefaultValue(const FString& InPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Resets the default value of a list of pin given the pinpaths.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ResetDefaultValueForPins(const TArray<FString>& InPinPaths, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Resets the default value of all pins of a given node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ResetDefaultValueForAllPinsOnNode(const FName& InNodeName, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Resets the default value of all pins of a list of nodes
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ResetDefaultValueForAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Adds an override to the given pin path
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddOverrideToPin(const FString& InPinPath, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Adds an override to a given list of pin paths
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddOverrideToPins(const TArray<FString>& InPinPaths, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Adds an override to all pins on a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddOverrideToAllPinsOnNode(const FName& InNodeName, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Adds an override to all pins on a list of nodes
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddOverrideToAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Clears an override on a given pin path
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearOverrideOnPin(const FString& InPinPath, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Clears the overrides on a given list of pin paths
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearOverrideOnPins(const TArray<FString>& InPinPaths, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Clears the overrides for all pins on a node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearOverrideOnAllPinsOnNode(const FName& InNodeName, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	// Clears the overrides for all pins of a list of nodes
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearOverrideOnAllPinsOnNodes(const TArray<FName>& InNodeNames, bool bSetupUndo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString AddAggregatePin(const FString& InNodeName, const FString& InPinName, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveAggregatePin(const FString& InPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

#if UE_RIGVM_AGGREGATE_NODES_ENABLED
	UE_API FString AddAggregatePin(URigVMNode* InNode, const FString& InPinName, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UE_API bool RemoveAggregatePin(URigVMPin* InPin, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
#endif

	// Adds an array element pin to the end of an array pin.
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString AddArrayPin(const FString& InArrayPinPath, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Duplicates an array element pin.
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString DuplicateArrayPin(const FString& InArrayElementPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Inserts an array element pin into an array pin.
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FString InsertArrayPin(const FString& InArrayPinPath, int32 InIndex = -1, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes an array element pin from an array pin.
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveArrayPin(const FString& InArrayElementPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes all (but one) array element pin from an array pin.
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ClearArrayPin(const FString& InArrayPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the size of the array pin
	// This causes a PinArraySizeChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetArrayPinSize(const FString& InArrayPinPath, int32 InSize, const FString& InDefaultValue = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Binds a pin to a variable (or removes the binding given NAME_None)
	// This causes a PinBoundVariableChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool BindPinToVariable(const FString& InPinPath, const FString& InNewBoundVariablePath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes the binging of a pin to a variable
	// This causes a PinBoundVariableChanged modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool UnbindPinFromVariable(const FString& InPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Turns a variable node into one or more bindings
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool MakeBindingsFromVariableNode(const FName& InNodeName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Turns a binding to a variable node
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool MakeVariableNodeFromBinding(const FString& InPinPath, const FVector2D& InNodePosition = FVector2D::ZeroVector, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Promotes a pin to a variable
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool PromotePinToVariable(const FString& InPinPath, bool bCreateVariableNode, const FVector2D& InNodePosition = FVector2D::ZeroVector, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	
	// Promotes a pin to a local variable
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool PromotePinToLocalVariable(const FString& InPinPath, bool bCreateVariableNode, const FVector2D& InNodePosition = FVector2D::ZeroVector, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a link to the graph.
	// This causes a LinkAdded modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddLink(const FString& InOutputPinPath, const FString& InInputPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, ERigVMPinDirection InUserDirection = ERigVMPinDirection::Output, bool bCreateCastNode = false);

	// Removes a link from the graph.
	// This causes a LinkRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool BreakLink(const FString& InOutputPinPath, const FString& InInputPinPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes all links on a given pin from the graph.
	// This might cause multiple LinkRemoved modified event.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool BreakAllLinks(const FString& InPinPath, bool bAsInput = true, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds an exposed pin to the graph controlled by this
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FName AddExposedPin(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false, bool bIsInputVariable = false);

	// Removes an exposed pin from the graph controlled by this
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveExposedPin(const FName& InPinName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Renames an exposed pin in the graph controlled by this
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RenameExposedPin(const FName& InOldPinName, const FName& InNewPinName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Changes the type of an exposed pin in the graph controlled by this
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool ChangeExposedPinType(const FName& InPinName, const FString& InCPPType, const FName& InCPPTypeObjectPath, UPARAM(ref) bool& bSetupUndoRedo, bool bSetupOrphanPins = true, bool bPrintPythonCommand = false);

	// Sets the index for an exposed pin. This can be used to move the pin up and down on the node.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetExposedPinIndex(const FName& InPinName, int32 InNewIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Marks an exposed pin as defined as an input variable
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool MarkExposedPinAsInputVariable(const FName& InPinName, bool bDefinedAsExternalVariable, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	
	UFUNCTION(BlueprintPure, Category = RigVMController)
	UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeaderByName(FString InHostPath, FName InFunctionName) const;

	UFUNCTION(BlueprintPure, Category = RigVMController)
	UE_API FRigVMGraphFunctionHeader FindGraphFunctionHeader(FRigVMGraphFunctionIdentifier InFunctionIdentifier) const;

	UFUNCTION(BlueprintPure, Category = RigVMController)
	UE_API FRigVMGraphFunctionIdentifier FindGraphFunctionIdentifier(FString InHostPath, FName InFunctionName) const;

	// Adds a function reference / invocation to the graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMFunctionReferenceNode* AddFunctionReferenceNodeFromDescription(const FRigVMGraphFunctionHeader& InFunctionDefinition, const FVector2D& InNodePosition = FVector2D::ZeroVector, const
	                                                                     FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMFunctionReferenceNode* AddExternalFunctionReferenceNode(const FString& InHostPath, const FName& InFunctionName, const FVector2D& InNodePosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMFunctionReferenceNode* AddFunctionReferenceNode(URigVMLibraryNode* InFunctionDefinition, const FVector2D& InNodePosition = FVector2D::ZeroVector, const FString& InNodeName = TEXT(""), bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SwapFunctionReferenceByName(const FName& InFunctionReferenceNodeName, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SwapFunctionReference(URigVMFunctionReferenceNode* InFunctionReferenceNode, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SwapAllFunctionReferences(const FRigVMGraphFunctionIdentifier& InOldFunctionIdentifier, const FRigVMGraphFunctionIdentifier& InNewFunctionIdentifier, bool bSetupOrphanPins, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the remapped variable on a function reference node
	UE_DEPRECATED(5.8, "Please use BindPinToVariable instead.")
	UFUNCTION(BlueprintCallable, Category = RigVMController, meta=(DeprecatedFunction, DeprecationMessage="Function has been deprecated, please use BindPinToVariable instead."))
    UE_API bool SetRemappedVariable(URigVMFunctionReferenceNode* InFunctionRefNode, const FName& InInnerVariableName, const FName& InOuterVariableName, bool bSetupUndoRedo = true);

	// Adds a function definition to a function library graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMLibraryNode* AddFunctionToLibrary(const FName& InFunctionName, bool bMutable, const FVector2D& InNodePosition = FVector2D::ZeroVector, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Removes a function from a function library graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveFunctionFromLibrary(const FName& InFunctionName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Renames a function in the function library
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RenameFunction(const FName& InOldFunctionName, const FName& InNewFunctionName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Mark a function as public/private in the function library
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool MarkFunctionAsPublic(const FName& InFunctionName, bool bInIsPublic, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Returns true if a function is marked as public in the function library
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool IsFunctionPublic(const FName& InFunctionName);

	// Sets whether a function is pure (no execute pins) or mutable (has execute pins).
	// Call this on the function library controller.
	// Returns true if the function's purity was successfully changed.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetFunctionIsPure(const FName& InFunctionName, bool bIsPure, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Creates a variant of a function given the name of an existing function variant
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMLibraryNode* CreateFunctionVariant(const FName& InFunctionName, const FName& InVariantName = NAME_None, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a default tag to a function variant
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddDefaultTagToFunctionVariant(const FName& InFunctionName, const FName& InTagName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	
	// Adds a tag to a function variant
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool AddTagToFunctionVariant(const FName& InFunctionName, const FRigVMTag& InTag, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Adds a tag to a function variant
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveTagFromFunctionVariant(const FName& InFunctionName, const FName& InTagName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Returns all variant refs related to the given function
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API TArray<FRigVMVariantRef> FindVariantsOfFunction(const FName& InFunctionName);

	/** Resets the function's guid to a new one and splits it from the former variant set */
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SplitFunctionVariant(const FName& InFunctionName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	/** Merges the function's guid with a provided one to join the variant set */
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool JoinFunctionVariant(const FName& InFunctionName, const FGuid& InGuid, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Add a local variable to the graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FRigVMGraphVariableDescription AddLocalVariable(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, const FString& InDefaultValue, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Add a local variable to the graph given a struct object path name.
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API FRigVMGraphVariableDescription AddLocalVariableFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo = true);

	// Remove a local variable from the graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RemoveLocalVariable(const FName& InVariableName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Rename a local variable from the graph
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool RenameLocalVariable(const FName& InVariableName, const FName& InNewVariableName, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Sets the type of the local variable
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetLocalVariableType(const FName& InVariableName, const FString& InCPPType, UObject* InCPPTypeObject, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetLocalVariableTypeFromObjectPath(const FName& InVariableName, const FString& InCPPType, const FString& InCPPTypeObjectPath, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetLocalVariableDefaultValue(const FName& InVariableName, const FString& InDefaultValue, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Reorder a local variable to a new index (UFUNCTION wrapper - converts name to GUID)
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool SetLocalVariableIndex(const FName& InVariableName, int32 InNewIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// Reorder a local variable to a new index (primary GUID-based implementation)
	UE_API bool SetLocalVariableIndex(const FGuid& InVariableGuid, int32 InNewIndex, bool bSetupUndoRedo = true, bool bPrintPythonCommand = false);

	// creates the options struct for a given workflow
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API URigVMUserWorkflowOptions* MakeOptionsForWorkflow(UObject* InSubject, const FRigVMUserWorkflow& InWorkflow);

	// performs all actions representing the workflow
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API bool PerformUserWorkflow(const FRigVMUserWorkflow& InWorkflow, const URigVMUserWorkflowOptions* InOptions, bool bSetupUndoRedo = true);

	// Determine affected function references for a potential bulk edit on a library node
	UE_API TArray<TSoftObjectPtr<URigVMFunctionReferenceNode>> GetAffectedReferences(ERigVMControllerBulkEditType InEditType, bool bForceLoad = false);

	// Determine affected assets for a potential bulk edit on a library node
	UE_API TArray<FAssetData> GetAffectedAssets(ERigVMControllerBulkEditType InEditType, bool bForceLoad = false);

	// A delegate to retrieve the list of external variables
	FRigVMController_GetExternalVariablesDelegate GetExternalVariablesDelegate;

	// A delegate to retrieve the current bytecode of the graph
	FRigVMController_GetByteCodeDelegate GetCurrentByteCodeDelegate;

	// A delegate to localize a function on demand
	FRigVMController_RequestLocalizeFunctionDelegate RequestLocalizeFunctionDelegate;

	// A delegate to create a new blueprint member variable
	FRigVMController_RequestNewExternalVariableDelegate RequestNewExternalVariableDelegate;

	// A delegate to ask the host / client for a dialog to confirm a bulk edit
	FRigVMController_RequestBulkEditDialogDelegate RequestBulkEditDialogDelegate;

	// A delegate to ask the host / client for a dialog to confirm a bulk edit
	FRigVMController_RequestBreakLinksDialogDelegate RequestBreakLinksDialogDelegate;

	// A delegate to ask the host / client for a dialog to select a pin type
	FRigVMController_RequestPinTypeSelectionDelegate RequestPinTypeSelectionDelegate;

	// A delegate to inform the host / client about the progress during a bulk edit
	FRigVMController_OnBulkEditProgressDelegate OnBulkEditProgressDelegate;

	// A delegate to request the client to follow a hyper link
	FRigVMController_RequestJumpToHyperlinkDelegate RequestJumpToHyperlinkDelegate;

	// A delegate to request to configure an options instance for a node workflow
	FRigVMController_ConfigureWorkflowOptionsDelegate ConfigureWorkflowOptionsDelegate; 

	UE_API void AddPinRedirector(bool bInput, bool bOutput, const FString& OldPinPath, const FString& NewPinPath);
	UE_API void ClearPinRedirectors();

	// Removes nodes which went stale.
	UE_API void RemoveStaleNodes();

#if WITH_EDITOR
	UE_API bool ShouldRedirectPin(UScriptStruct* InOwningStruct, const FString& InOldRelativePinPath, FString& InOutNewRelativePinPath) const;
	UE_API bool ShouldRedirectPin(const FString& InOldPinPath, FString& InOutNewPinPath) const;


	struct FRepopulatePinsNodeData
	{
		URigVMNode* Node = nullptr;
		uint32 PreviousPinHash = 0;
		FRigVMPinInfoArray PreviousPinInfos;
		FRigVMPinInfoArray NewPinInfos;
		TArray<int32> NewPinsToAdd;
		TArray<int32> PreviousPinsToRemove;
		TArray<int32> PreviousPinsToOrphan;
		TArray<int32> PreviousPinsToUpdate;
		bool bSetupOrphanPinsForThisNode = false;
		bool bFollowCoreRedirectors = false;
		bool bRequirePinStates = false;
		bool bRecreateLinks = false;
		bool bRequireRecreateLinks = false;
	};

	// bInRecurseIntoCollapseNodes: when true (default), if InNode is a collapse node the function recurses
	// into its contained graph and resolves a controller for that sub-graph via GetControllerForGraph. Once
	// the recursion starts, all nested collapse nodes are walked (the flag is not propagated as a per-level
	// switch — it gates the top-level entry only). Callers that only need the node's own pin info should
	// pass false to avoid the recursion (and the side-effect of creating a cached sub-graph controller in
	// FRigVMClient::Controllers).
	UE_API void GenerateRepopulatePinsNodeData(TArray<FRepopulatePinsNodeData>& NodesPinData, URigVMNode* InNode, bool bInFollowCoreRedirectors = true, bool bInSetupOrphanedPins = false, bool bInRecreateLinks = false, bool bInRecurseIntoCollapseNodes = true);
	UE_API void OrphanPins(const TArray<FRepopulatePinsNodeData>& NodesPinData);
	UE_API void RepopulatePins(const TArray<FRepopulatePinsNodeData>& NodesPinData);
	UE_API bool CorrectExecutePinsOnNode(URigVMNode* InOutNode);

	UE_API void RepopulatePinsOnNode(URigVMNode* InNode, bool bFollowCoreRedirectors = true, bool bSetupOrphanedPins = false, bool bRecreateLinks = false);
	UE_API bool GenerateNewPinInfos(const FRigVMRegistry& Registry, URigVMNode* InNode, const FRigVMPinInfoArray& PreviousPinInfos, FRigVMPinInfoArray& NewPinInfos, const bool bSetupOrphanPinsForThisNode);
	UE_API void GenerateRepopulatePinLists(const FRigVMRegistry& Registry, FRepopulatePinsNodeData& NodeData);
	UE_API void RepopulatePinsOnNode(const FRigVMRegistry& Registry, const FRepopulatePinsNodeData& NodeData);
	UE_API void RemovePinsDuringRepopulate(URigVMNode* InNode, TArray<URigVMPin*>& InPins, bool bSetupOrphanedPins);

	// removes any orphan pins that no longer holds a link
	// @param bRelayLinks If true we'll try to relay the links back to their original pins 
	UE_API bool RemoveUnusedOrphanedPins(URigVMNode* InNode, bool bRelayLinks = false);

	// Update filtered permutations, and propagate both ways of the link before adding this link
	UE_API bool PrepareToLink(URigVMPin* FirstToResolve, URigVMPin* SecondToResolve, bool bSetupUndoRedo);

#endif

	UE_API bool FullyResolveTemplateNode(URigVMTemplateNode* InNode, int32 InPermutationIndex, bool bSetupUndoRedo);

	FRigVMUnitNodeCreatedContext& GetUnitNodeCreatedContext() { return UnitNodeCreatedContext; }

	// Wires the unit node delegates to the default controller delegates.
	// this is used only within the RigVM Editor currently.
	UE_API void SetupDefaultUnitNodeDelegates(TDelegate<FName(FRigVMExternalVariable, FString)> InCreateExternalVariableDelegate);
	UE_API void ResetUnitNodeDelegates();

	// A flag that can be used to turn off pin default value validation if necessary
	bool bValidatePinDefaults;

	UE_API const FRigVMByteCode* GetCurrentByteCode() const;

	UE_API void ReportInfo(const FString& InMessage) const;
	UE_API void ReportWarning(const FString& InMessage) const;
	UE_API void ReportError(const FString& InMessage) const;
	UE_API void ReportAndNotifyInfo(const FString& InMessage) const;
	UE_API void ReportAndNotifyWarning(const FString& InMessage) const;
	UE_API void ReportAndNotifyError(const FString& InMessage) const;
	UE_API void ReportPinTypeChange(URigVMPin* InPin, const FString& InNewCPPType);
	UE_API void SendUserFacingNotification(const FString& InMessage, float InDuration = 0.f, const UObject* InSubject = nullptr, const FName& InBrushName = TEXT("MessageLog.Warning")) const;
	UE_API void RequestJumpToHyperLink(const URigVMNode* InSubject);

	template <typename... Types>
	void ReportInfof(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportInfo(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportWarningf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportWarning(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportErrorf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportError(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportAndNotifyInfof(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportAndNotifyInfo(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportAndNotifyWarningf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportAndNotifyWarning(FString::Printf(Fmt, Args...));
	}

	template <typename... Types>
	void ReportAndNotifyErrorf(UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args) const
	{
		ReportAndNotifyError(FString::Printf(Fmt, Args...));
	}

	/**
	 * Helper function to disable a series of checks that can be ignored during a unit test
	 */
	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API void SetIsRunningUnitTest(bool bIsRunning);

private:

	UE_API FName AddExposedPin_Internal(const FName& InPinName, ERigVMPinDirection InDirection, const FString& InCPPType, const FName& InCPPTypeObjectPath, const FString& InDefaultValue, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bIsInputVariable, bool bAllowExecutePinManagement);
	UE_API bool RemoveExposedPin_Internal(const FName& InPinName, bool bSetupUndoRedo, bool bPrintPythonCommand, bool bAllowExecutePinManagement);
	UE_API bool ChangeExposedPinType_Internal(const FName& InPinName, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool& bSetupUndoRedo, bool bSetupOrphanPins, bool bPrintPythonCommand, bool bAllowExecutePinManagement);

	UPROPERTY(BlueprintReadOnly, Category = RigVMController, meta = (ScriptName = "ModifiedEvent", AllowPrivateAccess = "true"))
	FRigVMGraphModifiedDynamicEvent ModifiedEventDynamic;

	FRigVMGraphModifiedEvent ModifiedEventStatic;
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	UE_API bool IsValidNodeForGraph(const URigVMNode* InNode);
	UE_API bool IsValidPinForGraph(const URigVMPin* InPin);
	UE_API bool IsValidLinkForGraph(const URigVMLink* InLink);
	UE_API void AddPinsForStruct(UStruct* InStruct, URigVMNode* InNode, URigVMPin* InParentPin, ERigVMPinDirection InPinDirection, const FString& InDefaultValue, bool bAutoExpandArrays, const FRigVMPinInfoArray* PreviousPins = nullptr);
	UE_API void AddPinsForArray(FArrayProperty* InArrayProperty, URigVMNode* InNode, URigVMPin* InParentPin, ERigVMPinDirection InPinDirection, const TArray<FString>& InDefaultValues, bool bAutoExpandArrays);
	UE_API void AddPinsForTemplate(const FRigVMTemplate* InTemplate, const FRigVMTemplateTypeMap& InPinTypeMap, URigVMNode* InNode);
	UE_API void ConfigurePinFromProperty(FProperty* InProperty, URigVMPin* InOutPin, ERigVMPinDirection InPinDirection = ERigVMPinDirection::Invalid) const;
	UE_API void ConfigurePinFromPin(URigVMPin* InOutPin, const URigVMPin* InPin, bool bCopyDisplayName = false);
	UE_API void ConfigurePinFromArgument(URigVMPin* InOutPin, const FRigVMGraphFunctionArgument& InArgument, bool bCopyDisplayName = false);
	UE_API bool ResetPinDefaultValue(URigVMPin* InPin, bool bSetupUndoRedo);
	static UE_API FString GetPinInitialDefaultValue(const URigVMPin* InPin);
	static UE_API FString GetPinInitialDefaultValueFromStruct(UScriptStruct* ScriptStruct, const URigVMPin* InPin, uint32 InOffset);
	UE_API URigVMPin* InsertArrayPin(URigVMPin* ArrayPin, int32 InIndex, const FString& InDefaultValue, bool bSetupUndoRedo);
	UE_API bool RemovePin(URigVMPin* InPinToRemove, bool bSetupUndoRedo, bool bForceBreakLinks = false);
	UE_API FProperty* FindPropertyForPin(const FString& InPinPath);
	UE_API bool BindPinToVariable(URigVMPin* InPin, const FString& InNewBoundVariablePath, bool bSetupUndoRedo, const FString& InVariableNodeName = FString());
	UE_API bool UnbindPinFromVariable(URigVMPin* InPin, bool bSetupUndoRedo);
	UE_API bool MakeBindingsFromVariableNode(URigVMVariableNode* InNode, bool bSetupUndoRedo);
	UE_API bool PromotePinToVariable(URigVMPin* InPin, bool bCreateLocalVariable, bool bCreateVariableNode, const FVector2D& InNodePosition, bool bSetupUndoRedo);
	UE_API URigVMInjectionInfo* InjectNodeIntoPin(const FString& InPinPath, bool bAsInput, const FName& InInputPinName, const FName& InOutputPinName, bool bSetupUndoRedo = true);
	UE_API URigVMInjectionInfo* InjectNodeIntoPin(URigVMPin* InPin, bool bAsInput, const FName& InInputPinName, const FName& InOutputPinName, bool bSetupUndoRedo = true);
	UE_API URigVMNode* EjectNodeFromPin(URigVMPin* InPin, bool bSetupUndoRedo = true, bool bPrintPythonCommands = false);

protected:

	UE_API bool IsValidGraph() const;
	UE_API bool IsValidSchema() const;
	UE_API bool IsGraphEditable() const;

#if WITH_EDITOR
	UE_API void RewireLinks(URigVMPin* OldPin, URigVMPin* NewPin, bool bAsInput, bool bSetupUndoRedo, TArray<URigVMLink*> InLinks = TArray<URigVMLink*>());
#endif

	UE_API virtual const UClass* GetNodeClassForTemplate(FRigVMTemplate* InTemplate) const;
	UE_API bool EjectAllInjectedNodes(URigVMNode* InNode, bool bSetupUndoRedo = true, bool bPrintPythonCommands = false);

public:
#if WITH_EDITOR
	UE_API URigVMUnitNode* AddUnitNode(UScriptStruct* InScriptStruct, TSubclassOf<URigVMUnitNode> InUnitNodeClass, const FName& InMethodName, const FVector2D& InPosition, const FString& InNodeName, bool bSetupUndoRedo, bool bPrintPythonCommand);
#endif

	// try to reconnect source and target pins after a node deletion
	UE_API void RelinkSourceAndTargetPins(URigVMNode* RigNode, bool bSetupUndoRedo = true);

	UE_API bool AddLink(URigVMPin* OutputPin, URigVMPin* InputPin, bool bSetupUndoRedo = true, ERigVMPinDirection InUserDirection = ERigVMPinDirection::Invalid, bool bCreateCastNode = false, bool bIsRestoringLinks = false, FString* OutFailureReason = nullptr);
	UE_API bool BreakLink(URigVMPin* OutputPin, URigVMPin* InputPin, bool bSetupUndoRedo = true);
	UE_API bool BreakAllLinks(URigVMPin* Pin, bool bAsInput, bool bSetupUndoRedo = true);
	void EnableTypeCasting(bool bEnabled = true) { bEnableTypeCasting = bEnabled; }

private:
	UE_API bool BreakAllLinksRecursive(URigVMPin* Pin, bool bAsInput, bool bTowardsParent, bool bSetupUndoRedo);
	UE_API bool SetPinExpansion(URigVMPin* InPin, bool bIsExpanded, bool bSetupUndoRedo = true);
	UE_API void ExpandPinRecursively(URigVMPin* InPin, bool bSetupUndoRedo);
	UE_API bool SetPinIsWatched(URigVMPin* InPin, bool bIsWatched, bool bSetupUndoRedo);
	UE_API bool SetPinDisplayName(URigVMPin* InPin, const FString& InDisplayName, bool bSetupUndoRedo);
	UE_API bool AddEmptyPinCategory(const URigVMNode* InNode, const FString& InPinCategory, bool bSetupUndoRedo);
	UE_API bool SetPinCategory(URigVMPin* InPin, const FString& InCategory, int32 InIndexInCategory, bool bSetupUndoRedo);
	UE_API bool RemovePinCategory(const URigVMNode* InNode, const FString& InPinCategory, bool bSetupUndoRedo);
	UE_API bool RenamePinCategory(const URigVMNode* InNode, const FString& InOldPinCategory, const FString& InNewPinCategory, bool bSetupUndoRedo);
	UE_API bool SetPinCategoryIndex(const URigVMNode* InNode, const FString& InPinCategory, int32 InNewIndex, bool bSetupUndoRedo);
	UE_API bool SetPinCategoryExpansion(const URigVMNode* InNode, const FString& InPinCategory, bool bIsExpanded, bool bSetupUndoRedo);
	UE_API bool SetPinIndexInCategory(URigVMPin* InPin, int32 InIndexInCategory, bool bSetupUndoRedo);
	UE_API bool SetNodeLayout(const URigVMNode* InNode, FRigVMNodeLayout InLayout, bool bSetupUndoRedo, bool bPrintPythonCommand);
	UE_API bool ClearNodeLayout(const URigVMNode* InNode, bool bSetupUndoRedo, bool bPrintPythonCommand);
	UE_API bool SetPinCategories(const FName& InNodeName, const TArray<FString>& InCategories, bool bSetupUndoRedo);
	UE_API bool SetPinCategories(const URigVMNode* InNode, const TArray<FString>& InCategories, bool bSetupUndoRedo);
	UE_API bool SetVariableName(URigVMVariableNode* InVariableNode, const FName& InVariableName, bool bSetupUndoRedo);
	static UE_API void ForEveryPinRecursively(URigVMPin* InPin, TFunction<void(URigVMPin*)> OnEachPinFunction);
	static UE_API void ForEveryPinRecursively(URigVMNode* InNode, TFunction<void(URigVMPin*)> OnEachPinFunction);
	UE_API URigVMCollapseNode* CollapseNodes(const TArray<URigVMNode*>& InNodes, const FString& InCollapseNodeName, bool bSetupUndoRedo, bool bIsAggregate);
	UE_API TArray<URigVMNode*> ExpandLibraryNode(URigVMLibraryNode* InNode, bool bSetupUndoRedo);
	UE_API URigVMFunctionReferenceNode* PromoteCollapseNodeToFunctionReferenceNode(URigVMCollapseNode* InCollapseNode, bool bSetupUndoRedo, const FString& InExistingFunctionDefinitionPath);
	UE_API URigVMCollapseNode* PromoteFunctionReferenceNodeToCollapseNode(URigVMFunctionReferenceNode* InFunctionRefNode, bool bSetupUndoRedo, bool bRemoveFunctionDefinition);
	UE_API void SetReferencedFunction(URigVMFunctionReferenceNode* InFunctionRefNode, URigVMLibraryNode* InNewReferencedNode, bool bSetupUndoRedo);

	UE_API void RefreshFunctionPins(URigVMNode* InNode, bool bSetupUndoRedo = false);

	UE_API void ReportRemovedLink(const FString& InSourcePinPath, const FString& InTargetPinPath, const FString& Reason = FString());
public:
	struct FPinState
	{
		ERigVMPinDirection Direction;
		FString CPPType;
		UObject* CPPTypeObject;
		FString DefaultValue;
		ERigVMPinDefaultValueType DefaultValueType;
		bool bIsExpanded;
		TArray<URigVMInjectionInfo*> InjectionInfos;
		TArray<URigVMInjectionInfo::FWeakInfo> WeakInjectionInfos;
	};

	UE_API TMap<FString, FString> GetRedirectedPinPaths(URigVMNode* InNode) const;
	UE_API FPinState GetPinState(const URigVMPin* InPin, bool bStoreWeakInjectionInfos = false) const;
	UE_API TMap<FString, FPinState> GetPinStates(URigVMNode* InNode, bool bStoreWeakInjectionInfos = false) const;
	UE_API void ApplyPinState(URigVMPin* InPin, const FPinState& InPinState, bool bSetupUndoRedo = false);
	UE_API void ApplyPinStates(URigVMNode* InNode, const TMap<FString, FPinState>& InPinStates, const TMap<FString, FString>& InRedirectedPinPaths = TMap<FString, FString>(), bool bSetupUndoRedo = false);

protected:
	static UE_API void PostProcessDefaultValue(const URigVMPin* Pin, FString& OutDefaultValue);

private:

	static UE_API FLinearColor GetColorFromMetadata(const FString& InMetadata);
	static UE_API void CreateDefaultValueForStructIfRequired(UScriptStruct* InStruct, FString& InOutDefaultValue);
	static UE_API void OverrideDefaultValueMember(const FString& InMemberName, const FString& InMemberValue, FString& InOutDefaultValue);

	UE_API void ResolveTemplateNodeMetaData(URigVMTemplateNode* InNode, bool bSetupUndoRedo);
	UE_API void ResolveTemplateNodeMetaData(URigVMTemplateNode* InNode, const TArray<int32>& InFilteredPermutationIndices, bool bSetupUndoRedo);

	// Changes Pin types if filtered types of a pin are unique
	UE_API bool UpdateTemplateNodePinTypes(URigVMTemplateNode* InNode, bool bSetupUndoRedo, bool bInitializeDefaultValue = true, TMap<URigVMPin*, TArray<TRigVMTypeIndex>> ProposedTypes = TMap<URigVMPin*, TArray<TRigVMTypeIndex>>());

	UE_API bool ChangePinType(const FString& InPinPath, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bSetupOrphanPins = true, bool bBreakLinks = true, bool bRemoveSubPins = true, bool bInitializeDefaultValue = true);
	UE_API bool ChangePinType(URigVMPin* InPin, const FString& InCPPType, const FName& InCPPTypeObjectPath, bool bSetupUndoRedo, bool bSetupOrphanPins = true, bool bBreakLinks = true, bool bRemoveSubPins = true, bool bInitializeDefaultValue = true);
	UE_API bool ChangePinType(URigVMPin* InPin, const FString& InCPPType,UObject* InCPPTypeObject, bool bSetupUndoRedo, bool bSetupOrphanPins = true, bool bBreakLinks = true, bool bRemoveSubPins = true, bool bInitializeDefaultValue = true);
	UE_API bool ChangePinType(URigVMPin* InPin, TRigVMTypeIndex InTypeIndex, bool bSetupUndoRedo, bool bSetupOrphanPins = true, bool bBreakLinks = true, bool bRemoveSubPins = true, bool bInitializeDefaultValue = true);

	UE_API bool MarkPackageDirty(const UObject* InObjectToDirty) const;
	UE_API bool RenameObject(UObject* InObjectToRename, const TCHAR* InNewName, UObject* InNewOuter = nullptr, ERenameFlags InFlags = REN_None) const;
	UE_API void DestroyObject(UObject* InObjectToDestroy) const ;
	static UE_API URigVMPin* MakeExecutePin(URigVMNode* InNode, const FName& InName);
	static UE_API bool MakeExecutePin(URigVMPin* InOutPin);
	UE_API bool AddGraphNode(URigVMNode* InNode, bool bNotify);
	UE_API void AddNodePin(URigVMNode* InNode, URigVMPin* InPin);
	static UE_API void AddSubPin(URigVMPin* InParentPin, URigVMPin* InPin);
	static UE_API bool EnsurePinValidity(URigVMPin* InPin, bool bRecursive);
	static UE_API void ValidatePin(URigVMPin* InPin);

	// recreate the CPP type strings for variables that reference a type object
	// they can get out of sync when the variable references a user defined struct
	UE_API bool EnsureLocalVariableValidity();
	
	UE_API TArray<FRigVMExternalVariable> GetAllVariables(const bool bIncludeInputArguments = false) const;

	UE_API void RefreshFunctionReferences(const URigVMLibraryNode* InFunctionDefinition, bool bSetupUndoRedo, bool bLoadIfNecessary);
	UE_API void PropagateNotificationToFunctionReferences(const URigVMLibraryNode* InFunctionDefinition, ERigVMGraphNotifType InNotifType, UObject* InSubject, bool bLoadIfNecessary);
	
	// Mirror of the cross-graph pin layout propagation in ChangeExposedPinType_Internal —
	// not on the action stack on its own, so FRigVMChangePinTypeAction::Undo/Redo replays it.
	UE_API void PropagateExposedPinTypeChangeToReferencesAndInterface(const URigVMLibraryNode* InFunctionDefinition, bool bSetupOrphanPins);

public:
	
	UE_API FRigVMExternalVariable GetVariableByProperty(const FProperty* InExternalVariableProperty) const;
	UE_API FRigVMExternalVariable GetVariableByName(const FName& InExternalVariableName, const bool bIncludeInputArguments = false) const;
	UE_API FRigVMExternalVariable GetVariableByGuid(const FGuid& InExternalVariableGuid, const bool bIncludeInputArguments = false) const;

	struct FLinkedPath
	{
		FLinkedPath()
			: bSourceNodeIsInjected(false)
			, bTargetNodeIsInjected(false)
		{}

		FLinkedPath(URigVMLink* InLink);

		URigVMGraph* GetGraph(URigVMGraph* InGraph = nullptr) const;
		FString GetPinPathRepresentation() const;
		URigVMPin* GetSourcePin(URigVMGraph* InGraph = nullptr) const;
		URigVMPin* GetTargetPin(URigVMGraph* InGraph = nullptr) const;

		friend uint32 GetTypeHash(const FLinkedPath& InPath);

		bool operator ==(const FLinkedPath& InOther) const { return GetTypeHash(*this) == GetTypeHash(InOther); }
		bool operator !=(const FLinkedPath& InOther) const { return !(*this == InOther); }
		
		TSoftObjectPtr<URigVMGraph> GraphPtr;
		FString SourcePinPath;
		FString TargetPinPath;
		FString OriginalPinPathRepresentation;
		bool bSourceNodeIsInjected;
		bool bTargetNodeIsInjected;
	};

	struct FRestoreLinkedPathSettings
	{
		FRestoreLinkedPathSettings()
			: bFollowCoreRedirectors(false)
			, bRelayToOrphanPins(false)
			, bIsImportingFromText(false)
			, UserDirection(ERigVMPinDirection::Invalid)
		{}

		bool bFollowCoreRedirectors;
		bool bRelayToOrphanPins;
		bool bIsImportingFromText;
		ERigVMPinDirection UserDirection;
		TMap<FString, FString> NodeNameMap;
		TMap<FString,FRigVMController_PinPathRemapDelegate> RemapDelegates;
		FRigVMController_CheckPinCompatibilityDelegate CompatibilityDelegate;
	};

	UE_API TArray<FLinkedPath> GetLinkedPaths() const;
	static UE_API TArray<FLinkedPath> GetLinkedPaths(const TArray<URigVMLink*>& InLinks);
	static UE_API TArray<FLinkedPath> GetLinkedPaths(URigVMNode* InNode, bool bIncludeInjectionNodes = false);
	static UE_API TArray<FLinkedPath> GetLinkedPaths(const TArray<URigVMNode*>& InNodes, bool bIncludeInjectionNodes = false);
	static UE_API TArray<FLinkedPath> GetLinkedPaths(const URigVMPin* InPin, bool bSourceLinksRecursive = false, bool bTargetLinksRecursive = false);
	UE_API bool BreakLinkedPaths(const TArray<FLinkedPath>& InLinkedPaths, bool bSetupUndoRedo = false, bool bRelyOnBreakLink = true);
	UE_API bool RestoreLinkedPaths(
		const TArray<FLinkedPath>& InLinkedPaths,
		const FRestoreLinkedPathSettings& InSettings = FRestoreLinkedPathSettings(),
		bool bSetupUndoRedo = false);
	UE_API TArray<FLinkedPath> RemapLinkedPaths(
		const TArray<FLinkedPath>& InLinkedPaths,
		const FRestoreLinkedPathSettings& InSettings = FRestoreLinkedPathSettings(),
		bool bSetupUndoRedo = false);
	UE_API bool FastBreakLinkedPaths(const TArray<FLinkedPath>& InLinkedPaths, bool bSetupUndoRedo = false);
	UE_API URigVMLink* FindLinkFromPinPathRepresentation(const FString& InPinPathRepresentation, bool bLookForDetachedLink) const;
	UE_API void ProcessDetachedLinks(const FRestoreLinkedPathSettings& InSettings = FRestoreLinkedPathSettings());

#if WITH_EDITOR
	// Registers this template node's use for later determining the commonly used types
	UE_API void RegisterUseOfTemplate(const URigVMTemplateNode* InNode);

	// Inquire on the commonly used types for a template node. This can be used to resolve a node without user input (as a default)
	UE_API FRigVMTemplate::FTypeMap GetCommonlyUsedTypesForTemplate(const URigVMTemplateNode* InNode) const;
#endif

	UFUNCTION(BlueprintPure, Category = RigVMController)
	UE_API URigVMActionStack* GetActionStack() const;
	UE_API URigVMActionStack* GetActionStack_NoLock() const;

	UFUNCTION(BlueprintCallable, Category = RigVMController)
	UE_API void SetActionStack(URigVMActionStack* InActionStack);
	UE_API void SetActionStack_NoLock(URigVMActionStack* InActionStack);

	UE_API URigVMNode* ConvertRerouteNodeToDispatch(URigVMRerouteNode* InRerouteNode, const FName& InTemplateNotation, bool bSetupUndoRedo, bool bPrintPythonCommand);

protected:

	UE_API IRigVMClientHost* GetClientHost_Internal(const URigVMGraph* InGraph) const;
	UE_API URigVMPin* CreatePinFromPinInfo(const FRigVMRegistry& InRegistry, const FRigVMPinInfoArray& InPreviousPinInfos, const FRigVMPinInfo& InPinInfo, const FString& InPinPath, UObject* InOuter);
	void UpdateGraphSectionsIfRequired();

	// backwards compatibility code
	UE_API FRigVMClientPatchResult PatchRerouteNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchUnitNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchDispatchNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchBranchNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchIfSelectNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchArrayNodesOnLoad();
	UE_API FRigVMClientPatchResult PatchReduceArrayFloatDoubleConvertsionsOnLoad();
	UE_API FRigVMClientPatchResult PatchInvalidLinksOnWildcards();
	UE_API FRigVMClientPatchResult PatchFunctionsWithInvalidReturnPaths();
	UE_API FRigVMClientPatchResult PatchExecutePins();
	UE_API FRigVMClientPatchResult PatchLazyPins();
	UE_API FRigVMClientPatchResult PatchConstantPins();
	UE_API FRigVMClientPatchResult PatchPinDefaultValues();
	UE_API FRigVMClientPatchResult PatchUserDefinedStructPinNames();
	UE_API FRigVMClientPatchResult PatchLocalVariableTypes();
	UE_API FRigVMClientPatchResult PatchPinGuidInCollapseNodes();
	UE_API FRigVMClientPatchResult PatchExternalVariablesToArguments(bool bInSetupUndoRedo=false);

	UE_API ERigVMPinDefaultValueType GetDefaultValueType(const URigVMPin* InPin, const FString& InDefaultValue) const;

	template<typename T>
	static void SortGraphElementsByGraphDepth(TArray<T*>& InOutElements, bool bReverse = false)
	{
		if(InOutElements.IsEmpty())
		{
			return;
		}

		int32 MinDepth = InOutElements[0]->GetGraphDepth();
		int32 MaxDepth = MinDepth;
		int32 Increment = 1;
		TMap<int32, TArray<T*>> ElementsPerDepth;
		for(T* Element : InOutElements)
		{
			const int32 Depth = Element->GetGraphDepth(); 
			ElementsPerDepth.FindOrAdd(Depth).Add(Element);
			MinDepth = FMath::Min<int32>(MinDepth, Depth);
			MaxDepth = FMath::Max<int32>(MaxDepth, Depth);
		}

		if(bReverse)
		{
			Swap(MinDepth, MaxDepth);
			Increment = -1;
		}
		
		InOutElements.Reset();
		for(int32 Depth = MinDepth; /* no exit condition */; Depth += Increment)
		{
			if(const TArray<T*>* Elements = ElementsPerDepth.Find(Depth))
			{
				InOutElements.Append(*Elements);
			}

			if(Depth == MaxDepth)
			{
				break;
			}
		}
	}

	template<typename T>
	static void SortGraphElementsByImportOrder(
		TArray<TObjectPtr<T>> InOutElements,
		const TArray<T*>& InElementsInImportOrder,
		const TArray<T*>& InPreviousElementPriorToImport
	) {
		Algo::SortBy(InOutElements, [InElementsInImportOrder, InPreviousElementPriorToImport](T* Element) -> int32
		{
			const int32 ImportOrderIndex = InElementsInImportOrder.Find(Element);
			if(ImportOrderIndex != INDEX_NONE)
			{
				return ImportOrderIndex + InPreviousElementPriorToImport.Num();
			}
			return InPreviousElementPriorToImport.Find(Element);
		});
	}
	
private: 
	UPROPERTY(transient)
	TArray<TObjectPtr<URigVMGraph>> Graphs;

	UPROPERTY(transient, DuplicateTransient)
	TSubclassOf<URigVMSchema> SchemaClass;

	mutable TWeakObjectPtr<URigVMActionStack> WeakActionStack;
	mutable FDelegateHandle ActionStackHandle;

	bool bSuspendNotifications;
	bool bSuspendMarkingPackageDirty;
	bool bSuspendRefreshingFunctionReferences;
	bool bReportWarningsAndErrors;
	bool bIgnoreRerouteCompactnessChanges;
	ERigVMPinDirection UserLinkDirection;
	bool bEnableTypeCasting;
	bool bAllowPrivateFunctions;
	bool bSuspendASTLinkChecks;
	TOptional<ERigVMPinDefaultValueType> OptionalDefaultValueType;

	// temporary maps used for pin redirection
	// only valid between Detach & ReattachLinksToPinObjects
	TMap<FString, FString> InputPinRedirectors;
	TMap<FString, FString> OutputPinRedirectors;

	struct FRigVMStructPinRedirectorKey
	{
		FRigVMStructPinRedirectorKey()
		{
		}

		FRigVMStructPinRedirectorKey(UScriptStruct* InScriptStruct, const FString& InPinPathInNode)
		: Struct(InScriptStruct)
		, PinPathInNode(InPinPathInNode)
		{
		}

		friend uint32 GetTypeHash(const FRigVMStructPinRedirectorKey& Cache)
		{
			return HashCombine(GetTypeHash(Cache.Struct), GetTypeHash(Cache.PinPathInNode));
		}

		bool operator ==(const FRigVMStructPinRedirectorKey& Other) const
		{
			return Struct == Other.Struct && PinPathInNode == Other.PinPathInNode;
		}

		bool operator !=(const FRigVMStructPinRedirectorKey& Other) const
		{
			return Struct != Other.Struct || PinPathInNode != Other.PinPathInNode;
		}

		UScriptStruct* Struct;
		FString PinPathInNode;
	};

	static UE_API TMap<FRigVMStructPinRedirectorKey, FString> PinPathCoreRedirectors;
	FTransactionallySafeCriticalSection PinPathCoreRedirectorsLock;
	mutable FTransactionallySafeCriticalSection ActionStackLock;

	FRigVMUnitNodeCreatedContext UnitNodeCreatedContext;

	bool bIsTransacting; // Performing undo/redo transaction
	bool bIsRunningUnitTest;
	bool bIsFullyResolvingTemplateNode;

public:
	
	bool bSuspendTemplateComputation;

private:

#if WITH_EDITOR
	bool bRegisterTemplateNodeUsage;
#endif

	bool bEnableSchemaRemoveNodeCheck;

	int32 UpdateGraphSectionsBracket;

	friend class URigVMGraph;
	friend class URigVMPin;
	friend class URigVMActionStack;
	friend struct FRigVMBaseAction;
	friend class URigVMCompiler;
	friend class IRigVMEditorAssetInterface;
	friend struct FRigVMControllerObjectFactory;
	friend struct FRigVMSetPinDefaultValueAction;
	friend struct FRigVMAddRerouteNodeAction;
	friend struct FRigVMChangePinTypeAction;
	friend struct FRigVMInjectNodeIntoPinAction;
	friend struct FRigVMEjectNodeFromPinAction;
	friend struct FRigVMChangeNodePinCategoriesAction;
	friend struct FRigVMSetPinCategoryAction;
	friend class FRigVMParserAST;
	friend class FRigVMControllerCompileBracketScope;
	friend struct FRigVMPinInfoArray;
	friend class FRigVMControllerNotifGuard;
	friend class FRigVMDefaultValueTypeGuard;
	friend class FRigVMControllerASTLinkCheckGuard;
	friend struct FRigVMClient;
	friend struct FRigVMActionWrapper;
	friend class URigVMSchema;
	friend class URigVMEdGraphFunctionRefNodeSpawner;
	friend class FRigVMControllerGraphSectionsScope;
	friend class FRigVMMinimalEnvironment;
};

class FRigVMControllerNotifGuard
{
public:

	FRigVMControllerNotifGuard(URigVMController* InController, bool bInSuspendNotifications = true)
		: Controller(InController)
	{
		bPreviousSuspendNotifications = Controller->bSuspendNotifications;
		Controller->bSuspendNotifications = bInSuspendNotifications;
	}

	~FRigVMControllerNotifGuard()
	{
		Controller->bSuspendNotifications = bPreviousSuspendNotifications;
	}

private:

	URigVMController* Controller;
	bool bPreviousSuspendNotifications;
};

class FRigVMDefaultValueTypeGuard
{
public:

	FRigVMDefaultValueTypeGuard(URigVMController* InController, ERigVMPinDefaultValueType InDefaultValueType = ERigVMPinDefaultValueType::Override, bool bForce = false)
		: Controller(InController)
	{
		bPreviousDefaultValueType = Controller->OptionalDefaultValueType;
		if(!bPreviousDefaultValueType.IsSet() || bForce)
		{
			Controller->OptionalDefaultValueType = InDefaultValueType;
		}
	}

	~FRigVMDefaultValueTypeGuard()
	{
		Controller->OptionalDefaultValueType = bPreviousDefaultValueType;
	}

private:

	URigVMController* Controller;
	TOptional<ERigVMPinDefaultValueType> bPreviousDefaultValueType;
};

class FRigVMControllerASTLinkCheckGuard
{
public:

	explicit FRigVMControllerASTLinkCheckGuard(URigVMController* InController, bool bInSuspendASTLinkChecks = true)
		: Controller(InController)
	{
		bPreviousSuspendASTLinkChecks = Controller->bSuspendASTLinkChecks;
		Controller->bSuspendASTLinkChecks = bInSuspendASTLinkChecks;
	}

	~FRigVMControllerASTLinkCheckGuard()
	{
		Controller->bSuspendASTLinkChecks = bPreviousSuspendASTLinkChecks;
	}

private:

	URigVMController* Controller;
	bool bPreviousSuspendASTLinkChecks;
};


USTRUCT()
struct FRigVMController_CommonTypePerTemplate
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RigVMController")
	TMap<FString,int32> Counts;
};

/**
 * Default settings for the RigVM Controller
 */
UCLASS(MinimalAPI, config = EditorSettings)
class URigVMControllerSettings : public UObject
{
public:
	UE_API URigVMControllerSettings(const FObjectInitializer& Initializer);

	GENERATED_BODY()

	/**
	 * When adding a link to an execute pin on a template node,
	 * this functionality automatically resolves the template node to the
	 * most commonly used type.
	 */
	UPROPERTY(EditAnywhere, Category = "RigVMController")
	bool bAutoResolveTemplateNodesWhenLinkingExecute;

	/** The commonly used types for a template node */
	UPROPERTY()
	TMap<FName,FRigVMController_CommonTypePerTemplate> TemplateDefaultTypes;
};

class FRigVMControllerGraphSectionsScope
{
public:
	FRigVMControllerGraphSectionsScope(URigVMController* InController)
	: Controller(InController)
	{
		if (Controller)
		{
			Controller->UpdateGraphSectionsBracket++;
		}
	}

	~FRigVMControllerGraphSectionsScope()
	{
		if (Controller)
		{
			Controller->UpdateGraphSectionsBracket--;
		}
	}

private:

	URigVMController* Controller;
};

#undef UE_API
