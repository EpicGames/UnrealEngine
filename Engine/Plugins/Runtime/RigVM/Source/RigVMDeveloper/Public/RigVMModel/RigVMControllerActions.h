// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/PropertyPortFlags.h"
#include "RigVMController.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif
#include "RigVMClient.h"
#include "UObject/StructOnScope.h"
#include "RigVMControllerActions.generated.h"

// If this is set to one the action stack will log any created action
// and invoked undo / redo on those actions and print them as an indented log.
#ifndef RIGVM_ACTIONSTACK_VERBOSE_LOG
#define RIGVM_ACTIONSTACK_VERBOSE_LOG 0
#endif 

/**
 * ================================================================================
 * The RigVMController doesn't rely on the transaction system for performing,
 * tracking, undoing and redoing changes. Instead it uses an action stack which
 * stores small serialized structs for each occured action. The reason for this 
 * is the subscription model and Python support: We need the Graph to broadcast
 * events to all subscribers - independently from where the action is coming from.
 * This includes UI views, scripting or undo / redo.
 * Each action supports the concept of 'merging', so multiple color change actions
 * for example can be merged into a single action. This avoids the need for tracking
 * action scope - and makes the integration with UI code simple.
 * The Controller's ActionStack integrates into the editor's transaction stack
 * using the ActionIndex property: Transactions on that property cause the 
 * actionstack to consecutively undo or redo actions until the expected stack
 * size is reached.
 * ================================================================================
 */

struct FRigVMBaseAction;

/**
 * The action key is used for serializing and storing an action in the stack,
 * or within another action.
 */
USTRUCT()
struct FRigVMActionKey
{
	GENERATED_BODY()

	UPROPERTY()
	FString ScriptStructPath;

	UPROPERTY()
	FString ExportedText;

	UScriptStruct* GetScriptStruct() const;
	TSharedPtr<FStructOnScope> GetAction() const;

	template<class ActionType>
	void Set(const ActionType& InAction)
	{
		UScriptStruct* ScriptStruct = ActionType::StaticStruct();
		FRigVMActionKey Key;
		ScriptStructPath = ScriptStruct->GetPathName();
		
		TArray<uint8, TAlignedHeapAllocator<16>> DefaultStructData;
		DefaultStructData.AddZeroed(ScriptStruct->GetStructureSize());
		ScriptStruct->InitializeDefaultValue(DefaultStructData.GetData());
		
		ScriptStruct->ExportText(ExportedText, &InAction, DefaultStructData.GetData(), nullptr, PPF_None, nullptr);
	}
};

/**
 * The action wrapper is used to extract an action from a serialized key.
 */
struct FRigVMActionWrapper
{
public:
	RIGVMDEVELOPER_API FRigVMActionWrapper(const FRigVMActionKey& Key);
	RIGVMDEVELOPER_API ~FRigVMActionWrapper();

	RIGVMDEVELOPER_API const UScriptStruct* GetScriptStruct() const;
	RIGVMDEVELOPER_API FRigVMBaseAction* GetAction(URigVMController* InLastController = nullptr) const;
	RIGVMDEVELOPER_API FString ExportText() const;

private:
	FRigVMActionWrapper(const FRigVMActionWrapper& Other) = delete;
	FRigVMActionWrapper& operator = (const FRigVMActionWrapper& Other) = delete;

	TSharedPtr<FStructOnScope> StructOnScope;
};

/**
 * A tuple 
 * access to sub actions, merge functionality as well as undo and redo
 * base implementations.
 */
USTRUCT()
struct FRigVMActionNodeContent
{
	GENERATED_BODY()

	UPROPERTY()
	FString Old;

	UPROPERTY()
	FString New;
};

/**
 * The base action is the base struct for all actions, and provides
 * access to sub actions, merge functionality as well as undo and redo
 * base implementations.
 */
USTRUCT()
struct FRigVMBaseAction
{
	GENERATED_BODY()

public:
	
	static const inline TCHAR* RedoPrefix = TEXT("Redo");
	static const inline TCHAR* UndoPrefix = TEXT("Undo");
	static const inline TCHAR* AddActionPrefix = TEXT("Add Action");
	static const inline TCHAR* BeginActionPrefix = TEXT("Begin Action");
	static const inline TCHAR* EndActionPrefix = TEXT("End Action");
	static const inline TCHAR* CancelActionPrefix = TEXT("Cancel Action");

	// Default constructor
	RIGVMDEVELOPER_API FRigVMBaseAction(URigVMController* InController);

	// Default destructor
	virtual ~FRigVMBaseAction() = default;

	// Returns the controller of this action
	RIGVMDEVELOPER_API URigVMController* GetController() const;

	// Access to the actions script struct. this needs to be overloaded
	virtual UScriptStruct* GetScriptStruct() const { return FRigVMBaseAction::StaticStruct(); }

	// Returns true if this action is empty has no effect
	virtual bool IsEmpty() const { return SubActions.IsEmpty(); }

	// Returns the title of the action - used for the Edit menu's undo / redo
	virtual FString GetTitle() const { return Title; }

	// Sets the title of the action - used for the Edit menu's undo / redo
	virtual void SetTitle(const FString& InTitle) { Title = InTitle; }

	// Trys to merge the action with another action and 
	// returns true if successfull.
	RIGVMDEVELOPER_API virtual bool Merge(const FRigVMBaseAction* Other);

	// Returns true if this action makes the other action obsolete.
	RIGVMDEVELOPER_API virtual bool MakesObsolete(const FRigVMBaseAction* Other) const;

	// Returns true if this action can undo / redo
	bool CanUndoRedo() const { return GetController() != nullptr; }

	// Un-does the action and returns true if successfull.
	RIGVMDEVELOPER_API virtual bool Undo();

	// Re-does the action and returns true if successfull.
	RIGVMDEVELOPER_API virtual bool Redo();

	// Adds a child / sub action to this one
	template<class ActionType>
	void AddAction(const ActionType& InAction)
	{
		check(InAction.CanUndoRedo());
		FRigVMActionKey Key;
		Key.Set<ActionType>(InAction);
		SubActions.Add(Key);
	}

	RIGVMDEVELOPER_API bool StoreNode(const URigVMNode* InNode, bool bIsPriorChange = true);
	RIGVMDEVELOPER_API bool RestoreNode(const FName& InNodeName, bool bIsUndoing);

#if RIGVM_ACTIONSTACK_VERBOSE_LOG
	
	// Logs the action to the console / output log
	RIGVMDEVELOPER_API void LogAction(const FString& InPrefix) const;

#endif

	// Empty constructor
	FRigVMBaseAction()
		: WeakController(nullptr)
		, Title(TEXT("Action"))
	{
	}

protected:

	RIGVMDEVELOPER_API void EnsureControllerValidity() const;

	mutable TWeakObjectPtr<URigVMController> WeakController;

	UPROPERTY()
	FSoftObjectPath ControllerPath;

	UPROPERTY()
	FSoftObjectPath ClientHostPath;

	UPROPERTY()
	FString ControllerModelPath;

	UPROPERTY()
	FString Title;

	UPROPERTY()
	TArray<FRigVMActionKey> SubActions;

	UPROPERTY()
	TMap<FName, FRigVMActionNodeContent> ExportedNodes;

	friend struct FRigVMActionWrapper;
	friend class URigVMActionStack;
};

/**
 * The Action Stack can be used to track actions happening on a
 * Graph. Currently the only owner of the ActionStack is the Controller.
 * Actions can be added to the stack, or they can be understood as
 * scopes / brackets. For this you can use BeginAction / EndAction / CancelAction
 * to open / close a bracket. Open brackets automatically record additional
 * actions occuring during the bracket's lifetime.
 */
UCLASS(MinimalAPI)
class URigVMActionStack : public UObject
{
	GENERATED_BODY()

	virtual ~URigVMActionStack()
	{
		ensure(CurrentActions.IsEmpty());
	}

public:

	static RIGVMDEVELOPER_API URigVMActionStack* GetDisabledActionStack();

	// Begins an action and opens a bracket / scope.
	template<class ActionType>
	void BeginAction(ActionType& InAction)
	{
		if(IsDisabled())
		{
			return;
		}
#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		TGuardValue<int32> TabDepthGuard(LogActionDepth, CurrentActions.Num());
		LogAction<ActionType>(InAction, FRigVMBaseAction::BeginActionPrefix);
#endif

		if (CurrentActions.Num() > 0)
		{
			// catch erroreous duplicate calls to begin action
			ensure(CurrentActions.Last() != &InAction);
		}
		CurrentActions.Add((FRigVMBaseAction*)&InAction);
		
		ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketOpened, nullptr, nullptr);
	}

	// Ends an action and closes a bracket / scope.
	template<class ActionType>
	void EndAction(ActionType& InAction, bool bPerformMerge = false)
	{
		if(IsDisabled())
		{
			return;
		}
		ensure(CurrentActions.Num() > 0);
		ensure((FRigVMBaseAction*)&InAction == CurrentActions.Last());
		CurrentActions.Pop();

		{
#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
			TGuardValue<bool> LogGuard(bSuspendLogActions, true);
#endif
			AddAction(InAction, bPerformMerge);
		}

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		TGuardValue<int32> TabDepthGuard(LogActionDepth, CurrentActions.Num());
		LogAction<ActionType>(InAction, FRigVMBaseAction::EndActionPrefix);
#endif
		ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketClosed, nullptr, nullptr);
	}

	// Cancels an action, closes a bracket / scope and discards all 
	// actions to this point.
	template<class ActionType>
	void CancelAction(ActionType& InAction)
	{
		if(IsDisabled())
		{
			return;
		}
		ensure(CurrentActions.Num() > 0);
		ensure((FRigVMBaseAction*)&InAction == CurrentActions.Last());
		CurrentActions.Pop();

		InAction.Undo();

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		TGuardValue<int32> TabDepthGuard(LogActionDepth, CurrentActions.Num());
		LogAction<ActionType>(InAction, FRigVMBaseAction::CancelActionPrefix);
#endif
		ModifiedEvent.Broadcast(ERigVMGraphNotifType::InteractionBracketCanceled, nullptr, nullptr);
	}

	// Adds an action to the stack. Optionally this can perform
	// a potential merge of this action with the previous action to
	// compact the stack.
	template<class ActionType>
	void AddAction(const ActionType& InAction, bool bPerformMerge = false, bool bMarkDirty = true)
	{
		if(IsDisabled())
		{
			return;
		}
#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
		TGuardValue<int32> TabDepthGuard(LogActionDepth, CurrentActions.Num());
		LogAction<ActionType>(InAction, FRigVMBaseAction::AddActionPrefix);
#endif
		TArray<FRigVMActionKey>* ActionList = &UndoActions;
		if (CurrentActions.Num() > 0)
		{
			ActionList = &CurrentActions[CurrentActions.Num()-1]->SubActions;
		}

		bool bMergeIfPossible = false;
		bool bIgnoreAction = false;
		
		if (ActionList->Num() > 0 && InAction.SubActions.Num() == 0)
		{
			const FRigVMActionWrapper Wrapper((*ActionList)[ActionList->Num() - 1]);
			if (Wrapper.GetAction()->SubActions.Num() == 0)
			{
				FRigVMBaseAction* OtherAction = Wrapper.GetAction();

				if (ActionList->Last().ScriptStructPath == ActionType::StaticStruct()->GetPathName())
				{
					if(bPerformMerge)
					{
						bMergeIfPossible = OtherAction->Merge(&InAction);
						if(bMergeIfPossible)
						{
							(*ActionList)[ActionList->Num()-1].ExportedText = Wrapper.ExportText();
						}
					}
				}

				if(!bMergeIfPossible)
				{
					if(InAction.MakesObsolete(OtherAction))
					{
						// if this new action cancels out the last one we don't
						// need to perform either of the actions
						ActionList->Pop();
						bIgnoreAction = true;
					}
				}
			}
		}

		if (!bMergeIfPossible && !bIgnoreAction)
		{
			FRigVMActionKey Key;
			Key.Set<ActionType>(InAction);

			ActionList->Add(Key);

			if (CurrentActions.Num() == 0)
			{
				RedoActions.Reset();

#if WITH_EDITOR
				const FScopedTransaction Transaction(FText::FromString(InAction.GetTitle()));
				SetFlags(RF_Transactional);
				Modify(bMarkDirty);
#endif

				ActionIndex = ActionIndex + 1;
			}
		}
	}

	// Opens an undo bracket / scope to record actions into.
	// This is primary useful for Python.
	UFUNCTION()
	RIGVMDEVELOPER_API bool OpenUndoBracket(URigVMController* InController, const FString& InTitle);

	// Closes an undo bracket / scope.
	// This is primary useful for Python.
	UFUNCTION()
	RIGVMDEVELOPER_API bool CloseUndoBracket(URigVMController* InController);

	// Cancels an undo bracket / scope.
	// This is primary useful for Python.
	UFUNCTION()
	RIGVMDEVELOPER_API bool CancelUndoBracket(URigVMController* InController);

	// Pops the last action from the undo stack and perform undo on it.
	// Note: This should really only be used for unit tests,
	// use the GEditor's main Undo method instead.
	UFUNCTION()
	RIGVMDEVELOPER_API bool Undo(URigVMController* InController);

	// Pops the last action from the redo stack and perform redo on it.
	// Note: This should really only be used for unit tests,
	// use the GEditor's main Redo method instead.
	UFUNCTION()
	RIGVMDEVELOPER_API bool Redo(URigVMController* InController);

#if WITH_EDITOR
	RIGVMDEVELOPER_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent) override;
#endif

	FRigVMGraphModifiedEvent& OnModified() { return ModifiedEvent; }

private:

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	template<class ActionType>
	void LogAction(const ActionType& InAction, const FString& InPrefix)
	{
		LogAction(ActionType::StaticStruct(), InAction, InPrefix);
	}
	
	RIGVMDEVELOPER_API void LogAction(const UScriptStruct* InActionStruct, const FRigVMBaseAction& InAction, const FString& InPrefix);
#endif

	RIGVMDEVELOPER_API bool IsDisabled() const;
	
	UPROPERTY()
	int32 ActionIndex;

	UPROPERTY(NonTransactional)
	TArray<FRigVMActionKey> UndoActions;

	UPROPERTY(NonTransactional)
	TArray<FRigVMActionKey> RedoActions;

	TArray<FRigVMBaseAction*> CurrentActions;
	TArray<FRigVMBaseAction*> BracketActions;

	FRigVMGraphModifiedEvent ModifiedEvent;

#if RIGVM_ACTIONSTACK_VERBOSE_LOG		
	bool bSuspendLogActions = false;
	int32 LogActionDepth = 0;
#endif
	
	friend struct FRigVMBaseAction;
	friend class URigVMController;
};


/**
 * An action injecting a node into a pin
 */
USTRUCT()
struct FRigVMInjectNodeIntoPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMInjectNodeIntoPinAction();
	FRigVMInjectNodeIntoPinAction(URigVMController* InController, URigVMInjectionInfo* InInjectionInfo);
	virtual ~FRigVMInjectNodeIntoPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMInjectNodeIntoPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	bool bAsInput;

	UPROPERTY()
	FName InputPinName;

	UPROPERTY()
	FName OutputPinName;

	UPROPERTY()
	FString NodePath;
};

/**
 * An action ejecting a node from a pin
 */
USTRUCT()
struct FRigVMEjectNodeFromPinAction : public FRigVMInjectNodeIntoPinAction
{
	GENERATED_BODY()

public:

	FRigVMEjectNodeFromPinAction();
	FRigVMEjectNodeFromPinAction(URigVMController* InController, URigVMInjectionInfo* InInjectionInfo);
	virtual ~FRigVMEjectNodeFromPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMEjectNodeFromPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;
};

/**
 * An action removing one or more nodes from the graph.
 */
USTRUCT()
struct FRigVMRemoveNodesAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveNodesAction();
	FRigVMRemoveNodesAction(URigVMController* InController, TArray<URigVMNode*> InNodes);
	virtual ~FRigVMRemoveNodesAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRemoveNodesAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	TArray<FName> NodeNames;

	UPROPERTY()
	FString ExportedContent;	
};

/**
 * An action selecting or deselecting a node in the graph.
 */
USTRUCT()
struct FRigVMSetNodeSelectionAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeSelectionAction();
	FRigVMSetNodeSelectionAction(URigVMController* InController, URigVMGraph* InGraph, TArray<FName> InNewSelection);
	virtual ~FRigVMSetNodeSelectionAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeSelectionAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	TArray<FName> NewSelection;

	UPROPERTY()
	TArray<FName> OldSelection;
};

/**
 * An action setting a node's position in the graph.
 */
USTRUCT()
struct FRigVMSetNodePositionAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodePositionAction();
	FRigVMSetNodePositionAction(URigVMController* InController, URigVMNode* InNode, const FVector2D& InNewPosition);
	virtual ~FRigVMSetNodePositionAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodePositionAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FVector2D OldPosition;

	UPROPERTY()
	FVector2D NewPosition;
};

/**
 * An action setting a node's size in the graph.
 */
USTRUCT()
struct FRigVMSetNodeSizeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeSizeAction();
	FRigVMSetNodeSizeAction(URigVMController* InController, URigVMNode* InNode, const FVector2D& InNewSize);
	virtual ~FRigVMSetNodeSizeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeSizeAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FVector2D OldSize;

	UPROPERTY()
	FVector2D NewSize;
};

/**
 * An action setting a node's title in the graph.
 */
USTRUCT()
struct FRigVMSetNodeTitleAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeTitleAction();
	FRigVMSetNodeTitleAction(URigVMController* InController, URigVMNode* InNode, const FString& InNewTitle);
	virtual ~FRigVMSetNodeTitleAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeTitleAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FString OldTitle;

	UPROPERTY()
	FString NewTitle;
};

/**
 * An action setting a node's color in the graph.
 */
USTRUCT()
struct FRigVMSetNodeColorAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeColorAction();
	FRigVMSetNodeColorAction(URigVMController* InController, URigVMNode* InNode, const FLinearColor& InNewColor);
	virtual ~FRigVMSetNodeColorAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeColorAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FLinearColor OldColor;

	UPROPERTY()
	FLinearColor NewColor;
};

/**
 * An action setting a node's category in the graph.
 */
USTRUCT()
struct FRigVMSetNodeCategoryAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeCategoryAction();
	FRigVMSetNodeCategoryAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewCategory);
	virtual ~FRigVMSetNodeCategoryAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeCategoryAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FString OldCategory;

	UPROPERTY()
	FString NewCategory;
};


/**
 * An action setting a node's keywords in the graph.
 */
USTRUCT()
struct FRigVMSetNodeKeywordsAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeKeywordsAction();
	FRigVMSetNodeKeywordsAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewKeywords);
	virtual ~FRigVMSetNodeKeywordsAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeKeywordsAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FString OldKeywords;

	UPROPERTY()
	FString NewKeywords;
};

/**
* An action setting a node's description in the graph.
*/
USTRUCT()
struct FRigVMSetNodeDescriptionAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetNodeDescriptionAction();
	FRigVMSetNodeDescriptionAction(URigVMController* InController, URigVMCollapseNode* InNode, const FString& InNewDescription);
	virtual ~FRigVMSetNodeDescriptionAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetNodeDescriptionAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FString OldDescription;

	UPROPERTY()
	FString NewDescription;
};

/**
 * An action setting a comment node's text in the graph.
 */
USTRUCT()
struct FRigVMSetCommentTextAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetCommentTextAction();
	FRigVMSetCommentTextAction(URigVMController* InController, URigVMCommentNode* InNode, const FString& InNewText, const int32& InNewFontSize, const bool& bInNewBubbleVisible, const bool& bInNewColorBubble);
	virtual ~FRigVMSetCommentTextAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetCommentTextAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodePath;

	UPROPERTY()
	FString OldText;

	UPROPERTY()
	FString NewText;

	UPROPERTY()
	int32 OldFontSize;

	UPROPERTY()
	int32 NewFontSize;

	UPROPERTY()
	bool bOldBubbleVisible;

	UPROPERTY()
	bool bNewBubbleVisible;

	UPROPERTY()
	bool bOldColorBubble;

	UPROPERTY()
	bool bNewColorBubble;
};

/**
 * An action renaming a variable in the graph.
 */
USTRUCT(meta=(Deprecated = "5.1"))
struct FRigVMRenameVariableAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRenameVariableAction();
	FRigVMRenameVariableAction(URigVMController* InController, const FGuid& InVariableGuid, const FName& InOldVariableName, const FName& InNewVariableName);
	virtual ~FRigVMRenameVariableAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRenameVariableAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;
	
	UPROPERTY()
	FGuid VariableGuid;

	UPROPERTY()
	FString OldVariableName;

	UPROPERTY()
	FString NewVariableName;
};

/**
 * An action setting a pin's expansion state in the graph.
 */
USTRUCT()
struct FRigVMSetPinExpansionAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinExpansionAction();
	FRigVMSetPinExpansionAction(URigVMController* InController, URigVMPin* InPin, bool bNewIsExpanded);
	virtual ~FRigVMSetPinExpansionAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinExpansionAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	bool OldIsExpanded;

	UPROPERTY()
	bool NewIsExpanded;
};

/**
 * An action setting a pin's watch state in the graph.
 */
USTRUCT()
struct FRigVMSetPinWatchAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinWatchAction();
	FRigVMSetPinWatchAction(URigVMController* InController, URigVMPin* InPin, bool bNewIsWatched);
	virtual ~FRigVMSetPinWatchAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinWatchAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	bool OldIsWatched;

	UPROPERTY()
	bool NewIsWatched;
};

/**
 * An action setting a pin's display name in the graph.
 */
USTRUCT()
struct FRigVMSetPinDisplayNameAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinDisplayNameAction();
	FRigVMSetPinDisplayNameAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewDisplayName);
	virtual ~FRigVMSetPinDisplayNameAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinDisplayNameAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	FString OldDisplayName;

	UPROPERTY()
	FString NewDisplayName;
};

/**
 * An action setting a pin's category in the graph.
 */
USTRUCT()
struct FRigVMSetPinCategoryAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinCategoryAction();
	FRigVMSetPinCategoryAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewCategory);
	virtual ~FRigVMSetPinCategoryAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinCategoryAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	FString OldCategory;

	UPROPERTY()
	int32 OldPinIndex;

	UPROPERTY()
	FString NewCategory;
};

/**
 * An action setting a pin's category in the graph.
 */
USTRUCT()
struct FRigVMChangeNodePinCategoriesAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMChangeNodePinCategoriesAction();
	FRigVMChangeNodePinCategoriesAction(URigVMController* InController, const URigVMNode* InNode);
	virtual ~FRigVMChangeNodePinCategoriesAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMChangeNodePinCategoriesAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;
	
	void UpdateAfterModification(const URigVMNode* InNode);

	UPROPERTY()
	FString NodeName;

	UPROPERTY()
	TArray<FString> OldCategories;

	UPROPERTY()
	TArray<FString> NewCategories;
};

/**
 * An action to change the expansion on a pin's category in the graph.
 */
USTRUCT()
struct FRigVMSetPinCategoryExpansionAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinCategoryExpansionAction();
	FRigVMSetPinCategoryExpansionAction(URigVMController* InController, const URigVMNode* InNode, const FString& InPinCategory);
	virtual ~FRigVMSetPinCategoryExpansionAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinCategoryExpansionAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString NodeName;

	UPROPERTY()
	FString PinCategory;

	UPROPERTY()
	bool bOldExpansionState;
};

/**
 * An action setting a pin's index within a category.
 */
USTRUCT()
struct FRigVMSetPinIndexInCategoryAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinIndexInCategoryAction();
	FRigVMSetPinIndexInCategoryAction(URigVMController* InController, URigVMPin* InPin, int32 InNewIndexInCategory);
	virtual ~FRigVMSetPinIndexInCategoryAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinIndexInCategoryAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	int32 OldIndexInCategory;

	UPROPERTY()
	int32 NewIndexInCategory;
};

/**
 * An action setting a pin's default value in the graph.
 */
USTRUCT()
struct FRigVMSetPinDefaultValueAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinDefaultValueAction();
	FRigVMSetPinDefaultValueAction(URigVMController* InController, URigVMPin* InPin, const FString& InNewDefaultValue);
	virtual ~FRigVMSetPinDefaultValueAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinDefaultValueAction::StaticStruct(); }
	virtual bool Merge(const FRigVMBaseAction* Other);
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	FString OldDefaultValue;

	UPROPERTY()
	FString NewDefaultValue;

	UPROPERTY()
	ERigVMPinDefaultValueType OldDefaultValueType;

	UPROPERTY()
	ERigVMPinDefaultValueType NewDefaultValueType;
};

/**
 * An action inserting a new array pin in the graph.
 */
USTRUCT()
struct FRigVMInsertArrayPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMInsertArrayPinAction();
	FRigVMInsertArrayPinAction(URigVMController* InController, URigVMPin* InArrayPin, int32 InIndex, const FString& InNewDefaultValue, const ERigVMPinDefaultValueType& InNewDefaultValueType);
	virtual ~FRigVMInsertArrayPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMInsertArrayPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString ArrayPinPath;

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	FString NewDefaultValue;

	UPROPERTY()
	ERigVMPinDefaultValueType NewDefaultValueType;
};

/**
 * An action removing an array pin from the graph.
 */
USTRUCT()
struct FRigVMRemoveArrayPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveArrayPinAction();
	FRigVMRemoveArrayPinAction(URigVMController* InController, URigVMPin* InArrayElementPin);
	virtual ~FRigVMRemoveArrayPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRemoveArrayPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString ArrayPinPath;

	UPROPERTY()
	int32 Index;

	UPROPERTY()
	FString DefaultValue;

	UPROPERTY()
	ERigVMPinDefaultValueType DefaultValueType;
};

/**
 * An action adding a new link to the graph.
 */
USTRUCT()
struct FRigVMAddLinkAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMAddLinkAction();
	FRigVMAddLinkAction(URigVMController* InController, URigVMPin* InOutputPin, URigVMPin* InInputPin);
	virtual ~FRigVMAddLinkAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMAddLinkAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString OutputPinPath;

	UPROPERTY()
	FString InputPinPath;
};

/**
 * An action removing a link from the graph.
 */
USTRUCT()
struct FRigVMBreakLinkAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMBreakLinkAction();
	FRigVMBreakLinkAction(URigVMController* InController, URigVMPin* InOutputPin, URigVMPin* InInputPin);
	virtual ~FRigVMBreakLinkAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMBreakLinkAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FSoftObjectPath GraphPath;

	UPROPERTY()
	FString OutputPinPath;

	UPROPERTY()
	FString InputPinPath;
};

/**
 * An action changing a pin type
 */
USTRUCT()
struct FRigVMChangePinTypeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMChangePinTypeAction();
	FRigVMChangePinTypeAction(URigVMController* InController, URigVMPin* InPin, int32 InTypeIndex, bool InSetupOrphanPins, bool InBreakLinks, bool InRemoveSubPins);
	virtual ~FRigVMChangePinTypeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMChangePinTypeAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	int32 OldTypeIndex;

	UPROPERTY()
	int32 NewTypeIndex;

	UPROPERTY()
	bool bSetupOrphanPins;

	UPROPERTY()
	bool bBreakLinks;

	UPROPERTY()
	bool bRemoveSubPins;
};

/**
 * An action to collapse a selection of nodes
 */
USTRUCT()
struct FRigVMCollapseNodesAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMCollapseNodesAction();
	FRigVMCollapseNodesAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes, const FString& InNodePath, const bool bIsAggregate);
	virtual ~FRigVMCollapseNodesAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMCollapseNodesAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString LibraryNodePath;

	UPROPERTY()
	FString CollapsedNodesContent;

	UPROPERTY()
	TArray<FString> CollapsedNodesPaths;

	UPROPERTY()
	bool bIsAggregate;
};

/**
 * An action to expand a library node into its content
 */
USTRUCT()
struct FRigVMExpandNodeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMExpandNodeAction();
	FRigVMExpandNodeAction(URigVMController* InController, URigVMLibraryNode* InLibraryNode);
	virtual ~FRigVMExpandNodeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMExpandNodeAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString LibraryNodePath;

	UPROPERTY()
	FString LibraryNodeContent;

	UPROPERTY()
	TArray<FString> ExpandedNodePaths;
};

/**
 * An action renaming a node in the graph.
 */
USTRUCT()
struct FRigVMRenameNodeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRenameNodeAction();
	FRigVMRenameNodeAction(URigVMController* InController, const FName& InOldNodeName, const FName& InNewNodeName);
	virtual ~FRigVMRenameNodeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRenameNodeAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString OldNodeName;

	UPROPERTY()
	FString NewNodeName;
};

/**
 * An action exposing a pin as a parameter
 */
USTRUCT()
struct FRigVMAddExposedPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMAddExposedPinAction();
	FRigVMAddExposedPinAction(URigVMController* InController, URigVMPin* InPin);
	virtual ~FRigVMAddExposedPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMAddExposedPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinName;
	
	UPROPERTY()
	ERigVMPinDirection Direction;

	UPROPERTY()
	FString CPPType;

	UPROPERTY()
	FString CPPTypeObjectPath;
	
	UPROPERTY()
	FString DefaultValue;
};

/**
 * An action exposing a pin as a parameter
 */
USTRUCT()
struct FRigVMRemoveExposedPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveExposedPinAction();
	FRigVMRemoveExposedPinAction(URigVMController* InController, URigVMPin* InPin);
	virtual ~FRigVMRemoveExposedPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRemoveExposedPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinName;
	
	UPROPERTY()
	ERigVMPinDirection Direction;

	UPROPERTY()
	FString CPPType;

	UPROPERTY()
	FString CPPTypeObjectPath;

	UPROPERTY()
	FString DefaultValue;

	UPROPERTY()
	int32 PinIndex;
};

/**
 * An action renaming an exposed in the graph.
 */
USTRUCT()
struct FRigVMRenameExposedPinAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRenameExposedPinAction();
	FRigVMRenameExposedPinAction(URigVMController* InController, const FName& InOldPinName, const FName& InNewPinName);
	virtual ~FRigVMRenameExposedPinAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRenameExposedPinAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString OldPinName;

	UPROPERTY()
	FString NewPinName;
};

/**
 * An action to reorder pins on a node
 */
USTRUCT()
struct FRigVMSetPinIndexAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMSetPinIndexAction();
	FRigVMSetPinIndexAction(URigVMController* InController, URigVMPin* InPin, int32 InNewIndex);
	virtual ~FRigVMSetPinIndexAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetPinIndexAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	int32 OldIndex;

	UPROPERTY()
	int32 NewIndex;
};

/**
 * An action to mark an exposed pin as an input variable
 */
USTRUCT()
struct FRigVMMarkExposedPinAsInputVariableAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMMarkExposedPinAsInputVariableAction();
	FRigVMMarkExposedPinAsInputVariableAction(URigVMController* InController, URigVMPin* InPin, bool bNewDefinedAsExternalVariableState);
	virtual ~FRigVMMarkExposedPinAsInputVariableAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMMarkExposedPinAsInputVariableAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString PinPath;

	UPROPERTY()
	bool OldState;

	UPROPERTY()
	bool NewState;
};

/**
* An action to add a local variable.
*/
USTRUCT()
struct FRigVMAddLocalVariableAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMAddLocalVariableAction();
	FRigVMAddLocalVariableAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable);
	virtual ~FRigVMAddLocalVariableAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMAddLocalVariableAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FRigVMGraphVariableDescription LocalVariable;
};

/**
* An action to remove a local variable.
*/
USTRUCT()
struct FRigVMRemoveLocalVariableAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveLocalVariableAction();
	FRigVMRemoveLocalVariableAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable);
	virtual ~FRigVMRemoveLocalVariableAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRemoveLocalVariableAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FRigVMGraphVariableDescription LocalVariable;
};

/**
* An action to rename a local variable.
*/
USTRUCT()
struct FRigVMRenameLocalVariableAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRenameLocalVariableAction();
	FRigVMRenameLocalVariableAction(URigVMController* InController, const FName& InOldName, const FName& InNewName);
	virtual ~FRigVMRenameLocalVariableAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRenameLocalVariableAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName OldVariableName;

	UPROPERTY()
	FName NewVariableName;
};

/**
* An action to change the type of a local variable.
*/
USTRUCT()
struct FRigVMChangeLocalVariableTypeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:
	FRigVMChangeLocalVariableTypeAction();
	FRigVMChangeLocalVariableTypeAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable, const FString& InCPPType, UObject* InCPPTypeObject);
	virtual ~FRigVMChangeLocalVariableTypeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMChangeLocalVariableTypeAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FRigVMGraphVariableDescription LocalVariable;

	UPROPERTY()
	FString CPPType;

	UPROPERTY()
	TObjectPtr<UObject> CPPTypeObject;
};

/**
* An action to change the default value of a local variable.
*/
USTRUCT()
struct FRigVMChangeLocalVariableDefaultValueAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:
	FRigVMChangeLocalVariableDefaultValueAction();
	FRigVMChangeLocalVariableDefaultValueAction(URigVMController* InController, const FRigVMGraphVariableDescription& InLocalVariable, const FString& InDefaultValue);
	virtual ~FRigVMChangeLocalVariableDefaultValueAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMChangeLocalVariableDefaultValueAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FRigVMGraphVariableDescription LocalVariable;

	UPROPERTY()
	FString DefaultValue;
};

/**
 * An action to reorder a local variable to a new index
 */
USTRUCT()
struct FRigVMSetLocalVariableIndexAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:
	FRigVMSetLocalVariableIndexAction();
	FRigVMSetLocalVariableIndexAction(URigVMController* InController, const FGuid& InVariableGuid, int32 InOldIndex, int32 InNewIndex);
	virtual ~FRigVMSetLocalVariableIndexAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMSetLocalVariableIndexAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FGuid VariableGuid;

	UPROPERTY()
	int32 OldIndex;

	UPROPERTY()
	int32 NewIndex;
};

/**
 * An action to promote a function to collapse node or vice versa
 */
USTRUCT()
struct FRigVMPromoteNodeAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMPromoteNodeAction();
	FRigVMPromoteNodeAction(URigVMController* InController, const URigVMNode* InNodeToPromote, const FString& InNodePath, const FString& InFunctionDefinitionPath);
	virtual ~FRigVMPromoteNodeAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMPromoteNodeAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString LibraryNodePath;

	UPROPERTY()
	FString FunctionDefinitionPath;

	UPROPERTY()
	bool bFromFunctionToCollapseNode;
};

/**
 * An action marking a function as public/private.
 */
USTRUCT()
struct FRigVMMarkFunctionPublicAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMMarkFunctionPublicAction();
	FRigVMMarkFunctionPublicAction(URigVMController* InController, const FName& InFunctionName, bool bInIsPublic);
	virtual ~FRigVMMarkFunctionPublicAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMMarkFunctionPublicAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName FunctionName;
	
	UPROPERTY()
	bool bIsPublic;
};

/**
 * An action creating a function variant
 */
USTRUCT()
struct FRigVMCreateFunctionVariantAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMCreateFunctionVariantAction();
	FRigVMCreateFunctionVariantAction(URigVMController* InController, const FName& InFunctionName, const FName& InNewFunctionName);
	virtual ~FRigVMCreateFunctionVariantAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMCreateFunctionVariantAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	FName NewFunctionName;
};

/**
 * An action adding a tag to a function variant
 */
USTRUCT()
struct FRigVMAddFunctionVariantTagAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMAddFunctionVariantTagAction();
	FRigVMAddFunctionVariantTagAction(URigVMController* InController, const FName& InFunctionName, const FRigVMTag& InTag);
	virtual ~FRigVMAddFunctionVariantTagAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMCreateFunctionVariantAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	FRigVMTag FunctionTag;
};

/**
 * An action removing a tag from a function variant
 */
USTRUCT()
struct FRigVMRemoveFunctionVariantTagAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveFunctionVariantTagAction();
	FRigVMRemoveFunctionVariantTagAction(URigVMController* InController, const FName& InFunctionName, const FName& InTagName);
	virtual ~FRigVMRemoveFunctionVariantTagAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMCreateFunctionVariantAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName FunctionName;

	UPROPERTY()
	FRigVMTag FunctionTag;
};

/**
 * An action importing nodes and links from text
 */
USTRUCT()
struct FRigVMImportFromTextAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMImportFromTextAction();
	FRigVMImportFromTextAction(URigVMController* InController, const FString& InContent, const TArray<FName>& InTopLevelNodeNames);
	FRigVMImportFromTextAction(URigVMController* InController, URigVMNode* InNode, bool bIncludeExteriorLinks = true);
	FRigVMImportFromTextAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes, bool bIncludeExteriorLinks = true);
	void SetContent(const TArray<URigVMNode*>& InNodes, bool bIncludeExteriorLinks = true);
	virtual ~FRigVMImportFromTextAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMImportFromTextAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FString Content;

	UPROPERTY()
	TArray<FName> TopLevelNodeNames;
};

/**
 * An action to add store / restore a single node
 */
USTRUCT()
struct FRigVMReplaceNodesAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMReplaceNodesAction();
	FRigVMReplaceNodesAction(URigVMController* InController, const TArray<URigVMNode*>& InNodes);
	virtual ~FRigVMReplaceNodesAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMReplaceNodesAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;
};

/**
 * An action to add a trait to a node
 */
USTRUCT()
struct FRigVMAddTraitAction : public FRigVMBaseAction
{
	GENERATED_BODY()

public:

	FRigVMAddTraitAction();
	FRigVMAddTraitAction(URigVMController* InController, const URigVMNode* InNode, const FName& InTraitName, const UScriptStruct* InTraitScriptStruct, const FString& InTraitDefault, int32 InPinIndex);
	virtual ~FRigVMAddTraitAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMAddTraitAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;

	UPROPERTY()
	FName NodeName;

	UPROPERTY()
	FName TraitName;

	UPROPERTY()
	FString ScriptStructPath;

	UPROPERTY()
	FString TraitDefault;

	UPROPERTY()
	int32 PinIndex;
};

/**
 * An action to remove a trait from a node
 */
USTRUCT()
struct FRigVMRemoveTraitAction : public FRigVMAddTraitAction
{
	GENERATED_BODY()

public:

	FRigVMRemoveTraitAction();
	FRigVMRemoveTraitAction(URigVMController* InController, const URigVMNode* InNode, const FName& InTraitName, const UScriptStruct* InTraitScriptStruct, const FString& InTraitDefault, int32 InPinIndex);
	virtual ~FRigVMRemoveTraitAction() = default;
	virtual UScriptStruct* GetScriptStruct() const override { return FRigVMRemoveTraitAction::StaticStruct(); }
	virtual bool Undo() override;
	virtual bool Redo() override;
};
