// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMClient.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMExternalDependency.generated.h"

#define UE_API RIGVMDEVELOPER_API

USTRUCT()
struct FRigVMExternalDependency
{
	GENERATED_BODY()

public:

	FRigVMExternalDependency()
		: ExternalPath()
		, InternalPath()
		, Category(NAME_None)
	{}

	FRigVMExternalDependency(const FString& InPath, const FName& InCategory)
		: ExternalPath(InPath)
		, InternalPath()
		, Category(InCategory)
	{}

	bool IsExternal() const { return InternalPath.IsEmpty(); }
	bool IsInternal() const { return !IsExternal(); }

	const FString& GetExternalPath() const { return ExternalPath; }
	const FString& GetInternalPath() const { return InternalPath; }
	const FName& GetCategory() const { return Category; }

	bool operator==(const FRigVMExternalDependency& InOther) const
	{
		return ExternalPath == InOther.ExternalPath &&
			InternalPath == InOther.InternalPath &&
			Category == InOther.Category;
	}

private:

	UPROPERTY()
	FString ExternalPath;

	UPROPERTY()
	FString InternalPath;

	UPROPERTY()
	FName Category;
};

UINTERFACE(MinimalAPI)
class URigVMExternalDependencyManager : public UInterface
{
	GENERATED_BODY()
};

// Interface to deal with mapping external dependencies
class IRigVMExternalDependencyManager
{
	GENERATED_BODY()

public:
	
	UE_API virtual const TArray<FName>& GetExternalDependencyCategories() const;
	virtual TArray<FRigVMExternalDependency> GetExternalDependenciesForCategory(const FName& InCategory) const = 0;
	UE_API TArray<FRigVMExternalDependency> GetAllExternalDependencies() const;

	static inline const FLazyName UserDefinedEnumCategory = FLazyName(TEXT("UserDefinedEnum"));
	static inline const FLazyName UserDefinedStructCategory = FLazyName(TEXT("UserDefinedStruct"));
	static inline const FLazyName RigVMGraphFunctionCategory = FLazyName(TEXT("RigVMGraphFunction"));

protected:

	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMClient* InClient) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionStore* InFunctionStore) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionData* InFunction) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMGraphFunctionHeader* InHeader) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const FRigVMFunctionCompilationData* InCompilationData) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMGraph* InGraph) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMNode* InNode) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const URigVMPin* InPin) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UStruct* InStruct) const;
	UE_API virtual void CollectExternalDependencies(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UEnum* InEnum) const;
	UE_API virtual void CollectExternalDependenciesForCPPTypeObject(TArray<FRigVMExternalDependency>& OutDependencies, const FName& InCategory, const UObject* InObject) const;

	static UE_API TArray<FName> DependencyCategories;

	friend class IRigVMEditorAssetInterface;
};

#undef UE_API
