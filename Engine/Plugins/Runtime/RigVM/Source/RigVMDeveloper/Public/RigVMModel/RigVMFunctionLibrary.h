// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMGraph.h"
#include "RigVMBuildData.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "RigVMFunctionLibrary.generated.h"

#define UE_API RIGVMDEVELOPER_API

DECLARE_DELEGATE_RetVal(const FSoftObjectPath, URigVMFunctionLibrary_GetFunctionHostObjectPath);

/**
 * The Function Library is a graph used only to store
 * the sub graphs used for functions.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMFunctionLibrary : public URigVMGraph
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMFunctionLibrary();

	// URigVMGraph interface
	UE_API virtual FString GetNodePath() const override;
	UE_API virtual URigVMFunctionLibrary* GetDefaultFunctionLibrary() const override;
	// end URigVMGraph interface

	// Returns all of the stored functions
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<URigVMLibraryNode*> GetFunctions() const;

	// Finds a function by name
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMLibraryNode* FindFunction(const FName& InFunctionName) const;

	// Finds a function by a node within a function (or a sub graph of that)
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
    UE_API URigVMLibraryNode* FindFunctionForNode(URigVMNode* InNode) const;

	// Returns all references for a given function name
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > GetReferencesForFunction(const FName& InFunctionName);

	// Returns all references for a given function name
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray< FString > GetReferencePathsForFunction(const FName& InFunctionName);

	/**
	* Iterator function to invoke a lambda / TFunction for each reference of a function
	* @param InFunctionName The function name to iterate all references for
	* @param PerReferenceFunction The function to invoke for each reference
	* @param bLoadIfNecessary If true, will load packages when needed
	*/
	UE_API void ForEachReference(const FName& InFunctionName, TFunction<void(URigVMFunctionReferenceNode*)> PerReferenceFunction, bool bLoadIfNecessary = true) const;

	/**
	* Iterator function to invoke a lambda / TFunction for each reference of a function
	* @param InFunctionName The function name to iterate all references for
	* @param PerReferenceFunction The function to invoke for each reference
	*/
	UE_API void ForEachReferenceSoftPtr(const FName& InFunctionName, TFunction<void(TSoftObjectPtr<URigVMFunctionReferenceNode>)> PerReferenceFunction) const;

	// Returns a function that has been previously localized based on the provided function to localize.
	// We maintain meta data on what functions have been created locally based on which other ones,
	// and use this method to avoid redundant localizations.
	UE_API URigVMLibraryNode* FindPreviouslyLocalizedFunction(FRigVMGraphFunctionIdentifier InFunctionToLocalize);

	UE_API const FSoftObjectPath GetFunctionHostObjectPath() const;
	URigVMFunctionLibrary_GetFunctionHostObjectPath GetFunctionHostObjectPathDelegate;

	bool IsFunctionPublic(const FName& InFunctionName) const { return PublicFunctionNames.Contains(InFunctionName); }

	UE_API const FRigVMVariant* GetFunctionVariant(const FName& InFunctionName) const;
	UE_API FRigVMVariant* GetFunctionVariant(const FName& InFunctionName);

private:

	UPROPERTY()
	TArray<FName> PublicFunctionNames;

	UPROPERTY()
	TMap<FName, FRigVMVariant> FunctionToVariant;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap< TObjectPtr<URigVMLibraryNode>, FRigVMFunctionReferenceArray > FunctionReferences_DEPRECATED;
#endif

	// A map which stores a library node per original pathname.
	// The source pathname is the full path of the source function that was localized
	// to the local copy stored in the value of the pair.
	UPROPERTY()
	TMap< FString, TObjectPtr<URigVMLibraryNode> > LocalizedFunctions;

	friend class URigVMController;
	friend class URigVMCompiler;
	friend class IRigVMEditorAssetInterface;
	friend class IControlRigEditorAssetInterface;
	friend struct FRigVMClient;
	friend class UControlRigBlueprint;
};

#undef UE_API
