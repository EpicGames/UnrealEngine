// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMGraphSection.h"
#include "RigVMLink.h"
#include "RigVMNotifications.h"
#include "Nodes/RigVMVariableNode.h"
#include "Nodes/RigVMParameterNode.h"
#include "Nodes/RigVMFunctionEntryNode.h"
#include "Nodes/RigVMFunctionReturnNode.h"
#include "RigVMCompiler/RigVMAST.h"
#include "ScribbleGraph.h"
#include "RigVMGraph.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMFunctionLibrary;
struct FRigVMClient;


/**
 * The Graph represents a Function definition
 * using Nodes as statements.
 * Graphs can be compiled into a URigVM using the 
 * FRigVMCompiler. 
 * Graphs provide access to its Nodes, Pins and
 * Links.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMGraph : public UObject
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMGraph();

	UE_API virtual void PostLoad() override;

	// Returns all of the Nodes within this Graph.
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API const TArray<URigVMNode*>& GetNodes() const;

	// Returns all of the Links within this Graph.
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API const TArray<URigVMLink*>& GetLinks() const;

	// Returns true if the graph contains a link given its string representation 
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API bool ContainsLink(const FString& InPinPathRepresentation) const;

	// Returns all of the contained graphs
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<URigVMGraph*> GetContainedGraphs(bool bRecursive = false) const;

	// Returns the parent graph of this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
    UE_API URigVMGraph* GetParentGraph() const;

	// Returns the root / top level parent graph of this graph (or this if it is the root graph)
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
    UE_API URigVMGraph* GetRootGraph() const;

	// Returns the root / top level parent graph of this graph (or this if it is the root graph)
	UFUNCTION(BlueprintPure, Category = RigVMGraph)
	UE_API int32 GetGraphDepth() const;

	// Returns true if this graph is a root / top level graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
    UE_API bool IsRootGraph() const;

	// Returns the entry node of this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMFunctionEntryNode* GetEntryNode() const;
	
	// Returns the return node of this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMFunctionReturnNode* GetReturnNode() const;

	// Returns array of event names
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<FName> GetEventNames() const;

	// Returns a list of unique Variable descriptions within this Graph.
	// Multiple Variable Nodes can share the same description.
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<FRigVMGraphVariableDescription> GetVariableDescriptions() const;

	// Returns the path of this graph as defined by its invoking nodes
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API virtual FString GetNodePath() const;

	// Returns the name of this graph (as defined by the node path)
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API FString GetGraphName() const;

	// Returns a Node given its name (or nullptr).
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMNode* FindNodeByName(const FName& InNodeName) const;

	// Returns a Node given its path (or nullptr).
	// (for now this is the same as finding a node by its name.)
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMNode* FindNode(const FString& InNodePath) const;

	// Returns a Pin given its path, for example "Node.Color.R".
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMPin* FindPin(const FString& InPinPath) const;

	// Returns a link given its string representation,
	// for example "NodeA.Color.R -> NodeB.Translation.X"
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMLink* FindLink(const FString& InLinkPinPathRepresentation) const;

	// Returns true if a Node with a given name is selected.
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API bool IsNodeSelected(const FName& InNodeName) const;

	// Returns the names of all currently selected Nodes.
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API const TArray<FName>& GetSelectNodes() const;

	// Returns true if a node matches the currently selected subset
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API bool IsNodeHighlighted(const FName& InNodeName) const;

	// Returns true if this graph is the top level graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API bool IsTopLevelGraph() const;

	// Returns the locally available function library
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API virtual URigVMFunctionLibrary* GetDefaultFunctionLibrary() const;

	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API void SetDefaultFunctionLibrary(URigVMFunctionLibrary* InFunctionLibrary);

	UE_API TArray<FRigVMExternalVariable> GetExternalVariables() const;

	// Returns the local variables of this function
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<FRigVMGraphVariableDescription> GetLocalVariables(bool bIncludeInputArguments = false) const;

	// Returns the input arguments of this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<FRigVMGraphVariableDescription> GetInputArguments() const;
	
	// Returns the output arguments of this graph
    UFUNCTION(BlueprintCallable, Category = RigVMGraph)
    UE_API TArray<FRigVMGraphVariableDescription> GetOutputArguments() const;

	// Returns the input variables of this function
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API TArray<FRigVMGraphVariableDescription> GetInputVariables() const;

	// Returns the schema used by this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API URigVMSchema* GetSchema() const;

	// Returns the schema class used by this graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	TSubclassOf<URigVMSchema> GetSchemaClass() const { return SchemaClass; }

	// Sets the schema class on the graph
	UFUNCTION(BlueprintCallable, Category = RigVMGraph)
	UE_API void SetSchemaClass(TSubclassOf<URigVMSchema> InSchemaClass);

	// Returns the modified event, which can be used to 
	// subscribe to changes happening within the Graph.
	UE_API FRigVMGraphModifiedEvent& OnModified();

	// Sets the execute context struct type to use
	UE_API void SetExecuteContextStruct(UScriptStruct* InExecuteContextStruct);
	
	UE_API UScriptStruct* GetExecuteContextStruct() const;

	UE_API void PrepareCycleChecking(URigVMPin* InPin, bool bAsInput);
	UE_API virtual bool CanLink(URigVMPin* InSourcePin, URigVMPin* InTargetPin, FString* OutFailureReason, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection = ERigVMPinDirection::IO, bool bEnableTypeCasting = true);
	
	UE_API TSharedPtr<FRigVMParserAST> GetDiagnosticsAST(bool bForceRefresh = false, TArray<URigVMLink*> InLinksToSkip = TArray<URigVMLink*>());
	UE_API TSharedPtr<FRigVMParserAST> GetRuntimeAST(const FRigVMParserASTSettings& InSettings = FRigVMParserASTSettings::Optimized(), bool bForceRefresh = false);
	UE_API void ClearAST(bool bClearDiagnostics = true, bool bClearRuntime = true);

	UE_API virtual uint32 GetStructureHash() const;
	uint32 GetSerializedStructureHash() const { return LastStructureHash; }

	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;

   	UE_API bool AreSectionsEnabled() const;
	const TArray<FRigVMGraphSection>& GetSelectedSections() const { return SelectedSections; }
	const TArray<FRigVMGraphSection>& GetSectionsMatchingTheSelection() const { return SectionsMatchingTheSelection; }
	UE_API TArray<FRigVMGraphSection> GetMatchingSections(const FRigVMGraphSection& InSection) const;
	UE_API void UpdateSections(bool bForce = false);

	UE_API TSharedPtr<FScribbleGraphData> SetupScribbleGraph(bool bCreateIfMissing = false);
	
private:

	FRigVMGraphModifiedEvent ModifiedEvent;
	UE_API void Notify(ERigVMGraphNotifType InNotifType, UObject* InSubject);

	UPROPERTY()
	TArray<TObjectPtr<URigVMNode>> Nodes;

	UPROPERTY()
	TArray<TObjectPtr<URigVMLink>> Links;

	UPROPERTY(transient)
	TArray<TObjectPtr<URigVMLink>> DetachedLinks;

	UPROPERTY()
	TArray<FName> SelectedNodes;

	UPROPERTY()
	TWeakObjectPtr<URigVMGraph> DefaultFunctionLibraryPtr;

	UPROPERTY()
	TObjectPtr<UScriptStruct> ExecuteContextStruct;

	TSharedPtr<FRigVMParserAST> DiagnosticsAST;
	TSharedPtr<FRigVMParserAST> RuntimeAST;

#if WITH_EDITOR
	TArray<TSharedPtr<FString>> VariableNames;
	TArray<TSharedPtr<FString>> ParameterNames;
#endif

	UPROPERTY()
	uint32 LastStructureHash;

	UPROPERTY()
	bool bEditable;

	UPROPERTY()
	TArray<FRigVMGraphVariableDescription> LocalVariables;

	UPROPERTY()
	TSubclassOf<URigVMSchema> SchemaClass;

	UPROPERTY(transient)
	bool bSectionsEnabled;

	UPROPERTY(transient)
	TArray<FRigVMGraphSection> SelectedSections;

	UPROPERTY(transient)
	TArray<FRigVMGraphSection> SectionsMatchingTheSelection;

	UE_API bool IsNameAvailable(const FString& InName) const;
	UE_API bool IsNameAvailable(FName InName) const;

	UPROPERTY()
	FScribbleGraph ScribbleGraph;

	friend class URigVMController;
	friend class URigVMBlueprint;
	friend class IRigVMEditorAssetInterface;
	friend class FRigVMControllerCompileBracketScope;
	friend class URigVMCompiler;
	friend class URigVMSchema;
	friend class URigVMNode;
	friend struct FRigVMClient;
	friend struct FRigVMControllerObjectFactory;
	friend class FRigVMFunctionArgumentLayout;
	friend class FRigVMFunctionArgumentReorderDragDropHandler;
	friend class FRigVMGraphDetailCustomizationImpl;
	friend class URigVMLibraryNode;
};

#undef UE_API
