// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMNode.h"
#include "RigVMGraphSection.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMGraph;

USTRUCT()
struct FRigVMGraphSectionLink
{
	GENERATED_BODY()

	UPROPERTY()
	int32 SourceNodeIndex = INDEX_NONE;
	
	UPROPERTY()
	int32 TargetNodeIndex = INDEX_NONE;

	UPROPERTY()
	uint32 SourceNodeHash = UINT32_MAX;
	
	UPROPERTY()
	uint32 TargetNodeHash = UINT32_MAX;

	UPROPERTY()
	FString SourcePinPath;
	
	UPROPERTY()
	FString TargetPinPath;

	using FPinTuple = TTuple<const URigVMPin*, const URigVMPin*>;
	static URigVMPin* FindSourcePinSkippingReroutes(const URigVMLink* InLink);
	static TArray<URigVMPin*> FindTargetPinsSkippingReroutes(const URigVMLink* InLink);
	static TArray<FPinTuple> FindLinksSkippingReroutes(const URigVMNode* InNode);
	const URigVMNode* FindSourceNode(const URigVMNode* InTargetNode) const;
	TArray<const URigVMNode*> FindTargetNodes(const URigVMNode* InSourceNode) const;
};

USTRUCT()
struct FRigVMGraphSection
{
	GENERATED_BODY()

public:

	UE_API FRigVMGraphSection();
	UE_API FRigVMGraphSection(const TArray<URigVMNode*>& InNodes);

	UE_API static TArray<FRigVMGraphSection> GetSectionsFromSelection(const URigVMGraph* InGraph);
	
	UE_API bool IsValid() const;
	UE_API static bool IsValidNode(const URigVMNode* InNode);

	UPROPERTY()
	uint32 Hash;

	UPROPERTY()
	TArray<FName> Nodes;

	UPROPERTY()
	TArray<uint32> NodeHashes;

	UPROPERTY()
	TArray<int32> LeafNodes;

	UPROPERTY()
	TArray<FRigVMGraphSectionLink> Links;

	UE_API TArray<FRigVMGraphSection> FindMatches(const URigVMGraph* InGraph) const;
	UE_API TArray<FRigVMGraphSection> FindMatches(const TArray<URigVMNode*>& InAvailableNodes) const;
	
	UE_API bool ContainsNode(const FName& InNodeName) const;
	UE_API bool MatchesNode(const URigVMNode* InNode, const TArray<URigVMNode*>& InAvailableNodes, TArray<URigVMNode*>* OutMatchingNodesForSet = nullptr) const;

private:

	void Reset();

	static TArray<TArray<URigVMNode*>> GetNodeIslands(const TArray<URigVMNode*>& InNodes);
	
	static uint32 GetNodeHash(const URigVMNode* InNode);
	static uint32 GetLinkHash(const FRigVMGraphSectionLink& InLink);
	
	bool MatchesNode_Impl(const URigVMNode* InNode, const TArray<URigVMNode*>& InAvailableNodes, int32 InNodeIndex, TArray<URigVMNode*>& VisitedNodes) const;

	friend struct FRigVMGraphSectionLink;
};

#undef UE_API
