// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RigVMPin.h"
#include "RigVMLink.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMGraph;

/**
 * The Link represents a connection between two Pins
 * within a Graph. The Link can be accessed on the 
 * Graph itself - or through the URigVMPin::GetLinks()
 * method.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMLink : public UObject
{
	GENERATED_BODY()

public:

	// Default constructor
	URigVMLink() = default;

	// Serialization override
	UE_API virtual void Serialize(FArchive& Ar) override;

	// Returns the current index of this Link within its owning Graph.
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API int32 GetLinkIndex() const;

	// Returns the Link's owning Graph/
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMGraph* GetGraph() const;

	// Returns the graph nesting depth of this link
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API int32 GetGraphDepth() const;

	// Returns the source Pin of this Link (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMPin* GetSourcePin() const;

	// Returns the target Pin of this Link (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMPin* GetTargetPin() const;

	// Returns the source Node of this Link (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMNode* GetSourceNode() const;

	// Returns the target Node of this Link (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMNode* GetTargetNode() const;

	// Returns the source pin's path pin of this Link
	UE_API FString GetSourcePinPath() const;

	// Returns the target pin's path pin of this Link
	UE_API FString GetTargetPinPath() const;

	// Sets the source pin's path pin of this Link
	UE_API bool SetSourcePinPath(const FString& InPinPath);

	// Sets the target pin's path pin of this Link
	UE_API bool SetTargetPinPath(const FString& InPinPath);

	// Sets the target pin's path pin of this Link
	UE_API bool SetSourceAndTargetPinPaths(const FString& InSourcePinPath, const FString& InTargetPinPath);

	// Returns the opposite Pin of this Link given one of its edges (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API URigVMPin* GetOppositePin(const URigVMPin* InPin) const;

	// Returns a string representation of the Link,
	// for example: "NodeA.Color.R -> NodeB.Translation.X"
	// note: can be split again using SplitPinPathRepresentation
	UFUNCTION(BlueprintCallable, Category = RigVMLink)
	UE_API FString GetPinPathRepresentation() const;

	// Returns a string representation of the Link given the two pin paths
	// for example: "NodeA.Color.R -> NodeB.Translation.X"
	// note: can be split again using SplitPinPathRepresentation
	static UE_API FString GetPinPathRepresentation(const FString& InSourcePinPath, const FString& InTargetPinPath);

	// Splits a pin path representation of a link
	// for example: "NodeA.Color.R -> NodeB.Translation.X"
	// into its two pin paths
	static UE_API bool SplitPinPathRepresentation(const FString& InString, FString& OutSource, FString& OutTarget);

	UE_API void UpdatePinPaths();
	UE_API void UpdatePinPointers() const;
	UE_API bool Detach();
	
private:

	// Returns true if the link is attached.
	// Attached links rely on the pin pointers first and the pin path second.
	// Deattached links never rely on the pin pointers and always try to resolve from string.
	UE_API bool IsAttached() const;
	UE_API bool Attach(FString* OutFailureReason = nullptr);

	UPROPERTY()
	FString SourcePinPath;

	UPROPERTY()
	FString TargetPinPath;

	mutable URigVMPin* SourcePin = nullptr;
	mutable URigVMPin* TargetPin = nullptr;
};

#undef UE_API
