// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * A cached value which relies on a version providing object.
 * The versioned object needs to provide uint32 GetCachedValueVersion()
 * Values within the cache are considered valid if they are a) set and
 * b) the version of the cache matches the version of the versioned object.
 */
template<typename VersionObjectType, typename ValueType>
class TRigVMModelCachedValue
{
public:

	// default empty constructor
	TRigVMModelCachedValue()
		: LastObjectVersion(UINT_MAX)
	{
	}

	// constructor for an empty cache bound to a versioned object
	TRigVMModelCachedValue(const VersionObjectType* InObject)
		: VersionedObjectWeakPtr(InObject)
		, LastObjectVersion(UINT_MAX)
	{
	}

	// constructor for a valid cache bound to a versioned object
	TRigVMModelCachedValue(VersionObjectType* InObject, const ValueType& InValue)
		: VersionedObjectWeakPtr(InObject)
		, LastObjectVersion(UINT_MAX)
	{
		Set(InObject ? InObject->GetCachedValueVersion() : 0, InValue);
	}

	// returns true if the cache is bound, the value has been set and the version is current
	bool IsValid() const
	{
		if(Value.IsSet())
		{
			if(const VersionObjectType* VersionedObject = VersionedObjectWeakPtr.Get())
			{
				return VersionedObject->GetCachedValueVersion() == LastObjectVersion;
			}
		}
		return false;
	}

	// returns true if the cache is bound to a versioned object
	bool IsBound() const
	{
		return VersionedObjectWeakPtr.IsValid();
	}

	// binds the cache to a new versioned object
	void Bind(const VersionObjectType* InVersionedObject)
	{
		if(VersionedObjectWeakPtr.IsValid())
		{
			if(VersionedObjectWeakPtr.Get() == InVersionedObject)
			{
				return;
			}
		}
		VersionedObjectWeakPtr = InVersionedObject;
		ResetCachedValue();
	}

	// unbinds the cache from a versioned object
	void Unbind()
	{
		Bind(nullptr);
	}

	// returns the cached value if it is valid - or otherwise returns the default
	const ValueType& Get(const ValueType& InDefault) const
	{
		if(IsValid())
		{
			return GetValue();
		}
		return InDefault;
	}

	// returns the cached value
	const ValueType& GetValue() const
	{
		check(IsValid());
		return Value.GetValue();
	}

	// sets the cached value and updates the last version given the bound object
	void Set(const ValueType& InValue)
	{
		const VersionObjectType* VersionedObject = VersionedObjectWeakPtr.Get();
		check(VersionedObject);
		
		Value = InValue;
		LastObjectVersion = VersionedObject->GetCachedValueVersion(); 
	}

	// resets the cached value
	void ResetCachedValue()
	{
		Value.Reset();
		LastObjectVersion = UINT_MAX;
	}

	// sets the value using an assignment operator
	TRigVMModelCachedValue& operator =(const ValueType& InValueType)
	{
		Set(InValueType);
		return *this;
	}

	// compares the cached value using a comparison operator
	bool operator ==(const ValueType& InValueType) const
	{
		if(IsValid())
		{
			return GetValue() == InValueType;
		}
		return false;
	}

	// compares the cached value usinga  comparison operator
	bool operator !=(const ValueType& InValueType) const
	{
		if(IsValid())
		{
			return GetValue() != InValueType;
		}
		return true;
	}

private:;

	TWeakObjectPtr<const VersionObjectType> VersionedObjectWeakPtr;
	TOptional<ValueType> Value;
	uint32 LastObjectVersion;
};