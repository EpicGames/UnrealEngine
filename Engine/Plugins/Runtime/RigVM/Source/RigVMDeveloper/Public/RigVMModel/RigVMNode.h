// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMPin.h"
#include "RigVMTraitDefaultValueStruct.h"
#include "RigVMCore/RigVM.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMCore/RigVMUserWorkflow.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "UObject/StructOnScope.h"
#include "RigVMNode.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMGraph;

namespace ERigVMNodeDefaultValueOverrideState
{
	enum Type : int
	{
		None, // No override.
		SomePins, // Some pins have been overriden
		AllPins, // All pins contains an override 
	};
}

UENUM()
enum class ERigVMNodeColorType : uint8
{
	FromMetadata,	// The color was determined from node metadata
	UserDefined,	// The color was supplied/edited by a user operation
};

/**
 * The Node represents a single statement within a Graph. 
 * Nodes can represent values such as Variables / Parameters,
 * they can represent Function Invocations or Control Flow
 * logic statements (such as If conditions of For loops).
 * Additionally Nodes are used to represent Comment statements.
 * Nodes contain Pins to represent parameters for Function
 * Invocations or Value access on Variables / Parameters.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMNode : public UObject
{
	GENERATED_BODY()

public:

	// Default constructor
	UE_API URigVMNode();

	// Default destructor
	UE_API virtual ~URigVMNode();

	UE_API void Serialize(FArchive& Ar) override;

	// Returns the a . separated string containing all of the
	// names used to reach this Node within the Graph.
	// (for now this is the same as the Node's name)
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API FString GetNodePath(bool bRecursive = false) const;

	// Splits a NodePath at the start, so for example "CollapseNodeA|CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA" and "CollapseNodeB|CollapseNodeC"
	static UE_API bool SplitNodePathAtStart(const FString& InNodePath, FString& LeftMost, FString& Right);

	// Splits a NodePath at the end, so for example "CollapseNodeA|CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA|CollapseNodeB" and "CollapseNodeC"
	static UE_API bool SplitNodePathAtEnd(const FString& InNodePath, FString& Left, FString& RightMost);

	// Splits a NodePath into all segments, so for example "Node.Color.R" becomes ["Node", "Color", "R"]
	static UE_API bool SplitNodePath(const FString& InNodePath, TArray<FString>& Parts);

	// Joins a NodePath from to segments, so for example "CollapseNodeA" and "CollapseNodeB|CollapseNodeC" becomes "CollapseNodeA|CollapseNodeB|CollapseNodeC"
	static UE_API FString JoinNodePath(const FString& Left, const FString& Right);

	// Joins a NodePath from to segments, so for example ["CollapseNodeA", "CollapseNodeB", "CollapseNodeC"] becomes "CollapseNodeA|CollapseNodeB|CollapseNodeC"
	static UE_API FString JoinNodePath(const TArray<FString>& InParts);

	// Returns the current index of the Node within the Graph.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API int32 GetNodeIndex() const;

	// Returns all of the top-level Pins of this Node.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual const TArray<URigVMPin*>& GetPins() const;

	// Returns all of the Pins of this Node (including SubPins).
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<URigVMPin*> GetAllPinsRecursively() const;

	// Returns all user defined categories on this node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual TArray<FString> GetPinCategories() const;

	// Returns the name of pin category
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API FString GetPinCategoryName(const FString InCategory) const;

	// Returns all sub user defined categories of a given parent category
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<FString> GetSubPinCategories(const FString InCategory, bool bOnlyExisting = false, bool bRecursive = false) const;

	// Returns the parent pin category of the given category (or an empty string in case there's no parent)
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API FString GetParentPinCategory(const FString InCategory, bool bOnlyExisting = false) const;

	// Returns all parent categories of a given 
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<FString> GetParentPinCategories(const FString InCategory, bool bOnlyExisting = false, bool bIncludeSelf = false) const;

	// Returns the depth of the category (starting with 0 for "Foo" and 2 for "Foo|Bar|Waldo")
	static UE_API int32 GetPinCategoryDepth(const FString& InCategory);

	// Returns all pins for a given category
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<URigVMPin*> GetPinsForCategory(FString InCategory) const;

	// Returns all pins for a given category
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsPinCategoryExpanded(FString InCategory) const;

	const FString& GetLastAffectedPinCategory() const { return LastAffectedPinCategory; }

	// Returns the pin UI layout for this node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FRigVMNodeLayout GetNodeLayout(bool bIncludeEmptyCategories = false) const;

	// Returns the default value for a given pin
	UE_API FString GetOriginalPinDefaultValue(const URigVMPin* InPin) const;

	// Returns 0 if there's no override on this node,
	// 1 if some pins have an override and
	// 2 if all pins are overriden
	UE_API ERigVMNodeDefaultValueOverrideState::Type GetPinDefaultValueOverrideState() const;

	// Returns a Pin given it's partial pin path below
	// this node (for example: "Color.R")
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API URigVMPin* FindPin(const FString& InPinPath) const;

	// Returns a root pin given its name
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API URigVMPin* FindRootPinByName(const FName& InPinName) const;

	// Returns the first execute pin
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API URigVMPin* FindExecutePin() const;

	// Returns all of the top-level orphaned Pins of this Node.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
    UE_API virtual const TArray<URigVMPin*>& GetOrphanedPins() const;

	// Returns true if the node has orphaned pins - which leads to a compiler error
	UFUNCTION(BlueprintPure, Category = RigVMNode)
    bool HasOrphanedPins() const { return GetOrphanedPins().Num() > 0; }

	// Returns the Graph of this Node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API URigVMGraph* GetGraph() const;

	// Returns the top level / root Graph of this Node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
    UE_API URigVMGraph* GetRootGraph() const;

	// Returns the graph nesting depth of this node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API int32 GetGraphDepth() const;

	// Returns the injection info of this Node (or nullptr)
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API URigVMInjectionInfo* GetInjectionInfo() const;

	// Returns the title of this Node - used for UI.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FString GetNodeTitle() const;

	// Returns currently node title set in the node, without any transformations
	// Warning : The raw title might be ignored by some nodes overrides to GetNodeTitle
	UE_API const FString& GetNodeTitleRaw() const;
	
	// Get the subtitle of the node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FString GetNodeSubTitle() const;

	// Returns true if this node can be renamed by the user
	UE_API virtual bool SupportsRenaming() const;

	// Returns the 2d position of this node - used for UI.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API FVector2D GetPosition() const;

	// Returns the 2d size of this node - used for UI.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API FVector2D GetSize() const;

	// Returns the color of this node - used for UI.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FLinearColor GetNodeColor() const;

	// Returns the icon of this node - used for UI.
	UE_API virtual FSlateIcon GetNodeIcon() const;

	// Returns the tooltip of this node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FText GetToolTipText() const;

	// Returns true if this Node is currently selected.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsSelected() const;

	// Returns true if this Node should be highlighted
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsHighlighted() const;

	// Returns true if this is an injected node.
	// Injected nodes are managed by pins are are not visible to the user.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsInjected() const;

	// Returns true if this should be visible in the UI
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsVisibleInUI() const;

	// Returns true if this Node has no side-effects
	// and no internal state.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool IsPure() const;

	// Returns true if the node is defined as non-varying
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	virtual bool IsDefinedAsConstant() const { return false; }

	// Returns true if the node is defined as varying
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	virtual bool IsDefinedAsVarying() const { return false; }

	// Returns true if this Node has side effects or
	// internal state.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool IsMutable() const;

	// Returns true if this node has an unknown type pin
	UE_API bool HasWildCardPin() const;

	virtual bool ContributesToResult() const { return IsMutable(); }

	// Returns true if this Node is the beginning of a scope
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool IsEvent() const;

	// Returns the name of the event
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual FName GetEventName() const;

	// Returns true if this node can only exist once in a graph
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool CanOnlyExistOnce() const;

	// Returns true if the node has any input pins
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool HasInputPin(bool bIncludeIO = true) const;

	// Returns true if the node has any io pins
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool HasIOPin() const;

	// Returns true if the node has any lazily evaluating pins
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool HasLazyPin(bool bOnlyConsiderPinsWithLinks = false) const;

	// Returns true if the node has any output pins
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool HasOutputPin(bool bIncludeIO = true) const;

	// Returns true if the node has any pins of the provided direction
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual bool HasPinOfDirection(ERigVMPinDirection InDirection) const;

	// Returns true if this Node is linked to another 
	// given node through any of the Nodes' Pins.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsLinkedTo(URigVMNode* InNode) const;

	// Returns all links to any pin on this node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<URigVMLink*> GetLinks() const;

	// Returns a list of Nodes connected as sources to
	// this Node as the target.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<URigVMNode*> GetLinkedSourceNodes() const;

	// Returns a list of Nodes connected as targets to
	// this Node as the source.
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API TArray<URigVMNode*> GetLinkedTargetNodes() const;

	// Returns the name of the node prior to the renaming
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	FName GetPreviousFName() const { return PreviousName; }

	// Returns the indices of associated instructions for this node
	UE_API const TArray<int32>& GetInstructionsForVM(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const;

	// Returns the number of visited / run instructions for this node
	UE_API virtual int32 GetInstructionVisitedCount(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const;

	// Returns the accumulated duration of all of instructions for this node
	UE_API double GetInstructionMicroSeconds(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const;

	// return true if this node is a loop node
	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API bool IsLoopNode() const;

	// return true if this node is a control flow node
	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API bool IsControlFlowNode() const;

	// returns the names of the control flow blocks of this node
	UE_API virtual const TArray<FName>& GetControlFlowBlocks() const;

	// returns true if a control flow block requires slicing
	UE_API virtual const bool IsControlFlowBlockSliced(const FName& InBlockName) const;

	// returns true if the node is within a loop
	UE_API virtual bool IsWithinLoop() const;

	// returns true if the node can be upgraded
	UFUNCTION(BlueprintPure, Category = RigVMNode)
	virtual bool CanBeUpgraded() const { return GetUpgradeInfo().IsValid(); }

	// returns all supported workflows of the node
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual TArray<FRigVMUserWorkflow> GetSupportedWorkflows(ERigVMUserWorkflowType InType, const UObject* InSubject) const;

	UE_DEPRECATED(5.7, "Please, use HasEarlyExitMarker")
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	bool HasBreakpoint() const { return false; }

	UE_DEPRECATED(5.7, "Please, use SetHasEarlyExitMarker")
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	void SetHasBreakpoint(const bool bValue) {}

	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool HasEarlyExitMarker() const;

	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API void SetHasEarlyExitMarker(const bool bValue);

	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API bool IsExcludedByEarlyExit() const;

	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API void SetIsExcludedByEarlyExit(bool bIsExcluded);

	UE_DEPRECATED(5.7, "Please, use HasEarlyExitMarker")
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	bool ExecutionIsHaltedAtThisNode() const { return HasEarlyExitMarker(); }

	UE_DEPRECATED(5.7, "Please, use SetHasEarlyExitMarker")
	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	void SetExecutionIsHaltedAtThisNode(const bool bValue) { SetHasEarlyExitMarker(bValue); }

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API virtual bool IsAggregate() const;

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API virtual URigVMPin* GetFirstAggregatePin() const;

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API virtual URigVMPin* GetSecondAggregatePin() const;

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API virtual URigVMPin* GetOppositeAggregatePin() const;

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API virtual bool IsInputAggregate() const;

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	virtual TArray<URigVMPin*> GetAggregateInputs() const { return {}; }

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	virtual TArray<URigVMPin*> GetAggregateOutputs() const { return {}; }

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	virtual FName GetNextAggregateName(const FName& InLastAggregatePinName) const { return NAME_None; }

	UFUNCTION(BlueprintCallable, Category = RigVMNode)
	UE_API virtual URigVMLibraryNode* FindFunctionForNode() const;

	virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const { return FRigVMStructUpgradeInfo(); }

	UE_API virtual uint32 GetStructureHash() const;

	// allows the node to support non-native pins
	virtual bool HasNonNativePins() const { return !TraitRootPinNames.IsEmpty(); }

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API TArray<URigVMPin*> GetTraitPins() const;

	const TArray<FString>& GetTraitNames() const { return TraitRootPinNames; }

	UFUNCTION(BlueprintPure, Category = RigVMNode)
	UE_API bool IsTraitPin(FName InName) const;

	UE_API bool IsTraitPin(const URigVMPin* InTraitPin) const;

	UE_API URigVMPin* FindTrait(const FName& InName, const FString& InSubPinPath = FString()) const;

	UE_API URigVMPin* FindTrait(const URigVMPin* InTraitPin) const;

	UE_API TSharedPtr<FStructOnScope> GetTraitInstance(const FName& InName, bool bUseDefaultValueFromPin = true) const;

	UE_API TSharedPtr<FStructOnScope> GetTraitInstance(const URigVMPin* InTraitPin, bool bUseDefaultValueFromPin = true) const;

	UE_API UScriptStruct* GetTraitScriptStruct(const FName& InName) const;

	UE_API UScriptStruct* GetTraitScriptStruct(const URigVMPin* InTraitPin) const;
	
	virtual bool IsOutDated() const
	{
		return false;
	}
	
	virtual FString GetDeprecatedMetadata() const
	{
		return FString();
	}

	// returns the display name for a pin
	UE_API FName GetDisplayNameForPin(const FString& InPinPath) const;
	UE_API virtual FName GetDisplayNameForPin(const URigVMPin* InPin) const;

	// returns the display name for a struct member
	static UE_API FName GetDisplayNameForStructMember(const URigVMPin* InPin);
	static UE_API FName GetDisplayNameForStructMember(const UStruct* InStruct, const FString& InPath);

	// returns the display name within a property
	static UE_API FName GetDisplayNameForProperty(const FProperty* InProperty, const FString& InRemainingPath);

	// returns the category for a pin
	UE_API virtual FString GetCategoryForPin(const FString& InPinPath) const;

	// returns the index of a pin within a category
	UE_API virtual int32 GetIndexInCategoryForPin(const FString& InPinPath) const;

	UE_API virtual FText GetTypedToolTipText(const URigVMPin* InPin, const FText& ToolTipBody) const;

private:

	static const inline TCHAR* NodeColorName = TEXT("NodeColor");

	UE_API bool IsLinkedToRecursive(URigVMPin* InPin, URigVMNode* InNode) const;
	UE_API void GetLinkedNodesRecursive(URigVMPin* InPin, bool bLookForSources, TArray<URigVMNode*>& OutNodes) const;

protected:

	UE_API virtual void InvalidateCache();
	UE_API virtual TArray<int32> GetInstructionsForVMImpl(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy = FRigVMASTProxy()) const; 
	UE_API virtual FText GetToolTipTextForPin(const URigVMPin* InPin) const;
	virtual bool AllowsLinksOn(const URigVMPin* InPin) const { return true; }
	virtual bool ShouldInputPinComputeLazily(const URigVMPin* InPin) const { return false; }
	virtual bool IsPinConstant(const URigVMPin* InPin) const { return false; }
	UE_API virtual FString GetOriginalDefaultValueForRootPin(const URigVMPin* InRootPin) const;

	UE_API void UpdateTraitRootPinNames();

public:
	const uint32& GetCachedValueVersion() const
	{
		return NodeVersion;
	}

protected:
	UE_API void IncrementVersion();

	UPROPERTY()
	FString NodeTitle;

	UPROPERTY()
	FVector2D Position;

	UPROPERTY()
	FVector2D Size;

	UPROPERTY()
	FLinearColor NodeColor;

	UPROPERTY()
	ERigVMNodeColorType NodeColorType;

	UPROPERTY(transient)
	FName PreviousName;

	UPROPERTY(transient)
	bool bHasEarlyExitMarker;

	UPROPERTY(transient)
	bool bIsExcludedByEarlyExit;

	UPROPERTY()
	TArray<FString> TraitRootPinNames;

	// root trait pin store their default value in a separate property bag so that
	// things like soft object ptr can be used and tracked in a uproperty 
	UPROPERTY()
	TMap<FString, FRigVMTraitDefaultValueStruct> TraitDefaultValues;
	
private:

	UPROPERTY()
	TArray<TObjectPtr<URigVMPin>> Pins;

	UPROPERTY()
	TArray<TObjectPtr<URigVMPin>> OrphanedPins;

protected:
	
	UPROPERTY()
	TArray<FString> PinCategories;

	UPROPERTY()
	TMap<FString, bool> PinCategoryExpansion;

	FString LastAffectedPinCategory;

	uint32 NodeVersion;
	
private:

#if WITH_EDITOR
	struct FProfilingCache
	{
		mutable int32 VisitedCount;
		mutable double MicroSeconds;
		mutable TArray<int32> Instructions;
	};
	UE_API const FProfilingCache* UpdateProfilingCacheIfNeeded(const FRigVMExtendedExecuteContext& Context, URigVM* InVM, const FRigVMASTProxy& InProxy) const;
	mutable uint32 ProfilingHash;
	mutable TMap<uint32, TSharedPtr<FProfilingCache>> ProfilingCache;
	static UE_API TArray<int32> EmptyInstructionArray;
#endif

	mutable TMap<FString, FString> CachedOriginalPinDefaultValues;
	
	friend class URigVMController;
	friend class URigVMGraph;
	friend class URigVMPin;
	friend class URigVMCompiler;
	friend class FRigVMLexer;
	friend class URigVMSchema;
};

#undef UE_API
