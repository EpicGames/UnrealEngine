// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "RigVMNotifications.generated.h"

class URigVMGraph;

/**
 * The Graph Notification Type is used to differentiate
 * between all of the changes that can happen within a graph.
 */
UENUM(BlueprintType)
enum class ERigVMGraphNotifType : uint8
{
	GraphChanged, // The graph has changed / a new graph has been picked (Subject == nullptr)
	NodeAdded, // A node has been added to the graph (Subject == URigVMNode)
	NodeRemoved, // A node has been removed from the graph (Subject == URigVMNode)
	NodeSelected, // A node has been selected (Subject == URigVMNode)
	NodeDeselected, // A node has been deselected (Subject == URigVMNode)
	NodeSelectionChanged, // The set of selected nodes has changed (Subject == nullptr)
	NodePositionChanged, // A node's position has changed (Subject == URigVMNode)
	NodeSizeChanged, // A node's size has changed (Subject == URigVMNode)
	NodeTitleChanged, // A node's title has changed (Subject == URigVMNode)
	NodeColorChanged, // A node's color has changed (Subject == URigVMNode)
	PinAdded, // A pin has been added to a given node (Subject == URigVMPin)
	PinRemoved, // A pin has been removed from a given node (Subject == URigVMPin)
	PinRenamed, // A pin has been renamed (Subject == URigVMPin)
	PinExpansionChanged, // A pin's expansion state has changed(Subject == URigVMPin)
	PinWatchedChanged, // A pin's watch state has changed (Subject == URigVMPin)
	PinArraySizeChanged, // An array pin's size has changed (Subject == URigVMPin)
	PinDefaultValueChanged, // A pin's default value has changed (Subject == URigVMPin)
	PinDirectionChanged, // A pin's direction has changed (Subject == URigVMPin)
	PinTypeChanged, // A pin's data type has changed (Subject == URigVMPin)
	PinIndexChanged, // A pin's index has changed (Subject == URigVMPin)
	LinkAdded, // A link has been added (Subject == URigVMLink)
	LinkRemoved, // A link has been removed (Subject == URigVMLink)
	CommentTextChanged, // A comment node's text has changed (Subject == URigVMCommentNode)
	VariableAdded, // A variable has been added (Subject == URigVMVariableNode)
	VariableRemoved, // A variable has been removed (Subject == URigVMVariableNode)
	VariableRenamed, // A variable has been renamed (Subject == URigVMVariableNode)
	InteractionBracketOpened, // A bracket has been opened (Subject == nullptr)
	InteractionBracketClosed, // A bracket has been opened (Subject == nullptr)
	InteractionBracketCanceled, // A bracket has been canceled (Subject == nullptr)
	PinBoundVariableChanged, // A pin has been bound or unbound to / from a variable (Subject == URigVMPin)
	NodeRenamed, // A node has been renamed in the graph (Subject == URigVMNode)
	FunctionRenamed, // A function has been renamed in the graph (Subject == URigVMLibraryNode)
	NodeReferenceChanged, // A node has changed it's referenced function
	NodeCategoryChanged, // A node's category has changed (Subject == URigVMNode)
	NodeKeywordsChanged, // A node's keywords have changed (Subject == URigVMNode)
	NodeDescriptionChanged, // A node's description has changed (Subject == URigVMNode)
	VariableRemappingChanged, // (DEPRECATED) A function reference node's remapping has changed (Subject == URigVMFunctionReferenceNode)
	LibraryTemplateChanged, // The definition of a library node's template has changed (Subject == URigVMLibraryNode)
	FunctionAccessChanged, // The function has been made public/private (Subject == URigVMLibraryNode)
	VariantTagsChanged, // The tags in the header of this function variant have changed (Subject == URigVMLibraryNode)
	PinDisplayNameChanged, // The display name of a pin has changed - requiring a rebuild of the node user interface (Subject == URigVMPin) 
	PinCategoryChanged, // The category of a pin has changed - requiring a rebuild of the node user interface (Subject == URigVMPin)
	PinCategoriesChanged, // The category list of a node has changed - requiring a rebuild of the node user interface (Subject == URigVMNode)
	PinCategoryExpansionChanged, // The category of a pin expanded / collapsed (Subject == URigVMNode)
	FunctionVariantGuidChanged, // The guid for a function has changed (Subject == URigVMLibraryNode)
	NodeEarlyExitChanged, // The node's early exit state has changed
	LocalVariableDefaultValueChanged, // A local variable default value has been set (Subject == URigVMLibraryNode)
	LocalVariableAdded, // A local variable has been added to a function (Subject == URigVMLibraryNode)
	LocalVariableRemoved, // A local variable has been removed from a function (Subject == URigVMLibraryNode)
	LocalVariableRenamed, // A local variable has been renamed inside a function (Subject == URigVMLibraryNode)
	LocalVariableTypeChanged, // A local variable has been renamed inside a function (Subject == URigVMLibraryNode)
	LocalVariableIndexChanged, // A local variable's index has changed (Subject == URigVMLibraryNode)
	Invalid // The max for this enum (used for guarding)
};

// A delegate for subscribing / reacting to graph modifications.
DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FRigVMGraphModifiedEvent, ERigVMGraphNotifType /* type */, URigVMGraph* /* graph */, UObject* /* subject */);

// A dynamic delegate for subscribing / reacting to graph modifications (used for Python integration).
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FRigVMGraphModifiedDynamicEvent, ERigVMGraphNotifType, NotifType, URigVMGraph*, Graph, UObject*, Subject);