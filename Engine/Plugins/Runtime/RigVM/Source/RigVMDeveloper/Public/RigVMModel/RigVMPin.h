// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "UObject/ObjectMacros.h"
#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMByteCode.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMCompiler/RigVMASTProxy.h"
#include "UObject/StructOnScope.h"
#include <RigVMCore/RigVMTrait.h>
#include "RigVMModelCachedValue.h"
#include "RigVMCore/RigVMVariableDescription.h"
#include "RigVMPin.generated.h"

#define UE_API RIGVMDEVELOPER_API

class URigVMGraph;
class URigVMNode;
class URigVMUnitNode;
class URigVMPin;
class URigVMLink;
class URigVMVariableNode;

extern RIGVMDEVELOPER_API TAutoConsoleVariable<bool> CVarRigVMEnablePinOverrides;

/**
 * The Injected Info is used for injecting a node on a pin.
 * Injected nodes are not visible to the user, but they are normal
 * nodes on the graph.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMInjectionInfo: public UObject
{
	GENERATED_BODY()

public:

	URigVMInjectionInfo()
	{
		bInjectedAsInput = true;
	}

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<URigVMUnitNode> UnitNode_DEPRECATED;
#endif

	UPROPERTY()
	TObjectPtr<URigVMNode> Node;

	UPROPERTY()
	bool bInjectedAsInput;

	UPROPERTY()
	TObjectPtr<URigVMPin> InputPin;

	UPROPERTY()
	TObjectPtr<URigVMPin> OutputPin;

	// Returns the graph of this injected node.
	UFUNCTION(BlueprintCallable, Category = RigVMInjectionInfo)
	UE_API URigVMGraph* GetGraph() const;

	// Returns the pin of this injected node.
	UFUNCTION(BlueprintCallable, Category = RigVMInjectionInfo)
	UE_API URigVMPin* GetPin() const;

	struct FWeakInfo
	{
		TWeakObjectPtr<URigVMNode> Node;
		bool bInjectedAsInput;
		FName InputPinName;
		FName OutputPinName;
	};

	UE_API FWeakInfo GetWeakInfo() const;
};

UENUM()
enum class ERigVMPinDefaultValueType : uint8
{
	AutoDetect, // Detect if this is an unchanged or overridden value based on the delta
	Unset, // The value is unchanged and will remain the original default
	Override, // The value is overridden by the user and should stay like this no matter what
	KeepValueType, // The value type should be kept as well as the value itself (don't touch this)
};

/**
 * The Pin represents a single connector / pin on a node in the RigVM model.
 * Pins can be connected based on rules. Pins also provide access to a 'PinPath',
 * which essentially represents . separated list of names to reach the pin within
 * the owning graph. PinPaths are unique.
 * In comparison to the EdGraph Pin the URigVMPin supports the concept of 'SubPins',
 * so child / parent relationships between pins. A FVector Pin for example might
 * have its X, Y and Z components as SubPins. Array Pins will have its elements as
 * SubPins, and so on.
 * A URigVMPin is owned solely by a URigVMNode.
 */
UCLASS(MinimalAPI, BlueprintType)
class URigVMPin : public UObject
{
	GENERATED_BODY()

public:
	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;

	// A struct to store a pin override value
	struct FPinOverrideValue 
	{
		FPinOverrideValue()
			: DefaultValue()
			, BoundVariablePath()
		{}

		FPinOverrideValue(URigVMPin* InPin)
			: DefaultValue(InPin->GetDefaultValue())
			, BoundVariablePath(InPin->GetBoundVariablePath())
		{
		}

		FPinOverrideValue(URigVMPin* InPin, const TPair<FRigVMASTProxy, const TMap<FRigVMASTProxy, FPinOverrideValue>&>& InOverride)
			: DefaultValue(InPin->GetDefaultValue(InOverride))
			, BoundVariablePath(InPin->GetBoundVariablePath())
		{
		}

		FString DefaultValue;
		FString BoundVariablePath;
	};

	// A map used to override pin default values
	typedef TMap<FRigVMASTProxy, FPinOverrideValue> FPinOverrideMap;
	typedef TPair<FRigVMASTProxy, const FPinOverrideMap&> FPinOverride;
	static UE_API const URigVMPin::FPinOverrideMap EmptyPinOverrideMap;
	static UE_API const FPinOverride EmptyPinOverride;

	// Splits a PinPath at the start, so for example "Node.Color.R" becomes "Node" and "Color.R"
	static UE_API bool SplitPinPathAtStart(const FString& InPinPath, FString& LeftMost, FString& Right);
	static UE_API bool SplitPinPathAtStart(FStringView InPinPath, FStringView& LeftMost, FStringView& Right);

	// Splits a PinPath at the start, so for example "Node.Color.R" becomes "Node.Color" and "R"
	static UE_API bool SplitPinPathAtEnd(const FString& InPinPath, FString& Left, FString& RightMost);

	// Splits a PinPath into all segments, so for example "Node.Color.R" becomes ["Node", "Color", "R"]
	static UE_API bool SplitPinPath(const FString& InPinPath, TArray<FString>& Parts);

	// Joins a PinPath from to segments, so for example "Node.Color" and "R" becomes "Node.Color.R"
	static UE_API FString JoinPinPath(const FString& Left, const FString& Right);

	// Joins a PinPath from to segments, so for example ["Node", "Color", "R"] becomes "Node.Color.R"
	static UE_API FString JoinPinPath(const TArray<FString>& InParts);

	// Splits the default value into name-value pairs
	static UE_API TArray<FString> SplitDefaultValue(const FString& InDefaultValue);
	// Joins a collection of element DefaultValues into a default value for an array of those elements
	static UE_API FString GetDefaultValueForArray(TConstArrayView<FString> InDefaultValues);

	// Default constructor
	UE_API URigVMPin();

	// returns true if the name of this pin matches a given name
	UE_API bool NameEquals(FStringView InName, bool bFollowCoreRedirectors = false) const;

	// Returns a . separated path containing all names of the pin and its owners,
	// this includes the node name, for example "Node.Color.R"
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetPinPath(bool bUseNodePath = false) const;

	// Returns a . separated path containing all names of the pin and its owners
	// until we hit the provided parent pin.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetSubPinPath(const URigVMPin* InParentPin, bool bIncludeParentPinName = false) const;

	// Returns the category on a pin. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetCategory() const;

	// Returns index within a category on a pin. The category is UI relevant only and used
	// to order pins in the user interface of the node as well as on the details panel.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API int32 GetIndexInCategory() const;

	// Returns a . separated path containing all names of the pin within its main
	// memory owner / storage. This is typically used to create an offset pointer
	// within memory (FRigVMRegisterOffset).
	// So for example for a PinPath such as "Node.Transform.Translation.X" the 
	// corresponding SegmentPath is "Translation.X", since the transform is the
	// storage / memory.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetSegmentPath(bool bIncludeRootPin = false) const;

	// Populates an array of pins which will be reduced to the same operand in the
	// VM. This includes Source-Target pins in different nodes, pins in collapse and
	// referenced function nodes, and their corresponding entry and return nodes.
	UE_API void GetExposedPinChain(TArray<const URigVMPin*>& OutExposedPins) const;

	// Returns the display label of the pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FName GetDisplayName() const;

	// Returns the direction of the pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API ERigVMPinDirection GetDirection() const;

	// Returns true if the pin is currently expanded
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsExpanded() const;

	// Returns true if the pin is defined as a constant value / literal
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsDefinedAsConstant() const;

	// Returns true if the pin is defined as an input variable
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsDefinedAsInputVariable() const;

	// Returns the description of the input variable
	UE_API FRigVMGraphVariableDescription GetInputVariableDescription() const; 

	// Returns true if the pin should be watched
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool RequiresWatch(const bool bCheckExposedPinChain = false) const;
	
	// Returns true if the data type of the Pin is a enum
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsEnum() const;

	// Returns true if the data type of the Pin is a struct
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsStruct() const;

	// Returns true if the Pin is a SubPin within a struct
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsStructMember() const;

	// Returns true if the data type of the Pin is a uobject
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsUObject() const;

	// Returns true if the data type of the Pin is a interface
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsInterface() const;

	// Returns true if the data type of the Pin is an array
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsArray() const;

	// Returns true if the Pin is a SubPin within an array
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsArrayElement() const;

	// Returns true if this pin represents a dynamic array
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsDynamicArray() const;

	// Returns true if this data type is referenced counted
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	bool IsReferenceCountedContainer() const { return IsDynamicArray(); }

	// Returns true if this pin's value may be executed lazily
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsLazy() const;

	// Returns the index of the Pin within the node / parent Pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API int32 GetPinIndex() const;

	// Returns the absolute index of the Pin within the node / parent Pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API int32 GetAbsolutePinIndex() const;

	// Returns the number of elements within an array Pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API int32 GetArraySize() const;

	// Returns the C++ data type of the pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetCPPType() const;

	// Returns the C++ data type of an element of the Pin array
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetArrayElementCppType() const;

	// Returns the argument type this pin would represent within a template
	UE_API FRigVMTemplateArgumentType GetTemplateArgumentType() const;

	// Returns the argument type index this pin would represent within a template
	UE_API TRigVMTypeIndex GetTypeIndex() const;

	// Returns true if the C++ data type is FString or FName
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsStringType() const;

	// Returns true if the C++ data type is an execute context
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsExecuteContext() const;

	// Returns true if the C++ data type is unknown
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsWildCard() const;

	// Returns true if any of the subpins is a wildcard
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool ContainsWildCardSubPin() const;

	// Returns true if this pin is an array that should be displayed as elements only
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsFixedSizeArray() const;

	// Returns true if this pin is an array that should be displayed as elements only
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool ShouldOnlyShowSubPins() const;

	// Returns true if this pin's subpins should be hidden in the UI
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool ShouldHideSubPins() const;

	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetOriginalDefaultValue() const;

	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool HasOriginalDefaultValue() const;
	
	// Returns the default value of the Pin as a string.
	// Note that this value is computed based on the Pin's
	// SubPins - so for example for a FVector typed Pin
	// the default value is actually composed out of the
	// default values of the X, Y and Z SubPins.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetDefaultValue() const;

	// Returns the default value with an additional override ma
	UE_API FString GetDefaultValue(const FPinOverride& InOverride, bool bAdaptValueForPinType = true) const;

	// Returns the default value object if this pin represents a UObject
	UE_API UObject* GetDefaultValueObject() const;

	// Returns the default value as stored by the user.
	UE_API FString GetDefaultValueStoredByUserInterface() const;

	// Returns true if the default value provided is valid
	UFUNCTION(BlueprintPure, Category = RigVMPin)
	UE_API bool IsValidDefaultValue(const FString& InDefaultValue) const;

	// Returns true if the default value was ever changed by the user
	UE_DEPRECATED(5.6, "Please use URigVMPin::HasDefaultValueOverride()")
	UFUNCTION(BlueprintPure, Category = RigVMPin)
	UE_API bool HasUserProvidedDefaultValue() const;

	// Returns true if the default value was ever changed by the user
    UFUNCTION(BlueprintPure, Category = RigVMPin)
	UE_API bool HasDefaultValueOverride() const;

	// Returns true if the pin can / may provide a default value 
	UFUNCTION(BlueprintPure, Category = RigVMPin)
	ERigVMPinDefaultValueType GetDefaultValueType() const { return DefaultValueType; }

	// Returns true if the pin can / may provide a default value 
	UFUNCTION(BlueprintPure, Category = RigVMPin)
	UE_API bool CanProvideDefaultValue() const;

	// Returns the default value clamped with the limit meta values defined by the UPROPERTY in URigVMUnitNodes 
	UE_API FString ClampDefaultValueFromMetaData(const FString& InDefaultValue) const;

	// Returns the keyed metadata associated with this pin, if any
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FString GetMetaData(FName InKey) const;

	// Returns whether the keyed metadata associated with this pin exists (can be empty)
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool HasMetaData(FName InKey) const;
	
	// Returns the name of a custom widget to be used
	// for editing the Pin.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FName GetCustomWidgetName() const;

	// Returns the tooltip of this pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API FText GetToolTipText() const;

	// Returns the struct of the data type of the Pin,
	// or nullptr otherwise.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API UObject* GetCPPTypeObject() const;

	// Returns the struct of the data type of the Pin,
	// or nullptr otherwise.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API UScriptStruct* GetScriptStruct() const;

	// Returns the parent struct of the data type of the Pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API UScriptStruct* GetParentScriptStruct(const URigVMUnitNode* FallbackNode) const;

	// Returns the enum of the data type of the Pin,
	// or nullptr otherwise.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API UEnum* GetEnum() const;

	// Returns the parent Pin - or nullptr if the Pin
	// is nested directly below a node.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMPin* GetParentPin() const;

	// Returns the top-most parent Pin, so for example
	// for "Node.Transform.Translation.X" this returns
	// the Pin for "Node.Transform".
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMPin* GetRootPin() const;

	// Returns true if this pin is a root pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsRootPin() const;

	// Returns the pin to be used for a link.
	// This might differ from this actual pin, since
	// the pin might contain injected nodes.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMPin* GetPinForLink() const;

	// Returns the link that represents the connection
	// between this pin and InOtherPin. nullptr is returned
	// if the pins are not connected.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMLink* FindLinkForPin(const URigVMPin* InOtherPin) const;

	// Returns the original pin for a pin on an injected
	// node. This can be used to determine where a link
	// should go in the user interface
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMPin* GetOriginalPinFromInjectedNode() const;

	// Returns all of the SubPins of this one.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API const TArray<URigVMPin*>& GetSubPins() const;

	// Returns all of the SubPins of this one including sub-sub-pins
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API TArray<URigVMPin*> GetAllSubPinsRecursively() const;

	// Returns a SubPin given a name / path or nullptr.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMPin* FindSubPin(const FString& InPinPath) const;
	UE_API URigVMPin* FindSubPinView(FStringView InPinPath) const;

	// Returns true if this Pin is linked to another Pin
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API bool IsLinkedTo(const URigVMPin* InPin) const;

	// Returns true if the pin has any link
	UE_API bool IsLinked(bool bRecursive = false) const;

	// Returns all of the links linked to this Pin.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API const TArray<URigVMLink*>& GetLinks() const;

	// Returns all of the linked source Pins,
	// using this Pin as the target.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API TArray<URigVMPin*> GetLinkedSourcePins(bool bRecursive = false) const;

	// Returns all of the linked target Pins,
	// using this Pin as the source.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API TArray<URigVMPin*> GetLinkedTargetPins(bool bRecursive = false) const;

	// Returns all of the source pins
	// using this Pin as the target.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API TArray<URigVMLink*> GetSourceLinks(bool bRecursive = false) const;

	// Returns all of the target links,
	// using this Pin as the source.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API TArray<URigVMLink*> GetTargetLinks(bool bRecursive = false) const;

	// Returns the node of this Pin.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMNode* GetNode() const;

	// Returns the graph of this Pin.
	UFUNCTION(BlueprintCallable, Category = RigVMPin)
	UE_API URigVMGraph* GetGraph() const;

	// Returns true is the two provided source and target Pins
	// can be linked to one another.
	static UE_API bool CanLink(const URigVMPin* InSourcePin, const URigVMPin* InTargetPin, FString* OutFailureReason, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection = ERigVMPinDirection::IO, bool bInAllowWildcard = false, bool bEnableTypeCasting = true);

	// Returns true if this pin has injected nodes
	bool HasInjectedNodes() const { return InjectionInfos.Num() > 0; }

	// Returns true if this pin has injected nodes
	UE_API bool HasInjectedUnitNodes() const;

	// Returns the injected nodes this pin contains.
	const TArray<URigVMInjectionInfo*> GetInjectedNodes() const { return InjectionInfos; }

	UE_API URigVMVariableNode* GetBoundVariableNode() const;

	// Returns the variable bound to this pin (or NAME_None)
	UE_API const FString GetBoundVariablePath() const;

	// Returns the variable bound to this pin (or NAME_None)
	UE_API const FString GetBoundVariablePath(const FPinOverride& InOverride) const;

	// Returns the variable bound to this pin (or NAME_None)
	UE_API FString GetBoundVariableName() const;

	// Returns true if this pin is bound to a variable
	UE_API bool IsBoundToVariable() const;

	// Returns true if this pin is bound to a variable
	UE_API bool IsBoundToVariable(const FPinOverride& InOverride) const;

	// Returns true if this pin is bound to an external variable
	UE_API bool IsBoundToExternalVariable() const;

	// Returns true if this pin is bound to a local variable
	UE_API bool IsBoundToLocalVariable() const;

	// Returns true if this pin is bound to an input argument
	UE_API bool IsBoundToInputArgument() const;

	// Returns true if the pin can be bound to a given variable
	UE_API bool CanBeBoundToVariable(const FRigVMExternalVariable& InExternalVariable, const FString& InSegmentPath = FString()) const;

	// Returns true if the pin should not show up on a node, but in the details panel
	UE_API bool ShowInDetailsPanelOnly() const;

	// Returns an external variable matching this pin's type
	UE_API FRigVMExternalVariable ToExternalVariable() const;

	// Returns true if the pin has been orphaned
	UE_API bool IsOrphanPin() const;

	UE_API uint32 GetStructureHash() const;

	// Returns true if this pin represents a trait 
	UFUNCTION(BlueprintPure, Category = RigVMPin)
	UE_API bool IsTraitPin() const;

	// Returns true if this pin represents a trait's programmatic pin
	UE_API bool IsProgrammaticPin() const;

	// Get all the sub-pins that are programmatic
	UE_API TArray<URigVMPin*> GetProgrammaticSubPins() const;

	// Returns the trait backing up this pin
	UE_API TSharedPtr<FStructOnScope> GetTraitInstance(bool bUseDefaultValueFromPin = true) const;

	// Returns the struct of the trait backing up this pin
	UE_API UScriptStruct* GetTraitScriptStruct() const;

	UE_API const uint32& GetNodeCachedValueVersion() const;
	UE_API const uint32& GetCachedValueVersion() const;

private:

	UE_API void UpdateTypeInformationIfRequired() const;
	UE_API void SetNameFromIndex();
	UE_API void SetDisplayName(const FName& InDisplayName);
	UE_API void IncrementVersion(bool bAffectParentPin = true, bool bAffectSubPins = true);

	UE_API void GetExposedPinChainImpl(TArray<const URigVMPin*>& OutExposedPins, TArray<const URigVMPin*>& VisitedPins) const;
	
	UE_API void ClearPinPathCache() const;

	UPROPERTY()
	FName DisplayName;

	// if new members are added to the pin in the future 
	// it is important to search for all existing usages of all members
	// to make sure things are copied/initialized properly
	
	UPROPERTY()
	ERigVMPinDirection Direction;

	UPROPERTY()
	bool bIsExpanded;

	UPROPERTY()
	bool bIsConstant;

	UPROPERTY()
	bool bIsInputVariable;

	UPROPERTY(transient)
	bool bRequiresWatch;

	UPROPERTY()
	bool bIsDynamicArray;

	UPROPERTY()
	bool bIsLazy;

	UPROPERTY()
	FString CPPType;

	// serialize object ptr here to keep track of the latest version of the type object,
	// type object can reference assets like user defined struct, which can be renamed
	// or moved to new locations, serializing the type object with the pin
	// ensure automatic update whenever those things happen
	UPROPERTY()
	TObjectPtr<UObject> CPPTypeObject;

	UPROPERTY()
	FName CPPTypeObjectPath;

	UPROPERTY()
	FString DefaultValue;

	/** If the default value for this pin should be an object, we store a pointer to it */
	UPROPERTY()
	TObjectPtr<UObject> DefaultValueObject;

	UPROPERTY()
	ERigVMPinDefaultValueType DefaultValueType;

	UPROPERTY()
	FName CustomWidgetName;

	UPROPERTY()
	TArray<TObjectPtr<URigVMPin>> SubPins;

	UPROPERTY(transient)
	TArray<TObjectPtr<URigVMLink>> Links;

	UPROPERTY()
	TArray<TObjectPtr<URigVMInjectionInfo>> InjectionInfos;

	UPROPERTY()
	FString UserDefinedCategory;

	UPROPERTY()
	int32 IndexInCategory;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString BoundVariablePath_DEPRECATED;
#endif

	uint32 PinVersion;
	mutable uint32 CombinedPinVersion;
	
	mutable FString LastKnownCPPType;
	mutable TRigVMTypeIndex LastKnownTypeIndex;
	mutable TRigVMModelCachedValue<URigVMPin, bool> CachedIsStringType;
	mutable TRigVMModelCachedValue<URigVMPin, FString> CachedDefaultValue;
	mutable TRigVMModelCachedValue<URigVMPin, FString> CachedAdaptedDefaultValue;
	mutable TRigVMModelCachedValue<URigVMPin, uint32> CachedCPPTypeObjectHash;
	mutable TRigVMModelCachedValue<URigVMPin, bool> CachedShowInDetailsPanelOnly;
	mutable TRigVMModelCachedValue<URigVMPin, FString> CachedPinPath;
	mutable TRigVMModelCachedValue<URigVMPin, FString> CachedPinPathWithNodePath;
	mutable TRigVMModelCachedValue<URigVMPin, FString> CachedPinCategory;
	mutable TRigVMModelCachedValue<URigVMPin, FName> CachedDisplayName;
	mutable TRigVMModelCachedValue<URigVMPin, bool> CachedDefaultValueOverride;
	mutable TRigVMModelCachedValue<URigVMPin, bool> CachedHasOriginalDefaultValue;

	static const inline TCHAR* OrphanPinPrefix = TEXT("Orphan::");

	friend class URigVMController;
	friend class URigVMGraph;
	friend class URigVMNode;
	friend class URigVMLink;
	friend class FRigVMParserAST;
	friend struct FRigVMSetPinDisplayNameAction;
	friend struct FRigVMSetPinCategoryAction;
	friend class URigVMLibraryNode;
	friend class URigVMEdGraphNode;
	friend struct FRigVMClient;
	friend class SRigVMGraphNode;
	friend struct FRigVMPinInfoArray;
};

class FRigVMPinDefaultValueImportErrorContext : public FOutputDevice
{
public:

	int32 NumErrors;

	FRigVMPinDefaultValueImportErrorContext( ELogVerbosity::Type InMaxVerbosity = ELogVerbosity::Warning )
		: FOutputDevice()
		, NumErrors(0)
		, MaxVerbosity(InMaxVerbosity)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if(Verbosity <= MaxVerbosity)
		{
			NumErrors++;
		}
	}

	ELogVerbosity::Type GetMaxVerbosity() const { return MaxVerbosity; }

private:
	ELogVerbosity::Type MaxVerbosity;
};

#undef UE_API
