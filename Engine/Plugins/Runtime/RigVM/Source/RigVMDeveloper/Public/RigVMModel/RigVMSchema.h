// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMCore/RigVMRegistry.h"
#include "RigVMCore/RigVMFunction.h"
#include "RigVMCore/RigVMTemplate.h"
#include "RigVMCore/RigVMDispatchFactory.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMGraph.h"
#include "RigVMSchema.generated.h"

#define UE_API RIGVMDEVELOPER_API

#define RIGVMSCHEMA_DEFAULT_FUNCTION_BODY \
const URigVMGraph* Graph = nullptr; \
if (InController) \
{ \
	Graph = InController->GetGraph(); \
	check(Graph); \
	if (!InController->IsTransacting() && !IsGraphEditable(Graph)) \
	{ \
		return false; \
	} \
}

class URigVMController;

/**
 * The Schema is used to determine which actions are allowed
 * on a graph. This includes any topological change.
 */
UCLASS(MinimalAPI)
class URigVMSchema : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	// Default constructor
	UE_API URigVMSchema();

	// Returns the execute context struct this schema is using
	UScriptStruct* GetExecuteContextStruct() const { return ExecuteContextStruct; }

	// Returns the corresponding Ed Graph Schema class
	UE_API virtual UClass* GetEdGraphSchemaClass() const;

	// Returns the corresponding runtime asset class to use for this schema
	UE_API virtual UClass* GetRuntimeAssetClass() const;

	// Returns the corresponding editor asset class to use for this schema
	UE_API virtual UClass* GetEditorAssetClass() const;

	// Returns true if a graph supports a given type
	UE_API bool SupportsType(URigVMController* InController, TRigVMTypeIndex InTypeIndex) const;
	UE_API virtual bool SupportsType_NoLock(URigVMController* InController, TRigVMTypeIndex InTypeIndex, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports a given unit function
	UE_API bool SupportsUnitFunction(URigVMController* InController, const FRigVMFunction* InUnitFunction) const;
	UE_API virtual bool SupportsUnitFunction_NoLock(URigVMController* InController, const FRigVMFunction* InUnitFunction, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports a given dispatch factory
	UE_API bool SupportsDispatchFactory(URigVMController* InController, const FRigVMDispatchFactory* InDispatchFactory) const;
	UE_API virtual bool SupportsDispatchFactory_NoLock(URigVMController* InController, const FRigVMDispatchFactory* InDispatchFactory, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports a given template
	UE_API bool SupportsTemplate(URigVMController* InController, const FRigVMTemplate* InTemplate) const;
	UE_API virtual bool SupportsTemplate_NoLock(URigVMController* InController, const FRigVMTemplate* InTemplate, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports a given graph based function
	UE_API bool SupportsGraphFunction(URigVMController* InController, const FRigVMGraphFunctionHeader* InGraphFunction) const;
	UE_API virtual bool SupportsGraphFunction_NoLock(URigVMController* InController, const FRigVMGraphFunctionHeader* InGraphFunction, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports a given external variable
	UE_API bool SupportsExternalVariable(URigVMController* InController, const FRigVMExternalVariable* InExternalVariable) const;
	UE_API virtual bool SupportsExternalVariable_NoLock(URigVMController* InController, const FRigVMExternalVariable* InExternalVariable, FRigVMRegistryHandle& InRegistry) const;

	// Returns true if a graph supports pin categories / node layouts
	UE_API virtual bool SupportsNodeLayouts(const URigVMGraph* InGraph) const;

	// Returns true if the pin for a given struct should be unfolded into subpins for a graph
	UE_API virtual bool ShouldUnfoldStruct(URigVMController* InController, const UStruct* InStruct) const;

	// Returns true if a node name is valid for a given graph
	UE_API virtual bool IsValidNodeName(const URigVMGraph* InGraph, const FName& InNodeName) const;

	// Returns true if a given node can be added the graph
	UE_API virtual bool CanAddNode(URigVMController* InController, const URigVMNode* InNode) const;

	// Returns true if a given node can be removed from the graph
	UE_API virtual bool CanRemoveNode(URigVMController* InController, const URigVMNode* InNode) const;

	// Returns true if a given node can be added the graph
	UE_API virtual bool CanRenameNode(URigVMController* InController, const URigVMNode* InNode, const FName& InNewNodeName) const;

	// Returns true if a node can moved to a new position
	UE_API virtual bool CanMoveNode(URigVMController* InController, const URigVMNode* InNode, const FVector2D& InNewPosition = FVector2D::ZeroVector) const;

	// Returns true if a node can resized to a new size
	UE_API virtual bool CanResizeNode(URigVMController* InController, const URigVMNode* InNode, const FVector2D& InNewSize = {100, 100}) const;

	// Returns true if a node supports setting a node title
	UE_API virtual bool CanSetNodeTitle(URigVMController* InController, const URigVMNode* InNode) const;

	// Returns true if a node can recolored to a new color
	UE_API virtual bool CanRecolorNode(URigVMController* InController, const URigVMNode* InNode, const FLinearColor& InNewColor = FLinearColor::White) const;

	// Returns true if a link can be added between two pins
	UE_API virtual bool CanAddLink(URigVMController* InController, const URigVMPin* InSourcePin, const URigVMPin* InTargetPin, const FRigVMByteCode* InByteCode, ERigVMPinDirection InUserLinkDirection = ERigVMPinDirection::IO, bool bInAllowWildcard = false, bool bEnableTypeCasting = true, FString* OutFailureReason = nullptr) const;

	// Returns true if a link exists between two pins and can be broken / removed
	UE_API virtual bool CanBreakLink(URigVMController* InController, const URigVMPin* InSourcePin, const URigVMPin* InTargetPin) const;

	// Returns true if a set of nodes can be collapsed
	UE_API virtual bool CanCollapseNodes(URigVMController* InController, const TArrayView<URigVMNode* const>& InNodesToCollapse) const;

	// Returns true if a node can be expanded within a graph
	UE_API virtual bool CanExpandNode(URigVMController* InController, const URigVMNode* InNodeToExpand) const;

	// Returns true if a pin should be unfolded (represented by its subpins)
	UE_API virtual bool CanUnfoldPin(URigVMController* InController, const URigVMPin* InPinToUnfold) const;

	// Returns true if a variable can be bound to a pin
	UE_API virtual bool CanBindVariable(URigVMController* InController, const URigVMPin* InPinToBind, const FRigVMExternalVariable* InVariableToBind, const FString& InNewBoundVariablePath) const;

	// Returns true if a variable can be unbound from a pin
	UE_API virtual bool CanUnbindVariable(URigVMController* InController, const URigVMPin* InBoundPin) const;

	// Returns true if functions definitions (not refs) can be added to the graph
	UE_API virtual bool CanAddFunction(URigVMController* InController, const URigVMNode* InFunctionNode) const;

	// Returns true if a function can be removed
	UE_API virtual bool CanRemoveFunction(URigVMController* InController, const URigVMNode* InFunctionNode) const;
	
	static int32 GetMaxNameLength() { return 100; }
	UE_API virtual FString GetSanitizedName(const FString& InName, bool bAllowPeriod, bool bAllowSpace) const;
	UE_API virtual FString GetSanitizedGraphName(const FString& InName) const;
	UE_API virtual FString GetSanitizedNodeName(const FString& InName) const;
	UE_API virtual FString GetSanitizedVariableName(const FString& InName) const;
	UE_API virtual FString GetSanitizedPinName(const FString& InName) const;
	UE_API virtual FString GetSanitizedPinPath(const FString& InName) const;
	UE_API virtual FString GetGraphOuterName(const URigVMGraph* InGraph) const;
	UE_API virtual	FString GetValidNodeName(const URigVMGraph* InGraph, const FString& InPrefix) const;
	static UE_API void SanitizeName(FString& InOutName, bool bAllowPeriod, bool bAllowSpace);

	// Returns a unique name based on a IsNameAvailable predicate
	static UE_API FName GetUniqueName(const FName& InName, TFunction<bool(const FName&)> IsNameAvailablePredicate, bool bAllowPeriod, bool bAllowSpace);

protected:

	// Sets the execute context struct this schema is using
	UE_API void SetExecuteContextStruct(UScriptStruct* InExecuteContextStruct);

	UE_API bool IsGraphEditable(const URigVMGraph* InGraph) const;
	UE_API TObjectPtr<URigVMNode> FindEventNode(URigVMController* InController, const UScriptStruct* InScriptStruct) const;
	
	
	virtual bool ShouldPatchGraphSchema() const { return true; }

	UPROPERTY(transient)
	TObjectPtr<UScriptStruct> ExecuteContextStruct;
	TArray<UStruct*> ValidExecuteContextStructs;
	FRigVMRegistry* Registry;

	/*
	bool bIsTransacting; // Performing undo/redo transaction
	bool bAllowPrivateFunctions;
	bool bIgnoreFunctionEntryReturnNodes;
	mutable FString LastError;
	mutable FString LastWarning;
	mutable FString LastErrorOrWarning;
	*/

	friend struct FRigVMClient;
	friend class URigVMController;
	friend struct FRigVMBaseAction;
};

#undef UE_API
