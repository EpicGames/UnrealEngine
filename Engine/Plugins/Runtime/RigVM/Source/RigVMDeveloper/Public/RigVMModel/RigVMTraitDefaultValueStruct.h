// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "StructUtils/PropertyBag.h"
#include "RigVMTraitDefaultValueStruct.generated.h"

USTRUCT()
struct FRigVMTraitDefaultValueStruct
{
	GENERATED_BODY()

	static const TCHAR* DefaultValuePropertyName;

	UPROPERTY()
	FInstancedPropertyBag PropertyBag;
	
	void Init(UScriptStruct* InTraitScriptStruct);
	void SetValue(const FString& InDefaultValue);
	FString GetValue() const;
};