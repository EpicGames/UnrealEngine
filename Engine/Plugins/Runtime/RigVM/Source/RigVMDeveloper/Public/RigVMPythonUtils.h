// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "Math/Transform.h"
#include "Math/Vector2D.h"
#include "RigVMDeveloperModule.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"

class UEnum;

namespace RigVMPythonUtils
{
	/** How should PythonizeName adjust the final name? */
	enum EPythonizeNameCase : uint8
	{
		/** lower_snake_case */
		Lower,
		/** UPPER_SNAKE_CASE */
		Upper,
	};
	
	RIGVMDEVELOPER_API FString PythonizeName(FStringView InName, const EPythonizeNameCase InNameCase = EPythonizeNameCase::Lower);

	RIGVMDEVELOPER_API FString TransformToPythonString(const FTransform& Transform);

	RIGVMDEVELOPER_API FString Vector2DToPythonString(const FVector2D& Vector);

	RIGVMDEVELOPER_API FString LinearColorToPythonString(const FLinearColor& Color);

	RIGVMDEVELOPER_API FString EnumValueToPythonString(UEnum* Enum, int64 Value);
	
	template<typename T>
	FString EnumValueToPythonString(int64 Value)
	{
		return EnumValueToPythonString(StaticEnum<T>(), Value);
	}
	
#if WITH_EDITOR
	RIGVMDEVELOPER_API void Print(const FString& BlueprintTitle, const FString& InMessage);

	RIGVMDEVELOPER_API void PrintPythonContext(const FString& InBlueprintName);

#endif
}
