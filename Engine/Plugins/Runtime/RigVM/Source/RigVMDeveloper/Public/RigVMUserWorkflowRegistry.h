// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "RigVMCore/RigVMUserWorkflow.h"
#include "RigVMModel/RigVMNode.h"
#include "Templates/Tuple.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include "RigVMUserWorkflowRegistry.generated.h"

#define UE_API RIGVMDEVELOPER_API

class UScriptStruct;
struct FFrame;

DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(TArray<FRigVMUserWorkflow>, FRigVMUserWorkflowProvider, const UObject*, InSubject);

UCLASS(MinimalAPI, BlueprintType)
class URigVMUserWorkflowRegistry : public UObject
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = FRigVMUserWorkflowRegistry)
   	static UE_API URigVMUserWorkflowRegistry* Get();

	UFUNCTION(BlueprintCallable, Category = FRigVMUserWorkflowRegistry)
	UE_API int32 RegisterProvider(const UScriptStruct* InStruct, FRigVMUserWorkflowProvider InProvider);

	UFUNCTION(BlueprintCallable, Category = FRigVMUserWorkflowRegistry)
	UE_API int32 RegisterProviderForFunction(const FString InFunctionNotation, FRigVMUserWorkflowProvider InProvider);

	UFUNCTION(BlueprintCallable, Category = FRigVMUserWorkflowRegistry)
	UE_API void UnregisterProvider(int32 InHandle);

	UFUNCTION(BlueprintPure, Category = FRigVMUserWorkflowRegistry)
	UE_API TArray<FRigVMUserWorkflow> GetWorkflows(ERigVMUserWorkflowType InType, const UScriptStruct* InStruct, const UObject* InSubject) const;

private:

	int32 MaxHandle = 0;
	mutable TArray<TTuple<int32, const UScriptStruct*, FRigVMUserWorkflowProvider>> ProvidersForUnits;
	mutable TArray<TTuple<int32, const FString, FRigVMUserWorkflowProvider>> ProvidersForDispatchFunctions;
};

#undef UE_API
