// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class RigVMDeveloper : ModuleRules
{
    public RigVMDeveloper(ReadOnlyTargetRules Target) : base(Target)
    {
	    CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
                "RigVM",
                "VisualGraphUtils",
                "KismetCompiler",
                "Projects",
                "Json",
                "Scribble",
                "AssetTools",
                "AnimationCore"
            }
        );

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"EditorFramework",
                    "UnrealEd",
					"Slate",
					"SlateCore",
					"MessageLog",
					"BlueprintGraph",
					"GraphEditor",
					"ApplicationCore",
                }
			);

            bool bAddBlueprintEditorDependency = true;
            PublicDefinitions.Add("WITH_RIGVMLEGACYEDITOR=" + (bAddBlueprintEditorDependency ? '1' : '0'));
            if (bAddBlueprintEditorDependency)
            {
	            PrivateDependencyModuleNames.Add("Kismet");
            }
            
            PrivateIncludePathModuleNames.Add("RigVMEditor");
            DynamicallyLoadedModuleNames.Add("RigVMEditor");
        }
        PrivateIncludePaths.Add(Path.Combine(EngineDirectory,"Source/ThirdParty/inja/inja-3.5.0/single_include"));
        PrivateIncludePaths.Add(Path.Combine(EngineDirectory,"Source/ThirdParty/inja/inja-3.5.0/third_party/include"));
    }
}
