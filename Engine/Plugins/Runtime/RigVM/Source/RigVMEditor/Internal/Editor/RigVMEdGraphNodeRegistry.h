// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "Templates/SharedPointer.h"
#include "UObject/ScriptInterface.h"
#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtr.h"

class IRigVMEditorAssetInterface;
class UClass;
class UObject;
class URigVMEdGraphNode;
class URigVMGraph;
enum class ERigVMGraphNotifType : uint8;

#define UE_API RIGVMEDITOR_API

namespace UE::RigVMEditor
{
	/** Template to track existence of Rig VM Ed Graph Nodes for nodes of a specific Rig VM Node type in editor. */
	class FRigVMEdGraphNodeRegistry
		: public TSharedFromThis<FRigVMEdGraphNodeRegistry>
	{
		struct FPrivateToken { explicit FPrivateToken() = default; };

	public:
		FRigVMEdGraphNodeRegistry() = default;
		
		/** Constructor, not ment to be used, instead use GetOrCreateRegistry to create instances. */
		FRigVMEdGraphNodeRegistry(const TScriptInterface<IRigVMEditorAssetInterface>& InRigVMAssetInterface, UClass* InRigVMNodeClass, FPrivateToken);

		/** 
		 * Creates a registry instance. 
		 * Note, The registry will be destroyed when the all references are released.
		 *
		 * @param InRigVMAssetInterface			The Asset for which this registry is constructed 
		 * @param InRigVMNodeClass				The Rig VM Node Class for which Ed Graph nodes are held
		 * @param bForceUpdate					(Optional) If set to true, the registry instantly updates.
		 * 
		 * @return								The registry instance
		 */
		UE_API static TSharedRef<FRigVMEdGraphNodeRegistry> GetOrCreateRegistry(
			const TScriptInterface<IRigVMEditorAssetInterface>& InRigVMAssetInterface, 
			UClass* InRigVMNodeClass,
			const bool bForceUpdate = false);

		/** Returns the currently registered ed graph nodes which correspond to the node type and have connected pins */
		UE_API const TArray<TWeakObjectPtr<URigVMEdGraphNode>>& GetConnectedEdGrapNodes() const { return WeakConnectedEdGraphNodes; }

		/** Returns the currently registered ed graph nodes which correspond to the node type and do not have any connected pins */
		UE_API const TArray<TWeakObjectPtr<URigVMEdGraphNode>>& GetDisconnectedEdGrapNodes() const { return WeakDisconnectedEdGraphNodes; }

		/** Delegate broadcasted when the registry was updated */
		FSimpleMulticastDelegate OnPostRegistryUpdated;

		/** Returns the node type this registry is tracking */
		const UClass* GetType() const { return RigVMNodeClass; }

	private:
		/** Initializes the registry */
		void Initialize();

		/** Requests to Update the registry */
		void RequestUpdate();

		/**
		 * Forces the registry to update. Mind the registry tracks changes and notifies via OnPostRegistryUpdated.
		 * ForceUpdate only needs to be called where handling changes via OnPostRegistryUpdated is not adequate.
		 */
		void ForceUpdate();

		/** Called after a graph was modified */
		void PostGraphModified(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

		/** Returns true if the node is considered connected */
		bool IsNodeConnected(const URigVMEdGraphNode& EdGraphNode, const bool bOnlyConsiderExecPin) const;

		/** The currently registered nodes that have any pin connected */
		TArray<TWeakObjectPtr<URigVMEdGraphNode>> WeakConnectedEdGraphNodes;

		/** The currently registered nodes that do not have any pin connected */
		TArray<TWeakObjectPtr<URigVMEdGraphNode>> WeakDisconnectedEdGraphNodes;

		/** The asset interface this registry refers to */
		TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakAssetInterface;

		/** The Rig VM Node type that is tracked in this registry */
		UClass* RigVMNodeClass = nullptr;

		/** Timer handle to notify when the registry was fully updated */
		FTimerHandle RequestUpdateTimerHandle;

		/** Map of registry IDs with their unique registry, useful to reuse registries of same type and asset */
		static TMap<FName, TWeakPtr<FRigVMEdGraphNodeRegistry>> TypeIDToRegistryMap;
	};
}

#undef UE_API
