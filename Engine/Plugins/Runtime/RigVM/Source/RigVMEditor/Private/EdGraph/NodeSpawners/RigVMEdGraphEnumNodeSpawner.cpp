// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphEnumNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "BlueprintNodeTemplateCache.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"

#include "RigVMModel/Nodes/RigVMEnumNode.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphEnumNodeSpawner"

URigVMEdGraphEnumNodeSpawner::URigVMEdGraphEnumNodeSpawner(UEnum* InEnum, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
{
	NodeClass = URigVMEdGraphNode::StaticClass();
	Enum = InEnum;

	MenuName = InMenuDesc;
	MenuTooltip  = InTooltip;
	MenuCategory = InCategory;
	MenuKeywords = FText::FromString(TEXT("Enum"));
	MenuIcon = FSlateIcon(TEXT("RigVMEditorStyle"), TEXT("RigVM.Unit"));
}

FString URigVMEdGraphEnumNodeSpawner::GetSpawnerSignature() const
{
	return FString("RigUnit=" + Enum->GetFName().ToString());
}

URigVMEdGraphNode* URigVMEdGraphEnumNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);
	if (bIsTemplateNode)
	{
		TArray<FPinInfo> Pins;
		Pins.Emplace(URigVMEnumNode::EnumValueName, ERigVMPinDirection::Output, RigVMTypeUtils::Int32TypeName, nullptr);
		return SpawnTemplateNode(ParentGraph, Pins);
	}

	// First create a backing member for our node
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(RigGraph == nullptr) return nullptr;
	FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
	check(RigBlueprint);

#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	URigVMController* Controller = RigBlueprint->GetController(ParentGraph);

	Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), URigVMEnumNode::EnumName));

	if (URigVMEnumNode* ModelNode = Controller->AddEnumNode(*Enum->GetPathName(), Location, URigVMEnumNode::EnumName, true, true))
	{
		NewNode = Cast<URigVMEdGraphNode>(RigGraph->FindNodeForModelNodeName(ModelNode->GetFName()));

		if (NewNode)
		{
			Controller->ClearNodeSelection(true);
			Controller->SelectNode(ModelNode, true, true);
		}

		Controller->CloseUndoBracket();
	}
	else
	{
		Controller->CancelUndoBracket();
	}


	return NewNode;
}

#undef LOCTEXT_NAMESPACE

