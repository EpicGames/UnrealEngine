// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphFunctionRefNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/EditorEngine.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "K2Node_Variable.h"
#include "BlueprintNodeTemplateCache.h"
#include "RigVMBlueprintUtils.h"
#include "ScopedTransaction.h"
#include "Editor/RigVMEditorTools.h"

#if WITH_EDITOR
#include "Editor.h"
#include "SGraphActionMenu.h"
#include "GraphEditorSettings.h" 
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphFunctionRefNodeSpawner"

// 3 possible creation types:
// - CreateFromFunction(URigVMLibraryNode) --> This is a local function. Valid Header.
// - CreateFromAssetData(const FAssetData& InAssetData, const FRigVMGraphFunctionHeader& InPublicFunction) --> Public function. Valid Header.
// - CreateFromAssetData(const FAssetData& InAssetData, const FRigVMOldPublicFunctionData& InPublicFunction) --> Public function. Valid AssetData and Header.Name

URigVMEdGraphFunctionRefNodeSpawner::URigVMEdGraphFunctionRefNodeSpawner(URigVMLibraryNode* InFunction)
{
	check(InFunction);

	ReferencedPublicFunctionHeader = InFunction->GetFunctionHeader();
	NodeClass = URigVMEdGraphNode::StaticClass();
	bIsLocalFunction = true;
	FunctionPath = InFunction->GetPathName();

	const FString Category = InFunction->GetNodeCategory();

	MenuName = FText::FromString(InFunction->GetName());
	MenuTooltip = InFunction->GetToolTipText();

	static const FString LocalFunctionString = TEXT("Local Function");
	if(MenuTooltip.IsEmpty())
	{
		MenuTooltip = FText::FromString(LocalFunctionString);
	}
	else
	{
		static constexpr TCHAR Format[] = TEXT("%s\n\n%s");
		MenuTooltip = FText::FromString(FString::Printf(Format, *MenuTooltip.ToString(), *LocalFunctionString));
	}
	
	MenuCategory = FText::FromString(Category);
	MenuKeywords = FText::FromString(InFunction->GetNodeKeywords());

	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	if (MenuKeywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuKeywords = FText::FromString(TEXT(" "));
	}

	MenuIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");

#if WITH_EDITOR
	if (InFunction->IsMutable())
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
	}
	else
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->PureFunctionCallNodeTitleColor;
	}
#endif
}

URigVMEdGraphFunctionRefNodeSpawner::URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMGraphFunctionHeader& InPublicFunction)
{
	ReferencedPublicFunctionHeader = InPublicFunction;
	NodeClass = URigVMEdGraphNode::StaticClass();
	bIsLocalFunction = false;
	AssetPath = InAssetData.ToSoftObjectPath(); 	

	MenuName = FText::FromName(InPublicFunction.Name);
	MenuCategory = FText::FromString(InPublicFunction.Category);
	MenuKeywords = FText::FromString(InPublicFunction.Keywords);
	MenuTooltip = InPublicFunction.GetTooltip();

	const FString PackagePathString = InAssetData.PackageName.ToString();
	if(MenuTooltip.IsEmpty())
	{
		MenuTooltip = FText::FromString(PackagePathString);
	}
	else
	{
		static constexpr TCHAR Format[] = TEXT("%s\n\n%s");
		MenuTooltip = FText::FromString(FString::Printf(Format, *MenuTooltip.ToString(), *PackagePathString));
	}

	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	if (MenuKeywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuKeywords = FText::FromString(TEXT(" "));
	}

	MenuIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");

#if WITH_EDITOR
	if (InPublicFunction.IsMutable())
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
	}
	else
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->PureFunctionCallNodeTitleColor;
	}
#endif
	
}

URigVMEdGraphFunctionRefNodeSpawner::URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMOldPublicFunctionData& InPublicFunction)
{
	NodeClass = URigVMEdGraphNode::StaticClass();
	bIsLocalFunction = false;

	FRigVMGraphFunctionHeader& Header = ReferencedPublicFunctionHeader;
	Header.Name = InPublicFunction.Name;
	Header.Arguments.Reserve(InPublicFunction.Arguments.Num());
	for (const FRigVMOldPublicFunctionArg& Arg : InPublicFunction.Arguments)
	{
		FRigVMGraphFunctionArgument NewArgument;
		NewArgument.Name = Arg.Name;
		NewArgument.Direction = Arg.Direction;
		NewArgument.bIsArray = Arg.bIsArray;
		NewArgument.CPPType = Arg.CPPType;
		NewArgument.CPPTypeObject = FSoftObjectPath(Arg.CPPTypeObjectPath.ToString());
		Header.Arguments.Add(NewArgument);
	}
	AssetPath = InAssetData.ToSoftObjectPath();

	const FString Category = InPublicFunction.Category;

	MenuName = FText::FromName(InPublicFunction.Name);
	MenuCategory = FText::FromString(Category);
	MenuKeywords = FText::FromString(InPublicFunction.Keywords);

	if(const FRigVMEditorAssetInterfacePtr ReferencedBlueprint = InAssetData.FastGetAsset(false))
	{
		if(const URigVMFunctionLibrary* FunctionLibrary = ReferencedBlueprint->GetLocalFunctionLibrary())
		{
			if(const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(InPublicFunction.Name))
			{
				MenuTooltip = FunctionNode->GetToolTipText();
			}
		}
	}

	const FString ObjectPathString = InAssetData.GetObjectPathString();
	if(MenuTooltip.IsEmpty())
	{
		MenuTooltip = FText::FromString(ObjectPathString);
	}
	else
	{
		static constexpr TCHAR Format[] = TEXT("%s\n\n%s");
		MenuTooltip = FText::FromString(FString::Printf(Format, *MenuTooltip.ToString(), *ObjectPathString));
	}

	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	if (MenuKeywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuKeywords = FText::FromString(TEXT(" "));
	}

	MenuIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");

#if WITH_EDITOR
	if (InPublicFunction.IsMutable())
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
	}
	else
	{
		MenuIconTint = GetDefault<UGraphEditorSettings>()->PureFunctionCallNodeTitleColor;
	}
#endif
}

FString URigVMEdGraphFunctionRefNodeSpawner::GetSpawnerSignature() const
{
	FString SignatureString = TEXT("Invalid RigFunction");
	if (ReferencedPublicFunctionHeader.IsValid())
	{
		SignatureString = 
			FString::Printf(TEXT("RigFunction=%s"),
			*ReferencedPublicFunctionHeader.GetHash());
	}
	else
	{
		SignatureString = 
			FString::Printf(TEXT("RigFunction=%s:%s"),
			*GetResolvedAssetPath().ToString(),
			*ReferencedPublicFunctionHeader.Name.ToString());
	}

	if(bIsLocalFunction)
	{
		SignatureString += TEXT(" (local)");
	}

	return SignatureString;
}

URigVMEdGraphNode* URigVMEdGraphFunctionRefNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	// if we are trying to build the real function ref - but we haven't loaded the asset yet...
	if(!FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph))
	{
		if (GetResolvedAssetPath().IsValid())
		{
			if (UObject* BlueprintObj = GetResolvedAssetPath().TryLoad())
			{
				if (BlueprintObj->Implements<URigVMEditorAssetInterface>())
				{
					FRigVMEditorAssetInterfacePtr Blueprint = BlueprintObj;
					ReferencedPublicFunctionHeader = Blueprint->GetLocalFunctionLibrary()->FindFunction(ReferencedPublicFunctionHeader.Name)->GetFunctionHeader();
				}
			}
		}
	}

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);
	if(ReferencedPublicFunctionHeader.IsValid() && !bIsTemplateNode)
	{
#if WITH_EDITOR
		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}
#endif

		FRigVMEditorAssetInterfacePtr Blueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
		NewNode = SpawnNode(ParentGraph, Blueprint, ReferencedPublicFunctionHeader, Location);
	}
	else
	{
		// we are only going to get here if we are spawning a template node
		TArray<FPinInfo> Pins;
		for(const FRigVMGraphFunctionArgument& Arg : ReferencedPublicFunctionHeader.Arguments)
		{
			Pins.Emplace(Arg.Name, Arg.Direction, Arg.CPPType, Arg.CPPTypeObject.Get());
		}

		NewNode = SpawnTemplateNode(ParentGraph, Pins);
	}

	return NewNode;
}

URigVMEdGraphNode* URigVMEdGraphFunctionRefNodeSpawner::SpawnNode(UEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr Blueprint, FRigVMGraphFunctionHeader& InFunction, FVector2D const Location)
{
	URigVMEdGraphNode* NewNode = nullptr;
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);

	if (Blueprint != nullptr && RigGraph != nullptr)
	{
		FName Name = FRigVMBlueprintUtils::ValidateName(Blueprint, InFunction.Name.ToString());
		URigVMController* Controller = Blueprint->GetController(ParentGraph);

		Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

		if (URigVMFunctionReferenceNode* ModelNode = Controller->AddFunctionReferenceNodeFromDescription(InFunction, Location, Name.ToString(), true, true))
		{
			NewNode = Cast<URigVMEdGraphNode>(RigGraph->FindNodeForModelNodeName(ModelNode->GetFName()));
			check(NewNode);

			if (NewNode)
			{
				Controller->ClearNodeSelection(true);
				Controller->SelectNode(ModelNode, true, true);

				URigVMEdGraphUnitNodeSpawner::HookupMutableNode(ModelNode, Blueprint);
			}

			for(URigVMNode* OtherModelNode : ModelNode->GetGraph()->GetNodes())
			{
				if(OtherModelNode == ModelNode)
				{
					continue;
				}
				
				URigVMFunctionReferenceNode* ExistingFunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(OtherModelNode);
				if(ExistingFunctionReferenceNode == nullptr)
				{
					continue;
				}

				if(!(ExistingFunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer == InFunction.LibraryPointer))
				{
					continue;
				}
			}

			Controller->CloseUndoBracket();
		}
		else
		{
			Controller->CancelUndoBracket();
		}
	}
	return NewNode;
}

bool URigVMEdGraphFunctionRefNodeSpawner::IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const
{
	if(URigVMEdGraphNodeSpawner::IsTemplateNodeFilteredOut(InAssets, InGraphs, InPins))
	{
		return true;
	}

	// filter outdated functions
	if(ReferencedPublicFunctionHeader.IsValid())
	{
		for(const FRigVMTag& Tag : ReferencedPublicFunctionHeader.Variant.Tags)
		{
			if(Tag.bMarksSubjectAsInvalid)
			{
				return true;
			}
		}
	}
	
	if(bIsLocalFunction)
	{
		if(ReferencedPublicFunctionHeader.IsValid())
		{
			for (UObject* Blueprint : InAssets)
			{
				if (IRigVMEditorAssetInterface* Asset = Cast<IRigVMEditorAssetInterface>(Blueprint))
				{
					if (Asset->GetRigVMClientHost()->GetRigVMGraphFunctionHost() != ReferencedPublicFunctionHeader.GetFunctionHostObject())
					{
						return true;
					}
				}
			}
		}
	}
	const FString ReferencedAssetObjectPathString = ReferencedPublicFunctionHeader.LibraryPointer.GetNodeSoftPath().GetAssetName();
	for (UObject* Blueprint : InAssets)
	{
		if(Blueprint->GetPathName() == ReferencedAssetObjectPathString)
		{
			return true;
		}
	}
	return false;
}

FSoftObjectPath URigVMEdGraphFunctionRefNodeSpawner::GetResolvedAssetPath() const
{
	if (!FunctionPath.IsEmpty())
	{
		const FAssetData Asset = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(FunctionPath, true);
		AssetPath = Asset.ToSoftObjectPath();
		FunctionPath.Reset();
	}

	return AssetPath;
}

#undef LOCTEXT_NAMESPACE

