// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphInvokeEntryNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/EditorEngine.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "BlueprintNodeTemplateCache.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMModel/Nodes/RigVMInvokeEntryNode.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphInvokeEntryNodeSpawner"

URigVMEdGraphInvokeEntryNodeSpawner::URigVMEdGraphInvokeEntryNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FName& InEntryName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
{
	Blueprint = InBlueprint;
	EntryName = InEntryName;
	NodeClass = URigVMEdGraphNode::StaticClass();

	MenuName = InMenuDesc;
	MenuTooltip  = InTooltip;
	MenuCategory = InCategory;
	MenuKeywords = FText::FromString(TEXT("Event,Entry,Invoke,Run,Launch,Start"));
	MenuIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
}

FString URigVMEdGraphInvokeEntryNodeSpawner::GetSpawnerSignature() const
{
	return FString("InvokeEntryNode=" + EntryName.ToString());
}

URigVMEdGraphNode* URigVMEdGraphInvokeEntryNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);

	// First create a backing member for our node
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(RigGraph == nullptr) return nullptr;
	FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
	check(RigBlueprint);

#if WITH_EDITOR
	if (GEditor && !bIsTemplateNode)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	FString NodeName;
	if(bIsTemplateNode)
	{
		// since we are removing the node at the end of this function
		// we need to create a unique here.
		static constexpr TCHAR InvokeEntryNodeNameFormat[] = TEXT("InvokeEntryNode_%s");
		NodeName = FString::Printf(InvokeEntryNodeNameFormat, *EntryName.ToString());

		TArray<FPinInfo> Pins;

		static UScriptStruct* ExecuteScriptStruct = FRigVMExecuteContext::StaticStruct();
		static const FLazyName ExecuteStructName(*ExecuteScriptStruct->GetStructCPPName());
		Pins.Emplace(FRigVMStruct::ExecuteName, ERigVMPinDirection::IO, ExecuteStructName, ExecuteScriptStruct);

		return SpawnTemplateNode(ParentGraph, Pins, *NodeName);
	}

	URigVMController* Controller = RigBlueprint->GetController(ParentGraph);
	Controller->OpenUndoBracket(TEXT("Add Run Event"));

	if (URigVMNode* ModelNode = Controller->AddInvokeEntryNode(EntryName, Location, NodeName, true, true))
	{
		for (UEdGraphNode* Node : ParentGraph->Nodes)
		{
			if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
			{
				if (RigNode->GetModelNodeName() == ModelNode->GetFName())
				{
					NewNode = RigNode;
					break;
				}
			}
		}

		if (NewNode)
		{
			Controller->ClearNodeSelection(true);
			Controller->SelectNode(ModelNode, true, true);
		}
		Controller->CloseUndoBracket();
	}
	else
	{
		Controller->CancelUndoBracket();
	}


	return NewNode;
}

bool URigVMEdGraphInvokeEntryNodeSpawner::IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const
{
	if(URigVMEdGraphNodeSpawner::IsTemplateNodeFilteredOut(InAssets, InGraphs, InPins))
	{
		return true;
	}

	if (Blueprint.IsValid())
	{
		if (!InAssets.Contains(Blueprint.GetObject()))
		{
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

