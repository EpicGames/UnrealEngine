// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMStringUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEdGraphNodeSpawner)

#define LOCTEXT_NAMESPACE "RigVMEdGraphNodeSpawner"

bool URigVMEdGraphNodeSpawner::IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const
{
	check(RelatedBlueprintClass);

	for(const UObject* Asset : InAssets)
	{
		if(Asset->GetClass() != RelatedBlueprintClass)
		{
			return true;
		}
	}

	return false;
}

void URigVMEdGraphNodeSpawner::SetRelatedBlueprintClass(UClass* InClass)
{
	RelatedBlueprintClass = InClass;
}

URigVMEdGraphNode* URigVMEdGraphNodeSpawner::SpawnTemplateNode(URigVMEdGraph* InParentGraph, const TArray<FPinInfo>& InPins, const FName& InNodeName)
{
	URigVMEdGraphNode* NewNode = NewObject<URigVMEdGraphNode>(InParentGraph, InNodeName);
	InParentGraph->AddNode(NewNode, false);

	NewNode->CreateNewGuid();
	NewNode->PostPlacedNewNode();

	for(const FPinInfo& PinInfo : InPins)
	{
		const FName PinName = *RigVMStringUtils::JoinPinPath(NewNode->GetName(), PinInfo.Name.ToString());
		
		if(PinInfo.Direction ==  ERigVMPinDirection::Input ||
			PinInfo.Direction ==  ERigVMPinDirection::IO)
		{
			UEdGraphPin* InputPin = UEdGraphPin::CreatePin(NewNode);
			InputPin->PinName = PinName;
			NewNode->Pins.Add(InputPin);

			InputPin->Direction = EGPD_Input;
			InputPin->PinType = RigVMTypeUtils::PinTypeFromCPPType(PinInfo.CPPType, PinInfo.CPPTypeObject);
		}

		if(PinInfo.Direction ==  ERigVMPinDirection::Output ||
			PinInfo.Direction ==  ERigVMPinDirection::IO)
		{
			UEdGraphPin* OutputPin = UEdGraphPin::CreatePin(NewNode);
			OutputPin->PinName = PinName;
			NewNode->Pins.Add(OutputPin);

			OutputPin->Direction = EGPD_Output;
			OutputPin->PinType = RigVMTypeUtils::PinTypeFromCPPType(PinInfo.CPPType, PinInfo.CPPTypeObject);
		}
	}

	NewNode->SetFlags(RF_Transactional);

	return NewNode;
}

FBlueprintNodeSignature URigVMEdGraphNodeBlueprintSpawner::GetSpawnerSignature() const
{
	return RigVMSpawner->GetSpawnerSignature();
}

FBlueprintActionUiSpec URigVMEdGraphNodeBlueprintSpawner::GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const
{
	UEdGraph* TargetGraph = (Context.Graphs.Num() > 0) ? Context.Graphs[0] : nullptr;
	FBlueprintActionUiSpec MenuSignature = PrimeDefaultUiSpec(TargetGraph);

	DynamicUiSignatureGetter.ExecuteIfBound(Context, Bindings, &MenuSignature);
	return MenuSignature;
}

UEdGraphNode* URigVMEdGraphNodeBlueprintSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	return RigVMSpawner->Invoke(Cast<URigVMEdGraph>(ParentGraph), Location);
}

bool URigVMEdGraphNodeBlueprintSpawner::IsTemplateNodeFilteredOut(FBlueprintActionFilter const& Filter) const
{
	TArray<UObject*> BlueprintObjs;
	Algo::Transform(Filter.Context.Blueprints, BlueprintObjs, [](UBlueprint* Blueprint) { return Blueprint; });
	return RigVMSpawner->IsTemplateNodeFilteredOut(BlueprintObjs, Filter.Context.Graphs, Filter.Context.Pins);
}

void URigVMEdGraphNodeBlueprintSpawner::Prime()
{
	// we expect that you don't need a node template to construct menu entries
	// from this, so we choose not to pre-cache one here
}





#undef LOCTEXT_NAMESPACE

