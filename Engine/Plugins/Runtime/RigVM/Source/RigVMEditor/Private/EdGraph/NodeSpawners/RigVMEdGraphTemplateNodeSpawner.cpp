// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphTemplateNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/EditorEngine.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "K2Node_Variable.h"
#include "BlueprintNodeTemplateCache.h"
#include "RigVMBlueprintUtils.h"
#include "ScopedTransaction.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphTemplateNodeSpawner"

URigVMEdGraphTemplateNodeSpawner::URigVMEdGraphTemplateNodeSpawner(const FName& InNotation, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
{
	TemplateNotation = InNotation;
	Template =	FRigVMRegistry::Get().FindTemplate(InNotation);
	NodeClass = URigVMEdGraphNode::StaticClass();

	MenuName = InMenuDesc;
	MenuTooltip  = InTooltip;
	MenuCategory = InCategory;

	if (const FRigVMTemplate* RigVMTemplate = FRigVMRegistry::Get().FindTemplate(InNotation))
	{
#if WITH_EDITOR
		FString KeywordsMetadata = RigVMTemplate->GetKeywords();
		MenuKeywords = FText::FromString(KeywordsMetadata);
#endif
	}

	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	//
	// @TODO: maybe UPROPERTY() fields should have keyword metadata like functions
	if (MenuKeywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuKeywords = FText::FromString(TEXT(" "));
	}

	// @TODO: should use details customization-like extensibility system to provide editor only data like this
	MenuIcon = FSlateIcon(TEXT("RigVMEditorStyle"), TEXT("RigVM.Unit"));
}

FString URigVMEdGraphTemplateNodeSpawner::GetSpawnerSignature() const
{
	const int32 NotationHash = (int32)GetTypeHash(TemplateNotation);
	return FString("RigVMTemplate_") + FString::FromInt(NotationHash);
}

bool URigVMEdGraphTemplateNodeSpawner::IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const
{
	if(URigVMEdGraphNodeSpawner::IsTemplateNodeFilteredOut(InAssets, InGraphs, InPins))
	{
		return true;
	}

	for (const UEdGraphPin* Pin : InPins)
	{
		for (UEdGraph* Graph : InGraphs)
		{
			if (!Graph->Nodes.Contains(Pin->GetOwningNode()))
			{
				continue;
			}

			if (const URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(Graph))
			{
				if (const URigVMGraph* RigGraph = EdGraph->GetModel())
				{
					if (const URigVMPin* ModelPin = RigGraph->FindPin(Pin->GetName()))
					{
						// Only filter when the source pin is not a wildcard
						if (ModelPin->GetCPPType() != RigVMTypeUtils::GetWildCardCPPType())
						{
							if (Template)
							{
								if(ModelPin->IsExecuteContext())
								{
									FRigVMDispatchContext DispatchContext;
									if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelPin->GetNode()))
									{
										DispatchContext = DispatchNode->GetDispatchContext();
									}

									if(Template->NumExecuteArguments(DispatchContext) > 0)
									{
										return false;
									}
								}
								
								for (int32 i=0; i<Template->NumArguments(); ++i)
								{
									const FRigVMTemplateArgument* Argument = Template->GetArgument(i);
									if (Template->ArgumentSupportsTypeIndex(Argument->GetName(), ModelPin->GetTypeIndex()))
									{
										return false;
									}
								}
								return true;
							}									
						}
					}					
				}
			}
		}
	}

	return false;
}

URigVMEdGraphNode* URigVMEdGraphTemplateNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	if(Template)
	{
#if WITH_EDITOR
		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}
#endif

		bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);
		if(bIsTemplateNode)
		{
			const int32 NotationHash = (int32)GetTypeHash(Template->GetNotation());
			const FString TemplateName = TEXT("RigVMTemplate_") + FString::FromInt(NotationHash);
			FName Name = *TemplateName;

			const FRigVMRegistry& Registry = FRigVMRegistry::Get();

			TArray<FPinInfo> Pins;

			const FRigVMDispatchContext Context;
			for (int32 Index = 0; Index < Template->NumExecuteArguments(Context); Index++)
			{
				const FRigVMExecuteArgument* Argument = Template->GetExecuteArgument(Index, Context);
				check(Argument);
				static UScriptStruct* ExecuteScriptStruct = FRigVMExecuteContext::StaticStruct();
				static const FLazyName ExecuteStructName(*ExecuteScriptStruct->GetStructCPPName());
				Pins.Emplace(Argument->Name, Argument->Direction, ExecuteStructName, ExecuteScriptStruct);
			}

			for (int32 Index = 0; Index < Template->NumArguments(); Index++)
			{
				const FRigVMTemplateArgument* Argument = Template->GetArgument(Index);
				check(Argument);

				FName CPPType = RigVMTypeUtils::GetWildCardCPPTypeName();
				UObject* CPPTypeObject = RigVMTypeUtils::GetWildCardCPPTypeObject();

				if(Argument->IsSingleton())
				{
					const TRigVMTypeIndex TypeIndex = Argument->GetTypeIndex(0);
					const FRigVMTemplateArgumentType& Type = Registry.GetType(TypeIndex);
					CPPType = Type.CPPType;
					CPPTypeObject = Type.CPPTypeObject;
				}
				
				Pins.Emplace(Argument->GetName(), Argument->GetDirection(), CPPType, CPPTypeObject);
			}
			NewNode = SpawnTemplateNode(ParentGraph, Pins, Name);
			if(NewNode)
			{
				NewNode->ModelNodePath = Template->GetNotation().ToString();
			}
			return NewNode;
		}

		FRigVMEditorAssetInterfacePtr Blueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
		NewNode = SpawnNode(ParentGraph, Blueprint, Template, Location);
	}

	return NewNode;
}

URigVMEdGraphNode* URigVMEdGraphTemplateNodeSpawner::SpawnNode(URigVMEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr RigBlueprint, const FRigVMTemplate* Template, FVector2D const Location)
{
	URigVMEdGraphNode* NewNode = nullptr;
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);

	if (RigBlueprint != nullptr && RigGraph != nullptr)
	{
		const FName Name = FRigVMBlueprintUtils::ValidateName(RigBlueprint, Template->GetNodeName().ToString());
		URigVMController* Controller = RigBlueprint->GetController(ParentGraph);

		Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

		if (URigVMTemplateNode* ModelNode = Controller->AddTemplateNode(Template->GetNotation(), Location, Name.ToString(), true, true))
		{
			NewNode = Cast<URigVMEdGraphNode>(RigGraph->FindNodeForModelNodeName(ModelNode->GetFName()));

			if (NewNode)
			{
				Controller->ClearNodeSelection(true);
				Controller->SelectNode(ModelNode, true, true);

				URigVMEdGraphUnitNodeSpawner::HookupMutableNode(ModelNode, RigBlueprint);

				if(NewNode->GetCanRenameNode())
				{
					NewNode->RequestRename();
				}
			}

			Controller->CloseUndoBracket();
		}
		else
		{
			Controller->CancelUndoBracket();
		}
	}
	return NewNode;
}

#undef LOCTEXT_NAMESPACE

