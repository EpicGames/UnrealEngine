// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/EditorEngine.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "K2Node_Variable.h"
#include "BlueprintNodeTemplateCache.h"
#include "FindInBlueprintManager.h"
#include "RigVMBlueprintUtils.h"
#include "ScopedTransaction.h"
#include "RigVMFunctions/Execution/RigVMFunction_UserDefinedEvent.h"
#include "RigVMPropertyUtils.h"

#if WITH_EDITOR
#include "Editor.h"
#include "SGraphActionMenu.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphUnitNodeSpawner"

URigVMEdGraphUnitNodeSpawner::URigVMEdGraphUnitNodeSpawner(UScriptStruct* InStruct, const FName& InMethodName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
{
	StructTemplate = InStruct;
	MethodName = InMethodName;
	NodeClass = URigVMEdGraphNode::StaticClass();

	MenuName = InMenuDesc;
	MenuTooltip  = InTooltip;
	MenuCategory = InCategory;

	FString KeywordsMetadata, TemplateNameMetadata;
	InStruct->GetStringMetaDataHierarchical(FRigVMStruct::KeywordsMetaName, &KeywordsMetadata);
	if(!TemplateNameMetadata.IsEmpty())
	{
		if(KeywordsMetadata.IsEmpty())
		{
			KeywordsMetadata = TemplateNameMetadata;
		}
		else
		{
			KeywordsMetadata = KeywordsMetadata + TEXT(",") + TemplateNameMetadata;
		}
	}
	MenuKeywords = FText::FromString(KeywordsMetadata);

	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	//
	// @TODO: maybe UPROPERTY() fields should have keyword metadata like functions
	if (MenuKeywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuKeywords = FText::FromString(TEXT(" "));
	}

	// @TODO: should use details customization-like extensibility system to provide editor only data like this
	FStructOnScope StructScope(InStruct);
	const FRigVMStruct* StructInstance = (const FRigVMStruct*)StructScope.GetStructMemory();
	if(StructInstance->GetEventName().IsNone())
	{
		if(InStruct->HasMetaDataHierarchical(FRigVMStruct::IconMetaName))
		{
			FString IconPath;
			const int32 NumOfIconPathNames = 4;
			
			FName IconPathNames[NumOfIconPathNames] = {
				NAME_None, // StyleSetName
				NAME_None, // StyleName
				NAME_None, // SmallStyleName
				NAME_None  // StatusOverlayStyleName
			};

			InStruct->GetStringMetaDataHierarchical(FRigVMStruct::IconMetaName, &IconPath);

			int32 NameIndex = 0;

			while (!IconPath.IsEmpty() && NameIndex < NumOfIconPathNames)
			{
				FString Left;
				FString Right;

				if (!IconPath.Split(TEXT("|"), &Left, &Right))
				{
					Left = IconPath;
				}

				IconPathNames[NameIndex] = FName(*Left);

				NameIndex++;
				IconPath = Right;
			}
			MenuIcon = FSlateIcon(IconPathNames[0], IconPathNames[1], IconPathNames[2], IconPathNames[3]);
		}
		else
		{
			MenuIcon = FSlateIcon(TEXT("RigVMEditorStyle"), TEXT("RigVM.Unit"));
		}
	}
	else
	{
		MenuIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	}
}

FString URigVMEdGraphUnitNodeSpawner::GetSpawnerSignature() const
{
	return FString("RigUnit=" + StructTemplate->GetFName().ToString());
}

URigVMEdGraphNode* URigVMEdGraphUnitNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	if(StructTemplate)
	{
#if WITH_EDITOR
		if (GEditor)
		{
			GEditor->CancelTransaction(0);
		}
#endif

		FRigVMEditorAssetInterfacePtr Blueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
		NewNode = SpawnNode(ParentGraph, Blueprint, StructTemplate, MethodName, Location);
	}

	return NewNode;
}

URigVMEdGraphNode* URigVMEdGraphUnitNodeSpawner::SpawnNode(URigVMEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr RigBlueprint, UScriptStruct* StructTemplate, const FName& InMethodName, FVector2D const Location)
{
	URigVMEdGraphNode* NewNode = nullptr;
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);

	if (RigBlueprint != nullptr && RigGraph != nullptr)
	{
		bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);

		FName Name = bIsTemplateNode ? *StructTemplate->GetStructCPPName() : FRigVMBlueprintUtils::ValidateName(RigBlueprint, StructTemplate->GetFName().ToString());

		if(bIsTemplateNode)
		{
			TArray<FPinInfo> Pins;
			for (TFieldIterator<FProperty> It(StructTemplate); It; ++It)
			{
				FPinInfo Pin;
				const FProperty* Property = *It;
				Pin.Name = Property->GetFName();
				RigVMPropertyUtils::GetTypeFromProperty(Property, Pin.CPPType, Pin.CPPTypeObject);
				if(Pin.CPPType.IsNone())
				{
					continue;
				}

				if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Pin.CPPTypeObject))
				{
					static UScriptStruct* ExecuteScriptStruct = FRigVMExecuteContext::StaticStruct();
					static const FLazyName ExecuteStructName(*ExecuteScriptStruct->GetStructCPPName());

					if(ScriptStruct->IsChildOf(ExecuteScriptStruct))
					{
						Pin.CPPType = ExecuteStructName;
						Pin.CPPTypeObject = ExecuteScriptStruct;
					}
				}

				const bool bInput = Property->HasMetaData(FRigVMStruct::InputMetaName);
				const bool bOutput = Property->HasMetaData(FRigVMStruct::OutputMetaName);

				if(bInput && bOutput)
				{
					Pin.Direction = ERigVMPinDirection::IO;
				}
				else if(bInput)
				{
					Pin.Direction = ERigVMPinDirection::Input;
				}
				else if(bOutput)
				{
					Pin.Direction = ERigVMPinDirection::Output;
				}

				Pins.Add(Pin);
			}
			return SpawnTemplateNode(ParentGraph, Pins, Name);
		}

		FStructOnScope StructScope(StructTemplate);
		const FRigVMStruct* StructInstance = (const FRigVMStruct*)StructScope.GetStructMemory();
		const FName EventName = StructInstance->GetEventName();

		// events can only exist once across all uber graphs
		if(!EventName.IsNone())
		{
			if(StructInstance->CanOnlyExistOnce() &&
				(StructTemplate != FRigVMFunction_UserDefinedEvent::StaticStruct()))
			{
				const TArray<URigVMGraph*> Models = RigBlueprint->GetAllModels();
				for(URigVMGraph* Model : Models)
				{
					if(Model->IsRootGraph() && !Model->IsA<URigVMFunctionLibrary>())
					{
						for(const URigVMNode* Node : Model->GetNodes())
						{
							if(Node->GetEventName() == EventName)
							{
								RigBlueprint->OnRequestJumpToHyperlink().Execute(Node);
								return nullptr;
							}
						}
					}
				}
			}
		}
		
		URigVMController* Controller = RigBlueprint->GetController(ParentGraph);
		if(Controller == nullptr)
		{
			return nullptr;
		}

		Controller->OpenUndoBracket(FString::Printf(TEXT("Add '%s' Node"), *Name.ToString()));

		FRigVMUnitNodeCreatedContext& UnitNodeCreatedContext = Controller->GetUnitNodeCreatedContext();
		TScriptInterface<IRigVMClientHost> ClientHost = RigBlueprint.GetObject();
		FRigVMUnitNodeCreatedContext::FScope ReasonScope(UnitNodeCreatedContext, ERigVMNodeCreatedReason::NodeSpawner, ClientHost.GetInterface());

		if (URigVMUnitNode* ModelNode = Controller->AddUnitNode(StructTemplate, InMethodName, Location, Name.ToString(), true, true))
		{
			NewNode = Cast<URigVMEdGraphNode>(RigGraph->FindNodeForModelNodeName(ModelNode->GetFName()));
			check(NewNode);

			if (NewNode)
			{
				if(StructTemplate == FRigVMFunction_UserDefinedEvent::StaticStruct())
				{
					const URigVMEdGraphSchema* Schema = CastChecked<URigVMEdGraphSchema>(NewNode->GetSchema());
					
					// ensure uniqueness for the event name
					TArray<FName> ExistingEventNames;

					const TArray<URigVMGraph*> Models = RigBlueprint->GetAllModels();
					for(URigVMGraph* Model : Models)
					{
						for(const URigVMNode* Node : Model->GetNodes())
						{
							if(Node != ModelNode)
							{
								const FName ExistingEventName = Node->GetEventName();
								if(!ExistingEventName.IsNone())
								{
									ExistingEventNames.AddUnique(ExistingEventName);
								}
							}
						}
					}

					FName SafeEventName = Controller->GetSchema()->GetUniqueName(EventName, [ExistingEventNames, Schema](const FName& NameToCheck) -> bool
					{
						return !ExistingEventNames.Contains(NameToCheck) && !Schema->IsRigVMDefaultEvent(NameToCheck);
					}, false, true);

					Controller->SetPinDefaultValue(ModelNode->FindPin(TEXT("EventName"))->GetPinPath(), SafeEventName.ToString(), false, true, false, true);
				}

				Controller->ClearNodeSelection(true);
				Controller->SelectNode(ModelNode, true, true);

				HookupMutableNode(ModelNode, RigBlueprint);
			}

#if WITH_EDITORONLY_DATA
			/*
			// Todo: we should remove this functionality imho
			if (!bIsTemplateNode)
			{
				const FControlRigSettingsPerPinBool* ExpansionMapPtr = UControlRigEditorSettings::Get()->RigUnitPinExpansion.Find(ModelNode->GetScriptStruct()->GetName());
				if (ExpansionMapPtr)
				{
					const FControlRigSettingsPerPinBool& ExpansionMap = *ExpansionMapPtr;

					for (const TPair<FString, bool>& Pair : ExpansionMap.Values)
					{
						FString PinPath = FString::Printf(TEXT("%s.%s"), *ModelNode->GetName(), *Pair.Key);
						Controller->SetPinExpansion(PinPath, Pair.Value, bIsUserFacingNode);
					}
				}

				FString UsedFilterString = SGraphActionMenu::LastUsedFilterText;
				if (!UsedFilterString.IsEmpty())
				{
					UsedFilterString = UsedFilterString.ToLower();

					if (UEnum* RigElementTypeEnum = StaticEnum<ERigElementType>())
					{
						ERigElementType UsedElementType = ERigElementType::None;
						const int32 NumEnumValues = RigElementTypeEnum->NumEnums();

						for (int32 EnumIndex = 0; EnumIndex < NumEnumValues; EnumIndex++)
						{
							const FString EnumText = RigElementTypeEnum->GetDisplayNameTextByIndex(EnumIndex).ToString().ToLower();
							if (!EnumText.IsEmpty() && UsedFilterString.Contains(EnumText))
							{
								UsedElementType = (ERigElementType)RigElementTypeEnum->GetValueByIndex(EnumIndex);
								break;
							}
						}

						if (UsedElementType != ERigElementType::None)
						{
							TArray<URigVMPin*> ModelPins = ModelNode->GetAllPinsRecursively();
							for (URigVMPin* ModelPin : ModelPins)
							{
								if (ModelPin->GetCPPTypeObject() == FRigElementKey::StaticStruct())
								{
									if (URigVMPin* TypePin = ModelPin->FindSubPin(TEXT("Type")))
									{
										FString DefaultValue = RigElementTypeEnum->GetDisplayNameTextByValue((int64)UsedElementType).ToString();
										Controller->SetPinDefaultValue(TypePin->GetPinPath(), DefaultValue);
										break;
									}
								}
							}
						}
					}
				}
			}
			*/
#endif

			Controller->CloseUndoBracket();

			if(NewNode->GetCanRenameNode())
			{
				NewNode->RequestRename();
			}
		}
		else
		{
			Controller->CancelUndoBracket();
		}
	}
	return NewNode;
}

void URigVMEdGraphUnitNodeSpawner::HookupMutableNode(URigVMNode* InModelNode, FRigVMEditorAssetInterfacePtr InRigBlueprint)
{
	if(!InRigBlueprint->GetRigVMEditorSettings()->bAutoLinkMutableNodes)
	{
		return;
	}
	
	URigVMController* Controller = InRigBlueprint->GetController(InModelNode->GetGraph());

	Controller->ClearNodeSelection(true);
	Controller->SelectNode(InModelNode, true, true);

	// see if the node has an execute pin
	URigVMPin* ModelNodeExecutePin = nullptr;
	for (URigVMPin* Pin : InModelNode->GetPins())
	{
		if (Pin->GetScriptStruct())
		{
			if (Pin->GetScriptStruct()->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				if (Pin->GetDirection() == ERigVMPinDirection::IO || Pin->GetDirection() == ERigVMPinDirection::Input)
				{
					ModelNodeExecutePin = Pin;
					break;
				}
			}
		}
	}

	// we have an execute pin - so we have to hook it up
	if (ModelNodeExecutePin)
	{
		URigVMPin* ClosestOtherModelNodeExecutePin = nullptr;

		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		if (Schema->LastPinForCompatibleCheck)
		{
			URigVMPin* FromPin = Controller->GetGraph()->FindPin(Schema->LastPinForCompatibleCheck->GetName());
			if (FromPin)
			{
				if (FromPin->IsExecuteContext() &&
					(FromPin->GetDirection() == ERigVMPinDirection::IO || FromPin->GetDirection() == ERigVMPinDirection::Output))
				{
					ClosestOtherModelNodeExecutePin = FromPin;
				}
			}

		}

		if (ClosestOtherModelNodeExecutePin == nullptr)
		{
			double ClosestDistance = DBL_MAX;
			
			for (URigVMNode* OtherModelNode : Controller->GetGraph()->GetNodes())
			{
				if (OtherModelNode == InModelNode)
				{
					continue;
				}

				for (URigVMPin* Pin : OtherModelNode->GetPins())
				{
					if (Pin->IsExecuteContext())
					{
						if (Pin->GetDirection() == ERigVMPinDirection::IO || Pin->GetDirection() == ERigVMPinDirection::Output)
						{
							if (Pin->GetLinkedTargetPins().Num() == 0)
							{
								double Distance = (OtherModelNode->GetPosition() - InModelNode->GetPosition()).Size();
								if (Distance < ClosestDistance)
								{
									ClosestOtherModelNodeExecutePin = Pin;
									ClosestDistance = Distance;
									break;
								}
							}
						}
					}
				}
			}
		}

		if (ClosestOtherModelNodeExecutePin)
		{
			Controller->AddLink(ClosestOtherModelNodeExecutePin->GetPinPath(), ModelNodeExecutePin->GetPinPath(), true);
		}
	}
}

#undef LOCTEXT_NAMESPACE

