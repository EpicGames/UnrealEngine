// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/NodeSpawners/RigVMEdGraphVariableNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor/EditorEngine.h"
#include "ObjectEditorUtils.h"
#include "EditorCategoryUtils.h"
#include "K2Node_Variable.h"
#include "BlueprintNodeTemplateCache.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEdGraphVariableNodeSpawner"

URigVMEdGraphVariableNodeSpawner::URigVMEdGraphVariableNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FRigVMExternalVariable& InExternalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
{
	Blueprint = InBlueprint;
	ExternalVariable = InExternalVariable;
	bIsGetter = bInIsGetter;
	NodeClass = URigVMEdGraphNode::StaticClass();

	MenuName = FText::FromString(FString::Printf(TEXT("%s %s"), bInIsGetter ? TEXT("Get") : TEXT("Set"), *InMenuDesc.ToString()));
	MenuTooltip  = InTooltip;
	MenuCategory = InCategory;
	MenuKeywords = FText::FromString(TEXT("Variable"));

	FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(InExternalVariable);
	MenuIcon = UK2Node_Variable::GetVarIconFromPinType(PinType, MenuIconTint);
}

URigVMEdGraphVariableNodeSpawner::URigVMEdGraphVariableNodeSpawner(
	FRigVMEditorAssetInterfacePtr InBlueprint, URigVMGraph* InGraphOwner, const FRigVMGraphVariableDescription& InLocalVariable,
	bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)
		: URigVMEdGraphVariableNodeSpawner(InBlueprint, InLocalVariable.ToExternalVariable(), bInIsGetter, InMenuDesc, InCategory, InTooltip)
{
	bIsLocalVariable = true;
	GraphOwner = InGraphOwner;
}

bool URigVMEdGraphVariableNodeSpawner::IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const
{
	if(URigVMEdGraphNodeSpawner::IsTemplateNodeFilteredOut(InAssets, InGraphs, InPins))
	{
		return true;
	}

	if (Blueprint.IsValid())
	{
		if (!InAssets.Contains(Blueprint.GetObject()))
		{
			return true;
		}

		if (bIsLocalVariable)
		{
			bool bIsFiltered = true;
			if (InGraphs.Num() == 1 && GraphOwner.IsValid())
			{
				if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(InGraphs[0]))
				{
					if (GraphOwner.Get() == Graph->GetModel())
					{
						bIsFiltered = false;
					}
				}
			}

			if (bIsFiltered)
			{
				return true;
			}
		}
	}
	return false;
}

FString URigVMEdGraphVariableNodeSpawner::GetSpawnerSignature() const
{
	return FString("ExternalVariable=" + ExternalVariable.GetName().ToString());
}

URigVMEdGraphNode* URigVMEdGraphVariableNodeSpawner::Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const
{
	URigVMEdGraphNode* NewNode = nullptr;

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);

	// First create a backing member for our node
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(ParentGraph);
	if(RigGraph == nullptr) return nullptr;
	FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
	check(RigBlueprint);

	FName MemberName = NAME_None;

#if WITH_EDITOR
	if (GEditor && !bIsTemplateNode)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	FString ObjectPath;
	if (ExternalVariable.GetCPPTypeObject())
	{
		ObjectPath = ExternalVariable.GetCPPTypeObject()->GetPathName();
	}

	const FString TypeName = ExternalVariable.GetExtendedCPPType().ToString();

	FString NodeName;
	if(bIsTemplateNode)
	{
		// for template controllers let's rely on locally defined nodes
		// without a backing model node. access to local variables or
		// input arguments doesn't work on the template controller.
		
		static constexpr TCHAR VariableNodeNameFormat[] = TEXT("VariableNode_%s_%s");
		static const FString GetterPrefix = TEXT("Getter");
		static const FString SetterPrefix = TEXT("Setter");
		NodeName = FString::Printf(VariableNodeNameFormat, bIsGetter ? *GetterPrefix : *SetterPrefix, *ExternalVariable.GetName().ToString());

		TArray<FPinInfo> Pins;

		if(!bIsGetter)
		{
			static UScriptStruct* ExecuteScriptStruct = FRigVMExecuteContext::StaticStruct();
			static const FLazyName ExecuteStructName(*ExecuteScriptStruct->GetStructCPPName());
			Pins.Emplace(FRigVMStruct::ExecuteName, ERigVMPinDirection::IO, ExecuteStructName, ExecuteScriptStruct);
		}
		
		Pins.Emplace(
			URigVMVariableNode::ValueName,
			bIsGetter ? ERigVMPinDirection::Output : ERigVMPinDirection::Input,
			ExternalVariable.GetExtendedCPPType(),
			const_cast<UObject*>(ExternalVariable.GetCPPTypeObject()));
		
		return SpawnTemplateNode(ParentGraph, Pins, *NodeName);
	}

	URigVMController* Controller = RigBlueprint->GetController(ParentGraph);
	Controller->OpenUndoBracket(TEXT("Add Variable"));

	if (URigVMNode* ModelNode = Controller->AddVariableNodeFromObjectPath(ExternalVariable.GetName(), TypeName, ObjectPath, bIsGetter, FString(), Location, NodeName, true, true))
	{
		for (UEdGraphNode* Node : ParentGraph->Nodes)
		{
			if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Node))
			{
				if (RigNode->GetModelNodeName() == ModelNode->GetFName())
				{
					NewNode = RigNode;
					break;
				}
			}
		}

		if (NewNode)
		{
			Controller->ClearNodeSelection(true);
			Controller->SelectNode(ModelNode, true, true);
		}
		Controller->CloseUndoBracket();
	}
	else
	{
		Controller->CancelUndoBracket();
	}


	return NewNode;
}

#undef LOCTEXT_NAMESPACE

