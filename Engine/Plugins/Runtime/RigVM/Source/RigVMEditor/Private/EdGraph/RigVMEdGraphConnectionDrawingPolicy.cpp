// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraphConnectionDrawingPolicy.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMHost.h"
#include "RigVMEditorAsset.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMModel/RigVMController.h"
#include "Kismet2/BlueprintEditorUtils.h"

void FRigVMEdGraphConnectionDrawingPolicy::SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins)
{
	UEdGraphPin* Pin = StartPin->GetPinObj();
	if (Pin != nullptr)
	{
		URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Pin->GetOwningNode());
		if(RigNode)
		{
			if(URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(Pin->GetName()))
			{
				ModelPin->GetGraph()->PrepareCycleChecking(ModelPin->GetPinForLink(), Pin->Direction == EGPD_Input);
			}
		}
	}
	FKismetConnectionDrawingPolicy::SetIncompatiblePinDrawState(StartPin, VisiblePins);
}

void FRigVMEdGraphConnectionDrawingPolicy::ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins)
{
	if (VisiblePins.Num() > 0)
	{
		TSharedRef<SWidget> WidgetRef = *VisiblePins.begin();
		const SGraphPin* PinWidget = (const SGraphPin*)&WidgetRef.Get();
		UEdGraphPin* Pin = PinWidget->GetPinObj();
		if (Pin != nullptr)
		{
			if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Pin->GetOwningNode()))
			{
				if (URigVMGraph* ModelGraph = RigNode->GetModel())
				{
					ModelGraph->PrepareCycleChecking(nullptr, true);
				}
			}
		}
	}
	FKismetConnectionDrawingPolicy::ResetIncompatiblePinDrawState(VisiblePins);
}

void FRigVMEdGraphConnectionDrawingPolicy::BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries)
{
	FKismetConnectionDrawingPolicy::BuildPinToPinWidgetMap(InPinGeometries);

	// Add any sub-pins to the widget map if they arent there already, but with their parents geometry.
	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		struct Local
		{
			static void AddSubPins_Recursive(UEdGraphPin* PinObj, TMap<UEdGraphPin*, TSharedPtr<SGraphPin>>& InPinToPinWidgetMap, TSharedPtr<SGraphPin>& InGraphPinWidget)
			{
				if (PinObj)
				{
					for(UEdGraphPin* SubPin : PinObj->SubPins)
					{
						// Only add to the pin-to-pin widget map if the sub-pin widget is not there already
						TSharedPtr<SGraphPin>* SubPinWidgetPtr = InPinToPinWidgetMap.Find(SubPin);
						if(SubPinWidgetPtr == nullptr)
						{
							SubPinWidgetPtr = &InGraphPinWidget;
						}

						TSharedPtr<SGraphPin> PinWidgetPtr = *SubPinWidgetPtr;
						InPinToPinWidgetMap.Add(SubPin, PinWidgetPtr);
						AddSubPins_Recursive(SubPin, InPinToPinWidgetMap, PinWidgetPtr);
					}
				}
			}
		};

		TSharedPtr<SGraphPin> GraphPinWidget = StaticCastSharedRef<SGraphPin>(ConnectorIt.Key());
		Local::AddSubPins_Recursive(GraphPinWidget->GetPinObj(), PinToPinWidgetMap, GraphPinWidget);
	}
}

void FRigVMEdGraphConnectionDrawingPolicy::DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	auto DrawPin = [this, &ArrangedNodes](UEdGraphPin* ThePin, TSharedRef<SWidget>& InSomePinWidget)
	{
		if (ThePin->Direction == EGPD_Output)
		{
			for (int32 LinkIndex=0; LinkIndex < ThePin->LinkedTo.Num(); ++LinkIndex)
			{
				FArrangedWidget* LinkStartWidgetGeometry = nullptr;
				FArrangedWidget* LinkEndWidgetGeometry = nullptr;

				UEdGraphPin* TargetPin = ThePin->LinkedTo[LinkIndex];

				DetermineLinkGeometry(ArrangedNodes, InSomePinWidget, ThePin, TargetPin, /*out*/ LinkStartWidgetGeometry, /*out*/ LinkEndWidgetGeometry);

				if (( LinkEndWidgetGeometry && LinkStartWidgetGeometry ) && !IsConnectionCulled( *LinkStartWidgetGeometry, *LinkEndWidgetGeometry ))
				{
					FConnectionParams Params;
					DetermineWiringStyle(ThePin, TargetPin, /*inout*/ Params);

					if(UseLowDetailConnections())
					{
						const float StartFudgeX = 4.0f;
						const float EndFudgeX = 4.0f;

						const FVector2D StartPoint = FGeometryHelper::VerticalMiddleRightOf(LinkStartWidgetGeometry->Geometry) - FVector2D(StartFudgeX, 0.0f);
						const FVector2D EndPoint = FGeometryHelper::VerticalMiddleLeftOf(LinkEndWidgetGeometry->Geometry) - FVector2D(EndFudgeX, 0);

						static TArray<FVector2D> LinePoints;
						LinePoints.SetNum(2);
						LinePoints[0] = StartPoint;
						LinePoints[1] = EndPoint;

						FSlateDrawElement::MakeLines(DrawElementsList, WireLayerID, FPaintGeometry(), LinePoints, ESlateDrawEffect::None, Params.WireColor);
					}
					else
					{
						DrawSplineWithArrow(LinkStartWidgetGeometry->Geometry, LinkEndWidgetGeometry->Geometry, Params);
					}
				}
			}
		}
	};

	for (TMap<TSharedRef<SWidget>, FArrangedWidget>::TIterator ConnectorIt(InPinGeometries); ConnectorIt; ++ConnectorIt)
	{
		TSharedRef<SWidget> SomePinWidget = ConnectorIt.Key();
		SGraphPin& PinWidget = static_cast<SGraphPin&>(SomePinWidget.Get());
		
		struct Local
		{
			static void DrawSubPins_Recursive(UEdGraphPin* PinObj, TSharedRef<SWidget>& InSomePinWidget, const TFunctionRef<void(UEdGraphPin* Pin, TSharedRef<SWidget>& PinWidget)>& DrawPinFunction)
			{
				DrawPinFunction(PinObj, InSomePinWidget);

				for(UEdGraphPin* SubPin : PinObj->SubPins)
				{
					DrawSubPins_Recursive(SubPin, InSomePinWidget, DrawPinFunction);
				}
			}
		};

		Local::DrawSubPins_Recursive(PinWidget.GetPinObj(), SomePinWidget, DrawPin);
	}
}

void FRigVMEdGraphConnectionDrawingPolicy::DetermineLinkGeometry(
	FArrangedChildren& ArrangedNodes, 
	TSharedRef<SWidget>& OutputPinWidget,
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	/*out*/ FArrangedWidget*& StartWidgetGeometry,
	/*out*/ FArrangedWidget*& EndWidgetGeometry
	)
{
	if (TSharedPtr<SGraphPin>* pOutputWidget = PinToPinWidgetMap.Find(OutputPin))
	{
		StartWidgetGeometry = PinGeometries->Find((*pOutputWidget).ToSharedRef());
	}
	
	if (TSharedPtr<SGraphPin>* pInputWidget = PinToPinWidgetMap.Find(InputPin))
	{
		EndWidgetGeometry = PinGeometries->Find((*pInputWidget).ToSharedRef());
	}
}

bool FRigVMEdGraphConnectionDrawingPolicy::ShouldChangeTangentForReouteControlPoint(URigVMEdGraphNode* Node)
{
	bool bPinReversed = false;
	int32 InputPin = 0, OutputPin = 0;
	if (Node->ShouldDrawNodeAsControlPointOnly(InputPin, OutputPin))
	{
		if (bool* pResult = RerouteNodeToReversedDirectionMap.Find(Node))
		{
			// This case triggers if multiple wires share the same reroute node
			return *pResult;
		}
		else
		{
			FVector2f AverageLeftPin;
			FVector2f AverageRightPin;
			FVector2f CenterPin;

			const TArray<UEdGraphPin*>& Pins = Node->GetAllPins();

			// InputPin and OutputPin shared the same position, it does not matter which one we use.
			bool bCenterValid = FindPinCenter(Pins[OutputPin], /*out*/ CenterPin);

			bool bLeftValid = GetAverageConnectedPositionForPin(Pins[InputPin], AverageLeftPin);
			bool bRightValid = GetAverageConnectedPositionForPin(Pins[OutputPin], AverageRightPin);

			if (bLeftValid && bRightValid)
			{
				bPinReversed = AverageRightPin.X < AverageLeftPin.X;
			}
			else if (bCenterValid)
			{
				if (bLeftValid)
				{
					bPinReversed = CenterPin.X < AverageLeftPin.X;
				}
				else if (bRightValid)
				{
					bPinReversed = AverageRightPin.X < CenterPin.X;
				}
			}

			// We don't need to clear the map because Drawing Policy is generated/deleted for each OnPaint()
			RerouteNodeToReversedDirectionMap.Add(Node, bPinReversed);
		} 
	}

	return bPinReversed;
}

// Average of the positions of all pins connected to InPin
bool FRigVMEdGraphConnectionDrawingPolicy::GetAverageConnectedPositionForPin(UEdGraphPin* InPin, FVector2f& OutPos) const
{
	FVector2f Result = FVector2f::ZeroVector;
	int32 ResultCount = 0;

	for (UEdGraphPin* LinkedPin : InPin->LinkedTo)
	{
		FVector2f CenterPoint;
		if (FindPinCenter(LinkedPin, /*out*/ CenterPoint))
		{
			Result += CenterPoint;
			ResultCount++;
		}
	}

	if (ResultCount > 0)
	{
		OutPos = Result * (1.0f / static_cast<float>(ResultCount));
		return true;
	}
	else
	{
		return false;
	}
}

void FRigVMEdGraphConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	FKismetConnectionDrawingPolicy::DetermineWiringStyle(OutputPin, InputPin, Params);
	if (OutputPin == nullptr || InputPin == nullptr || UseLowDetailConnections())
	{
		return;
	}

	if(OutputPin->Direction == EGPD_Input)
	{
		Swap(OutputPin, InputPin);
	}

	URigVMEdGraphNode* OutputNode = Cast<URigVMEdGraphNode>(OutputPin->GetOwningNode());
	URigVMEdGraphNode* InputNode = Cast<URigVMEdGraphNode>(InputPin->GetOwningNode());
	if (OutputNode && InputNode)
	{
		// If the output or input connect to a Reroute Node(Node Knot/Control Point) that is going backwards, we will flip the direction on values going into them
		{
			if (ShouldChangeTangentForReouteControlPoint(OutputNode))
			{
				Params.StartDirection = EGPD_Input;
			}

			if (ShouldChangeTangentForReouteControlPoint(InputNode))
			{
				Params.EndDirection = EGPD_Output;
			}
		}

		bool bInjectionIsSelected = false;
		URigVMPin* OutputModelPin = OutputNode->GetModelPinFromPinPath(OutputPin->GetName());
		URigVMPin* InputModelPin = InputNode->GetModelPinFromPinPath(InputPin->GetName());

		if (OutputModelPin)
		{
			OutputModelPin = OutputModelPin->GetPinForLink();
			if (URigVMInjectionInfo* OutputInjection = OutputModelPin->GetNode()->GetInjectionInfo())
			{
				if (OutputModelPin->GetNode()->IsSelected())
				{
					bInjectionIsSelected = true;
				}
			}
		}

		if (!bInjectionIsSelected)
		{
			if (InputModelPin)
			{
				InputModelPin = InputModelPin->GetPinForLink();
				if (InputModelPin)
				{
					if (URigVMInjectionInfo* InputInjection = InputModelPin->GetNode()->GetInjectionInfo())
					{
						if (InputModelPin->GetNode()->IsSelected())
						{
							bInjectionIsSelected = true;
						}
					}
				}
			}
		}

		if (bInjectionIsSelected)
		{
			Params.WireThickness = Settings->TraceAttackWireThickness;
			Params.WireColor = Settings->TraceAttackColor;
		}

		if (OutputModelPin && InputModelPin)
		{
			// Assume visited unless debugging
			bool bVisited = true;
			int32 OutputInstructionIndex = OutputNode->GetInstructionIndex(false);
			int32 InputInstructionIndex = InputNode->GetInstructionIndex(true);

			if (OutputInstructionIndex != INDEX_NONE && InputInstructionIndex != INDEX_NONE)
			{
				if (FRigVMEditorAssetInterfacePtr RigBlueprint = FRigVMBlueprintUtils::FindAssetForNode(OutputNode))
				{
					if (URigVMHost* Host = Cast<URigVMHost>(RigBlueprint->GetObjectBeingDebugged()))
					{
						// Assume not visited unless visited when debugging last run
						bVisited = false;
						if (const URigVM* VM = Host->GetVM())
						{
							if (VM->WasInstructionVisitedDuringLastRun(Host->GetRigVMExtendedExecuteContext(), OutputInstructionIndex) &&
								VM->WasInstructionVisitedDuringLastRun(Host->GetRigVMExtendedExecuteContext(), InputInstructionIndex))
							{
								bVisited = true;
							}
						}
					}
				}
			}

			if (bVisited)
			{
				//Params.bDrawBubbles = true;
				Params.WireThickness = Settings->DefaultExecutionWireThickness;
			}
			else
			{
				Params.WireColor = Params.WireColor * 0.5f;
			}
		}

		const float FadedState = FMath::Min(OutputNode->GetFadedOutState(), InputNode->GetFadedOutState());
		Params.WireColor.A *= FadedState;
	}
}
