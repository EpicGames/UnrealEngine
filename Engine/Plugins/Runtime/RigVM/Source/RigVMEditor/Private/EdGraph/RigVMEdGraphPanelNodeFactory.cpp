// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraphPanelNodeFactory.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraphNode_Comment.h"
#include "Widgets/SRigVMGraphNode.h"
#include "Widgets/SRigVMGraphNodeComment.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Widgets/SRigVMGraphNodeKnot.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/Nodes/RigVMRerouteNode.h"

FName FRigVMEdGraphPanelNodeFactory::GetFactoryName() const
{
	return IRigVMEditorAssetInterface::RigVMPanelNodeFactoryName;
}

TSharedPtr<SGraphNode> FRigVMEdGraphPanelNodeFactory::CreateNode(UEdGraphNode* Node) const
{
	// we need to check if this is the right factory for the implementation
	if(const URigVMEdGraph* EdGraph = Node->GetTypedOuter<URigVMEdGraph>())
	{
		if(const FRigVMEditorAssetInterfacePtr Blueprint = EdGraph->GetAssetDefaultObject())
		{
			if(Blueprint->GetPanelNodeFactoryName() != GetFactoryName())
			{
				return nullptr;
			}
		}
	}

	if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node))
	{
		int32 InputPin = 0, OutputPin = 0;
		if (Node->ShouldDrawNodeAsControlPointOnly(InputPin, OutputPin))
		{
			TSharedRef<SGraphNode> GraphNode = SNew(SRigVMGraphNodeKnot, Node);
			return GraphNode;
		}

		TSharedRef<SGraphNode> GraphNode = 
			SNew(SRigVMGraphNode)
			.GraphNodeObj(RigVMEdGraphNode);

		GraphNode->SlatePrepass();
		RigVMEdGraphNode->SetDimensions(GraphNode->GetDesiredSize());
		return GraphNode;
	}
	
	if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
	{
		if (const UEdGraphSchema* Schema = CommentNode->GetSchema())
		{
			if (Schema->IsA(URigVMEdGraphSchema::StaticClass()))
			{
				TSharedRef<SGraphNode> GraphNode =
					SNew(SRigVMGraphNodeComment, CommentNode);

				GraphNode->SlatePrepass();
				return GraphNode;
			}
		}
	}

	return nullptr;
}
