// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraph/RigVMEdGraphPanelPinFactory.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Widgets/SRigVMGraphPinNameList.h"
#include "Widgets/SRigVMGraphPinCurveFloat.h"
#include "Widgets/SRigVMGraphPinVariableName.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "Widgets/SRigVMGraphPinUserDataNameSpace.h"
#include "Widgets/SRigVMGraphPinUserDataPath.h"
#include "Widgets/SRigVMGraphPinQuat.h"
#include "Widgets/SRigVMGraphPinCategory.h"
#include "KismetPins/SGraphPinExec.h"
#include "SGraphPinComboBox.h"
#include "RigVMHost.h"
#include "NodeFactory.h"
#include "EdGraphSchema_K2.h"
#include "Curves/CurveFloat.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "Widgets/SRigVMGraphPinEnumPicker.h"

FName FRigVMEdGraphPanelPinFactory::GetFactoryName() const
{
	return IRigVMEditorAssetInterface::RigVMPanelPinFactoryName;
}

TSharedPtr<SGraphPin> FRigVMEdGraphPanelPinFactory::CreatePin(UEdGraphPin* InPin) const
{
	if(InPin == nullptr)
	{
		return nullptr;
	}
	
	// we need to check if this is the right factory for the implementation
	if(const UEdGraphNode* EdGraphNode = InPin->GetOuter())
	{
		if(const URigVMEdGraph* EdGraph = EdGraphNode->GetTypedOuter<URigVMEdGraph>())
		{
			if(const FRigVMEditorAssetInterfacePtr Blueprint = EdGraph->GetAssetDefaultObject())
			{
				if(Blueprint->GetPanelPinFactoryName() != GetFactoryName())
				{
					return nullptr;
				}
			}
		}
	}

	TSharedPtr<SGraphPin> InternalResult = CreatePin_Internal(InPin);
	if(InternalResult.IsValid())
	{
		return InternalResult;
	}

	// if the graph we are looking at is not a rig vm graph - let's not do this
	if (const UEdGraphNode* OwningNode = InPin->GetOwningNode())
	{
		// only create pins within rig vm graphs
		if (Cast<URigVMEdGraph>(OwningNode->GetGraph()) == nullptr)
		{
			return nullptr;
		}
	}

	return FNodeFactory::CreateK2PinWidget(InPin);
}

TSharedPtr<SGraphPin> FRigVMEdGraphPanelPinFactory::CreatePin_Internal(UEdGraphPin* InPin) const
{
	if (InPin)
	{
		if (const UEdGraphNode* OwningNode = InPin->GetOwningNode())
		{
			// only create pins within rig vm graphs
			if (Cast<URigVMEdGraph>(OwningNode->GetGraph()) == nullptr)
			{
				return nullptr;
			}
		}

		if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(InPin->GetOwningNode()))
		{
			URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(RigNode->GetGraph());

			URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(InPin->GetName());
			if (ModelPin)
			{
				if ((ModelPin->GetDirection() == ERigVMPinDirection::Input || ModelPin->GetDirection() == ERigVMPinDirection::Visible) && 
					(ModelPin->IsBoundToVariable() || ModelPin->IsDefinedAsInputVariable()))
				{
					if (FRigVMEditorAssetInterfacePtr Blueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(RigGraph))
					{
						return SNew(SRigVMGraphPinVariableBinding, InPin)
							.ModelPins({ModelPin})
							.Asset(Blueprint);
					}
				}

				FName CustomWidgetName = ModelPin->GetCustomWidgetName();
				if (CustomWidgetName == TEXT("EntryName"))
				{
					return SNew(SRigVMGraphPinNameList, InPin)
						.ModelPin(ModelPin)
						.OnGetNameListContent_UObject(RigGraph, &URigVMEdGraph::GetEntryNameList);
				}
				if (CustomWidgetName == TEXT("VariableName"))
				{
					return SNew(SRigVMGraphPinVariableName, InPin);
				}
				else if (CustomWidgetName == TEXT("UserDataNameSpace"))
				{
					return SNew(SRigVMGraphPinUserDataNameSpace, InPin);
				}
				else if (CustomWidgetName == TEXT("UserDataPath"))
				{
					return SNew(SRigVMGraphPinUserDataPath, InPin)
						.ModelPins({ModelPin});
				}

				if (ModelPin->GetCPPTypeObject() == UEnum::StaticClass())
				{
					return SNew(SRigVMGraphPinEnumPicker, InPin)
						.ModelPin(ModelPin);
				}
			}
			else if(const URigVMNode* ModelNode = RigNode->GetModelNode())
			{
				if(ModelNode->GetPinCategories().Contains(InPin->GetName()))
				{
					return SNew(SRigVMGraphPinCategory, InPin);
				}
			}

			if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
			{
				const UStruct* Struct = Cast<UStruct>(InPin->PinType.PinSubCategoryObject);
				if (Struct && Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					return SNew(SGraphPinExec, InPin);
				}
				if (InPin->PinType.PinSubCategoryObject == FRuntimeFloatCurve::StaticStruct())
				{
					return SNew(SRigVMGraphPinCurveFloat, InPin);
				}
				if (ModelPin && (InPin->PinType.PinSubCategoryObject == TBaseStructure<FQuat>::Get()))
				{
					return SNew(SRigVMGraphPinQuat, InPin).ModelPin(ModelPin);
				}
			}
		}
	}

	return nullptr;
}
