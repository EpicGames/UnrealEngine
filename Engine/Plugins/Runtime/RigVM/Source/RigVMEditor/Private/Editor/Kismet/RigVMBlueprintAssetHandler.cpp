// Copyright Epic Games, Inc. All Rights Reserved.

#if !WITH_RIGVMLEGACYEDITOR

#include "Editor/Kismet/RigVMBlueprintAssetHandler.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "Blueprint/BlueprintSupport.h"
#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Engine/World.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

class FLevelBlueprintAssetHandler : public IRigVMBlueprintAssetHandler
{
	virtual UBlueprint* RetrieveBlueprint(UObject* InObject) const override
	{
		UWorld* World = CastChecked<UWorld>(InObject);

		const bool bDontCreate = true;
		return World->PersistentLevel ? World->PersistentLevel->GetLevelScriptBlueprint(bDontCreate) : nullptr;
	}

	virtual bool AssetContainsBlueprint(const FAssetData& InAssetData) const
	{
		// Worlds are only considered to contain a blueprint if they have FiB data
		return InAssetData.TagsAndValues.Contains(FBlueprintTags::FindInBlueprintsData) || InAssetData.TagsAndValues.Contains(FBlueprintTags::UnversionedFindInBlueprintsData);
	}
};


class FBlueprintAssetTypeHandler : public IRigVMBlueprintAssetHandler
{
	virtual UBlueprint* RetrieveBlueprint(UObject* InObject) const override
	{
		// The object is the blueprint for UBlueprint (and derived) assets
		return CastChecked<UBlueprint>(InObject);
	}

	virtual bool AssetContainsBlueprint(const FAssetData& InAssetData) const
	{
		return true;
	}
};

FRigVMBlueprintAssetHandler::FRigVMBlueprintAssetHandler()
{
	// Register default handlers
	RegisterHandler<FLevelBlueprintAssetHandler>(UWorld::StaticClass()->GetClassPathName());
	RegisterHandler<FBlueprintAssetTypeHandler>(UBlueprint::StaticClass()->GetClassPathName());
}

FRigVMBlueprintAssetHandler& FRigVMBlueprintAssetHandler::Get()
{
	static FRigVMBlueprintAssetHandler Singleton;
	return Singleton;
}

void FRigVMBlueprintAssetHandler::RegisterHandler(FTopLevelAssetPath EligibleClass, TUniquePtr<IRigVMBlueprintAssetHandler>&& InHandler)
{
	ClassNames.Add(EligibleClass);
	Handlers.Add(MoveTemp(InHandler));
}

const IRigVMBlueprintAssetHandler* FRigVMBlueprintAssetHandler::FindHandler(const UClass* InClass) const
{
	UClass* StopAtClass = UObject::StaticClass();
	while (InClass && InClass != StopAtClass)
	{
		int32 Index = ClassNames.IndexOfByKey(InClass->GetClassPathName());
		if (Index != INDEX_NONE)
		{
			return Handlers[Index].Get();
		}

		InClass = InClass->GetSuperClass();
	}

	return nullptr;
}

#endif