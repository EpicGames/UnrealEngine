// Copyright Epic Games, Inc. All Rights Reserved.


#include "Editor/Kismet/RigVMBlueprintCompilerExtension.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMBlueprintCompilerExtension)

class FKismetCompilerContext;

URigVMBlueprintCompilerExtension::URigVMBlueprintCompilerExtension(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
}

#if !WITH_RIGVMLEGACYEDITOR

void URigVMBlueprintCompilerExtension::BlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FRigVMBlueprintCompiledData& Data)
{
	// common entry point in case we need to add logging, profiling, etc
	ProcessBlueprintCompiled(CompilationContext, Data);
}

#endif
