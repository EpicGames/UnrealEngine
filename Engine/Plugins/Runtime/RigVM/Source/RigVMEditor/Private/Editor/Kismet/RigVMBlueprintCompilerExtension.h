// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Containers/Array.h"
#include "KismetCompiler.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "RigVMBlueprintCompilerExtension.generated.h"

#define UE_API RIGVMEDITOR_API

class UEdGraph;

USTRUCT()
struct FRigVMBlueprintCompiledData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UEdGraph>> IntermediateGraphs;
};

UCLASS(MinimalAPI, Abstract)
class URigVMBlueprintCompilerExtension : public UObject
{
	GENERATED_BODY()
	
public:
	UE_API URigVMBlueprintCompilerExtension(const FObjectInitializer& ObjectInitializer);
#if !WITH_RIGVMLEGACYEDITOR

	UE_API void BlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FRigVMBlueprintCompiledData& Data);

protected:
	/** 
	 * Override this if you're interested in running logic after class layout has been
	 * generated, but before bytecode and member variables have been 
	 */
	virtual void ProcessBlueprintCompiled(const FKismetCompilerContext& CompilationContext, const FRigVMBlueprintCompiledData& Data) {}
#endif
};

#undef UE_API
