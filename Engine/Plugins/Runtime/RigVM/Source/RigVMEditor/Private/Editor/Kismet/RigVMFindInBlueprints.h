// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !WITH_RIGVMLEGACYEDITOR

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "RigVMFindInBlueprintManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#endif
#include "RigVMFindInBlueprints.generated.h"

#if !WITH_RIGVMLEGACYEDITOR

class FRigVMEditorBase;
class FRigVMImaginaryFiBData;
class FJsonValue;
class FUICommandList;
class ITableRow;
class SDockTab;
class SVerticalBox;
class SWidget;
class UBlueprint;
class UClass;
class UObject;
struct FGeometry;
struct FKeyEvent;
struct FSlateBrush;

typedef STreeView<FRigVMSearchResult>  SRigVMTreeViewType;

DECLARE_DELEGATE_OneParam(FRigVMOnSearchComplete, TArray<FRigVMImaginaryFiBDataSharedPtr>&);

/** Some utility functions to help with Find-in-Blueprint functionality */
namespace RigVMFindInBlueprintsHelpers
{
	// Stores an FText as if it were an FString, does zero advanced comparisons needed for true FText comparisons
	struct FSimpleFTextKeyStorage
	{
		FText Text;

		FSimpleFTextKeyStorage(FText InText)
			: Text(InText)
		{

		}

		bool operator==(const FSimpleFTextKeyStorage& InObject) const
		{
			return Text.ToString() == InObject.Text.ToString() || Text.BuildSourceString() == InObject.Text.BuildSourceString();
		}
	};

	/** Utility function to find the ancestor class or interface from which a function is inherited. */
	RIGVMEDITOR_API UClass* GetFunctionOriginClass(const UFunction* Function);

	/** Constructs a search term for a function using Find-in-Blueprints search syntax */
	RIGVMEDITOR_API bool ConstructSearchTermFromFunction(const UFunction* Function, FString& SearchTerm);

	static uint32 GetTypeHash(const RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage& InObject)
	{
		return GetTypeHash(InObject.Text.BuildSourceString());
	}

	/** Looks up a JsonValue's FText from the passed lookup table */
	FText AsFText(TSharedPtr< FJsonValue > InJsonValue, const TMap<int32, FText>& InLookupTable);

	/** Looks up a JsonValue's FText from the passed lookup table */
	FText AsFText(int32 InValue, const TMap<int32, FText>& InLookupTable);

	bool IsTextEqualToString(const FText& InText, const FString& InString);

	/**
	 * Retrieves the pin type as a string value
	 *
	 * @param InPinType		The pin type to look at
	 *
	 * @return				The pin type as a string in format [category]'[sub-category object]'
	 */
	FString GetPinTypeAsString(const FEdGraphPinType& InPinType);

	/**
	 * Parses a pin type from passed in key names and values
	 *
	 * @param InKey					The key name for what the data should be translated as
	 * @param InValue				Value to be be translated
	 * @param InOutPinType			Modifies the PinType based on the passed parameters, building it up over multiple calls
	 * @return						TRUE when the parsing is successful
	 */
	bool ParsePinType(FText InKey, FText InValue, FEdGraphPinType& InOutPinType);

	/**
	* Iterates through all the given tree node's children and tells the tree view to expand them
	*/
	void ExpandAllChildren(FRigVMSearchResult InTreeNode, TSharedPtr<STreeView<TSharedPtr<FRigVMFindInBlueprintsResult>>> InTreeView);
}

/** Class used to denote an empty search result */
class FRigVMFindInBlueprintsNoResult : public FRigVMFindInBlueprintsResult
{
public:
	FRigVMFindInBlueprintsNoResult(const FText& InDisplayText)
		:FRigVMFindInBlueprintsResult(InDisplayText)
	{
	}

	/** FRigVMFindInBlueprintsResult Interface */
	virtual FReply OnClick() override
	{
		// Do nothing on click.
		return FReply::Handled();
	}
	/** End FRigVMFindInBlueprintsResult Interface */
};

/** Graph nodes use this class to store their data */
class FRigVMFindInBlueprintsGraphNode : public FRigVMFindInBlueprintsResult
{
public:
	FRigVMFindInBlueprintsGraphNode();
	virtual ~FRigVMFindInBlueprintsGraphNode() = default;

	/** FRigVMFindInBlueprintsResult Interface */
	virtual FReply OnClick() override;
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	virtual void FinalizeSearchData() override;
	virtual UObject* GetObject(UBlueprint* InBlueprint) const override;
	/** End FRigVMFindInBlueprintsResult Interface */

private:
	/** The Node Guid to find when jumping to the node */
	FGuid NodeGuid;

	/** The glyph brush for this node */
	FSlateIcon Glyph;

	/** The glyph color for this node */
	FLinearColor GlyphColor;

	/*The class this item refers to */
	UClass* Class;

	/*The class name this item refers to */
	FString ClassName;
};

/** Pins use this class to store their data */
class FRigVMFindInBlueprintsPin : public FRigVMFindInBlueprintsResult
{
public:
	FRigVMFindInBlueprintsPin(FString InSchemaName);
	virtual ~FRigVMFindInBlueprintsPin() = default;

	/** FRigVMFindInBlueprintsResult Interface */
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	virtual void FinalizeSearchData() override;
	/** End FRigVMFindInBlueprintsResult Interface */

private:
	/** The name of the schema this pin exists under */
	FString SchemaName;

	/** The pin that this search result refers to */
	FEdGraphPinType PinType;

	/** Pin's icon color */
	FSlateColor IconColor;
};

/** Property data is stored here */
class FRigVMFindInBlueprintsProperty : public FRigVMFindInBlueprintsResult
{
public:
	FRigVMFindInBlueprintsProperty();
	virtual ~FRigVMFindInBlueprintsProperty() = default;

	/** FRigVMFindInBlueprintsResult Interface */
	virtual FReply OnClick() override;
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	virtual void FinalizeSearchData() override;
	/** End FRigVMFindInBlueprintsResult Interface */

private:
	/** The pin that this search result refers to */
	FEdGraphPinType PinType;

	/** The default value of a property as a string */
	FString DefaultValue;

	/** TRUE if the property is an SCS_Component */
	bool bIsSCSComponent;
};

/** Graphs, such as functions and macros, are stored here */
class FRigVMFindInBlueprintsGraph : public FRigVMFindInBlueprintsResult
{
public:
	FRigVMFindInBlueprintsGraph(EGraphType InGraphType);
	virtual ~FRigVMFindInBlueprintsGraph() = default;

	/** FRigVMFindInBlueprintsResult Interface */
	virtual FReply OnClick() override;
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	/** End FRigVMFindInBlueprintsResult Interface */

private:
	/** The type of graph this represents */
	EGraphType GraphType;
};

// Cache bar widgets.
enum class ERigVMFiBCacheBarWidget
{
	ProgressBar,
	CloseButton,
	CancelButton,
	CacheAllUnindexedButton,
	CurrentAssetNameText,
	UnresponsiveEditorWarningText,
	ShowCacheFailuresButton,
	ShowCacheStatusText
};

// Search bar widgets.
enum class ERigVMFiBSearchBarWidget
{
	StatusText,
	Throbber,
	ProgressBar,
};

#endif

// Whether the Find-in-Blueprints window allows the user to load and resave all assets with out-of-date Blueprint search metadata
UENUM()
enum class ERigVMFiBIndexAllPermission
{
	// Users may not automatically load all Blueprints with out-of-date search metadata
	None,
	// Users may automatically load all Blueprints with out-of-date search metadata, but not resave
	LoadOnly,
	// Users may automatically checkout, load and resave all Blueprints with out-of-date search metadata
	CheckoutAndResave
};

#if !WITH_RIGVMLEGACYEDITOR

/*Widget for searching for (functions/events) across all blueprints or just a single blueprint */
class SRigVMFindInBlueprints: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRigVMFindInBlueprints )
		: _bIsSearchWindow(true)
		, _bHideSearchBar(false)
		, _bHideFindGlobalButton(false)
		, _ContainingTab()
	{}
		SLATE_ARGUMENT(bool, bIsSearchWindow)
		SLATE_ARGUMENT(bool, bHideSearchBar)
		SLATE_ARGUMENT(bool, bHideFindGlobalButton)
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ContainingTab)
	SLATE_END_ARGS()

	RIGVMEDITOR_API void Construct(const FArguments& InArgs, TSharedPtr<class FRigVMEditorBase> InBlueprintEditor = nullptr);
	RIGVMEDITOR_API ~SRigVMFindInBlueprints();

	/** Focuses this widget's search box, and changes the mode as well, and optionally the search terms */
	RIGVMEDITOR_API void FocusForUse(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false);

	/**
	 * Submits a search query
	 *
	 * @param InSearchString						String to search using
	 * @param bInIsFindWithinBlueprint				TRUE if searching within the current Blueprint only
	 * @param InSearchOptions						Optional search parameters.
	 * @param InOnSearchComplete					Callback when the search is complete, passing the filtered imaginary data (if any).
	 */
	RIGVMEDITOR_API void MakeSearchQuery(FString InSearchString, bool bInIsFindWithinBlueprint, const FRigVMStreamSearchOptions& InSearchOptions = FRigVMStreamSearchOptions(), FRigVMOnSearchComplete InOnSearchComplete = FRigVMOnSearchComplete());

	/** Called when caching Blueprints is started */
	RIGVMEDITOR_API void OnCacheStarted(ERigVMFiBCacheOpType InOpType, ERigVMFiBCacheOpFlags InOpFlags);
	
	/** Called when caching Blueprints is complete */
	RIGVMEDITOR_API void OnCacheComplete(ERigVMFiBCacheOpType InOpType, ERigVMFiBCacheOpFlags InOpFlags);

	/**
	 * Asynchronously caches all Blueprints below a specified version.
	 *
	 * @param InOptions								Options to configure the caching task
	 */
	RIGVMEDITOR_API void CacheAllBlueprints(const FRigVMFindInBlueprintCachingOptions& InOptions);

	/** If this is a global find results widget, returns the host tab's unique ID. Otherwise, returns NAME_None. */
	RIGVMEDITOR_API FName GetHostTabId() const;

	/** If this is a global find results widget, ask the host tab to close */
	RIGVMEDITOR_API void CloseHostTab();

	/** Determines if this context does not accept syncing from an external source */
	bool IsLocked() const
	{
		return bIsLocked;
	}

	/** Determines whether a search query is actively in progress */
	RIGVMEDITOR_API bool IsSearchInProgress() const;

	/** SWidget overrides */
	RIGVMEDITOR_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Clears the currently visible results */
	RIGVMEDITOR_API void ClearResults();

private:
	/** Processes results of the ongoing async stream search */
	RIGVMEDITOR_API EActiveTimerReturnType UpdateSearchResults( double InCurrentTime, float InDeltaTime );

	/** Register any Find-in-Blueprint commands */
	RIGVMEDITOR_API void RegisterCommands();

	/*Called when user changes the text they are searching for */
	RIGVMEDITOR_API void OnSearchTextChanged(const FText& Text);

	/*Called when user changes commits text to the search box */
	RIGVMEDITOR_API void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/* Get the children of a row */
	RIGVMEDITOR_API void OnGetChildren( FRigVMSearchResult InItem, TArray< FRigVMSearchResult >& OutChildren );

	/* Called when user double clicks on a new result */
	RIGVMEDITOR_API void OnTreeSelectionDoubleClicked( FRigVMSearchResult Item );

	/* Called when a new row is being generated */
	RIGVMEDITOR_API TSharedRef<ITableRow> OnGenerateRow(FRigVMSearchResult InItem, const TSharedRef<STableViewBase>& OwnerTable);
	
	/** Launches a thread for streaming more content into the results widget */
	RIGVMEDITOR_API void LaunchStreamThread(const FString& InSearchValue, const FRigVMStreamSearchOptions& InSearchOptions = FRigVMStreamSearchOptions(), FRigVMOnSearchComplete InOnSearchComplete = FRigVMOnSearchComplete());

	/** Returns the percent complete on the search for the progress bar */
	RIGVMEDITOR_API TOptional<float> GetPercentCompleteSearch() const;

	/** Returns the search bar visiblity for the given widget */
	RIGVMEDITOR_API EVisibility GetSearchBarWidgetVisiblity(ERigVMFiBSearchBarWidget InSearchBarWidget) const;

	/** Adds the "cache" bar at the bottom of the Find-in-Blueprints widget, to notify the user that the search is incomplete */
	RIGVMEDITOR_API void ConditionallyAddCacheBar();

	/** Callback to remove the "cache" bar when a button is pressed */
	RIGVMEDITOR_API FReply OnRemoveCacheBar();

	/** Callback to return the cache bar's display text, informing the user of the situation */
	RIGVMEDITOR_API FText GetCacheBarStatusText() const;

	/** Callback to return the current asset name during a cache operation */
	RIGVMEDITOR_API FText GetCacheBarCurrentAssetName() const;

	/** Whether user is allowed to initiate loading and indexing all blueprints with out-of-date metadata */
	RIGVMEDITOR_API bool CanCacheAllUnindexedBlueprints() const;

	/** Callback to cache all unindexed Blueprints */
	RIGVMEDITOR_API FReply OnCacheAllUnindexedBlueprints();

	/** Callback to export a list of all blueprints that need reindexing */
	RIGVMEDITOR_API FReply OnExportUnindexedAssetList();

	/** Callback to cache all Blueprints according to the given options */
	RIGVMEDITOR_API FReply OnCacheAllBlueprints(const FRigVMFindInBlueprintCachingOptions& InOptions);

	/** Callback to cancel the caching process */
	RIGVMEDITOR_API FReply OnCancelCacheAll();

	/** Retrieves the current index of the Blueprint caching process */
	RIGVMEDITOR_API int32 GetCurrentCacheIndex() const;

	/** Gets the percent complete of the caching process */
	RIGVMEDITOR_API TOptional<float> GetPercentCompleteCache() const;

	/** Returns the caching bar's visibility, it goes invisible when there is nothing to be cached. The next search will remove this bar or make it visible again */
	RIGVMEDITOR_API EVisibility GetCacheBarVisibility() const;

	/** Returns the cache bar visibility for the given widget */
	RIGVMEDITOR_API EVisibility GetCacheBarWidgetVisibility(ERigVMFiBCacheBarWidget InCacheBarWidget) const;

	/** Returns TRUE if Blueprint caching is in progress */
	RIGVMEDITOR_API bool IsCacheInProgress() const;

	/** Returns the BG image used for the cache bar */
	RIGVMEDITOR_API const FSlateBrush* GetCacheBarImage() const;

	/** Callback to build the context menu when right clicking in the tree */
	RIGVMEDITOR_API TSharedPtr<SWidget> OnContextMenuOpening();

	/** Helper function to select all items */
	RIGVMEDITOR_API void SelectAllItemsHelper(FRigVMSearchResult InItemToSelect);

	/** Callback when user attempts to select all items in the search results */
	RIGVMEDITOR_API void OnSelectAllAction();

	/** Callback when user attempts to copy their selection in the Find-in-Blueprints */
	RIGVMEDITOR_API void OnCopyAction();

	/** Called when the user clicks the global find results button */
	RIGVMEDITOR_API FReply OnOpenGlobalFindResults();

	/** Called when the host tab is closed (if valid) */
	RIGVMEDITOR_API void OnHostTabClosed(TSharedRef<SDockTab> DockTab);

	/** Called when the lock button is clicked in a global find results tab */
	RIGVMEDITOR_API FReply OnLockButtonClicked();

	/** Returns the image used for the lock button in a global find results tab */
	RIGVMEDITOR_API const FSlateBrush* OnGetLockButtonImage() const;

private:
	/** Pointer back to the blueprint editor that owns us */
	TWeakPtr<class FRigVMEditorBase> EditorPtr;
	
	/* The tree view displays the results */
	TSharedPtr<SRigVMTreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<class SSearchBox> SearchTextField;
	
	/* This buffer stores the currently displayed results */
	TArray<FRigVMSearchResult> ItemsFound;

	/** In Find Within Blueprint mode, we need to keep a handle on the root result, because it won't show up in the tree */
	FRigVMSearchResult RootSearchResult;

	/* The string to highlight in the results */
	FText HighlightText;

	/* The string to search for */
	FString	SearchValue;

	/** Thread object that searches through Blueprint data on a separate thread */
	TSharedPtr< class FRigVMStreamSearch> StreamSearch;

	/** Vertical box, used to add and remove widgets dynamically */
	TWeakPtr< SVerticalBox > MainVerticalBox;

	/** Weak pointer to the cache bar slot, so it can be removed */
	TWeakPtr< SWidget > CacheBarSlot;

	/** Callback when search is complete */
	FRigVMOnSearchComplete OnSearchComplete;

	/** Cached count of out of date Blueprints from last search. */
	int32 OutOfDateWithLastSearchBPCount;

	/** Cached version that was last searched */
	ERigVMFiBVersion LastSearchedFiBVersion;

	/** Commands handled by this widget */
	TSharedPtr< FUICommandList > CommandList;

	/** Tab hosting this widget. May be invalid. */
	TWeakPtr<SDockTab> HostTab;

	/** Last cached asset path (used during continuous cache operations). */
	mutable FSoftObjectPath LastCachedAssetPath;

	/** Should we search within the current blueprint only (rather than all blueprints) */
	bool bIsInFindWithinBlueprintMode;

	/** True if current search should not be changed by an external source */
	bool bIsLocked;

	/** True if progress bar widgets should be hidden */
	bool bHideProgressBars;

	/** True if users should be allowed to close the cache bar while caching */
	bool bShowCacheBarCloseButton;

	/** True if users should be allowed to cancel the active caching operation */
	bool bShowCacheBarCancelButton;

	/** True if the unresponsive warning text should be visible in the cache bar */
	bool bShowCacheBarUnresponsiveEditorWarningText;

	/** True if cache bar should remain visible after a caching operation has ended */
	bool bKeepCacheBarProgressVisible;
};
#endif
