// Copyright Epic Games, Inc. All Rights Reserved.

#if !WITH_RIGVMLEGACYEDITOR

#include "Editor/Kismet/RigVMImaginaryBlueprintData.h"

#include "Async/Async.h"
#include "Async/Future.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/StringConv.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/StringTableCore.h"
#include "Misc/CString.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Serialization/Archive.h"
#include "Serialization/JsonTypes.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

class UObject;

#define LOCTEXT_NAMESPACE "RigVMFindInBlueprints"

///////////////////////
// FRigVMSearchableValueInfo

FText FRigVMSearchableValueInfo::GetDisplayText(const TMap<int32, FText>& InLookupTable) const
{
	FText Result;
	if (!DisplayText.IsEmpty() || LookupTableKey == -1)
	{
		Result = DisplayText;
	}
	else
	{
		Result = RigVMFindInBlueprintsHelpers::AsFText(LookupTableKey, InLookupTable);
	}

	if (Result.IsFromStringTable() && FTextInspector::GetSourceString(Result) == &FStringTableEntry::GetPlaceholderSourceString() && !IsInGameThread())
	{
		// String Table asset references in FiB may be unresolved as we can't load the asset on the search thread
		// To solve this we send a request to the game thread to load the asset and wait for the result
		FName TableId;
		FString Key;
		if (FTextInspector::GetTableIdAndKey(Result, TableId, Key) && IStringTableEngineBridge::IsStringTableFromAsset(TableId))
		{
			TPromise<bool> Promise;

			// Run the request on the game thread, filling the promise when done
			AsyncTask(ENamedThreads::GameThread, [TableId, &Promise]()
			{
				FName ResolvedTableId = TableId;
				if (IStringTableEngineBridge::CanFindOrLoadStringTableAsset())
				{
					IStringTableEngineBridge::FullyLoadStringTableAsset(ResolvedTableId); // Trigger the asset load
				}
				Promise.SetValue(true); // Signal completion
			});

			// Get the promise value to block until the AsyncTask has completed
			Promise.GetFuture().Get();
		}
	}

	return Result;
}

////////////////////////////
// FRigVMComponentUniqueDisplay

bool FRigVMComponentUniqueDisplay::operator==(const FRigVMComponentUniqueDisplay& Other)
{
	// Two search results in the same object/sub-object should never have the same display string ({Key}: {Value} pairing)
	return SearchResult.IsValid() && Other.SearchResult.IsValid() && SearchResult->GetDisplayString().CompareTo(Other.SearchResult->GetDisplayString()) == 0;
}

///////////////////////
// FRigVMImaginaryFiBData

FCriticalSection FRigVMImaginaryFiBData::ParseChildDataCriticalSection;

FRigVMImaginaryFiBData::FRigVMImaginaryFiBData(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: UnparsedJsonObject(InUnparsedJsonObject)
	, LookupTablePtr(InLookupTablePtr)
	, Outer(InOuter)
	, bHasParsedJsonObject(false)
	, bRequiresInterlockedParsing(false)
{
	// Backwards-compatibility; inherit the flag that only allows one thread at a time into the JSON parsing logic.
	const FRigVMImaginaryFiBDataSharedPtr OuterPtr = Outer.Pin();
	if (OuterPtr.IsValid())
	{
		bRequiresInterlockedParsing = OuterPtr->bRequiresInterlockedParsing;
	}
}

FRigVMSearchResult FRigVMImaginaryFiBData::CreateSearchResult(FRigVMSearchResult InParent) const
{
	CSV_SCOPED_TIMING_STAT(RigVMFindInBlueprint, CreateSearchResult);

	FRigVMSearchResult ReturnSearchResult = CreateSearchResult_Internal(SearchResultTemplate);
	if (ReturnSearchResult.IsValid())
	{
		ReturnSearchResult->Parent = InParent;

		if (!FRigVMFindInBlueprintSearchManager::Get().ShouldEnableSearchResultTemplates())
		{
			for (const TPair<RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage, FRigVMSearchableValueInfo>& TagsAndValues : ParsedTagsAndValues)
			{
				if (TagsAndValues.Value.IsCoreDisplay() || !TagsAndValues.Value.IsSearchable())
				{
					FText Value = TagsAndValues.Value.GetDisplayText(*LookupTablePtr);
					ReturnSearchResult->ParseSearchInfo(TagsAndValues.Key.Text, Value);
				}
			}
		}
	}

	return ReturnSearchResult;
}

FRigVMSearchResult FRigVMImaginaryFiBData::CreateSearchTree(FRigVMSearchResult InParentSearchResult, FRigVMImaginaryFiBDataWeakPtr InCurrentPointer, TArray< const FRigVMImaginaryFiBData* >& InValidSearchResults, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InMatchingSearchComponents)
{
	CSV_SCOPED_TIMING_STAT(RigVMFindInBlueprint, CreateSearchTree);
	CSV_CUSTOM_STAT(RigVMFindInBlueprint, CreateSearchTreeIterations, 1, ECsvCustomStatOp::Accumulate);

	FRigVMImaginaryFiBDataSharedPtr CurrentDataPtr = InCurrentPointer.Pin();
	if (FRigVMImaginaryFiBData* CurrentData = CurrentDataPtr.Get())
	{
		FRigVMSearchResult CurrentSearchResult = CurrentData->CreateSearchResult(InParentSearchResult);
		bool bValidSearchResults = false;

		// Check all children first, to see if they are valid in the search results
		for (FRigVMImaginaryFiBDataSharedPtr ChildData : CurrentData->ParsedChildData)
		{
			FRigVMSearchResult Result = CreateSearchTree(CurrentSearchResult, ChildData, InValidSearchResults, InMatchingSearchComponents);
			if (Result.IsValid())
			{
				bValidSearchResults = true;
				CurrentSearchResult->Children.Add(Result);
			}
		}

		// If the children did not match the search results but this item does, then we will want to return true.
		// Include "tag+value" categories in the search tree, as the relevant results need to be added as children.
		const bool bInvalidSearchResultsCategory = CurrentData->IsCategory() && !CurrentData->IsTagAndValueCategory();
		if (!bValidSearchResults && !bInvalidSearchResultsCategory && (InValidSearchResults.Find(CurrentData) != INDEX_NONE || InMatchingSearchComponents.Find(CurrentData)))
		{
			bValidSearchResults = true;
		}

		if (bValidSearchResults)
		{
			TArray< FRigVMComponentUniqueDisplay > SearchResultList;
			InMatchingSearchComponents.MultiFind(CurrentData, SearchResultList, true);
			CurrentSearchResult->Children.Reserve(CurrentSearchResult->Children.Num() + SearchResultList.Num());

			// Add any data that matched the search results as a child of our search result
			for (FRigVMComponentUniqueDisplay& SearchResultWrapper : SearchResultList)
			{
				SearchResultWrapper.SearchResult->Parent = CurrentSearchResult;
				CurrentSearchResult->Children.Add(SearchResultWrapper.SearchResult);
			}
			return CurrentSearchResult;
		}
	}
	return nullptr;
}

bool FRigVMImaginaryFiBData::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return true;
}

bool FRigVMImaginaryFiBData::CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	// Always compatible with the AllFilter
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter;
}

void FRigVMImaginaryFiBData::ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::RigVMSearchable*/)
{
	if (UnparsedJsonObject.IsValid())
	{
		if (InSearchabilityOverride & ERigVMSearchableValueStatus::RigVMSearchable)
		{
			TSharedPtr< FJsonObject > MetaDataField;
			for (auto MapValues : UnparsedJsonObject->Values)
			{
				FText KeyText = RigVMFindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
				if (!KeyText.CompareTo(FRigVMFindInBlueprintSearchTags::FiBMetaDataTag))
				{
					MetaDataField = MapValues.Value->AsObject();
					break;
				}
			}

			if (MetaDataField.IsValid())
			{
				TSharedPtr<FRigVMFiBMetaData, ESPMode::ThreadSafe> MetaDataFiBInfo = MakeShareable(new FRigVMFiBMetaData(AsShared(), MetaDataField, LookupTablePtr));
				MetaDataFiBInfo->ParseAllChildData_Internal();

				if (MetaDataFiBInfo->IsHidden() && MetaDataFiBInfo->IsExplicit())
				{
					InSearchabilityOverride = ERigVMSearchableValueStatus::RigVMExplicitySearchableHidden;
				}
				else if (MetaDataFiBInfo->IsExplicit())
				{
					InSearchabilityOverride = ERigVMSearchableValueStatus::RigVMExplicitySearchable;
				}
			}
		}

		for( auto MapValues : UnparsedJsonObject->Values )
		{
			FText KeyText = RigVMFindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
			TSharedPtr< FJsonValue > JsonValue = MapValues.Value;

			if (!KeyText.CompareTo(FRigVMFindInBlueprintSearchTags::FiBMetaDataTag))
			{
				// Do not let this be processed again
				continue;
			}
			if (!TrySpecialHandleJsonValue(KeyText, JsonValue))
			{
				TArray<FRigVMSearchableValueInfo> ParsedValues;
				ParseJsonValue(KeyText, KeyText, JsonValue, ParsedValues, false, InSearchabilityOverride);

				if (FRigVMFindInBlueprintSearchManager::Get().ShouldEnableSearchResultTemplates())
				{
					for (const FRigVMSearchableValueInfo& ParsedValue : ParsedValues)
					{
						if (ParsedValue.IsCoreDisplay() || !ParsedValue.IsSearchable())
						{
							// If necessary, create the search result template.
							if (!SearchResultTemplate.IsValid())
							{
								FRigVMSearchResult NullTemplate;
								SearchResultTemplate = CreateSearchResult_Internal(NullTemplate);
								check(SearchResultTemplate.IsValid());
							}

							// Parse out meta values used for display and cache them in the template.
							SearchResultTemplate->ParseSearchInfo(KeyText, ParsedValue.GetDisplayText(*LookupTablePtr));
						}
					}
				}
			}
		}
	}

	UnparsedJsonObject.Reset();
}

void FRigVMImaginaryFiBData::ParseAllChildData(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::RigVMSearchable*/)
{
	CSV_SCOPED_TIMING_STAT(RigVMFindInBlueprint, ParseAllChildData);
	CSV_CUSTOM_STAT(RigVMFindInBlueprint, ParseAllChildDataIterations, 1, ECsvCustomStatOp::Accumulate);

	if (bRequiresInterlockedParsing)
	{
		ParseChildDataCriticalSection.Lock();
	}

	if (!bHasParsedJsonObject)
	{
		ParseAllChildData_Internal(InSearchabilityOverride);
		bHasParsedJsonObject = true;
	}

	if (bRequiresInterlockedParsing)
	{
		ParseChildDataCriticalSection.Unlock();
	}
}

void FRigVMImaginaryFiBData::ParseJsonValue(FText InKey, FText InDisplayKey, TSharedPtr< FJsonValue > InJsonValue, TArray<FRigVMSearchableValueInfo>& OutParsedValues, bool bIsInArray/*=false*/, ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::RigVMSearchable*/)
{
	ERigVMSearchableValueStatus SearchabilityStatus = (InSearchabilityOverride == ERigVMSearchableValueStatus::RigVMSearchable)? GetSearchabilityStatus(InKey.ToString()) : InSearchabilityOverride;
	if( InJsonValue->Type == EJson::Array)
	{
		TSharedPtr< FRigVMCategorySectionHelper, ESPMode::ThreadSafe > ArrayCategory = MakeShareable(new FRigVMCategorySectionHelper(AsShared(), LookupTablePtr, InKey, true));
		ParsedChildData.Add(ArrayCategory);
		TArray<TSharedPtr< FJsonValue > > ArrayList = InJsonValue->AsArray();
		for( int32 ArrayIdx = 0; ArrayIdx < ArrayList.Num(); ++ArrayIdx)
		{
			TSharedPtr< FJsonValue > ArrayValue = ArrayList[ArrayIdx];
			ArrayCategory->ParseJsonValue(InKey, FText::FromString(FString::FromInt(ArrayIdx)), ArrayValue, OutParsedValues, /*bIsInArray=*/true, SearchabilityStatus);
		}
	}
	else if (InJsonValue->Type == EJson::Object)
	{
		TSharedPtr< FRigVMCategorySectionHelper, ESPMode::ThreadSafe > SubObjectCategory = MakeShareable(new FRigVMCategorySectionHelper(AsShared(), InJsonValue->AsObject(), LookupTablePtr, InDisplayKey, bIsInArray));
		SubObjectCategory->ParseAllChildData(SearchabilityStatus);
		ParsedChildData.Add(SubObjectCategory);
	}
	else
	{
		FRigVMSearchableValueInfo& ParsedValue = OutParsedValues.AddDefaulted_GetRef();
		if (InJsonValue->Type == EJson::String)
		{
			ParsedValue = FRigVMSearchableValueInfo(InDisplayKey, FCString::Atoi(*InJsonValue->AsString()), SearchabilityStatus);
		}
		else
		{
			// For everything else, there's this. Numbers come here and will be treated as strings
			ParsedValue = FRigVMSearchableValueInfo(InDisplayKey, FText::FromString(InJsonValue->AsString()), SearchabilityStatus);
		}

		ParsedTagsAndValues.Add(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(InKey), ParsedValue);
	}
}

FText FRigVMImaginaryFiBData::CreateSearchComponentDisplayText(FText InKey, FText InValue) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Key"), InKey);
	Args.Add(TEXT("Value"), InValue);
	return FText::Format(LOCTEXT("ExtraSearchInfo", "{Key}: {Value}"), Args);
}

bool FRigVMImaginaryFiBData::TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InOutMatchingSearchComponents) const
{
	bool bMatchesSearchQuery = false;
	for(const TPair< RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage, FRigVMSearchableValueInfo >& ParsedValues : ParsedTagsAndValues )
	{
		if (ParsedValues.Value.IsSearchable() && !ParsedValues.Value.IsExplicitSearchable())
		{
			FText Value = ParsedValues.Value.GetDisplayText(*LookupTablePtr);
			FString ValueAsString = Value.ToString();
			ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));
			bool bMatchesSearch = TextFilterUtils::TestBasicStringExpression(MoveTemp(ValueAsString), InValue, InTextComparisonMode) || TextFilterUtils::TestBasicStringExpression(Value.BuildSourceString(), InValue, InTextComparisonMode);

			if (bMatchesSearch && !ParsedValues.Value.IsCoreDisplay())
			{
				FRigVMSearchResult SearchResult = MakeShared<FRigVMFindInBlueprintsResult>(CreateSearchComponentDisplayText(ParsedValues.Value.GetDisplayKey(), Value));
				InOutMatchingSearchComponents.Add(this, FRigVMComponentUniqueDisplay(SearchResult));
			}

			bMatchesSearchQuery |= bMatchesSearch;
		}
	}
	// Any children that are treated as a TagAndValue Category should be added for independent searching
	for (const FRigVMImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		if (Child->IsTagAndValueCategory())
		{
			bMatchesSearchQuery |= Child->TestBasicStringExpression(InValue, InTextComparisonMode, InOutMatchingSearchComponents);
		}
	}

	return bMatchesSearchQuery;
}

bool FRigVMImaginaryFiBData::TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InOutMatchingSearchComponents) const
{
	bool bMatchesSearchQuery = false;
	for (const TPair< RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage, FRigVMSearchableValueInfo >& TagsValuePair : ParsedTagsAndValues)
	{
		if (TagsValuePair.Value.IsSearchable())
		{
			if (TagsValuePair.Key.Text.ToString() == InKey.ToString() || TagsValuePair.Key.Text.BuildSourceString() == InKey.ToString())
			{
				FText Value = TagsValuePair.Value.GetDisplayText(*LookupTablePtr);
				FString ValueAsString = Value.ToString();
				ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));
				bool bMatchesSearch = TextFilterUtils::TestComplexExpression(MoveTemp(ValueAsString), InValue, InComparisonOperation, InTextComparisonMode) || TextFilterUtils::TestComplexExpression(Value.BuildSourceString(), InValue, InComparisonOperation, InTextComparisonMode);

				if (bMatchesSearch && !TagsValuePair.Value.IsCoreDisplay())
				{
					FRigVMSearchResult SearchResult = MakeShared<FRigVMFindInBlueprintsResult>(CreateSearchComponentDisplayText(TagsValuePair.Value.GetDisplayKey(), Value));
					InOutMatchingSearchComponents.Add(this, FRigVMComponentUniqueDisplay(SearchResult));
				}
				bMatchesSearchQuery |= bMatchesSearch;
			}
		}
	}

	// Any children that are treated as a TagAndValue Category should be added for independent searching
	for (const FRigVMImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		if (Child->IsTagAndValueCategory())
		{
			bMatchesSearchQuery |= Child->TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode, InOutMatchingSearchComponents);
		}
	}
	return bMatchesSearchQuery;
}

UObject* FRigVMImaginaryFiBData::GetObject(UBlueprint* InBlueprint) const
{
	return CreateSearchResult(nullptr)->GetObject(InBlueprint);
}

void FRigVMImaginaryFiBData::DumpParsedObject(FArchive& Ar, int32 InTreeLevel) const
{
	FString CommaStr = TEXT(",");
	for (int32 i = 0; i < InTreeLevel; ++i)
	{
		Ar.Serialize(TCHAR_TO_ANSI(*CommaStr), CommaStr.Len());
	}

	DumpParsedObject_Internal(Ar);

	for (const TPair< RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage, FRigVMSearchableValueInfo >& TagsValuePair : ParsedTagsAndValues)
	{
		FText Value = TagsValuePair.Value.GetDisplayText(*LookupTablePtr);
		FString ValueAsString = Value.ToString();
		ValueAsString.ReplaceInline(TEXT(" "), TEXT(""));

		FString LineStr = FString::Printf(TEXT(",%s:%s"), *TagsValuePair.Key.Text.ToString(), *ValueAsString);
		Ar.Serialize(TCHAR_TO_ANSI(*LineStr), LineStr.Len());
	}

	FString NewLine(TEXT("\n"));
	Ar.Serialize(TCHAR_TO_ANSI(*NewLine), NewLine.Len());

	for (const FRigVMImaginaryFiBDataSharedPtr& Child : ParsedChildData)
	{
		Child->DumpParsedObject(Ar, InTreeLevel + 1);
	}

	if (InTreeLevel == 0)
	{
		Ar.Serialize(TCHAR_TO_ANSI(*NewLine), NewLine.Len());
	}
}

///////////////////////////
// FRigVMFiBMetaData

FRigVMFiBMetaData::FRigVMFiBMetaData(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, bIsHidden(false)
	, bIsExplicit(false)
{
}

bool FRigVMFiBMetaData::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	bool bResult = false;
	if (InKey.ToString() == FRigVMFiBMD::FiBSearchableExplicitMD)
	{
		bIsExplicit = true;
		bResult = true;
	}
	else if (InKey.ToString() == FRigVMFiBMD::FiBSearchableHiddenExplicitMD)
	{
		bIsExplicit = true;
		bIsHidden = true;
		bResult = true;
	}
	ensure(bResult);
	return bResult;
}

///////////////////////////
// FRigVMCategorySectionHelper

FRigVMCategorySectionHelper::FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory)
	: FRigVMImaginaryFiBData(InOuter, nullptr, InLookupTablePtr)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
{
}

FRigVMCategorySectionHelper::FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
{
}

FRigVMCategorySectionHelper::FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory, FRigVMCategorySectionHelperCallback InSpecialHandlingCallback)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, SpecialHandlingCallback(InSpecialHandlingCallback)
	, CategoryName(InCategoryName)
	, bIsTagAndValue(bInTagAndValueCategory)
{

}

bool FRigVMCategorySectionHelper::CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return true;
}

FRigVMSearchResult FRigVMCategorySectionHelper::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsResult>(*InTemplate);
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsResult>(CategoryName);
	}
}

void FRigVMCategorySectionHelper::ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::RigVMSearchable*/)
{
	if (UnparsedJsonObject.IsValid() && SpecialHandlingCallback.IsBound())
	{
		SpecialHandlingCallback.Execute(UnparsedJsonObject, ParsedChildData);
		UnparsedJsonObject.Reset();
	}
	else
	{
		bool bHasMetaData = false;
		bool bHasOneOtherItem = false;
		if (UnparsedJsonObject.IsValid() && UnparsedJsonObject->Values.Num() == 2)
		{
			for( auto MapValues : UnparsedJsonObject->Values )
			{
				FText KeyText = RigVMFindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
				if (!KeyText.CompareTo(FRigVMFindInBlueprintSearchTags::FiBMetaDataTag))
				{
					bHasMetaData = true;
				}
				else
				{
					bHasOneOtherItem = true;
				}
			}

			// If we have metadata and only one other item, we should be treated like a tag and value category
			bIsTagAndValue |= (bHasOneOtherItem && bHasMetaData);
		}

		FRigVMImaginaryFiBData::ParseAllChildData_Internal(InSearchabilityOverride);
	}
}

void FRigVMCategorySectionHelper::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMCategorySectionHelper,CategoryName:%s,IsTagAndValueCategory:%s"), *CategoryName.ToString(), IsTagAndValueCategory() ? TEXT("true") : TEXT("false"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

//////////////////////////////////////////
// FRigVMImaginaryBlueprint

FRigVMImaginaryBlueprint::FRigVMImaginaryBlueprint(const FString& InBlueprintName, const FString& InBlueprintPath, const FString& InBlueprintParentClass, const TArray<FString>& InInterfaces, const FString& InUnparsedStringData, FRigVMSearchDataVersionInfo InVersionInfo)
	: FRigVMImaginaryFiBData(nullptr)
	, BlueprintPath(InBlueprintPath)
{
	ParseToJson(InVersionInfo, InUnparsedStringData);
	LookupTablePtr = &LookupTable;
	ParsedTagsAndValues.Add(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(FRigVMFindInBlueprintSearchTags::FiB_Name), FRigVMSearchableValueInfo(FRigVMFindInBlueprintSearchTags::FiB_Name, FText::FromString(InBlueprintName), ERigVMSearchableValueStatus::RigVMExplicitySearchable));
	ParsedTagsAndValues.Add(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(FRigVMFindInBlueprintSearchTags::FiB_Path), FRigVMSearchableValueInfo(FRigVMFindInBlueprintSearchTags::FiB_Path, FText::FromString(InBlueprintPath), ERigVMSearchableValueStatus::RigVMExplicitySearchable));
	ParsedTagsAndValues.Add(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(FRigVMFindInBlueprintSearchTags::FiB_ParentClass), FRigVMSearchableValueInfo(FRigVMFindInBlueprintSearchTags::FiB_ParentClass, FText::FromString(InBlueprintParentClass), ERigVMSearchableValueStatus::RigVMExplicitySearchable));

	TSharedPtr< FRigVMCategorySectionHelper, ESPMode::ThreadSafe > InterfaceCategory = MakeShareable(new FRigVMCategorySectionHelper(nullptr, &LookupTable, FRigVMFindInBlueprintSearchTags::FiB_Interfaces, true));
	for( int32 InterfaceIdx = 0; InterfaceIdx < InInterfaces.Num(); ++InterfaceIdx)
	{
		const FString& Interface = InInterfaces[InterfaceIdx];
		FText Key = FText::FromString(FString::FromInt(InterfaceIdx));
		FRigVMSearchableValueInfo Value(Key, FText::FromString(Interface), ERigVMSearchableValueStatus::RigVMExplicitySearchable);
		InterfaceCategory->AddKeyValuePair(FRigVMFindInBlueprintSearchTags::FiB_Interfaces, Value);
	}
	ParsedChildData.Add(InterfaceCategory);
}

FRigVMSearchResult FRigVMImaginaryBlueprint::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsResult>(*InTemplate);
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsResult>(ParsedTagsAndValues.Find(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(FRigVMFindInBlueprintSearchTags::FiB_Path))->GetDisplayText(LookupTable));
	}
}

void FRigVMImaginaryBlueprint::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMImaginaryBlueprint"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

UBlueprint* FRigVMImaginaryBlueprint::GetBlueprint() const
{
	return Cast<UBlueprint>(GetObject(nullptr));
}

bool FRigVMImaginaryBlueprint::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter || InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMBlueprintFilter;
}

bool FRigVMImaginaryBlueprint::CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMNodesFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPinsFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMGraphsFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMUberGraphsFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMFunctionsFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMMacrosFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPropertiesFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMVariablesFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMComponentsFilter ||
		FRigVMImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

void FRigVMImaginaryBlueprint::ParseToJson(FRigVMSearchDataVersionInfo InVersionInfo, const FString& UnparsedStringData)
{
	UnparsedJsonObject = FRigVMFindInBlueprintSearchManager::ConvertJsonStringToObject(InVersionInfo, UnparsedStringData, LookupTable);
}

bool FRigVMImaginaryBlueprint::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	bool bResult = false;

	if(!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Properties))
	{
		// Pulls out all properties (variables) for this Blueprint
		TArray<TSharedPtr< FJsonValue > > PropertyList = InJsonValue->AsArray();
		for( TSharedPtr< FJsonValue > PropertyValue : PropertyList )
		{
			ParsedChildData.Add(MakeShareable(new FRigVMImaginaryProperty(AsShared(), PropertyValue->AsObject(), &LookupTable)));
		}
		bResult = true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Functions))
	{
		ParseGraph(InJsonValue, FRigVMFindInBlueprintSearchTags::FiB_Functions.ToString(), GT_Function);
		bResult = true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Macros))
	{
		ParseGraph(InJsonValue, FRigVMFindInBlueprintSearchTags::FiB_Macros.ToString(), GT_Macro);
		bResult = true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_UberGraphs))
	{
		ParseGraph(InJsonValue, FRigVMFindInBlueprintSearchTags::FiB_UberGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_SubGraphs))
	{
		ParseGraph(InJsonValue, FRigVMFindInBlueprintSearchTags::FiB_SubGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_ExtensionGraphs))
	{
		ParseGraph(InJsonValue, FRigVMFindInBlueprintSearchTags::FiB_ExtensionGraphs.ToString(), GT_Ubergraph);
		bResult = true;
	}
	else if(!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Components))
	{
		TArray<TSharedPtr< FJsonValue > > ComponentList = InJsonValue->AsArray();
		TSharedPtr< FJsonObject > ComponentsWrapperObject(new FJsonObject);
		ComponentsWrapperObject->Values.Add(FRigVMFindInBlueprintSearchTags::FiB_Components.ToString(), InJsonValue);
		ParsedChildData.Add(MakeShareable(new FRigVMCategorySectionHelper(AsShared(), ComponentsWrapperObject, &LookupTable, FRigVMFindInBlueprintSearchTags::FiB_Components, false, FRigVMCategorySectionHelper::FRigVMCategorySectionHelperCallback::CreateRaw(this, &FRigVMImaginaryBlueprint::ParseComponents))));
		bResult = true;
	}

	if (!bResult)
	{
		bResult = FRigVMImaginaryFiBData::TrySpecialHandleJsonValue(InKey, InJsonValue);
	}
	return bResult;
}

void FRigVMImaginaryBlueprint::ParseGraph( TSharedPtr< FJsonValue > InJsonValue, FString InCategoryTitle, EGraphType InGraphType )
{
	TArray<TSharedPtr< FJsonValue > > GraphList = InJsonValue->AsArray();
	for( TSharedPtr< FJsonValue > GraphValue : GraphList )
	{
		ParsedChildData.Add(MakeShareable(new FRigVMImaginaryGraph(AsShared(), GraphValue->AsObject(), &LookupTable, InGraphType)));
	}
}

void FRigVMImaginaryBlueprint::ParseComponents(TSharedPtr< FJsonObject > InJsonObject, TArray<FRigVMImaginaryFiBDataSharedPtr>& OutParsedChildData)
{
	// Pulls out all properties (variables) for this Blueprint
	TArray<TSharedPtr< FJsonValue > > ComponentList = InJsonObject->GetArrayField(FRigVMFindInBlueprintSearchTags::FiB_Components.ToString());
	for( TSharedPtr< FJsonValue > ComponentValue : ComponentList )
	{
		OutParsedChildData.Add(MakeShareable(new FRigVMImaginaryComponent(AsShared(), ComponentValue->AsObject(), &LookupTable)));
	}
}

//////////////////////////
// FRigVMImaginaryGraph

FRigVMImaginaryGraph::FRigVMImaginaryGraph(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, EGraphType InGraphType)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, GraphType(InGraphType)
{
}

FRigVMSearchResult FRigVMImaginaryGraph::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsGraph>(*StaticCastSharedPtr<FRigVMFindInBlueprintsGraph>(InTemplate));
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsGraph>(GraphType);
	}
}

void FRigVMImaginaryGraph::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMImaginaryGraph"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

bool FRigVMImaginaryGraph::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMGraphsFilter ||
		(GraphType == GT_Ubergraph && InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMUberGraphsFilter) ||
		(GraphType == GT_Function && InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMFunctionsFilter) ||
		(GraphType == GT_Macro && InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMMacrosFilter);
}

bool FRigVMImaginaryGraph::CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPinsFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMNodesFilter ||
		(GraphType == GT_Function && InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPropertiesFilter) ||
		(GraphType == GT_Function && InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMVariablesFilter) ||
		FRigVMImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

ERigVMSearchableValueStatus FRigVMImaginaryGraph::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Name, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMCoreDisplayItem;
	}

	return ERigVMSearchableValueStatus::RigVMSearchable;
}

bool FRigVMImaginaryGraph::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Nodes))
	{
		TArray< TSharedPtr< FJsonValue > > NodeList = InJsonValue->AsArray();

		for( TSharedPtr< FJsonValue > NodeValue : NodeList )
		{
			ParsedChildData.Add(MakeShareable(new FRigVMImaginaryGraphNode(AsShared(), NodeValue->AsObject(), LookupTablePtr)));
		}
		return true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Properties))
	{
		// Pulls out all properties (local variables) for this graph
		TArray<TSharedPtr< FJsonValue > > PropertyList = InJsonValue->AsArray();
		for( TSharedPtr< FJsonValue > PropertyValue : PropertyList )
		{
			ParsedChildData.Add(MakeShareable(new FRigVMImaginaryProperty(AsShared(), PropertyValue->AsObject(), LookupTablePtr)));
		}
		return true;
	}
	return false;
}

//////////////////////////////////////
// FRigVMImaginaryGraphNode

FRigVMImaginaryGraphNode::FRigVMImaginaryGraphNode(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
{
}

FRigVMSearchResult FRigVMImaginaryGraphNode::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsGraphNode>(*StaticCastSharedPtr<FRigVMFindInBlueprintsGraphNode>(InTemplate));
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsGraphNode>();
	}
}

void FRigVMImaginaryGraphNode::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMImaginaryGraphNode"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

bool FRigVMImaginaryGraphNode::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter || InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMNodesFilter;
}

bool FRigVMImaginaryGraphNode::CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPinsFilter ||
		FRigVMImaginaryFiBData::CanCallFilter(InSearchQueryFilter);
}

ERigVMSearchableValueStatus FRigVMImaginaryGraphNode::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Name, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_NativeName, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Comment, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMCoreDisplayItem;
	}
	else if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Glyph, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_GlyphStyleSet, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_GlyphColor, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_NodeGuid, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMNotSearchable;
	}
	else if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_ClassName, InKey))
	{
		return ERigVMSearchableValueStatus::RigVMExplicitySearchable;
	}
	return ERigVMSearchableValueStatus::RigVMSearchable;
}

bool FRigVMImaginaryGraphNode::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_Pins))
	{
		TArray< TSharedPtr< FJsonValue > > PinsList = InJsonValue->AsArray();

		for( TSharedPtr< FJsonValue > Pin : PinsList )
		{
			ParsedChildData.Add(MakeShareable(new FRigVMImaginaryPin(AsShared(), Pin->AsObject(), LookupTablePtr, SchemaName)));
		}
		return true;
	}
	else if (!InKey.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_SchemaName))
	{
		// Previously extracted
		return true;
	}

	return false;
}

void FRigVMImaginaryGraphNode::ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::RigVMSearchable*/)
{
	if (UnparsedJsonObject.IsValid())
	{
		TSharedPtr< FJsonObject > JsonObject = UnparsedJsonObject;
		// Very important to get the schema first, other bits of data depend on it
		for (auto MapValues : UnparsedJsonObject->Values)
		{
			FText KeyText = RigVMFindInBlueprintsHelpers::AsFText(FCString::Atoi(*MapValues.Key), *LookupTablePtr);
			if (!KeyText.CompareTo(FRigVMFindInBlueprintSearchTags::FiB_SchemaName))
			{
				TSharedPtr< FJsonValue > SchemaNameValue = MapValues.Value;
				SchemaName = RigVMFindInBlueprintsHelpers::AsFText(SchemaNameValue, *LookupTablePtr).ToString();
				break;
			}
		}

		FRigVMImaginaryFiBData::ParseAllChildData_Internal(InSearchabilityOverride);
	}
}

///////////////////////////////////////////
// FRigVMImaginaryProperty

FRigVMImaginaryProperty::FRigVMImaginaryProperty(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
{
}

bool FRigVMImaginaryProperty::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPropertiesFilter ||
		InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMVariablesFilter;
}

FRigVMSearchResult FRigVMImaginaryProperty::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsProperty>(*StaticCastSharedPtr<FRigVMFindInBlueprintsProperty>(InTemplate));
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsProperty>();
	}
}

void FRigVMImaginaryProperty::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMImaginaryProperty"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

ERigVMSearchableValueStatus FRigVMImaginaryProperty::GetSearchabilityStatus(FString InKey)
{
	// This is a non-ideal way to assign searchability vs being a core display item and will be resolved in future versions of the FiB data in the AR
	if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Name, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMCoreDisplayItem;
	}
	else if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_PinCategory, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_PinSubCategory, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_ObjectClass, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsArray, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsReference, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsSCSComponent, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMExplicitySearchableHidden;
	}
	return ERigVMSearchableValueStatus::RigVMSearchable;
}

//////////////////////////////
// FRigVMImaginaryComponent

FRigVMImaginaryComponent::FRigVMImaginaryComponent(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr)
	: FRigVMImaginaryProperty(InOuter, InUnparsedJsonObject, InLookupTablePtr)
{
}

bool FRigVMImaginaryComponent::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return FRigVMImaginaryProperty::IsCompatibleWithFilter(InSearchQueryFilter) || InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMComponentsFilter;
}

//////////////////////////////
// FRigVMImaginaryPin

FRigVMImaginaryPin::FRigVMImaginaryPin(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FString InSchemaName)
	: FRigVMImaginaryFiBData(InOuter, InUnparsedJsonObject, InLookupTablePtr)
	, SchemaName(InSchemaName)
{
}

bool FRigVMImaginaryPin::IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const
{
	return InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMAllFilter || InSearchQueryFilter == ERigVMSearchQueryFilter::RigVMPinsFilter;
}

FRigVMSearchResult FRigVMImaginaryPin::CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const
{
	if (InTemplate.IsValid())
	{
		return MakeShared<FRigVMFindInBlueprintsPin>(*StaticCastSharedPtr<FRigVMFindInBlueprintsPin>(InTemplate));
	}
	else
	{
		return MakeShared<FRigVMFindInBlueprintsPin>(SchemaName);
	}
}

void FRigVMImaginaryPin::DumpParsedObject_Internal(FArchive& Ar) const
{
	FString OutputString = FString::Printf(TEXT("FRigVMImaginaryPin"));
	Ar.Serialize(TCHAR_TO_ANSI(*OutputString), OutputString.Len());
}

ERigVMSearchableValueStatus FRigVMImaginaryPin::GetSearchabilityStatus(FString InKey)
{
	if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_Name, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_NativeName, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMCoreDisplayItem;
	}
	else if (RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_PinCategory, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_PinSubCategory, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_ObjectClass, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsArray, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsReference, InKey)
		|| RigVMFindInBlueprintsHelpers::IsTextEqualToString(FRigVMFindInBlueprintSearchTags::FiB_IsSCSComponent, InKey)
		)
	{
		return ERigVMSearchableValueStatus::RigVMExplicitySearchableHidden;
	}
	return ERigVMSearchableValueStatus::RigVMSearchable;
}

bool FRigVMImaginaryPin::TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue)
{
	return false;
}

#undef LOCTEXT_NAMESPACE

#endif