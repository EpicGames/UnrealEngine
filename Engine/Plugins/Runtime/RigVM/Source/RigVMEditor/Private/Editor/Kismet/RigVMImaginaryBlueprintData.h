// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !WITH_RIGVMLEGACYEDITOR

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "Editor/Kismet/RigVMFindInBlueprintManager.h"
#include "Editor/Kismet/RigVMFindInBlueprints.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilterUtils.h"
#include "Templates/Atomic.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FArchive;
class FJsonObject;
class FJsonValue;
class UBlueprint;
class UObject;

enum ERigVMSearchableValueStatus
{
	RigVMNotSearchable = 0x00000000, // Cannot search this value, it is used for display purposes only
	RigVMSearchable = 0x00000001, // Generically searchable, value will appear as a sub-item and has no sub-data
	RigVMHidden = 0x00000002, // Item will not be shown
	RigVMExplicit = 0x00000004, // Item must be explicitly requested via the tag


	RigVMCoreDisplayItem = RigVMHidden | RigVMSearchable, // Core display items are searchable but should not display as sub-items because their data is presented in another fashion.
	RigVMExplicitySearchable = RigVMExplicit | RigVMSearchable, // Will only be allowed to be found if searching using a tag
	RigVMExplicitySearchableHidden = RigVMExplicit | RigVMSearchable | RigVMHidden, // Will only be allowed to be found if searching using a tag but will not display the tag in the results (because it is a CoreDisplayItem)
	RigVMAllSearchable = RigVMCoreDisplayItem | RigVMExplicitySearchable, // Covers all searchability methods
};

class FRigVMSearchableValueInfo
{
public:
	FRigVMSearchableValueInfo()
		: SearchableValueStatus(ERigVMSearchableValueStatus::RigVMNotSearchable)
		, LookupTableKey(-1)
	{
	}

	FRigVMSearchableValueInfo(FText InDisplayKey, int32 InLookUpTableKey)
		: SearchableValueStatus(ERigVMSearchableValueStatus::RigVMSearchable)
		, DisplayKey(InDisplayKey)
		, LookupTableKey(InLookUpTableKey)
	{
	}

	FRigVMSearchableValueInfo(FText InDisplayKey, FText InDisplayText)
		: SearchableValueStatus(ERigVMSearchableValueStatus::RigVMSearchable)
		, DisplayKey(InDisplayKey)
		, LookupTableKey(-1)
		, DisplayText(InDisplayText)
	{
	}

	FRigVMSearchableValueInfo(FText InDisplayKey, int32 InLookUpTableKey, ERigVMSearchableValueStatus InSearchableValueStatus)
		: SearchableValueStatus(InSearchableValueStatus)
		, DisplayKey(InDisplayKey)
		, LookupTableKey(InLookUpTableKey)
	{
	}

	FRigVMSearchableValueInfo(FText InDisplayKey, FText InDisplayText, ERigVMSearchableValueStatus InSearchableValueStatus)
		: SearchableValueStatus(InSearchableValueStatus)
		, DisplayKey(InDisplayKey)
		, LookupTableKey(-1)
		, DisplayText(InDisplayText)
	{
	}

	/** Returns TRUE if the data is searchable */
	bool IsSearchable() const { return (SearchableValueStatus & ERigVMSearchableValueStatus::RigVMSearchable) != 0; }

	/** Returns TRUE if the item should be treated as a CoreDisplayItem, which is searchable but not displayed */
	bool IsCoreDisplay() const { return (SearchableValueStatus & ERigVMSearchableValueStatus::RigVMCoreDisplayItem) == ERigVMSearchableValueStatus::RigVMCoreDisplayItem; }

	/** Returns TRUE if the item should only be searchable if explicitly searched for using the tag */
	bool IsExplicitSearchable() const { return (SearchableValueStatus & ERigVMSearchableValueStatus::RigVMExplicitySearchable) == ERigVMSearchableValueStatus::RigVMExplicitySearchable; }

	/** Returns the display text to use for this item */
	FText GetDisplayText(const TMap<int32, FText>& InLookupTable) const;

	/** Returns the display key for this item */
	FText GetDisplayKey() const
	{
		return DisplayKey;
	}

protected:
	/** The searchability status of this item */
	ERigVMSearchableValueStatus SearchableValueStatus;

	/** Key that this item is associated with, used for display purposes */
	FText DisplayKey;

	/** Key to use to lookup into a table if DisplayText does not override */
	int32 LookupTableKey;

	/** Text value to use instead of a lookup into the table */
	FText DisplayText;
};

/** Struct to contain search results and help compare them for uniqueness. */
struct FRigVMComponentUniqueDisplay
{
	FRigVMComponentUniqueDisplay( FRigVMSearchResult InSearchResult )
		: SearchResult(InSearchResult)
	{}

	bool operator==(const FRigVMComponentUniqueDisplay& Other);

	/** Search result contained and used for comparing of uniqueness */
	FRigVMSearchResult SearchResult;
};

class FRigVMImaginaryFiBData : public ITextFilterExpressionContext, public TSharedFromThis<FRigVMImaginaryFiBData, ESPMode::ThreadSafe>
{
public:
	FRigVMImaginaryFiBData(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject = TSharedPtr<FJsonObject>(), TMap<int32, FText>* InLookupTablePtr = nullptr);

	/** ITextFilterExpressionContext Interface */
	// We don't actually use these overrides, see FFiBContextHelper for how we call the alternate functions. These will assert if they are accidentally called.
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override { ensure(0); return false; };
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override { ensure(0); return false; };
	/** End ITextFilterExpressionContext Interface */

	/** Returns TRUE if this item is a category type, which helps to organize child data */
	virtual bool IsCategory() const { return false; }

	/** Returns TRUE if this item is considered a Tag and Value category, where it's contents should be considered no different than the parent owner */
	virtual bool IsTagAndValueCategory() const { return false; }

	/** Checks if the filter is compatible with the current object, returns TRUE by default */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const;

	/** Checks if the filter can call functions for the passed filter, returns FALSE by default if the filter is not the AllFilter */
	virtual bool CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const;

	/** Parses, in a thread-safe manner, all child data, non-recursively, so children will be left in an unparsed Json state */
	void ParseAllChildData(ERigVMSearchableValueStatus InSearchabilityOverride = ERigVMSearchableValueStatus::RigVMSearchable);

	/** Test the given value against the strings extracted from the current item. Will return the matching search components if any (can return TRUE without having any if the search components are hidden) */
	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InOutMatchingSearchComponents) const;

	/** Perform a complex expression test for the current item. Will return the matching search components if any (can return TRUE without having any if the search components are hidden) */
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InOutMatchingSearchComponents) const;

	/** Returns the UObject represented by this Imaginary data give the UBlueprint owner. */
	virtual UObject* GetObject(UBlueprint* InBlueprint) const;

	/** This will return and force load the UBlueprint that owns this object data. */
	virtual UBlueprint* GetBlueprint() const
	{
		if (Outer.IsValid())
		{
			return Outer.Pin()->GetBlueprint();
		}
		return nullptr;
	}

	/** Requests internal creation of the search result and properly initializes the visual representation of the result */
	FRigVMSearchResult CreateSearchResult(FRigVMSearchResult InParent) const;

	/** Accessor for the parsed child data for this item */
	const TArray<FRigVMImaginaryFiBDataSharedPtr>& GetAllParsedChildData() const
	{
		return ParsedChildData;
	}

	/** Adds a KeyValue pair to the ParsedTagAndValues map */
	void AddKeyValuePair(FText InKey, FRigVMSearchableValueInfo& InValue)
	{
		ParsedTagsAndValues.Add(RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage(InKey), InValue);
	}

	/** Returns the Outer of this Imaginary data that directly owns it */
	FRigVMImaginaryFiBDataWeakPtr GetOuter() const
	{
		return Outer;
	}

	/** Called to enable interlocked parsing (only allow one thread at a time). In place to support backwards-compatibility with non-deferred indexing, may be removed later. */
	void EnableInterlockedParsing()
	{
		bRequiresInterlockedParsing = true;
	}

	/** Dumps the parsed object (including all children) to the given archive */
	RIGVMEDITOR_API void DumpParsedObject(FArchive& Ar, int32 InTreeLevel = 0) const;

	/** Builds a SearchTree ready to be displayed in the Find-in-Blueprints window */
	static FRigVMSearchResult CreateSearchTree(FRigVMSearchResult InParentSearchResult, FRigVMImaginaryFiBDataWeakPtr InCurrentPointer, TArray< const FRigVMImaginaryFiBData* >& InValidSearchResults, TMultiMap< const FRigVMImaginaryFiBData*, FRigVMComponentUniqueDisplay >& InMatchingSearchComponents);

protected:
	/**
	 * Checks if the Key has any special handling to be done, such as making a Pin out of it
	 *
	 * @param InKey			Key that the JsonValue was stored under
	 * @param InJsonValue	JsonValue to be specially parsed
	 * @return				TRUE if the JsonValue was specially handled, will not be further handled
	 */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue) { return false; }

	/** Returns the searchability status of a passed in Key, all Keys are searchable by default */
	virtual ERigVMSearchableValueStatus GetSearchabilityStatus(FString InKey) { return ERigVMSearchableValueStatus::RigVMSearchable; };

	/** Protected internal function which builds the search result for this item */
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const = 0;

	/** Creates a display string for this item in search results */
	FText CreateSearchComponentDisplayText(FText InKey, FText InValue) const;

	/** Helper function for parsing Json values into usable properties */
	void ParseJsonValue(FText InKey, FText InDisplayKey, TSharedPtr< FJsonValue > InJsonValue, TArray<FRigVMSearchableValueInfo>& OutParsedValues, bool bIsInArray = false, ERigVMSearchableValueStatus InSearchabilityOverride = ERigVMSearchableValueStatus::RigVMSearchable);

	/** Internal version of the ParseAllChildData function, handles the bulk of the work */
	virtual void ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride = ERigVMSearchableValueStatus::RigVMSearchable);

	/** Internal helper function for dumping parsed object info */
	virtual void DumpParsedObject_Internal(FArchive& Ar) const {}

protected:
	/** The unparsed Json object representing this item. Auto-cleared after parsing. Does not need to be declared as thread-safe because it's only accessed when parsing and that is already a critical section. */
	TSharedPtr< FJsonObject > UnparsedJsonObject;

	/** All parsed child data for this item. Must be declared as thread-safe because it may be accessed on different threads. */
	TArray<FRigVMImaginaryFiBDataSharedPtr> ParsedChildData;

	/** A mapping of tags to their values and searchability status */
	TMultiMap< RigVMFindInBlueprintsHelpers::FSimpleFTextKeyStorage, FRigVMSearchableValueInfo > ParsedTagsAndValues;

	/** Pointer to the lookup table to decompressed the Json strings back into fully formed FTexts */
	TMap<int32, FText>* LookupTablePtr;

	/** Outer of this object that owns it, used for climbing up the hierarchy. Must be declared as thread-safe because it may be accessed on different threads. */
	FRigVMImaginaryFiBDataWeakPtr Outer;

	/** Set after the JSON object has been parsed. */
	TAtomic<bool> bHasParsedJsonObject;

	/** Set if this instance requires interlocked parsing. */
	bool bRequiresInterlockedParsing;

	/** Allows for thread-safe parsing of the imaginary data. Only a single Imaginary data can be parsed at a time. */
	static FCriticalSection ParseChildDataCriticalSection;

private:
	/** If display meta is present, this will cache those values and is then used as a basis when constructing a search result tree */
	FRigVMSearchResult SearchResultTemplate;
};

class FRigVMFiBMetaData : public FRigVMImaginaryFiBData
{
public:
	FRigVMFiBMetaData(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr);

	/** Returns TRUE if the metadata is informing that the FProperty and children should be hidden */
	bool IsHidden() const
	{
		// While handled separately, when hidden it should always be explicit
		ensure(!bIsHidden || (bIsHidden && bIsExplicit));
		return bIsHidden;
	}

	/** Returns TRUE if the metadata is informing that the FProperty and children should be explicit */
	bool IsExplicit() const
	{
		return bIsExplicit;
	}

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override
	{
		return nullptr;
	}
	/** End FRigVMImaginaryFiBData Interface */

private:
	/** TRUE if the FProperty this metadata represents is hidden */
	bool bIsHidden;

	/** TRUE if the FProperty this metadata represents is explicit, should always be true if bIsHidden is true */
	bool bIsExplicit;
};

class FRigVMCategorySectionHelper : public FRigVMImaginaryFiBData
{
public:
	/** Callback declaration for handling special parsing of the items in the category */
	DECLARE_DELEGATE_TwoParams(FRigVMCategorySectionHelperCallback, TSharedPtr< FJsonObject >, TArray<FRigVMImaginaryFiBDataSharedPtr>&);

	FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory);
	FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory);
	FRigVMCategorySectionHelper(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FText InCategoryName, bool bInTagAndValueCategory, FRigVMCategorySectionHelperCallback InSpecialHandlingCallback);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCategory() const override { return true; }
	virtual bool IsTagAndValueCategory() const override { return bIsTagAndValue; }
	virtual bool CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */

	/** Returns the category name prepared for checking as a function */
	FString GetCategoryFunctionName() const
	{
		return CategoryName.BuildSourceString();
	}

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::ERigVMSearchableValueStatus::Searchable*/) override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** Callback to specially handle parsing of the Json Object instead of using generic handling */
	FRigVMCategorySectionHelperCallback SpecialHandlingCallback;

	/** The display text for this item in the search results */
	FText CategoryName;

	/** TRUE if this category should be considered no different than a normal Tag and Value in it's parent */
	bool bIsTagAndValue;
};

/** An "imaginary" representation of a UBlueprint, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryBlueprint : public FRigVMImaginaryFiBData
{
public:
	FRigVMImaginaryBlueprint(const FString& InBlueprintName, const FString& InBlueprintPath, const FString& InBlueprintParentClass, const TArray<FString>& InInterfaces, const FString& InUnparsedStringData, FRigVMSearchDataVersionInfo InVersionInfo);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	virtual bool CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	virtual UBlueprint* GetBlueprint() const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */

	/** Helper function to parse an array of Json Object representing graphs */
	void ParseGraph( TSharedPtr< FJsonValue > InJsonValue, FString InCategoryTitle, EGraphType InGraphType );

	/** Callback to specially parse an array of Json Objects representing components */
	void ParseComponents(TSharedPtr< FJsonObject > InJsonObject, TArray<FRigVMImaginaryFiBDataSharedPtr>& OutParsedChildData);

	/** Parses a raw string of Json to a Json object hierarchy */
	void ParseToJson(FRigVMSearchDataVersionInfo InVersionInfoconst, const FString& UnparsedStringData);

protected:
	/** The path for this Blueprint */
	FString BlueprintPath;

	/** Lookup table used as a compression tool for the FTexts stored in the Json object */
	TMap<int32, FText> LookupTable;
};

/** An "imaginary" representation of a UEdGraph, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryGraph : public FRigVMImaginaryFiBData
{
public:
	FRigVMImaginaryGraph(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, EGraphType InGraphType);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	virtual bool CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue) override;
	virtual ERigVMSearchableValueStatus GetSearchabilityStatus(FString InKey) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** The graph type being represented */
	EGraphType GraphType;
};

/** An "imaginary" representation of a UEdGraphNode, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryGraphNode : public FRigVMImaginaryFiBData
{
public:
	FRigVMImaginaryGraphNode(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	virtual bool CanCallFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue) override;
	virtual ERigVMSearchableValueStatus GetSearchabilityStatus(FString InKey) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void ParseAllChildData_Internal(ERigVMSearchableValueStatus InSearchabilityOverride/* = ERigVMSearchableValueStatus::Searchable*/) override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */

private:
	/** Schema name that manages this node */
	FString SchemaName;
};

/** An "imaginary" representation of a FProperty, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryProperty : public FRigVMImaginaryFiBData
{
public:
	FRigVMImaginaryProperty(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual ERigVMSearchableValueStatus GetSearchabilityStatus(FString InKey) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */
};

/** An "imaginary" representation of a FProperty of an instanced component, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryComponent : public FRigVMImaginaryProperty
{
public:
	FRigVMImaginaryComponent(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */
};

/** An "imaginary" representation of a UEdGraphPin, featuring raw strings or other imaginary objects in the place of more structured substances */
class FRigVMImaginaryPin : public FRigVMImaginaryFiBData
{
public:
	FRigVMImaginaryPin(FRigVMImaginaryFiBDataWeakPtr InOuter, TSharedPtr< FJsonObject > InUnparsedJsonObject, TMap<int32, FText>* InLookupTablePtr, FString InSchemaName);

	/** FRigVMImaginaryFiBData Interface */
	virtual bool IsCompatibleWithFilter(ERigVMSearchQueryFilter InSearchQueryFilter) const override;
	/** End FRigVMImaginaryFiBData Interface */

protected:
	/** FRigVMImaginaryFiBData Interface */
	virtual bool TrySpecialHandleJsonValue(FText InKey, TSharedPtr< FJsonValue > InJsonValue);
	virtual ERigVMSearchableValueStatus GetSearchabilityStatus(FString InKey) override;
	virtual FRigVMSearchResult CreateSearchResult_Internal(FRigVMSearchResult InTemplate) const override;
	virtual void DumpParsedObject_Internal(FArchive& Ar) const override;
	/** End FRigVMImaginaryFiBData Interface */

private:
	/** Schema name that manages this pin */
	FString SchemaName;
};
#endif