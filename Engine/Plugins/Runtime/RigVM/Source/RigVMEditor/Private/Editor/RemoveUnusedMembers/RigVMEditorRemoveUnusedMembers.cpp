// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorRemoveUnusedMembers.h"

#include "Algo/Transform.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "RigVMEditorRemoveUnusedMembersDialog"

namespace UE::RigVMEditor::Private
{
	template <>
	FText GetDialogTitle<URigVMFunctionReferenceNode>()
	{
		return LOCTEXT("RemoveUnusedFunctionsDialogTitle", "Remove Unused Functions");
	}

	template <>
	FText GetDialogTitle<URigVMVariableNode>()
	{
		return LOCTEXT("RemoveUnusedVariablesDialogTitle", "Remove Unused Variables");
	}

	/** Removes disconnected function nodes */
	template <>
	void RemoveDisconnectedNodes<URigVMFunctionReferenceNode>(
		URigVMController& Controller,
		const TSharedRef<FRigVMEdGraphNodeRegistry>& EdGraphNodeRegistry,
		const TArray<FName>& MemberNamesToRemove)
	{
		TArray<URigVMNode*> NodesToRemove;
		Algo::TransformIf(EdGraphNodeRegistry->GetDisconnectedEdGrapNodes(), NodesToRemove,
			[&MemberNamesToRemove](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				URigVMFunctionReferenceNode* FunctionReferenceNode = WeakEdGraphNode.IsValid() ?
					Cast<URigVMFunctionReferenceNode>(WeakEdGraphNode->GetModelNode())
					: nullptr;

				if (FunctionReferenceNode &&
					MemberNamesToRemove.Contains(FunctionReferenceNode->GetFunctionIdentifier().GetFunctionFName()))
				{
					return true;
				}

				return false;
			},
			[](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				URigVMFunctionReferenceNode* FunctionReferenceNode = WeakEdGraphNode.IsValid() ?
					Cast<URigVMFunctionReferenceNode>(WeakEdGraphNode->GetModelNode())
					: nullptr;

				check(FunctionReferenceNode);
				return FunctionReferenceNode;
			});

		if (!NodesToRemove.IsEmpty())
		{
			constexpr bool bSetupUndoRedo = true;
			constexpr bool bPrintPythonCommand = false;
			Controller.RemoveNodes(NodesToRemove, bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	/** Removes disconnected variable nodes */
	template <>
	void RemoveDisconnectedNodes<URigVMVariableNode>(
		URigVMController& Controller,
		const TSharedRef<FRigVMEdGraphNodeRegistry>& EdGraphNodeRegistry,
		const TArray<FName>& MemberNamesToRemove)
	{
		TArray<URigVMNode*> NodesToRemove;
		Algo::TransformIf(EdGraphNodeRegistry->GetDisconnectedEdGrapNodes(), NodesToRemove,
			[&MemberNamesToRemove](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				URigVMVariableNode* VariableNode = WeakEdGraphNode.IsValid() ?
					Cast<URigVMVariableNode>(WeakEdGraphNode->GetModelNode())
					: nullptr;

				if (VariableNode &&
					MemberNamesToRemove.Contains(VariableNode->GetVariableName()))
				{
					return true;
				}

				return false;
			},
			[](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				URigVMVariableNode* VariableNode = WeakEdGraphNode.IsValid() ?
					Cast<URigVMVariableNode>(WeakEdGraphNode->GetModelNode())
					: nullptr;
					
				check(VariableNode);
				return VariableNode;
			});

		if (!NodesToRemove.IsEmpty())
		{
			constexpr bool bSetupUndoRedo = true;
			constexpr bool bPrintPythonCommand = false;
			Controller.RemoveNodes(NodesToRemove, bSetupUndoRedo, bPrintPythonCommand);
		}
	}

	/** Removes function members */
	template <>
	void RemoveMembers<URigVMFunctionReferenceNode>(
		const TScriptInterface<IRigVMEditorAssetInterface>& Asset,
		const TArray<FName>& MemberNamesToRemove)
	{
		if (!Asset)
		{
			return;
		}

		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		if (!Schema)
		{
			return;
		}

		for (const FName& MemberNameToRemove : MemberNamesToRemove)
		{
			URigVMFunctionLibrary* Library = Asset->GetLocalFunctionLibrary();
			URigVMLibraryNode* Function = Library ? Library->FindFunction(MemberNameToRemove) : nullptr;
			if (Library && Function)
			{
				if (UEdGraph* EdGraph = Asset->GetEdGraph(Function->GetContainedGraph()))
				{
					Schema->TryDeleteGraph(EdGraph);
				}
			}
		}
	}

	/** Removes variable members */
	template <>
	void RemoveMembers<URigVMVariableNode>(
		const TScriptInterface<IRigVMEditorAssetInterface>& Asset,
		const TArray<FName>& MemberNamesToRemove)
	{
		UObject* Object = Asset ? Asset->GetObject() : nullptr;
		URigVMGraph* FocusedGraph = Asset ? Asset->GetFocusedModel() : nullptr;
		FRigVMClient* RigVMClient = Asset ? Asset->GetRigVMClient() : nullptr;
		URigVMController* Controller = RigVMClient ? RigVMClient->GetController(FocusedGraph) : nullptr;
		if (!Asset || 
			!Object ||
			!FocusedGraph ||
			!RigVMClient ||
			!Controller)
		{
			return;
		}

		// Variables
		Object->Modify();
		if (!Asset->BulkRemoveMemberVariables(MemberNamesToRemove))
		{
			return;
		}

		// Local variables
		for (const FRigVMGraphVariableDescription& VariableDescription : FocusedGraph->GetLocalVariables())
		{
			if (MemberNamesToRemove.Contains(VariableDescription.Name))
			{
				constexpr bool bSetupUndo = true;
				constexpr bool bPrintPythonCommand = false;
				Controller->RemoveLocalVariable(VariableDescription.Name, bSetupUndo, bPrintPythonCommand);
			}
		}
	}

	template<>
	void FindUnusedMemberNames<URigVMFunctionReferenceNode>(
		const TScriptInterface<IRigVMEditorAssetInterface>& InAsset,
		const TSharedRef<FRigVMEdGraphNodeRegistry>& InEdGraphNodeRegistry,
		TMap<FRigVMUnusedMemberCategory, TArray<FName>>& OutCategoryToUnusedMemberNamesMap)
	{
		TArray<FName> UnusedMemberNames;

		const FRigVMClient* RigVMClient = InAsset ? InAsset->GetRigVMClient() : nullptr;
		URigVMFunctionLibrary* Library = RigVMClient ? RigVMClient->GetFunctionLibrary() : nullptr;
		if (!RigVMClient || !Library)
		{
			return;
		}

		const TArray<TWeakObjectPtr<URigVMEdGraphNode>> ConnectedEdGraphNodes = InEdGraphNodeRegistry->GetConnectedEdGrapNodes();

		TArray<URigVMFunctionReferenceNode*> FunctionReferenceNodes;
		Algo::TransformIf(ConnectedEdGraphNodes, FunctionReferenceNodes,
			[](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				return
					WeakEdGraphNode.IsValid() &&
					Cast<URigVMFunctionReferenceNode>(WeakEdGraphNode->GetModelNode()) != nullptr;
			},
			[](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
			{
				check(WeakEdGraphNode.IsValid());
				return Cast<URigVMFunctionReferenceNode>(WeakEdGraphNode->GetModelNode());
			});

		TArray<FName>* PublicFunctionMemberNamesPtr = nullptr;
		TArray<FName>* PrivateFunctionMemberNamesPtr = nullptr;

		for (const URigVMLibraryNode* Function : Library->GetFunctions())
		{
			if (!Function)
			{
				continue;
			}

			const bool bConnectedInGraph = FunctionReferenceNodes.ContainsByPredicate(
				[Function](const URigVMFunctionReferenceNode* FunctionReferenceNode)
				{
					if (ensureMsgf(FunctionReferenceNode, TEXT("Unexpected nullptr in PrivateFunctionReferenceNodes")))
					{
						return FunctionReferenceNode->GetFunctionIdentifier().GetFunctionFName() == Function->GetFName();
					}

					return true;
				});

			if (bConnectedInGraph)
			{
				continue;
			}

			const FRigVMGraphFunctionIdentifier FunctionIdentifier = Function->GetFunctionIdentifier();
			const FSoftObjectPath& FunctionHostPath = Function->GetFunctionIdentifier().HostObject;
			const UObject* RigVMBlueprintGeneratedClassObj = FunctionHostPath.TryLoad();
			const IRigVMGraphFunctionHost* FunctionHost = Cast<const IRigVMGraphFunctionHost>(RigVMBlueprintGeneratedClassObj);
			const FRigVMGraphFunctionStore* FunctionStore = FunctionHost ? FunctionHost->GetRigVMGraphFunctionStore() : nullptr;

			const bool bPublicFunction = FunctionStore && FunctionStore->IsFunctionPublic(FunctionIdentifier);

			if (bPublicFunction)
			{
				if (!PublicFunctionMemberNamesPtr)
				{
					constexpr bool bSafeToRemove = false;
					const FRigVMUnusedMemberCategory PublicFunctionsCategory(
						"PublicFunctions",
						LOCTEXT("PublicFunctionsCategory", "Public Functions"),
						bSafeToRemove);

					PublicFunctionMemberNamesPtr = &OutCategoryToUnusedMemberNamesMap.Add(PublicFunctionsCategory);
				}
				check(PublicFunctionMemberNamesPtr);
				
				PublicFunctionMemberNamesPtr->Add(Function->GetFName());
			}
			else
			{
				if (!PrivateFunctionMemberNamesPtr)
				{
					constexpr bool bSafeToRemove = true;
					const FRigVMUnusedMemberCategory PrivateFunctionsCategory(
						"PrivateFunctions",
						LOCTEXT("PrivateFunctionsCategory", "Private Functions"),
						bSafeToRemove);

					PrivateFunctionMemberNamesPtr = &OutCategoryToUnusedMemberNamesMap.Add(PrivateFunctionsCategory);
				}
				check(PrivateFunctionMemberNamesPtr);
				
				PrivateFunctionMemberNamesPtr->Add(Function->GetFName());
			}
		}
	}

	template<>
	void FindUnusedMemberNames<URigVMVariableNode>(
		const TScriptInterface<IRigVMEditorAssetInterface>& InAsset,
		const TSharedRef<FRigVMEdGraphNodeRegistry>& InEdGraphNodeRegistry,
		TMap<FRigVMUnusedMemberCategory, TArray<FName>>& OutCategoryToUnusedMemberNamesMap)
	{
		if (!InAsset)
		{
			return;
		}

		const auto IsVariableConnectedInGraph =
			[&InEdGraphNodeRegistry](const FRigVMGraphVariableDescription& VariableDescription)
			{
				const bool bConnectedInGraph = InEdGraphNodeRegistry->GetConnectedEdGrapNodes().ContainsByPredicate(
					[&VariableDescription](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
					{
						URigVMVariableNode* VariableNode = WeakEdGraphNode.IsValid() ?
							Cast<URigVMVariableNode>(WeakEdGraphNode->GetModelNode()) :
							nullptr;

						if (VariableNode)
						{
							return VariableNode->GetVariableName() == VariableDescription.Name;
						}

						return false;
					});

				return bConnectedInGraph;
			};

		// As of 5.7, variables are not publicly accessible hence all are considered safe to remove
		constexpr bool bSafeToRemove = true;

		TArray<FName>* VariableMemberNamesPtr = nullptr;
		TArray<FName>* LocalVariableMemberNamesPtr = nullptr;

		// Asset variables
		for (const FRigVMGraphVariableDescription& VariableDescription : InAsset->GetAssetVariables())
		{
			if (IsVariableConnectedInGraph(VariableDescription))
			{
				continue;
			}

			if (!VariableMemberNamesPtr)
			{
				const FRigVMUnusedMemberCategory VariablesCategory(
					"Variables",
					LOCTEXT("VariablesCategory", "Variables"),
					bSafeToRemove);

				VariableMemberNamesPtr = &OutCategoryToUnusedMemberNamesMap.Add(VariablesCategory);
			}
			check(VariableMemberNamesPtr);
			
			VariableMemberNamesPtr->Add(VariableDescription.Name);
		}

		// Local variables
		if (URigVMGraph* FocusedGraph = InAsset->GetFocusedModel())
		{
			for (const FRigVMGraphVariableDescription& VariableDescription : FocusedGraph->GetLocalVariables())
			{
				if (IsVariableConnectedInGraph(VariableDescription))
				{
					continue;
				}

				if (!LocalVariableMemberNamesPtr)
				{
					const FRigVMUnusedMemberCategory LocalVariablesCategory(
						"LocalVariables",
						LOCTEXT("LocalVariablesCategory", "Local Variables"),
						bSafeToRemove);

					LocalVariableMemberNamesPtr = &OutCategoryToUnusedMemberNamesMap.Add(LocalVariablesCategory);
				}
				check(LocalVariableMemberNamesPtr);
				
				LocalVariableMemberNamesPtr->Add(VariableDescription.Name);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
