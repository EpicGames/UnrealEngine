// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/RigVMEdGraphNodeRegistry.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RigVMEditorAsset.h"
#include "RigVMEditorRemoveUnusedMembersCategory.h"
#include "RigVMEditorRemoveUnusedMembersDialog.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/Nodes/RigVMVariableNode.h"
#include "RigVMModel/RigVMClient.h"
#include "Widgets/Notifications/SNotificationList.h"
#include <type_traits>

namespace UE::RigVMEditor
{
	/** Concept for types that support RemoveUnusedMembersFromAsset */
	template <typename TRigVMNode>
	concept ConceptTypeSupportsRemoveUnusedMembers = requires
	{
		std::is_base_of_v<URigVMFunctionReferenceNode, TRigVMNode> ||
		std::is_base_of_v<URigVMVariableNode, TRigVMNode>;
	};

	namespace Private
	{
		/** Returns a type specific dialog title */
		template <typename TRigVMNode> requires ConceptTypeSupportsRemoveUnusedMembers<TRigVMNode>
		FText GetDialogTitle();

		/** Finds unused member names. Considers members with disconnected nodes as unused */
		template <typename TRigVMNode> requires ConceptTypeSupportsRemoveUnusedMembers<TRigVMNode>
		void FindUnusedMemberNames(
			const TScriptInterface<IRigVMEditorAssetInterface>& InAsset,
			const TSharedRef<FRigVMEdGraphNodeRegistry>& InEdGraphNodeRegistry,
			TMap<FRigVMUnusedMemberCategory, TArray<FName>>& OutCategoryToUnusedMemberNamesMap);

		/** Removes disconnected nodes */
		template <typename TRigVMNode> requires ConceptTypeSupportsRemoveUnusedMembers<TRigVMNode>
		void RemoveDisconnectedNodes(
			URigVMController& Controller,
			const TSharedRef<FRigVMEdGraphNodeRegistry>& EdGraphNodeRegistry,
			const TArray<FName>& MemberNamesToRemove);

		/** Removes members */
		template <typename TRigVMNode> requires ConceptTypeSupportsRemoveUnusedMembers<TRigVMNode>
		void RemoveMembers(
			const TScriptInterface<IRigVMEditorAssetInterface>& Asset, 
			const TArray<FName>& MemberNamesToRemove);
	}

	/**
	 * Utility to remove unused members from a Rig VM asset.
	 *
	 * @param Asset							The Asset for which the nodes are removed
	 * @param TransactionText				(Optional) The transaction text used when nodes are being removed. If empty or already transacting, no transaction is created.
	 * @param AllMembersReferencedMessage	(Optional) The notification message displayed if all members are referenced and none was removed. If empty, no message is displayed.
	 */
	template <typename TRigVMNode> requires ConceptTypeSupportsRemoveUnusedMembers<TRigVMNode>
	void RemoveUnusedMembersFromAsset(
		const TScriptInterface<IRigVMEditorAssetInterface>& Asset,
		const FText& TransactionText = FText::GetEmpty(),
		const FText& AllMembersReferencedMessage = FText::GetEmpty())
	{
		using namespace UE::RigVMEditor::Private;

		URigVMController* Controller = Asset ? Asset->GetController() : nullptr;
		const FRigVMClient* RigVMClient = Asset ? Asset->GetRigVMClient() : nullptr;
		if (!Asset || 
			!Asset->GetObject() ||
			!RigVMClient || 
			!Controller)
		{
			return;
		}

		// Find unused member names		
		constexpr bool bForceUpdate = true;
		const TSharedRef<FRigVMEdGraphNodeRegistry> EdGraphNodeRegistry = 
			FRigVMEdGraphNodeRegistry::GetOrCreateRegistry(Asset, TRigVMNode::StaticClass(), bForceUpdate);

		TMap<FRigVMUnusedMemberCategory, TArray<FName>> CategoryToUnusedMemberNamesMap;
		FindUnusedMemberNames<TRigVMNode>(Asset, EdGraphNodeRegistry, CategoryToUnusedMemberNamesMap);

		if (CategoryToUnusedMemberNamesMap.IsEmpty())
		{
			return;
		}

		// Present a dialog to let the user select which members to remove
		const FRigVMEditorRemoveUnusedMembersDialog::FResult DialogResult = 
			FRigVMEditorRemoveUnusedMembersDialog::Open(GetDialogTitle<TRigVMNode>(), CategoryToUnusedMemberNamesMap);

		if (DialogResult.AppReturnType != EAppReturnType::Ok)
		{
			return;
		}

		if (DialogResult.MemberNames.IsEmpty() && 
			!AllMembersReferencedMessage.IsEmpty())
		{
			// Notify if nothing was removed
			FNotificationInfo Info(AllMembersReferencedMessage);
			Info.bFireAndForget = true;
			Info.FadeOutDuration = 5.0f;
			Info.ExpireDuration = 5.0f;

			FSlateNotificationManager::Get().AddNotification(Info);
		}
		else
		{
			const bool bTransact = !GIsTransacting && !TransactionText.IsEmpty();
			if (bTransact)
			{
				Asset->GetObject()->Modify();
				Controller->OpenUndoBracket(TransactionText.ToString());
			}

			Private::RemoveDisconnectedNodes<TRigVMNode>(*Controller, EdGraphNodeRegistry, DialogResult.MemberNames);
			Private::RemoveMembers<TRigVMNode>(Asset, DialogResult.MemberNames);

			if (bTransact)
			{
				Controller->CloseUndoBracket();
			}
		}
	}
}
