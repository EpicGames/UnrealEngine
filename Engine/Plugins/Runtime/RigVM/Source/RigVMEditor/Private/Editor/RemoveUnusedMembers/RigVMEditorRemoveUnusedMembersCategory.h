// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

class IRigVMEditorAssetInterface;
template <typename InInterfaceType> class TScriptInterface;

namespace UE::RigVMEditor
{	
	/** A category for related unused members */
	struct FRigVMUnusedMemberCategory
	{
		FRigVMUnusedMemberCategory() = default;

		FRigVMUnusedMemberCategory(const FName InID, const FText& InCategoryNameText, const bool bInSafeToRemove)
			: ID(InID)
			, CategoryNameText(InCategoryNameText)
			, bSafeToRemove(bInSafeToRemove)
		{}

		/** The category ID */
		FName ID;

		/** The text for this category */
		FText CategoryNameText;

		/** If true, the members in this category can be safely removed from the Rig */
		bool bSafeToRemove = false;

		bool operator==(const FRigVMUnusedMemberCategory& Other) const
		{
			return Other.ID == ID;
		}

		bool operator!=(const FRigVMUnusedMemberCategory& Other) const
		{
			return Other.ID != ID;
		}

		friend uint32 GetTypeHash(const FRigVMUnusedMemberCategory& Category)
		{
			return GetTypeHash(Category.ID);
		}
	};
}
