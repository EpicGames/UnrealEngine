// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorRemoveUnusedMembersDialog.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/RemoveUnusedMembers/SRigVMEditorRemoveUnusedMembersDialog.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "RigVMEditorRemoveUnusedMembersDialog"

namespace UE::RigVMEditor
{
	FRigVMEditorRemoveUnusedMembersDialog::FResult FRigVMEditorRemoveUnusedMembersDialog::Open(
		const FText& DialogTitle,
		const TMap<FRigVMUnusedMemberCategory, TArray<FName>>& CategoryToUnusedMemberNamesMap)
	{
		const TSharedPtr<SWindow> ParentWindow = []() -> TSharedPtr<SWindow>
			{
				if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
				{
					IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
					return MainFrame.GetParentWindow();
				}

				return nullptr;
			}();

		if (!ParentWindow.IsValid())
		{	
			return FResult();
		}

		const FVector2D WindowSize(480.0, 640.0);

		const TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(DialogTitle)
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(WindowSize);

		FResult Result;
		Window->SetContent
		(
			SNew(SRigVMEditorRemoveUnusedMembersDialog, Window, CategoryToUnusedMemberNamesMap)
			.OnDialogEnded_Lambda(
				[&Result](const EAppReturnType::Type AppReturnType, const TArray<FName>& SelectedMemberNames)
				{
					Result.AppReturnType = AppReturnType;
					Result.MemberNames = SelectedMemberNames;
				})
		);

		constexpr bool bSlowTaskWindow = false;
		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, bSlowTaskWindow);

		return Result;
	}
}

#undef LOCTEXT_NAMESPACE
