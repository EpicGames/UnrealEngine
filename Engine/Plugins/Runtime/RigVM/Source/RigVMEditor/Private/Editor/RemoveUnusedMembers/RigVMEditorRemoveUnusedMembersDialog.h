// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "RigVMEditorRemoveUnusedMembersCategory.h"
#include "UObject/NameTypes.h"

class IRigVMEditorAssetInterface;
template <typename InInterfaceType> class TScriptInterface;

namespace UE::RigVMEditor
{
	/** Displays a dialog to remove unsued members when opened */
	struct FRigVMEditorRemoveUnusedMembersDialog
	{
		/** The result of the dialog */
		struct FResult
		{
			/** The return type picked by the user */
			EAppReturnType::Type AppReturnType = EAppReturnType::Cancel;

			/** The array of member names selected to be removed */
			TArray<FName> MemberNames;
		};

		/**
		 * Opens the dialog.
		 *
		 * @param DialogTitle						The tile of the dialog
		 * @param CategoryToUnusedMemberNamesMap	A map of categories with their unused member names
		 *
		 * @return									The Result of the dialog.
		 */
		static FResult Open(
			const FText& DialogTitle, 
			const TMap<FRigVMUnusedMemberCategory, TArray<FName>>& CategoryToUnusedMemberNamesMap);
	};
}
