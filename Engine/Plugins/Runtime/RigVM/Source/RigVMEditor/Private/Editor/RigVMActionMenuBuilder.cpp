// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMActionMenuBuilder.h"
#include "UObject/UnrealType.h"
#include "BlueprintEditorSettings.h"
#include "Settings/EditorStyleSettings.h"
#include "Engine/Blueprint.h"
#include "Editor/EditorEngine.h"
#include "BlueprintNodeBinder.h"
#include "Editor/RigVMActionMenuItem.h"
#include "BlueprintActionFilter.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintVariableNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "ObjectEditorUtils.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"

#if WITH_RIGVMLEGACYEDITOR
#include "BlueprintDragDropMenuItem.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMActionMenuBuilder"
DEFINE_LOG_CATEGORY_STATIC(LogRigVMActionMenuItemFactory, Log, All);

/*******************************************************************************
 * FRigVMActionMenuItemFactory
 ******************************************************************************/

class FRigVMActionMenuItemFactory
{
public:
	/** 
	 * Menu item factory constructor. Sets up the blueprint context, which
	 * is utilized when configuring blueprint menu items' names/tooltips/etc.
	 *
	 * @param  Context	The blueprint context for the menu being built.
	 */
	FRigVMActionMenuItemFactory(FBlueprintActionContext const& Context);

	/** A root category to perpend every menu item with */
	FText RootCategory;
	/** Cached context for the blueprint menu being built */
	FBlueprintActionContext const& Context;
	
	/**
	 * Spawns a new FRigVMActionMenuItem with the node-spawner. Constructs
	 * the menu item's category, name, tooltip, etc.
	 * 
	 * @param  Action			The node-spawner that the new menu item should wrap.
	 * @return A newly allocated FRigVMActionMenuItem (which wraps the supplied action).
	 */
	TSharedPtr<FRigVMActionMenuItem> MakeActionMenuItem(FBlueprintActionInfo const& ActionInfo);

	
private:
	/**
	 * Utility getter function that retrieves the blueprint context for the menu
	 * items being made.
	 * 
	 * @return The first blueprint out of the cached FBlueprintActionContext.
	 */
	UBlueprint* GetTargetBlueprint() const;

	/**
	 * @return 
	 */
	UEdGraph* GetTargetGraph() const;

	/**
	 * @param  ActionInfo
	 * @return
	 */
	FBlueprintActionUiSpec GetActionUiSignature(FBlueprintActionInfo const& ActionInfo);
};

//------------------------------------------------------------------------------
FRigVMActionMenuItemFactory::FRigVMActionMenuItemFactory(FBlueprintActionContext const& ContextIn)
	: RootCategory(FText::GetEmpty())
	, Context(ContextIn)
{
}

//------------------------------------------------------------------------------
TSharedPtr<FRigVMActionMenuItem> FRigVMActionMenuItemFactory::MakeActionMenuItem(FBlueprintActionInfo const& ActionInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMActionMenuItemFactory::MakeActionMenuItem);

	FBlueprintActionUiSpec UiSignature = GetActionUiSignature(ActionInfo);

	UBlueprintNodeSpawner const* Action = ActionInfo.NodeSpawner;
	FRigVMActionMenuItem* NewMenuItem = new FRigVMActionMenuItem(Action, UiSignature, IBlueprintNodeBinder::FBindingSet(), FText::FromString(RootCategory.ToString() + TEXT('|') + UiSignature.Category.ToString()));

	return MakeShareable(NewMenuItem);
}

//------------------------------------------------------------------------------
UBlueprint* FRigVMActionMenuItemFactory::GetTargetBlueprint() const
{
	UBlueprint* TargetBlueprint = nullptr;
	if (Context.Blueprints.Num() > 0)
	{
		TargetBlueprint = Context.Blueprints[0];
	}
	return TargetBlueprint;
}

//------------------------------------------------------------------------------
UEdGraph* FRigVMActionMenuItemFactory::GetTargetGraph() const
{
	UEdGraph* TargetGraph = nullptr;
	if (Context.Graphs.Num() > 0)
	{
		TargetGraph = Context.Graphs[0];
	}
	else
	{
		UBlueprint* Blueprint = GetTargetBlueprint();
		check(Blueprint != nullptr);
		
		if (Blueprint->UbergraphPages.Num() > 0)
		{
			TargetGraph = Blueprint->UbergraphPages[0];
		}
		// TODO sara-s figure this out
		// else if (Context.EditorPtr.IsValid())
		// {
		// 	TargetGraph = Context.EditorPtr.Pin()->GetFocusedGraph();
		// }
	}
	return TargetGraph;
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec FRigVMActionMenuItemFactory::GetActionUiSignature(FBlueprintActionInfo const& ActionInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMActionMenuItemFactory::GetActionUiSignature);

	UBlueprintNodeSpawner const* Action = ActionInfo.NodeSpawner;

	UEdGraph* TargetGraph = GetTargetGraph();
	Action->PrimeDefaultUiSpec(TargetGraph);

	return Action->GetUiSpec(Context, ActionInfo.GetBindings());
}


/*******************************************************************************
 * Static FRigVMActionMenuBuilder Helpers
 ******************************************************************************/

namespace FRigVMActionMenuBuilderImpl
{
	/** Defines a sub-section of the overall blueprint menu (filter, heading, etc.) */
	struct FMenuSectionDefinition
	{
	public:
		FMenuSectionDefinition(FBlueprintActionFilter const& SectionFilter);

		/** A filter for this section of the menu */
		FBlueprintActionFilter Filter;
		
		/** Sets the root category for menu items in this section. */
		void SetSectionHeading(FText const& RootCategory);
		/** Gets the root category for menu items in this section. */
		FText const& GetSectionHeading() const;

		
		/**
		 * Filters the supplied action and if it passes, spawns a new 
		 * FRigVMActionMenuItem for the specified menu (does not add the 
		 * item to the menu-builder itself).
		 *
		 * @param  DatabaseAction	The node-spawner that the new menu item should wrap.
		 * @return An empty TSharedPtr if the action was filtered out, otherwise a newly allocated FRigVMActionMenuItem.
		 */
		TArray< TSharedPtr<FEdGraphSchemaAction> > MakeMenuItems(FBlueprintActionInfo& DatabaseAction);

		
	private:
		/** In charge of spawning menu items for this section (holds category/ordering information)*/
		FRigVMActionMenuItemFactory ItemFactory;
	};

	/** A utility for building the menu item list based on a set of action descriptors */
	struct FMenuItemListAddHelper
	{
		/** Reset for a new menu build */
		void Reset(int32 NewSize)
		{
			NextIndex = 0;
			PendingActionList.Reset(NewSize);
		}

		/** Add a new pending action */
		void AddPendingAction(FBlueprintActionInfo&& Action)
		{
			PendingActionList.Add(Forward<FBlueprintActionInfo>(Action));
		}

		/** @return the next pending action and advance */
		FBlueprintActionInfo* GetNextAction()
		{
			return PendingActionList.IsValidIndex(NextIndex) ? &PendingActionList[NextIndex++] : nullptr;
		}

		/** @return the allocated size of the pending action list */
		SIZE_T GetAllocatedSize() const
		{
			return PendingActionList.GetAllocatedSize();
		}

		/** @return the total number of actions that are still pending */
		int32 GetNumPendingActions() const
		{
			return PendingActionList.IsValidIndex(NextIndex) ? PendingActionList.Num() - NextIndex : 0;
		}

		/** @return the total number of actions that were added to the pending list */
		int32 GetNumTotalAddedActions() const
		{
			return PendingActionList.Num();
		}

	private:
		/** Keeps track of the next action list item to process */
		int32 NextIndex = 0;

		/** All actions pending menu items for the current context */
		TArray<FBlueprintActionInfo> PendingActionList;
	};
}

//------------------------------------------------------------------------------
FRigVMActionMenuBuilderImpl::FMenuSectionDefinition::FMenuSectionDefinition(FBlueprintActionFilter const& SectionFilterIn)
	: Filter(SectionFilterIn)
	, ItemFactory(Filter.Context)
{
}

//------------------------------------------------------------------------------
void FRigVMActionMenuBuilderImpl::FMenuSectionDefinition::SetSectionHeading(FText const& RootCategory)
{
	ItemFactory.RootCategory = RootCategory;
}

//------------------------------------------------------------------------------
FText const& FRigVMActionMenuBuilderImpl::FMenuSectionDefinition::GetSectionHeading() const
{
	return ItemFactory.RootCategory;
}

//------------------------------------------------------------------------------
TArray< TSharedPtr<FEdGraphSchemaAction> > FRigVMActionMenuBuilderImpl::FMenuSectionDefinition::MakeMenuItems(FBlueprintActionInfo& DatabaseAction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMenuSectionDefinition::MakeMenuItems);

	TSharedPtr<FEdGraphSchemaAction> UnBoundMenuEntry;
	bool bPassedFilter = !Filter.IsFiltered(DatabaseAction);

	if (!UnBoundMenuEntry.IsValid() && bPassedFilter)
	{
		UnBoundMenuEntry = ItemFactory.MakeActionMenuItem(DatabaseAction);
	}

	TArray< TSharedPtr<FEdGraphSchemaAction> > MenuItems;
	if (UnBoundMenuEntry.IsValid())
	{
		MenuItems.Add(UnBoundMenuEntry);
	}

	return MenuItems;
}


/*******************************************************************************
 * FRigVMActionMenuBuilder
 ******************************************************************************/

//------------------------------------------------------------------------------
FRigVMActionMenuBuilder::FRigVMActionMenuBuilder(EConfigFlags ConfigFlags)
{
	bUsePendingActionList = !!(ConfigFlags & EConfigFlags::UseTimeSlicing);
	MenuItemListAddHelper = MakeShared<FRigVMActionMenuBuilderImpl::FMenuItemListAddHelper>();
}

//------------------------------------------------------------------------------
void FRigVMActionMenuBuilder::Empty()
{
	FGraphActionListBuilderBase::Empty();
	MenuSections.Empty();
}

//------------------------------------------------------------------------------
void FRigVMActionMenuBuilder::AddMenuSection(FBlueprintActionFilter const& Filter, FText const& Heading/* = FText::GetEmpty()*/)
{
	using namespace FRigVMActionMenuBuilderImpl;
	
	TSharedRef<FMenuSectionDefinition> SectionDescRef = MakeShareable(new FMenuSectionDefinition(Filter));
	SectionDescRef->SetSectionHeading(Heading);

	MenuSections.Add(SectionDescRef);
}

//------------------------------------------------------------------------------
void FRigVMActionMenuBuilder::RebuildActionList()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMActionMenuBuilder::RebuildActionList);

	using namespace FRigVMActionMenuBuilderImpl;

	FGraphActionListBuilderBase::Empty();
	
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	FBlueprintActionDatabase::FActionRegistry const& ActionRegistry = ActionDatabase.GetAllActions();

	if (bUsePendingActionList)
	{
		MenuItemListAddHelper->Reset(ActionRegistry.Num());
	}

	for (auto Iterator(ActionRegistry.CreateConstIterator()); Iterator; ++Iterator)
	{
		const FObjectKey& ObjKey = Iterator->Key;
		const FBlueprintActionDatabase::FActionList& ActionList = Iterator->Value;

		if (UObject* ActionObject = ObjKey.ResolveObjectPtr())
		{
			for (UBlueprintNodeSpawner const* NodeSpawner : ActionList)
			{
				FBlueprintActionInfo BlueprintAction(ActionObject, NodeSpawner);

				if (bUsePendingActionList)
				{
					MenuItemListAddHelper->AddPendingAction(MoveTemp(BlueprintAction));
				}
				else
				{
					MakeMenuItems(BlueprintAction);
				}
			}
		}
		else
		{
			// Remove this (invalid) entry on the next tick.
			ActionDatabase.DeferredRemoveEntry(ObjKey);
		}
	}
}

void FRigVMActionMenuBuilder::MakeMenuItems(FBlueprintActionInfo& InAction)
{
	using namespace FRigVMActionMenuBuilderImpl;

	for (TSharedRef<FMenuSectionDefinition> const& MenuSection : MenuSections)
	{
		for (TSharedPtr<FEdGraphSchemaAction> MenuEntry : MenuSection->MakeMenuItems(InAction))
		{
			AddAction(MenuEntry);
		}
	}
}

int32 FRigVMActionMenuBuilder::GetNumPendingActions() const
{
	return MenuItemListAddHelper->GetNumPendingActions();
}

float FRigVMActionMenuBuilder::GetPendingActionsProgress() const
{
	const float NumPendingActions = static_cast<float>(MenuItemListAddHelper->GetNumPendingActions());
	const float NumTotalAddedActions = static_cast<float>(MenuItemListAddHelper->GetNumTotalAddedActions());

	check(NumTotalAddedActions > 0.0f);
	return 1.0f - (NumPendingActions / NumTotalAddedActions);
}

bool FRigVMActionMenuBuilder::ProcessPendingActions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMActionMenuBuilder::ProcessPendingActions);

	using namespace FRigVMActionMenuBuilderImpl;

	bool bProcessedActions = false;
	const double StartTime = FPlatformTime::Seconds();
	const float MaxTimeThresholdSeconds = static_cast<float>(GetDefault<UBlueprintEditorSettings>()->ContextMenuTimeSlicingThresholdMs) / 1000.0f;

	FBlueprintActionInfo* CurrentAction = MenuItemListAddHelper->GetNextAction();
	while (CurrentAction)
	{
		bProcessedActions = true;

		MakeMenuItems(*CurrentAction);

		if ((FPlatformTime::Seconds() - StartTime) < MaxTimeThresholdSeconds)
		{
			CurrentAction = MenuItemListAddHelper->GetNextAction();
		}
		else
		{
			break;
		}
	}

	return bProcessedActions;
}

#undef LOCTEXT_NAMESPACE
