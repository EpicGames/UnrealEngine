// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMActionMenuUtils.h"

#include "BlueprintNodeSpawner.h"
#include "Editor/RigVMActionMenuBuilder.h"
#include "Editor/RigVMActionMenuItem.h"
#include "K2Node.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"


class UEdGraph;

#define LOCTEXT_NAMESPACE "RigVMActionMenuUtils"

//------------------------------------------------------------------------------
void FRigVMActionMenuUtils::MakeContextMenu(FBlueprintActionContext const& Context, bool bIsContextSensitive, uint32 ClassTargetMask, FRigVMActionMenuBuilder& MenuOut)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMActionMenuUtils::MakeContextMenu);

	FBlueprintActionFilter::EFlags FilterFlags = FBlueprintActionFilter::BPFILTER_NoFlags;
	FBlueprintActionFilter MainMenuFilter(FilterFlags);
	MainMenuFilter.Context = Context;
	MainMenuFilter.Context.SelectedObjects.Empty();

	//--------------------------------------
	// Defining Menu Sections
	//--------------------------------------	

	MenuOut.Empty();
	
	MenuOut.AddMenuSection(MainMenuFilter, FText::GetEmpty());

	//--------------------------------------
	// Building the Menu
	//--------------------------------------

	MenuOut.RebuildActionList();
}

//------------------------------------------------------------------------------
const UK2Node* FRigVMActionMenuUtils::ExtractNodeTemplateFromAction(const TSharedPtr<FEdGraphSchemaAction>& PaletteAction)
{
	if (PaletteAction.IsValid())
	{
		return ExtractNodeTemplateFromAction(*PaletteAction);
	}

	return nullptr;
}

const UK2Node* FRigVMActionMenuUtils::ExtractNodeTemplateFromAction(const FEdGraphSchemaAction& PaletteAction)
{
	UK2Node const* TemplateNode = nullptr;
	FName const ActionId = PaletteAction.GetTypeId();
	if (ActionId == FRigVMActionMenuItem::StaticGetTypeId())
	{
		const FRigVMActionMenuItem& NewNodeActionMenuItem = (const FRigVMActionMenuItem&)PaletteAction;
		TemplateNode = Cast<UK2Node>(NewNodeActionMenuItem.GetRawAction()->GetTemplateNode());
	}

	return TemplateNode;
}

#undef LOCTEXT_NAMESPACE
