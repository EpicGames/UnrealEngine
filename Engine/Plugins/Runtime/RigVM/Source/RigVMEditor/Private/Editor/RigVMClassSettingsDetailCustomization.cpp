// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMClassSettingsDetailCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "RigVMPythonLogDetails.h"
#include "Editor/RigVMEditor.h"
#include "Editor/SRigVMDetailsInspector.h"

#define LOCTEXT_NAMESPACE "RigVMClassSettingsDetailCustomization"

TSharedRef<IDetailCustomization> FRigVMClassSettingsDetailCustomization::MakeInstance(TSharedPtr<IRigVMEditor> InEditor)
{
	const TArray<UObject*>* Objects = (InEditor.IsValid() ? InEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = (*Objects)[0])
		{
			return MakeShareable(new FRigVMClassSettingsDetailCustomization(InEditor, Blueprint));
		}
		else if ((*Objects)[0]->Implements<URigVMRuntimeAssetInterface>())
		{
			TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = (*Objects)[0];
			return MakeShareable(new FRigVMClassSettingsDetailCustomization(InEditor, RuntimeAsset->GetEditorOnlyData()));
		}
	}

	return MakeShareable(new FRigVMClassSettingsDetailCustomization((TSharedPtr<IRigVMEditor>)nullptr, FRigVMEditorAssetInterfacePtr(nullptr)));
}

FRigVMClassSettingsDetailCustomization::FRigVMClassSettingsDetailCustomization(TSharedPtr<IRigVMEditor> InEditor, FRigVMEditorAssetInterfacePtr Asset)
		: EditorPtr(InEditor)
		, AssetPtr(Asset)
{}

void FRigVMClassSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor.IsValid())
	{
		return;
	}

	TStrongObjectPtr<UObject> AssetObj = AssetPtr.GetWeakObjectPtr().Pin();
	FRigVMEditorAssetInterfacePtr Asset;
	if (AssetObj.IsValid())
	{
		Asset = AssetObj.Get();
	}
	else
	{
		return;
	}

	if (Editor->IsDetailsPanelEditingClassDefaults())
	{
		CustomizeDefaults(DetailLayout, Editor, Asset);
	}
	else
	{
		CustomizeSettings(DetailLayout, Editor, Asset);
	}

	if (TSharedPtr<class SRigVMDetailsInspector> Inspector = Editor->GetRigVMInspector())
	{
		TSharedPtr<IDetailsView> DetailsView;
		DetailsView = Editor->GetRigVMInspector()->GetPropertyView();

		if (DetailsView.IsValid())
		{
			DetailsView->OnFinishedChangingProperties().AddSP(this, &FRigVMClassSettingsDetailCustomization::OnFinishedChangingSettings);
		}
	}
}

void FRigVMClassSettingsDetailCustomization::CustomizeSettings(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IRigVMEditor> Editor, FRigVMEditorAssetInterfacePtr Asset)
{
	DetailLayout.HideCategory(TEXT("Default"));
	
	// Rig Graph Display Settings
	IDetailCategoryBuilder& UserInterfaceCategory = DetailLayout.EditCategory("User Interface", LOCTEXT("UserInterfaceCategory", "User Interface"));
	{
		FRigVMEdGraphDisplaySettings& DisplaySettings = Asset->GetRigGraphDisplaySettings();
		TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(DisplaySettings.StaticStruct(), (uint8*) &DisplaySettings);
		UserInterfaceCategory.AddExternalStructure(StructOnScope);
	}

	// VM
	IDetailCategoryBuilder& VMCategory = DetailLayout.EditCategory("VM", LOCTEXT("VMCategory", "VM"));
	{
		// VMCompile Settings
		{
			FAddPropertyParams AddPropertyParams;
			VMCategory.AddExternalObjectProperty({Asset.GetObject()}, GET_MEMBER_NAME_CHECKED(URigVMEditorAsset, CompileSettings), EPropertyLocation::Default, AddPropertyParams);
		}
	}

	// Imports (Default Namespaces, Imported Namespaces)
	
	// Python Log Settings (Copy Python Script, Run Python Context)
	IDetailCategoryBuilder& PythonCategory = DetailLayout.EditCategory("Python", LOCTEXT("PythonCategory", "Python"));
	{
		PythonCategory.AddCustomRow(LOCTEXT("PythonButtons", "Python"))
		[
			SNew(SRigVMPythonLogDetails, Asset)
		];
	}
}

void FRigVMClassSettingsDetailCustomization::CustomizeDefaults(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IRigVMEditor> Editor, FRigVMEditorAssetInterfacePtr Asset)
{
	DetailLayout.HideCategory(TEXT("VM"));
	DetailLayout.HideCategory(TEXT("Variant"));
	DetailLayout.HideCategory(TEXT("Dependencies"));

	IDetailCategoryBuilder& VariablesCategory = DetailLayout.EditCategory("Variables", LOCTEXT("VariablesCategory", "Variables"));
	if (FRigVMPropertyBag* InstancedPropertyBag = Asset->GetVariablesPropertyBag())
	{
		FStructView StructView = InstancedPropertyBag->GetMutableValue();
		if (StructView.IsValid())
		{
			TSharedRef<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(
				StructView.GetScriptStruct(),
				StructView.GetMemory()
				);
			FAddPropertyParams Params;
			Params.HideRootObjectNode(true);
			Params.CreateCategoryNodes(true);
			VariablesCategory.AddExternalStructureProperty(StructOnScope, NAME_None, EPropertyLocation::Default, Params);
		}
	}
}

void FRigVMClassSettingsDetailCustomization::OnFinishedChangingSettings(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return;
	}

	TStrongObjectPtr<UObject> AssetObj = AssetPtr.GetWeakObjectPtr().Pin();
	if (!AssetObj)
	{
		return;
	}

	if (FRigVMEditorAssetInterfacePtr Asset = AssetObj.Get())
	{
		Asset->MarkAssetAsModified();
		Asset->RecompileVM();
	}
}

#undef LOCTEXT_NAMESPACE
