// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMCommentNodeDetailCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "RigVMEditorAsset.h"
#include "RigVMCore/RigVM.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "UnrealEngine.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "CommentNodeDetails"

void FRigVMCommentNodeDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ObjectsBeingCustomized.Reset();

	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		URigVMCommentNode* Node = CastChecked<URigVMCommentNode>(DetailObject.Get());
		ObjectsBeingCustomized.Add(Node);
	}

	if(!ObjectsBeingCustomized.IsEmpty() && ObjectsBeingCustomized[0].IsValid())
	{
		if (IRigVMEditorAssetInterface* InterfacePtr = ObjectsBeingCustomized[0]->GetImplementingOuter<IRigVMEditorAssetInterface>())
		{
			BlueprintBeingCustomized = InterfacePtr->GetObject();
		}
	}

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("Comment Node"));
	
	const UEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	Category.AddCustomRow( LOCTEXT("CommentNodeText", "Comment Text") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CommentNodeText", "Comment Text"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SEditableTextBox)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FRigVMCommentNodeDetailCustomization::GetText)
		.OnTextCommitted(this, &FRigVMCommentNodeDetailCustomization::SetText)
	];

	Category.AddCustomRow( LOCTEXT("CommentNodeColor", "Comment Color") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CommentNodeColor", "Comment Color"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SColorBlock)
		.Color(this, &FRigVMCommentNodeDetailCustomization::GetColor)
		.OnMouseButtonDown(this, &FRigVMCommentNodeDetailCustomization::OnChooseColor)
		.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
		.ShowBackgroundForAlpha(true)
		.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
		.Size(FVector2D(70.0f, 20.0f))
		.CornerRadius(FVector4(4.0f,4.0f,4.0f,4.0f))
	];

	Category.AddCustomRow( LOCTEXT("CommentNodeShowBubble", "Show Bubble") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CommentNodeShowBubble", "Show Bubble"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SCheckBox)
		.IsChecked(this, &FRigVMCommentNodeDetailCustomization::IsShowingBubbleEnabled)
		.OnCheckStateChanged(this, &FRigVMCommentNodeDetailCustomization::OnShowingBubbleStateChanged)
	];

	Category.AddCustomRow( LOCTEXT("CommentNodeColorBubble", "Color Bubble") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CommentNodeColorBubble", "Color Bubble"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SCheckBox)
		.IsChecked(this, &FRigVMCommentNodeDetailCustomization::IsColorBubbleEnabled)
		.OnCheckStateChanged(this, &FRigVMCommentNodeDetailCustomization::OnColorBubbleStateChanged)
	];

	Category.AddCustomRow( LOCTEXT("CommentNodeFontSize", "Font Size") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("CommentNodeFontSize", "Font Size"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SNumericEntryBox<int32>)
		.Value(this, &FRigVMCommentNodeDetailCustomization::GetFontSize)
		.OnValueCommitted(this, &FRigVMCommentNodeDetailCustomization::OnFontSizeChanged)
	];
}

void FRigVMCommentNodeDetailCustomization::CleanupInvalidObjects() const
{
	ObjectsBeingCustomized = ObjectsBeingCustomized.FilterByPredicate([](const TWeakObjectPtr<URigVMCommentNode>& InNode)
		{
			return InNode.IsValid();
		});
}

void FRigVMCommentNodeDetailCustomization::GetValuesFromNode(TWeakObjectPtr<URigVMCommentNode> WeakNode)
{
	if (TStrongObjectPtr<URigVMCommentNode> Node = WeakNode.Pin())
	{
		CommentText = Node->GetCommentText();
		bShowingBubble = Node->GetCommentBubbleVisible();
		bBubbleColorEnabled = Node->GetCommentColorBubble();
		FontSize = Node->GetCommentFontSize();
	}
}

void FRigVMCommentNodeDetailCustomization::SetValues(TWeakObjectPtr<URigVMCommentNode> WeakNode)
{
	if (BlueprintBeingCustomized)
	{
		if (TStrongObjectPtr<URigVMCommentNode> Node = WeakNode.Pin())
		{
			URigVMController* Controller = BlueprintBeingCustomized->GetController(Node->GetGraph());
			Controller->SetCommentText(Node.Get(), CommentText, FontSize, bShowingBubble, bBubbleColorEnabled, true, true);
		}
	}
}

FText FRigVMCommentNodeDetailCustomization::GetText() const
{
	CleanupInvalidObjects();
	if (!ObjectsBeingCustomized.IsEmpty())
	{
		FString Value = ObjectsBeingCustomized[0]->GetCommentText();
		for (int32 i=1; i<ObjectsBeingCustomized.Num(); ++i)
		{
			if (ObjectsBeingCustomized[i].IsValid())
			{
				if (!Value.Equals(ObjectsBeingCustomized[i]->GetCommentText()))
				{
					Value = TEXT("Multiple Values");
					break;
				}
			}
		}
		return FText::FromString(Value);
	}
	return FText::GetEmpty();
}

void FRigVMCommentNodeDetailCustomization::SetText(const FText& InNewText, ETextCommit::Type InCommitType)
{
	CleanupInvalidObjects();
	for (TWeakObjectPtr<URigVMCommentNode> WeakNode : ObjectsBeingCustomized)
	{
		GetValuesFromNode(WeakNode);
		CommentText = InNewText.ToString();
		SetValues(WeakNode);
	}
}

FLinearColor FRigVMCommentNodeDetailCustomization::GetColor() const
{
	CleanupInvalidObjects();
	if (!ObjectsBeingCustomized.IsEmpty())
	{
		FLinearColor Value = ObjectsBeingCustomized[0]->GetNodeColor();
		for (int32 i=1; i<ObjectsBeingCustomized.Num(); ++i)
		{
			if (ObjectsBeingCustomized[i].IsValid())
			{
				if (!Value.Equals(ObjectsBeingCustomized[i]->GetNodeColor()))
				{
					Value = FLinearColor::Black;
					break;
				}
			}
		}
		return Value;
	}
	return FLinearColor::Black;
}

FReply FRigVMCommentNodeDetailCustomization::OnChooseColor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.InitialColor = GetColor();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FRigVMCommentNodeDetailCustomization::OnColorPicked);
	OpenColorPicker(PickerArgs);
	return FReply::Handled();
}

void FRigVMCommentNodeDetailCustomization::OnColorPicked(FLinearColor LinearColor)
{
	if (BlueprintBeingCustomized)
	{
		CleanupInvalidObjects();
		for (TWeakObjectPtr<URigVMCommentNode> WeakNode : ObjectsBeingCustomized)
		{
			if (TStrongObjectPtr<URigVMCommentNode> Node = WeakNode.Pin())
			{
				URigVMController* Controller = BlueprintBeingCustomized->GetController(Node->GetGraph());
				Controller->SetNodeColor(Node.Get(), LinearColor, true, true);
				
			}
		}
	}
}

ECheckBoxState FRigVMCommentNodeDetailCustomization::IsShowingBubbleEnabled() const
{
	CleanupInvalidObjects();
	if (!ObjectsBeingCustomized.IsEmpty())
	{
		bool Value = ObjectsBeingCustomized[0]->GetCommentBubbleVisible();
		for (int32 i=1; i<ObjectsBeingCustomized.Num(); ++i)
		{
			if (ObjectsBeingCustomized[i].IsValid())
			{
				if (Value != (ObjectsBeingCustomized[i]->GetCommentBubbleVisible()))
				{
					Value = false;
					break;
				}
			}
		}
		return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMCommentNodeDetailCustomization::OnShowingBubbleStateChanged(ECheckBoxState InValue)
{
	CleanupInvalidObjects();
	for (TWeakObjectPtr<URigVMCommentNode> WeakNode : ObjectsBeingCustomized)
	{
		GetValuesFromNode(WeakNode);
		bShowingBubble = InValue == ECheckBoxState::Checked ? true : false;
		SetValues(WeakNode);
	}
}

ECheckBoxState FRigVMCommentNodeDetailCustomization::IsColorBubbleEnabled() const
{
	CleanupInvalidObjects();
	if (!ObjectsBeingCustomized.IsEmpty())
	{
		bool Value = ObjectsBeingCustomized[0]->GetCommentColorBubble();
		for (int32 i=1; i<ObjectsBeingCustomized.Num(); ++i)
		{
			if (ObjectsBeingCustomized[i].IsValid())
			{
				if (Value != (ObjectsBeingCustomized[i]->GetCommentColorBubble()))
				{
					Value = false;
					break;
				}
			}
		}
		return Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMCommentNodeDetailCustomization::OnColorBubbleStateChanged(ECheckBoxState InValue)
{
	CleanupInvalidObjects();
	for (TWeakObjectPtr<URigVMCommentNode> WeakNode : ObjectsBeingCustomized)
	{
		GetValuesFromNode(WeakNode);
		bBubbleColorEnabled = InValue == ECheckBoxState::Checked ? true : false;
		SetValues(WeakNode);
	}
}

TOptional<int> FRigVMCommentNodeDetailCustomization::GetFontSize() const
{
	CleanupInvalidObjects();
	if (!ObjectsBeingCustomized.IsEmpty())
	{
		TOptional<int32> Value = ObjectsBeingCustomized[0]->GetCommentFontSize();
		for (int32 i=1; i<ObjectsBeingCustomized.Num(); ++i)
		{
			if (ObjectsBeingCustomized[i].IsValid())
			{
				if (Value != (ObjectsBeingCustomized[i]->GetCommentFontSize()))
				{
					return TOptional<int32>();
				}
			}
		}
		return Value;
	}
	return TOptional<int32>();
}

void FRigVMCommentNodeDetailCustomization::OnFontSizeChanged(int32 InValue, ETextCommit::Type Arg)
{
	CleanupInvalidObjects();
	for (TWeakObjectPtr<URigVMCommentNode> WeakNode : ObjectsBeingCustomized)
	{
		GetValuesFromNode(WeakNode);
		FontSize = InValue;
		SetValues(WeakNode);
	}
}

#undef LOCTEXT_NAMESPACE
