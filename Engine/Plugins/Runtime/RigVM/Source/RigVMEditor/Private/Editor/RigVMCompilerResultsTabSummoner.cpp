// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMCompilerResultsTabSummoner.h"
#include "Editor/RigVMNewEditor.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "RigVMEditor"

FRigVMCompilerResultsTabSummoner::FRigVMCompilerResultsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(TabID(), InHostingApp)
{
	TabLabel = LOCTEXT("CompilerResultsTabTitle", "Compiler Results");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("CompilerResultsView", "Compiler Results");
	ViewMenuTooltip = LOCTEXT("CompilerResultsView_ToolTip", "Show compiler results of all functions and variables");
}


TSharedRef<SWidget> FRigVMCompilerResultsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FRigVMNewEditor> EditorPtr = StaticCastSharedPtr<FRigVMNewEditor>(HostingApp.Pin());

	return EditorPtr->GetCompilerResults();
}



#undef LOCTEXT_NAMESPACE
