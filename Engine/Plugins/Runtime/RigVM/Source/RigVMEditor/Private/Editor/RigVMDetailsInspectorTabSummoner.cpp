// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMDetailsInspectorTabSummoner.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "Editor/RigVMNewEditor.h"

#define LOCTEXT_NAMESPACE "RigVMDetailsInspectorTabSummoner"

FRigVMDetailsInspectorTabSummoner::FRigVMDetailsInspectorTabSummoner(const TSharedRef<FRigVMNewEditor>& InRigVMEditor)
	: FWorkflowTabFactory(TabID(), InRigVMEditor.Get().GetHostingApp())
	, RigVMEditor(InRigVMEditor)
{
	TabLabel = LOCTEXT("RigVMDetailsInspectorTabLabel", "Details");
	TabIcon = FSlateIcon(TEXT("RigVMEditorStyle"), "DetailsInspector.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigVMDetailsInspector_ViewMenu_Desc", "Details");
	ViewMenuTooltip = LOCTEXT("RigVMDetailsInspector_ViewMenu_ToolTip", "Show the RigVM Details tab");
}

TSharedRef<SWidget> FRigVMDetailsInspectorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SRigVMDetailsInspector> Inspector = SNew(SRigVMDetailsInspector)
		.Editor(RigVMEditor)
		.OnFinishedChangingProperties(FOnFinishedChangingProperties::FDelegate::CreateSP(RigVMEditor.Pin().Get(), &FRigVMNewEditor::OnFinishedChangingProperties));
	
	RigVMEditor.Pin()->SetInspector(Inspector);
	return Inspector;
}

TSharedRef<SDockTab> FRigVMDetailsInspectorTabSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> Tab = FWorkflowTabFactory::SpawnTab(Info);

	TSharedPtr<FRigVMNewEditor> BlueprintEditorPtr = StaticCastSharedPtr<FRigVMNewEditor>(HostingApp.Pin());
	BlueprintEditorPtr->GetRigVMInspector()->SetOwnerTab(Tab);

	BlueprintEditorPtr->GetRigVMInspector()->GetPropertyView()->SetHostTabManager(Info.TabManager);

	return Tab;
}

#undef LOCTEXT_NAMESPACE 
