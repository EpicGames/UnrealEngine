// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMDetailsViewWrapperObject.h"

#include "PropertyCustomizationHelpers.h"
#include "RigVMDeveloperModule.h"
#include "RigVMCore/RigVMStruct.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "Modules/ModuleManager.h"
#include "Algo/Sort.h"
#include "RigVMTypeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMDetailsViewWrapperObject)

#if WITH_EDITOR
#include "PropertyEditorModule.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Editor/RigVMLocalVariableDetailCustomization.h"
#endif

TMap<URigVMDetailsViewWrapperObject::FPerClassInfo, UClass*> URigVMDetailsViewWrapperObject::InfoToClass;
TMap<UClass*, URigVMDetailsViewWrapperObject::FPerClassInfo> URigVMDetailsViewWrapperObject::ClassToInfo;
TSet<UClass*> URigVMDetailsViewWrapperObject::OutdatedClassToRecreate;
TMap<UClass*, TArray<URigVMDetailsViewWrapperObject*>> URigVMDetailsViewWrapperObject::InstancePool;

static const TCHAR DiscardedWrapperClassTemplateName[] = TEXT("DiscardedWrapperClassTemplate");

URigVMDetailsViewWrapperObject::URigVMDetailsViewWrapperObject()
	: bIsSettingValue(false)
{
}

UClass* URigVMDetailsViewWrapperObject::GetClassForStruct(UScriptStruct* InStruct, bool bCreateIfNeeded) const
{
	check(InStruct != nullptr);

	if(UClass** ExistingClass = InfoToClass.Find(InStruct))
	{
		// if properties in a class has changed (due to user defined struct change)
		// we have to recreate the class
		if (OutdatedClassToRecreate.Contains(*ExistingClass))
		{
			OutdatedClassToRecreate.Remove(*ExistingClass);

			InfoToClass.Remove(InStruct);
			ClassToInfo.Remove(*ExistingClass);

			FString DiscardedWrapperClassName;
			static int32 DiscardedWrapperClassIndex = 0;
			do
			{
				DiscardedWrapperClassName = FString::Printf(TEXT("%s_%d"), DiscardedWrapperClassTemplateName, DiscardedWrapperClassIndex++);
				if(StaticFindObjectFast(nullptr, GetTransientPackage(), *DiscardedWrapperClassName) == nullptr)
				{
					break;
				}
			}
			while (DiscardedWrapperClassIndex < INT_MAX);

			(*ExistingClass)->Rename(*DiscardedWrapperClassName, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			
			(*ExistingClass)->RemoveFromRoot();
		}
		else
		{
			return *ExistingClass;
		}
	}

	if(!bCreateIfNeeded)
	{
		return nullptr;
	}

	UClass* SuperClass = GetClass();
	const FName WrapperClassName(FString::Printf(TEXT("%s_%s"), *SuperClass->GetName(), *InStruct->GetStructCPPName()));

	UClass* WrapperClass = NewObject<UClass>(
		GetTransientPackage(),
		WrapperClassName,
		RF_Public | RF_Transient
 		);

	// make sure this doesn't get garbage collected
	WrapperClass->AddToRoot();

	// Eviscerate the class.
	WrapperClass->PurgeClass(false);
	WrapperClass->PropertyLink = SuperClass->PropertyLink;

	WrapperClass->SetSuperStruct(SuperClass);
	WrapperClass->ClassWithin = UObject::StaticClass();
	WrapperClass->ClassConfigName = SuperClass->ClassConfigName;
	WrapperClass->ClassFlags |= CLASS_NotPlaceable | CLASS_Hidden;
	WrapperClass->SetMetaData(TEXT("DisplayName"), *InStruct->GetDisplayNameText().ToString());

	struct Local
	{
		static bool IsStructHashable(const UScriptStruct* InStructType)
		{
			if (InStructType->IsNative())
			{
				return InStructType->GetCppStructOps() && InStructType->GetCppStructOps()->HasGetTypeHash();
			}
			else
			{
				for (TFieldIterator<FProperty> It(InStructType); It; ++It)
				{
					if (CastField<FBoolProperty>(*It))
					{
						continue;
					}
					else if (!It->HasAllPropertyFlags(CPF_HasGetValueTypeHash))
					{
						return false;
					}
				}
				return true;
			}
		}
	};

	// duplicate all properties from the struct to the wrapper object
	FField::FLinkedListBuilder PropertyListBuilder(&WrapperClass->ChildProperties);

	for (TFieldIterator<FProperty> PropertyIt(InStruct); PropertyIt; ++PropertyIt)
	{
		const FProperty* InProperty = *PropertyIt;
		FProperty* NewProperty = CastFieldChecked<FProperty>(FField::Duplicate(InProperty, WrapperClass, InProperty->GetFName()));
		check(NewProperty);
		FField::CopyMetaData(InProperty, NewProperty);

		if (NewProperty->HasMetaData(TEXT("Input")) || NewProperty->HasMetaData(TEXT("Visible")))
		{
			// filter out execute pins
			bool bIsEditable = true;
			if (FStructProperty* StructProperty = CastField<FStructProperty>(NewProperty))
			{
				if (StructProperty->Struct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					bIsEditable = false;
				}
			}

			if(bIsEditable)
			{
				NewProperty->SetPropertyFlags(NewProperty->GetPropertyFlags() | CPF_Edit);
			}
		}

		PropertyListBuilder.AppendNoTerminate(*NewProperty);
	}

	// Update the class
	WrapperClass->Bind();
	WrapperClass->StaticLink(true);
	
	// Similar to FConfigPropertyHelperDetails::CustomizeDetails, this is required for GC to work properly
	WrapperClass->AssembleReferenceTokenStream();

	InfoToClass.Add(InStruct, WrapperClass);
	ClassToInfo.Add(WrapperClass, InStruct);

	UObject* CDO = WrapperClass->GetDefaultObject(true);
	CDO->AddToRoot();

	// import the defaults from the struct onto the class
	TSharedPtr<FStructOnScope> DefaultStruct = MakeShareable(new FStructOnScope(InStruct));
	CopyPropertiesForUnrelatedStructs((uint8*)CDO, WrapperClass, DefaultStruct->GetStructMemory(), DefaultStruct->GetStruct());

#if WITH_EDITOR
	if(InStruct->IsChildOf(FRigVMGraphVariableDescription::StaticStruct()))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		if (!PropertyEditorModule.GetClassNameToDetailLayoutNameMap().Contains(WrapperClassName))
		{
			PropertyEditorModule.RegisterCustomClassLayout(WrapperClassName, FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMLocalVariableDetailCustomization::MakeInstance));
		}
	}
#endif

	return WrapperClass;
}

URigVMDetailsViewWrapperObject* URigVMDetailsViewWrapperObject::MakeInstance(UClass* InWrapperObjectClass, UObject* InOuter, UScriptStruct* InStruct, uint8* InStructMemory, UObject* InSubject)
{
	check(InWrapperObjectClass != nullptr);
	check(InStruct != nullptr);

	InOuter = InOuter == nullptr ? GetTransientPackage() : InOuter;

	const URigVMDetailsViewWrapperObject* CDO = CastChecked<URigVMDetailsViewWrapperObject>(InWrapperObjectClass->GetDefaultObject());
	const UClass* WrapperClass = CDO->GetClassForStruct(InStruct);
	if(WrapperClass == nullptr)
	{
		return nullptr;
	}
	
	URigVMDetailsViewWrapperObject* Instance = GetOrCreateNewInstance(WrapperClass, InOuter);

	if (Instance)
	{
		Instance->SetContent(InStructMemory, InStruct);
		Instance->SetSubject(InSubject);
	}

	return Instance;
}

UScriptStruct* URigVMDetailsViewWrapperObject::GetWrappedStruct() const
{
	return ClassToInfo.FindChecked(GetClass()).ScriptStruct;
}

UClass* URigVMDetailsViewWrapperObject::GetClassForNodes(TArray<URigVMNode*> InNodes, bool bCreateIfNeeded) const
{
	if(InNodes.IsEmpty())
	{
		return nullptr;
	}

	static TArray<ERigVMPinDirection> PinDirectionsToCheck = {
		ERigVMPinDirection::IO,
		ERigVMPinDirection::Input,
		ERigVMPinDirection::Visible
	};

	// determine if all nodes are unit nodes and match their script struct
	TArray<UScriptStruct*> UnitStructs;
	TArray<const FRigVMDispatchFactory*> DispatchFactories;
	TArray<const FRigVMGraphFunctionIdentifier> FunctionIdentifiers;
	
	const UClass* NodeClass = InNodes[0]->GetClass();
	bool bMatchingNodes = true;
	for(URigVMNode* Node : InNodes)
	{
		if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
		{
			if(UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
			{
				UnitStructs.AddUnique(ScriptStruct);
			}
		}
		else if(URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(Node))
		{
			if(const FRigVMDispatchFactory* Factory = DispatchNode->GetFactory())
			{
				DispatchFactories.AddUnique(Factory);
			}
		}
		else if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
		{
			FunctionIdentifiers.AddUnique(FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer);
		}

		if(NodeClass != Node->GetClass())
		{
			bMatchingNodes = false;
		}
	}

	if(bMatchingNodes)
	{
		if(!UnitStructs.IsEmpty())
		{
			bMatchingNodes = UnitStructs.Num() == 1;
		}
		else if(!DispatchFactories.IsEmpty())
		{
			bMatchingNodes = DispatchFactories.Num() == 1;
		}
		else if(!FunctionIdentifiers.IsEmpty())
		{
			bMatchingNodes = FunctionIdentifiers.Num() == 1;
		}
	}

	// find the matching pins on all passed in nodes
	TArray<URigVMPin*> PinsToInspect;

	for(const ERigVMPinDirection PinDirectionToCheck : PinDirectionsToCheck)
	{
		for(URigVMPin* Pin : InNodes[0]->GetPins())
		{
			if(Pin->IsExecuteContext())
			{
				continue;
			}
			
			if(Pin->GetDirection() != PinDirectionToCheck)
			{
				continue;
			}
			
			bool bInspectPin = true;
			
			for(int32 NodeIndex=1; NodeIndex<InNodes.Num(); NodeIndex++)
			{
				URigVMPin* OtherPin = InNodes[NodeIndex]->FindPin(Pin->GetName());
				if(OtherPin == nullptr)
				{
					bInspectPin = false;
					break;
				}

				if(Pin->GetCPPType() != OtherPin->GetCPPType())
				{
					bInspectPin = false;
					break;
				}
			}

			if(bInspectPin)
			{
				PinsToInspect.Add(Pin);
			}
		}
	}

	TArray<FString> PinNotations;
	for(URigVMPin* Pin : PinsToInspect)
	{
		PinNotations.Add(FString::Printf(TEXT("%s %s"), *Pin->GetCPPType(), *Pin->GetName()));
	}

	FString Notation = FString::Join(PinNotations, TEXT(", "));
	if(UnitStructs.Num() > 0)
	{
		// sort the structs to ensure we get the same notation each time
		Algo::Sort(UnitStructs, [](UScriptStruct* A, UScriptStruct* B) -> bool
		{
			return A->GetStructCPPName() > B->GetStructCPPName();
		});

		TArray<FString> StructNames;
		for(UScriptStruct* UnitStruct : UnitStructs)
		{
			StructNames.Add(UnitStruct->GetStructCPPName());
		}
		Notation = FString::Printf(TEXT("%s(%s)"), *FString::Join(StructNames, TEXT("|")), *Notation);
	}
	if(DispatchFactories.Num() > 0)
	{
		// sort the factories to ensure we get the same notation each time
		Algo::Sort(DispatchFactories, [](const FRigVMDispatchFactory* A, const FRigVMDispatchFactory* B) -> bool
		{
			return B->GetFactoryName().LexicalLess(A->GetFactoryName());
		});

		TArray<FString> FactoryNames;
		for(const FRigVMDispatchFactory* Factory : DispatchFactories)
		{
			FactoryNames.Add(Factory->GetFactoryName().ToString());
		}
		Notation = FString::Printf(TEXT("%s(%s)"), *FString::Join(FactoryNames, TEXT("|")), *Notation);
	}
	if(FunctionIdentifiers.Num() > 0)
	{
		// sort the functions to ensure we get the same notation each time
		Algo::Sort(FunctionIdentifiers, [](const FRigVMGraphFunctionIdentifier& A, const FRigVMGraphFunctionIdentifier& B) -> bool
		{
			return A.GetLibraryNodePath() > B.GetLibraryNodePath();
		});

		TArray<FString> IdentifierStrings;
		for(const FRigVMGraphFunctionIdentifier& Identifier : FunctionIdentifiers)
		{
			IdentifierStrings.Add(Identifier.GetLibraryNodePath());
		}
		Notation = FString::Printf(TEXT("%s(%s)"), *FString::Join(IdentifierStrings, TEXT("|")), *Notation);
	}

	const FPerClassInfo PerClassInfo(Notation);
	if(UClass** ExistingClass = InfoToClass.Find(PerClassInfo))
	{
		// if properties in a class has changed (due to user defined struct change)
		// we have to recreate the class
		if (OutdatedClassToRecreate.Contains(*ExistingClass))
		{
			OutdatedClassToRecreate.Remove(*ExistingClass);

			InfoToClass.Remove(PerClassInfo);
			ClassToInfo.Remove(*ExistingClass);

			FString DiscardedWrapperClassName;
			static int32 DiscardedWrapperClassIndex = 0;
			do
			{
				DiscardedWrapperClassName = FString::Printf(TEXT("%s_%d"), DiscardedWrapperClassTemplateName, DiscardedWrapperClassIndex++);
				if(StaticFindObjectFast(nullptr, GetTransientPackage(), *DiscardedWrapperClassName) == nullptr)
				{
					break;
				}
			}
			while (DiscardedWrapperClassIndex < INT_MAX);

			(*ExistingClass)->Rename(*DiscardedWrapperClassName, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			
			(*ExistingClass)->RemoveFromRoot();
		}
		else
		{
			return *ExistingClass;
		}
	}

	if(!bCreateIfNeeded)
	{
		return nullptr;
	}

	UClass* SuperClass = GetClass();
	const int32 HashForNotation = (int32)GetTypeHash(PerClassInfo);
	const FName WrapperClassName(FString::Printf(TEXT("%s_%d"), *SuperClass->GetName(), HashForNotation));

	UClass* WrapperClass = NewObject<UClass>(
		GetTransientPackage(),
		WrapperClassName,
		RF_Public | RF_Transient
		 );

	// make sure this doesn't get garbage collected
	WrapperClass->AddToRoot();

	// Eviscerate the class.
	WrapperClass->PurgeClass(false);
	WrapperClass->PropertyLink = SuperClass->PropertyLink;

	WrapperClass->SetSuperStruct(SuperClass);
	WrapperClass->ClassWithin = UObject::StaticClass();
	WrapperClass->ClassConfigName = SuperClass->ClassConfigName;
	WrapperClass->ClassFlags |= CLASS_NotPlaceable | CLASS_Hidden;

	WrapperClass->SetMetaData(TEXT("DisplayName"), TEXT("RigVM Node"));

	// create properties - one for each pin to inspect
	FField::FLinkedListBuilder PropertyListBuilder(&WrapperClass->ChildProperties);

	for(URigVMPin* Pin : PinsToInspect)
	{
		FProperty* Property = nullptr;
		FProperty** ElementProperty = &Property;
		FFieldVariant PropertyOwner = WrapperClass;

		const FString CPPType = Pin->GetCPPType();
		FString BaseCPPType = CPPType;
		if(RigVMTypeUtils::IsArrayType(CPPType))
		{
			BaseCPPType = RigVMTypeUtils::BaseTypeFromArrayType(CPPType);
			
			// create an array property as a container for the tail
			FArrayProperty* ArrayProperty = new FArrayProperty(PropertyOwner, Pin->GetFName());
			Property = ArrayProperty;
			ElementProperty = &ArrayProperty->Inner;
			PropertyOwner = ArrayProperty;

			// In case the base type is also an array (RigVM supports up to 2 nested levels of arrays)
			if (RigVMTypeUtils::IsArrayType(BaseCPPType))
			{
				BaseCPPType = RigVMTypeUtils::BaseTypeFromArrayType(BaseCPPType);
				
				// create an array property as a container for the tail
				FArrayProperty* InnerArrayProperty = new FArrayProperty(PropertyOwner, Pin->GetFName());
				(*ElementProperty) = InnerArrayProperty;
				ElementProperty = &InnerArrayProperty->Inner;
				PropertyOwner = InnerArrayProperty;
			}
		}

		if(BaseCPPType.Equals(RigVMTypeUtils::BoolType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FBoolProperty(PropertyOwner, Pin->GetFName());
		}
		else if(BaseCPPType.Equals(RigVMTypeUtils::Int32Type, ESearchCase::IgnoreCase) ||
			BaseCPPType.Equals(RigVMTypeUtils::IntType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FIntProperty(PropertyOwner, Pin->GetFName());
		}
		else if(BaseCPPType.Equals(RigVMTypeUtils::FloatType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FFloatProperty(PropertyOwner, Pin->GetFName());
		}
		else if(BaseCPPType.Equals(RigVMTypeUtils::DoubleType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FDoubleProperty(PropertyOwner, Pin->GetFName());
		}
		else if(BaseCPPType.Equals(RigVMTypeUtils::FStringType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FStrProperty(PropertyOwner, Pin->GetFName());
		}
		else if(BaseCPPType.Equals(RigVMTypeUtils::FNameType, ESearchCase::IgnoreCase))
		{
			(*ElementProperty) = new FNameProperty(PropertyOwner, Pin->GetFName());
		}
		else if(Pin->GetCPPTypeObject())
		{
			if(UEnum* Enum = Cast<UEnum>(Pin->GetCPPTypeObject()))
			{
				FByteProperty* ByteProperty = new FByteProperty(PropertyOwner, Pin->GetFName());
				ByteProperty->Enum = Enum;
				(*ElementProperty) = ByteProperty;
			}
			else if(UClass* Class = Cast<UClass>(Pin->GetCPPTypeObject()))
			{
				const bool bIsClass = RigVMTypeUtils::IsUClassType(Pin->GetCPPType());
				if(bIsClass)
				{
					FClassProperty* ClassProperty = new FClassProperty(PropertyOwner, Pin->GetFName());
					ClassProperty->MetaClass = Class; 
					(*ElementProperty) = ClassProperty;
				}
				else
				{
					FObjectProperty* ObjectProperty = new FObjectProperty(PropertyOwner, Pin->GetFName());
					ObjectProperty->PropertyClass = Class; 
					(*ElementProperty) = ObjectProperty;
				}
			}
			else if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Pin->GetCPPTypeObject()))
			{
				FStructProperty* StructProperty = new FStructProperty(PropertyOwner, Pin->GetFName());
				StructProperty->Struct = ScriptStruct;
				(*ElementProperty) = StructProperty;
			}
		}

		if(Property == nullptr)
		{
			return nullptr;
		}
		if (ElementProperty && *ElementProperty == nullptr)
		{
			return nullptr;
		}
		Property->SetPropertyFlags(CPF_Edit);

		for(UScriptStruct* UnitStruct : UnitStructs)
		{
			if(const FProperty* UnitProperty = UnitStruct->FindPropertyByName(Property->GetFName()))
			{
				if(Property->SameType(UnitProperty))
				{
					FField::CopyMetaData(UnitProperty, Property);
					break;
				}
			}
		}

		Property->SetMetaData(TEXT("Category"), TEXT("Nodes"));

		PropertyListBuilder.AppendNoTerminate(*Property);
	}

	// Update the class
	WrapperClass->Bind();
	WrapperClass->StaticLink(true);
	
	// Similar to FConfigPropertyHelperDetails::CustomizeDetails, this is required for GC to work properly
	WrapperClass->AssembleReferenceTokenStream();

	InfoToClass.Add(PerClassInfo, WrapperClass);
	ClassToInfo.Add(WrapperClass, PerClassInfo);

	URigVMDetailsViewWrapperObject* CDO = Cast<URigVMDetailsViewWrapperObject>(WrapperClass->GetDefaultObject(true));
	CDO->AddToRoot();

	// setup the CDO's defaults to match the units
	if(UnitStructs.Num() > 0)
	{
		UScriptStruct* UnitStruct = UnitStructs[0];
		FStructOnScope DefaultStructScope(UnitStruct);
		const FRigVMStruct* DefaultStruct = (const FRigVMStruct*)DefaultStructScope.GetStructMemory();

		for (TFieldIterator<FProperty> UnitProperty(UnitStruct); UnitProperty; ++UnitProperty)
		{
			if(FProperty* Property = WrapperClass->FindPropertyByName(UnitProperty->GetFName()))
			{
				if(Property->SameType(*UnitProperty))
				{
					const uint8* SourceMemory = UnitProperty->ContainerPtrToValuePtr<uint8>(DefaultStruct);
					uint8* TargetMemory = Property->ContainerPtrToValuePtr<uint8>(CDO);
					Property->CopyCompleteValue(TargetMemory, SourceMemory);
				}
			}
		}
	}
	
	if(bMatchingNodes)
	{
		if(DispatchFactories.Num() > 0)
		{
			const FRigVMDispatchFactory* Factory = DispatchFactories[0];

			for(URigVMPin* Pin : PinsToInspect)
			{
				const FString DefaultValue = Factory->GetArgumentDefaultValue(Pin->GetFName(), Pin->GetTypeIndex());
				if(!DefaultValue.IsEmpty())
				{
					if(FProperty* Property = WrapperClass->FindPropertyByName(Pin->GetFName()))
					{
						uint8* TargetMemory = Property->ContainerPtrToValuePtr<uint8>(CDO);
						
						// use error pipe to ignore errors
						FRigVMPinDefaultValueImportErrorContext ErrorPipe;
						Property->ImportText_Direct(*DefaultValue, TargetMemory, CDO, PPF_None, &ErrorPipe);
					}
				}
			}
		}

		if(FunctionIdentifiers.Num() > 0)
		{
			const FRigVMGraphFunctionIdentifier Identifier = FunctionIdentifiers[0];
			const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(Identifier);

			for(URigVMPin* Pin : PinsToInspect)
			{
				const FRigVMGraphFunctionArgument* Argument = Header.Arguments.FindByPredicate([Pin](const FRigVMGraphFunctionArgument& InArgument)-> bool
				{
					return InArgument.Name == Pin->GetFName();
				});
				if(Argument)
				{
					if(!Argument->DefaultValue.IsEmpty())
					{
						if(FProperty* Property = WrapperClass->FindPropertyByName(Pin->GetFName()))
						{
							uint8* TargetMemory = Property->ContainerPtrToValuePtr<uint8>(CDO);
							
							// use error pipe to ignore errors
							FRigVMPinDefaultValueImportErrorContext ErrorPipe;
							Property->ImportText_Direct(*Argument->DefaultValue, TargetMemory, CDO, PPF_None, &ErrorPipe);
						}
					}
				}
			}
		}

	}

#if WITH_EDITOR
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	EditModule.RegisterCustomClassLayout(WrapperClass->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMWrappedNodeDetailCustomization::MakeInstance));
#endif


	return WrapperClass;	
}

URigVMDetailsViewWrapperObject* URigVMDetailsViewWrapperObject::MakeInstance(UClass* InWrapperObjectClass, UObject* InOuter, TArray<URigVMNode*> InNodes, URigVMNode* InSubject)
{
	InOuter = InOuter == nullptr ? GetTransientPackage() : InOuter;

	const URigVMDetailsViewWrapperObject* CDO = CastChecked<URigVMDetailsViewWrapperObject>(InWrapperObjectClass->GetDefaultObject());
	const UClass* WrapperClass = CDO->GetClassForNodes(InNodes);
	if(WrapperClass == nullptr)
	{
		return nullptr;
	}
	
	URigVMDetailsViewWrapperObject* Instance = GetOrCreateNewInstance(WrapperClass, InOuter);

	if (Instance)
	{
		Instance->SetContent(InSubject);
		Instance->SetSubject(InSubject);

		InSubject->GetGraph()->OnModified().AddUObject(Instance, &URigVMDetailsViewWrapperObject::HandleModifiedEvent);
	}
	
	return Instance;

}

void URigVMDetailsViewWrapperObject::MarkOutdatedClass(UClass* InClass)
{
	if (InClass)
	{
		OutdatedClassToRecreate.Add(InClass);
		ClearClassPool(InClass);
	}
}

bool URigVMDetailsViewWrapperObject::IsValidClass(UClass* InClass)
{
	if (!InClass)
	{
		return false;
	}
	
	if (!InClass->IsChildOf(StaticClass()))
	{
		return false;
	}
		
	if (InClass->GetName().Contains(DiscardedWrapperClassTemplateName))
	{
		return false;
	}

	return true;
}

bool URigVMDetailsViewWrapperObject::IsActive() const
{
	return SubjectPtr.IsValid() && IsValidClass(GetClass());
}

URigVMDetailsViewWrapperObject* URigVMDetailsViewWrapperObject::GetOrCreateNewInstance(const UClass* InWrapperObjectClass, UObject* InOuter)
{
	URigVMDetailsViewWrapperObject* Instance = nullptr;

	// Try to get from pool first
	if (TArray<URigVMDetailsViewWrapperObject*>* Pool = InstancePool.Find(InWrapperObjectClass))
	{
		// Filter out any invalid instances
		Pool->RemoveAll([](URigVMDetailsViewWrapperObject* Candidate)
		{
			return !IsValid(Candidate);
		});
		
		if (Pool->Num() > 0)
		{
			Instance = Pool->Pop();
			
			// Rename to new outer if needed
			if (Instance->GetOuter() != InOuter)
			{
				Instance->Rename(nullptr, InOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			}
		}
	}
	
	// Otherwise, create new
	if (Instance == nullptr || !IsValid(Instance))
	{
		Instance = NewObject<URigVMDetailsViewWrapperObject>(InOuter, InWrapperObjectClass, NAME_None, RF_Public | RF_Transient | RF_TextExportTransient | RF_DuplicateTransient);
	}
	
	return Instance;
}

void URigVMDetailsViewWrapperObject::ReturnToPool(URigVMDetailsViewWrapperObject* InInstance)
{
	if (!InInstance || !IsValid(InInstance))
		return;

	UClass* InstanceClass = InInstance->GetClass();

	// Don't pool instances of outdated classes
	if (OutdatedClassToRecreate.Contains(InstanceClass))
	{
		InInstance->RemoveFromRoot();
		InInstance->MarkAsGarbage();
		return;
	}

	TArray<URigVMDetailsViewWrapperObject*>& Pool = InstancePool.FindOrAdd(InstanceClass);

	// Don't exceed max pool size
	if (Pool.Num() >= MaxPoolSizePerClass)
	{
		InInstance->RemoveFromRoot();
		InInstance->MarkAsGarbage();
		return;
	}

	// Reset state and add to pool
	InInstance->ResetForReuse();
	Pool.Add(InInstance);
}

void URigVMDetailsViewWrapperObject::ClearClassPool(UClass* InClass)
{
	TArray<URigVMDetailsViewWrapperObject*> Pool;
	if (InstancePool.RemoveAndCopyValue(InClass, Pool))
	{
		for (URigVMDetailsViewWrapperObject* Instance : Pool)
		{
			if (IsValid(Instance))
			{
				Instance->RemoveFromRoot();
				Instance->MarkAsGarbage();
			}
		}
	}
}

void URigVMDetailsViewWrapperObject::ClearAllClassPools()
{
	// During shutdown, we don't need to remove instances individually
	if (!IsEngineExitRequested())
	{
		TArray<UClass*> Keys;
		InstancePool.GetKeys(Keys);

		for (UClass* Key : Keys)
		{
			ClearClassPool(Key);
		}
	}
	
	InstancePool.Empty();
}

FString URigVMDetailsViewWrapperObject::GetWrappedNodeNotation() const
{
	return ClassToInfo.FindChecked(GetClass()).Notation;
}

void URigVMDetailsViewWrapperObject::SetContent(const uint8* InStructMemory, const UStruct* InStruct)
{
	CopyPropertiesForUnrelatedStructs((uint8*)this, GetClass(), InStructMemory, InStruct);
}

void URigVMDetailsViewWrapperObject::GetContent(uint8* OutStructMemory, const UStruct* InStruct) const
{
	CopyPropertiesForUnrelatedStructs(OutStructMemory, InStruct, (const uint8*)this, GetClass());
}

void URigVMDetailsViewWrapperObject::SetContent(URigVMNode* InNode)
{
	for(URigVMPin* Pin : InNode->GetPins())
	{
		SetContentForPin(Pin);
	}
}

UObject* URigVMDetailsViewWrapperObject::GetSubject() const
{
	if(SubjectPtr.IsValid())
	{
		return SubjectPtr.Get();
	}
	return nullptr;
}

void URigVMDetailsViewWrapperObject::SetSubject(UObject* InSubject)
{
	SubjectPtr = TWeakObjectPtr<UObject>(InSubject);
}

void URigVMDetailsViewWrapperObject::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FString PropertyPath;

	const TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* PropertyNode = PropertyChangedEvent.PropertyChain.GetHead();
	do
	{
		FString PropertyName = PropertyNode->GetValue()->GetNameCPP();
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ToggleEditable && PropertyNode->GetValue() == PropertyChangedEvent.Property)
		{
			bool bNegate;
			if (FProperty* EditProperty = PropertyCustomizationHelpers::GetEditConditionProperty(PropertyChangedEvent.Property, bNegate))
			{
				PropertyName = EditProperty->GetName();
			}
		}
		int32 Index = PropertyChangedEvent.GetArrayIndex(PropertyName);
		if (Index != INDEX_NONE)
		{
			PropertyName = FString::Printf(TEXT("%s[%d]"), *PropertyName, Index);
		}
		if(PropertyPath.IsEmpty())
		{
			PropertyPath = PropertyName;
		}
		else
		{
			PropertyPath = FString::Printf(TEXT("%s->%s"), *PropertyPath, *PropertyName);
		}
		PropertyNode = PropertyNode->GetNextNode();
	}
	while(PropertyNode);

	TGuardValue<bool> ValueSettingGuard(bIsSettingValue, true);
	WrappedPropertyChangedChainEvent.Broadcast(this, PropertyPath, PropertyChangedEvent);
}

void URigVMDetailsViewWrapperObject::ResetForReuse()
{
	// Unbind from graph events if this was a node wrapper
	if (URigVMNode* Node = Cast<URigVMNode>(GetSubject()))
	{
		if (URigVMGraph* Graph = Node->GetGraph())
		{
			Graph->OnModified().RemoveAll(this);
		}
	}
	
	// Clear the subject reference
	SubjectPtr.Reset();

	// Clear the event delegate bindings
	WrappedPropertyChangedChainEvent.Clear();

	// Reset the value setting guard
	bIsSettingValue = false;

	// Reset all UPROPERTY values to defaults from CDO
	UClass* MyClass = GetClass();
	UObject* CDO = MyClass->GetDefaultObject();
	for (TFieldIterator<FProperty> It(MyClass); It; ++It)
	{
		FProperty* Property = *It;
		// Skip properties from parent class (URigVMDetailsViewWrapperObject itself)
		if (Property->GetOwnerClass() != MyClass)
			continue;

		uint8* DestPtr = Property->ContainerPtrToValuePtr<uint8>(this);
		const uint8* SrcPtr = Property->ContainerPtrToValuePtr<uint8>(CDO);
		Property->CopyCompleteValue(DestPtr, SrcPtr);
	}
	
	// Move to transient package so we don't hold references to the old outer
	Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
	AddToRoot();  // Ensure pooled instances won't be GC'd
}

void URigVMDetailsViewWrapperObject::CopyPropertiesForUnrelatedStructs(uint8* InTargetMemory,
	const UStruct* InTargetStruct, const uint8* InSourceMemory, const UStruct* InSourceStruct)
{
	check(InTargetMemory);
	check(InTargetStruct);
	check(InSourceMemory);
	check(InSourceStruct);
	
	for (TFieldIterator<FProperty> PropertyIt(InTargetStruct); PropertyIt; ++PropertyIt)
	{
		const FProperty* TargetProperty = *PropertyIt;
		const FProperty* SourceProperty = InSourceStruct->FindPropertyByName(TargetProperty->GetFName());
		if(SourceProperty == nullptr)
		{
			continue;
		}
		check(TargetProperty->SameType(SourceProperty));

		uint8* TargetPropertyMemory = TargetProperty->ContainerPtrToValuePtr<uint8>(InTargetMemory);
		const uint8* SourcePropertyMemory = SourceProperty->ContainerPtrToValuePtr<uint8>(InSourceMemory);
		TargetProperty->CopyCompleteValue(TargetPropertyMemory, SourcePropertyMemory);
	}
}

void URigVMDetailsViewWrapperObject::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph,
	UObject* InSubject)
{
	if(InNotifType != ERigVMGraphNotifType::PinDefaultValueChanged)
	{
		return;
	}
	
	URigVMPin* Pin = CastChecked<URigVMPin>(InSubject);
	if(Pin->GetNode() != GetSubject())
	{
		return;
	}

	SetContentForPin(Pin->GetRootPin());
}

void URigVMDetailsViewWrapperObject::SetContentForPin(URigVMPin* InPin)
{
	if(bIsSettingValue)
	{
		return;
	}
	TGuardValue<bool> SetValueGuard(bIsSettingValue, true);
	
	ensure(InPin->IsRootPin());

	const FString DefaultValue = InPin->GetDefaultValue();
	if(DefaultValue.IsEmpty())
	{
		return;
	}

	class FRigVMDetailsViewWrapperObjectImportErrorContext : public FOutputDevice
	{
	public:

		int32 NumErrors;

		FRigVMDetailsViewWrapperObjectImportErrorContext()
			: FOutputDevice()
			, NumErrors(0)
		{
		}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			UE_LOGF(LogRigVMDeveloper, Error, "Error Importing To Hierarchy: %ls", V);
			NumErrors++;
		}
	};

	// it's possible that the pin has been filtered out when creating the class
	if(const FProperty* Property = GetClass()->FindPropertyByName(InPin->GetFName()))
	{
		FRigVMDetailsViewWrapperObjectImportErrorContext ErrorPipe;
		Property->ImportText_InContainer(*DefaultValue, this, nullptr, EPropertyPortFlags::PPF_None, &ErrorPipe);
	}
}

