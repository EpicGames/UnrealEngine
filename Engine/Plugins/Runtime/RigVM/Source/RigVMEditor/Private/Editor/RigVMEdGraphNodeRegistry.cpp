// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMEdGraphNodeRegistry.h"

#include "EdGraph/RigVMEdGraph.h"
#include "Editor.h"
#include "RigVMEditorAsset.h"
#include "RigVMModel/RigVMNode.h"

namespace UE::RigVMEditor
{
	TMap<FName, TWeakPtr<FRigVMEdGraphNodeRegistry>> FRigVMEdGraphNodeRegistry::TypeIDToRegistryMap;

	TSharedRef<FRigVMEdGraphNodeRegistry> FRigVMEdGraphNodeRegistry::GetOrCreateRegistry(
		const TScriptInterface<IRigVMEditorAssetInterface>& InRigVMAssetInterface, 
		UClass* InRigVMNodeClass,
		const bool bForceUpdate)
	{		
		if (!ensureMsgf(InRigVMNodeClass->IsChildOf(URigVMNode::StaticClass()), TEXT("Node class is not a RigVMNode type, creating empty Rig VM Node Registry")) ||
			!ensureMsgf(InRigVMAssetInterface, TEXT("Rig VM Asset Interface is invalid, creating empty Rig VM Node Registry")))
		{
			return MakeShared<FRigVMEdGraphNodeRegistry>();
		}

		// Lazily cleanup existing registries
		for (auto It = TypeIDToRegistryMap.CreateIterator(); It; ++It)
		{
			if (!(*It).Value.IsValid())
			{
				It.RemoveCurrent();
			}
		}

		// Reuse registries where possible
		const FName TypeID = *(InRigVMNodeClass->GetFName().ToString() + TEXT("_") + InRigVMAssetInterface->GetObject()->GetFName().ToString());

		const TWeakPtr<FRigVMEdGraphNodeRegistry>* InstancePtr = TypeIDToRegistryMap.Find(TypeID);
		if (InstancePtr && InstancePtr->IsValid())
		{
			if (bForceUpdate)
			{
				(*InstancePtr).Pin()->ForceUpdate();
			}

			return (*InstancePtr).Pin().ToSharedRef();
		}
		else
		{
			const TSharedRef<FRigVMEdGraphNodeRegistry> NewRegistry = MakeShared<FRigVMEdGraphNodeRegistry>(InRigVMAssetInterface, InRigVMNodeClass, FRigVMEdGraphNodeRegistry::FPrivateToken());
			NewRegistry->Initialize();
			
			if (bForceUpdate)
			{
				NewRegistry->ForceUpdate();
			}
			else
			{
				NewRegistry->RequestUpdate();
			}

			TypeIDToRegistryMap.Add(TypeID, NewRegistry);

			return NewRegistry;
		}
	}

	FRigVMEdGraphNodeRegistry::FRigVMEdGraphNodeRegistry(const TScriptInterface<IRigVMEditorAssetInterface>& InRigVMAssetInterface, UClass* InRigVMNodeClass, FRigVMEdGraphNodeRegistry::FPrivateToken)
		: WeakAssetInterface(InRigVMAssetInterface)
		, RigVMNodeClass(InRigVMNodeClass)
	{}

	void FRigVMEdGraphNodeRegistry::Initialize()
	{
		FRigVMClient* ClientPtr = WeakAssetInterface.IsValid() ? WeakAssetInterface->GetRigVMClient() : nullptr;
		if (!ensureMsgf(WeakAssetInterface.IsValid() && ClientPtr, TEXT("Cannot update Rig VM Node Registry, Rig VM Asset Interface or Client is invalid")))
		{
			return;
		}

		ClientPtr->GetPostGraphModified().AddSP(this, &FRigVMEdGraphNodeRegistry::PostGraphModified);
	}

	void FRigVMEdGraphNodeRegistry::RequestUpdate()
	{
		if (!RequestUpdateTimerHandle.IsValid())
		{
			RequestUpdateTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(
				[WeakThis = AsWeak(), this]()
				{
					if (WeakThis.IsValid())
					{
						ForceUpdate();
						RequestUpdateTimerHandle.Invalidate();
					}
				});
		}
	}

	void FRigVMEdGraphNodeRegistry::ForceUpdate()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRigVMEdGraphNodeRegistry::ForceUpdate);

		if (!WeakAssetInterface.IsValid())
		{
			return;
		}

		TArray<UEdGraph*> EdGraphs;
		WeakAssetInterface->GetAllEdGraphs(EdGraphs);

		TArray<TWeakObjectPtr<URigVMEdGraphNode>> NewWeakConnectedEdGraphNodes;
		NewWeakConnectedEdGraphNodes.Reserve(WeakConnectedEdGraphNodes.Num());
		TArray<TWeakObjectPtr<URigVMEdGraphNode>> NewWeakDisconnectedEdGraphNodes;
		NewWeakDisconnectedEdGraphNodes.Reserve(WeakDisconnectedEdGraphNodes.Num());

		const bool bOnlyConsiderExecPin = RigVMNodeClass->IsChildOf(URigVMFunctionReferenceNode::StaticClass());
		for (UEdGraph* Graph : EdGraphs)
		{
			if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph))
			{
				TArray<URigVMEdGraphNode*> EdGraphNodes;
				RigVMEdGraph->GetNodesOfClass<URigVMEdGraphNode>(EdGraphNodes);

				for (URigVMEdGraphNode* EdGraphNode : EdGraphNodes)
				{
					URigVMNode* ModelNode = EdGraphNode->GetModelNode();
					if (ModelNode &&
						ModelNode->GetClass() == RigVMNodeClass)
					{
						const bool bConnectedNode = IsNodeConnected(*EdGraphNode, bOnlyConsiderExecPin);

						if (bConnectedNode)
						{
							NewWeakConnectedEdGraphNodes.Add(EdGraphNode);
						}
						else
						{
							NewWeakDisconnectedEdGraphNodes.Add(EdGraphNode);
						}
					}
				}
			}
		}

		if (NewWeakConnectedEdGraphNodes != WeakConnectedEdGraphNodes ||
			NewWeakDisconnectedEdGraphNodes != WeakDisconnectedEdGraphNodes)
		{
			WeakConnectedEdGraphNodes = NewWeakConnectedEdGraphNodes;
			WeakDisconnectedEdGraphNodes = NewWeakDisconnectedEdGraphNodes;

			OnPostRegistryUpdated.Broadcast();
		}
	}

	void FRigVMEdGraphNodeRegistry::PostGraphModified(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
	{
		if (InNotifType == ERigVMGraphNotifType::NodeAdded ||
			InNotifType == ERigVMGraphNotifType::NodeRemoved ||
			InNotifType == ERigVMGraphNotifType::LinkAdded ||
			InNotifType == ERigVMGraphNotifType::LinkRemoved)
		{
			RequestUpdate();
		}
	}

	bool FRigVMEdGraphNodeRegistry::IsNodeConnected(const URigVMEdGraphNode& EdGraphNode, const bool bOnlyConsiderExecPin) const
	{
		if (EdGraphNode.Pins.IsEmpty())
		{
			return false;
		}

		// For functions with exec pins, only consider them connected when an exec pin is connected
		if (bOnlyConsiderExecPin && 
			!EdGraphNode.GetExecutePins().IsEmpty())
		{
			return EdGraphNode.GetExecutePins().ContainsByPredicate(
				[](const URigVMPin* Pin)
				{
					return Pin && !Pin->GetLinks().IsEmpty();
				});
		}
		else
		{
			return EdGraphNode.Pins.ContainsByPredicate(
				[](const UEdGraphPin* Pin)
				{
					return Pin && !Pin->LinkedTo.IsEmpty();
				});
		}
	}
}
