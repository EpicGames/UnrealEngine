// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMEditor.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintCompilationManager.h"
#include "EdGraphNode_Comment.h"
#include "EditorModeManager.h"
#include "EulerTransform.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "RigVMEditorAsset.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMEditorCommands.h"
#include "RigVMPythonUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Editor/RigVMEditorMenuContext.h"
#include "Editor/RigVMEditorTools.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Editor/RigVMLegacyEditor.h"
#include "Editor/RigVMNewEditor.h"
#include "Editor/Transactor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMFunctions/RigVMFunction_ControlFlow.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Widgets/SRigVMBulkEditDialog.h"
#include "Widgets/SRigVMGraphBreakLinksWidget.h"
#include "Widgets/SRigVMGraphFunctionBulkEditWidget.h"
#include "Widgets/SRigVMSwapAssetReferencesWidget.h"
#include "Widgets/SRigVMSwapFunctionsWidget.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Stats/StatsHierarchical.h"
#include "ToolMenuContext.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SScrollBox.h"
#include "IStructureDetailsView.h"
#include "PropertyPath.h"
#include "Editor/RigVMDetailsInspectorTabSummoner.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "Overrides/OverrideStatusDetailsObjectFilter.h"
#include "Overrides/OverrideStatusDetailsWidgetBuilder.h"
#include "Overrides/OverrideStatusDetailsWidgetBuilder.h"
#include "GraphEditorActions.h"
#include "GraphEditorSettings.h"
#include "RigVMDeveloperTypeUtils.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMEditor"


FRigVMEditorBase::FRigVMEditorBase()
	: bAnyErrorsLeft(false)
	, KnownInstructionLimitWarnings()
	, LastDebuggedHost()
	, bSuspendDetailsPanelRefresh(false)
	, bDetailsPanelRequiresClear(false)
	, bAllowBulkEdits(false)
	, bIsSettingObjectBeingDebugged(false)
	, bRigVMEditorInitialized(false)
	, bIsCompilingThroughUI(false)
	, WrapperObjects()
	, LastEventQueue()
	, bRunGraphOnce(false)
{
}

void FRigVMEditorBase::UnbindEditor()
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	RigVMEditorClosedDelegate.Broadcast(this, RigVMBlueprint);

	GraphExplorerWidget = nullptr;
	ClearDetailObject();

	if(PropertyChangedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(PropertyChangedHandle);
	}
	
	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::CancelPIE.RemoveAll(this);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestedOpen().RemoveAll(this);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().RemoveAll(this);

	UnbindCommands();

	if (RigVMBlueprint)
	{
		// clear editor related data from the debugged control rig instance 
		RigVMBlueprint->SetObjectBeingDebugged(nullptr);

		IRigVMEditorAssetInterface::sCurrentlyOpenedRigVMBlueprints.Remove(RigVMBlueprint);

		RigVMBlueprint->OnRefreshEditor().RemoveAll(this);
		RigVMBlueprint->OnVariableDropped().RemoveAll(this);
		RigVMBlueprint->OnNodeDoubleClicked().RemoveAll(this);
		RigVMBlueprint->OnGraphImported().RemoveAll(this);
		RigVMBlueprint->OnRequestLocalizeFunctionDialog().RemoveAll(this);
		RigVMBlueprint->OnRequestBulkEditDialog().Unbind();
		RigVMBlueprint->OnRequestBreakLinksDialog().Unbind();
		RigVMBlueprint->OnRequestPinTypeSelectionDialog().Unbind();
		RigVMBlueprint->OnRequestJumpToHyperlink().Unbind();
		RigVMBlueprint->OnReportCompilerMessage().RemoveAll(this);

		RigVMBlueprint->OnModified().RemoveAll(this);
		RigVMBlueprint->OnVMCompiled().RemoveAll(this);
		RigVMBlueprint->OnRequestInspectObject().RemoveAll(this);
		RigVMBlueprint->OnRequestInspectMemoryStorage().RemoveAll(this);

		for(UEdGraph* Graph : RigVMBlueprint->GetUberGraphs())
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph))
			{
				RigGraph->OnGraphNodeClicked.RemoveAll(this);
			}
		}

#if WITH_EDITOR
		RigVMBlueprint->OnGetFocusedGraph().Unbind();
#endif

		if (URigVMHost* RigVMHost = GetRigVMHost())
		{
			RigVMHost->OnInitialized_AnyThread().RemoveAll(this);
			RigVMHost->OnExecuted_AnyThread().RemoveAll(this);
			RigVMHost->GetDebugInfo().ExecutionHalted().RemoveAll(this);
		}
	}

	FRigVMEditorModule::GetOnRequestFindNodeReferences().RemoveAll(this);

	if (bRequestedReopen)
	{
		// Sometimes FPersonaToolkit::SetPreviewMesh will request an asset editor close and reopen. If
		// SetPreviewMesh is called from within the editor, the close will not fully take effect until
		// the callback finishes, so the open editor action will fail. In that case, let's make sure we
		// detect a reopen requested, and open the editor again on the next tick.
		bRequestedReopen = false;
		FSoftObjectPath AssetToReopen = RigVMBlueprint->GetAssetObjectForEditor();
		GEditor->GetTimerManager()->SetTimerForNextTick([AssetToReopen]()
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetToReopen);
		});
	}
}

FRigVMEditorBase* FRigVMEditorBase::GetFromAssetEditorInstance(IAssetEditorInstance* Instance)
{
	FRigVMEditorBase* Editor = nullptr;
	FWorkflowCentricApplication* App = static_cast<FWorkflowCentricApplication*>(Instance);

	TSharedRef<FAssetEditorToolkit> SharedApp = App->AsShared();
#if WITH_RIGVMLEGACYEDITOR
	if (SharedApp->IsBlueprintEditor())
	{
		TSharedPtr<FRigVMLegacyEditor> LegacyEditor = StaticCastSharedPtr<FRigVMLegacyEditor>(App->AsShared().ToSharedPtr());
		Editor = LegacyEditor.Get();
	}
	else
	{
		TSharedPtr<FRigVMNewEditor> NewEditor = StaticCastSharedPtr<FRigVMNewEditor>(App->AsShared().ToSharedPtr());
		Editor = NewEditor.Get();
	}
#else
	TSharedPtr<FRigVMNewEditor> NewEditor = StaticCastSharedPtr<FRigVMNewEditor>(App->AsShared().ToSharedPtr());
	Editor = NewEditor.Get();
#endif
	return Editor;
}

void FRigVMEditorBase::InitRigVMEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	check(InRigVMBlueprint);

#if WITH_RIGVMLEGACYEDITOR
	if (InRigVMBlueprint->IsBlueprintAsset())
	{
		FBlueprintCompilationManager::FlushCompilationQueue(nullptr);
	}
#else
	// TODO: sara-s remove once blueprint backend is replaced
	// FRigVMBlueprintCompilationManager::FlushCompilationQueue(nullptr);
#endif

	CreateEditorToolbar();

	FEditorDelegates::EndPIE.AddRaw(this, &FRigVMEditorBase::OnPIEStopped);
	FEditorDelegates::CancelPIE.AddRaw(this, &FRigVMEditorBase::OnPIEStopped, false);

	// Build up a list of objects being edited in this asset editor
	TArray<UObject*> ObjectsBeingEdited;
	ObjectsBeingEdited.Add(InRigVMBlueprint.GetObject());

	// Initialize the asset editor and spawn tabs
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, GetEditorAppName(), FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsBeingEdited);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestedOpen().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleAssetRequestedOpen);
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestClose().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleAssetRequestClose);

	CreateDefaultCommands();
	
	if (UModeManagerInteractiveToolsContext* ToolsContext = GetToolkitEditorModeManager().GetInteractiveToolsContext())
	{
		ToolsContext->SetViewportInteractionsEnabled(true);
	}

	TArray<FRigVMEditorAssetInterfacePtr> Blueprints;
	Blueprints.Add(InRigVMBlueprint);
	InRigVMBlueprint->InitializeModelIfRequired();

	CommonInitialization(Blueprints, false);

	// Refresh the class actions
	{
		UClass* ActionKey = InRigVMBlueprint->GetObject()->GetClass();
		FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
		FBlueprintActionDatabase::FActionRegistry const& ActionRegistry = ActionDatabase.GetAllActions();
		ActionDatabase.ClearAssetActions(ActionKey);
		ActionDatabase.RefreshClassActions(ActionKey);
	}
	
	// user-defined-struct can change even after load
	// refresh the models such that pins are updated to match
	// the latest struct member layout
	InRigVMBlueprint->RefreshAllModels(ERigVMLoadType::CheckUserDefinedStructs);

	{
		TArray<UEdGraph*> EdGraphs;
		InRigVMBlueprint->GetAllEdGraphs(EdGraphs);

		for (UEdGraph* Graph : EdGraphs)
		{
			URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph);
			if (RigVMEdGraph == nullptr)
			{
				continue;
			}

			RigVMEdGraph->InitializeFromAsset(InRigVMBlueprint);
		}

	}

	IRigVMEditorAssetInterface::sCurrentlyOpenedRigVMBlueprints.AddUnique(InRigVMBlueprint);

	InRigVMBlueprint->OnModified().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleModifiedEvent);
	InRigVMBlueprint->OnVMCompiled().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleVMCompiledEvent);
	InRigVMBlueprint->OnRequestInspectObject().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::SetDetailObjects);
	InRigVMBlueprint->OnRequestInspectMemoryStorage().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::SetMemoryStorageDetails);

	BindCommands();

	TSharedPtr<FApplicationMode> ApplicationMode = CreateEditorMode();
	if(ApplicationMode.IsValid())
	{
		AddApplicationMode(
			GetEditorModeName(),
			ApplicationMode.ToSharedRef()
		);
	}

	ExtendMenu();
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	if(ApplicationMode.IsValid())
	{
		// Activate the initial mode (which will populate with a real layout)
		SetCurrentMode(GetEditorModeName());

		// Activate our edit mode
		GetToolkitEditorModeManager().SetDefaultMode(GetEditorModeName());
		GetToolkitEditorModeManager().ActivateMode(GetEditorModeName());
	}

	{
		TGuardValue<bool> GuardCompileReEntry(bIsCompilingThroughUI, true); // avoid redundant compilation, as it will be done at RebuildGraphFromModel
		UpdateRigVMHost();
	}
	
	// Post-layout initialization
	PostLayoutBlueprintEditorInitialization();

	// tabs opened before reload
	FString ActiveTabNodePath;
	TArray<FString> OpenedTabNodePaths;

	if (ShouldOpenGraphByDefault() && (Blueprints.Num() > 0))
	{
		bool bBroughtGraphToFront = false;
		for(UEdGraph* Graph : Blueprints[0]->GetUberGraphs())
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(Graph))
			{
				if (!bBroughtGraphToFront)
				{
					OpenGraphAndBringToFront(Graph, false);
					bBroughtGraphToFront = true;
				}

				RigGraph->OnGraphNodeClicked.AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnGraphNodeClicked);
				ActiveTabNodePath = RigGraph->ModelNodePath;
			}
		}
	}

	{
		if (URigVMGraph* Model = InRigVMBlueprint->GetDefaultModel())
		{
			if (Model->GetNodes().Num() == 0)
			{
				CreateEmptyGraphContent(InRigVMBlueprint->GetController());
			}
			else
			{
				// remember all ed graphs which were visible as tabs
				TArray<UEdGraph*> EdGraphs;
				InRigVMBlueprint->GetAllEdGraphs(EdGraphs);

				for (UEdGraph* EdGraph : EdGraphs)
				{
					if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(EdGraph))
					{
						TArray<TSharedPtr<SDockTab>> TabsForEdGraph;
						FindOpenTabsContainingDocument(EdGraph, TabsForEdGraph);

						if (TabsForEdGraph.Num() > 0)
						{
							OpenedTabNodePaths.Add(RigVMEdGraph->ModelNodePath);

							if(RigVMEdGraph->bIsFunctionDefinition)
							{
								CloseDocumentTab(RigVMEdGraph);
							}
						}
					}
				}

				InRigVMBlueprint->RebuildGraphFromModel();

				// selection state does not need to be persistent, even though it is saved in the RigVM.
				for (URigVMGraph* Graph : InRigVMBlueprint->GetAllModels())
				{
					InRigVMBlueprint->GetController(Graph)->ClearNodeSelection(false);
				}

				if (UPackage* Package = InRigVMBlueprint->GetObject()->GetOutermost())
				{
					Package->SetDirtyFlag(InRigVMBlueprint->IsMarkedDirtyDuringLoad());
				}
			}
		}

		InRigVMBlueprint->OnRefreshEditor().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleRefreshEditorFromBlueprint);
		InRigVMBlueprint->OnVariableDropped().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleVariableDroppedFromBlueprint);

		InRigVMBlueprint->OnNodeDoubleClicked().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnNodeDoubleClicked);
		InRigVMBlueprint->OnGraphImported().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnGraphImported);
		InRigVMBlueprint->OnRequestLocalizeFunctionDialog().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnRequestLocalizeFunctionDialog);
		InRigVMBlueprint->OnRequestBulkEditDialog().BindSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnRequestBulkEditDialog);
		InRigVMBlueprint->OnRequestBreakLinksDialog().BindSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnRequestBreakLinksDialog);
		InRigVMBlueprint->OnRequestPinTypeSelectionDialog().BindSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnRequestPinTypeSelectionDialog);
		InRigVMBlueprint->OnRequestJumpToHyperlink().BindSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleJumpToHyperlink);
#if WITH_EDITOR
		InRigVMBlueprint->OnGetFocusedGraph().BindSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::GetFocusedModel);
#endif
	}

	for (const FString& OpenedTabNodePath : OpenedTabNodePaths)
	{
		if (UEdGraph* EdGraph = InRigVMBlueprint->GetEdGraph(OpenedTabNodePath))
		{
			OpenDocument(EdGraph, FDocumentTracker::RestorePreviousDocument);
		}
	}

	if(ShouldOpenGraphByDefault())
	{
		if (UEdGraph* ActiveGraph = InRigVMBlueprint->GetEdGraph(ActiveTabNodePath))
		{
			OpenGraphAndBringToFront(ActiveGraph, true);
		}
	}

	FRigVMBlueprintUtils::HandleRefreshAllNodes(InRigVMBlueprint);

	if (Blueprints.Num() > 0)
	{
		if(Blueprints[0]->GetAssetStatus() == RVMA_Error || 
			Blueprints[0]->GetAssetStatus() == RVMA_Dirty || 
			Blueprints[0]->GetAssetStatus() == RVMA_Unknown)
		{
			Compile();
		}
	}

	TWeakPtr<IRigVMEditor> ThisWeakPtr = SharedRef().ToWeakPtr();
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [ThisWeakPtr, InRigVMBlueprint]()
	{
		if (TSharedPtr<IRigVMEditor> Editor = ThisWeakPtr.Pin())
		{
			// no need to do anything if the the CR is not opened anymore
		   // (i.e. destructor has been called before that task actually got a chance to start)
		   if (!IRigVMEditorAssetInterface::sCurrentlyOpenedRigVMBlueprints.Contains(InRigVMBlueprint))
		   {
			   return;		
		   }
		
		   TSharedPtr<FTabManager> TabManager = Editor->GetTabManager();
		   if (!TabManager.IsValid())
		   {
			   return;
		   }
		
		   // Always show the myblueprint tab
		   static const FTabId MyBlueprintTabId(Editor->GetGraphExplorerWidgetID());
		   if (!TabManager->FindExistingLiveTab(MyBlueprintTabId).IsValid())
		   {
			   TabManager->TryInvokeTab(MyBlueprintTabId);
		   }
		}
	});

	bRigVMEditorInitialized = true;
	InRigVMBlueprint->ClearPinWatches();

#if WITH_EDITOR
	FString BlueprintName = InRigVMBlueprint->GetObject()->GetPathName();
	RigVMPythonUtils::PrintPythonContext(BlueprintName);
#endif

	TArray<UScriptStruct*> StructsToCustomize = {
		TBaseStructure<FVector>::Get(),
		TBaseStructure<FVector2D>::Get(),
		TBaseStructure<FVector4>::Get(),
		TBaseStructure<FRotator>::Get(),
		TBaseStructure<FQuat>::Get(),
		TBaseStructure<FTransform>::Get(),
		TBaseStructure<FEulerTransform>::Get(),
	};

	auto Task = [&]<typename T>(T Inspector, UScriptStruct* StructToCustomize)
	{
		Inspector->GetPropertyView()->RegisterInstancedCustomPropertyTypeLayout(StructToCustomize->GetFName(),
				   FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]()
				   {
					   return FRigVMGraphMathTypeDetailCustomization::MakeInstance();
				   }));
	};

	TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
	for(UScriptStruct* StructToCustomize : StructsToCustomize)
	{
#if WITH_RIGVMLEGACYEDITOR
		if (SharedApp->IsBlueprintEditor())
		{
			if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
			{
				Task(Inspector, StructToCustomize);
			}
		}
#endif
		if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
		{
			Task(Inspector, StructToCustomize);
		}
	}

#if WITH_RIGVMLEGACYEDITOR
	if (SharedApp->IsBlueprintEditor())
	{
		if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
		{
			Inspector->GetPropertyView()->RegisterInstancedCustomPropertyTypeLayout(UEnum::StaticClass()->GetFName(),
			FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]()
			{
				return FRigVMGraphEnumDetailCustomization::MakeInstance();
			}));
		}
	}
	
	
#endif
	if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
	{
		Inspector->GetPropertyView()->RegisterInstancedCustomPropertyTypeLayout(UEnum::StaticClass()->GetFName(),
		   FOnGetPropertyTypeCustomizationInstance::CreateLambda([=]()
		   {
			   return FRigVMGraphEnumDetailCustomization::MakeInstance();
		   }));
	}

	PropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnPropertyChanged);

	FRigVMEditorModule::GetOnRequestFindNodeReferences().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnRequestFindNodeReferences);
}

void FRigVMEditorBase::HandleAssetRequestedOpen(UObject* InObject)
{
	if (InObject == GetRigVMAssetInterface()->GetObject())
	{
		bRequestedReopen = true;

		// This reopen request is useful only if when Persona SetPreviewMesh is set, and it automatically closes and reopens the editor
		// If this open editor request happens without closing the editor (for some other reason), we actually want to forget the reopen
		// Lets set a timer so that if a close editor does not happen, we forget the reopen request
		TWeakPtr<IRigVMEditor> WeakThis = SharedRef().ToWeakPtr();
		FTimerHandle RequestedReopenTimerHandle;
		GEditor->GetTimerManager()->SetTimer(RequestedReopenTimerHandle, [WeakThis]()
			{
				if (TSharedPtr<IRigVMEditor> StrongThis = WeakThis.Pin())
				{
					FRigVMEditorBase* Editor = static_cast<FRigVMEditorBase*>(StrongThis.Get());
					Editor->bRequestedReopen = false;
				}
			}, 2.0f, false, 1.0f);
	}
}

void FRigVMEditorBase::HandleAssetRequestClose(UObject* InObject, EAssetEditorCloseReason InReason)
{
	if (InObject == GetRigVMAssetInterface()->GetObject())
	{
		bRequestedReopen = false;
	}
}

const FName FRigVMEditorBase::GetEditorModeName() const
{
	return FRigVMEditorModes::RigVMEditorMode;
}

URigVMBlueprint* FRigVMEditorBase::GetRigVMBlueprint() const
{
	return Cast<URigVMBlueprint>(GetRigVMAssetInterface().GetObject());
}

FRigVMEditorAssetInterfacePtr FRigVMEditorBase::GetRigVMAssetInterface() const
{
	const TArray<UObject*>& EditingObjs = GetEditingBlueprints();
	for (UObject* Obj : EditingObjs)
	{
		if (Obj->Implements<URigVMEditorAssetInterface>()) 
		{
			return Obj;
		}
	}
	return nullptr;
}

TSubclassOf<UEdGraphSchema> FRigVMEditorBase::GetDefaultSchemaClass() const
{
	if(const FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		return RigVMBlueprint->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
	}
	return URigVMEdGraphSchema::StaticClass();
}

bool FRigVMEditorBase::InEditingMode() const
{
	// always allow editing - also during PIE.
	return true;
}

void FRigVMEditorBase::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	if (GraphExplorerWidget.IsValid())
	{
		GraphExplorerWidget->Refresh();
	}
}

void FRigVMEditorBase::Tick(float DeltaTime)
{
	// tick the  rigvm host
	if (const FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
	{
		if (LastDebuggedHost != GetCustomDebugObjectLabel(Blueprint->GetObjectBeingDebugged()))
		{
			TArray<FRigVMCustomDebugObject> DebugList;
			GetDebugObjects(DebugList);

			for (const FRigVMCustomDebugObject& DebugObject : DebugList)
			{
				if (DebugObject.NameOverride == LastDebuggedHost)
				{
					GetRigVMAssetInterface()->SetObjectBeingDebugged(DebugObject.Object);
					break;
				}
			}
		}
	}
}

void FRigVMEditorBase::BringToolkitToFront()
{
	if (IsHosted())
	{
		BringToolkitToFrontImpl();
	}
}

FName FRigVMEditorBase::GetToolkitFName() const
{
	return FName("RigVMEditor");
}

FName FRigVMEditorBase::GetToolkitContextFName() const
{
	return GetToolkitFName();
}

FText FRigVMEditorBase::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "RigVM Editor");
}

FText FRigVMEditorBase::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(GetRigVMAssetInterface()->GetObject());
}

FString FRigVMEditorBase::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "RigVM Editor ").ToString();
}

FLinearColor FRigVMEditorBase::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.25f, 0.35f, 0.5f );
}

void FRigVMEditorBase::InitToolMenuContextImpl(FToolMenuContext& MenuContext)
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		URigVMGraph* Model = nullptr;
		URigVMNode* Node = nullptr;
		URigVMPin* Pin = nullptr;
		
		if (UGraphNodeContextMenuContext* GraphNodeContext = MenuContext.FindContext<UGraphNodeContextMenuContext>())
		{
			if (GraphNodeContext->Node)
			{
				Model = RigVMBlueprint->GetModel(GraphNodeContext->Graph);
				if(Model)
				{
					Node = Model->FindNodeByName(GraphNodeContext->Node->GetFName());
				}
			}
		
			if (GraphNodeContext->Pin && Node)
			{
				Pin = Model->FindPin(GraphNodeContext->Pin->GetName());
			}
		}
		
		URigVMEditorMenuContext* RigVMEditorMenuContext = NewObject<URigVMEditorMenuContext>();
		const FRigVMEditorGraphMenuContext GraphMenuContext = FRigVMEditorGraphMenuContext(Model, Node, Pin);
		RigVMEditorMenuContext->Init(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), GraphMenuContext);

		MenuContext.AddObject(RigVMEditorMenuContext);
	}
}

bool FRigVMEditorBase::TransactionObjectAffectsBlueprintImpl(UObject* InTransactedObject)
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint == nullptr)
	{
		return false;
	}

	if (InTransactedObject->GetOuter() == GetFocusedController())
	{
		return false;
	}
	return true;
}

bool FRigVMEditorBase::CanAddNewLocalVariable() const
{
	if (const URigVMGraph* Graph = GetFocusedModel())
	{
		const URigVMGraph* ParentGraph = Graph->GetParentGraph();
		if (ParentGraph && ParentGraph->IsA<URigVMFunctionLibrary>())
		{
			return true;
		}
	}
	return false;
}

void FRigVMEditorBase::OnAddNewLocalVariable()
{
	if (!CanAddNewLocalVariable())
	{
		return;
	}

	FRigVMGraphVariableDescription LastTypeVar;
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromPinType(GetLastPinTypeUsed(), CPPType, &CPPTypeObject);
	LastTypeVar.ChangeType(CPPType, CPPTypeObject);
	FRigVMGraphVariableDescription NewVar = GetFocusedController()->AddLocalVariable(TEXT("NewLocalVar"), LastTypeVar.CPPType, LastTypeVar.CPPTypeObject, LastTypeVar.DefaultValue, true, true);
	if(NewVar.Name.IsNone())
	{
		LogSimpleMessage( LOCTEXT("AddLocalVariable_Error", "Adding new local variable failed.") );
	}
	else
	{
		RenameNewlyAddedAction(NewVar.Name);
	}
}

void FRigVMEditorBase::OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription)
{
	if (!CanAddNewLocalVariable())
	{
		return;
	}

	FRigVMGraphVariableDescription TypeVar;
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromPinType(VariableDescription.VarType, CPPType, &CPPTypeObject);
	TypeVar.ChangeType(CPPType, CPPTypeObject);
	FRigVMGraphVariableDescription NewVar = GetFocusedController()->AddLocalVariable(VariableDescription.VarName, TypeVar.CPPType, TypeVar.CPPTypeObject, VariableDescription.DefaultValue, true, true);
	if(NewVar.Name.IsNone())
	{
		LogSimpleMessage( LOCTEXT("PasteLocalVariable_Error", "Pasting new local variable failed.") );
	}
}

void FRigVMEditorBase::DeleteSelectedNodes()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint == nullptr)
	{
		return;
	}

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	SetUISelectionState(NAME_None);

	bool bRelinkPins = false;
	TArray<URigVMNode*> NodesToRemove;

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UEdGraphNode* Node = Cast<UEdGraphNode>(*NodeIt))
		{
			if (Node->CanUserDeleteNode())
			{
				AnalyticsTrackNodeEvent(GetRigVMAssetInterface().GetInterface(), Node, true);
				if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node))
				{
					bRelinkPins = bRelinkPins || FSlateApplication::Get().GetModifierKeys().IsShiftDown();

					if(URigVMNode* ModelNode = GetFocusedController()->GetGraph()->FindNodeByName(*RigVMEdGraphNode->ModelNodePath))
					{
						NodesToRemove.Add(ModelNode);
					}
				}
				else if (const UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
				{
					if(URigVMNode* ModelNode = GetFocusedController()->GetGraph()->FindNodeByName(CommentNode->GetFName()))
					{
						NodesToRemove.Add(ModelNode);
					}
				}
				else
				{
					Node->GetGraph()->RemoveNode(Node);
				}
			}
		}
	}

	if(NodesToRemove.IsEmpty())
	{
		return;
	}

	GetFocusedController()->OpenUndoBracket(TEXT("Delete selected nodes"));
	if(bRelinkPins && NodesToRemove.Num() == 1)
	{
		GetFocusedController()->RelinkSourceAndTargetPins(NodesToRemove[0], true);;
	}
	GetFocusedController()->RemoveNodes(NodesToRemove, true);
	GetFocusedController()->CloseUndoBracket();
}

bool FRigVMEditorBase::CanDeleteNodes() const
{
	return true;
}

void FRigVMEditorBase::CopySelectedNodes()
{
	FString ExportedText = GetFocusedController()->ExportSelectedNodesToText();
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
}

bool FRigVMEditorBase::CanCopyNodes() const
{
	return GetFocusedModel()->GetSelectNodes().Num() > 0;
}

bool FRigVMEditorBase::CanPasteNodes() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	return GetFocusedController()->CanImportNodesFromText(TextToImport);
}

void FRigVMEditorBase::CutSelectedNodes()
{
	CopySelectedNodes();
	DeleteSelectedNodes();
}

bool FRigVMEditorBase::CanCutNodes() const
{
	return CanCopyNodes() && CanDeleteNodes();
}

void FRigVMEditorBase::DuplicateNodes()
{
	// Copy and paste current selection
	CopySelectedNodes();
	PasteNodes();
}

bool FRigVMEditorBase::CanDuplicateNodes() const
{
	return CanCopyNodes() && IsEditable(GetFocusedGraph());
}

FReply FRigVMEditorBase::OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition,
                                                     UEdGraph* InGraph)
{
	if(!InChord.HasAnyModifierKeys())
	{
		if(URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(InGraph))
		{
			if(URigVMController* Controller = RigVMEdGraph->GetController())
			{
				if(InChord.Key == EKeys::B)
				{
					Controller->AddUnitNode(FRigVMFunction_ControlFlowBranch::StaticStruct(), FRigVMStruct::ExecuteName, FDeprecateSlateVector2D(InPosition), FString(), true, true);
				}
				else if (InChord.Key == EKeys::F)
				{
					URigVMTemplateNode* NewNode = Controller->AddTemplateNode(FRigVMDispatch_ArrayIterator().GetTemplateNotation(), FDeprecateSlateVector2D(InPosition), FString(), true, true);
				}
			}
		}
	}

	return FReply::Unhandled();
}

bool FRigVMEditorBase::JumpToHyperlinkImpl(const UObject* ObjectReference, bool bRequestRename)
{
	if(const URigVMEdGraph* Graph = Cast<URigVMEdGraph>(ObjectReference))
	{
		OpenGraphAndBringToFront((UEdGraph*)Graph, true);
		return true;
	}
	return false;
}

void FRigVMEditorBase::AddNewFunctionVariant(const UEdGraph* InOriginalFunction)
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif
	
	if (const URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(InOriginalFunction))
	{
		if (URigVMGraph* RigVMGraph = RigVMEdGraph->GetModel())
		{
			if (URigVMFunctionLibrary* FunctionLibrary = Cast<URigVMFunctionLibrary>(RigVMGraph->GetParentGraph()))
			{
				FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
				URigVMController* Controller = RigVMBlueprint->GetController(FunctionLibrary);
				if (const URigVMLibraryNode* VariantNode = Controller->CreateFunctionVariant(RigVMGraph->GetOuter()->GetFName(), NAME_None, true, true))
				{
					if (const UEdGraph* NewGraph = RigVMBlueprint->GetEdGraph(VariantNode->GetContainedGraph()))
					{
						OpenDocument(NewGraph, FDocumentTracker::OpenNewDocument);
					}
				}
			}
		}
	}
}

void FRigVMEditorBase::PostUndoImpl(bool bSuccess)
{
	const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount());
	PostTransaction(bSuccess, Transaction, false);
}

void FRigVMEditorBase::PostRedoImpl(bool bSuccess)
{
	const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount() - 1);
	PostTransaction(bSuccess, Transaction, true);
}

void FRigVMEditorBase::PostTransaction(bool bSuccess, const FTransaction* Transaction, bool bIsRedo)
{
	if(Transaction == nullptr)
	{
		return;
	}

	const FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if(Blueprint == nullptr)
	{
			return;
	}

	const UPackage* Package = Blueprint->GetObject()->GetPackage();

	TArray<UObject*> TransactedObjects;
	Transaction->GetTransactionObjects(TransactedObjects);
	
	for(const UObject* Object : TransactedObjects)
	{
		if(Object->GetPackage() == Package)
		{
			ForceEditorRefresh(ERefreshRigVMEditorReason::PostUndo);
			return;
		}
	}
}

void FRigVMEditorBase::OnStartWatchingPin()
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		GetFocusedController()->SetPinIsWatched(Pin->GetName(), true);
	}
}

bool FRigVMEditorBase::CanStartWatchingPin() const
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		if (URigVMPin* ModelPin = GetFocusedModel()->FindPin(Pin->GetName()))
		{
			return ModelPin->GetParentPin() == nullptr &&
					!ModelPin->RequiresWatch();
		}
	}
	return false;
}

void FRigVMEditorBase::OnStopWatchingPin()
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		GetFocusedController()->SetPinIsWatched(Pin->GetName(), false);
	}
}

bool FRigVMEditorBase::CanStopWatchingPin() const
{
	if (UEdGraphPin* Pin = GetCurrentlySelectedPin())
	{
		if (URigVMPin* ModelPin = GetFocusedModel()->FindPin(Pin->GetName()))
		{
			return ModelPin->RequiresWatch();
		}
	}
	return false;
}

TStatId FRigVMEditorBase::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FRigVMEditorBase, STATGROUP_Tickables);
}

void FRigVMEditorBase::PasteNodes()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	GetFocusedController()->OpenUndoBracket(TEXT("Pasted Nodes."));

	FVector2D PasteLocation = FSlateApplication::Get().GetCursorPos();

	TSharedPtr<SDockTab> ActiveTab = GetDocumentManager()->GetActiveTab();
	if (ActiveTab.IsValid())
	{
		TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
		if (GraphEditor.IsValid())
		{
			PasteLocation = GraphEditor->GetPasteLocation2f();

		}
	}

	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	URigVMController* FocusedController = GetFocusedController();

	const bool bPastePerformed = UE::RigVM::Editor::Tools::PasteNodes(PasteLocation, TextToImport, FocusedController, GetFocusedModel(), GetRigVMAssetInterface()->GetLocalFunctionLibrary(), GetRigVMAssetInterface()->GetRigVMClientHost()->GetRigVMGraphFunctionHost().GetInterface());
	if (bPastePerformed)
	{
		FocusedController->CloseUndoBracket();
	}
	else
	{
		FocusedController->CancelUndoBracket();
	}

}

URigVMHost* FRigVMEditorBase::GetRigVMHost() const
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		if(RigVMBlueprint->GetEditorHost() && IsValid(RigVMBlueprint->GetEditorHost()))
		{
			return RigVMBlueprint->GetEditorHost();
		}
	}
	return nullptr;
}

UObject* FRigVMEditorBase::GetOuterForHost() const
{
	if (FRigVMEditorAssetInterfacePtr EditorAsset = GetRigVMAssetInterface())
	{
		if (EditorAsset->IsBlueprintAsset())
		{
			return EditorAsset.GetObject();
		}
		
		return EditorAsset->GetRuntimeAssetInterface().GetObject();
	}
	return nullptr;
}

UClass* FRigVMEditorBase::GetDetailWrapperClass() const
{
	return URigVMDetailsViewWrapperObject::StaticClass();
}

bool FRigVMEditorBase::SelectLocalVariable(const UEdGraph* Graph, const FName& VariableName)
{
	if (const URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph))
	{
		if (URigVMGraph* RigVMGraph = RigVMEdGraph->GetModel())
		{
			for (FRigVMGraphVariableDescription& Variable : RigVMGraph->GetLocalVariables())
			{
				if (Variable.Name == VariableName)
				{
					URigVMDetailsViewWrapperObject* WrapperObject = URigVMDetailsViewWrapperObject::MakeInstance(
						GetDetailWrapperClass(), GetRigVMAssetInterface()->GetObject(), Variable.StaticStruct(), (uint8*)&Variable, RigVMGraph);
					WrapperObject->GetWrappedPropertyChangedChainEvent().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnWrappedPropertyChangedChainEvent);
					WrapperObject->AddToRoot();

					TArray<UObject*> Objects = {WrapperObject};
					SetDetailObjects(Objects, false);
					return true;
				}
			}
		}
	}
	return false;
}

bool FRigVMEditorBase::SelectVariable(const FName& VariableName)
{
	if (FRigVMEditorAssetInterfacePtr Asset = GetRigVMAssetInterface())
	{
		const FProperty* Prop = Asset->FindGeneratedPropertyByName(VariableName);
		UPropertyWrapper* PropWrap = (Prop ? const_cast<FProperty*>(Prop)->GetUPropertyWrapper() : nullptr);
		if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = GetRigVMInspector())
		{
			SRigVMDetailsInspector::FShowDetailsOptions Options(FText::FromName(VariableName));
			Options.bForceRefresh = true;
			
			if (RigVMInspector.IsValid())
			{
				RigVMInspector->ShowDetailsForSingleObject(PropWrap, Options);
				return true;
			}
		}
	}
	return false;
}

void FRigVMEditorBase::CreateDefaultCommandsImpl()
{
	if (GetRigVMAssetInterface())
	{
		// FBlueprintEditor::CreateDefaultCommands();
	}
	else
	{
		GetToolkitCommands()->MapAction( FGenericCommands::Get().Undo, 
			FExecuteAction::CreateSP( StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::UndoAction ));
		GetToolkitCommands()->MapAction( FGenericCommands::Get().Redo, 
			FExecuteAction::CreateSP( StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::RedoAction ));
	}
}

void FRigVMEditorBase::OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList)
{
	if (GraphEditorCommandsList.IsValid())
	{
		GraphEditorCommandsList->MapAction(FGraphEditorCommands::Get().OpenInNewTab,
			FExecuteAction::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnOpenSelectedNodesInNewTab),
			FCanExecuteAction::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::CanOpenSelectedNodesInNewTab),
			FIsActionButtonVisible::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::CanOpenSelectedNodesInNewTab)
		);
	}
}

void FRigVMEditorBase::Compile()
{
	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

		FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
		if (RigVMBlueprint == nullptr)
		{
			return;
		}

		// Disable single frame evaluation mode
		SetRunGraphOnce(false);

		// force to disable the supended notif brackets
		RigVMBlueprint->GetRigVMClient()->bSuspendModelNotificationsForOthers = false;
		RigVMBlueprint->bSuspendModelNotificationsForSelf = false;

		RigVMBlueprint->GetCompileLog().Messages.Reset();

		FString LastDebuggedObjectName = GetCustomDebugObjectLabel(RigVMBlueprint->GetObjectBeingDebugged());
		RigVMBlueprint->SetObjectBeingDebugged(nullptr);

		TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjects();

		if (URigVMHost* RigVMHost = GetRigVMHost())
		{
			RigVMHost->OnInitialized_AnyThread().Clear();
			RigVMHost->OnExecuted_AnyThread().Clear();
			RigVMHost->GetDebugInfo().ExecutionHalted().RemoveAll(this);
		}

		SetHost(nullptr);
		{
			TGuardValue<bool> GuardCompileReEntry(bIsCompilingThroughUI, true);
			CompileImpl();
			RigVMBlueprint->InitializeArchetypeInstances();
			UpdateRigVMHost();
		}

		if (URigVMHost* RigVMHost = GetRigVMHost())
		{
			RigVMLog.Reset();
			RigVMHost->SetLog(&RigVMLog);

			if (URigVM* VM = RigVMHost->GetVM())
			{
				FRigVMInstructionArray Instructions = VM->GetInstructions();

				if (Instructions.Num() <= 1) // just the "done" operator
				{
					FNotificationInfo Info(LOCTEXT("ControlRigBlueprintCompilerEmptyRigMessage", "The asset you compiled doesn't do anything. Did you forget to add a Begin_Execution node?"));
					Info.bFireAndForget = true;
					Info.FadeOutDuration = 5.0f;
					Info.ExpireDuration = 5.0f;
					TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
					NotificationPtr->SetCompletionState(SNotificationItem::CS_Success);
				}
			}
		}

		TArray<FRigVMCustomDebugObject> DebugList;
		GetDebugObjects(DebugList);

		for (const FRigVMCustomDebugObject& DebugObject : DebugList)
		{
			if (DebugObject.NameOverride == LastDebuggedObjectName)
			{
				RigVMBlueprint->SetObjectBeingDebugged(DebugObject.Object);
			}
		}

		// invalidate all node titles
		TArray<UEdGraph*> EdGraphs;
		RigVMBlueprint->GetAllEdGraphs(EdGraphs);
		for (UEdGraph* EdGraph : EdGraphs)
		{
			URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(EdGraph);
			if (RigVMEdGraph == nullptr)
			{
				continue;
			}

			for (UEdGraphNode* EdNode : RigVMEdGraph->Nodes)
			{
				if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(EdNode))
				{
					RigVMEdGraphNode->InvalidateNodeTitle();
				}
			}
		}

		// store the defaults from the CDO back on the new variables list
		bool bAnyVariableValueChanged = false;
		for(FRigVMGraphVariableDescription& NewVariable : RigVMBlueprint->GetAssetVariables())
		{
			bAnyVariableValueChanged |= UpdateDefaultValueForVariable(NewVariable, false);
		}
		if (bAnyVariableValueChanged)
		{
			// Go over all the instances to update the default values from CDO
			for (const FRigVMCustomDebugObject& DebugObject : DebugList)
			{
				if (URigVMHost* DebuggedHost = Cast<URigVMHost>(DebugObject.Object))
				{
					if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = DebuggedHost->GetGeneratedByAsset())
					{
						RuntimeAsset->InitializeInstance(DebuggedHost);
					}
					else
					{
						DebuggedHost->CopyExternalVariableDefaultValuesFromCDO();
					}
				}
			}
		}
	}

	// enable this for creating a new unit test
	// DumpUnitTestCode();

	// FStatsHierarchical::EndMeasurements();
	// FMessageLog LogForMeasurements("ControlRigLog");
	// FStatsHierarchical::DumpMeasurements(LogForMeasurements);
}

void FRigVMEditorBase::SaveAsset_Execute()
{
	LastDebuggedHost = GetCustomDebugObjectLabel(GetRigVMAssetInterface()->GetObjectBeingDebugged());
	SaveAsset_Execute_Impl();

	UpdateRigVMHost();
}

void FRigVMEditorBase::SaveAssetAs_Execute()
{
	LastDebuggedHost = GetCustomDebugObjectLabel(GetRigVMAssetInterface()->GetObjectBeingDebugged());
	SaveAssetAs_Execute_Impl();

	UpdateRigVMHost();
}

bool FRigVMEditorBase::IsEditable(UEdGraph* InGraph) const
{
	if(!IsGraphInCurrentBlueprint(InGraph))
	{
		return false;
	}
	
	if(FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		// aggregate graphs are always read only
		if(const URigVMGraph* Model = RigVMBlueprint->GetModel(InGraph))
		{
			if(Model->GetOuter()->IsA<URigVMAggregateNode>())
			{
				return false;
			}
		}
		const bool bIsReadOnly = RigVMBlueprint->IsEditingLocked();
		const bool bIsEditable = !bIsReadOnly;
		InGraph->bEditable = bIsEditable;
		return bIsEditable;
	}

	return IsEditableImpl(InGraph);
}

bool FRigVMEditorBase::IsCompilingEnabled() const
{
	return true;
}

FText FRigVMEditorBase::GetGraphDecorationString(UEdGraph* InGraph) const
{
	return FText::GetEmpty();
}

void FRigVMEditorBase::OnSelectedNodesChangedImpl(const TSet<UObject*>& NewSelection)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(GetFocusedGraph());
	if (RigVMEdGraph == nullptr)
	{
		return;
	}

	if (RigVMEdGraph->bIsSelecting || GIsTransacting)
	{
		return;
	}

	TGuardValue<bool> SelectGuard(RigVMEdGraph->bIsSelecting, true);

	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint)
	{
		TArray<FName> NodeNamesToSelect;
		for (UObject* Object : NewSelection)
		{
			if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Object))
			{
				NodeNamesToSelect.Add(RigVMEdGraphNode->GetModelNodeName());
			}
			else if(UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
			{
				NodeNamesToSelect.Add(Node->GetFName());
			}
		}
		GetFocusedController()->SetNodeSelection(NodeNamesToSelect, true, true);
	}
}

void FRigVMEditorBase::OnBlueprintChangedImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (!bRigVMEditorInitialized)
	{
		return;
	}

	OnBlueprintChangedInnerImpl(InBlueprint, bIsJustBeingCompiled);

	if(InBlueprint == GetRigVMAssetInterface().GetInterface())
	{
		if(bIsJustBeingCompiled)
		{
			UpdateRigVMHost();

			if (!LastDebuggedHost.IsEmpty())
			{
				TArray<FRigVMCustomDebugObject> DebugList;
				GetDebugObjects(DebugList);

				for (const FRigVMCustomDebugObject& DebugObject : DebugList)
				{
					if (DebugObject.NameOverride == LastDebuggedHost)
					{
						GetRigVMAssetInterface()->SetObjectBeingDebugged(DebugObject.Object);
						LastDebuggedHost.Empty();
						break;
					}
				}
			}
		}
	}}

void FRigVMEditorBase::ForceEditorRefresh(ERefreshRigVMEditorReason::Type Reason)
{
	if(Reason == ERefreshRigVMEditorReason::UnknownReason)
	{
		// we mark the reason as just compiled since we don't want to
		// update the graph(s) all the time during compilation
		Reason = ERefreshRigVMEditorReason::BlueprintCompiled;
	}
	RefreshEditorsImpl(Reason);
	if (GraphExplorerWidget.IsValid())
	{
		GraphExplorerWidget->Refresh();
	}
}

void FRigVMEditorBase::SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	SetupGraphEditorEventsImpl(InGraph, InEvents);

	InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleCreateGraphActionMenu);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnNodeTitleCommitted);
}

FActionMenuContent FRigVMEditorBase::HandleCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	return OnCreateGraphActionMenu(InGraph, InNodePosition, InDraggedPins, bAutoExpand, InOnMenuClosed);
}

void FRigVMEditorBase::OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (const UEdGraphNode_Comment* CommentBeingChanged = Cast<UEdGraphNode_Comment>(NodeBeingChanged))
	{
		if (GetRigVMAssetInterface())
		{
			GetFocusedController()->SetCommentTextByName(CommentBeingChanged->GetFName(), NewText.ToString(), CommentBeingChanged->FontSize, CommentBeingChanged->bCommentBubbleVisible, CommentBeingChanged->bColorCommentBubble, true, true);
		}
	}
	else if (const URigVMEdGraphNode* RigVMNodeBeingChanged = Cast<URigVMEdGraphNode>(NodeBeingChanged))
	{
		if(URigVMNode* ModelNode = RigVMNodeBeingChanged->GetModelNode())
		{
			if(URigVMController* Controller = GetFocusedController())
			{
				const FString NewNodeName = NewText.ToString();
				if(!ModelNode->GetName().Equals(NewNodeName, ESearchCase::CaseSensitive))
				{
					Controller->SetNodeTitle(ModelNode, *NewText.ToString(), true, true, true);
				}
			}
		}
	}
}

void FRigVMEditorBase::FocusInspectorOnGraphSelection(const TSet<UObject*>& NewSelection, bool bForceRefresh)
{
	// nothing to do here
}

void FRigVMEditorBase::AddReferencedObjects(FReferenceCollector& Collector)
{
	AddReferencedObjectsImpl(Collector);

	TWeakObjectPtr<URigVMHost> RigVMHost(GetRigVMHost());
	Collector.AddReferencedObject(RigVMHost);
}

void FRigVMEditorBase::BindCommands()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TSharedRef<FRigVMEditorBase> ThisRef = StaticCastSharedRef<FRigVMEditorBase>(SharedRef());
	TSharedRef<FUICommandList> ToolkitCommands = GetToolkitCommands();
	FRigVMEditorCommands& EditorCommands = FRigVMEditorCommands::Get();

	ToolkitCommands->MapAction(
		EditorCommands.AutoCompileGraph,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::ToggleAutoCompileGraph), 
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::CanAutoCompileGraph),
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::IsAutoCompileGraphOn));

	ToolkitCommands->MapAction(
		EditorCommands.LockGraph,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::ToggleLockGraph), 
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::CanLockGraph),
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::IsLockGraphOn));

	ToolkitCommands->MapAction(
		EditorCommands.ToggleEventQueue,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::ToggleEventQueue),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.RunGraphOnce,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::ToggleRunGraphOnce), 
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::CanRunGraphOnce),
		FIsActionChecked::CreateSP(ThisRef, &FRigVMEditorBase::IsRunGraphOnceOn));

	ToolkitCommands->MapAction(
		EditorCommands.FrameSelection,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::FrameSelection),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.SwapFunctionWithinAsset,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::SwapFunctionWithinAsset),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.SwapFunctionAcrossProject,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::SwapFunctionAcrossProject),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.SwapAssetReferences,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::SwapAssetReferences),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.ToggleProfiling,
		FExecuteAction::CreateSP(ThisRef, &FRigVMEditorBase::ToggleProfiling),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.TogglePreviewHere,
		FExecuteAction::CreateLambda([ThisRef]()
		{
			if (const FRigVMEditorAssetInterfacePtr RigVMAsset = ThisRef->GetRigVMAssetInterface())
			{
				if (const FRigVMClient* Client = RigVMAsset->GetRigVMClient())
				{
					if (const URigVMGraph* Graph = Client->GetModel(ThisRef->GetFocusedGraph()))
					{
						RigVMAsset->TogglePreviewHere(Graph);
					}
				}
			}
		}),
		FCanExecuteAction());

	ToolkitCommands->MapAction(
		EditorCommands.PreviewHereStepForward,
		FExecuteAction::CreateLambda([ThisRef]()
		{
			if (const FRigVMEditorAssetInterfacePtr RigVMAsset = ThisRef->GetRigVMAssetInterface())
			{
				RigVMAsset->PreviewHereStepForward();
			}
		}),
		FCanExecuteAction::CreateLambda([ThisRef]() -> bool
		{
			if (const FRigVMEditorAssetInterfacePtr RigVMAsset = ThisRef->GetRigVMAssetInterface())
			{
				return RigVMAsset->CanPreviewHereStepForward();
			}
			return false;
		}));
}

void FRigVMEditorBase::UnbindCommands()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	TSharedRef<FRigVMEditorBase> ThisRef = StaticCastSharedRef<FRigVMEditorBase>(SharedRef());
	TSharedRef<FUICommandList> ToolkitCommands = GetToolkitCommands();
	FRigVMEditorCommands& EditorCommands = FRigVMEditorCommands::Get();

	ToolkitCommands->UnmapAction(EditorCommands.AutoCompileGraph);
	ToolkitCommands->UnmapAction(EditorCommands.LockGraph);
	ToolkitCommands->UnmapAction(EditorCommands.ToggleEventQueue);
	ToolkitCommands->UnmapAction(EditorCommands.FrameSelection);
	ToolkitCommands->UnmapAction(EditorCommands.SwapFunctionWithinAsset);
	ToolkitCommands->UnmapAction(EditorCommands.SwapFunctionAcrossProject);
	ToolkitCommands->UnmapAction(EditorCommands.SwapAssetReferences);
	ToolkitCommands->UnmapAction(EditorCommands.ToggleProfiling);
	ToolkitCommands->UnmapAction(EditorCommands.TogglePreviewHere);
	ToolkitCommands->UnmapAction(EditorCommands.PreviewHereStepForward);
}

void FRigVMEditorBase::ToggleAutoCompileGraph()
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		RigVMBlueprint->SetAutoVMRecompile(!RigVMBlueprint->GetAutoVMRecompile());
		if (RigVMBlueprint->GetAutoVMRecompile())
		{
			RigVMBlueprint->RequestAutoVMRecompilation();
		}
	}
}

bool FRigVMEditorBase::IsAutoCompileGraphOn() const
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		return RigVMBlueprint->GetAutoVMRecompile();
	}
	return false;
}

void FRigVMEditorBase::ToggleLockGraph()
{
	if (FRigVMEditorAssetInterfacePtr AssetInterface = GetRigVMAssetInterface())
	{
		AssetInterface->SetEditingLocked(!AssetInterface->IsEditingLocked());
		AssetInterface->RequestAutoVMRecompilation();
	}
}

bool FRigVMEditorBase::IsLockGraphOn() const
{
	if (FRigVMEditorAssetInterfacePtr AssetInterface = GetRigVMAssetInterface())
	{
		return AssetInterface->IsEditingLocked();
	}
	return false;
}

bool FRigVMEditorBase::CanLockGraph() const
{
	if (FRigVMEditorAssetInterfacePtr AssetInterface = GetRigVMAssetInterface())
	{
		return AssetInterface->CanSetEditingLocked();
	}
	return false;
}

void FRigVMEditorBase::ToggleEventQueue()
{
	SetEventQueue(LastEventQueue);
}

void FRigVMEditorBase::ToggleRunGraphOnce()
{
	SetRunGraphOnce(!bRunGraphOnce);
}

void FRigVMEditorBase::SetRunGraphOnce(bool InRunGraphOnceEnabled)
{
	if (bRunGraphOnce == InRunGraphOnceEnabled)
	{
		return;
	}
	bRunGraphOnce = InRunGraphOnceEnabled;
	OnRunGraphOnceToggled(bRunGraphOnce);
}

bool FRigVMEditorBase::IsRunGraphOnceOn() const
{
	if (CanRunGraphOnce())
	{
		return bRunGraphOnce;
	}
	return false;
}

TSharedRef<SWidget> FRigVMEditorBase::GenerateEventQueueMenuContent()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());
	GenerateEventQueueMenuContent(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

void FRigVMEditorBase::GenerateEventQueueMenuContent(FMenuBuilder& MenuBuilder)
{
}

FMenuBuilder FRigVMEditorBase::GenerateBulkEditMenu()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());
	MenuBuilder.BeginSection(TEXT("Functions"), LOCTEXT("Functions", "Functions"));
	MenuBuilder.AddMenuEntry(FRigVMEditorCommands::Get().SwapFunctionWithinAsset, TEXT("SwapFunctionWithinAsset"), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon());
	MenuBuilder.AddMenuEntry(FRigVMEditorCommands::Get().SwapFunctionAcrossProject, TEXT("SwapFunctionAcrossProject"), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon());
	MenuBuilder.EndSection();
	// MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("Asset", "Asset"));
	// MenuBuilder.AddMenuEntry(FRigVMEditorCommands::Get().SwapAssetReferences, TEXT("SwapAssetReferences"), TAttribute<FText>(), TAttribute<FText>(), FSlateIcon());
	// MenuBuilder.EndSection();
	return MenuBuilder;
}

TSharedRef<SWidget> FRigVMEditorBase::GenerateBulkEditMenuContent()
{
	FMenuBuilder MenuBuilder = GenerateBulkEditMenu();
	return MenuBuilder.MakeWidget();
}

void FRigVMEditorBase::OnActiveTabChanged( TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated )
{
	if (!NewlyActivated.IsValid())
	{
		TArray<UObject*> ObjArray;
#if WITH_RIGVMLEGACYEDITOR
		TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
		if (SharedApp->IsBlueprintEditor())
		{
			if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
			{
				Inspector->ShowDetailsForObjects(ObjArray);
			}
		}
		
#endif
		if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
		{
			Inspector->ShowDetailsForObjects(ObjArray);
		}
	}
}

void FRigVMEditorBase::UndoAction()
{
	GEditor->UndoTransaction();
}

void FRigVMEditorBase::RedoAction()
{
	GEditor->RedoTransaction();
}

void FRigVMEditorBase::OnNewDocumentClicked(ECreatedDocumentType GraphType)
{
	if (GraphType == FRigVMNewEditor::CGT_NewFunctionGraph)
	{
		if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
		{
			if (URigVMController* Controller = RigVMBlueprint->GetOrCreateController(RigVMBlueprint->GetLocalFunctionLibrary()))
			{
				if (const URigVMLibraryNode* FunctionNode = Controller->AddFunctionToLibrary(TEXT("New Function"), true, FVector2D::ZeroVector, true, true))
				{
					if (const UEdGraph* NewGraph = RigVMBlueprint->GetEdGraph(FunctionNode->GetContainedGraph()))
					{
						OpenDocument(NewGraph, FDocumentTracker::OpenNewDocument);
						if (GraphExplorerWidget.IsValid())
						{
							GraphExplorerWidget->Refresh();
						}
						RenameNewlyAddedAction(FunctionNode->GetFName());
					}
				}
			}
		}
	}
	else if(GraphType == FRigVMNewEditor::CGT_NewEventGraph)
	{
		if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
		{
			const UClass* EdGraphSchemaClass = RigVMBlueprint->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
			const URigVMEdGraphSchema* SchemaCDO = CastChecked<URigVMEdGraphSchema>(EdGraphSchemaClass->GetDefaultObject());

			if(URigVMGraph* Model = RigVMBlueprint->AddModel(SchemaCDO->GetRootGraphName().ToString()))
			{
				if (const UEdGraph* NewGraph = RigVMBlueprint->GetEdGraph(Model))
				{
					OpenDocument(NewGraph, FDocumentTracker::OpenNewDocument);
					if (GraphExplorerWidget.IsValid())
					{
						GraphExplorerWidget->Refresh();
					}
					RenameNewlyAddedAction(NewGraph->GetFName());
				}
			}
		}
	}
}

bool FRigVMEditorBase::IsSectionVisibleImpl(RigVMNodeSectionID::Type InSectionID) const
{
	switch (InSectionID)
	{
		case RigVMNodeSectionID::GRAPH:
		case RigVMNodeSectionID::VARIABLE:
		case RigVMNodeSectionID::FUNCTION:
		{
			return true;
		}
		case RigVMNodeSectionID::LOCAL_VARIABLE:
		{
			if(const URigVMGraph* Graph = GetFocusedModel())
			{
				const URigVMGraph* ParentGraph = Graph->GetParentGraph();
				if (ParentGraph && ParentGraph->IsA<URigVMFunctionLibrary>())
				{
					return true;
				}
			}
		}
		default:
		{
			break;
		}
	}
	return false;
}

bool FRigVMEditorBase::NewDocument_IsVisibleForTypeImpl(ECreatedDocumentType GraphType) const
{
	switch(GraphType)
	{
		case ECreatedDocumentType::CGT_NewMacroGraph:
		case ECreatedDocumentType::CGT_NewAnimationLayer:
		{
			return false;
		}
		default:
		{
			break;
		}
	}
	return true;
}

FGraphAppearanceInfo FRigVMEditorBase::GetGraphAppearance(UEdGraph* InGraph) const
{
	FGraphAppearanceInfo AppearanceInfo = GetGraphAppearanceImpl(InGraph);

	if (const FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_RigVMEditor", "RigVM");

		if (RigVMBlueprint->IsEditingLocked())
		{
			AppearanceInfo.InstructionFade = 1;
			AppearanceInfo.InstructionText = FText::FromString(TEXT("This graph is locked."));
		}

		if(URigVMHost* RigVMHost = GetRigVMHost())
		{
#if RIGVM_TRACE_ENABLED
			if (RigVMHost->IsPlayingRewindDebugTrace())
			{
				AppearanceInfo.InstructionFade = 1;
				AppearanceInfo.InstructionText = FText::FromString(TEXT("Rewind Debug in Progress."));
			} else
#endif
			if(RigVMHost->GetVM() && RigVMHost->GetVM()->IsNativized())
			{
				AppearanceInfo.InstructionFade = 1;
				AppearanceInfo.InstructionText = FText::FromString(
					FString::Printf(TEXT("Running a nativized VM (U%s)."), *RigVMHost->GetVM()->GetClass()->GetName())
				);
			}

			if(RigVMHost->VMRuntimeSettings.bEnableProfiling)
			{
				static constexpr TCHAR Format[] = TEXT("Total %.02f µs");
				AppearanceInfo.WarningText = FText::FromString(FString::Printf(Format, (float)RigVMBlueprint->GetRigGraphDisplaySettings().TotalMicroSeconds));
			}
		}
	}

	return AppearanceInfo;
}

void FRigVMEditorBase::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint == nullptr)
	{
		return;
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::NodeSelectionChanged:
		case ERigVMGraphNotifType::NodeSelected:
		case ERigVMGraphNotifType::NodeDeselected:
		{
			if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(RigVMBlueprint->GetEdGraph(InGraph)))
			{
				TSharedPtr<SGraphEditor> GraphEd = GetGraphEditor(RigVMEdGraph);
				URigVMNode* Node = Cast<URigVMNode>(InSubject);
				if (InNotifType == ERigVMGraphNotifType::NodeSelectionChanged)
				{
					const TArray<FName> SelectedNodes = InGraph->GetSelectNodes();
					if (!SelectedNodes.IsEmpty())
					{
						Node = Cast<URigVMNode>(InGraph->FindNodeByName(SelectedNodes.Last()));	
					}
				}

				if (GraphEd.IsValid() && Node != nullptr)
				{
					SetDetailViewForGraph(Node->GetGraph());

					if (!RigVMEdGraph->bIsSelecting)
					{
						TGuardValue<bool> SelectingGuard(RigVMEdGraph->bIsSelecting, true);
						if (UEdGraphNode* EdNode = RigVMEdGraph->FindNodeForModelNodeName(Node->GetFName()))
						{
							GraphEd->SetNodeSelection(EdNode, InNotifType == ERigVMGraphNotifType::NodeSelected);
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinDefaultValueChanged:
		{
			const URigVMPin* Pin = Cast<URigVMPin>(InSubject);
			if(const URigVMPin* RootPin = Pin->GetRootPin())
			{
				const FString DefaultValue = Pin->GetDefaultValue();
				if(!DefaultValue.IsEmpty())
				{
					// sync the value change with the unit(s) displayed 
					TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjects();
					for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
					{
						if (SelectedObject.IsValid())
						{
							if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
							{
								if(WrapperObject->GetSubject() == Pin->GetNode())
								{
									if(const FProperty* Property = WrapperObject->GetClass()->FindPropertyByName(RootPin->GetFName()))
									{
										uint8* PropertyStorage = Property->ContainerPtrToValuePtr<uint8>(WrapperObject);

										// traverse to get to the target pin
										if(Pin != RootPin)
										{
											FString SegmentPath = Pin->GetSegmentPath();
											const FRigVMPropertyPath PropertyTraverser(Property, SegmentPath);
											PropertyStorage = PropertyTraverser.GetData<uint8>(PropertyStorage, Property);
											Property = PropertyTraverser.GetTailProperty();
										}

										// we are ok with not reacting to errors here
										if(Property && PropertyStorage)
										{
											FRigVMPinDefaultValueImportErrorContext ErrorPipe;
											Property->ImportText_Direct(*DefaultValue, PropertyStorage, nullptr, PPF_None, &ErrorPipe);
										}
									}
								}
							}
						}
					}
				}

				if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(RootPin->GetNode()))
				{
					if(UnitNode->IsEvent())
					{
						GetRigVMAssetInterface()->MarkAssetAsStructurallyModified();
						CacheNameLists();
					}
				}
			}
	
			break;
		}
		case ERigVMGraphNotifType::PinArraySizeChanged:
		case ERigVMGraphNotifType::PinBoundVariableChanged:
		case ERigVMGraphNotifType::PinTypeChanged:
		{
			URigVMPin* Pin = Cast<URigVMPin>(InSubject);

			if(Pin->GetNode()->IsSelected())
			{
				TArray<UObject*> Objects;
				Objects.Add(Pin->GetNode());
				SetDetailObjects(Objects);
			}
			break;
		}
		case ERigVMGraphNotifType::NodeRemoved:
		{
			if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(InSubject))
			{
				if (UEdGraph* EdGraph = RigVMBlueprint->GetEdGraph(CollapseNode->GetContainedGraph()))
				{
					CloseDocumentTab(EdGraph);
					ClearDetailObject();
				}
			}
			else if(URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(InSubject))
			{
				ClearDetailObject();
			}
				
			// fall through next case since we want to refresh the name lists
			// both for removing or adding an event
		}
		case ERigVMGraphNotifType::NodeAdded:
		{
			if(URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(InSubject))
			{
				if(UnitNode->IsEvent())
				{
					GetRigVMAssetInterface()->MarkAssetAsStructurallyModified();
					CacheNameLists();
				}
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

class FMemoryTypeMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FMemoryTypeMetaData, ISlateMetaData)

		FMemoryTypeMetaData(ERigVMMemoryType InMemoryType)
		: MemoryType(InMemoryType)
	{
	}
	ERigVMMemoryType MemoryType;
};

void FRigVMEditorBase::HandleVMCompiledEvent(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext)
{
	if (InCompiledObject->Implements<URigVMEditorAssetInterface>())
	{
		FRigVMEditorAssetInterfacePtr RigVMBlueprint = InCompiledObject;
		if (TSharedPtr<class IMessageLogListing> CompilerListing = GetCompilerResultsListing())
		{
			CompilerListing->ClearMessages();
			CompilerListing->AddMessages(RigVMBlueprint->GetCompileLog().Messages);
		}
		RigVMBlueprint->GetCompileLog().Messages.Reset();
		RigVMBlueprint->GetCompileLog().NumErrors = RigVMBlueprint->GetCompileLog().NumWarnings = 0;
	}

	RefreshDetailView();

	TArray<FName> TabIds;
	TabIds.Add(*FString::Printf(TEXT("RigVMMemoryDetails_%d"), (int32)ERigVMMemoryType::Literal));
	TabIds.Add(*FString::Printf(TEXT("RigVMMemoryDetails_%d"), (int32)ERigVMMemoryType::Work));
	TabIds.Add(*FString::Printf(TEXT("RigVMMemoryDetails_%d"), (int32)ERigVMMemoryType::Debug));

	for (const FName& TabId : TabIds)
	{
		TSharedPtr<SDockTab> ActiveTab = GetTabManager()->FindExistingLiveTab(TabId);
		if(ActiveTab)
		{
			if(ActiveTab->GetMetaData<FMemoryTypeMetaData>().IsValid())
			{
				ERigVMMemoryType MemoryType = ActiveTab->GetMetaData<FMemoryTypeMetaData>()->MemoryType;
				// TODO zzz : UE-195014 - Fix memory tab losing values on VM recompile
				FRigVMMemoryStorageStruct* Memory = InVM->GetMemoryByType(InContext, MemoryType);

			#if 1
				ActiveTab->RequestCloseTab();
				const TArray<FRigVMMemoryStorageStruct*> MemoryStorage = { Memory };
				SetMemoryStorageDetails(MemoryStorage);
			#else
				// TODO zzz : need a way to get the IStructureDetailsView
				TSharedRef<IStructureDetailsView> StructDetailsView = StaticCastSharedRef<IStructureDetailsView>(ActiveTab->GetContent());
				StructDetailsView->SetStructureProvider(MakeShared<FInstancePropertyBagStructureDataProvider>(*Memory));
			#endif
			}
		}
	}

	UpdateGraphCompilerErrors();
}

void FRigVMEditorBase::HandleVMExecutedEvent(URigVMHost* InHost, const FName& InEventName)
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		URigVMHost* DebuggedHost = Cast<URigVMHost>(RigVMBlueprint->GetObjectBeingDebugged());
		if (DebuggedHost == nullptr)
		{
			DebuggedHost = GetRigVMHost();
		}

		if(RigVMBlueprint->GetRigGraphDisplaySettings().NodeRunLimit > 1)
		{
			if(DebuggedHost)
			{
				if(URigVM* VM = DebuggedHost->GetVM())
				{
					bool bFoundLimitWarnings = false;
					
					const FRigVMByteCode& ByteCode = VM->GetByteCode();
					for(int32 InstructionIndex = 0; InstructionIndex < ByteCode.GetNumInstructions(); InstructionIndex++)
					{
						const int32 Count = VM->GetInstructionVisitedCount(DebuggedHost->GetRigVMExtendedExecuteContext(), InstructionIndex);
						if(Count > RigVMBlueprint->GetRigGraphDisplaySettings().NodeRunLimit)
						{
							bFoundLimitWarnings = true;

							const FString CallPath = VM->GetByteCode().GetCallPathForInstruction(InstructionIndex); 
							if(!KnownInstructionLimitWarnings.Contains(CallPath))
							{
								const FString Message = FString::Printf(
                                    TEXT("Instruction has hit the NodeRunLimit\n(ran %d times, limit is %d)\n\nYou can increase the limit in the class settings."),
                                    Count,
                                    RigVMBlueprint->GetRigGraphDisplaySettings().NodeRunLimit
                                );

								if(DebuggedHost->GetLog())
								{
									DebuggedHost->GetLog()->Entries.Add(
										FRigVMLog::FLogEntry(EMessageSeverity::Warning, InEventName, InstructionIndex, Message
									));
								}

								if(URigVMNode* Subject = Cast<URigVMNode>(VM->GetByteCode().GetSubjectForInstruction(InstructionIndex)))
								{
									FNotificationInfo Info(FText::FromString(Message));
									Info.bFireAndForget = true;
									Info.FadeOutDuration = 1.0f;
									Info.ExpireDuration = 5.0f;

									if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(RigVMBlueprint->GetEdGraph(Subject->GetGraph())))
									{
										if(UEdGraphNode* Node = EdGraph->FindNodeForModelNodeName(Subject->GetFName()))
										{
											Info.Hyperlink = FSimpleDelegate::CreateLambda([this, Node] ()
	                                        {
	                                            JumpToHyperlink(Node, false);
	                                        });
									
											Info.HyperlinkText = FText::FromString(Subject->GetName());
										}
									}

									TSharedPtr<SNotificationItem> NotificationPtr = FSlateNotificationManager::Get().AddNotification(Info);
									NotificationPtr->SetCompletionState(SNotificationItem::CS_Fail);
								}

								KnownInstructionLimitWarnings.Add(CallPath, Message);
							}
						}
					}

					if(!bFoundLimitWarnings)
					{
						KnownInstructionLimitWarnings.Reset();
					}
				}
			}
		}

		if(RigVMBlueprint->GetVMRuntimeSettings().bEnableProfiling)
		{
			if(DebuggedHost)
			{
				RigVMBlueprint->GetRigGraphDisplaySettings().SetTotalMicroSeconds(DebuggedHost->GetProfilingInfo().GetLastExecutionMicroSeconds());
			}

			if(RigVMBlueprint->GetRigGraphDisplaySettings().bAutoDetermineRange)
			{
				if(RigVMBlueprint->GetRigGraphDisplaySettings().LastMaxMicroSeconds < 0.0)
				{
					RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMinMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MinMicroSeconds); 
					RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMaxMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MaxMicroSeconds);
				}
				else if(RigVMBlueprint->GetRigGraphDisplaySettings().MaxMicroSeconds >= 0.0)
				{
					RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMinMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MinMicroSeconds); 
					RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMaxMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MaxMicroSeconds); 
				}

				RigVMBlueprint->GetRigGraphDisplaySettings().MinMicroSeconds = DBL_MAX; 
				RigVMBlueprint->GetRigGraphDisplaySettings().MaxMicroSeconds = (double)INDEX_NONE;
			}
			else
			{
				RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMinMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MinMicroSeconds); 
				RigVMBlueprint->GetRigGraphDisplaySettings().SetLastMaxMicroSeconds(RigVMBlueprint->GetRigGraphDisplaySettings().MaxMicroSeconds);
			}
		}
	}

	UpdateGraphCompilerErrors();
}

void FRigVMEditorBase::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	NotifyPreChangeImpl(PropertyAboutToChange);

	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		RigVMBlueprint->GetObject()->Modify();
	}
}

void FRigVMEditorBase::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	NotifyPostChangeImpl(PropertyChangedEvent, PropertyThatChanged);

	// we need to listen to changes for variables on the blueprint here since
	// OnFinishedChangingProperties is called only for top level property changes.
	// changes on a lower level property like transform under a user defined struct
	// only go through this.
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if(GetRigVMHost() && RigVMBlueprint)
	{
		bool bUseDebuggedObject = true; 
		if(PropertyChangedEvent.GetNumObjectsBeingEdited() == 1)
		{
			bUseDebuggedObject = !PropertyChangedEvent.GetObjectBeingEdited(0)->HasAnyFlags(RF_ClassDefaultObject);
		}
		
		const FName VarName = PropertyChangedEvent.MemberProperty->GetFName();
		for(FRigVMGraphVariableDescription& NewVariable : RigVMBlueprint->GetAssetVariables())
		{
			if(NewVariable.Name == VarName)
			{
				UpdateDefaultValueForVariable(NewVariable, bUseDebuggedObject);
				break;
			}
		}
	}
}

void FRigVMEditorBase::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();

	if (RigVMBlueprint)
	{
		if (PropertyChangedEvent.MemberProperty->GetNameCPP() == TEXT("VMCompileSettings"))
		{
			RigVMBlueprint->RecompileVM();
		}

		else if (PropertyChangedEvent.MemberProperty->GetNameCPP() == TEXT("VMRuntimeSettings"))
		{
			RigVMBlueprint->GetVMRuntimeSettings().Validate();
			RigVMBlueprint->PropagateRuntimeSettingsFromBPToInstances();
		}
	}
}

void FRigVMEditorBase::OnPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent)
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();

	if (RigVMBlueprint && InObject == RigVMBlueprint->GetObject())
	{
		// if the models have changed - we may need to close a document
		if(InEvent.MemberProperty == RigVMBlueprint->GetObject()->GetClass()->FindPropertyByName(TEXT("RigVMClient")) ||
			InEvent.MemberProperty == RigVMBlueprint->GetObject()->GetClass()->FindPropertyByName(TEXT("UbergraphPages")))
		{
			GetDocumentManager()->CleanInvalidTabs();
		}
	}
}

void FRigVMEditorBase::OnWrappedPropertyChangedChainEvent(URigVMDetailsViewWrapperObject* InWrapperObject,
	const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent)
{
	check(InWrapperObject);
	check(!WrapperObjects.IsEmpty());

	TGuardValue<bool> SuspendDetailsPanelRefresh(bSuspendDetailsPanelRefresh, true);

	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();

	FString PropertyPath = InPropertyPath;
	if(UScriptStruct* WrappedStruct = InWrapperObject->GetWrappedStruct())
	{
		if(WrappedStruct->IsChildOf(FRigVMGraphVariableDescription::StaticStruct()))
		{
			check(WrappedStruct == WrapperObjects[0]->GetWrappedStruct());
			
			const FRigVMGraphVariableDescription VariableDescription = InWrapperObject->GetContent<FRigVMGraphVariableDescription>();
			URigVMGraph* Graph = CastChecked<URigVMGraph>(InWrapperObject->GetSubject());
			URigVMController* Controller = RigVMBlueprint->GetController(Graph);
			if (PropertyPath == TEXT("Name"))
			{
				const FName OldVariableName = GetSelectedVariableName();
				if (!OldVariableName.IsNone())
				{
					for (FRigVMGraphVariableDescription& Variable : Graph->GetLocalVariables())
					{
						if (Variable.Name == OldVariableName)
						{
							Controller->RenameLocalVariable(OldVariableName, VariableDescription.Name);
							break;
						}
					}
				}

				ForceEditorRefresh();
				GetRigVMAssetInterface()->RequestAutoVMRecompilation();
			}
			else if (PropertyPath == TEXT("CPPType") || PropertyPath == TEXT("CPPTypeObject"))
			{			
				for (FRigVMGraphVariableDescription& Variable : Graph->GetLocalVariables())
				{
					if (Variable.Name == VariableDescription.Name)
					{
						Controller->SetLocalVariableType(Variable.Name, VariableDescription.CPPType, VariableDescription.CPPTypeObject);
						break;
					}
				}
				GetRigVMAssetInterface()->RequestAutoVMRecompilation();
			}
			else if (PropertyPath == TEXT("DefaultValue"))
			{
				for (FRigVMGraphVariableDescription& Variable : Graph->GetLocalVariables())
				{
					if (Variable.Name == VariableDescription.Name)
					{
						Controller->SetLocalVariableDefaultValue(Variable.Name, VariableDescription.DefaultValue, true, true);
						break;
					}
				}

				// Do not recompile now! That destroys the object that is currently being displayed (the literal memory storage), and can cause a crash.
				// The user has to manually trigger the recompilation.
			}		
		}
	}
	else if(!InWrapperObject->GetWrappedNodeNotation().IsEmpty())
	{
		URigVMNode* Node = CastChecked<URigVMNode>(InWrapperObject->GetSubject());

		const FName RootPinName = InPropertyChangedChainEvent.PropertyChain.GetHead()->GetValue()->GetFName();
		const FString RootPinNameString = RootPinName.ToString();
		FString PinPath = URigVMPin::JoinPinPath(Node->GetName(), RootPinNameString);
		URigVMController* Controller = GetRigVMAssetInterface()->GetController(Node->GetGraph());
		check(Controller);

		const FProperty* Property = WrapperObjects[0]->GetClass()->FindPropertyByName(RootPinName);
		uint8* PropertyStorage = nullptr;
		if (Property)
		{
			PropertyStorage = Property->ContainerPtrToValuePtr<uint8>(WrapperObjects[0].Get());

			// traverse to get to the target pin
			if(!InPropertyPath.Equals(RootPinNameString))
			{
				check(InPropertyPath.StartsWith(RootPinNameString));
				FString RemainingPropertyPath = InPropertyPath.Mid(RootPinNameString.Len());
				RemainingPropertyPath.RemoveFromStart(TEXT("->"));
                RemainingPropertyPath.ReplaceInline(TEXT("->"), TEXT("."));
				RemainingPropertyPath.RemoveFromStart(TEXT("["));
				RemainingPropertyPath.RemoveFromEnd(TEXT("]"));
				RemainingPropertyPath.ReplaceInline(TEXT("["), TEXT("."));
				RemainingPropertyPath.ReplaceInline(TEXT("]"), TEXT(""));
				
				if(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd)
				{
					PinPath = URigVMPin::JoinPinPath(PinPath, RemainingPropertyPath);
					
					const FRigVMPropertyPath PropertyTraverser(Property, RemainingPropertyPath);
					PropertyStorage = PropertyTraverser.GetData<uint8>(PropertyStorage, Property);
					Property = PropertyTraverser.GetTailProperty();
				}
				else if((InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayRemove) ||
					(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayClear) ||
					(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::Duplicate))
				{
					PinPath = URigVMPin::JoinPinPath(PinPath, RemainingPropertyPath);
				}
				else
				{
					// traverse each property one by one to make sure the expected pin exists.
					// this may not be the case for an array element.
					while(!RemainingPropertyPath.IsEmpty())
					{
						FString Left = RemainingPropertyPath, Right;
						(void)URigVMPin::SplitPinPathAtStart(RemainingPropertyPath, Left, Right);

						const FString NewPinPath = URigVMPin::JoinPinPath(PinPath, Left);

						// this may be an array pin which doesn't exist yet
						if(!Controller->GetGraph()->FindPin(NewPinPath))
						{
							break;
						}

						const FRigVMPropertyPath PropertyTraverser(Property, Left);
						PropertyStorage = PropertyTraverser.GetData<uint8>(PropertyStorage, Property);
						Property = PropertyTraverser.GetTailProperty();
						PinPath = NewPinPath;

						RemainingPropertyPath = Right;
					}
				}
			}
		}

		if (Property)
		{
			FString DefaultValue;
			
			if((InPropertyChangedChainEvent.ChangeType != EPropertyChangeType::ArrayRemove) &&
				(InPropertyChangedChainEvent.ChangeType != EPropertyChangeType::ArrayClear) &&
				(InPropertyChangedChainEvent.ChangeType != EPropertyChangeType::Duplicate))
			{
				if(PropertyStorage == nullptr)
				{
					// this may happen when we remove the last element from an array.
					// in that case just empty the array itself.
					if(const FProperty* ParentProperty = Property->GetOwnerProperty())
					{
						if(ParentProperty->IsA<FArrayProperty>())
						{
							DefaultValue = TEXT("()");
							FString Left, Right;
							verify(URigVMPin::SplitPinPathAtEnd(PinPath, Left, Right));
							PinPath = Left;
						}
					}
				}
				else
				{
					DefaultValue = FRigVMStruct::ExportToFullyQualifiedText(Property, PropertyStorage);
				}
			}
			
			if(Property->IsA<FStrProperty>() || Property->IsA<FNameProperty>())
			{
				DefaultValue.TrimCharInline(TEXT('\"'), nullptr);
			}
			
			if(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayAdd)
			{
				FString ArrayPinPath, ArrayElementIndex;
				verify(URigVMPin::SplitPinPathAtEnd(PinPath, ArrayPinPath, ArrayElementIndex));
				Controller->AddArrayPin(ArrayPinPath, DefaultValue, true, true);
			}
			else if(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayRemove)
			{
				Controller->RemoveArrayPin(PinPath, true, true);
			}
			else if(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayClear)
			{
				Controller->ClearArrayPin(PinPath, true, true);
			}
			else if(InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				Controller->DuplicateArrayPin(PinPath, true, true);
			}
			else if (!DefaultValue.IsEmpty())
			{
				const bool bInteractive = InPropertyChangedChainEvent.ChangeType == EPropertyChangeType::Interactive;
				Controller->SetPinDefaultValue(PinPath, DefaultValue, true, !bInteractive, true, !bInteractive);
			}
		}
	}
}

void FRigVMEditorBase::OnRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier& InFunction,
	URigVMController* InTargetController,
	IRigVMGraphFunctionHost* InTargetFunctionHost,
	bool bForce)
{
	UE::RigVM::Editor::Tools::OnRequestLocalizeFunctionDialog(InFunction, InTargetController, InTargetFunctionHost, bForce);
}

FRigVMController_BulkEditResult FRigVMEditorBase::OnRequestBulkEditDialog(FRigVMEditorAssetInterfacePtr InBlueprint,
	URigVMController* InController, URigVMLibraryNode* InFunction, ERigVMControllerBulkEditType InEditType)
{
	if (bAllowBulkEdits)
	{
		FRigVMController_BulkEditResult Result;
		Result.bCanceled = false; 
		Result.bSetupUndoRedo = false;
		return Result;
	}
	
	const TArray<FAssetData> FirstLevelReferenceAssets = InController->GetAffectedAssets(InEditType, false);
	if(FirstLevelReferenceAssets.Num() == 0)
	{
		return FRigVMController_BulkEditResult();
	}
	
	TSharedRef<SRigVMGraphFunctionBulkEditDialog> BulkEditDialog = SNew(SRigVMGraphFunctionBulkEditDialog)
	.Blueprint(InBlueprint)
	.Controller(InController)
	.Function(InFunction)
	.EditType(InEditType);

	FRigVMController_BulkEditResult Result;
	Result.bCanceled = BulkEditDialog->ShowModal() == EAppReturnType::Cancel; 
	Result.bSetupUndoRedo = false;

	if (!Result.bCanceled)
	{
		bAllowBulkEdits = true;
	}
	
	return Result;
}

bool FRigVMEditorBase::OnRequestBreakLinksDialog(TArray<URigVMLink*> InLinks)
{
	if(InLinks.Num() == 0)
	{
		return true;
	}

	TSharedRef<SRigVMGraphBreakLinksDialog> BreakLinksDialog = SNew(SRigVMGraphBreakLinksDialog)
	.Links(InLinks)
	.OnFocusOnLink(FRigVMOnFocusOnLinkRequestedDelegate::CreateLambda([&](URigVMLink* InLink)
	{
		HandleJumpToHyperlink(InLink);
	}));

	return BreakLinksDialog->ShowModal() == EAppReturnType::Ok; 
}

TRigVMTypeIndex FRigVMEditorBase::OnRequestPinTypeSelectionDialog(const TArray<TRigVMTypeIndex>& InTypes)
{
		if(InTypes.Num() == 0)
	{
		return true;
	}

	TRigVMTypeIndex Answer = INDEX_NONE;

	const FRigVMRegistry& Registry = FRigVMRegistry::Get();

	TArray<TSharedPtr<FName>> TypeNames;
	TMap<FName, uint8> TypeNameToIndex;
	TypeNames.Reserve(InTypes.Num());
	for (int32 i=0; i<InTypes.Num(); ++i)
	{
		const TRigVMTypeIndex& TypeIndex = InTypes[i];
		TRigVMTypeIndex FinalType = TypeIndex;
		if (FinalType == RigVMTypeUtils::TypeIndex::Float)
		{
			FinalType = RigVMTypeUtils::TypeIndex::Double;
		}
		if (FinalType == RigVMTypeUtils::TypeIndex::FloatArray)
		{
			FinalType = RigVMTypeUtils::TypeIndex::DoubleArray;
		}

		const FRigVMTemplateArgumentType& ArgumentType = Registry.GetType(FinalType);
		if (!TypeNames.ContainsByPredicate([&ArgumentType](const TSharedPtr<FName>& InName)
		{
			return *InName.Get() == ArgumentType.CPPType;
		}))
		{
			TypeNames.AddUnique(MakeShared<FName>(ArgumentType.CPPType));
			TypeNameToIndex.Add(ArgumentType.CPPType, static_cast<uint8>(i));
		}
	}
	TSharedPtr< SWindow > Window = SNew(SWindow)
		.Title(LOCTEXT("SelectPinType", "Select Pin Type"))
		.ScreenPosition(FSlateApplication::Get().GetCursorPos())
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew( SBorder )
			.Padding( 4.f )
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				SNew(SBox)
				.MaxDesiredHeight(static_cast<float>(300))
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					[
						SNew(SBox)
						.MaxDesiredHeight(static_cast<float>(300))
						[
							SNew(SScrollBox)
							+SScrollBox::Slot()
							[
								SNew(SListView<TSharedPtr<FName>>)
									.ListItemsSource(&TypeNames)
									.OnGenerateRow_Lambda([this, &Registry, &TypeNameToIndex, &InTypes](const TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& Owner)
									{
										TRigVMTypeIndex TypeIndex = InTypes[TypeNameToIndex.FindChecked(*InItem.Get())];
										const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(TypeIndex);
										const bool bIsArray = Type.IsArray();
										static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
										static const FLazyName ArrayTypeIcon(TEXT("Kismet.VariableList.ArrayTypeIcon"));

										const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromTypeIndex(TypeIndex);
										const URigVMEdGraphSchema* Schema = CastChecked<URigVMEdGraphSchema>(GetRigVMAssetInterface()->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass()->GetDefaultObject());
										const FLinearColor Color = Schema->GetPinTypeColor(PinType);
										
										return SNew(STableRow<TSharedPtr<FString>>, Owner)
												.Padding(FMargin(16, 4, 16, 4))
												[
													
													SNew(SHorizontalBox)
													+ SHorizontalBox::Slot()
													.AutoWidth()
													.VAlign(VAlign_Center)
													[
														SNew(SBox)
														.HeightOverride(16.0f)
														[
															SNew(SImage)
															.Image(bIsArray ? FAppStyle::GetBrush(ArrayTypeIcon) : FAppStyle::GetBrush(TypeIcon))
															.ColorAndOpacity(Color)
														]
													]

													+ SHorizontalBox::Slot()
													[
														SNew(STextBlock).Text(FText::FromName(*InItem.Get()))
													]
												];
									})
									.OnSelectionChanged_Lambda([&Answer, &TypeNames, &TypeNameToIndex, &InTypes](const TSharedPtr<FName> InName, ESelectInfo::Type)
									{
										Answer = InTypes[TypeNameToIndex.FindChecked(*InName.Get())];
										FSlateApplication::Get().GetActiveModalWindow()->RequestDestroyWindow();
									})
							]
						]
					]
				]
			]
		];

	GEditor->EditorAddModalWindow(Window.ToSharedRef());
	return Answer;
}

void FRigVMEditorBase::HandleJumpToHyperlink(const UObject* InSubject)
{
	FRigVMEditorAssetInterfacePtr RigBlueprint = GetRigVMAssetInterface();
	if(RigBlueprint == nullptr)
	{
		return;
	}

	const URigVMGraph* GraphToJumpTo = nullptr;
	const URigVMNode* NodeToJumpTo = nullptr;
	const URigVMPin* PinToJumpTo = nullptr;
	if(const URigVMNode* Node = Cast<URigVMNode>(InSubject))
	{
		GraphToJumpTo = Node->GetGraph();
		NodeToJumpTo = Node;

		if(const URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(Node))
		{
			if(CollapseNode->GetGraph()->IsA<URigVMFunctionLibrary>())
			{
				GraphToJumpTo = CollapseNode->GetContainedGraph();
				NodeToJumpTo = CollapseNode->GetEntryNode();
			}
		}
	}
	else if(const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
	{
		GraphToJumpTo = Pin->GetGraph();
		NodeToJumpTo = Pin->GetNode();
		PinToJumpTo = Pin;
	}
	else if(const URigVMLink* Link = Cast<URigVMLink>(InSubject))
	{
		GraphToJumpTo = Link->GetGraph();
		if(const URigVMPin* TargetPin = ((URigVMLink*)Link)->GetTargetPin())
		{
			NodeToJumpTo = TargetPin->GetNode();
			PinToJumpTo = TargetPin;
		}
	}

	if (GraphToJumpTo && NodeToJumpTo)
	{
		if(IRigVMEditorAssetInterface* OtherAsset = NodeToJumpTo->GetImplementingOuter<IRigVMEditorAssetInterface>())
		{
			if(OtherAsset != RigBlueprint.GetInterface())
			{
				UObject* OtherAssetObject = OtherAsset->GetAssetObjectForEditor();
				if (GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(OtherAssetObject))
				{
					IAssetEditorInstance* OtherEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(OtherAsset->GetObject(), /*bFocusIfOpen =*/true);
					if (FRigVMEditorBase* OtherRigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(OtherEditor))
					{
						OtherRigVMEditor->HandleJumpToHyperlink(NodeToJumpTo);
						return;
					}
				}
			}
					
			if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(OtherAsset->GetEdGraph(NodeToJumpTo->GetGraph())))
			{
				if(URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(EdGraph->FindNodeForModelNodeName(NodeToJumpTo->GetFName())))
				{
					if(PinToJumpTo)
					{
						if(const UEdGraphPin* EdGraphPin = EdGraphNode->FindPin(PinToJumpTo->GetSegmentPath(true)))
						{
							JumpToPin(EdGraphPin);
							return;
						}
					}
					
					JumpToNode(EdGraphNode);
					SetDetailObjects({EdGraphNode});
					return;
				}
				
				JumpToHyperlink(EdGraph);
			}
		}
	}
}

bool FRigVMEditorBase::UpdateDefaultValueForVariable(FRigVMGraphVariableDescription& InVariable, bool bUseDebuggedObject)
{
	bool bAnyValueChanged = false;
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		FString NewDefaultValue = RigVMBlueprint->GetVariableDefaultValue(InVariable.Name, bUseDebuggedObject);
		if (InVariable.DefaultValue != NewDefaultValue)
		{
			InVariable.DefaultValue = NewDefaultValue;
			bAnyValueChanged = true;
		}
		
	}
	return bAnyValueChanged;
}

void FRigVMEditorBase::UpdateRigVMHost()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();

	{
		bool bClearedHost = false;
		if (URigVMHost* CurrentHost = GetRigVMHost())
		{
			bClearedHost = true;
			UpdateRigVMHost_PreClearOldHost(CurrentHost);

			if(!IsValid(CurrentHost))
			{
				SetHost(nullptr);
			}
			
			// if this control rig is from a temporary step,
			// for example the reinstancing class, clear it 
			// and create a new one!
			if (!Blueprint->GetRuntimeAssetInterface()->IsObjectInstanceOfAsset(CurrentHost))
			{
				SetHost(nullptr);
			}
		}

		URigVMHost* RigVMHost = GetRigVMHost();
		if (RigVMHost == nullptr)
		{
			RigVMHost = Blueprint->CreateRigVMHostSuper(GetOuterForHost());
			SetHost(RigVMHost);
			
			// this is editing time rig
			RigVMHost->SetLog(&RigVMLog);

			RigVMHost->Initialize(true);
 		}
		check(IsValid(RigVMHost));

		RigVMHost->ResetEvaluationsLeft();

		CacheNameLists();

		// Make sure the object being debugged is the preview instance
		GetRigVMAssetInterface()->SetObjectBeingDebugged(RigVMHost);

		if(!bIsCompilingThroughUI)
		{
			TGuardValue<bool> UICompilationGuard(bIsCompilingThroughUI, true);
			Blueprint->GetObject()->SetFlags(RF_Transient);
			Blueprint->RecompileVM();
			Blueprint->GetObject()->ClearFlags(RF_Transient);
		}

		RigVMHost->OnInitialized_AnyThread().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleVMExecutedEvent);
		RigVMHost->OnExecuted_AnyThread().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleVMExecutedEvent);
		RigVMHost->RequestInit();
	}
}

void FRigVMEditorBase::CacheNameLists()
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		TArray<UEdGraph*> EdGraphs;
		RigVMBlueprint->GetAllEdGraphs(EdGraphs);

		for (UEdGraph* Graph : EdGraphs)
		{
			URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph);
			if (RigVMEdGraph == nullptr)
			{
				continue;
			}
			RigVMEdGraph->CacheEntryNameList();
		}
	}
}

void FRigVMEditorBase::OnCreateComment()
{
	TSharedPtr<SGraphEditor> GraphEditor = GetFocusedGraphEditor().Pin();
	if (GraphEditor.IsValid())
	{
		if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
		{
			if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph))
			{
				if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
				{
					if (URigVMController* Controller = Blueprint->GetController(RigVMEdGraph))
					{
						Controller->OpenUndoBracket(TEXT("Create Comment"));
						if (Blueprint->IsBlueprintAsset())
						{
							FEdGraphSchemaAction_K2AddComment CommentAction;
							UEdGraphNode* EdNode = CommentAction.PerformAction(Graph, NULL, GraphEditor->GetPasteLocation2f());
							if (UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(EdNode))
							{
								Controller->SetNodeColorByName(CommentNode->GetFName(), CommentNode->CommentColor, false);
								Controller->SetNodePositionByName(CommentNode->GetFName(), FVector2D(CommentNode->NodePosX, CommentNode->NodePosY), false);
							}
						}
						else
						{
							FSlateRect Bounds;
							FVector2D Size(400, 100);
							FVector2D Location = GraphEditor->GetPasteLocation2f();
							if (GraphEditor->GetBoundsForSelectedNodes(Bounds, 50.f))
							{
								Location = Bounds.GetTopLeft();
								Size = Bounds.GetSize();
							}
							Controller->AddCommentNode(TEXT("Comment"), Location, Size, GetDefault<UGraphEditorSettings>()->DefaultCommentNodeTitleColor, FString(), true, true);
						}
						Controller->CloseUndoBracket();
					}
				}
			}
		}
	}
}

TArray<TWeakObjectPtr<UObject>> FRigVMEditorBase::GetSelectedObjects() const
{
	// if the inspector shows wrapped objects - look in that array instead.
	// with recent weak object pointer changes on the property detail view
	// we cannot rely on the GetSelectedObjects being valid after blueprint compilation.

	auto Task = [this]<typename T>(T Inspector)
	{
		if(WrapperObjects.Num() == Inspector->GetSelectedObjects().Num())
		{
			TArray<TWeakObjectPtr<UObject>> WeakWrapperObjects;
			for(const TStrongObjectPtr<URigVMDetailsViewWrapperObject>& WrapperObjectPtr : WrapperObjects)
			{
				URigVMDetailsViewWrapperObject* WrapperObject = WrapperObjectPtr.Get();
				if(IsValid(WrapperObject->GetSubject()))
				{
					WeakWrapperObjects.Add(WrapperObject);
				}
			}
			return WeakWrapperObjects;
		}
		return Inspector->GetSelectedObjects();
	};
	
#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
	if (SharedApp->IsBlueprintEditor())
	{
		if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
		{
			return Task(Inspector);
		}
	}
#endif
	if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
	{
		return Task(Inspector);
	}
	return TArray<TWeakObjectPtr<UObject>>();
}

void FRigVMEditorBase::SetDetailObjects(const TArray<UObject*>& InObjects)
{
	SetDetailObjects(InObjects, true);
}

void FRigVMEditorBase::SetDetailObjects(const TArray<UObject*>& InObjects, bool bChangeUISelectionState)
{
	if(bSuspendDetailsPanelRefresh)
	{
		return;
	}

	if(InObjects.Num() == 1)
	{
		if(URigVMMemoryStorage* Memory = Cast<URigVMMemoryStorage>(InObjects[0]))
		{
			FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bHideSelectionTip = true;

			TSharedRef<IDetailsView> DetailsView = EditModule.CreateDetailView( DetailsViewArgs );
			TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.Label( LOCTEXT("RigVMMemoryDetails", "RigVM Memory Details") )
			.AddMetaData<FMemoryTypeMetaData>(FMemoryTypeMetaData(Memory->GetMemoryType()))
			.TabRole(ETabRole::NomadTab)
			[
				DetailsView
			];

			FName TabId = *FString::Printf(TEXT("RigVMMemoryDetails_%d"), (int32)Memory->GetMemoryType());
			if(TSharedPtr<SDockTab> ActiveTab = GetTabManager()->FindExistingLiveTab(TabId))
			{
				ActiveTab->RequestCloseTab();
			}

			GetTabManager()->InsertNewDocumentTab(
#if WITH_RIGVMLEGACYEDITOR
				GetRigVMInspector() ? FRigVMDetailsInspectorTabSummoner::TabID() : FBlueprintEditorTabs::DetailsID,
#else
				FRigVMDetailsInspectorTabSummoner::TabID(),
#endif
				TabId,
				FTabManager::FLastMajorOrNomadTab(TEXT("RigVMMemoryDetails")),
				DockTab
			);

			FFunctionGraphTask::CreateAndDispatchWhenReady([DetailsView, InObjects]()
			{
				
				DetailsView->SetObject(InObjects[0]);
				
			}, TStatId(), NULL, ENamedThreads::GameThread);

			return;
		}
	}

	if(bDetailsPanelRequiresClear)
	{
		ClearDetailObject(bChangeUISelectionState);
	}

	if (InObjects.Num() == 1)
	{
		if (InObjects[0]->GetClass()->GetDefaultObject() == InObjects[0])
		{
			EditClassDefaults_Clicked();
			return;
		}
		else if (InObjects[0] == GetRigVMAssetInterface()->GetObject())
		{
			EditGlobalOptions_Clicked();
			return;
		}
	}

	TArray<UObject*> FilteredObjects;

	TArray<URigVMNode*> ModelNodes;
	for(UObject* InObject : InObjects)
	{
		if (URigVMNode* ModelNode = Cast<URigVMNode>(InObject))
		{
			ModelNodes.Add(ModelNode);
		}
	}

	for(UObject* InObject : InObjects)
	{
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InObject))
		{
			if(!LibraryNode->IsA<URigVMFunctionReferenceNode>())
			{
				if (UEdGraph* EdGraph = GetRigVMAssetInterface()->GetEdGraph(LibraryNode->GetContainedGraph()))
				{
					FilteredObjects.AddUnique(EdGraph);
					ModelNodes.Remove(LibraryNode);
					continue;
				}
			}
		}
		else if (Cast<URigVMFunctionEntryNode>(InObject) || Cast<URigVMFunctionReturnNode>(InObject))
		{
			if (UEdGraph* EdGraph = GetRigVMAssetInterface()->GetEdGraph(CastChecked<URigVMNode>(InObject)->GetGraph()))
			{
				FilteredObjects.AddUnique(EdGraph);
				ModelNodes.Remove(Cast<URigVMNode>(InObject));
				continue;
			}
		}
		else if (URigVMCommentNode* CommentNode = Cast<URigVMCommentNode>(InObject))
		{
			TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
			if (SharedApp->IsBlueprintEditor())
			{
				if (URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(GetRigVMAssetInterface()->GetEdGraph(CastChecked<URigVMNode>(InObject)->GetGraph())))
				{
					if(UEdGraphNode* EdGraphNode = EdGraph->FindNodeForModelNodeName(CommentNode->GetFName()))
					{
						FilteredObjects.AddUnique(EdGraphNode);
						ModelNodes.Remove(CommentNode);
						continue;
					}
				}
			}
			else
			{
				FilteredObjects.AddUnique(CommentNode);
				ModelNodes.Remove(CommentNode);
				continue;
			}
		}

		if (URigVMNode* ModelNode = Cast<URigVMNode>(InObject))
		{
			// check if we know the dynamic class already
			const URigVMDetailsViewWrapperObject* CDOWrapper = CastChecked<URigVMDetailsViewWrapperObject>(GetDetailWrapperClass()->GetDefaultObject());
			(void)CDOWrapper->GetClassForNodes(ModelNodes, false);

			// create the wrapper object
			if(URigVMDetailsViewWrapperObject* WrapperObject = URigVMDetailsViewWrapperObject::MakeInstance(GetDetailWrapperClass(), GetRigVMAssetInterface()->GetObject(), ModelNodes, ModelNode))
			{
				WrapperObject->GetWrappedPropertyChangedChainEvent().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::OnWrappedPropertyChangedChainEvent);
				WrapperObject->AddToRoot();

				// todo: use transform widget for transforms
				// todo: use rotation widget for rotations
				
				FilteredObjects.Add(WrapperObject);
				continue;
			}
		}


		FilteredObjects.Add(InObject);
	}

	for(UObject* FilteredObject : FilteredObjects)
	{
		if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(FilteredObject))
		{
			WrapperObjects.Add(TStrongObjectPtr<URigVMDetailsViewWrapperObject>(WrapperObject));
		}
	}

	if(!ModelNodes.IsEmpty() && CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		TSharedPtr<FOverrideStatusDetailsViewObjectFilter> ObjectFilter = FOverrideStatusDetailsViewObjectFilter::Create();

		ObjectFilter->OnCanCreateWidget().BindLambda([](const FOverrideStatusSubject& InSubject) -> bool
		{
			return InSubject.Contains<URigVMDetailsViewWrapperObject>([](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper)
			{
				return Cast<URigVMNode>(InWrapper->GetSubject()) != nullptr;
			});
		});

		auto PropertyPathToPinPath = [](const FOverrideStatusSubject& InSubject) -> FString
		{
			FString PinPath = InSubject.GetPropertyPathString(TEXT("."));
			PinPath.ReplaceCharInline(TEXT('['), TEXT('.'));
			PinPath.ReplaceInline(TEXT("]"), TEXT(""));
			return PinPath;
		};

		ObjectFilter->OnGetStatus().BindLambda([PropertyPathToPinPath](const FOverrideStatusSubject& InSubject)
		{
			const FString PinPath = PropertyPathToPinPath(InSubject);
			return InSubject.GetStatus<URigVMDetailsViewWrapperObject>(
				[PinPath](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper) -> TOptional<EOverrideWidgetStatus::Type>
				{
					if(const URigVMNode* Node = Cast<URigVMNode>(InWrapper->GetSubject()))
					{
						if(PinPath.IsEmpty())
						{
							if(Node->GetPinDefaultValueOverrideState() != ERigVMNodeDefaultValueOverrideState::None)
							{
								return EOverrideWidgetStatus::ChangedInside;
							}
						}
						else
						{
							if(const URigVMPin* Pin = Node->FindPin(PinPath))
							{
								if(Pin->GetDefaultValueType() == ERigVMPinDefaultValueType::Override)
								{
									return EOverrideWidgetStatus::ChangedHere;
								}
								if(!Pin->GetSubPins().IsEmpty())
								{
									if(Pin->HasDefaultValueOverride())
									{
										return EOverrideWidgetStatus::ChangedInside;
									}
								}

								// if this is an old asset and the default value type has not yet
								// been determined - we need to fall back on comparing to the default.
								if(!Pin->HasOriginalDefaultValue())
								{
									return EOverrideWidgetStatus::ChangedHere;
								}
							}
						}
						return EOverrideWidgetStatus::None;
					}
					return {};
				}
			).Get(EOverrideWidgetStatus::Mixed);
		});

		ObjectFilter->OnAddOverride().BindLambda(
			[this, PropertyPathToPinPath](const FOverrideStatusSubject& InSubject)
			{
				const FString PinPath = PropertyPathToPinPath(InSubject);
				TArray<FString> PinPaths;
				TArray<FName> NodeNames;

				InSubject.ForEach<URigVMDetailsViewWrapperObject>(
					[PinPath, &PinPaths, &NodeNames](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper)
					{
						if(const URigVMNode* Node = Cast<URigVMNode>(InWrapper->GetSubject()))
						{
							if(PinPath.IsEmpty())
							{
								NodeNames.Add(Node->GetFName());
							}
							else
							{
								if(const URigVMPin* Pin = Node->FindPin(PinPath))
								{
									PinPaths.AddUnique(Pin->GetPinPath());
								}
							}
						}
					}
				);

				if(!PinPaths.IsEmpty())
				{
					if(GetFocusedController()->AddOverrideToPins(PinPaths))
					{
						return FReply::Handled();
					}
				}
				else if(!NodeNames.IsEmpty())
				{
					if(GetFocusedController()->AddOverrideToAllPinsOnNodes(NodeNames))
					{
						return FReply::Handled();
					}
				}

				return FReply::Unhandled();;
			});

		ObjectFilter->OnClearOverride().BindLambda(
			[this, PropertyPathToPinPath](const FOverrideStatusSubject& InSubject)
			{
				const FString PinPath = PropertyPathToPinPath(InSubject);
				TArray<FString> PinPaths;
				TArray<FName> NodeNames;

				InSubject.ForEach<URigVMDetailsViewWrapperObject>(
					[PinPath, &PinPaths, &NodeNames](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper)
					{
						if(const URigVMNode* Node = Cast<URigVMNode>(InWrapper->GetSubject()))
						{
							if(PinPath.IsEmpty())
							{
								NodeNames.Add(Node->GetFName());
							}
							else
							{
								if(const URigVMPin* Pin = Node->FindPin(PinPath))
								{
									PinPaths.AddUnique(Pin->GetPinPath());
								}
							}
						}
					}
				);
				
				if(!PinPaths.IsEmpty())
				{
					if(GetFocusedController()->ClearOverrideOnPins(PinPaths))
					{
						return FReply::Handled();
					}
				}
				else if(!NodeNames.IsEmpty())
				{
					if(GetFocusedController()->ClearOverrideOnAllPinsOnNodes(NodeNames))
					{
						return FReply::Handled();
					}
				}

				return FReply::Unhandled();
			});

		ObjectFilter->OnResetToDefault().BindLambda(
			[this, PropertyPathToPinPath](const FOverrideStatusSubject& InSubject)
			{
				const FString PinPath = PropertyPathToPinPath(InSubject);
				TArray<FString> PinPaths;
				TArray<FName> NodeNames;

				InSubject.ForEach<URigVMDetailsViewWrapperObject>(
					[PinPath, &PinPaths, &NodeNames](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper)
					{
						if(const URigVMNode* Node = Cast<URigVMNode>(InWrapper->GetSubject()))
						{
							if(PinPath.IsEmpty())
							{
								NodeNames.Add(Node->GetFName());
							}
							else
							{
								if(const URigVMPin* Pin = Node->FindPin(PinPath))
								{
									PinPaths.AddUnique(Pin->GetPinPath());
								}
							}
						}
					}
				);

				if(!PinPaths.IsEmpty())
				{					
					FRigVMDefaultValueTypeGuard _(GetFocusedController(), ERigVMPinDefaultValueType::KeepValueType, true);
					if(GetFocusedController()->ResetDefaultValueForPins(PinPaths))
					{
						return FReply::Handled();
					}
				}
				else if(!NodeNames.IsEmpty())
				{
					FRigVMDefaultValueTypeGuard _(GetFocusedController(), ERigVMPinDefaultValueType::KeepValueType, true);
					if(GetFocusedController()->ResetDefaultValueForAllPinsOnNodes(NodeNames))
					{
						return FReply::Handled();
					}
				}
				return FReply::Unhandled();
			});

		ObjectFilter->OnValueDiffersFromDefault().BindLambda(
			[this, PropertyPathToPinPath](const FOverrideStatusSubject& InSubject)
			{
				const FString PinPath = PropertyPathToPinPath(InSubject);
				return InSubject.Contains<URigVMDetailsViewWrapperObject>([PinPath](const FOverrideStatusObjectHandle<URigVMDetailsViewWrapperObject>& InWrapper)
				{
					if(const URigVMNode* Node = Cast<URigVMNode>(InWrapper->GetSubject()))
					{
						if(PinPath.IsEmpty())
						{
							for(const URigVMPin* Pin : Node->GetPins())
							{
								if(Pin->CanProvideDefaultValue())
								{
									if(!Pin->HasOriginalDefaultValue())
									{
										return true;
									}
								}
							}
						}
						else
						{
							if(const URigVMPin* Pin = Node->FindPin(PinPath))
							{
								return !Pin->HasOriginalDefaultValue();
							}
						}
					}
					return false;
				});
			});

		SetDetailObjectFilter(ObjectFilter);
	}

	auto Task = [&]<typename T, typename O>(T Inspector, O Options)
	{
		Inspector->ShowDetailsForObjects(FilteredObjects, Options);
	};

#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
	if (SharedApp->IsBlueprintEditor())
	{
		if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
		{
			SKismetInspector::FShowDetailsOptions Options;
			Options.bForceRefresh = true;
			Task(Inspector, Options);
		}
	}
#endif
	if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
	{
		SRigVMDetailsInspector::FShowDetailsOptions Options;
		Options.bForceRefresh = true;
		Task(Inspector, Options);
	}
	
	bDetailsPanelRequiresClear = true;
}

void FRigVMEditorBase::SetDetailObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InObjectFilter)
{
#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
	if (SharedApp->IsBlueprintEditor())
	{
		if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
		{
			Inspector->GetPropertyView()->SetObjectFilter(InObjectFilter);
		}
	}
#endif
	if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
	{
		Inspector->GetPropertyView()->SetObjectFilter(InObjectFilter);
	}
}

void FRigVMEditorBase::SetMemoryStorageDetails(const TArray<FRigVMMemoryStorageStruct*>& InStructs)
{
	if (bSuspendDetailsPanelRefresh)
	{
		return;
	}

	if (InStructs.Num() == 1)
	{
		if (FRigVMMemoryStorageStruct* Memory = InStructs[0])
		{
			FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

			FDetailsViewArgs DetailsViewArgs;
			DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
			DetailsViewArgs.bHideSelectionTip = true;

			FStructureDetailsViewArgs StructureViewArgs;

			TSharedRef<IStructureDetailsView> DetailsView = EditModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
			DetailsView->SetStructureProvider(MakeShared<FInstancePropertyBagStructureDataProvider>(*Memory));

			TSharedRef<SDockTab> DockTab = SNew(SDockTab)
				.Label(LOCTEXT("RigVMMemoryDetails", "RigVM Memory Details"))
				.AddMetaData<FMemoryTypeMetaData>(FMemoryTypeMetaData(Memory->GetMemoryType()))
				.TabRole(ETabRole::NomadTab)
				[
					DetailsView->GetWidget().ToSharedRef()
				];

			FName TabId = *FString::Printf(TEXT("RigVMMemoryDetails_%d"), (int32)Memory->GetMemoryType());
			if (TSharedPtr<SDockTab> ActiveTab = GetTabManager()->FindExistingLiveTab(TabId))
			{
				ActiveTab->RequestCloseTab();
			}

			GetTabManager()->InsertNewDocumentTab(
#if WITH_RIGVMLEGACYEDITOR
				GetRigVMInspector() ? FRigVMDetailsInspectorTabSummoner::TabID() : FBlueprintEditorTabs::DetailsID,
#else
				FRigVMDetailsInspectorTabSummoner::TabID(),
#endif
				TabId,
				FTabManager::FLastMajorOrNomadTab(TEXT("RigVMMemoryDetails")),
				DockTab
			);
			return;
		}
	}
}

void FRigVMEditorBase::SetDetailViewForGraph(URigVMGraph* InGraph)
{
	check(InGraph);
	
	if(bSuspendDetailsPanelRefresh)
	{
		return;
	}

	if(bDetailsPanelRequiresClear)
	{
		ClearDetailObject();
	}

	TArray<UObject*> SelectedNodes;
	TArray<FName> SelectedNodeNames = InGraph->GetSelectNodes();
	for(FName SelectedNodeName : SelectedNodeNames)
	{
		if(URigVMNode* Node = InGraph->FindNodeByName(SelectedNodeName))
		{
			SelectedNodes.Add(Node);
		}
	}

	SetDetailObjects(SelectedNodes);
}

void FRigVMEditorBase::SetDetailViewForFocusedGraph()
{
	if(bSuspendDetailsPanelRefresh)
	{
		return;
	}

	URigVMGraph* Model = GetFocusedModel();
	if(Model == nullptr)
	{
		return;
	}

	SetDetailViewForGraph(Model);
}

void FRigVMEditorBase::SetDetailViewForLocalVariable()
{
	FName VariableName;
	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjects();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				VariableName = WrapperObject->GetContent<FRigVMGraphVariableDescription>().Name;
				break;
			}
		}
	}
		
	SelectLocalVariable(GetFocusedGraph(), VariableName);
}

void FRigVMEditorBase::SetDetailViewForVariable()
{
	FName VariableName;
	if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = GetRigVMInspector())
	{
		TArray< TWeakObjectPtr<UObject> > SelectedObjects = RigVMInspector->GetSelectedObjects();
		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			if (SelectedObject.IsValid())
			{
				if(UPropertyWrapper* WrapperObject = Cast<UPropertyWrapper>(SelectedObject.Get()))
				{
					VariableName = WrapperObject->GetProperty()->GetFName();
					break;
				}
			}
		}
	}

	SelectVariable(VariableName);
}

void FRigVMEditorBase::RefreshDetailView()
{
	if(bSuspendDetailsPanelRefresh)
	{
		return;
	}
	if(DetailViewShowsLocalVariable())
	{
		SetDetailViewForLocalVariable();
	}
	else if(DetailViewShowsVariable())
	{
		SetDetailViewForVariable();
	}
	else if(DetailViewShowsAnyRigUnit())
	{
		SetDetailViewForFocusedGraph();
	}
	else
	{
		// detail view is showing other stuff; could be a BP variable for example
		// in this case wrapper objects are not in use, yet still rooted
		// and preventing their outer objects from getting GCed after a Compile()
		// so let's take the chance to manually clear them here.
		ClearDetailsViewWrapperObjects();
	}
}

bool FRigVMEditorBase::DetailViewShowsAnyRigUnit() const
{
	if (DetailViewShowsStruct(FRigVMStruct::StaticStruct()))
	{
		return true;
	}

	const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = GetSelectedObjects();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				const FString Notation = WrapperObject->GetWrappedNodeNotation();
				if(!Notation.IsEmpty())
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

bool FRigVMEditorBase::DetailViewShowsLocalVariable() const
{
	return DetailViewShowsStruct(FRigVMGraphVariableDescription::StaticStruct());
}

bool FRigVMEditorBase::DetailViewShowsVariable() const
{
	if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = GetRigVMInspector())
	{
		TArray< TWeakObjectPtr<UObject> > SelectedObjects = RigVMInspector->GetSelectedObjects();
		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			if (SelectedObject.IsValid())
			{
				if(UPropertyWrapper* WrapperObject = Cast<UPropertyWrapper>(SelectedObject.Get()))
				{
					return true;
				}
			}
		}
	}
	return false; 
}

bool FRigVMEditorBase::DetailViewShowsStruct(UScriptStruct* InStruct) const
{
	TArray< TWeakObjectPtr<UObject> > SelectedObjects = GetSelectedObjects();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.IsValid())
		{
			if(URigVMDetailsViewWrapperObject* WrapperObject = Cast<URigVMDetailsViewWrapperObject>(SelectedObject.Get()))
			{
				if(const UScriptStruct* WrappedStruct = WrapperObject->GetWrappedStruct())
				{
					if (WrappedStruct->IsChildOf(InStruct))
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void FRigVMEditorBase::ClearDetailObject(bool bChangeUISelectionState)
{
	if(bSuspendDetailsPanelRefresh)
	{
		return;
	}

	SetDetailObjectFilter(nullptr);
	ClearDetailsViewWrapperObjects();

	auto Task = [&]<typename T>(T Inspector)
	{
		if(Inspector.IsValid())
		{
			Inspector->GetPropertyView()->SetObjects(TArray<UObject*>(), true); // clear property view synchronously
			Inspector->ShowDetailsForObjects(TArray<UObject*>());
			Inspector->ShowSingleStruct(TSharedPtr<FStructOnScope>());
		}
	};

#if WITH_RIGVMLEGACYEDITOR
	TSharedRef<FAssetEditorToolkit> SharedApp = GetHostingApp().ToSharedRef();
	if (SharedApp->IsBlueprintEditor())
	{
		if (TSharedPtr<SKismetInspector> Inspector = GetKismetInspector())
		{
			Task(Inspector);
		}
	}
#endif
	if (TSharedPtr<SRigVMDetailsInspector> Inspector = GetRigVMInspector())
	{
		Task(Inspector);
	}


	if (bChangeUISelectionState)
	{
#if WITH_RIGVMLEGACYEDITOR
		SetUISelectionState(FBlueprintEditor::SelectionState_Graph);
#else
		SetUISelectionState(FRigVMNewEditor::SelectionState_Graph());
#endif
	}

	bDetailsPanelRequiresClear = false;
}

void FRigVMEditorBase::ClearDetailsViewWrapperObjects()
{
	for(const TStrongObjectPtr<URigVMDetailsViewWrapperObject>& WrapperObjectPtr : WrapperObjects)
	{
		if(WrapperObjectPtr.IsValid())
		{
			// Return to pool instead of destroying
			URigVMDetailsViewWrapperObject::ReturnToPool(WrapperObjectPtr.Get());
		}
	}
	WrapperObjects.Reset();
}

void FRigVMEditorBase::SetHost(URigVMHost* InHost)
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface())
	{
		if (IsValid(RigVMBlueprint->GetEditorHost()) && RigVMBlueprint->GetEditorHost()->GetOuter() == GetOuterForHost())
		{
			RigVMBlueprint->GetEditorHost()->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			RigVMBlueprint->GetEditorHost()->MarkAsGarbage();
		}
		RigVMBlueprint->GetEditorHost() = InHost;
		if(RigVMBlueprint->GetEditorHost() && IsValid(RigVMBlueprint->GetEditorHost()))
		{
			OnPreviewHostUpdated().Broadcast(this);
		}
	}
}

URigVMGraph* FRigVMEditorBase::GetFocusedModel() const
{
	const FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if (Blueprint == nullptr)
	{
		return nullptr;
	}

	const URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(GetFocusedGraph());
	return Blueprint->GetModel(EdGraph);
}

URigVMController* FRigVMEditorBase::GetFocusedController() const
{
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if (Blueprint == nullptr)
	{
		return nullptr;
	}
	return Blueprint->GetOrCreateController(GetFocusedModel());
}

TSharedPtr<SGraphEditor> FRigVMEditorBase::GetGraphEditor(UEdGraph* InEdGraph) const
{
	TArray< TSharedPtr<SDockTab> > GraphEditorTabs;
	GetDocumentManager()->FindAllTabsForFactory(GetGraphEditorTabFactory(), /*out*/ GraphEditorTabs);

	for (const TSharedPtr<SDockTab>& GraphEditorTab : GraphEditorTabs)
	{
		TSharedRef<SGraphEditor> Editor = StaticCastSharedRef<SGraphEditor>((GraphEditorTab)->GetContent());
		if (Editor->GetCurrentGraph() == InEdGraph)
		{
			return Editor;
		}
	}

	return TSharedPtr<SGraphEditor>();
}

void FRigVMEditorBase::ExtendMenu()
{
	if(MenuExtender.IsValid())
	{
		RemoveMenuExtender(MenuExtender);
		MenuExtender.Reset();
	}

	MenuExtender = MakeShareable(new FExtender);

	AddMenuExtender(MenuExtender);

	// add extensible menu if exists
	FRigVMEditorModule& RigVMEditorModule = FModuleManager::LoadModuleChecked<FRigVMEditorModule>("RigVMEditor");
	AddMenuExtender(RigVMEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingBlueprints()));
}

void FRigVMEditorBase::ExtendToolbar()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	// If the ToolbarExtender is valid, remove it before rebuilding it
	if(ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	FRigVMEditorModule& RigVMEditorModule = FModuleManager::LoadModuleChecked<FRigVMEditorModule>("RigVMEditor");
	AddToolbarExtender(RigVMEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingBlueprints()));

	TArray<FRigVMEditorModule::FRigVMEditorToolbarExtender> ToolbarExtenderDelegates = RigVMEditorModule.GetAllRigVMEditorToolbarExtenders();

	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if(ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), StaticCastSharedRef<FRigVMEditorBase>(SharedRef())));
		}
	}

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::FillToolbar, true)
	);
}

void FRigVMEditorBase::FillToolbar(FToolBarBuilder& ToolbarBuilder, bool bEndSection)
{
	ToolbarBuilder.BeginSection("Toolbar");
	{
		AddCompileWidget(ToolbarBuilder);
		
		AddLockEditingWidget(ToolbarBuilder);
		
		ToolbarBuilder.AddToolBarButton(
			FRigVMEditorCommands::Get().ToggleEventQueue,
			NAME_None, 
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::GetEventQueueLabel)),
			TAttribute<FText>(), 
			TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::GetEventQueueIcon))
		);
		
		FUIAction DefaultAction;
		ToolbarBuilder.AddComboButton(
			DefaultAction,
			FOnGetContent::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::GenerateEventQueueMenuContent),
			LOCTEXT("EventQueue_Label", "Available Events"),
			LOCTEXT("EventQueue_ToolTip", "Pick between different events / modes for testing the Control Rig"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
			true);

		AddRunOnceWidget(ToolbarBuilder);
		AddAutoCompileWidget(ToolbarBuilder);
		AddSelectedDebugObjectWidget(ToolbarBuilder);

		ToolbarBuilder.AddSeparator();

		FUIAction DefaultBulkEditAction;
		ToolbarBuilder.AddComboButton(
			DefaultBulkEditAction,
			FOnGetContent::CreateSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::GenerateBulkEditMenuContent),
			LOCTEXT("BulkEdit_Label", "Bulk Edit"),
			LOCTEXT("BulkEdit_ToolTip", "Perform changes across many nodes / assets"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
			false);

		AddSettingsAndDefaultWidget(ToolbarBuilder);
		AddFindWidget(ToolbarBuilder);
	}

	if(bEndSection)
	{
		ToolbarBuilder.EndSection();
	}
}

void FRigVMEditorBase::AddLockEditingWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddToolBarButton(
		FRigVMEditorCommands::Get().LockGraph,
		NAME_None,
		TAttribute<FText>::Create([this]() -> FText
		{
			if (FRigVMEditorAssetInterfacePtr EditorAssetInterface = GetRigVMAssetInterface())
			{
				return EditorAssetInterface->GetEditingLockedLabel();
			}
			return FText();
		}),
		TAttribute<FText>::Create([this]() -> FText
		{
			if (FRigVMEditorAssetInterfacePtr EditorAssetInterface = GetRigVMAssetInterface())
			{
				return EditorAssetInterface->GetEditingLockedTooltip();
			}
			return FText();
		}),
		FSlateIcon(FAppStyle::GetAppStyleSetName(),
		"Icons.Lock"),
		NAME_None,
		FNewMenuDelegate(),
		TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility
		{
			if (FRigVMEditorAssetInterfacePtr EditorAssetInterface = GetRigVMAssetInterface())
			{
				if (EditorAssetInterface->IsEditingLockedVisible())
				{
					return EVisibility::Visible;
				}
			}
			return EVisibility::Collapsed;
		})
	);
}

void FRigVMEditorBase::UpdateStaleWatchedPins()
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint == nullptr)
	{
		return;
	}

	TSet<UEdGraphPin*> AllPins;
	uint16 WatchCount;

	// Find all unique pins being watched
	RigVMBlueprint->ForeachPinWatch(
		[&AllPins, &WatchCount](UEdGraphPin* Pin)
		{
			++WatchCount;
			if (Pin == nullptr)
			{
				return; // ~continue
			}

			UEdGraphNode* OwningNode = Pin->GetOwningNode();
			// during node reconstruction, dead pins get moved to the transient 
			// package (so just in case this blueprint got saved with dead pin watches)
			if (OwningNode == nullptr)
			{
				return; // ~continue
			}

			if (!OwningNode->Pins.Contains(Pin))
			{
				return; // ~continue
			}

			AllPins.Add(Pin);
		}
	);

	// Refresh watched pins with unique pins (throw away null or duplicate watches)
	if (WatchCount != AllPins.Num())
	{
		RigVMBlueprint->SetAssetStatus(RVMA_Dirty);
	}

	RigVMBlueprint->ClearPinWatches();
	
	TArray<URigVMGraph*> Models = RigVMBlueprint->GetAllModels();
	for (URigVMGraph* Model : Models)
	{
		for (URigVMNode* ModelNode : Model->GetNodes())
		{
			TArray<URigVMPin*> ModelPins = ModelNode->GetAllPinsRecursively();
			for (URigVMPin* ModelPin : ModelPins)
			{
				if (ModelPin->RequiresWatch())
				{
					RigVMBlueprint->GetController(Model)->SetPinIsWatched(ModelPin->GetPinPath(), false, false);
				}
			}
		}
	}
	for (UEdGraphPin* Pin : AllPins)
	{
		RigVMBlueprint->AddPinWatch(Pin);
		UEdGraph* EdGraph = Pin->GetOwningNode()->GetGraph();
		RigVMBlueprint->GetController(EdGraph)->SetPinIsWatched(Pin->GetName(), true, false);
	}
}

void FRigVMEditorBase::HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint)
{
	Compile();
}

void FRigVMEditorBase::HandleVariableDroppedFromBlueprint(UObject* InSubject, FProperty* InVariableToDrop, const FVector2D& InDropPosition, const FVector2D& InScreenPosition)
{
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if (Blueprint == nullptr)
	{
		return;
	}

	URigVMController* Controller = GetFocusedController();
	check(Controller);

	FRigVMExternalVariable ExternalVariable = Controller->GetVariableByProperty(InVariableToDrop);
	if (!ExternalVariable.IsValid(true /* allow null ptr */))
	{
		// In some cases the variable property does not match because it comes from the SkeletalClass, not the GeneratedBlueprintClass (in the case of legacy)
		// Let's fall back to looking by name
		ExternalVariable = Controller->GetVariableByName(InVariableToDrop->GetFName());
	}
	
	if (!ExternalVariable.IsValid(true /* allow null ptr */))
	{
		return;
	}

	FMenuBuilder MenuBuilder(true, NULL);
	const FText SectionText = FText::FromString(FString::Printf(TEXT("Variable %s"), *ExternalVariable.GetName().ToString()));

	MenuBuilder.BeginSection("VariableDropped", SectionText);

	MenuBuilder.AddMenuEntry(
		FText::FromString(FString::Printf(TEXT("Get %s"), *ExternalVariable.GetName().ToString())),
		FText::FromString(FString::Printf(TEXT("Adds a getter node for variable %s"), *ExternalVariable.GetName().ToString())),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ExternalVariable, Controller, InDropPosition] {

				Controller->AddVariableNode(ExternalVariable.GetName(), ExternalVariable.GetExtendedCPPType().ToString(), const_cast<UObject*>(ExternalVariable.GetCPPTypeObject()), true, FString(), InDropPosition, FString(), true, true);

			}),
			FCanExecuteAction()
		)
	);

	MenuBuilder.AddMenuEntry(
		FText::FromString(FString::Printf(TEXT("Set %s"), *ExternalVariable.GetName().ToString())),
		FText::FromString(FString::Printf(TEXT("Adds a setter node for variable %s"), *ExternalVariable.GetName().ToString())),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([ExternalVariable, Controller, InDropPosition] {

				Controller->AddVariableNode(ExternalVariable.GetName(), ExternalVariable.GetExtendedCPPType().ToString(), const_cast<UObject*>(ExternalVariable.GetCPPTypeObject()), false, FString(), InDropPosition, FString(), true, true);

			}),
			FCanExecuteAction()
		)
	);

	MenuBuilder.EndSection();

	TSharedRef<SWidget> GraphEditorPanel = GetFocusedGraphEditor().Pin().ToSharedRef();

	// Show dialog to choose getter vs setter
	FSlateApplication::Get().PushMenu(
		GraphEditorPanel,
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		InScreenPosition,
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}

void FRigVMEditorBase::OnGraphNodeClicked(URigVMEdGraphNode* InNode, const FGeometry& InNodeGeometry, const FPointerEvent& InMouseEvent)
{
	if (InNode)
	{
		if (InNode->IsSelectedInEditor())
		{
			SetDetailViewForGraph(InNode->GetModel());
		}

		if (InMouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton))
		{
			if (URigVMController* Controller = GetFocusedController())
			{
				const bool ClearSelection = !InMouseEvent.GetModifierKeys().IsShiftDown();
				const bool SelectSourceNodes = !InMouseEvent.GetModifierKeys().IsControlDown();
				const bool SelectIsland = InMouseEvent.GetModifierKeys().IsAltDown();
				if (SelectIsland)
				{
					Controller->SelectNodeIslands({InNode->GetModelNodeName()}, ClearSelection);
				}
				else
				{
					Controller->SelectLinkedNodes({InNode->GetModelNodeName()}, SelectSourceNodes, ClearSelection, true);
				}
			}
		}
	}
}

void FRigVMEditorBase::OnNodeDoubleClicked(FRigVMEditorAssetInterfacePtr InBlueprint, URigVMNode* InNode)
{
	ensure(GetRigVMAssetInterface() == InBlueprint);

	if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InNode))
	{
		if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(LibraryNode))
		{
			if (const URigVMLibraryNode* ReferencedNode = FunctionReferenceNode->LoadReferencedNode())
			{
				HandleJumpToHyperlink(ReferencedNode);
				return;
			}
		}
		if(URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph())
		{
			if (UEdGraph* EdGraph = InBlueprint->GetEdGraph(ContainedGraph))
			{
				OpenGraphAndBringToFront(EdGraph, true);
			}
			else
			{
				UE_LOGF(LogRigVMEditor, Warning, "Could not open graph (%ls)", *LibraryNode->GetFunctionIdentifier().GetLibraryNodePath());
			}
		}
	}
}

void FRigVMEditorBase::OnGraphImported(UEdGraph* InEdGraph)
{
	check(InEdGraph);

	OpenDocument(InEdGraph, FDocumentTracker::OpenNewDocument);
	RenameNewlyAddedAction(InEdGraph->GetFName());
}

bool FRigVMEditorBase::OnActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const
{
	if (InAction->GetMenuDescription().ToString() == InName.ToString())
	{
		return true;
	}
	return false;
}

void FRigVMEditorBase::FrameSelection()
{
	if (SGraphEditor* GraphEd = GetFocusedGraphEditor().Pin().Get())
	{
		if(URigVMGraph* Model = GetFocusedModel())
		{
			const bool bFrameAll = Model->GetSelectNodes().Num() == 0;
			GraphEd->ZoomToFit(!bFrameAll);
		}
	}
}

void FRigVMEditorBase::SwapFunctionWithinAsset()
{
	const FAssetData Asset = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(GetRigVMAssetInterface()->GetObject()->GetPathName(), true);
	SwapFunctionForAssets({Asset}, true);
}

void FRigVMEditorBase::SwapFunctionAcrossProject()
{
	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	TArray<FAssetData> AllAssets;
	// TODO: sara-sc we might have a problem when trying to swap functions between URigVMBlueprint and URigVMAsset classes
	AssetRegistry.GetAssetsByClass(GetRigVMAssetInterface()->GetObject()->GetClass()->GetClassPathName(), AllAssets, true);
	SwapFunctionForAssets(AllAssets, false);
}

void FRigVMEditorBase::SwapFunctionForAssets(const TArray<FAssetData>& InAssets, bool bSetupUndo)
{
	SRigVMSwapFunctionsWidget::FArguments WidgetArgs;
	WidgetArgs
		.Assets(InAssets)
		.EnableUndo(bSetupUndo)
		.CloseOnSuccess(true);

	const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapFunctionsWidget>> SwapFunctionsDialog =
		SNew(SRigVMBulkEditDialog<SRigVMSwapFunctionsWidget>)
		.WindowSize(FVector2D(800.0f, 640.0f))
		.WidgetArgs(WidgetArgs);

	SwapFunctionsDialog->ShowNormal();
}

void FRigVMEditorBase::SwapAssetReferences()
{
	const FAssetData Asset = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(GetRigVMAssetInterface()->GetObject()->GetPathName(), true);
	
	SRigVMSwapAssetReferencesWidget::FArguments WidgetArgs;
	WidgetArgs
		.Source(Asset)
		.EnableUndo(false)
		.CloseOnSuccess(true);

	const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>> SwapFunctionsDialog =
		SNew(SRigVMBulkEditDialog<SRigVMSwapAssetReferencesWidget>)
		.WindowSize(FVector2D(800.0f, 640.0f))
		.WidgetArgs(WidgetArgs);

	SwapFunctionsDialog->ShowNormal();
}

void FRigVMEditorBase::ToggleProfiling()
{
	TScriptInterface<IRigVMEditorAssetInterface> Asset = GetRigVMAssetInterface();
	if (Asset && Asset->GetObject())
	{
		const FScopedTransaction Transaction(LOCTEXT("ToggleProfiler", "Toggle Profiler"));
		Asset.GetObject()->Modify();
		Asset->SetProfilingEnabled(!Asset->GetVMRuntimeSettings().bEnableProfiling);
	}
}

void FRigVMEditorBase::OnOpenSelectedNodesInNewTab()
{
	const FRigVMEditorAssetInterfacePtr RigVMAsset = GetRigVMAssetInterface();
	if (RigVMAsset == nullptr)
	{
		return;
	}

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(*NodeIt))
		{
			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(EdGraphNode->GetModelNode()))
			{
				UEdGraph* FoundEdGraph = nullptr;
				if (const URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph())
				{
					FoundEdGraph = RigVMAsset->GetEdGraph(ContainedGraph);
				}
				else if (const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(LibraryNode))
				{
					if (const URigVMLibraryNode* ReferencedNode = FunctionReferenceNode->LoadReferencedNode())
					{
						const URigVMFunctionEntryNode* EntryNode = ReferencedNode->GetEntryNode();
						if (EntryNode != nullptr && EntryNode->GetGraph() != nullptr)
						{
							FoundEdGraph = RigVMAsset->GetEdGraph(EntryNode->GetGraph());
						}
					}
				}

				if (FoundEdGraph)
				{
					OpenDocument(FoundEdGraph, FDocumentTracker::ForceOpenNewDocument);
				}
			}
		}
	}
}

bool FRigVMEditorBase::CanOpenSelectedNodesInNewTab() const
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (const URigVMEdGraphNode* EDGraphNode = Cast<URigVMEdGraphNode>(*NodeIt))
		{
			return Cast<URigVMLibraryNode>(EDGraphNode->GetModelNode()) != nullptr;
		}
	}

	return false;
}


void FRigVMEditorBase::OnGraphNodeDropToPerform(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	if (InDragDropOp->IsOfType<FRigVMGraphExplorerDragDropOp>())
	{
		TSharedPtr<FRigVMGraphExplorerDragDropOp> ExplorerOp = StaticCastSharedPtr<FRigVMGraphExplorerDragDropOp>(InDragDropOp);

		if (GetFocusedGraphEditor().IsValid())
		{
			const URigVMEdGraphSchema* Schema = CastChecked<URigVMEdGraphSchema>(GetRigVMAssetInterface()->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass()->GetDefaultObject());
			if (ExplorerOp->GetElement().Type == ERigVMExplorerElementType::Function)
			{
				if (FRigVMEditorAssetInterfacePtr TargetRigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph))
				{
					if (URigVMFunctionLibrary* Library = ExplorerOp->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
					{
						if (URigVMLibraryNode* LibraryNode = Library->FindFunction(*ExplorerOp->GetElement().Name))
						{
							Schema->RequestFunctionDropOnPanel(InGraph, LibraryNode->GetFunctionIdentifier(), FDeprecateSlateVector2D(InNodePosition), FDeprecateSlateVector2D(InScreenPosition));
						}
					}
				}
			}
			else if (ExplorerOp->GetElement().Type == ERigVMExplorerElementType::Variable)
			{
				if (FRigVMEditorAssetInterfacePtr TargetRigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph))
				{
					const FProperty* Property = TargetRigBlueprint->FindGeneratedPropertyByName(*ExplorerOp->GetElement().Name);
					Schema->RequestVariableDropOnPanel(InGraph, const_cast<FProperty*>(Property), InNodePosition, InScreenPosition);
				}
			}
			else if (ExplorerOp->GetElement().Type == ERigVMExplorerElementType::LocalVariable)
			{
				if (FRigVMEditorAssetInterfacePtr TargetRigBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(InGraph))
				{
					if (URigVMGraph* Graph = TargetRigBlueprint->GetFocusedModel())
					{
						for (FRigVMGraphVariableDescription LocalVariable : Graph->GetLocalVariables())
						{
							if (LocalVariable.Name == ExplorerOp->GetElement().Name)
							{
								if (URigVMController* Controller = TargetRigBlueprint->GetRigVMClient()->GetController(Graph))
								{
									FMenuBuilder MenuBuilder(true, nullptr);
									const FText VariableNameText = FText::FromName( LocalVariable.Name );

									MenuBuilder.BeginSection("BPVariableDroppedOn", VariableNameText );

									MenuBuilder.AddMenuEntry(
										FText::Format( LOCTEXT("CreateGetVariable", "Get {0}"), VariableNameText ),
										FText::Format( LOCTEXT("CreateVariableGetterToolTip", "Create Getter for variable '{0}'\n(Ctrl-drag to automatically create a getter)"), VariableNameText ),
										FSlateIcon(),
										FUIAction(
										FExecuteAction::CreateLambda([Controller, LocalVariable, InNodePosition]()
										{
											Controller->AddVariableNode(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, true, LocalVariable.DefaultValue, FDeprecateSlateVector2D(InNodePosition), FString(), true, true);
										}),
										FCanExecuteAction()));

									MenuBuilder.AddMenuEntry(
										FText::Format( LOCTEXT("CreateSetVariable", "Set {0}"), VariableNameText ),
										FText::Format( LOCTEXT("CreateVariableSetterToolTip", "Create Setter for variable '{0}'\n(Alt-drag to automatically create a setter)"), VariableNameText ),
										FSlateIcon(),
										FUIAction(
										FExecuteAction::CreateLambda([Controller, LocalVariable, InNodePosition]()
										{
											Controller->AddVariableNode(LocalVariable.Name, LocalVariable.CPPType, LocalVariable.CPPTypeObject, false, LocalVariable.DefaultValue, FDeprecateSlateVector2D(InNodePosition), FString(), true, true);
										}),
										FCanExecuteAction()));

									TSharedRef< SWidget > PanelWidget = GetGraphEditor(InGraph).ToSharedRef();
									// Show dialog to choose getter vs setter
									FSlateApplication::Get().PushMenu(
										PanelWidget,
										FWidgetPath(),
										MenuBuilder.MakeWidget(),
										InScreenPosition,
										FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu)
										);

									MenuBuilder.EndSection();
								}
							}
						}
					}
				}
			}
		}
	}
}

FSlateBrush const* FRigVMEditorBase::GetVarIconAndColorFromProperty(const FProperty* Property, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	SecondaryBrushOut = nullptr;
	if (Property != nullptr)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType PinType;
		if (K2Schema->ConvertPropertyToPinType(Property, PinType)) // use schema to get the color
		{
			return GetVarIconAndColorFromPinType(PinType, IconColorOut, SecondaryBrushOut, SecondaryColorOut);
		}
	}
	return FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon"));
}

FSlateBrush const* FRigVMEditorBase::GetVarIconAndColorFromPinType(const FEdGraphPinType& PinType, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	IconColorOut = K2Schema->GetPinTypeColor(PinType);
	SecondaryBrushOut = FBlueprintEditorUtils::GetSecondaryIconFromPin(PinType);
	SecondaryColorOut = K2Schema->GetSecondaryPinTypeColor(PinType);
	return FBlueprintEditorUtils::GetIconFromPin(PinType);
}

void FRigVMEditorBase::UpdateGraphCompilerErrors()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	URigVMHost* RigVMHost = GetRigVMHost();
	if (Blueprint && RigVMHost && RigVMHost->GetVM())
	{
		if (RigVMLog.Entries.Num() == 0 && !bAnyErrorsLeft)
		{
			return;
		}

		URigVM* VM = RigVMHost->GetVM();
		const FRigVMByteCode& ByteCode = VM->GetByteCode();

		TArray<UEdGraph*> EdGraphs;
		Blueprint->GetAllEdGraphs(EdGraphs);

		for (UEdGraph* Graph : EdGraphs)
		{
			URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(Graph);
			if (RigVMEdGraph == nullptr)
			{
				continue;
			}

			// reset all nodes and store them in the map
			bool bFoundWarning = false;
			bool bFoundError = false;
			
			for (UEdGraphNode* GraphNode : Graph->Nodes)
			{
				if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
				{
					bFoundError = bFoundError || RigVMEdGraphNode->ErrorType <= (int32)EMessageSeverity::Error;
					bFoundWarning = bFoundWarning || RigVMEdGraphNode->ErrorType <= (int32)EMessageSeverity::Warning;

					if(RigVMEdGraphNode->ErrorType <= (int32)EMessageSeverity::Warning)
					{
						if(!VM->WasInstructionVisitedDuringLastRun(RigVMHost->GetRigVMExtendedExecuteContext(), RigVMEdGraphNode->GetInstructionIndex(true)) &&
							!VM->WasInstructionVisitedDuringLastRun(RigVMHost->GetRigVMExtendedExecuteContext(), RigVMEdGraphNode->GetInstructionIndex(false)))
						{
							continue;
						}
					}
				}

				GraphNode->ErrorType = int32(EMessageSeverity::Info) + 1;
			}

			// update the nodes' error messages
			for (const FRigVMLog::FLogEntry& Entry : RigVMLog.Entries)
			{
				URigVMNode* ModelNode = Cast<URigVMNode>(ByteCode.GetSubjectForInstruction(Entry.InstructionIndex));
				if (ModelNode == nullptr)
				{
					continue;
				}

				UEdGraphNode* GraphNode = RigVMEdGraph->FindNodeForModelNodeName(ModelNode->GetFName());
				if (GraphNode == nullptr)
				{
					continue;
				}

				if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
				{
					// The node in this graph may have the same local node path,
					// but may be backed by another model node.
					if(RigVMEdGraphNode->GetModelNode() != ModelNode)
					{
						continue;
					}

					RigVMEdGraphNode->AddErrorInfo(Entry.Severity, Entry.Message);
				}

				bFoundError = bFoundError || Entry.Severity <= EMessageSeverity::Error;
				bFoundWarning = bFoundWarning || Entry.Severity <= EMessageSeverity::Warning;
			}

			bAnyErrorsLeft = false;
			for (UEdGraphNode* GraphNode : Graph->Nodes)
			{
				GraphNode->bHasCompilerMessage = GraphNode->ErrorType <= int32(EMessageSeverity::Info);
				bAnyErrorsLeft = bAnyErrorsLeft || GraphNode->bHasCompilerMessage; 
			}

			if (bFoundError)
			{
				Blueprint->SetAssetStatus(RVMA_Error);
				(void)Blueprint->GetObject()->MarkPackageDirty();
			}

			RigVMLog.RemoveRedundantEntries();
		}
	}

}

bool FRigVMEditorBase::IsPIERunning()
{
	return GEditor && (GEditor->PlayWorld != nullptr);
}

TArray<FName> FRigVMEditorBase::GetDefaultEventQueue() const
{
	return TArray<FName>();
}

TArray<FName> FRigVMEditorBase::GetEventQueue() const
{
	if (const URigVMHost* CurrentHost = GetRigVMHost())
	{
		return CurrentHost->GetEventQueue();
	}

	return GetDefaultEventQueue();
}

void FRigVMEditorBase::SetEventQueue(TArray<FName> InEventQueue)
{
	return SetEventQueue(InEventQueue, false);
}

void FRigVMEditorBase::SetEventQueue(TArray<FName> InEventQueue, bool bCompile)
{
	if (GetEventQueue() == InEventQueue)
	{
		return;
	}

	LastEventQueue = GetEventQueue();

	if (URigVMHost* CurrentHost = GetRigVMHost())
	{
		if (InEventQueue.Num() > 0)
		{
			CurrentHost->SetEventQueue(InEventQueue);
		}
	}
}

FSlateIcon FRigVMEditorBase::GetEventQueueIcon(const TArray<FName>& InEventQueue) const
{
	return FSlateIcon();
}

FSlateIcon FRigVMEditorBase::GetEventQueueIcon() const
{
	return GetEventQueueIcon(GetEventQueue());
}

void FRigVMEditorBase::GetDebugObjects(TArray<FRigVMCustomDebugObject>& DebugList) const
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = GetRigVMAssetInterface();
	if (RigVMBlueprint == nullptr)
	{
		return;
	}

	if (URigVMHost* CurrentHost = GetRigVMHost())
	{
		if(IsValid(CurrentHost))
		{
			FRigVMCustomDebugObject DebugObject;
			DebugObject.Object = CurrentHost;
			DebugObject.NameOverride = GetCustomDebugObjectLabel(CurrentHost);
			DebugList.Add(DebugObject);
		}
	}

	struct Local
	{
		static bool IsPendingKillOrUnreachableRecursive(UObject* InObject)
		{
			if (InObject != nullptr)
			{
				if (!IsValidChecked(InObject) || InObject->IsUnreachable())
				{
					return true;
				}
				return IsPendingKillOrUnreachableRecursive(InObject->GetOuter());
			}
			return false;
		}

		static bool OuterNameContainsRecursive(UObject* InObject, const FString& InStringToSearch)
		{
			if (InObject == nullptr)
			{
				return false;
			}

			UObject* InObjectOuter = InObject->GetOuter();
			if (InObjectOuter == nullptr)
			{
				return false;
			}

			if (InObjectOuter->GetName().Contains(InStringToSearch))
			{
				return true;
			}

			return OuterNameContainsRecursive(InObjectOuter, InStringToSearch);
		}
	};

	TArray<UObject*> ArchetypeInstances = RigVMBlueprint->GetArchetypeInstances(false, false);

	// run in two passes - find the PIE related objects first
	for (int32 Pass = 0; Pass < 2 ; Pass++)
	{
		for (UObject* Instance : ArchetypeInstances)
		{
			URigVMHost* InstancedHost = Cast<URigVMHost>(Instance);
			if (InstancedHost && IsValid(InstancedHost) && InstancedHost != GetRigVMHost())
			{
				if (InstancedHost->GetOuter() == nullptr)
				{
					continue;
				}

				UWorld* World = InstancedHost->GetWorld();
				if (World == nullptr)
				{
					continue;
				}

				// during pass 0 only do PIE instances,
				// and in pass 1 only do non PIE instances
				if((Pass == 1) == (World->IsPlayInEditor()))
				{
					continue;
				}

				// ensure to only allow preview actors in preview worlds
				if (World->IsPreviewWorld())
				{
					if (!Local::OuterNameContainsRecursive(InstancedHost, TEXT("Preview")))
					{
						continue;
					}
				}

				if (Local::IsPendingKillOrUnreachableRecursive(InstancedHost))
				{
					continue;
				}

				FRigVMCustomDebugObject DebugObject;
				DebugObject.Object = InstancedHost;
				DebugObject.NameOverride = GetCustomDebugObjectLabel(InstancedHost);
				DebugList.Add(DebugObject);
			}
		}
	}
}

void FRigVMEditorBase::HandleSetObjectBeingDebugged(UObject* InObject)
{
	if(URigVMHost* PreviouslyDebuggedHost = Cast<URigVMHost>(GetRigVMAssetInterface()->GetObjectBeingDebugged()))
	{
		if(!URigVMHost::IsGarbageOrDestroyed(PreviouslyDebuggedHost))
		{
			PreviouslyDebuggedHost->OnExecuted_AnyThread().RemoveAll(this);
			PreviouslyDebuggedHost->GetDebugInfo().ExecutionHalted().RemoveAll(this);
			PreviouslyDebuggedHost->SetIsInDebugMode(false);
		}
	}
	
	URigVMHost* DebuggedHost = Cast<URigVMHost>(InObject);

	if (DebuggedHost == nullptr)
	{
		// fall back to our default control rig (which still can be nullptr)
		if (GetRigVMAssetInterface() != nullptr && !bIsSettingObjectBeingDebugged)
		{
			TGuardValue<bool> GuardSettingObjectBeingDebugged(bIsSettingObjectBeingDebugged, true);
			GetRigVMAssetInterface()->SetObjectBeingDebugged(GetRigVMHost());
			return;
		}
	}

	if (FRigVMEditorAssetInterfacePtr RigBlueprint = GetRigVMAssetInterface())
	{
		if (URigVM* VM = RigBlueprint->GetVM(true))
		{
			if (VM->GetInstructions().Num() <= 1 /* only exit */)
			{
				RigBlueprint->RecompileVM();
				RigBlueprint->RequestRigVMInit();
			}
		}
	}

	if(DebuggedHost)
	{
		DebuggedHost->SetLog(&RigVMLog);
		DebuggedHost->OnExecuted_AnyThread().AddSP(StaticCastSharedRef<FRigVMEditorBase>(SharedRef()), &FRigVMEditorBase::HandleVMExecutedEvent);
	}

	RefreshDetailView();
	LastDebuggedHost = GetCustomDebugObjectLabel(DebuggedHost);
}

FString FRigVMEditorBase::GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const
{
	if (ObjectBeingDebugged == nullptr)
	{
		return FString();
	}

	if (ObjectBeingDebugged == GetRigVMHost())
	{
		static const FString EditorPreviewStr = TEXT("Editor Preview");
		return EditorPreviewStr;
	}

	if (const AActor* ParentActor = ObjectBeingDebugged->GetTypedOuter<AActor>())
	{
		if(const UWorld* World = ParentActor->GetWorld())
		{
			FString WorldLabel = GetDebugStringForWorld(World);
			if(World->IsPlayInEditor())
			{
				static const FString PIEPrefix = TEXT("PIE");
				WorldLabel = PIEPrefix;
			}
			return FString::Printf(TEXT("%s: %s in %s"), *WorldLabel, *GetRigVMAssetInterface()->GetObject()->GetName(), *ParentActor->GetActorLabel());
		}
	}
	

	return GetRigVMAssetInterface()->GetObject()->GetName();
}

void FRigVMEditorBase::OnPIEStopped(bool bSimulation)
{
	if(FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
	{
		Blueprint->SetObjectBeingDebugged(GetRigVMHost());
	}
}

void FRigVMEditorBase::ToggleFadeOutUnrelateNodes()
{
	GetMutableDefault<URigVMEditorSettings>()->bFadeOutUnrelatedNodes = !IsToggleFadeOutUnrelatedNodesChecked();
}

bool FRigVMEditorBase::IsToggleFadeOutUnrelatedNodesChecked() const
{
	return GetDefault<URigVMEditorSettings>()->bFadeOutUnrelatedNodes;
}

void FRigVMEditorBase::AddRunOnceWidget(FToolBarBuilder& ToolbarBuilder)
{
	TWeakPtr<IRigVMEditor> WeakThis = SharedRef().ToWeakPtr();

	ToolbarBuilder.AddToolBarButton(
		FRigVMEditorCommands::Get().RunGraphOnce,
		NAME_None,
		FText(),
		TAttribute<FText>(),
		TAttribute<FSlateIcon>::CreateLambda(
			[WeakThis]() -> FSlateIcon
			{
				static const FSlateIcon RunningContinuously(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlaySimulate");
				static const FSlateIcon RunningOnce(FAppStyle::GetAppStyleSetName(), "PlayWorld.SingleFrameAdvance.Small");
				if (WeakThis.IsValid())
				{
					if (TSharedPtr<IRigVMEditor> StrongThis = WeakThis.Pin())
					{
						const FRigVMEditorBase* Editor = static_cast<const FRigVMEditorBase*>(StrongThis.Get());
						if (Editor->IsRunGraphOnceOn())
						{
							return RunningOnce;
						}
					}
				}
				return RunningContinuously;
			}
		),
		NAME_None,
		FNewMenuDelegate(),
		TAttribute<EVisibility>(),
		FText()
	);
}

void FRigVMEditorBase::OnGraphEditorTick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime,
                                         TSharedRef<SGraphEditor> InGraphEditor, URigVMEdGraph* InGraph)
{
	// flashlight to lighten up nodes in proximity of the mouse
	const FGeometry PaintGeometry = InGraphEditor->GetPaintSpaceGeometry();
	FVector2D MousePosition = FSlateApplication::Get().GetCursorPos();
	if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(InGraphEditor))
	{
		MousePosition -= FVector2D(Window->GetPositionInScreen());
	}

	const bool bUseFlashLight = GetDefault<URigVMEditorSettings>()->bUseFlashLight;
	const float FlashLightRadius = 5.f * 64.f;

	if(bUseFlashLight && PaintGeometry.IsUnderLocation(MousePosition) && FlashLightRadius > SMALL_NUMBER)
	{
		FVector2f GraphEditorLocation = FVector2f::ZeroVector;
		float GraphEditorZoomAmount = 0;
		InGraphEditor->GetViewLocation(GraphEditorLocation, GraphEditorZoomAmount);

		const FVector2f WidgetPosition = PaintGeometry.AbsoluteToLocal(MousePosition) / GraphEditorZoomAmount + GraphEditorLocation;
		for (const TObjectPtr<UEdGraphNode>& Node : InGraph->Nodes)
		{
			URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node);
			if (RigVMEdGraphNode == nullptr)
			{
				continue;
			}

			RigVMEdGraphNode->ResetFadedOutState();

			const float PreviousFadedOutState = RigVMEdGraphNode->GetFadedOutState();
			if (PreviousFadedOutState < 1.f - SMALL_NUMBER)
			{
				FSlateRect Bounds;
				if (InGraphEditor->GetBoundsForNode(Node, Bounds, 0.f))
				{
					const FVector2f Center = Bounds.GetCenter();
					const float Radius = FlashLightRadius;
					const float Ratio = FVector2f::DistSquared(Center, WidgetPosition) / (Radius * Radius);
					if (Ratio < 1.f)
					{
						const float HalfRatio = 1.f - FMath::Clamp(Ratio - 0.5f, 0.f, .5f) / 0.5f;
						RigVMEdGraphNode->OverrideFadeOutState(FMath::Lerp(PreviousFadedOutState, 1.f, HalfRatio));
					}
				}
			}
		}
	}
	else
	{
		for (const TObjectPtr<UEdGraphNode>& Node : InGraph->Nodes)
		{
			URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(Node);
			if (RigVMEdGraphNode == nullptr)
			{
				continue;
			}
			RigVMEdGraphNode->ResetFadedOutState();
		}
	}
}

void FRigVMEditorBase::SetGraphExplorerWidget(TSharedPtr<SRigVMEditorGraphExplorer> InWidget)
{
	GraphExplorerWidget = InWidget;
	GetRigVMAssetInterface()->OnModified().AddSP(GraphExplorerWidget.ToSharedRef(), &SRigVMEditorGraphExplorer::HandleGraphModifiedEvent);
}

bool FRigVMEditorBase::IsEditingSingleBlueprint() const
{
	return GetRigVMAssetInterface() != nullptr;
}

void FRigVMEditorBase::OnRequestFindNodeReferences(const UE::RigVMEditor::FRigVMEditorFindNodeReferencesParams Params)
{
	if (!Params.WeakRigVMAsset.IsValid() ||
		Params.WeakRigVMAsset.Get() != GetRigVMAssetInterface().GetInterface())
	{
		return;
	}

	TScriptInterface<IRigVMRuntimeAssetInterface> GeneratedClass = Params.WeakRigVMAsset.Get()->GetRuntimeAssetInterface();
	const URigVMNode* Node = Params.WeakEdGraphNode.IsValid() ? Params.WeakEdGraphNode->GetModelNode() : nullptr;
	if (!GeneratedClass ||
		!Node)
	{
		return;
	}

	const FString SearchTerm = [Node, &Params]()
		{
			if (const URigVMVariableNode* VariableNode = Cast<const URigVMVariableNode>(Node))
			{
				return VariableNode->GetVariableName().ToString();
			}
			
			return Node->GetNodeTitle();
		}();

	const bool bFindWithinBlueprint = !Params.bSearchInAllBlueprints;
	SummonSearchUI(bFindWithinBlueprint, SearchTerm);
}

#undef LOCTEXT_NAMESPACE
