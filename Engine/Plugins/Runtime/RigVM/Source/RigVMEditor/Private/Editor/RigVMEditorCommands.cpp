// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorCommands.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "RigVMEditorCommands"

TSharedPtr<const FRigVMEditorCommands> FRigVMEditorCommands::GetSharedPtr()
{
	return GetInstance().Pin();
}

void FRigVMEditorCommands::RegisterCommands()
{
	UI_COMMAND(Compile, "Compile", "Compile the blueprint", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveOnCompile_Never, "Never", "Sets the save-on-compile option to 'Never', meaning that your Blueprints will not be saved when they are compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_SuccessOnly, "On Success Only", "Sets the save-on-compile option to 'Success Only', meaning that your Blueprints will be saved whenever they are successfully compiled", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(SaveOnCompile_Always, "Always", "Sets the save-on-compile option to 'Always', meaning that your Blueprints will be saved whenever they are compiled (even if there were errors)", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(JumpToErrorNode, "Jump to Error Node", "When enabled, then the Blueprint will snap focus to nodes producing an error during compilation", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ToggleFadeUnrelated, "FadeUnrelated", "Fade out unrelated nodes", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EditGlobalOptions, "Settings", "Edit Class Settings", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(EditClassDefaults, "Defaults", "Edit the initial values of your class.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(DeleteItem, "Delete", "Deletes the selected items and removes their nodes from the graph.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
	UI_COMMAND(ExecuteGraph, "Execute", "Execute the VM if On.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AutoCompileGraph, "Auto Compile", "Auto-compile the VM if On.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(LockGraph, "Nativized", "Runs a nativized backend if On.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ToggleEventQueue, "Toggle Event", "Toggle between the current and last running event", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RunGraphOnce, "Run Once", "Only evaluate a single frame if On.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FrameSelection, "Frame Selection", "Frames the selected nodes in the Graph View.", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(SwapFunctionWithinAsset, "Swap Function (Asset)", "Swaps a function for all occurrences within this asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwapFunctionAcrossProject, "Swap Function (Project)", "Swaps a function for all occurrences in the project.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SwapAssetReferences, "Swap Asset References", "Swaps an asset reference for another asset.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleProfiling, "Toggle Profiling", "Enables or disables heat map profiling for the graph.", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::P));
	UI_COMMAND(TogglePreviewHere, "Toggle Preview Here", "Run the graph up to this node", EUserInterfaceActionType::Button, FInputChord(EKeys::F9));
	UI_COMMAND(PreviewHereStepForward, "Step Forwards", "Run the graph up to the next node.", EUserInterfaceActionType::Button, FInputChord(EKeys::F10));

	UI_COMMAND(FindReferences, "Find References", "Find references of this item", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
	UI_COMMAND(FindReferencesByNameLocal, "By Name", "Find references of this item by name", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FindReferencesByNameGlobal, "By Name (All)", "Find references of this item by name in all blueprints", EUserInterfaceActionType::Button, FInputChord());
}

void FRigVMEditorCommands::BuildFindReferencesMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("FindReferences");
	{
		MenuBuilder.AddMenuEntry(FRigVMEditorCommands::Get().FindReferencesByNameLocal);
		MenuBuilder.AddMenuEntry(FRigVMEditorCommands::Get().FindReferencesByNameGlobal);
	}
	MenuBuilder.EndSection();
}

#undef LOCTEXT_NAMESPACE
