// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Editor/RigVMEditorStyle.h"

class FMenuBuilder;

class FRigVMEditorCommands : public TCommands<FRigVMEditorCommands>
{
public:
	FRigVMEditorCommands() : TCommands<FRigVMEditorCommands>
	(
		"RigVMAsset",
		NSLOCTEXT("Contexts", "RigVM", "RigVM Asset"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		FRigVMEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}

	static TSharedPtr<const FRigVMEditorCommands> GetSharedPtr();

	/** Compile the blueprint */
	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Never;
	TSharedPtr<FUICommandInfo> SaveOnCompile_SuccessOnly;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Always;
	TSharedPtr<FUICommandInfo> JumpToErrorNode;

	/** Edit global options */
	TSharedPtr<FUICommandInfo> ToggleFadeUnrelated;
	TSharedPtr<FUICommandInfo> EditGlobalOptions;
	TSharedPtr<FUICommandInfo> EditClassDefaults;
	
	/** Deletes the selected items and removes their nodes from the graph. */
	TSharedPtr< FUICommandInfo > DeleteItem;

	/** Toggle Execute the Graph */
	TSharedPtr< FUICommandInfo > ExecuteGraph;

	/** Toggle Auto Compilation in the Graph */
	TSharedPtr< FUICommandInfo > AutoCompileGraph;

	/** Toggle Locking of the Graph */
	TSharedPtr< FUICommandInfo > LockGraph;

	/** Toggle between this and the last event queue */
	TSharedPtr< FUICommandInfo > ToggleEventQueue;

	/** Run the graph once or continuously */
	TSharedPtr< FUICommandInfo > RunGraphOnce;

	/** Frames the selected nodes */
	TSharedPtr< FUICommandInfo > FrameSelection;

	/** Swap Function (Asset) */
	TSharedPtr< FUICommandInfo > SwapFunctionWithinAsset;

	/** Swap Function (Project) */
	TSharedPtr< FUICommandInfo > SwapFunctionAcrossProject;

	/** Swap Asset References */
	TSharedPtr< FUICommandInfo > SwapAssetReferences;

	/** Toggle heatmap profiling for the graph */
	TSharedPtr<FUICommandInfo> ToggleProfiling;

	/** Toggle the preview here state of the selected node */
	TSharedPtr< FUICommandInfo > TogglePreviewHere;

	/** Step to next node during preview */
	TSharedPtr< FUICommandInfo > PreviewHereStepForward;

	/** Find references for non-functions and variables */
	TSharedPtr< FUICommandInfo > FindReferences;

	/** Find local references for functions and variables by name */
	TSharedPtr< FUICommandInfo > FindReferencesByNameLocal;

	/** Find member references for functions and variables by name */
	TSharedPtr< FUICommandInfo > FindReferencesByNameGlobal;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

	/** Build "Find References" submenu when a context allows for it */
	static void BuildFindReferencesMenu(FMenuBuilder& MenuBuilder);
};
