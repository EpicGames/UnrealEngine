// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMEditorGraphExplorerTabSummoner.h"
#include "Widgets/SRigVMEditorGraphExplorer.h"
#include "Editor/RigVMNewEditor.h"

#define LOCTEXT_NAMESPACE "RigVMEditorGraphExplorerTabSummoner"

FRigVMEditorGraphExplorerTabSummoner::FRigVMEditorGraphExplorerTabSummoner(const TSharedRef<IRigVMEditor>& InRigVMEditor)
	: FWorkflowTabFactory(TabID(), InRigVMEditor.Get().GetHostingApp())
	, RigVMEditor(InRigVMEditor)
{
	TabLabel = LOCTEXT("RigVMEditorGraphExplorerTabLabel", "Graph Explorer");
	TabIcon = FSlateIcon(TEXT("RigVMEditorStyle"), "EditorGraphExplorer.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigVMEditorGraphExplorer_ViewMenu_Desc", "Graph Explorer");
	ViewMenuTooltip = LOCTEXT("RigVMEditorGraphExplorer_ViewMenu_ToolTip", "Show the RigVM Editor Graph Explorer tab");
}

TSharedRef<SWidget> FRigVMEditorGraphExplorerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	if (TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin())
	{
		TSharedRef<SRigVMEditorGraphExplorer> Explorer = SNew(SRigVMEditorGraphExplorer, Editor);
		Editor->SetGraphExplorerWidget(Explorer);
		return Explorer;
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE 
