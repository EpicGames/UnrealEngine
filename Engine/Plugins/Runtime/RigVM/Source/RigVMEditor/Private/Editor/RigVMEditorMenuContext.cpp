// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMEditorMenuContext.h"

#include "RigVMEditorAsset.h"
#include "RigVMBlueprintLegacy.h"
#include "Editor/RigVMNewEditor.h"
#include "Framework/Application/SlateApplication.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEditorMenuContext)

void URigVMEditorMenuContext::Init(TWeakPtr<IRigVMEditor> InRigVMEditor, const FRigVMEditorGraphMenuContext& InGraphMenuContext)
{
	WeakRigVMEditor = InRigVMEditor;
	GraphMenuContext = InGraphMenuContext;
}

URigVMBlueprint* URigVMEditorMenuContext::GetRigVMBlueprint() const
{
	FRigVMEditorAssetInterfacePtr Interface = GetRigVMAssetInterface();
	return Cast<URigVMBlueprint>(Interface->GetObject());
}

FRigVMEditorAssetInterfacePtr URigVMEditorMenuContext::GetRigVMAssetInterface() const
{
	if (const TSharedPtr<IRigVMEditor> Editor = WeakRigVMEditor.Pin())
	{
		return Editor->GetRigVMAssetInterface();
	}
	
	return nullptr;
}

URigVMHost* URigVMEditorMenuContext::GetRigVMHost() const
{
	if (FRigVMEditorAssetInterfacePtr RigBlueprint = GetRigVMAssetInterface())
	{
		if (URigVMHost* Host = RigBlueprint->GetDebuggedRigVMHost())
		{
			return Host;
		}
	}
	return nullptr;
}

bool URigVMEditorMenuContext::IsAltDown() const
{
	return FSlateApplication::Get().GetModifierKeys().IsAltDown();
}

FRigVMEditorGraphMenuContext URigVMEditorMenuContext::GetGraphMenuContext()
{
	return GraphMenuContext;
}

IRigVMEditor* URigVMEditorMenuContext::GetRigVMEditor() const
{
	if (const TSharedPtr<IRigVMEditor> Editor = WeakRigVMEditor.Pin())
	{
		return Editor.Get();
	}
	return nullptr;
}

