// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMEditorTools.h"
#include "Editor.h"
#include "RigVMDeveloperModule.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMFunctionLibrary.h"
#include "RigVMCore/RigVMGraphFunctionHost.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "Widgets/SRigVMGraphFunctionLocalizationWidget.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "UObject/UObjectIterator.h"

namespace UE::RigVM::Editor::Tools
{

RIGVMEDITOR_API bool PasteNodes(const FVector2D& PasteLocation
	, const FString& TextToImport
	, URigVMController* InFocusedController
	, URigVMGraph* InFocusedModel
	, URigVMFunctionLibrary* InLocalFunctionLibrary
	, IRigVMGraphFunctionHost* InGraphFunctionHost
	, bool bSetupUndoRedo
	, bool bPrintPythonCommands)

{
	if (bSetupUndoRedo)
	{
		InFocusedController->OpenUndoBracket(TEXT("Paste Nodes."));
	}

	TArray<FName> NodeNamesCreated = ImportNodesFromText(PasteLocation, TextToImport, InFocusedController, InFocusedModel, InLocalFunctionLibrary, InGraphFunctionHost, true, bPrintPythonCommands);

	const bool bPastePerformed = NodeNamesCreated.Num() > 0;
	if (bPastePerformed)
	{
		InFocusedController->CloseUndoBracket();
	}
	else
	{
		InFocusedController->CancelUndoBracket();
	}

	return bPastePerformed;
}

RIGVMEDITOR_API TArray<FName> ImportNodesFromText(const FVector2D& PasteLocation
	, const FString& TextToImport
	, URigVMController* InFocusedController
	, URigVMGraph* InFocusedModel
	, URigVMFunctionLibrary* InLocalFunctionLibrary
	, IRigVMGraphFunctionHost* InGraphFunctionHost
	, bool bSetupUndoRedo
	, bool bPrintPythonCommands)
{
	TGuardValue<FRigVMController_RequestLocalizeFunctionDelegate> RequestLocalizeDelegateGuard(
		InFocusedController->RequestLocalizeFunctionDelegate,
		FRigVMController_RequestLocalizeFunctionDelegate::CreateLambda([InFocusedController, InLocalFunctionLibrary, InGraphFunctionHost](FRigVMGraphFunctionIdentifier& InFunctionToLocalize)
			{
				OnRequestLocalizeFunctionDialog(InFunctionToLocalize, InFocusedController, InGraphFunctionHost, true);

				const URigVMLibraryNode* LocalizedFunctionNode = InLocalFunctionLibrary->FindPreviouslyLocalizedFunction(InFunctionToLocalize);
				return LocalizedFunctionNode != nullptr;
			})
	);

	if (bSetupUndoRedo)
	{
		InFocusedController->OpenUndoBracket(TEXT("Import Nodes."));
	}

	TArray<FName> NodeNames = InFocusedController->ImportNodesFromText(TextToImport, bSetupUndoRedo, bPrintPythonCommands);

	if (NodeNames.Num() > 0)
	{
		FBox2D Bounds;
		Bounds.bIsValid = false;

		TArray<FName> NodesToSelect;
		for (const FName& NodeName : NodeNames)
		{
			const URigVMNode* Node = InFocusedModel->FindNodeByName(NodeName);
			check(Node);

			if (Node->IsInjected())
			{
				continue;
			}
			NodesToSelect.Add(NodeName);

			FVector2D Position = Node->GetPosition();
			FVector2D Size = Node->GetSize();

			if (!Bounds.bIsValid)
			{
				Bounds.Min = Bounds.Max = Position;
				Bounds.bIsValid = true;
			}
			Bounds += Position;
			Bounds += Position + Size;
		}

		for (const FName& NodeName : NodesToSelect)
		{
			const URigVMNode* Node = InFocusedModel->FindNodeByName(NodeName);
			check(Node);

			FVector2D Position = Node->GetPosition();
			InFocusedController->SetNodePositionByName(NodeName, PasteLocation + Position - Bounds.GetCenter(), bSetupUndoRedo, false, bPrintPythonCommands);
		}

		InFocusedController->SetNodeSelection(NodesToSelect, bSetupUndoRedo, bPrintPythonCommands);
		
		if (NodeNames.Num() > 0)
		{
			InFocusedController->CloseUndoBracket();
		}
		else
		{
			InFocusedController->CancelUndoBracket();
		}
	}

	return NodeNames;
}

RIGVMEDITOR_API void OnRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier& InFunction
	, URigVMController* InTargetController
	, IRigVMGraphFunctionHost* InTargetFunctionHost
	, bool bForce)
{
	if (InTargetController != nullptr)
	{
		bool bIsPublic;
		if (FRigVMGraphFunctionData::FindFunctionData(InFunction, &bIsPublic))
		{
			if (bForce || bIsPublic)
			{
				TSharedRef<SRigVMGraphFunctionLocalizationDialog> LocalizationDialog = SNew(SRigVMGraphFunctionLocalizationDialog)
					.Function(InFunction)
					.GraphFunctionHost(InTargetFunctionHost);

				if (LocalizationDialog->ShowModal() != EAppReturnType::Cancel)
				{
					InTargetController->LocalizeFunctions(LocalizationDialog->GetFunctionsToLocalize(), true, true, true);
				}
			}
		}
	}
}
	
RIGVMEDITOR_API FAssetData FindAssetFromAnyPath(const FString& InPartialOrFullPath, bool bConvertToRootPath)
{
	UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
	if(bConvertToRootPath)
	{
		return EditorAssetSubsystem->FindAssetData(FSoftObjectPath(InPartialOrFullPath).GetWithoutSubPath().ToString());
	}
	return EditorAssetSubsystem->FindAssetData(InPartialOrFullPath);
}

FFilterByAssetTag::FFilterByAssetTag(TSharedPtr<FFrontendFilterCategory> InCategory, const FRigVMTag& InTag):
	FFrontendFilter(InCategory), Tag(InTag)
{}

bool FFilterByAssetTag::PassesFilter(const FContentBrowserItem& InItem) const
{
	FAssetData AssetData;
	if (InItem.Legacy_TryGetAssetData(AssetData))
	{
		static const FName AssetVariantPropertyName = TEXT("AssetVariant");
		const FProperty* AssetVariantProperty = AssetData.GetClass()->FindPropertyByName(AssetVariantPropertyName);
		const FString VariantStr = AssetData.GetTagValueRef<FString>(AssetVariantPropertyName);
		if(!VariantStr.IsEmpty())
		{
			FRigVMVariant AssetVariant;
			AssetVariantProperty->ImportText_Direct(*VariantStr, &AssetVariant, nullptr, EPropertyPortFlags::PPF_None);

			if (Tag.bMarksSubjectAsInvalid)
			{
				if (!AssetVariant.Tags.Contains(Tag))
				{
					return true;
				}
			}
			else
			{
				if (AssetVariant.Tags.Contains(Tag))
				{
					return true;
				}
			}
		}
		else
		{
			if (Tag.bMarksSubjectAsInvalid)
			{
				return true;
			}
		}
	}
	return false;
}


} // end namespace UE::RigVM::Editor::Tools

FAutoConsoleCommand RigVMCountCopyInstructions(TEXT("RigVM.CountCopyInstructions"),
	TEXT("Iterates all open RigVM VMs and counts the copy instructions"),
	FConsoleCommandDelegate::CreateLambda([]() {

		for (TObjectIterator<URigVM> RigVMIt; RigVMIt; ++RigVMIt)
		{
			const URigVM* VM = *RigVMIt;
			if(!IsValid(VM))
			{
				continue;
			}
			
			int32 CopiesRootToRoot = 0;
			int32 CopiesSubPinToRoot = 0;
			int32 CopiesToSubPin = 0;

			const FRigVMByteCode& ByteCode = VM->GetByteCode();
			const FRigVMInstructionArray Instructions = ByteCode.GetInstructions();

			for(const FRigVMInstruction& Instruction : Instructions)
			{
				if(Instruction.OpCode != ERigVMOpCode::Copy)
				{
					continue;
				}

				FRigVMOperandArray Operands = ByteCode.GetOperandsForOp(Instruction);
				check(Operands.Num() == 2);

				const FRigVMOperand& Source = Operands[0];
				const FRigVMOperand& Target = Operands[1];

				if(Source.GetRegisterOffset() == INDEX_NONE && Target.GetRegisterOffset() == INDEX_NONE)
				{
					CopiesRootToRoot++;
				}
				else if(Source.GetRegisterOffset() != INDEX_NONE && Target.GetRegisterOffset() == INDEX_NONE)
				{
					CopiesSubPinToRoot++;
				}
				else
				{
					CopiesToSubPin++;
				}
			}

			const FString PathName = VM->GetPathName();
			if(CopiesRootToRoot > 0)
			{
				UE_LOGF(LogRigVMDeveloper, Display, "%ls: Copies from root to root: %d out of %d instructions.", *PathName, CopiesRootToRoot, Instructions.Num());
			}
			if(CopiesSubPinToRoot > 0)
			{
				UE_LOGF(LogRigVMDeveloper, Display, "%ls: Copies from sub pin to root: %d out of %d instructions.", *PathName, CopiesSubPinToRoot, Instructions.Num());
			}
			if(CopiesToSubPin > 0)
			{
				UE_LOGF(LogRigVMDeveloper, Display, "%ls: Copies to sub pin (composition): %d out of %d instructions.", *PathName, CopiesToSubPin, Instructions.Num());
			}
		}
	})
);
