// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMExecutionStackCommands.h"

#define LOCTEXT_NAMESPACE "RigVMExecutionStackCommands"

void FRigVMExecutionStackCommands::RegisterCommands()
{
	UI_COMMAND(FocusOnSelection, "Focus On Selection", "Finds the selected operator's node in the graph.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GoToInstruction, "Go To Instruction", "Looks for a specific instruction by index and brings it into focus.", EUserInterfaceActionType::Button, FInputChord(EKeys::G, EModifierKey::Control));
	UI_COMMAND(SelectTargetInstructions, "Select Target Instruction(s)", "Looks up the target instructions and selects them.", EUserInterfaceActionType::Button, FInputChord(EKeys::T, EModifierKey::Control));
	UI_COMMAND(ToggleEarlyExitInstruction, "Toggle Preview Here Instruction", "Sets or unsets the preview here early exit instruction.", EUserInterfaceActionType::Button, FInputChord(EKeys::F9));
	UI_COMMAND(StepEarlyExitInstruction, "Step Preview Here Instruction", "Steps to the next early exist instruction.", EUserInterfaceActionType::Button, FInputChord(EKeys::F10));
}

#undef LOCTEXT_NAMESPACE
