// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMExecutionStackTabSummoner.h"
#include "Widgets/SRigVMExecutionStackView.h"
#include "Editor/RigVMNewEditor.h"

#define LOCTEXT_NAMESPACE "RigVMExecutionStackTabSummoner"

FRigVMExecutionStackTabSummoner::FRigVMExecutionStackTabSummoner(const TSharedRef<IRigVMEditor>& InRigVMEditor)
	: FWorkflowTabFactory(TabID, InRigVMEditor->GetHostingApp())
	, RigVMEditor(InRigVMEditor)
{
	TabLabel = LOCTEXT("RigVMExecutionStackTabLabel", "Execution Stack");
	TabIcon = FSlateIcon(TEXT("RigVMEditorStyle"), "ExecutionStack.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RigVMExecutionStack_ViewMenu_Desc", "Execution Stack");
	ViewMenuTooltip = LOCTEXT("RigVMExecutionStack_ViewMenu_ToolTip", "Show the Execution Stack tab");
}

TSharedRef<SWidget> FRigVMExecutionStackTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRigVMExecutionStackView, RigVMEditor.Pin().ToSharedRef());
}

#undef LOCTEXT_NAMESPACE 
