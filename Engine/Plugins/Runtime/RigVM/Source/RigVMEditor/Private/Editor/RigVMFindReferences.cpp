// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMFindReferences.h"

#include "K2Node_Variable.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Editor/RigVMNewEditor.h"
#include "Framework/Commands/GenericCommands.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Toolkits/ToolkitManager.h"
#include "Widgets/Input/SSearchBox.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "RigVMFindResults"


FString RigVMFindReferencesHelpers::GetPinTypeAsString(const FEdGraphPinType& InPinType)
{
	FString Result = InPinType.PinCategory.ToString();
	if(UObject* SubCategoryObject = InPinType.PinSubCategoryObject.Get())
	{
		Result += FString(" '") + SubCategoryObject->GetName() + "'";
	}
	else
	{
		Result += FString(" '") + InPinType.PinSubCategory.ToString() + "'";
	}

	return Result;
}

bool RigVMFindReferencesHelpers::ParsePinType(FText InKey, FText InValue, FEdGraphPinType& InOutPinType)
{
	bool bParsed = true;

	if(InKey.CompareTo(FRigVMSearchTags::FiB_PinCategory) == 0)
	{
		InOutPinType.PinCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_PinSubCategory) == 0)
	{
		InOutPinType.PinSubCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_ObjectClass) == 0)
	{
		InOutPinType.PinSubCategory = *InValue.ToString();
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_IsArray) == 0)
	{
		InOutPinType.ContainerType = (InValue.ToString().ToBool() ? EPinContainerType::Array : EPinContainerType::None);
	}
	else
	{
		bParsed = false;
	}

	return bParsed;
}

void RigVMFindReferencesHelpers::ExpandAllChildren(FRigVMFindResultPtr InTreeNode, TSharedPtr<STreeView<FRigVMFindResultPtr>> InTreeView)
{
	if (InTreeNode->Children.Num())
	{
		InTreeView->SetItemExpansion(InTreeNode, true);
		for (int32 i = 0; i < InTreeNode->Children.Num(); i++)
		{
			ExpandAllChildren(InTreeNode->Children[i], InTreeView);
		}
	}
}

//////////////////////////////////////////////////////////////////////////
// FBlueprintSearchResult

FRigVMFindResult::FRigVMFindResult(TWeakObjectPtr<URigVMBlueprint> InBlueprint)
	: WeakBlueprint(InBlueprint->GetObject())
{
}

FRigVMFindResult::FRigVMFindResult(TWeakObjectPtr<URigVMBlueprint> InBlueprint, const FText& InDisplayText )
	: WeakBlueprint(InBlueprint->GetObject()), DisplayText(InDisplayText)
{
}

FRigVMFindResult::FRigVMFindResult(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint)
	: WeakBlueprint(InBlueprint)
{
}

FRigVMFindResult::FRigVMFindResult(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint, const FText& InDisplayText )
	: WeakBlueprint(InBlueprint), DisplayText(InDisplayText)
{
}

FReply FRigVMFindResult::OnClick()
{
	// If there is a parent, handle it using the parent's functionality
	if(Parent.IsValid())
	{
		return Parent.Pin()->OnClick();
	}
	
	return FReply::Handled();
}

UObject* FRigVMFindResult::GetObject(FRigVMEditorAssetInterfacePtr InBlueprint) const
{
	return GetRigVMAssetInterface()->GetObject();
}

FText FRigVMFindResult::GetCategory() const
{
	return FText::GetEmpty();
}

TSharedRef<SWidget> FRigVMFindResult::CreateIcon() const
{
	const FSlateBrush* Brush = NULL;

	return 	SNew(SImage)
			.Image(Brush)
			.ColorAndOpacity(FStyleColors::Foreground)
			.ToolTipText( GetCategory() );
}

URigVMBlueprint* FRigVMFindResult::GetBlueprint() const
{
	return Cast<URigVMBlueprint>(GetRigVMAssetInterface().GetObject());
}

FRigVMEditorAssetInterfacePtr FRigVMFindResult::GetRigVMAssetInterface()
{
	if (TStrongObjectPtr<UObject> Blueprint = WeakBlueprint.GetWeakObjectPtr().Pin())
	{
		return FRigVMEditorAssetInterfacePtr(Blueprint.Get());
	}
	return nullptr;
}

const FRigVMEditorAssetInterfacePtr FRigVMFindResult::GetRigVMAssetInterface() const
{
	return const_cast<FRigVMFindResult*>(this)->GetRigVMAssetInterface();
}

FText FRigVMFindResult::GetDisplayString() const
{
	return DisplayText;
}

//////////////////////////////////////////////////////////
// FRigVMFindReferencesGraphNode

FRigVMFindReferencesGraphNode::FRigVMFindReferencesGraphNode(TWeakObjectPtr<URigVMBlueprint> InBlueprint)
	: FRigVMFindResult(FRigVMEditorAssetInterfacePtr(InBlueprint.Get())), Glyph(FAppStyle::GetAppStyleSetName(), "")
	, Class(nullptr)
{
}

FRigVMFindReferencesGraphNode::FRigVMFindReferencesGraphNode(FRigVMEditorAssetInterfacePtr InBlueprint)
	: FRigVMFindResult(InBlueprint), Glyph(FAppStyle::GetAppStyleSetName(), "")
	, Class(nullptr)
{
}

FReply FRigVMFindReferencesGraphNode::OnClick()
{
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if(Blueprint.GetObject())
	{
		UEdGraphNode* OutNode = NULL;
		if(	UEdGraphNode* GraphNode = FRigVMBlueprintUtils::GetNodeByGUID(Blueprint, NodeGuid) )
		{
			TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint->GetObject());
			if (!FoundAssetEditor)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint->GetAssetObjectForEditor());
				FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint->GetObject());
			}

			// If we found an Editor
			if (FoundAssetEditor.IsValid())
			{
				TSharedPtr<FRigVMNewEditor> Editor = StaticCastSharedPtr<FRigVMNewEditor>(FoundAssetEditor);
				Editor->FocusWindow();
				Editor->JumpToHyperlink(GraphNode, false);
			}
			return FReply::Handled();
		}
	}

	return FRigVMFindResult::OnClick();
}

TSharedRef<SWidget> FRigVMFindReferencesGraphNode::CreateIcon() const
{
	return 	SNew(SImage)
		.Image(Glyph.GetOptionalIcon())
		.ColorAndOpacity(GlyphColor)
		.ToolTipText( GetCategory() );
}

void FRigVMFindReferencesGraphNode::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FRigVMSearchTags::FiB_NodeGuid) == 0)
	{
		FString NodeGUIDAsString = InValue.ToString();
		FGuid::Parse(NodeGUIDAsString, NodeGuid);
	}

	if(InKey.CompareTo(FRigVMSearchTags::FiB_ClassName) == 0)
	{
		ClassName = InValue.ToString();
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_Glyph) == 0)
	{
		Glyph = FSlateIcon(Glyph.GetStyleSetName(), *InValue.ToString());
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_GlyphStyleSet) == 0)
	{
		Glyph = FSlateIcon(*InValue.ToString(), Glyph.GetStyleName());
	}
	else if(InKey.CompareTo(FRigVMSearchTags::FiB_GlyphColor) == 0)
	{
		GlyphColor.InitFromString(InValue.ToString());
	}
}

void FRigVMFindReferencesGraphNode::FinalizeSearchData()
{
	if(!ClassName.IsEmpty())
	{
		// Check the node subclasses and look for one with the same short name
		TArray<UClass*> NodeClasses;
		GetDerivedClasses(UEdGraphNode::StaticClass(), NodeClasses, /*bRecursive=*/true);

		for (UClass* FoundClass : NodeClasses)
		{
			if (FoundClass->GetName() == ClassName)
			{
				Class = FoundClass;
				break;
			}
		}

		ClassName.Empty();
	}
}

UObject* FRigVMFindReferencesGraphNode::GetObject(FRigVMEditorAssetInterfacePtr InBlueprint) const
{
	return FRigVMBlueprintUtils::GetNodeByGUID(InBlueprint, NodeGuid);
}

FText FRigVMFindReferencesGraphNode::GetCategory() const
{
	return LOCTEXT("NodeCategory", "Node");
}

//////////////////////////////////////////////////////////
// FRigVMFindReferencesPin

FRigVMFindReferencesPin::FRigVMFindReferencesPin(TWeakObjectPtr<URigVMBlueprint> InBlueprint, FString InSchemaName)
	: FRigVMFindResult(FRigVMEditorAssetInterfacePtr(InBlueprint.Get()))
	, SchemaName(InSchemaName)
	, IconColor(FSlateColor::UseForeground())
{
}

FRigVMFindReferencesPin::FRigVMFindReferencesPin(FRigVMEditorAssetInterfacePtr InBlueprint, FString InSchemaName)
	: FRigVMFindResult(InBlueprint)
	, SchemaName(InSchemaName)
	, IconColor(FSlateColor::UseForeground())
{
}

TSharedRef<SWidget> FRigVMFindReferencesPin::CreateIcon() const
{
	const FSlateBrush* Brush = nullptr;

	if( PinType.IsArray() )
	{
		Brush = FAppStyle::Get().GetBrush( TEXT("GraphEditor.ArrayPinIcon") );
	}
	else
	{
		Brush = FAppStyle::Get().GetBrush( TEXT("GraphEditor.PinIcon") );
	}

	return SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText(FText::FromString(RigVMFindReferencesHelpers::GetPinTypeAsString(PinType)));
}

void FRigVMFindReferencesPin::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FRigVMSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else
	{
		RigVMFindReferencesHelpers::ParsePinType(InKey, InValue, PinType);
	}
}

FText FRigVMFindReferencesPin::GetCategory() const
{
	return LOCTEXT("PinCategory", "Pin");
}

void FRigVMFindReferencesPin::FinalizeSearchData()
{
	if(!PinType.PinSubCategory.IsNone())
	{
		// This can either be a full path to an object, or a short name specific to the category
		if (FPackageName::IsShortPackageName(PinType.PinSubCategory))
		{
			// This could also be an old class name without the full path, but it's fine to ignore in that case
		}
		else
		{
			PinType.PinSubCategoryObject = FindObject<UObject>(UObject::StaticClass(), *PinType.PinSubCategory.ToString());
			if (PinType.PinSubCategoryObject.IsValid())
			{
				PinType.PinSubCategory = NAME_None;
			}
		}
	}

	if(!SchemaName.IsEmpty())
	{
		// Get all subclasses of schema and find the one with a matching short name
		TArray<UClass*> SchemaClasses;
		GetDerivedClasses(UEdGraphSchema::StaticClass(), SchemaClasses, /*bRecursive=*/true);

		for (UClass* FoundClass : SchemaClasses)
		{
			if (FoundClass->GetName() == SchemaName)
			{
				UEdGraphSchema* Schema = FoundClass->GetDefaultObject<UEdGraphSchema>();
				IconColor = Schema->GetPinTypeColor(PinType);
				break;
			}
		}

		SchemaName.Empty();
	}
}

//////////////////////////////////////////////////////////
// FRigVMFindReferencesVariable

FRigVMFindReferencesVariable::FRigVMFindReferencesVariable(TWeakObjectPtr<URigVMBlueprint> InBlueprint)
	: FRigVMFindResult(FRigVMEditorAssetInterfacePtr(InBlueprint.Get()))
{
}

FRigVMFindReferencesVariable::FRigVMFindReferencesVariable(FRigVMEditorAssetInterfacePtr InBlueprint)
	: FRigVMFindResult(InBlueprint)
{
}

TSharedRef<SWidget> FRigVMFindReferencesVariable::CreateIcon() const
{
	FLinearColor IconColor = FStyleColors::Foreground.GetSpecifiedColor();
	const FSlateBrush* Brush = UK2Node_Variable::GetVarIconFromPinType(PinType, IconColor).GetOptionalIcon();
	IconColor = UEdGraphSchema_K2::StaticClass()->GetDefaultObject<UEdGraphSchema_K2>()->GetPinTypeColor(PinType);

	return 	SNew(SImage)
		.Image(Brush)
		.ColorAndOpacity(IconColor)
		.ToolTipText( FText::FromString(RigVMFindReferencesHelpers::GetPinTypeAsString(PinType)) );
}

void FRigVMFindReferencesVariable::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FRigVMSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
	else
	{
		RigVMFindReferencesHelpers::ParsePinType(InKey, InValue, PinType);
	}
}

FText FRigVMFindReferencesVariable::GetCategory() const
{
	return LOCTEXT("Variable", "Variable");
}

void FRigVMFindReferencesVariable::FinalizeSearchData()
{
	if (!PinType.PinSubCategory.IsNone())
	{
		// This can either be a full path to an object, or a short name specific to the category
		if (FPackageName::IsShortPackageName(PinType.PinSubCategory))
		{
			// This could also be an old class name without the full path, but it's fine to ignore in that case
		}
		else
		{
			PinType.PinSubCategoryObject = FindObject<UObject>(UObject::StaticClass(), *PinType.PinSubCategory.ToString());
			if (PinType.PinSubCategoryObject.IsValid())
			{
				PinType.PinSubCategory = NAME_None;
			}
		}
	}
}

//////////////////////////////////////////////////////////
// FRigVMFindReferencesGraph

FRigVMFindReferencesGraph::FRigVMFindReferencesGraph(TWeakObjectPtr<URigVMBlueprint> InBlueprint, EGraphType InGraphType)
	: FRigVMFindResult(FRigVMEditorAssetInterfacePtr(InBlueprint.Get())), GraphType(InGraphType)
{
}
FRigVMFindReferencesGraph::FRigVMFindReferencesGraph(FRigVMEditorAssetInterfacePtr InBlueprint, EGraphType InGraphType)
	: FRigVMFindResult(InBlueprint), GraphType(InGraphType)
{
}

FReply FRigVMFindReferencesGraph::OnClick()
{
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if(Blueprint.GetObject())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint->GetAssetObjectForEditor());
		TSharedPtr< IToolkit > FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(Blueprint->GetObject());

		// If we found an Editor
		if (FoundAssetEditor.IsValid())
		{
			TSharedPtr<FRigVMNewEditor> Editor = StaticCastSharedPtr<FRigVMNewEditor>(FoundAssetEditor);

			TArray<UEdGraph*> BlueprintGraphs;
			Blueprint->GetAllEdGraphs(BlueprintGraphs);

			for(UEdGraph* Graph : BlueprintGraphs)
			{
				FGraphDisplayInfo DisplayInfo;
				Graph->GetSchema()->GetGraphDisplayInformation(*Graph, DisplayInfo);
			
				if(DisplayInfo.PlainName.EqualTo(DisplayText))
				{
					Editor->FocusWindow();
					Editor->JumpToHyperlink(Graph, false);
					break;
				}
			}
		}
	}
	else
	{
		return FRigVMFindResult::OnClick();
	}
	return FReply::Handled();
}

TSharedRef<SWidget> FRigVMFindReferencesGraph::CreateIcon() const
{
	const FSlateBrush* Brush = NULL;
	switch (GraphType)
	{
		case GT_Function:
			{
				Brush = FAppStyle::GetBrush(TEXT("GraphEditor.Function_16x"));
				break;
			}
		case GT_Ubergraph:
			{
				Brush = FAppStyle::GetBrush( TEXT("GraphEditor.EventGraph_16x") );
				break;
			}
		default:
			{
				break;
			}
	}
	
	return 	SNew(SImage)
		.Image(Brush)
		.ToolTipText( GetCategory() );
}

void FRigVMFindReferencesGraph::ParseSearchInfo(FText InKey, FText InValue)
{
	if(InKey.CompareTo(FRigVMSearchTags::FiB_Name) == 0)
	{
		DisplayText = InValue;
	}
}

FText FRigVMFindReferencesGraph::GetCategory() const
{
	if(GraphType == GT_Function)
	{
		return LOCTEXT("FunctionGraphCategory", "Function");
	}
	return LOCTEXT("GraphCategory", "Graph");
}

//////////////////////////////////////////////////////////////////////////
// SBlueprintSearch

void SRigVMFindReferences::Construct( const FArguments& InArgs, TSharedPtr<FRigVMEditorBase> InEditor)
{
	EditorPtr = InEditor;
	
	RegisterCommands();
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(8.f, 5.f, 8.f, 5.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SAssignNew(SearchTextField, SSearchBox)
					.HintText(LOCTEXT("BlueprintSearchHint", "Enter function or event name to find references..."))
					.OnTextChanged(this, &SRigVMFindReferences::OnSearchTextChanged)
					.OnTextCommitted(this, &SRigVMFindReferences::OnSearchTextCommitted)
					.DelayChangeNotificationsWhileTyping(false)
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(8.f, 8.f, 4.f, 0.f))
				[
					SAssignNew(TreeView, SRigVMFindResultTreeViewType)
						.TreeItemsSource( &ItemsFound )
						.OnGenerateRow( this, &SRigVMFindReferences::OnGenerateRow )
						.OnGetChildren( this, &SRigVMFindReferences::OnGetChildren )
						.OnMouseButtonDoubleClick(this,&SRigVMFindReferences::OnTreeSelectionDoubleClicked)
						.SelectionMode( ESelectionMode::Multi )
						.OnContextMenuOpening(this, &SRigVMFindReferences::OnContextMenuOpening)
				]
			]
		]
	];
}

void SRigVMFindReferences::RegisterCommands()
{
	CommandList = MakeShareable(new FUICommandList());

	CommandList->MapAction( FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SRigVMFindReferences::OnCopyAction));

	CommandList->MapAction( FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateSP(this, &SRigVMFindReferences::OnSelectAllAction) );
}

void SRigVMFindReferences::FocusForUse(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult)
{
	// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
	FWidgetPath FilterTextBoxWidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked( SearchTextField.ToSharedRef(), FilterTextBoxWidgetPath );

	// Set keyboard focus directly
	FSlateApplication::Get().SetKeyboardFocus( FilterTextBoxWidgetPath, EFocusCause::SetDirectly );

	if (!NewSearchTerms.IsEmpty())
	{
		SearchTextField->SetText(FText::FromString(NewSearchTerms));
		FindReferences(NewSearchTerms);

		// Select the first result
		if (bSelectFirstResult && ItemsFound.Num())
		{
			TSharedPtr<FRigVMFindResult> ItemToFocusOn = ItemsFound[0];

			// We want the first childmost item to select, as that is the item that is most-likely to be what was searched for (parents being graphs).
			// Will fail back upward as neccessary to focus on a focusable item
			while(ItemToFocusOn->Children.Num())
			{
				ItemToFocusOn = ItemToFocusOn->Children[0];
			}
			TreeView->SetSelection(ItemToFocusOn);
			ItemToFocusOn->OnClick();
		}
	}
}

void SRigVMFindReferences::FindReferences(const FString& SearchTerms)
{
	ItemsFound.Empty();
	ElementHashToResult.Empty();

	HighlightText = FText::FromString(SearchTerms);

	TSharedPtr<FRigVMEditorBase> Editor = EditorPtr.Pin();
	if (!Editor.IsValid())
	{
		return;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	if (!Blueprint)
	{
		return;
	}
	
	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

	auto FindOrAddGraphResult = [&](const UEdGraph* EdGraph)
	{
		FRigVMFindResultPtr GraphResult;
		if (FRigVMFindResultPtr* Found = ElementHashToResult.Find(GetTypeHash(EdGraph)))
		{
			GraphResult = *Found;
		}
		else
		{
			GraphResult = ItemsFound.Add_GetRef(FRigVMFindResultPtr(new FRigVMFindReferencesGraph(Blueprint, Schema->GetGraphType(EdGraph))));
			ElementHashToResult.Add(GetTypeHash(EdGraph), GraphResult);
		}

		FGraphDisplayInfo DisplayInfo;
		EdGraph->GetSchema()->GetGraphDisplayInformation(*EdGraph, DisplayInfo);

		GraphResult->ParseSearchInfo(FRigVMSearchTags::FiB_Name, DisplayInfo.PlainName);
		return GraphResult;
	};

	auto FindOrAddNodeResult = [&](const UEdGraphNode* EdNode, const TArray<struct FSearchTagDataPair>& NodeMetaData)
	{
		FRigVMFindResultPtr NodeResult;
		if (FRigVMFindResultPtr* Found = ElementHashToResult.Find(GetTypeHash(EdNode)))
		{
			NodeResult = *Found;
		}
		else
		{
			FRigVMFindResultPtr GraphResult = FindOrAddGraphResult(EdNode->GetGraph());
			NodeResult = FRigVMFindResultPtr(new FRigVMFindReferencesGraphNode(Blueprint));
			ElementHashToResult.Add(GetTypeHash(EdNode), NodeResult);
			GraphResult->Children.Add(NodeResult);
			NodeResult->Parent = GraphResult;

			for (const FSearchTagDataPair& MetadataTag : NodeMetaData)
			{
				NodeResult->ParseSearchInfo(MetadataTag.Key, MetadataTag.Value);
			}
		}
		return NodeResult;
	};

	auto FindOrAddPinResult = [&](const UEdGraphPin* EdPin, const TArray<struct FSearchTagDataPair>& PinMetaData, const TArray<struct FSearchTagDataPair>& NodeMetaData)
	{
		FRigVMFindResultPtr PinResult;
		if (FRigVMFindResultPtr* Found = ElementHashToResult.Find(GetTypeHash(EdPin)))
		{
			PinResult = *Found;
		}
		else
		{
			UEdGraphNode* EdNode = EdPin->GetOwningNode();
			
			const FString SchemaClassName = Blueprint->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass()->GetName();
			FRigVMFindResultPtr NodeResult = FindOrAddNodeResult(EdPin->GetOwningNode(), NodeMetaData);
			PinResult = FRigVMFindResultPtr(new FRigVMFindReferencesPin(Blueprint, SchemaClassName));
			ElementHashToResult.Add(GetTypeHash(EdPin), PinResult);
			NodeResult->Children.Add(PinResult);
			PinResult->Parent = NodeResult;

			for (const FSearchTagDataPair& MetadataTag : PinMetaData)
			{
				PinResult->ParseSearchInfo(MetadataTag.Key, MetadataTag.Value);
			}
		}
		return PinResult;
	};

	auto FindOrAddVariableResult = [&](const FName& VariableName, const UEdGraph* Graph, const TArray<UBlueprintExtension::FSearchTagDataPair>& VariableMetaData)
	{
		FRigVMFindResultPtr VariableResult;

		const uint32 Hash = HashCombine(GetTypeHash(VariableName), GetTypeHash(Graph));
		if (FRigVMFindResultPtr* Found = ElementHashToResult.Find(Hash))
		{
			VariableResult = *Found;
		}
		else
		{
			FRigVMFindResultPtr GraphResult;
			if (Graph)
			{
				GraphResult = FindOrAddGraphResult(Graph);
			}

			VariableResult = FRigVMFindResultPtr(new FRigVMFindReferencesVariable(Blueprint));
			ElementHashToResult.Add(Hash, VariableResult);
			if (GraphResult)
			{
				GraphResult->Children.Add(VariableResult);
				VariableResult->Parent = GraphResult;
			}
			else
			{
				ItemsFound.Add(VariableResult);
			}

			for (const UBlueprintExtension::FSearchTagDataPair& MetadataTag : VariableMetaData)
			{
				VariableResult->ParseSearchInfo(MetadataTag.Key, MetadataTag.Value);
			}
		}
		return VariableResult;
	};

	TArray<URigVMGraph*> Graphs = Blueprint->GetRigVMClient()->GetAllModels(true, true);
	for (const URigVMGraph* Graph : Graphs)
	{
		if (URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(Blueprint->GetEdGraph(Graph)))
		{
			for (TObjectPtr<UEdGraphNode> EdNode : EdGraph->Nodes)
			{
				TArray<struct FSearchTagDataPair> NodeMetadata;
				EdNode->AddSearchMetaDataInfo(NodeMetadata);

				const FSearchTagDataPair* MatchedPair = nullptr;
				for (const FSearchTagDataPair& MetadataTag : NodeMetadata)
				{
					if (MetadataTag.Value.ToString().Contains(SearchTerms))
					{
						MatchedPair = &MetadataTag;
						break;
					}
				}
				if (MatchedPair)
				{
					FRigVMFindResultPtr Result = FindOrAddNodeResult(EdNode, NodeMetadata);

					const FText DisplayText = FText::Format(LOCTEXT("RigVMFindReferencesValues", "{0}: {1}"), MatchedPair->Key, MatchedPair->Value);
					FRigVMFindResultPtr DetailResult = FRigVMFindResultPtr(new FRigVMFindResult(Blueprint, DisplayText));

					Result->Children.Add(DetailResult);
					DetailResult->Parent = Result;
				}

				for (UEdGraphPin* Pin : EdNode->GetAllPins())
				{
					TArray<struct FSearchTagDataPair> PinMetaData;
					EdNode->AddPinSearchMetaDataInfo(Pin, PinMetaData);

					const FSearchTagDataPair* MatchedPinPair = nullptr;
					for (const FSearchTagDataPair& MetadataTag : PinMetaData)
					{
						if (MetadataTag.Value.ToString().Contains(SearchTerms))
						{
							MatchedPinPair = &MetadataTag;
							break;
						}
					}

					if (MatchedPinPair)
					{
						FRigVMFindResultPtr Result = FindOrAddPinResult(Pin, PinMetaData, NodeMetadata);

						const FText DisplayText = FText::Format(LOCTEXT("RigVMFindReferencesValues", "{0}: {1}"), MatchedPinPair->Key, MatchedPinPair->Value);
						FRigVMFindResultPtr DetailResult = FRigVMFindResultPtr(new FRigVMFindResult(Blueprint, DisplayText));

						Result->Children.Add(DetailResult);
						DetailResult->Parent = Result;
					}
				}
			}

			TArray<FRigVMGraphVariableDescription> LocalVariables = Graph->GetLocalVariables();
			for (const FRigVMGraphVariableDescription& Variable : LocalVariables)
			{
				TArray<UBlueprintExtension::FSearchTagDataPair> LocalVariableMetadata;
				EdGraph->AddLocalVariableSearchMetaDataInfo(Variable.Name, LocalVariableMetadata);

				const UBlueprintExtension::FSearchTagDataPair* MatchedVariablePair = nullptr;
				for (const UBlueprintExtension::FSearchTagDataPair& MetadataTag : LocalVariableMetadata)
				{
					if (MetadataTag.Value.ToString().Contains(SearchTerms))
					{
						MatchedVariablePair = &MetadataTag;
						break;
					}
				}

				if (MatchedVariablePair)
				{
					FRigVMFindResultPtr Result = FindOrAddVariableResult(Variable.Name, EdGraph, LocalVariableMetadata);

					const FText DisplayText = FText::Format(LOCTEXT("RigVMFindReferencesValues", "{0}: {1}"), MatchedVariablePair->Key, MatchedVariablePair->Value);
					FRigVMFindResultPtr DetailResult = FRigVMFindResultPtr(new FRigVMFindResult(Blueprint, DisplayText));

					Result->Children.Add(DetailResult);
					DetailResult->Parent = Result;
				}
			}
		}
	}

	TArray<FRigVMGraphVariableDescription> BPVariables = Blueprint->GetAssetVariables();
	for (const FRigVMGraphVariableDescription& Variable : BPVariables)
	{
		if (FRigVMEditorAssetInterfacePtr BP = Blueprint)
		{
			TArray<UBlueprintExtension::FSearchTagDataPair> BPVariableMetadata;
			BP->AddVariableSearchMetaDataInfo(Variable.Name, BPVariableMetadata);

			const UBlueprintExtension::FSearchTagDataPair* MatchedVariablePair = nullptr;
			for (const UBlueprintExtension::FSearchTagDataPair& MetadataTag : BPVariableMetadata)
			{
				if (MetadataTag.Value.ToString().Contains(SearchTerms))
				{
					MatchedVariablePair = &MetadataTag;
					break;
				}
			}

			if (MatchedVariablePair)
			{
				FRigVMFindResultPtr Result = FindOrAddVariableResult(Variable.Name, nullptr, BPVariableMetadata);

				const FText DisplayText = FText::Format(LOCTEXT("RigVMFindReferencesValues", "{0}: {1}"), MatchedVariablePair->Key, MatchedVariablePair->Value);
				FRigVMFindResultPtr DetailResult = FRigVMFindResultPtr(new FRigVMFindResult(Blueprint, DisplayText));

				Result->Children.Add(DetailResult);
				DetailResult->Parent = Result;
			}
		}
	}

	TreeView->RequestTreeRefresh();
	for (FRigVMFindResultPtr Item : ItemsFound)
	{
		RigVMFindReferencesHelpers::ExpandAllChildren(Item, TreeView);
	}
}

void SRigVMFindReferences::OnSearchTextChanged( const FText& Text)
{
	SearchValue = Text.ToString();
}

void SRigVMFindReferences::OnSearchTextCommitted( const FText& Text, ETextCommit::Type CommitType )
{
	if (CommitType == ETextCommit::OnEnter)
	{
		FindReferences(SearchValue);
	}
}

TSharedRef<ITableRow> SRigVMFindReferences::OnGenerateRow( FRigVMFindResultPtr InItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	// Finalize the search data, this does some non-thread safe actions that could not be done on the separate thread.
	InItem->FinalizeSearchData();

	FFormatNamedArguments Args;
	Args.Add(TEXT("Category"), InItem->GetCategory());
	Args.Add(TEXT("DisplayTitle"), InItem->DisplayText);

	FText Tooltip = FText::Format(LOCTEXT("BlueprintResultSearchToolTip", "{Category} : {DisplayTitle}"), Args);

	return SNew( STableRow< TSharedPtr<FRigVMFindResult> >, OwnerTable )
		.Style( &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ShowParentsTableView.Row") )
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				InItem->CreateIcon()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(STextBlock)
					.Text(InItem.Get(), &FRigVMFindResult::GetDisplayString)
					.HighlightText(HighlightText)
					.ToolTipText(Tooltip)
			]
		];
}

void SRigVMFindReferences::OnGetChildren( FRigVMFindResultPtr InItem, TArray< FRigVMFindResultPtr >& OutChildren )
{
	OutChildren += InItem->Children;
}

void SRigVMFindReferences::OnTreeSelectionDoubleClicked( FRigVMFindResultPtr Item )
{
	if(Item.IsValid())
	{
		Item->OnClick();
	}
}

TSharedPtr<SWidget> SRigVMFindReferences::OnContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("BasicOperations");
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().SelectAll);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRigVMFindReferences::SelectAllItemsHelper(FRigVMFindResultPtr InItemToSelect)
{
	// Iterates over all children and recursively selects all items in the results
	TreeView->SetItemSelection(InItemToSelect, true);

	for( const auto& Child : InItemToSelect->Children )
	{
		SelectAllItemsHelper(Child);
	}
}

void SRigVMFindReferences::OnSelectAllAction()
{
	for( const auto& Item : ItemsFound )
	{
		SelectAllItemsHelper(Item);
	}
}

void SRigVMFindReferences::OnCopyAction()
{
	TArray< TSharedPtr<FRigVMFindResult> > SelectedItems = TreeView->GetSelectedItems();

	FString SelectedText;

	for(const TSharedPtr<FRigVMFindResult>& SelectedItem : SelectedItems)
	{
		// Add indents for each layer into the tree the item is
		for(auto ParentItem = SelectedItem->Parent; ParentItem.IsValid(); ParentItem = ParentItem.Pin()->Parent)
		{
			SelectedText += TEXT("\t");
		}

		// Add the display string
		SelectedText += SelectedItem->GetDisplayString().ToString();

		// Line terminator so the next item will be on a new line
		SelectedText += LINE_TERMINATOR;
	}

	// Copy text to clipboard
	FPlatformApplicationMisc::ClipboardCopy( *SelectedText );
}

FReply SRigVMFindReferences::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// BlueprintEditor's IToolkit code will handle shortcuts itself - but we can just use
	// simple slate handlers when we're standalone:
	if(!EditorPtr.IsValid() && CommandList.IsValid())
	{
		if( CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

#undef LOCTEXT_NAMESPACE