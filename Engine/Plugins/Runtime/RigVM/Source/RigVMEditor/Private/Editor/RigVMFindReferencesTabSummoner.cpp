// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMFindReferencesTabSummoner.h"
#include "Editor/RigVMNewEditor.h"
#include "Editor/RigVMFindReferences.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "RigVMEditor"

FRigVMFindReferencesTabSummoner::FRigVMFindReferencesTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
	: FWorkflowTabFactory(TabID(), InHostingApp)
{
	TabLabel = LOCTEXT("FindResultsTabTitle", "Find Results");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.FindResults");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("FindResultsView", "Find Results");
	ViewMenuTooltip = LOCTEXT("FindResultsView_ToolTip", "Show find results for searching in this blueprint");
}


TSharedRef<SWidget> FRigVMFindReferencesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FRigVMNewEditor> EditorPtr = StaticCastSharedPtr<FRigVMNewEditor>(HostingApp.Pin());

	return EditorPtr->GetFindResults();
}



#undef LOCTEXT_NAMESPACE
