// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMGraphDetailCustomization.h"
#include "Widgets/SWidget.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Styling/AppStyle.h"
#include "SPinTypeSelector.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "PropertyCustomizationHelpers.h"
#include "NodeFactory.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMHost.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "EditorCategoryUtils.h"
#include "IPropertyUtilities.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "RigVMStringUtils.h"
#include "Widgets/SRigVMGraphPinEnumPicker.h"
#include "Widgets/SRigVMVariantWidget.h"
#include "Widgets/SRigVMNodeLayoutWidget.h"
#include "ScopedTransaction.h"
#include "Editor/RigVMEditorTools.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "RigVMDeveloperTypeUtils.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/ObjectPtr.h"
#if WITH_RIGVMLEGACYEDITOR
#include "Editor/RigVMLegacyEditor.h"
#endif

#define LOCTEXT_NAMESPACE "RigVMGraphDetailCustomization"

// --- FTransform and EulerTransform customization data start ---
static const FName NAME_Translation(TEXT("Translation"));
static const FName NAME_Scale3D(TEXT("Scale3D"));
static const FName NAME_Scale(TEXT("Scale"));
static const FName TransformComponentNames[] = {NAME_Translation, NAME_Rotation, NAME_Scale3D};
static const FName EulerTransformComponentNames[] = { NAME_Location, NAME_Rotation, NAME_Scale };
static_assert(ESlateTransformComponent::Location == 0); // make sure the names array is in the same order than the components enum
static_assert(ESlateTransformComponent::Rotation == 1);
static_assert(ESlateTransformComponent::Scale == 2);
// --- FTransform and EulerTransform customization data end ---

static const FText RigVMGraphDetailCustomizationMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");

FRigVMFunctionArgumentReorderDragDropOp::FRigVMFunctionArgumentReorderDragDropOp(const TWeakObjectPtr<URigVMPin>& InPinPtr)
: PinPtr(InPinPtr)
{
}

void FRigVMFunctionArgumentReorderDragDropOp::Init()
{
	SetValidTarget(false);
	SetupDefaults();
	Construct();
}

void FRigVMFunctionArgumentReorderDragDropOp::SetValidTarget(bool IsValidTarget)
{
	if (IsValidTarget)
	{
		CurrentHoverText = LOCTEXT("PlaceArgumentHere", "Place Argument Here");
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
	}
	else
	{
		CurrentHoverText = LOCTEXT("CannotPlaceArgumentHere", "Cannot Place Argument Here");
		CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
	}
}

const URigVMPin* FRigVMFunctionArgumentReorderDragDropOp::GetPin() const
{
	if (PinPtr.IsValid())
	{
		return PinPtr.Get();
	}
	return nullptr;
}

TSharedPtr<FDragDropOperation> FRigVMFunctionArgumentReorderDragDropHandler::CreateDragDropOperation() const
{
	TSharedPtr<FRigVMFunctionArgumentReorderDragDropOp> DragOp = MakeShared<FRigVMFunctionArgumentReorderDragDropOp>(PinPtr);
	DragOp->Init();
	return DragOp;
}

bool FRigVMFunctionArgumentReorderDragDropHandler::AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	if (!WeakRigVMClientHost.IsValid())
	{
		return false;
	}

	IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
	if (!RigVMClientHost)
	{
		return false;
	}

	FRigVMClient* Client = RigVMClientHost->GetRigVMClient();
	if (!Client)
	{
		return false;
	}

	const TSharedPtr<FRigVMFunctionArgumentReorderDragDropOp> DragOp = DragDropEvent.GetOperationAs<FRigVMFunctionArgumentReorderDragDropOp>();
	check(DragOp.IsValid());
	
	const URigVMPin* DraggedPin = DragOp->GetPin();
	const URigVMPin* ThisPin = GetPin();
	check(DraggedPin && ThisPin && DraggedPin != ThisPin);
	check(DraggedPin->GetDirection() == ThisPin->GetDirection());

	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ThisPin->GetNode());
	if (!LibraryNode)
	{
		return false;
	}

	URigVMGraph* ContainedGraph = LibraryNode->GetContainedGraph();
	if (!ContainedGraph)
	{
		return false;
	}

	URigVMController* Controller = Client->GetOrCreateController(ContainedGraph);
	if (!Controller)
	{
		return false;
	}

	int32 NewPinIndex = ThisPin->GetPinIndex();
	if (DropZone == EItemDropZone::BelowItem)
	{
		if (NewPinIndex < DraggedPin->GetPinIndex())
		{
			NewPinIndex++;
		}
	}
	else if (DropZone == EItemDropZone::AboveItem)
	{
		if (NewPinIndex > DraggedPin->GetPinIndex())
		{
			NewPinIndex--;
		}
	}

	TGuardValue<bool> GuardEditGraph(LibraryNode->GetContainedGraph()->bEditable, true);
	Controller->SetExposedPinIndex(DraggedPin->GetFName(), NewPinIndex);
	return true;
}

TOptional<EItemDropZone> FRigVMFunctionArgumentReorderDragDropHandler::CanAcceptDrop(const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone) const
{
	const TSharedPtr<FRigVMFunctionArgumentReorderDragDropOp> DragOp = DragDropEvent.GetOperationAs<FRigVMFunctionArgumentReorderDragDropOp>();
	if (!DragOp.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	DragOp->SetValidTarget(false);
	
	const URigVMPin* DraggedPin = DragOp->GetPin();
	const URigVMPin* ThisPin = GetPin();
	if(DraggedPin == nullptr || ThisPin == nullptr || DraggedPin == ThisPin)
	{
		return TOptional<EItemDropZone>();
	}

	if (DraggedPin->GetDirection() != ThisPin->GetDirection())
	{
		return TOptional<EItemDropZone>();
	}

	if (DropZone == EItemDropZone::OntoItem)
	{
		DropZone = EItemDropZone::BelowItem;
	}

	if (DropZone == EItemDropZone::BelowItem)
	{
		if (DraggedPin->GetPinIndex() == ThisPin->GetPinIndex() + 1)
		{
			return TOptional<EItemDropZone>();
		}
	}
	else if (DropZone == EItemDropZone::AboveItem)
	{
		if (DraggedPin->GetPinIndex() == ThisPin->GetPinIndex() - 1)
		{
			return TOptional<EItemDropZone>();
		}
	}

	DragOp->SetValidTarget(true);
	return DropZone;
}

const URigVMPin* FRigVMFunctionArgumentReorderDragDropHandler::GetPin() const
{
	if (PinPtr.IsValid())
	{
		return PinPtr.Get();
	}
	return nullptr;
}

FRigVMFunctionArgumentGroupLayout::FRigVMFunctionArgumentGroupLayout(
	const TWeakObjectPtr<URigVMGraph>& InGraph,
	const TWeakInterfacePtr<IRigVMClientHost>& InRigVMClientHost,
	const TWeakPtr<IRigVMEditor>& InEditor,
	bool bInputs)
	: GraphPtr(InGraph)
	, WeakRigVMClientHost(InRigVMClientHost)
	, RigVMEditorPtr(InEditor)
	, bIsInputGroup(bInputs)
{
	if (WeakRigVMClientHost.IsValid())
	{
		WeakRigVMClientHost->OnModified().AddRaw(this, &FRigVMFunctionArgumentGroupLayout::HandleModifiedEvent);
	}
}

FRigVMFunctionArgumentGroupLayout::~FRigVMFunctionArgumentGroupLayout()
{
	if (WeakRigVMClientHost.IsValid())
	{
		WeakRigVMClientHost->OnModified().RemoveAll(this);
	}
}

void FRigVMFunctionArgumentGroupLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	bool WasContentAdded = false;
	if (GraphPtr.IsValid())
	{
		URigVMGraph* Graph = GraphPtr.Get();
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter()))
		{
			for (URigVMPin* Pin : LibraryNode->GetPins())
			{
				if ((bIsInputGroup && (Pin->GetDirection() == ERigVMPinDirection::Input || Pin->GetDirection() == ERigVMPinDirection::IO)) ||
					(!bIsInputGroup && (Pin->GetDirection() == ERigVMPinDirection::Output || Pin->GetDirection() == ERigVMPinDirection::IO)))
				{
					TSharedRef<class FRigVMFunctionArgumentLayout> ArgumentLayout = MakeShareable(new FRigVMFunctionArgumentLayout(
						Pin,
						GraphPtr,
						WeakRigVMClientHost,
						RigVMEditorPtr
					));
					ChildrenBuilder.AddCustomBuilder(ArgumentLayout);
					WasContentAdded = true;
				}
			}
		}
	}
	if (!WasContentAdded)
	{
		// Add a text widget to let the user know to hit the + icon to add parameters.
		ChildrenBuilder.AddCustomRow(FText::GetEmpty()).WholeRowContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoArgumentsAddedForRigVMHost", "Please press the + icon above to add parameters"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

void FRigVMFunctionArgumentGroupLayout::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	if (!GraphPtr.IsValid())
	{
		return;
	}
	
	URigVMGraph* Graph = GraphPtr.Get();
	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	if (LibraryNode == nullptr)
	{
		if (URigVMNode* Node = Cast<URigVMNode>(InSubject))
		{
			LibraryNode = Cast<URigVMLibraryNode>(Node);
		}
		if (URigVMPin* Pin = Cast<URigVMPin>(InSubject))
		{
			LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode());
		}
	}
	if (LibraryNode == nullptr)
	{
		return;
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::PinAdded:
		case ERigVMGraphNotifType::PinRenamed:
		case ERigVMGraphNotifType::PinRemoved:
		case ERigVMGraphNotifType::PinIndexChanged:
		case ERigVMGraphNotifType::PinTypeChanged:
		{
			URigVMPin* Pin = CastChecked<URigVMPin>(InSubject);
			if (Pin->GetNode() == LibraryNode ||
				(Pin->GetNode()->IsA<URigVMFunctionEntryNode>() && Pin->GetNode()->GetOuter() == Graph) ||
				(Pin->GetNode()->IsA<URigVMFunctionReturnNode>() && Pin->GetNode()->GetOuter() == Graph))
			{
				OnRebuildChildren.ExecuteIfBound();
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

class FRigVMFunctionArgumentPinTypeSelectorFilter : public IPinTypeSelectorFilter
{
public:
	FRigVMFunctionArgumentPinTypeSelectorFilter(const TWeakPtr<IRigVMEditor>& InRigVMEditor, const TWeakObjectPtr<URigVMGraph>& InGraph)
		: RigVMEditorPtr(InRigVMEditor), GraphPtr(InGraph)
	{
	}
	
	virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const override
	{
		if (!InItem.IsValid())
		{
			return false;
		}

		// Only allow an execute context pin if the graph doesnt have one already
		FString CPPType;
		UObject* CPPTypeObject = nullptr;
		RigVMTypeUtils::CPPTypeFromPinType(InItem.Get()->GetPinType(false), CPPType, &CPPTypeObject);
		if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(CPPTypeObject))
		{
			if (ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
			{
				if (GraphPtr.IsValid())
				{
					if (URigVMFunctionEntryNode* EntryNode = GraphPtr.Get()->GetEntryNode())
					{
						for (URigVMPin* Pin : EntryNode->GetPins())
						{
							if (Pin->IsExecuteContext())
							{
								return false;
							}
						}
					}
				}
			}
		}
		
		if (RigVMEditorPtr.IsValid())
		{
			TArray<TSharedPtr<IPinTypeSelectorFilter>> Filters;
			RigVMEditorPtr.Pin()->GetPinTypeSelectorFilters(Filters);
			for(const TSharedPtr<IPinTypeSelectorFilter>& Filter : Filters)
			{
				if(!Filter->ShouldShowPinTypeTreeItem(InItem))
				{
					return false;
				}
			}
			return true;
		}

		return false;
	}

private:

	TWeakPtr<IRigVMEditor> RigVMEditorPtr;
	
	TWeakObjectPtr<URigVMGraph> GraphPtr;
};

void FRigVMFunctionArgumentLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	CachedImg_Pin_Disconnected = FAppStyle::GetBrush(TEXT("Graph.Pin.Disconnected"));
	CachedImg_RefPin_Disconnected = FAppStyle::GetBrush(TEXT("Graph.RefPin.Disconnected"));
	const UEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

	ETypeTreeFilter TypeTreeFilter = ETypeTreeFilter::None;
	TypeTreeFilter |= ETypeTreeFilter::AllowExec;

	TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
	if (RigVMEditorPtr.IsValid())
	{
		CustomPinTypeFilters.Add(MakeShared<FRigVMFunctionArgumentPinTypeSelectorFilter>(RigVMEditorPtr, GraphPtr));
	}

	TAttribute<int32> ArgumentIndex = TAttribute<int32>::CreateSP(this, &FRigVMFunctionArgumentLayout::OnGetArgumentIndex); 
	
	NodeRow
		.NameContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ArgumentNameWidget, SEditableTextBox)
				.Text(this, &FRigVMFunctionArgumentLayout::OnGetArgNameText)
				.OnTextCommitted(this, &FRigVMFunctionArgumentLayout::OnArgNameTextCommitted)
				.ToolTipText(this, &FRigVMFunctionArgumentLayout::OnGetArgToolTipText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(!ShouldPinBeReadOnly())
				.OnVerifyTextChanged_Lambda([&](const FText& InNewText, FText& OutErrorMessage) -> bool
				{
					if (InNewText.IsEmpty())
					{
						OutErrorMessage = LOCTEXT("ArgumentNameEmpty", "Cannot have an argument with an emtpy string name.");
						return false;
					}
					else if (InNewText.ToString().Len() >= NAME_SIZE)
					{
						OutErrorMessage = LOCTEXT("ArgumentNameTooLong", "Name of argument is too long.");
						return false;
					}

					EValidatorResult Result = NameValidator.IsValid(InNewText.ToString(), false);
					OutErrorMessage = INameValidatorInterface::GetErrorText(InNewText.ToString(), Result);	

					return Result == EValidatorResult::Ok || Result == EValidatorResult::ExistingName;
				})
			]
		]
		.ValueContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FRigVMFunctionArgumentLayout::OnGetPinInfo)
				.OnPinTypePreChanged(this, &FRigVMFunctionArgumentLayout::OnPrePinInfoChange)
				.OnPinTypeChanged(this, &FRigVMFunctionArgumentLayout::PinInfoChanged)
				.Schema(Schema)
				.TypeTreeFilter(TypeTreeFilter)
				.bAllowArrays(false)
				.IsEnabled(!ShouldPinBeReadOnly(true))
				.CustomFilters(CustomPinTypeFilters)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(),"BlueprintEditor.CompactVariableTypeSelector")
				.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
				.OnGetMenuContent(this, &FRigVMFunctionArgumentLayout::GetContainerTypeMenuContent)
				.ContentPadding(0.0f)
				.IsEnabled(CanChangeContainerType())
				.Visibility(EVisibility::Visible)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(this, &FRigVMFunctionArgumentLayout::GetContainerTypeIconImage)
					.ColorAndOpacity(this, &FRigVMFunctionArgumentLayout::GetContainerTypeIconColor)
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(10, 0, 0, 0)
			.AutoWidth()
			[
				PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FRigVMFunctionArgumentLayout::OnRemoveClicked), LOCTEXT("FunctionArgDetailsClearTooltip", "Remove this parameter."), !IsPinEditingReadOnly())
			]
		]
		.DragDropHandler(MakeShared<FRigVMFunctionArgumentReorderDragDropHandler>(PinPtr, GraphPtr, WeakRigVMClientHost));
}

void FRigVMFunctionArgumentLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	// we don't show defaults here - we rely on a SRigVMGraphNode widget in the top of the details
}

void FRigVMFunctionArgumentLayout::OnRemoveClicked()
{
	if (PinPtr.IsValid() && WeakRigVMClientHost.IsValid())
	{
		URigVMPin* Pin = PinPtr.Get();
		IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
		{
			if (URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetContainedGraph()))
			{
				TGuardValue<bool> GuardEditGraph(LibraryNode->GetContainedGraph()->bEditable, true);
				Controller->RemoveExposedPin(Pin->GetFName(), true, true);
			}
		}
	}
}

int32 FRigVMFunctionArgumentLayout::OnGetArgumentIndex() const
{
	if (PinPtr.IsValid())
	{
		if(const URigVMPin* Pin = PinPtr.Get())
		{
			return Pin->GetPinIndex();
		}
	}
	return INDEX_NONE;
}

bool FRigVMFunctionArgumentLayout::ShouldPinBeReadOnly(bool bIsEditingPinType/* = false*/) const
{
	return IsPinEditingReadOnly(bIsEditingPinType);
}

bool FRigVMFunctionArgumentLayout::IsPinEditingReadOnly(bool bIsEditingPinType/* = false*/) const
{
	if (PinPtr.IsValid())
	{
		if(const URigVMPin* Pin = PinPtr.Get())
		{
			if(const URigVMNode* Node = Pin->GetNode())
			{
				if(Node->IsA<URigVMAggregateNode>())
				{
					return true;
				}
			}
		}
	}
	return false;
}

FText FRigVMFunctionArgumentLayout::OnGetArgNameText() const
{
	if (PinPtr.IsValid())
	{
		return FText::FromName(PinPtr.Get()->GetFName());
	}
	return FText();
}

FText FRigVMFunctionArgumentLayout::OnGetArgToolTipText() const
{
	return OnGetArgNameText(); // for now since we don't have tooltips
}

void FRigVMFunctionArgumentLayout::OnArgNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		if (!NewText.IsEmpty() && PinPtr.IsValid() && WeakRigVMClientHost.IsValid() && !ShouldPinBeReadOnly())
		{
			URigVMPin* Pin = PinPtr.Get();
			IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
			{
				if (URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetContainedGraph()))
				{
					const FString& NewName = NewText.ToString();
					TGuardValue<bool> GuardEditGraph(LibraryNode->GetContainedGraph()->bEditable, true);
					Controller->RenameExposedPin(Pin->GetFName(), *NewName, true, true);
				}
			}
		}
	}
}

FEdGraphPinType FRigVMFunctionArgumentLayout::OnGetPinInfo() const
{
	if (PinPtr.IsValid())
	{
		return URigVMEdGraphNode::GetPinTypeForModelPin(PinPtr.Get());
	}
	return FEdGraphPinType();
}

void FRigVMFunctionArgumentLayout::PinInfoChanged(const FEdGraphPinType& PinType)
{
	if (PinPtr.IsValid() && WeakRigVMClientHost.IsValid() && FBlueprintEditorUtils::IsPinTypeValid(PinType))
	{
		URigVMPin* Pin = PinPtr.Get();
		IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
		{
			if (URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetContainedGraph()))
			{
				FString CPPType;
				FName CPPTypeObjectName = NAME_None;
				RigVMTypeUtils::CPPTypeFromPinType(PinType, CPPType, CPPTypeObjectName);
				
				bool bSetupUndoRedo = true;
				TGuardValue<bool> GuardEditGraph(LibraryNode->GetContainedGraph()->bEditable, true);
				Controller->ChangeExposedPinType(Pin->GetFName(), CPPType, CPPTypeObjectName, bSetupUndoRedo, false, true);

				// If the controller has identified this as a bulk change, it has not added the actions to the action stack
				// We need to disable the transaction from the UI as well to keep them synced
				if (!bSetupUndoRedo)
				{
					GEditor->CancelTransaction(0);
				}
			}
		}
	}
}

void FRigVMFunctionArgumentLayout::OnPrePinInfoChange(const FEdGraphPinType& PinType)
{
	// not needed for rig vm
}

const TArray<TSharedPtr<FRigVMFunctionArgumentLayout::TContainerType>> FRigVMFunctionArgumentLayout::GetContainerTypes() const
{
		
	static TArray<TSharedPtr<TContainerType>> ContainerTypes;
	if (ContainerTypes.IsEmpty())
	{
		ContainerTypes.Add(MakeShared<TContainerType>(
			EPinContainerType::None,
			false, 
			LOCTEXT("ContainerTypeSingleNoRef", "Single"), 
			LOCTEXT("ContainerTypeSingleNoRefTooltip", "Single Value, Copied into the function"),
			FAppStyle::Get().GetBrush(TEXT("Kismet.VariableList.TypeIcon"))
		));
		ContainerTypes.Add(MakeShared<TContainerType>(
			EPinContainerType::Array,
			false, 
			LOCTEXT("ContainerTypeArrayNoRef", "Array"), 
			LOCTEXT("ContainerTypeArrayNoRefTooltip", "Array Value, Copied into the function"),
			FAppStyle::Get().GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"))
		));
		ContainerTypes.Add(MakeShared<TContainerType>(
			EPinContainerType::None,
			true, 
			LOCTEXT("ContainerTypeSingleRef", "Variable"), 
			LOCTEXT("ContainerTypeSingleRefTooltip", "Single Variable, Referenced into the function"),
			FAppStyle::Get().GetBrush(TEXT("GraphEditor.RefPinIcon"))
		));
		ContainerTypes.Add(MakeShared<TContainerType>(
			EPinContainerType::Array,
			true, 
			LOCTEXT("ContainerTypeArrayRef", "Array Variable"), 
			LOCTEXT("ContainerTypeArrayRefTooltip", "Array Variable, Referenced into the function"),
			FAppStyle::Get().GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"))
		));
	}

	return ContainerTypes;
}

bool FRigVMFunctionArgumentLayout::CanChangeContainerType() const
{
	if (const URigVMPin* Pin = PinPtr.Get())
	{
		if (Pin->IsExecuteContext())
		{
			return false;
		}

		if (const URigVMNode* Node = Pin->GetNode())
		{
			if (Node->IsA<URigVMAggregateNode>())
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

TSharedRef<SWidget> FRigVMFunctionArgumentLayout::GetContainerTypeMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, /*bCloseSelfOnly*/true);

	bool bRefsEnabled = true;
	if (const URigVMPin* Pin = PinPtr.Get())
	{
		if (Pin->IsExecuteContext())
		{
			bRefsEnabled = false;
		}
		else if (Pin->GetDirection() != ERigVMPinDirection::Input)
		{
			bRefsEnabled = false;
		}
		else if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
		{
			if (!Cast<URigVMFunctionLibrary>(LibraryNode->GetGraph()))
			{
				bRefsEnabled = false;
			}
		}
	}

	const TArray<TSharedPtr<TContainerType>>& ContainerTypes = GetContainerTypes();
	for (auto& Type : ContainerTypes)
	{
		check(Type);

		if (Type->Get<1>() && !bRefsEnabled)
		{
			continue;
		}
		
		FUIAction Action;
		Action.ExecuteAction.BindSP(this, &FRigVMFunctionArgumentLayout::OnContainerTypeSelectionChanged, Type);

		TSharedRef<SWidget> Widget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(Type->Get<4>())
				.ToolTipText(Type->Get<3>())
				.ColorAndOpacity(this, &FRigVMFunctionArgumentLayout::GetContainerTypeIconColor)
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(Type->Get<2>())
				.ToolTipText(Type->Get<3>())
			];

		MenuBuilder.AddMenuEntry(Action, Widget);
	}

	return MenuBuilder.MakeWidget();
}

const FSlateBrush* FRigVMFunctionArgumentLayout::GetContainerTypeIconImage() const
{
	if (const URigVMPin* Pin = PinPtr.Get())
	{
		if (!Pin->IsDefinedAsInputVariable())
		{
			if(!Pin->IsArray())
			{
				return GetContainerTypes()[0]->Get<4>();
			}
			return GetContainerTypes()[1]->Get<4>();
		}
		if(!Pin->IsArray())
		{
			return GetContainerTypes()[2]->Get<4>();
		}
		return GetContainerTypes()[3]->Get<4>();
	}
	return GetContainerTypes()[0]->Get<4>();
}

FSlateColor FRigVMFunctionArgumentLayout::GetContainerTypeIconColor() const
{
	if (const URigVMPin* Pin = PinPtr.Get())
	{
		if (!EdGraphSchemaPtr.IsValid())
		{
			if (const URigVMGraph* Graph = Pin->GetGraph())
			{
				if (UClass* EdGraphSchemaClass = Graph->GetSchema()->GetEdGraphSchemaClass())
				{
					EdGraphSchemaPtr = EdGraphSchemaClass->GetDefaultObject<URigVMEdGraphSchema>();
				}
			}
		}

		if (const URigVMEdGraphSchema* EdGraphSchema = EdGraphSchemaPtr.Get())
		{
			const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*Pin->GetCPPType(), Pin->GetCPPTypeObject());
			return EdGraphSchema->GetPinTypeColor(PinType);
		}
	}
	return FSlateColor::UseForeground();
}

void FRigVMFunctionArgumentLayout::OnContainerTypeSelectionChanged(TSharedPtr<TContainerType> InContainerType)
{
	check(InContainerType);
	
	if (PinPtr.IsValid() && WeakRigVMClientHost.IsValid())
	{
		URigVMPin* Pin = PinPtr.Get();
		IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Pin->GetNode()))
		{
			if (URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetContainedGraph()))
			{
				FRigVMControllerCompileBracketScope CompileBracketScope(Controller);

				FEdGraphPinType PinType = URigVMEdGraphNode::GetPinTypeForModelPin(PinPtr.Get());
				const EPinContainerType NewPinContainerType = InContainerType->Get<0>();
				if (NewPinContainerType != PinType.ContainerType)
				{
					PinType.ContainerType = NewPinContainerType;
					PinInfoChanged(PinType);
				}

				(void)Controller->MarkExposedPinAsInputVariable(Pin->GetFName(), InContainerType->Get<1>());
			}
		}
	}
}

FRigVMFunctionArgumentDefaultNode::FRigVMFunctionArgumentDefaultNode(
	const TWeakObjectPtr<URigVMGraph>& InGraph,
	const TWeakInterfacePtr<IRigVMClientHost>& InClientHost
)
	: GraphPtr(InGraph)
	, WeakRigVMClientHost(InClientHost)
	, bIsHandlingModifiedEvent(false)
{
	if (GraphPtr.IsValid() && WeakRigVMClientHost.IsValid())
	{
		IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
		RigVMClientHost->OnModified().AddRaw(this, &FRigVMFunctionArgumentDefaultNode::HandleModifiedEvent, false);

		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(GraphPtr->GetOuter()))
		{
			if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(RigVMClientHost->GetEditorObjectForRigVMGraph(LibraryNode->GetGraph())))
			{
				EdGraphOuterPtr = RigGraph;
				GraphChangedDelegateHandle = RigGraph->AddOnGraphChangedHandler(
					FOnGraphChanged::FDelegate::CreateRaw(this, &FRigVMFunctionArgumentDefaultNode::OnGraphChanged)
				);
			}
		}	

	}
}

FRigVMFunctionArgumentDefaultNode::~FRigVMFunctionArgumentDefaultNode()
{
	if (WeakRigVMClientHost.IsValid())
	{
		WeakRigVMClientHost->OnModified().RemoveAll(this);
	}
	
	if (EdGraphOuterPtr.IsValid())
	{
		if (GraphChangedDelegateHandle.IsValid())
		{
			EdGraphOuterPtr->RemoveOnGraphChangedHandler(GraphChangedDelegateHandle);
		}		
	}
}

void FRigVMFunctionArgumentDefaultNode::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (!GraphPtr.IsValid() || !WeakRigVMClientHost.IsValid())
	{
		return;
	}

	IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
	URigVMGraph* Graph = GraphPtr.Get();
	URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Graph->GetOuter());
	if (LibraryNode == nullptr)
	{
		return;
	}
	
	if (IRigVMAssetInterface* AssetInterfaceOuter = LibraryNode->GetImplementingOuter<IRigVMAssetInterface>())
	{
		MinimalEnvironment = MakeShared<FRigVMMinimalEnvironment>(AssetInterfaceOuter);
	}
	else if (IRigVMClientHost* ClientHostOuter = LibraryNode->GetImplementingOuter<IRigVMClientHost>())
	{
		MinimalEnvironment = MakeShared<FRigVMMinimalEnvironment>(ClientHostOuter, nullptr);
	}
	else
	{
		MinimalEnvironment = MakeShared<FRigVMMinimalEnvironment>(nullptr);
	}
	
	if (LibraryNode->GetTypedOuter<URigVMFunctionLibrary>())
	{
		MinimalEnvironment->SetFunctionReference(LibraryNode, LibraryNode->GetName());
	}
	else
	{
		MinimalEnvironment->SetEmptyCollapseNode(Cast<URigVMCollapseNode>(LibraryNode), LibraryNode->GetName());
	}

	if (URigVMGraph* Model = MinimalEnvironment->GetModel())
	{
		Model->OnModified().AddSP(this, &FRigVMFunctionArgumentDefaultNode::HandleModifiedEvent, true);
	}

	ChildrenBuilder.AddCustomRow(FText::GetEmpty())
	.WholeRowContent()
	.MaxDesiredWidth(980.f)
	[
		SAssignNew(PreviewWidget, SRigVMNodePreviewWidget)
		.Environment(MinimalEnvironment)
		.NodeEnabled(true)
	];
}

void FRigVMFunctionArgumentDefaultNode::OnGraphChanged(const FEdGraphEditAction& InAction)
{
	if (GraphPtr.IsValid() && WeakRigVMClientHost.IsValid())
	{
		OnRebuildChildren.ExecuteIfBound();
	}
}

void FRigVMFunctionArgumentDefaultNode::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject, bool bFromTargetToSource)
{
	if (bIsHandlingModifiedEvent)
	{
		return;
	}
	const TGuardValue<bool> GuardReEntry(bIsHandlingModifiedEvent, true);
	
	if (!MinimalEnvironment)
	{
		return;
	}

	if (!GraphPtr.IsValid())
	{
		return;
	}

	URigVMLibraryNode* SourceNode = Cast<URigVMLibraryNode>(GraphPtr->GetOuter());
	if (!SourceNode)
	{
		return;
	}

	URigVMLibraryNode* TargetNode = Cast<URigVMLibraryNode>(MinimalEnvironment->GetNode());
	if (!TargetNode)
	{
		return;
	}

	if (!WeakRigVMClientHost.IsValid())
	{
		return;
	}

	// we run the same handler on the notifications coming from the
	// minimal preview environment - as well as the graph in the asset.
	// we sync the pin states between these two environments.
	// reentry is guarded against at the top.
	URigVMController* Controller = nullptr;
	if (bFromTargetToSource)
	{
		if (TargetNode->GetGraph() != InGraph)
		{
			return;
		}
		Controller = WeakRigVMClientHost->GetController(SourceNode->GetGraph());
	}
	else
	{
		if (SourceNode->GetGraph() != InGraph)
		{
			return;
		}
		Controller = MinimalEnvironment->GetController();
	}
	if (!Controller)
	{
		return;
	}

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::PinDefaultValueChanged:
		case ERigVMGraphNotifType::PinExpansionChanged:
		case ERigVMGraphNotifType::PinArraySizeChanged:
		{
			if (const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
			{
				if (Pin->GetNode() != (bFromTargetToSource ? TargetNode : SourceNode))
				{
					return;
				}
				
				bool bSetupUndoRedo = !GIsTransacting;
				TSharedPtr<FScopedTransaction> Transaction;
				if (bSetupUndoRedo)
				{
					Transaction = MakeShared<FScopedTransaction>(LOCTEXT("UpdateFunctionArgument", "Update Function Argument"));
				}
				const FString SegmentPath = Pin->GetSegmentPath(true);
				const URigVMNode* OtherNode = Pin->GetNode() == TargetNode ? SourceNode : TargetNode;
				const FString PinPath = URigVMPin::JoinPinPath(OtherNode->GetName(), SegmentPath);
				(void)Controller->SetPinExpansion(PinPath, Pin->IsExpanded(), bSetupUndoRedo);
				(void)Controller->SetPinDefaultValue(PinPath, Pin->GetDefaultValue(), true, bSetupUndoRedo);
			}
			break;
		}
		case ERigVMGraphNotifType::PinAdded:
		case ERigVMGraphNotifType::PinIndexChanged:
		case ERigVMGraphNotifType::PinRemoved:
		case ERigVMGraphNotifType::PinRenamed:
		case ERigVMGraphNotifType::PinTypeChanged:
		{
			if (!bFromTargetToSource)
			{
				if (TargetNode->IsA<URigVMFunctionReferenceNode>())
				{
					return;
				}
				MinimalEnvironment->SynchronizeCollapseNode(Cast<URigVMCollapseNode>(SourceNode));
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

TSharedRef<IDetailCustomization> FRigVMGraphDetailCustomization::MakeInstance(TSharedPtr<IRigVMEditor> InEditor, const UClass* InExpectedBlueprintClass)
{
	const TArray<UObject*>* Objects = (InEditor.IsValid() ? InEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		UObject* EditorAssetObject = (*Objects)[0];
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = EditorAssetObject)
		{
			EditorAssetObject = RuntimeAsset->GetEditorOnlyData();
		}
		if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = EditorAssetObject)
		{
			if(RigVMBlueprint.GetObject()->GetClass() == InExpectedBlueprintClass)
			{
				return MakeShareable(new FRigVMGraphDetailCustomization(InEditor, RigVMBlueprint));
			}
		}
	}

	return MakeShareable(new FRigVMGraphDetailCustomization((TSharedPtr<IRigVMEditor>)nullptr, FRigVMEditorAssetInterfacePtr(nullptr)));
}

#if WITH_RIGVMLEGACYEDITOR
TSharedPtr<IDetailCustomization> FRigVMGraphDetailCustomization::MakeLegacyInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor, const UClass* InExpectedBlueprintClass)
{
	const TArray<UObject*>* Objects = (InBlueprintEditor.IsValid() ? InBlueprintEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		if ((*Objects)[0]->Implements<URigVMEditorAssetInterface>())
		{
			FRigVMEditorAssetInterfacePtr RigVMBlueprint = (*Objects)[0];
			if(RigVMBlueprint->GetObject()->GetClass() == InExpectedBlueprintClass)
			{
				return MakeShareable(new FRigVMGraphDetailCustomization(InBlueprintEditor, RigVMBlueprint));
			}
		}
	}

	return MakeShareable(new FRigVMGraphDetailCustomization((TSharedPtr<IRigVMEditor>)nullptr, FRigVMEditorAssetInterfacePtr(nullptr)));
}

FRigVMGraphDetailCustomization::FRigVMGraphDetailCustomization(TSharedPtr<IBlueprintEditor> RigVMigEditor, FRigVMEditorAssetInterfacePtr RigVMBlueprint)
	: RigVMBlueprintPtr(RigVMBlueprint)
	, RigVMGraphDetailCustomizationImpl(MakeShared<FRigVMGraphDetailCustomizationImpl>())
{
	RigVMEditorPtr = StaticCastSharedPtr<FRigVMLegacyEditor>(RigVMigEditor);
}
#endif

FRigVMGraphDetailCustomization::FRigVMGraphDetailCustomization(TSharedPtr<IRigVMEditor> RigVMigEditor, FRigVMEditorAssetInterfacePtr RigVMBlueprint)
	: RigVMEditorPtr(RigVMigEditor)
	, RigVMBlueprintPtr(RigVMBlueprint)
	, RigVMGraphDetailCustomizationImpl(MakeShared<FRigVMGraphDetailCustomizationImpl>())
{
}

void FRigVMGraphDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	GraphPtr = CastChecked<URigVMEdGraph>(Objects[0].Get());
	URigVMEdGraph* Graph = GraphPtr.Get();

	FRigVMEditorAssetInterfacePtr Blueprint = RigVMBlueprintPtr.GetObject();
	URigVMGraph* Model = nullptr;
	URigVMController* Controller = nullptr;

	if (Blueprint)
	{
		Model = Blueprint->GetModel(Graph);
		Controller = Blueprint->GetController(Model);
	}

	if (Blueprint == nullptr || Model == nullptr || Controller == nullptr)
	{
		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Graph", LOCTEXT("FunctionDetailsGraph", "Graph"));
		Category.AddCustomRow(FText::GetEmpty())
		[
			SNew(STextBlock)
			.Text(LOCTEXT("GraphPresentButNotEditable", "Graph is not editable."))
		];
		return;
	}
	RigVMGraphDetailCustomizationImpl->CustomizeDetails(DetailLayout, Model, Controller, TScriptInterface<IRigVMClientHost>(Blueprint->GetObject()).GetInterface(), RigVMEditorPtr);
}

void FRigVMGraphDetailCustomizationImpl::CustomizeDetails(IDetailLayoutBuilder& DetailLayout,
	URigVMGraph* InModel,
	URigVMController* InController,
	IRigVMClientHost* InRigVMClientHost,
	TWeakPtr<IRigVMEditor> InEditor)
{
	WeakModel = InModel;
	WeakController = InController;
	RigVMClientHost = InRigVMClientHost;
	RigVMEditorPtr = InEditor;

	bIsPickingColor = false;

	URigVMGraph* Model = WeakModel.Get();

	if (Model != nullptr && Model->IsTopLevelGraph())
	{
		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Graph", LOCTEXT("FunctionDetailsGraph", "Graph"));
		Category.AddCustomRow(FText::GetEmpty())
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GraphIsTopLevelGraph", "Top-level Graphs are not editable."))
			];
		return;
	}

	bool bIsFunction = false;
	bool bIsAggregate = false;
	if (Model)
	{
		if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
		{
			bIsFunction = LibraryNode->GetGraph()->IsA<URigVMFunctionLibrary>();
			bIsAggregate = LibraryNode->IsA<URigVMAggregateNode>();
		}
	}

	IDetailCategoryBuilder& InputsCategory = DetailLayout.EditCategory("Inputs", LOCTEXT("FunctionDetailsInputs", "Inputs"));
	TSharedRef<FRigVMFunctionArgumentGroupLayout> InputArgumentGroup = MakeShareable<FRigVMFunctionArgumentGroupLayout>(new FRigVMFunctionArgumentGroupLayout(
		Model,
		RigVMClientHost.Get(),
		RigVMEditorPtr,
		true));
	InputsCategory.AddCustomBuilder(InputArgumentGroup);

	if(!bIsAggregate)
	{
		TSharedRef<SHorizontalBox> InputsHeaderContentWidget = SNew(SHorizontalBox);

		InputsHeaderContentWidget->AddSlot()
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(FMargin(10.f, 0))
			.OnClicked(this, &FRigVMGraphDetailCustomizationImpl::OnAddNewInputClicked)
			.Visibility(this, &FRigVMGraphDetailCustomizationImpl::GetAddNewInputOutputVisibility)
			.HAlign(HAlign_Right)
			.ToolTipText(LOCTEXT("FunctionNewInputArgTooltip", "Create a new input argument"))
			.VAlign(VAlign_Center)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("FunctionNewInputArg")))
			.IsEnabled(this, &FRigVMGraphDetailCustomizationImpl::IsAddNewInputOutputEnabled)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
		InputsCategory.HeaderContent(InputsHeaderContentWidget);
	}
	
	IDetailCategoryBuilder& OutputsCategory = DetailLayout.EditCategory("Outputs", LOCTEXT("FunctionDetailsOutputs", "Outputs"));
	TSharedRef<FRigVMFunctionArgumentGroupLayout> OutputArgumentGroup = MakeShareable(new FRigVMFunctionArgumentGroupLayout(
		Model,
		RigVMClientHost,
		RigVMEditorPtr,
		false));
	OutputsCategory.AddCustomBuilder(OutputArgumentGroup);

	if(!bIsAggregate)
	{
		TSharedRef<SHorizontalBox> OutputsHeaderContentWidget = SNew(SHorizontalBox);

		OutputsHeaderContentWidget->AddSlot()
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(FMargin(10.f, 0))
			.OnClicked(this, &FRigVMGraphDetailCustomizationImpl::OnAddNewOutputClicked)
			.Visibility(this, &FRigVMGraphDetailCustomizationImpl::GetAddNewInputOutputVisibility)
			.HAlign(HAlign_Right)
			.ToolTipText(LOCTEXT("FunctionNewOutputArgTooltip", "Create a new output argument"))
			.VAlign(VAlign_Center)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("FunctionNewOutputArg")))
			.IsEnabled(this, &FRigVMGraphDetailCustomizationImpl::IsAddNewInputOutputEnabled)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
		OutputsCategory.HeaderContent(OutputsHeaderContentWidget);
	}
	
	IDetailCategoryBuilder& SettingsCategory = DetailLayout.EditCategory("NodeSettings", LOCTEXT("FunctionDetailsNodeSettings", "Node Settings"));

	if(bIsFunction)
	{
		// node category
		SettingsCategory.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Category")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigVMGraphDetailCustomizationImpl::GetNodeCategory)
			.OnTextCommitted(this, &FRigVMGraphDetailCustomizationImpl::SetNodeCategory)
			.OnVerifyTextChanged_Lambda([&](const FText& InNewText, FText& OutErrorMessage) -> bool
			{
				const FText NewText = FEditorCategoryUtils::GetCategoryDisplayString(InNewText);
				if (NewText.ToString().Len() >= NAME_SIZE)
				{
					OutErrorMessage = LOCTEXT("CategoryTooLong", "Name of category is too long.");
					return false;
				}
				
				if (RigVMClientHost.IsValid())
				{
					if (NewText.EqualTo(FText::FromString(RigVMClientHost->GetAssetName())))
					{
						OutErrorMessage = LOCTEXT("CategoryEqualsBlueprintName", "Cannot add a category with the same name as the owner asset.");
						return false;
					}
				}
				return true;
			})
		];

		// node keywords
		SettingsCategory.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Keywords")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SEditableTextBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigVMGraphDetailCustomizationImpl::GetNodeKeywords)
			.OnTextCommitted(this, &FRigVMGraphDetailCustomizationImpl::SetNodeKeywords)
		];

		// description
		SettingsCategory.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Description")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SMultiLineEditableText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FRigVMGraphDetailCustomizationImpl::GetNodeDescription)
			.OnTextCommitted(this, &FRigVMGraphDetailCustomizationImpl::SetNodeDescription)
		];

		if (AccessSpecifierStrings.IsEmpty())
		{
			AccessSpecifierStrings.Add(TSharedPtr<FRigVMStringWithTag>(new FRigVMStringWithTag(TEXT("Public"))));
			AccessSpecifierStrings.Add(TSharedPtr<FRigVMStringWithTag>(new FRigVMStringWithTag(TEXT("Private"))));
		}

		// access specifier
		SettingsCategory.AddCustomRow( LOCTEXT( "AccessSpecifier", "Access Specifier" ) )
        .NameContent()
        [
            SNew(STextBlock)
                .Text( LOCTEXT( "AccessSpecifier", "Access Specifier" ) )
                .Font( IDetailLayoutBuilder::GetDetailFont() )
        ]
        .ValueContent()
        [
            SNew(SComboButton)
            .ContentPadding(0.f)
            .ButtonContent()
            [
                SNew(STextBlock)
                    .Text(this, &FRigVMGraphDetailCustomizationImpl::GetCurrentAccessSpecifierName)
                    .Font( IDetailLayoutBuilder::GetDetailFont() )
            ]
            .MenuContent()
            [
                SNew(SListView<TSharedPtr<FRigVMStringWithTag> >)
                    .ListItemsSource( &AccessSpecifierStrings )
                    .OnGenerateRow(this, &FRigVMGraphDetailCustomizationImpl::HandleGenerateRowAccessSpecifier)
                    .OnSelectionChanged(this, &FRigVMGraphDetailCustomizationImpl::OnAccessSpecifierSelected)
            ]
        ];

		// pure function
		SettingsCategory.AddCustomRow(LOCTEXT("Pure", "Pure"))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("Pure", "Pure"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
				.IsChecked(this, &FRigVMGraphDetailCustomizationImpl::GetFunctionPureCheckState)
				.OnCheckStateChanged(this, &FRigVMGraphDetailCustomizationImpl::OnFunctionPureCheckStateChanged)
				.ToolTipText(LOCTEXT("PureTooltip", "Pure functions have no execute pins and cannot have side effects."))
		];

		// variant
		if(CVarRigVMEnableVariants.GetValueOnAnyThread())
		{
			FRigVMVariantWidgetContext VariantContext;
			if(const URigVMFunctionLibrary* FunctionLbirary = Model->GetTypedOuter<URigVMFunctionLibrary>())
			{
				VariantContext.ParentPath = FunctionLbirary->GetPathName();
			}
				
			SettingsCategory.AddCustomRow(FText::GetEmpty())
			.OverrideResetToDefault(FResetToDefaultOverride::Hide())
			.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
			{
				return IsValidFunction() ? EVisibility::Visible : EVisibility::Collapsed;
			}))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Variant")))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SRigVMVariantWidget)
				.Context(VariantContext)
				.Variant(this, &FRigVMGraphDetailCustomizationImpl::GetVariant)
				.SubjectVariantRef(this, &FRigVMGraphDetailCustomizationImpl::GetSubjectVariantRef)
				.VariantRefs(this, &FRigVMGraphDetailCustomizationImpl::GetVariantRefs)
				.OnVariantChanged(this, &FRigVMGraphDetailCustomizationImpl::OnVariantChanged)
				.OnBrowseVariantRef(this, &FRigVMGraphDetailCustomizationImpl::OnBrowseVariantRef)
				.OnGetTags(this, &FRigVMGraphDetailCustomizationImpl::OnGetAssignedTags)
				.OnAddTag(this, &FRigVMGraphDetailCustomizationImpl::OnAddAssignedTag)
				.OnRemoveTag(this, &FRigVMGraphDetailCustomizationImpl::OnRemoveAssignedTag)
				.CanAddTags(true)
				.EnableTagContextMenu(true)
			];
		}
	}

	// node color
	if(!bIsAggregate)
	{
		SettingsCategory.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(TEXT("Color")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Menu.Button")
			.OnClicked(this, &FRigVMGraphDetailCustomizationImpl::OnNodeColorClicked)
			[
				SAssignNew(ColorBlock, SColorBlock)
				.Color(this, &FRigVMGraphDetailCustomizationImpl::GetNodeColor)
				.Size(FVector2D(77, 16))
			]
		];
	}

	if(Model)
	{
		if(const URigVMSchema* Schema = Model->GetSchema())
		{
			if(Schema->SupportsNodeLayouts(Model))
			{
				SettingsCategory.AddCustomRow(FText::GetEmpty())
				.OverrideResetToDefault(FResetToDefaultOverride::Hide())
				.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
				{
					return IsValidFunction() ? EVisibility::Visible : EVisibility::Collapsed;
				}))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Layout")))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				[
					SNew(SRigVMNodeLayoutWidget)
					.OnGetUncategorizedPins(this, &FRigVMGraphDetailCustomizationImpl::GetUncategorizedPins)
					.OnGetCategories(this, &FRigVMGraphDetailCustomizationImpl::GetPinCategories)
					.OnGetElementCategory(this, &FRigVMGraphDetailCustomizationImpl::GetPinCategory)
					.OnGetElementIndexInCategory(this, &FRigVMGraphDetailCustomizationImpl::GetPinIndexInCategory)
					.OnGetElementLabel(this, &FRigVMGraphDetailCustomizationImpl::GetPinLabel)
					.OnGetElementColor(this, &FRigVMGraphDetailCustomizationImpl::GetPinColor)
					.OnGetElementIcon(this, &FRigVMGraphDetailCustomizationImpl::GetPinIcon)
					.OnCategoryAdded(this, &FRigVMGraphDetailCustomizationImpl::HandleCategoryAdded)
					.OnCategoryRemoved(this, &FRigVMGraphDetailCustomizationImpl::HandleCategoryRemoved)
					.OnCategoryRenamed(this, &FRigVMGraphDetailCustomizationImpl::HandleCategoryRenamed)
					.OnElementCategoryChanged(this, &FRigVMGraphDetailCustomizationImpl::HandlePinCategoryChanged)
					.OnElementLabelChanged(this, &FRigVMGraphDetailCustomizationImpl::HandlePinLabelChanged)
					.OnElementIndexInCategoryChanged(this, &FRigVMGraphDetailCustomizationImpl::HandlePinIndexInCategoryChanged)
					.OnValidateCategoryName(this, &FRigVMGraphDetailCustomizationImpl::HandleValidateCategoryName)
					.OnValidateElementName(this, &FRigVMGraphDetailCustomizationImpl::HandleValidatePinDisplayName)
					.OnGetStructuralHash(this, &FRigVMGraphDetailCustomizationImpl::GetNodeLayoutHash)
				];
			}
		}
	}

	IDetailCategoryBuilder& DefaultsCategory = DetailLayout.EditCategory("NodeDefaults", LOCTEXT("FunctionDetailsNodeDefaults", "Node Defaults"));
	TSharedRef<FRigVMFunctionArgumentDefaultNode> DefaultsArgumentNode = MakeShareable(new FRigVMFunctionArgumentDefaultNode(
		Model,
		RigVMClientHost));
	DefaultsCategory.AddCustomBuilder(DefaultsArgumentNode);
}

bool FRigVMGraphDetailCustomizationImpl::IsAddNewInputOutputEnabled() const
{
	return true;
}

EVisibility FRigVMGraphDetailCustomizationImpl::GetAddNewInputOutputVisibility() const
{
	return EVisibility::Visible;
}

FReply FRigVMGraphDetailCustomizationImpl::OnAddNewInputClicked()
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMController* Controller = WeakController.Get())
		{
			FName ArgumentName = TEXT("Argument");
			FString CPPType = TEXT("bool");
			FName CPPTypeObjectPath = NAME_None;
			FString DefaultValue = TEXT("False");

			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(WeakModel->GetOuter()))
			{
				if (LibraryNode->GetPins().Num() > 0)
				{
					URigVMPin* LastPin = LibraryNode->GetPins().Last();
					if (!LastPin->IsExecuteContext())
					{
						// strip off any tailing number from for example Argument_2
						FString StrippedArgumentName = LastPin->GetName();
						FString LastChars = StrippedArgumentName.Right(1);
						StrippedArgumentName.LeftChopInline(1);
						while(LastChars.IsNumeric() && !StrippedArgumentName.IsEmpty())
						{
							LastChars = StrippedArgumentName.Right(1);
							StrippedArgumentName.LeftChopInline(1);

							if(LastChars.StartsWith(TEXT("_")))
							{
								LastChars.Reset();
								break;
							}
						}

						StrippedArgumentName = StrippedArgumentName + LastChars;
						if(!StrippedArgumentName.IsEmpty())
						{
							ArgumentName = *StrippedArgumentName;
						}

						RigVMTypeUtils::CPPTypeFromPin(LastPin, CPPType, CPPTypeObjectPath);						
						DefaultValue = LastPin->GetDefaultValue();
					}
				}

				TGuardValue<bool> GuardEditGraph(LibraryNode->GetContainedGraph()->bEditable, true);
				Controller->AddExposedPin(ArgumentName, ERigVMPinDirection::Input, CPPType, CPPTypeObjectPath, DefaultValue, true, true);
			}
		}
	}
	return FReply::Unhandled();
}

FReply FRigVMGraphDetailCustomizationImpl::OnAddNewOutputClicked()
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMController* Controller = WeakController.Get())
		{
			FName ArgumentName = TEXT("Argument");
			FString CPPType = TEXT("bool");
			FName CPPTypeObjectPath = NAME_None;
			FString DefaultValue = TEXT("False");
			// todo: base decisions on types on last argument

			if (URigVMGraph* Model = WeakModel.Get())
			{
				TGuardValue<bool> GuardEditGraph(Model->bEditable, true);
				Controller->AddExposedPin(ArgumentName, ERigVMPinDirection::Output, CPPType, CPPTypeObjectPath, DefaultValue, true, true);
			}
		}
	}
	return FReply::Unhandled();
}

FText FRigVMGraphDetailCustomizationImpl::GetNodeCategory() const
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				return FText::FromString(OuterNode->GetNodeCategory());
			}
		}
	}

	return FText();
}

void FRigVMGraphDetailCustomizationImpl::SetNodeCategory(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				if (URigVMController* Controller = RigVMClientHost->GetOrCreateController(OuterNode->GetGraph()))
				{
					Controller->SetNodeCategory(OuterNode, InNewText.ToString(), true, false, true);
				}
			}
		}
	}
}

FText FRigVMGraphDetailCustomizationImpl::GetNodeKeywords() const
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				return FText::FromString(OuterNode->GetNodeKeywords());
			}
		}
	}

	return FText();
}

void FRigVMGraphDetailCustomizationImpl::SetNodeKeywords(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				if (URigVMController* Controller = RigVMClientHost->GetOrCreateController(OuterNode->GetGraph()))
				{
					Controller->SetNodeKeywords(OuterNode, InNewText.ToString(), true, false, true);
				}
			}
		}
	}
}

FText FRigVMGraphDetailCustomizationImpl::GetNodeDescription() const
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				return FText::FromString(OuterNode->GetNodeDescription());
			}
		}
	}

	return FText();
}

void FRigVMGraphDetailCustomizationImpl::SetNodeDescription(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				if (URigVMController* Controller = RigVMClientHost->GetOrCreateController(OuterNode->GetGraph()))
				{
					Controller->SetNodeDescription(OuterNode, InNewText.ToString(), true, false, true);
				}
			}
		}
	}
}

FLinearColor FRigVMGraphDetailCustomizationImpl::GetNodeColor() const
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				return OuterNode->GetNodeColor();
			}
		}
	}
	return FLinearColor::White;
}

void FRigVMGraphDetailCustomizationImpl::SetNodeColor(FLinearColor InColor, bool bSetupUndoRedo)
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMCollapseNode* OuterNode = Cast<URigVMCollapseNode>(Model->GetOuter()))
			{
				if (URigVMController* Controller = RigVMClientHost->GetOrCreateController(OuterNode->GetGraph()))
				{
					Controller->SetNodeColor(OuterNode, InColor, bSetupUndoRedo, bIsPickingColor, true);
				}
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::OnNodeColorBegin()
{
	bIsPickingColor = true;
}
void FRigVMGraphDetailCustomizationImpl::OnNodeColorEnd()
{
	bIsPickingColor = false;
}

void FRigVMGraphDetailCustomizationImpl::OnNodeColorCancelled(FLinearColor OriginalColor)
{
	SetNodeColor(OriginalColor, true);
}

FReply FRigVMGraphDetailCustomizationImpl::OnNodeColorClicked()
{
	FColorPickerArgs PickerArgs = FColorPickerArgs(GetNodeColor(), FOnLinearColorValueChanged::CreateSP(this, &FRigVMGraphDetailCustomizationImpl::SetNodeColor, true));
	PickerArgs.ParentWidget = ColorBlock;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = false;
	PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &FRigVMGraphDetailCustomizationImpl::OnNodeColorBegin);
	PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &FRigVMGraphDetailCustomizationImpl::OnNodeColorEnd);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FRigVMGraphDetailCustomizationImpl::OnNodeColorCancelled);
	OpenColorPicker(PickerArgs);
	return FReply::Handled();
}

TArray<TSharedPtr<FRigVMStringWithTag>> FRigVMGraphDetailCustomizationImpl::AccessSpecifierStrings;

FText FRigVMGraphDetailCustomizationImpl::GetCurrentAccessSpecifierName() const
{
	if (WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* RigVMGraph = WeakModel.Get())
		{
			if (URigVMLibraryNode* LibraryNode = RigVMGraph->GetTypedOuter<URigVMLibraryNode>())
			{
				if(RigVMClientHost->GetLocalFunctionLibrary()->IsFunctionPublic(LibraryNode->GetFName()))
				{
					return FText::FromString(AccessSpecifierStrings[0]->GetString()); // public
				}
			}
		}
	}

	return FText::FromString(AccessSpecifierStrings[1]->GetString()); // private
}

void FRigVMGraphDetailCustomizationImpl::OnAccessSpecifierSelected(TSharedPtr<FRigVMStringWithTag> SpecifierName, ESelectInfo::Type SelectInfo)
{
	if(WeakModel.IsValid() && WeakController.IsValid())
	{
		if (URigVMGraph* RigVMGraph = WeakModel.Get())
		{
			if (URigVMLibraryNode* LibraryNode = RigVMGraph->GetTypedOuter<URigVMLibraryNode>())
			{
				if(SpecifierName->Equals(TEXT("Private")))
				{
					RigVMClientHost->MarkFunctionPublic(LibraryNode->GetFName(), false);
				}
				else
				{
					RigVMClientHost->MarkFunctionPublic(LibraryNode->GetFName(), true);
				}
			}
		}
	}
}

TSharedRef<ITableRow> FRigVMGraphDetailCustomizationImpl::HandleGenerateRowAccessSpecifier(TSharedPtr<FRigVMStringWithTag> SpecifierName, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow< TSharedPtr<FRigVMStringWithTag> >, OwnerTable)
        .Content()
        [
            SNew( STextBlock ) 
                .Text(FText::FromString(SpecifierName->GetString()) )
        ];
}

ECheckBoxState FRigVMGraphDetailCustomizationImpl::GetFunctionPureCheckState() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (URigVMGraph* RigVMGraph = WeakModel.Get())
		{
			if (URigVMCollapseNode* CollapseNode = RigVMGraph->GetTypedOuter<URigVMCollapseNode>())
			{
				return CollapseNode->IsPure() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMGraphDetailCustomizationImpl::OnFunctionPureCheckStateChanged(ECheckBoxState NewState)
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (URigVMGraph* RigVMGraph = WeakModel.Get())
		{
			if (URigVMLibraryNode* LibraryNode = RigVMGraph->GetTypedOuter<URigVMLibraryNode>())
			{
				if (URigVMGraph* FunctionLibrary = RigVMClientHost->GetLocalFunctionLibrary())
				{
					if (URigVMController* FunctionLibraryController = RigVMClientHost->GetOrCreateController(FunctionLibrary))
					{
						const bool bIsPure = (NewState == ECheckBoxState::Checked);
						FunctionLibraryController->SetFunctionIsPure(LibraryNode->GetFName(), bIsPure);
					}
				}
			}
		}
	}
}

bool FRigVMGraphDetailCustomizationImpl::IsValidFunction() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
			{
				return LibraryNode->GetFunctionHeader(RigVMClientHost->GetRigVMGraphFunctionHost().GetInterface()).IsValid();
			}
		}
	}
	return false;
}

FRigVMVariant FRigVMGraphDetailCustomizationImpl::GetVariant() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
			{
				return LibraryNode->GetFunctionHeader(RigVMClientHost->GetRigVMGraphFunctionHost().GetInterface()).Variant;
			}
		}
	}
	return FRigVMVariant();
}

FRigVMVariantRef FRigVMGraphDetailCustomizationImpl::GetSubjectVariantRef() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
			{
				return FRigVMVariantRef(LibraryNode, GetVariant());
			}
		}
	}
	return FRigVMVariantRef();
}

TArray<FRigVMVariantRef> FRigVMGraphDetailCustomizationImpl::GetVariantRefs() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
			{
				const FRigVMGraphFunctionHeader Header = LibraryNode->GetFunctionHeader(RigVMClientHost->GetRigVMGraphFunctionHost().GetInterface());
				return Header.LibraryPointer.GetVariants(false);
			}
		}
	}
	return TArray<FRigVMVariantRef>();
}

void FRigVMGraphDetailCustomizationImpl::OnVariantChanged(const FRigVMVariant& InVariant)
{
	// todo: update the function's variant info
}

void FRigVMGraphDetailCustomizationImpl::OnBrowseVariantRef(const FRigVMVariantRef& InVariantRef)
{
	const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(InVariantRef.ObjectPath);
	if(Header.IsValid())
	{
		if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Header.LibraryPointer.GetNodeSoftPath().TryLoad()))
		{
			if (FRigVMEditorAssetInterfacePtr EditorAsset = IRigVMEditorAssetInterface::GetInterfaceOuter(LibraryNode))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorAsset->GetAssetObjectForEditor());
			
				if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(EditorAsset.GetObject(), true))
				{
					if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
					{
						RigVMEditor->HandleJumpToHyperlink(LibraryNode);
					}
				}
			}
		}
	}
	else
	{
		const FAssetData AssetData = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(InVariantRef.ObjectPath.ToString(), true);
		if(AssetData.IsValid())
		{
			const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			ContentBrowserModule.Get().SyncBrowserToAssets({AssetData});
		}
	}
}

TArray<FRigVMTag> FRigVMGraphDetailCustomizationImpl::OnGetAssignedTags() const
{
	return GetVariant().Tags;
}

void FRigVMGraphDetailCustomizationImpl::OnAddAssignedTag(const FName& InTagName)
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMGraph* FunctionLibrary = RigVMClientHost->GetLocalFunctionLibrary())
			{
				if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
				{
					const FString& FunctionName = LibraryNode->GetFunctionHeader().LibraryPointer.GetFunctionName();
					URigVMController* FunctionLibraryController = RigVMClientHost->GetOrCreateController(FunctionLibrary);
					FunctionLibraryController->AddDefaultTagToFunctionVariant(*FunctionName, InTagName);
				}
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::OnRemoveAssignedTag(const FName& InTagName)
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (URigVMGraph* FunctionLibrary = RigVMClientHost->GetLocalFunctionLibrary())
			{
				if (const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
				{
					const FString& FunctionName = LibraryNode->GetFunctionHeader().LibraryPointer.GetFunctionName();
					URigVMController* FunctionLibraryController = RigVMClientHost->GetOrCreateController(FunctionLibrary);
					FunctionLibraryController->RemoveTagFromFunctionVariant(*FunctionName, InTagName);
				}
			}
		}
	}
}

URigVMLibraryNode* FRigVMGraphDetailCustomizationImpl::GetLibraryNode() const
{
	if (WeakModel.IsValid() && RigVMClientHost.IsValid())
	{
		if (const URigVMGraph* Model = WeakModel.Get())
		{
			if (const URigVMGraph* FunctionLibrary = RigVMClientHost->GetLocalFunctionLibrary())
			{
				if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Model->GetOuter()))
				{
					if(LibraryNode->GetGraph() == FunctionLibrary)
					{
						return LibraryNode;
					}
				}
			}
		}
	}
	return nullptr;
}

URigVMNode* FRigVMGraphDetailCustomizationImpl::GetNodeForLayout() const
{
	return GetLibraryNode();
}

const FRigVMNodeLayout* FRigVMGraphDetailCustomizationImpl::GetNodeLayout() const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		CachedNodeLayout = Node->GetNodeLayout(true);
		return &CachedNodeLayout.GetValue();
	}
	return nullptr;
}

TArray<FString> FRigVMGraphDetailCustomizationImpl::GetUncategorizedPins() const
{
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		TArray<FString> PinPaths;
		const TArray<URigVMPin*> AllPins = Node->GetAllPinsRecursively();
		PinPaths.Reserve(AllPins.Num());
		for(const URigVMPin* Pin : AllPins)
		{
			if(Pin->IsExecuteContext())
			{
				continue;
			}
			if(Pin->GetDirection() != ERigVMPinDirection::Input &&
				Pin->GetDirection() != ERigVMPinDirection::Visible)
			{
				continue;
			}
			if(!Pin->GetCategory().IsEmpty())
			{
				continue;
			}
			PinPaths.Add(Pin->GetSegmentPath(true));
		}
		return PinPaths;
	}
	return TArray<FString>();
}

TArray<FRigVMPinCategory> FRigVMGraphDetailCustomizationImpl::GetPinCategories() const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		return NodeLayout->Categories;
	}
	return TArray<FRigVMPinCategory>();
}

FString FRigVMGraphDetailCustomizationImpl::GetPinCategory(FString InPinPath) const
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				return Pin->GetCategory();
			}
		}
	}
	return FString();
}

int32 FRigVMGraphDetailCustomizationImpl::GetPinIndexInCategory(FString InPinPath) const
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				return Pin->GetIndexInCategory();
			}
		}
	}
	return INDEX_NONE;
}

FString FRigVMGraphDetailCustomizationImpl::GetPinLabel(FString InPinPath) const
{
	if (const FRigVMNodeLayout* NodeLayout = GetNodeLayout())
	{
		if(const FString* DisplayName = NodeLayout->FindDisplayName(InPinPath))
		{
			return *DisplayName;
		}
	}
	return FString();
}

FLinearColor FRigVMGraphDetailCustomizationImpl::GetPinColor(FString InPinPath) const
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				if(const URigVMEdGraphSchema* Schema = Cast<URigVMEdGraphSchema>(RigVMClientHost->GetRigVMEdGraphSchemaClass()->GetDefaultObject()))
				{
					const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*Pin->GetCPPType(), Pin->GetCPPTypeObject());
					return Schema->GetPinTypeColor(PinType);
				}
			}
		}
	}
	return FLinearColor::White;
}

const FSlateBrush* FRigVMGraphDetailCustomizationImpl::GetPinIcon(FString InPinPath) const
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(const URigVMPin* Pin = Node->FindPin(InPinPath))
			{
				const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(*Pin->GetCPPType(), Pin->GetCPPTypeObject());
				return FBlueprintEditorUtils::GetIconFromPin(PinType, /* bIsLarge = */ false);
			}
		}
	}
	return nullptr;
}

void FRigVMGraphDetailCustomizationImpl::HandleCategoryAdded(FString InCategory)
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = RigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->AddEmptyPinCategory(Node->GetFName(), InCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::HandleCategoryRemoved(FString InCategory)
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = RigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->RemovePinCategory(Node->GetFName(), InCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::HandleCategoryRenamed(FString InOldCategory, FString InNewCategory)
{
	if(RigVMClientHost.IsValid())
	{
		if(const URigVMNode* Node = GetNodeForLayout())
		{
			if(URigVMController* Controller = RigVMClientHost->GetController(Node->GetGraph()))
			{
				Controller->RenamePinCategory(Node->GetFName(), InOldCategory, InNewCategory);
				CachedNodeLayout.Reset();
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::HandlePinCategoryChanged(FString InPinPath, FString InCategory)
{
	if(RigVMClientHost.IsValid())
	{
		if (const URigVMLibraryNode* LibraryNode = GetLibraryNode())
		{
			if(const URigVMPin* Pin = LibraryNode->FindPin(InPinPath))
			{
				if(URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetGraph()))
				{
					Controller->SetPinCategory(Pin->GetPinPath(), InCategory);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::HandlePinLabelChanged(FString InPinPath, FString InNewLabel)
{
	if(RigVMClientHost.IsValid())
	{
		if (const URigVMLibraryNode* LibraryNode = GetLibraryNode())
		{
			if(const URigVMPin* Pin = LibraryNode->FindPin(InPinPath))
			{
				if(URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetGraph()))
				{
					Controller->SetPinDisplayName(Pin->GetPinPath(), InNewLabel);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}

void FRigVMGraphDetailCustomizationImpl::HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory)
{
	if(RigVMClientHost.IsValid())
	{
		if (const URigVMLibraryNode* LibraryNode = GetLibraryNode())
		{
			if(const URigVMPin* Pin = LibraryNode->FindPin(InPinPath))
			{
				if(URigVMController* Controller = RigVMClientHost->GetController(LibraryNode->GetGraph()))
				{
					Controller->SetPinIndexInCategory(Pin->GetPinPath(), InIndexInCategory);
					CachedNodeLayout.Reset();
				}
			}
		}
	}
}

bool FRigVMGraphDetailCustomizationImpl::ValidateName(FString InNewName, FText& OutErrorMessage)
{
	if(InNewName.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyNamesAreNotAllowed", "Empty names are not allowed.");
		return false;
	}

	if(FChar::IsDigit(InNewName[0]))
	{
		OutErrorMessage = LOCTEXT("NamesCannotStartWithADigit", "Names cannot start with a digit.");
		return false;
	}

	for (int32 i = 0; i < InNewName.Len(); ++i)
	{
		TCHAR& C = InNewName[i];

		const bool bGoodChar = FChar::IsAlpha(C) ||					 // Any letter
			(C == '_') || (C == '-') || (C == ' ') ||				 // _  - space anytime
			FChar::IsDigit(C);										 // 0-9 anytime

		if (!bGoodChar)
		{
			const FText Character = FText::FromString(InNewName.Mid(i, 1));
			OutErrorMessage = FText::Format(LOCTEXT("CharacterNotAllowedFormat", "'{0}' not allowed."), Character);
			return false;
		}
	}

	if (InNewName.Len() > 100)
	{
		OutErrorMessage = LOCTEXT("NameIsTooLong", "Name is too long.");
		return false;
	}

	return true;
}

bool FRigVMGraphDetailCustomizationImpl::HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		const FString ParentCategory = Node->GetParentPinCategory(InCategoryPath);
		if(!ParentCategory.IsEmpty())
		{
			const TArray<FString> SiblingCategories = Node->GetSubPinCategories(ParentCategory);
			const FString NewNameSuffix = TEXT("|") + InNewName;
			if(SiblingCategories.ContainsByPredicate([InNewName, NewNameSuffix](const FString& Category)
			{
				return Category.Equals(InNewName, ESearchCase::IgnoreCase) || Category.EndsWith(NewNameSuffix, ESearchCase::IgnoreCase); 
			}))
			{
				OutErrorMessage = LOCTEXT("NameIsAlreadyUsed", "Duplicate name.");
				return false;
			}
		} 
	}
	return true;
}

bool FRigVMGraphDetailCustomizationImpl::HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage)
{
	if(!ValidateName(InNewName, OutErrorMessage))
	{
		return false;
	}
	if(const URigVMNode* Node = GetNodeForLayout())
	{
		if(const URigVMPin* Pin = Node->FindPin(InPinPath))
		{
			const FString Category = Pin->GetCategory();
			if(!Category.IsEmpty())
			{
				const TArray<URigVMPin*> PinsInCategory = Node->GetPinsForCategory(Category);
				if(PinsInCategory.ContainsByPredicate([InNewName](const URigVMPin* PinInCategory)
				{
					return PinInCategory->GetDisplayName().ToString().Equals(InNewName, ESearchCase::IgnoreCase);
				}))
				{
					OutErrorMessage = LOCTEXT("NameIsAlreadyUsedInCategory", "Duplicate name (category).");
					return false;
				}
			}

			if(const URigVMPin* ParentPin = Pin->GetParentPin())
			{
				const TArray<URigVMPin*> SubPins = ParentPin->GetSubPins();
				if(SubPins.ContainsByPredicate([InNewName](const URigVMPin* SubPin)
				{
					return SubPin->GetDisplayName().ToString().Equals(InNewName, ESearchCase::IgnoreCase);
				}))
				{
					OutErrorMessage = LOCTEXT("NameIsAlreadyUsedWithinPin", "Duplicate name (parent pin).");
					return false;
				}
			}
		}
	}
	return true;
}

uint32 FRigVMGraphDetailCustomizationImpl::GetNodeLayoutHash() const
{
	uint32 Hash = 0;
	if(const FRigVMNodeLayout* Layout = GetNodeLayout())
	{
		Hash = HashCombine(Hash, GetTypeHash(*Layout));
	}
	const TArray<FString> UncategorizedPins = GetUncategorizedPins();
	for(const FString& UncategorizedPin : UncategorizedPins)
	{
		Hash = HashCombine(Hash, GetTypeHash(UncategorizedPin));
	}
	return Hash;
}

FRigVMWrappedNodeDetailCustomization::FRigVMWrappedNodeDetailCustomization()
: BlueprintBeingCustomized(nullptr)
{
}

TSharedRef<IDetailCustomization> FRigVMWrappedNodeDetailCustomization::MakeInstance()
{
	return MakeShareable(new FRigVMWrappedNodeDetailCustomization);
}

void FRigVMWrappedNodeDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailLayout.GetObjectsBeingCustomized(DetailObjects);
	if (DetailObjects.Num() == 0)
	{
		return;
	}

	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		URigVMDetailsViewWrapperObject* WrapperObject = CastChecked<URigVMDetailsViewWrapperObject>(DetailObject.Get());
		if (!WrapperObject->IsActive())
		{
			continue;
		}
		
		if(BlueprintBeingCustomized == nullptr)
		{
			BlueprintBeingCustomized = IRigVMEditorAssetInterface::GetInterfaceOuter(WrapperObject);
		}

		ObjectsBeingCustomized.Add(WrapperObject);
		NodesBeingCustomized.Add(CastChecked<URigVMNode>(WrapperObject->GetSubject()));
	}

	if (BlueprintBeingCustomized == nullptr || ObjectsBeingCustomized.IsEmpty() || NodesBeingCustomized.IsEmpty())
	{
		return;
	}

	UClass* WrapperClass = ObjectsBeingCustomized[0]->GetClass();

	if(NodesBeingCustomized.Num() == 1)
	{
		if(NodesBeingCustomized[0].IsValid())
		{
			if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(NodesBeingCustomized[0].Get()))
			{
				const FRigVMGraphFunctionHeader& Header = FunctionReferenceNode->GetReferencedFunctionHeader();
				const FRigVMGraphFunctionIdentifier& Identifier = Header.LibraryPointer;
				
				IDetailCategoryBuilder& FunctionCategory = DetailLayout.EditCategory("Function", LOCTEXT("Function", "Function"), ECategoryPriority::Uncommon);
				FunctionCategory.InitiallyCollapsed(false);

				FunctionCategory.AddCustomRow(LOCTEXT("FunctionName", "FunctionName"))
					.NameContent()
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Name")))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SNew(STextBlock)
						.Text(FText::FromString(Identifier.GetFunctionName()))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					];

				FunctionCategory.AddCustomRow(LOCTEXT("FunctionPath", "FunctionPath"))
				.NameContent()
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Path")))
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
					.ContentPadding(0.f)
					.Text(FText::FromString(Identifier.GetLibraryNodePath()))
					.OnClicked_Lambda([Header]() -> FReply
					{
						if(const URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Header.LibraryPointer.GetNodeSoftPath().TryLoad()))
						{
							if (FRigVMEditorAssetInterfacePtr EditorAsset = IRigVMEditorAssetInterface::GetInterfaceOuter(LibraryNode))
							{
								GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorAsset->GetAssetObjectForEditor());
								
								if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(EditorAsset.GetObject(), true))
								{
									if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
									{
										RigVMEditor->HandleJumpToHyperlink(LibraryNode);
										return FReply::Handled();
									}
								}
							}
						}
						return FReply::Unhandled();
					})
				];
			}
		}
	}

	// determine the order of things
	typedef TTuple<const FProperty*, FRigVMPropertyPath, FString> TPropertyToToShow;
	TArray<TPropertyToToShow> PropertiesToShow;

	bool bInspectingOnlyOneNodeType = NodesBeingCustomized.Num() == 1;
	if(NodesBeingCustomized.Num() > 1)
	{
		const UClass* NodeClass = nullptr;
		TArray<TTuple<FString,const UScriptStruct*>> Traits;
		FName TemplateNotation(NAME_None);
		FRigVMNodeLayout NodeLayout;
		for(const TWeakObjectPtr<URigVMNode>& NodePtr : NodesBeingCustomized)
		{
			if(!NodePtr.IsValid())
			{
				continue;
			}
			
			const URigVMNode* Node = NodePtr.Get();
			if(NodeClass == nullptr)
			{
				// when looking at the first node - remember the relevant bits
				NodeClass = Node->GetClass();

				if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
				{
					TemplateNotation = TemplateNode->GetNotation();
				}

				NodeLayout = Node->GetNodeLayout();

				for(const FString& TraitName : Node->GetTraitNames())
				{
					Traits.Emplace(TraitName, Node->GetTraitScriptStruct(*TraitName));
				}
			}
			else
			{
				if(Node->GetClass() != NodeClass)
				{
					bInspectingOnlyOneNodeType = false;
					break;
				}

				if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
				{
					if(!TemplateNotation.IsEqual(TemplateNode->GetNotation()))
					{
						bInspectingOnlyOneNodeType = false;
						break;
					}
				}

				if(NodeLayout != Node->GetNodeLayout())
				{
					bInspectingOnlyOneNodeType = false;
					break;
				}

				if(Node->GetTraitNames().Num() != Traits.Num())
				{
					bInspectingOnlyOneNodeType = false;
					break;
				}
				
				for(int32 TraitIndex = 0; TraitIndex < Traits.Num(); TraitIndex++)
				{
					const FString TraitName = Node->GetTraitNames()[TraitIndex];
					if(TraitName != Traits[TraitIndex].Get<0>())
					{
						bInspectingOnlyOneNodeType = false;
						break;
					}
					if(Traits[TraitIndex].Get<1>() != Node->GetTraitScriptStruct(*TraitName))
					{
						bInspectingOnlyOneNodeType = false;
						break;
					}
				}
				if(!bInspectingOnlyOneNodeType)
				{
					break;
				}
			}
		}
	}

	const URigVMNode* NodeWithCategories = nullptr;
	if(bInspectingOnlyOneNodeType)
	{
		// determine if we should be using pin categories to display the node
		for(const TWeakObjectPtr<URigVMNode>& NodePtr : NodesBeingCustomized)
		{
			if(NodePtr.IsValid())
			{
				NodeWithCategories = NodePtr.Get();
				if(NodeWithCategories->GetPinCategories().IsEmpty())
				{
					NodeWithCategories = nullptr;
				}
				break;
			}
		}
	}

	if(NodeWithCategories)
	{
		const FRigVMNodeLayout NodeLayout = NodeWithCategories->GetNodeLayout();
		for(const FRigVMPinCategory& Category : NodeLayout.Categories)
		{
			for(const FString& PinPath : Category.Elements)
			{
				FString Left, Right;
				if(!URigVMPin::SplitPinPathAtStart(PinPath, Left, Right))
				{
					Left = PinPath;
				}
				if(const FProperty* Property = WrapperClass->FindPropertyByName(*Left))
				{
					FRigVMPropertyPath PropertyPath;
					if(!Right.IsEmpty())
					{
						PropertyPath = FRigVMPropertyPath(Property, Right);
					}
					PropertiesToShow.Emplace(Property, PropertyPath, Category.Path);
				}
			}
		}
	}
	else
	{
		// if we don't have a pin category layout let's just use all root properties
		for (TFieldIterator<FProperty> PropertyIt(WrapperClass); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			const FName PropertyName = Property->GetFName();
			TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(PropertyName, WrapperClass);
			if (!PropertyHandle->IsValidHandle())
			{
				continue;
			}
			PropertiesToShow.Emplace(Property, FRigVMPropertyPath(), FString());
		}
	}

	// now loop over all of the properties and display them
	TArray<TSharedPtr<IPropertyHandle>> PropertiesAddedToLayout;
	FRigVMNodeLayout NodeLayout;
	if(NodeWithCategories)
	{
		NodeLayout = NodeWithCategories->GetNodeLayout();
	}
	for (const TTuple<const FProperty*, FRigVMPropertyPath, FString>& Tuple : PropertiesToShow)
	{
		const FProperty* Property = Tuple.Get<0>();
		const FRigVMPropertyPath& PropertyPath = Tuple.Get<1>();
		FString PinPath = Property->GetName();
		if(PropertyPath.IsValid())
		{
			PinPath = URigVMPin::JoinPinPath(PinPath, PropertyPath.ToString());
		}
		const FString Category = Tuple.Get<2>();
		
		TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(Property->GetFName(), WrapperClass);
		if (!PropertyHandle->IsValidHandle())
		{
			continue;
		}
		for(const FRigVMPropertyPathSegment& Segment : PropertyPath.GetSegments())
		{
			switch(Segment.Type)
			{
				case ERigVMPropertyPathSegmentType::StructMember:
				{
					PropertyHandle = PropertyHandle->GetChildHandle(Segment.Name);
					break;
				}
				case ERigVMPropertyPathSegmentType::ArrayElement:
				{
					PropertyHandle = PropertyHandle->GetChildHandle(Segment.Index);
					break;
				}
				case ERigVMPropertyPathSegmentType::MapValue:
				{
					// not supported just yet
					checkNoEntry();
					break;
				}
			}
			if (!PropertyHandle->IsValidHandle())
			{
				break;
			}
		}
		if (!PropertyHandle->IsValidHandle())
		{
			continue;
		}

		const URigVMPin* Pin = nullptr;
		for(const TWeakObjectPtr<URigVMNode>& Node : NodesBeingCustomized)
		{
			if(Node.IsValid())
			{
				Pin = Node->FindPin(PinPath);
				if(Pin)
				{
					break;
				}
			}	
		}

		PropertiesAddedToLayout.Add(PropertyHandle);

		TAttribute<bool> HasDefaultValueOverride = TAttribute<bool>::CreateLambda([this, PinPath, PropertyHandle]() -> bool
		{
			if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
			{
				for(const TWeakObjectPtr<URigVMNode>& Node : NodesBeingCustomized)
				{
					if(Node.IsValid())
					{
						if(const URigVMPin* Pin = Node->FindPin(PinPath))
						{
							if(Pin->HasDefaultValueOverride())
							{
								return true;
							}
						}
					}	
				}
			}
			return PropertyHandle->DiffersFromDefault();
		});

		FResetToDefaultOverride ResetToDefault = FResetToDefaultOverride::Create(
			HasDefaultValueOverride,
			FSimpleDelegate::CreateLambda([this, PropertyHandle, PinPath]()
			{
				FScopedTransaction Transaction(LOCTEXT("ResetValueToDefault", "Reset Value To Default"));
				const URigVMGraph* Graph = NodesBeingCustomized[0]->GetGraph();
				URigVMController* Controller = BlueprintBeingCustomized->GetController(Graph);
				FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Unset);

				Controller->OpenUndoBracket(TEXT("Reset pin default value"));
				for(const TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
				{
					if(const URigVMPin* Pin = Node->FindPin(PinPath))
					{
						Controller->ResetPinDefaultValue(Pin->GetPinPath());
					}
				}
				Controller->CloseUndoBracket();
			})
		);

		static const FSlateFontInfo NameFont = FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") );

		FText LabelOverride;
		if(const FString* DisplayName = NodeLayout.FindDisplayName(PinPath))
		{
			LabelOverride = FText::FromString(*DisplayName);
		}
		TSharedRef<SWidget> LabelWidget = PropertyHandle->CreatePropertyNameWidget(LabelOverride);

		/*
		// in the future we may want some visual alignment of the label widget on top of the 
		// reset arrow on the right to indicate the state of the default value change
		TSharedRef<SHorizontalBox> LabelWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SBorder)
			.HAlign(HAlign_Left)
			.BorderImage_Lambda([HasDefaultValueOverride]() -> const FSlateBrush*
			{
				if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
				{
					if(HasDefaultValueOverride.Get())
					{
						static const FSlateBrush* BorderBrush = FAppStyle::Get().GetBrush("FloatingBorder");
						return BorderBrush;
					}
				}
				return nullptr;
			})
			.BorderBackgroundColor_Lambda([HasDefaultValueOverride]() -> FSlateColor
			{
				if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
				{
					if(HasDefaultValueOverride.Get())
					{
						return FSlateColor(FLinearColor::Red);
					}
				}
				return FSlateColor(EStyleColor::Background);
			})
			[
				SNew(STextBlock)
				.Text(PropertyHandle->GetPropertyDisplayName())
				.Font(NameFont)
			]
		];
		*/

		/*
		auto GetOverrideStatus = [this, PinPath, PropertyHandle]() -> EOverrideWidgetStatus::Type
		{
			if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
			{
				for(const TWeakObjectPtr<URigVMNode>& Node : NodesBeingCustomized)
				{
					if(Node.IsValid())
					{
						if(const URigVMPin* Pin = Node->FindPin(PinPath))
						{
							if(Pin->GetDefaultValueType() == ERigVMPinDefaultValueType::Override)
							{
								return PropertyHandle->DiffersFromDefault() ? EOverrideWidgetStatus::ChangedHere : EOverrideWidgetStatus::ChangedToDefault; 
							}
							if(Pin->HasDefaultValueOverride())
							{
								return EOverrideWidgetStatus::ChangedInside;
							}
							if(Pin->GetDefaultValueType() == ERigVMPinDefaultValueType::Unset)
							{
								return EOverrideWidgetStatus::None;
							}
						}
					}	
				}
			}
			return PropertyHandle->DiffersFromDefault() ? EOverrideWidgetStatus::ChangedHere : EOverrideWidgetStatus::None;
		};

		TSharedRef<SWidget> OverrideWidget = SNullWidget::NullWidget;
		
		if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
		{
			OverrideWidget = SNew(SOverrideStatusWidget)
			.Status_Lambda(GetOverrideStatus)
			.MenuContent_Lambda([this, PinPath, PropertyHandle, GetOverrideStatus]
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				const EOverrideWidgetStatus::Type Status = GetOverrideStatus();
				switch(Status)
				{
					case EOverrideWidgetStatus::None:
					case EOverrideWidgetStatus::Inherited:
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("OverrideValueWithCurrent", "Set Override"),
							LOCTEXT("OverrideValueWithCurrentTooltip", "Overrides value while keeping the value (locks the value in place)."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this, PinPath]()
								{
									const URigVMGraph* Graph = NodesBeingCustomized[0]->GetGraph();
									URigVMController* Controller = BlueprintBeingCustomized->GetController(Graph);

									Controller->OpenUndoBracket(TEXT("Set Override"));
										
									for(const TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
									{
										if(const URigVMPin* Pin = Node->FindPin(PinPath))
										{
											FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Override);
											if(Pin->CanProvideDefaultValue())
											{
												FString DefaultValue = Pin->GetDefaultValue();
												if(DefaultValue.IsEmpty())
												{
													DefaultValue = Pin->GetOriginalDefaultValue();
												}
												if(!DefaultValue.IsEmpty())
												{
													Controller->SetPinDefaultValue(Pin->GetPinPath(), DefaultValue);
												}
											}
										}
									}

									Controller->CloseUndoBracket();
								})
							)
						);
						break;
					}
					case EOverrideWidgetStatus::ChangedHere:
					case EOverrideWidgetStatus::ChangedToDefault:
					case EOverrideWidgetStatus::ChangedInside:
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("RemoveOverride", "Remove Override"),
							LOCTEXT("RemoveOverrideTooltip", "Removes the override and restores the inherited value."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([this, PinPath]()
								{
									const URigVMGraph* Graph = NodesBeingCustomized[0]->GetGraph();
									URigVMController* Controller = BlueprintBeingCustomized->GetController(Graph);

									Controller->OpenUndoBracket(TEXT("Remove Override"));
										
									for(const TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
									{
										if(const URigVMPin* Pin = Node->FindPin(PinPath))
										{
											Controller->ResetPinDefaultValue(Pin->GetPinPath(), true);
										}
									}

									Controller->CloseUndoBracket();
								})
							)
						);
						break;
					}
					case EOverrideWidgetStatus::Undetermined:
					case EOverrideWidgetStatus::Uninitialized:
					default:
					{
						break;
					}
				}
				return MenuBuilder.MakeWidget();
			});
		}
		*/
		
		IDetailPropertyRow* Row = nullptr;
		if(NodeWithCategories)
		{
			DetailLayout.HideProperty(PropertyHandle);
			FString Left, CategoryName;
			if(!RigVMStringUtils::SplitNodePathAtEnd(Category, Left, CategoryName))
			{
				CategoryName = Category;
			}
			Row = &DetailLayout.EditCategory(*Category, FText::FromString(CategoryName))
				.AddProperty(PropertyHandle);
		}
		else
		{
			Row = DetailLayout.EditDefaultProperty(PropertyHandle);
		}
		
		// check if any / all pins are bound to a variable
		int32 PinsBoundToVariable = 0;
		TArray<URigVMPin*> ModelPins;
		for(TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
		{
			if(URigVMPin* ModelPin = Node->FindPin(Property->GetName()))
			{
				ModelPins.Add(ModelPin);
				PinsBoundToVariable += ModelPin->IsBoundToVariable() ? 1 : 0;
			}
		}

		if(PinsBoundToVariable > 0)
		{
			if(PinsBoundToVariable == ModelPins.Num())
			{
				Row->CustomWidget()
				.NameContent()
				[
					LabelWidget
				]
				.ValueContent()
				[
				SNew(SRigVMGraphVariableBinding)
					.ModelPins(ModelPins)
					.Asset(BlueprintBeingCustomized)
				];
				continue;
			}
			else // in this case some pins are bound, and some are not - we'll hide the input value widget
			{
				Row->CustomWidget()
				.NameContent()
				[
					LabelWidget
				];
				continue;
			}
		}
		
		if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
        {
        	FString CustomWidgetName = NameProperty->GetMetaData(TEXT("CustomWidget"));
        	if (!CustomWidgetName.IsEmpty())
        	{
        		URigVMEdGraph* GraphBeingCustomized = Cast<URigVMEdGraph>(
        			BlueprintBeingCustomized->GetEdGraph(NodesBeingCustomized[0]->GetGraph()));
        		ensure(GraphBeingCustomized);
        		
        		const TArray<TSharedPtr<FRigVMStringWithTag>>* NameList = GraphBeingCustomized->GetNameListForWidget(CustomWidgetName);
        		if (NameList)
        		{
        			TSharedPtr<SRigVMGraphPinNameListValueWidget> NameListWidget;

        			Row->CustomWidget()
					.NameContent()
					[
						LabelWidget
					]
					.ValueContent()
					[
						SAssignNew(NameListWidget, SRigVMGraphPinNameListValueWidget)
						.OptionsSource(NameList)
						.OnGenerateWidget(this, &FRigVMWrappedNodeDetailCustomization::MakeNameListItemWidget)
						.OnSelectionChanged(this, &FRigVMWrappedNodeDetailCustomization::OnNameListChanged, NameProperty, DetailLayout.GetPropertyUtilities())
						.OnComboBoxOpening(this, &FRigVMWrappedNodeDetailCustomization::OnNameListComboBox, NameProperty, NameList)
						.InitiallySelectedItem(GetCurrentlySelectedItem(NameProperty, NameList))
						.Content()
						[
							SNew(STextBlock)
							.Text(this, &FRigVMWrappedNodeDetailCustomization::GetNameListText, NameProperty)
							.ColorAndOpacity_Lambda([this, NameProperty]() -> FSlateColor
							{
								static FText NoneText = LOCTEXT("None", "None"); 
								if(GetNameListText(NameProperty).EqualToCaseIgnored(NoneText))
								{
									return FSlateColor(FLinearColor::Red);
								}
								return FSlateColor::UseForeground();
							})
						]
					]
        			.OverrideResetToDefault(ResetToDefault);

        			NameListWidgets.Add(Property->GetFName(), NameListWidget);
       				continue;
        		}
        		
        		Row->CustomWidget()
				.NameContent()
				[
					LabelWidget
				]
        		.OverrideResetToDefault(ResetToDefault);

        		continue;
        	}
        }

		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		Row->GetDefaultWidgets(NameWidget, ValueWidget, /*bAddWidgetDecoration*/true);

		Row->CustomWidget(/*bShowChildren*/true)
			.NameContent()
			[
				LabelWidget
			]
			.ValueContent()
			[
				ValueWidget.ToSharedRef()
			]
			.OverrideResetToDefault(ResetToDefault);
	}

	// now loop over all handles and determine expansion states of the corresponding pins
	for (int32 Index = 0; Index < PropertiesAddedToLayout.Num(); Index++)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = PropertiesAddedToLayout[Index];
		FProperty* Property = PropertyHandle->GetProperty();

		// certain properties we don't look at for expansion states
		if(FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if(StructProperty->Struct == TBaseStructure<FVector>::Get() ||
				StructProperty->Struct == TBaseStructure<FVector2D>::Get() ||
				StructProperty->Struct == TBaseStructure<FRotator>::Get() ||
				StructProperty->Struct == TBaseStructure<FQuat>::Get())
			{
				continue;
			}
		}

		bool bFound = false;
		const FString PinPath = PropertyHandle->GeneratePathToProperty();
		for(TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
		{
			if(URigVMPin* Pin = Node->FindPin(PinPath))
			{
				bFound = true;
				
				if(Pin->IsExpanded())
				{
					if(IDetailPropertyRow* Row = DetailLayout.EditDefaultProperty(PropertyHandle))
					{
						Row->ShouldAutoExpand(true);
					}
					break;
				}
			}
		}

		if(!bFound)
		{
			continue;
		}

		uint32 NumChildren = 0;
		PropertyHandle->GetNumChildren(NumChildren);
		for(uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			PropertiesAddedToLayout.Add(PropertyHandle->GetChildHandle(ChildIndex));
		}
	}

	// hide all root properties not listed in the properties to show list
	for (TFieldIterator<FProperty> PropertyIt(WrapperClass); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if(!PropertiesToShow.ContainsByPredicate([Property](const TPropertyToToShow& Tuple) -> bool
		{
			return Tuple.Get<0>() == Property && !Tuple.Get<1>().IsValid();
		}))
		{
			const FName PropertyName = Property->GetFName();
			TSharedPtr<IPropertyHandle> PropertyHandle = DetailLayout.GetProperty(PropertyName, WrapperClass);
			if (!PropertyHandle->IsValidHandle())
			{
				continue;
			}
			DetailLayout.HideProperty(PropertyHandle);
		}
	}

	CustomizeLiveValues(DetailLayout);
}

TSharedRef<SWidget> FRigVMWrappedNodeDetailCustomization::MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem)
{
	//TODO: make this prettier
	return 	SNew(STextBlock).Text(FText::FromString(InItem->GetStringWithTag()));// .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

FText FRigVMWrappedNodeDetailCustomization::GetNameListText(const FNameProperty* InProperty) const
{
	FText FirstText;
	for(TWeakObjectPtr<URigVMDetailsViewWrapperObject> ObjectBeingCustomized : ObjectsBeingCustomized)
	{
		if (ObjectBeingCustomized.IsValid())
		{
			if (FName* Value = InProperty->ContainerPtrToValuePtr<FName>(ObjectBeingCustomized.Get()))
			{
				FText Text = FText::FromName(*Value);
				if(FirstText.IsEmpty())
				{
					FirstText = Text;
				}
				else if(!FirstText.EqualTo(Text))
				{
					return RigVMGraphDetailCustomizationMultipleValues;
				}
			}	
		}
	}
	return FirstText;
}

TSharedPtr<FRigVMStringWithTag> FRigVMWrappedNodeDetailCustomization::GetCurrentlySelectedItem(const FNameProperty* InProperty, const TArray<TSharedPtr<FRigVMStringWithTag>>* InNameList) const
{
	FString CurrentItem = GetNameListText(InProperty).ToString();
	for (const TSharedPtr<FRigVMStringWithTag>& Item : *InNameList)
	{
		if (Item->Equals(CurrentItem))
		{
			return Item;
		}
	}
	return TSharedPtr<FRigVMStringWithTag>();
}


void FRigVMWrappedNodeDetailCustomization::SetNameListText(const FText& NewTypeInValue, ETextCommit::Type, const FNameProperty* InProperty, TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	URigVMGraph* Graph = NodesBeingCustomized[0]->GetGraph();
	URigVMController* Controller = BlueprintBeingCustomized->GetController(Graph);

	Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InProperty->GetName()));
	
	for(TWeakObjectPtr<URigVMNode> Node : NodesBeingCustomized)
	{
		if(URigVMPin* Pin = Node->FindPin(InProperty->GetName()))
		{
			FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Override);
			Controller->SetPinDefaultValue(Pin->GetPinPath(), NewTypeInValue.ToString(), false, true, false, true);
		}
	}

	Controller->CloseUndoBracket();
}

void FRigVMWrappedNodeDetailCustomization::OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo, const FNameProperty* InProperty, TSharedRef<IPropertyUtilities> PropertyUtilities)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		const FString& NewValue = NewSelection->GetString();
		SetNameListText(FText::FromString(NewValue), ETextCommit::OnEnter, InProperty, PropertyUtilities);
	}
}

void FRigVMWrappedNodeDetailCustomization::OnNameListComboBox(const FNameProperty* InProperty, const TArray<TSharedPtr<FRigVMStringWithTag>>* InNameList)
{
	TSharedPtr<SRigVMGraphPinNameListValueWidget> Widget = NameListWidgets.FindChecked(InProperty->GetFName());
	const TSharedPtr<FRigVMStringWithTag> CurrentlySelected = GetCurrentlySelectedItem(InProperty, InNameList);
	Widget->SetSelectedItem(CurrentlySelected);
}

void FRigVMWrappedNodeDetailCustomization::CustomizeLiveValues(IDetailLayoutBuilder& DetailLayout)
{
	if(ObjectsBeingCustomized.Num() > 1)
	{
		return;
	}
	
	URigVMHost* DebuggedHost = Cast<URigVMHost>(BlueprintBeingCustomized->GetObjectBeingDebugged());
	if(DebuggedHost == nullptr)
	{
		return;
	}

	URigVM* VM = DebuggedHost->GetVM();
	if(VM == nullptr)
	{
		return;
	}

	URigVMDetailsViewWrapperObject* FirstWrapper = ObjectsBeingCustomized[0].Get();
	URigVMNode* FirstNode = NodesBeingCustomized[0].Get();
	if(FirstNode->GetTypedOuter<URigVMFunctionLibrary>())
	{
		return;
	}

	TSharedPtr<FRigVMParserAST> AST = FirstNode->GetGraph()->GetRuntimeAST(BlueprintBeingCustomized->GetVMCompileSettings().ASTSettings, false);
	if(!AST.IsValid())
	{
		return;
	}

	FRigVMByteCode& ByteCode = VM->GetByteCode();
	if(ByteCode.GetFirstInstructionIndexForSubject(FirstNode) == INDEX_NONE)
	{
		return;
	}

	/*
	IDetailCategoryBuilder& DebugCategory = DetailLayout.EditCategory("DebugLiveValues", LOCTEXT("DebugLiveValues", "Inspect Live Values"), ECategoryPriority::Uncommon);
	DebugCategory.InitiallyCollapsed(true);

	for(URigVMPin* Pin : FirstNode->GetPins())
	{
		if (Pin->IsExecuteContext())
		{
			continue;
		}
		
		// only show hidden pins in debug mode
		if(Pin->GetDirection() == ERigVMPinDirection::Hidden)
		{
			if(!DebuggedHost->IsInDebugMode())
			{
				continue;
			}
		}
		
		URigVMPin* SourcePin = Pin;
		if(BlueprintBeingCustomized->VMCompileSettings.ASTSettings.bFoldAssignments)
		{
			do
			{
				TArray<URigVMPin*> SourcePins = SourcePin->GetLinkedSourcePins(false);
				if(SourcePins.Num() > 0)
				{
					SourcePin = SourcePins[0];
				}
				else
				{
					break;
				}
			}
			while(SourcePin->GetNode()->IsA<URigVMRerouteNode>());
		}
		
		TArray<const FRigVMExprAST*> Expressions = AST->GetExpressionsForSubject(SourcePin);
		if(Expressions.Num() == 0 && SourcePin != Pin)
		{
			SourcePin = Pin;
			Expressions = AST->GetExpressionsForSubject(Pin);
		}

		bool bHasVar = false;
		for(const FRigVMExprAST* Expression : Expressions)
		{
			if(Expression->IsA(FRigVMExprAST::EType::Literal))
			{
				continue;
			}
			else if(Expression->IsA(FRigVMExprAST::EType::Var))
			{
				bHasVar = true;
				break;
			}
		}

		TArray<const FRigVMExprAST*> FilteredExpressions;
		for(const FRigVMExprAST* Expression : Expressions)
		{
			if(Expression->IsA(FRigVMExprAST::EType::Literal))
			{
				if(bHasVar)
				{
					continue;
				}
				FilteredExpressions.Add(Expression);
			}
			else if(Expression->IsA(FRigVMExprAST::EType::Var))
			{
				FilteredExpressions.Add(Expression);
			}
			else if(Expression->IsA(FRigVMExprAST::EType::CachedValue))
			{
				const FRigVMCachedValueExprAST* CachedValueExpr = Expression->To<FRigVMCachedValueExprAST>();
				FilteredExpressions.Add(CachedValueExpr->GetVarExpr());
			}
		}

		bool bAddedProperty = false;
		int32 SuffixIndex = 1;
		FString NameSuffix;

		auto UpdateRow = [&](IDetailPropertyRow* PropertyRow)
		{
			PropertyRow->DisplayName(FText::FromString(FString::Printf(TEXT("%s%s"), *Pin->GetName(), *NameSuffix)));
			PropertyRow->IsEnabled(false);

			SuffixIndex++;
			bAddedProperty = true;
			NameSuffix = FString::Printf(TEXT("_%d"), SuffixIndex);
		};

		static const FAddPropertyParams AddPropertyParams = FAddPropertyParams().ForceShowProperty();

		TArray<FRigVMOperand> KnownOperands;
		for(const FRigVMExprAST* Expression : FilteredExpressions)
		{
			const FRigVMVarExprAST* VarExpr = Expression->To<FRigVMVarExprAST>();

			FString PinHash = URigVMCompiler::GetPinHash(SourcePin, VarExpr, false);
			const FRigVMOperand* Operand = BlueprintBeingCustomized->PinToOperandMap.Find(PinHash);
			if(Operand)
			{
				if(Operand->GetRegisterOffset() != INDEX_NONE)
				{
					continue;
				}
				if(KnownOperands.Contains(*Operand))
				{
					continue;
				}

				const FProperty* Property = nullptr;

				TArray<FRigVMMemoryStorageStruct*> ExternalStructs;
				TArray<UObject*> ExternalObjects;

				if(Operand->GetMemoryType() == ERigVMMemoryType::External)
				{
					if(!VM->GetExternalVariableDefs().IsValidIndex(Operand->GetRegisterIndex()))
					{
						continue;
					}
					ExternalObjects.Add(DebuggedHost);
					Property = VM->GetExternalVariableDefs()[Operand->GetRegisterIndex()].Property; 
				}
				else
				{
					FRigVMMemoryStorageStruct* Memory = DebuggedHost->GetMemoryByType(Operand->GetMemoryType());
					if(Memory == nullptr)
					{
						continue;
					}

					if(!Memory->IsValidIndex(Operand->GetRegisterIndex()))
					{
						continue;
					}
					
					Property = Memory->GetProperty(Operand->GetRegisterIndex());
					if(Property == nullptr)
					{
						continue;
					}

					ExternalStructs.Add(Memory);
				}

				check(ExternalObjects.Num() > 0 || ExternalStructs.Num() > 0);
				check(Property);

				if (!ExternalObjects.IsEmpty())
				{
					if(IDetailPropertyRow* PropertyRow = DebugCategory.AddExternalObjectProperty(ExternalObjects, Property->GetFName(), EPropertyLocation::Default, AddPropertyParams))
					{
						UpdateRow(PropertyRow);
					}
				}

				for (FRigVMMemoryStorageStruct* Memory : ExternalStructs)
				{
					if (IDetailPropertyRow* PropertyRow = DebugCategory.AddExternalStructureProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(*Memory), Property->GetFName(), EPropertyLocation::Default, AddPropertyParams))
					{
						UpdateRow(PropertyRow);
					}
				}

				KnownOperands.Add(*Operand);
			}
		}

		if(!bAddedProperty)
		{
			TSharedPtr<IPropertyHandle> PinHandle = DetailLayout.GetProperty(Pin->GetFName());
			if(PinHandle.IsValid())
			{
				// we'll build a new custom row. adding the same property again
				// causes the property to be marked customized - thus it won't
				// show correctly in the default category.
				DebugCategory.AddCustomRow(FText::FromName(Pin->GetFName()))
				.NameContent()
				[
					PinHandle->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					PinHandle->CreatePropertyValueWidget()
				]
				.IsEnabled(false);
			}
		}
	}
	*/
}

FRigVMGraphEnumDetailCustomization::FRigVMGraphEnumDetailCustomization()
: BlueprintBeingCustomized(nullptr)
, GraphBeingCustomized(nullptr)
{
}

void FRigVMGraphEnumDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	StructsBeingCustomized.Reset();
	InPropertyHandle->GetOuterStructs(StructsBeingCustomized);

	for (UObject* Object : Objects)
	{
		ObjectsBeingCustomized.Add(Object);

		if(BlueprintBeingCustomized == nullptr)
		{
			BlueprintBeingCustomized = IRigVMEditorAssetInterface::GetInterfaceOuter(Object);
		}

		if(GraphBeingCustomized == nullptr)
		{
			GraphBeingCustomized = Object->GetTypedOuter<URigVMGraph>();
		}
	}

	FProperty* Property = InPropertyHandle->GetProperty();
	const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property);

	HeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
			.MinDesiredWidth(375.f)
			.MaxDesiredWidth(375.f)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
					.MinDesiredWidth(150.f)
					.MaxDesiredWidth(400.f)
					[
						SNew(SRigVMEnumPicker)
						.IsEnabled(true)
						.OnEnumChanged(this, &FRigVMGraphEnumDetailCustomization::HandleControlEnumChanged, InPropertyHandle)
						.GetCurrentEnum_Lambda([this, InPropertyHandle]()
						{
							UEnum* Enum = nullptr;
							FEditPropertyChain PropertyChain;
							TArray<int32> PropertyArrayIndices;
							bool bEnabled;
							if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
							{
								return Enum;
							}

							const TArray<uint8*> MemoryBlocks = GetMemoryBeingCustomized();
							for(uint8* MemoryBlock: MemoryBlocks)
							{
								if(MemoryBlock)
								{
									if (UEnum** CurrentEnum = ContainerMemoryBlockToEnumPtr(MemoryBlock, PropertyChain, PropertyArrayIndices))
									{
										Enum = *CurrentEnum;
									}
								}
							}
							return Enum;
						})
					]
				];
}

void FRigVMGraphEnumDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// nothing to do here
}

void FRigVMGraphEnumDetailCustomization::HandleControlEnumChanged(TSharedPtr<FString> InEnumPath, ESelectInfo::Type InSelectType, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	if (ObjectsBeingCustomized.IsEmpty() && StructsBeingCustomized.IsEmpty())
	{
		return;
	}
		
	FEditPropertyChain PropertyChain;
	TArray<int32> PropertyArrayIndices;
	bool bEnabled;
	if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
	{
		return;
	}
	
	URigVMController* Controller = nullptr;
	if(BlueprintBeingCustomized && GraphBeingCustomized)
	{
		Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized);
		Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InPropertyHandle->GetProperty()->GetName()));
	}

	const EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet;

	const TArray<uint8*> AllMemoryBeingCustomized = GetMemoryBeingCustomized();
	for (int32 Index = 0; Index < AllMemoryBeingCustomized.Num(); Index++)
	{
		const uint8* Memory = AllMemoryBeingCustomized[Index];
		if (Memory != nullptr && InPropertyHandle->IsValidHandle())
		{
			UEnum** CurrentEnum = ContainerMemoryBlockToEnumPtr((uint8*)Memory, PropertyChain, PropertyArrayIndices);
			if (CurrentEnum)
			{
				const UEnum* PreviousEnum = *CurrentEnum;
				*CurrentEnum = FindObject<UEnum>(nullptr, **InEnumPath.Get(), EFindObjectFlags::None);

				if (PreviousEnum != *CurrentEnum)
				{
					InPropertyHandle->NotifyPostChange(ChangeType);
				}
			}
		}
	}

	if(Controller)
	{
		Controller->CloseUndoBracket();
	}
}

template<typename VectorType, int32 NumberOfComponents>
void FRigVMGraphMathTypeDetailCustomization::MakeVectorHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	typedef typename VectorType::FReal NumericType;
	typedef SNumericVectorInputBox<NumericType, VectorType, NumberOfComponents> SLocalVectorInputBox;

	FEditPropertyChain PropertyChain;
	TArray<int32> PropertyArrayIndices;
	bool bEnabled;
	if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
	{
		return;
	}

	typename SLocalVectorInputBox::FArguments Args;
	Args.Font(IDetailLayoutBuilder::GetDetailFont());
	Args.IsEnabled(bEnabled);
	Args.AllowSpin(true);
	Args.SpinDelta(0.01f);
	Args.bColorAxisLabels(true);
	Args.X_Lambda([this, InPropertyHandle]()
		{
			return GetVectorComponent<VectorType, NumericType>(InPropertyHandle, 0);
		});
	Args.OnXChanged_Lambda([this, InPropertyHandle](NumericType Value)
		{
			OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 0, Value, false);
		});
	Args.OnXCommitted_Lambda([this, InPropertyHandle](NumericType Value, ETextCommit::Type CommitType)
		{
			OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 0, Value, true, CommitType);
		});
	Args.Y_Lambda([this, InPropertyHandle]()
		{
			return GetVectorComponent<VectorType, NumericType>(InPropertyHandle, 1);
		});
	Args.OnYChanged_Lambda([this, InPropertyHandle](NumericType Value)
		{
			OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 1, Value, false);
		});
	Args.OnYCommitted_Lambda([this, InPropertyHandle](NumericType Value, ETextCommit::Type CommitType)
		{
			OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 1, Value, true, CommitType);
		});

	ExtendVectorArgs<VectorType>(InPropertyHandle, &Args);

	HeaderRow
		.IsEnabled(bEnabled)
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(375.f)
		.MaxDesiredWidth(375.f)
		.HAlign(HAlign_Left)
		[
			SArgumentNew(Args, SLocalVectorInputBox)
		];
}

template<typename RotationType>
void FRigVMGraphMathTypeDetailCustomization::MakeRotationHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FEditPropertyChain PropertyChain;
	TArray<int32> PropertyArrayIndices;
	bool bEnabled;
	if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
	{
		return;
	}
		
	typedef typename RotationType::FReal NumericType;
	typedef SAdvancedRotationInputBox<NumericType> SLocalRotationInputBox;
	typename SLocalRotationInputBox::FArguments Args;
	Args.Font(IDetailLayoutBuilder::GetDetailFont());
	Args.IsEnabled(bEnabled);
	Args.AllowSpin(true);
	Args.bColorAxisLabels(true);

	ExtendRotationArgs<RotationType>(InPropertyHandle, &Args);

	HeaderRow
		.IsEnabled(bEnabled)
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(375.f)
		.MaxDesiredWidth(375.f)
		.HAlign(HAlign_Left)
		[
			SArgumentNew(Args, SLocalRotationInputBox)
		];
}

template <typename TransformType>
void FRigVMGraphMathTypeDetailCustomization::ConfigureTransformWidgetArgs(TSharedRef<IPropertyHandle> InPropertyHandle, typename SAdvancedTransformInputBox<TransformType>::FArguments& WidgetArgs, TConstArrayView<FName> ComponentNames)
{
	FEditPropertyChain PropertyChain;
	TArray<int32> PropertyArrayIndices;
	bool bEnabled;
	if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
	{
		return;
	}
	
	typedef typename TransformType::FReal FReal;
	WidgetArgs.IsEnabled(bEnabled);
	WidgetArgs.AllowEditRotationRepresentation(true);
	WidgetArgs.UseQuaternionForRotation(IsQuaternionBasedRotation<TransformType>());

	static TransformType Identity = TransformType::Identity;

	void* ContainerMemory = nullptr;
	TSharedPtr<FStructOnScope> DefaultStruct = nullptr;

	TArray<TSharedPtr<FStructOnScope>> StructsBeingCustomized;
	InPropertyHandle->GetOuterStructs(StructsBeingCustomized);

	if (!StructsBeingCustomized.IsEmpty())
	{
		DefaultStruct = MakeShareable(new FStructOnScope(StructsBeingCustomized[0]->GetStruct()));
		ContainerMemory = DefaultStruct->GetStructMemory();
	}
	else
	{
		TArray<UObject*> ObjectsBeingCustomized;
		InPropertyHandle->GetOuterObjects(ObjectsBeingCustomized);
		if (!ObjectsBeingCustomized.IsEmpty())
		{
			if (ObjectsBeingCustomized[0] != nullptr)
			{
				ContainerMemory = (uint8*)ObjectsBeingCustomized[0]->GetClass()->GetDefaultObject();
			}
		}
	}

	if (!ContainerMemory)
	{
		return;
	}
	TransformType DefaultValue = ContainerMemoryBlockToValueRef<TransformType>((uint8*)ContainerMemory, Identity, PropertyChain, PropertyArrayIndices);
	
	TSharedPtr<IPropertyHandle> TranslationHandle = InPropertyHandle->GetChildHandle(ComponentNames[ESlateTransformComponent::Location]);
	TSharedPtr<IPropertyHandle> RotationHandle = InPropertyHandle->GetChildHandle(ComponentNames[ESlateTransformComponent::Rotation]);
	TSharedPtr<IPropertyHandle> ScaleHandle = InPropertyHandle->GetChildHandle(ComponentNames[ESlateTransformComponent::Scale]);

	if (!TranslationHandle.IsValid() || !RotationHandle.IsValid() || !ScaleHandle.IsValid())
	{
		return;
	}

	const auto GetTranslation = [TranslationHandle]() -> FVector
		{
			FVector Translation = FVector::ZeroVector;
			if (TranslationHandle && TranslationHandle->IsValidHandle())
			{
				TranslationHandle->GetValue(Translation);
			}
			return Translation;
		};
	const auto GetRotation = [RotationHandle]() -> FQuat
		{
			FQuat Rotation = FQuat::Identity;
			if (RotationHandle && RotationHandle->IsValidHandle())
			{
				RotationHandle->GetValue(Rotation);
			}
			return Rotation;
		};
	const auto GetScale3D = [ScaleHandle]() -> FVector
		{
			FVector Scale = FVector::ZeroVector;
			if (ScaleHandle && ScaleHandle->IsValidHandle())
			{
				ScaleHandle->GetValue(Scale);
			}
			return Scale;
		};

	const auto SetTranslation = [TranslationHandle](const FVector& InValue)
		{
			if (TranslationHandle && TranslationHandle->IsValidHandle())
			{
				TranslationHandle->SetValue(InValue);
			}
		};
	const auto SetRotation = [RotationHandle](const FQuat& InValue)
		{
			if (RotationHandle && RotationHandle->IsValidHandle())
			{
				RotationHandle->SetValue(InValue);
			}
		};
	const auto SetScale3D = [ScaleHandle](const FVector& InValue)
		{
			if (ScaleHandle && ScaleHandle->IsValidHandle())
			{
				ScaleHandle->SetValue(InValue);
			}
		};

	const auto GetTransformComponent = [GetTranslation, GetRotation, GetScale3D](ESlateTransformComponent::Type InTransformComponent) -> TransformType
		{
			TransformType Transform = TransformType::Identity;
			switch (InTransformComponent)
			{
				case ESlateTransformComponent::Location:
				{
					const FVector Translation = GetTranslation();
					Transform.SetLocation(Translation);
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					const FQuat Rotation = GetRotation();
					Transform.SetRotation(Rotation);
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					const FVector Scale = GetScale3D();
					Transform.SetScale3D(Scale);
					break;
				}
				case ESlateTransformComponent::Max: // It means all components
				{
					Transform.SetLocation(GetTranslation());
					Transform.SetRotation(GetRotation());
					Transform.SetScale3D(GetScale3D());
					break;
				}
				default:
				{
					check(false);
				}
			}
			return Transform;
		};
	const auto SetTransformComponent = [InPropertyHandle, SetTranslation, SetRotation, SetScale3D](ESlateTransformComponent::Type InTransformComponent, const TransformType& InValue)
		{
			switch (InTransformComponent)
			{
				case ESlateTransformComponent::Location:
				{
					SetTranslation(InValue.GetLocation());
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					SetRotation(InValue.GetRotation());
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					SetScale3D(InValue.GetScale3D());
					break;
				}
				case ESlateTransformComponent::Max: // It means all components
				{
					SetTranslation(InValue.GetLocation());
					SetRotation(InValue.GetRotation());
					SetScale3D(InValue.GetScale3D());
					break;
				}
				default:
				{
					check(false);
				}
			}
		};

	WidgetArgs.DiffersFromDefault_Lambda([this, InPropertyHandle, DefaultValue, GetTransformComponent](ESlateTransformComponent::Type InTransformComponent) -> bool
	{
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return false;
		}

		const TransformType Transform = GetTransformComponent(InTransformComponent);
		switch(InTransformComponent)
		{
			case ESlateTransformComponent::Location:
			{
				if (!Transform.GetLocation().Equals(DefaultValue.GetLocation()))
				{
					return true;
				}
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				if (!Transform.Rotator().Equals(DefaultValue.Rotator()))
				{
					return true;
				}
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				if (!Transform.GetScale3D().Equals(DefaultValue.GetScale3D()))
				{
					return true;
				}
				break;
			}
			default:
			{
				if (!Transform.Equals(DefaultValue))
				{
					return true;
				}
				break;
			}
		}
		return false;
	});

	WidgetArgs.OnGetNumericValue_Lambda([this, InPropertyHandle, GetTransformComponent](
		ESlateTransformComponent::Type InTransformComponent,
		ESlateRotationRepresentation::Type InRotationRepresentation,
		ESlateTransformSubComponent::Type InTransformSubComponent) -> TOptional<FReal>
	{
		TOptional<FReal> Result;
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return Result;
		}

		const TransformType Transform = GetTransformComponent(InTransformComponent);
		const TOptional<FReal> Value = SAdvancedTransformInputBox<TransformType>::GetNumericValueFromTransform(
			Transform,
			InTransformComponent,
			InRotationRepresentation,
			InTransformSubComponent);

		if(Value.IsSet())
		{
			Result = Value;
		}
		return Result;
	});
	

	auto OnNumericValueChanged = [this, InPropertyHandle, GetTransformComponent, SetTransformComponent](
		ESlateTransformComponent::Type InTransformComponent, 
		ESlateRotationRepresentation::Type InRotationRepresentation, 
		ESlateTransformSubComponent::Type InSubComponent,
		FReal InValue,
		bool bIsCommit,
		ETextCommit::Type InCommitType = ETextCommit::Default)
	{
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return;
		}
		
		URigVMController* Controller = nullptr;
		if (TStrongObjectPtr<UObject> Blueprint = BlueprintBeingCustomized.GetWeakObjectPtr().Pin(); GraphBeingCustomized.IsValid())
		{
			Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized.Get());
			if(bIsCommit)
			{
				Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InPropertyHandle->GetProperty()->GetName()));
			}
		}

		TransformType Transform = GetTransformComponent(InTransformComponent);
		SAdvancedTransformInputBox<TransformType>::ApplyNumericValueChange(
			Transform,
			InValue,
			InTransformComponent,
			InRotationRepresentation,
			InSubComponent);
		SetTransformComponent(InTransformComponent, Transform);

		if (Controller && bIsCommit)
		{
			Controller->CloseUndoBracket();
		}
	};

	WidgetArgs.OnNumericValueChanged_Lambda([OnNumericValueChanged](
		ESlateTransformComponent::Type InTransformComponent, 
		ESlateRotationRepresentation::Type InRotationRepresentation, 
		ESlateTransformSubComponent::Type InSubComponent,
		FReal InValue)
	{
		OnNumericValueChanged(InTransformComponent, InRotationRepresentation, InSubComponent, InValue, false);
	});

	WidgetArgs.OnNumericValueCommitted_Lambda([OnNumericValueChanged](
		ESlateTransformComponent::Type InTransformComponent, 
		ESlateRotationRepresentation::Type InRotationRepresentation, 
		ESlateTransformSubComponent::Type InSubComponent,
		FReal InValue, 
		ETextCommit::Type InCommitType)
	{
		OnNumericValueChanged(InTransformComponent, InRotationRepresentation, InSubComponent, InValue, true, InCommitType);
	});

	WidgetArgs.OnResetToDefault_Lambda([this, DefaultValue, InPropertyHandle, SetTransformComponent](ESlateTransformComponent::Type InTransformComponent)
	{
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return;
		}
		
		URigVMController* Controller = nullptr;
		if (TStrongObjectPtr<UObject> Blueprint = BlueprintBeingCustomized.GetWeakObjectPtr().Pin(); GraphBeingCustomized.IsValid())
		{
			Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized.Get());
			if(Controller)
			{
				Controller->OpenUndoBracket(FString::Printf(TEXT("Reset %s to Default"), *InPropertyHandle->GetProperty()->GetName()));
			}
		}

		TransformType Transform;
		switch(InTransformComponent)
		{
			case ESlateTransformComponent::Location:
			{
				Transform.SetLocation(DefaultValue.GetLocation());
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				Transform.SetRotation(DefaultValue.GetRotation());
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				Transform.SetScale3D(DefaultValue.GetScale3D());
				break;
			}
			case ESlateTransformComponent::Max:
			{
				Transform.SetLocation(DefaultValue.GetLocation());
				Transform.SetRotation(DefaultValue.GetRotation());
				Transform.SetScale3D(DefaultValue.GetScale3D());
				break;
			}
			default:
			{
				check(false);
			}
		}
		SetTransformComponent(InTransformComponent, Transform);
				
		if(Controller)
		{
			Controller->CloseUndoBracket();
		}
	});

	WidgetArgs.OnCopyToClipboard_Lambda([this, InPropertyHandle, GetTransformComponent](
		ESlateTransformComponent::Type InTransformComponent
		)
	{
		TOptional<FReal> Result;
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return;
		}

		const TransformType Transform = GetTransformComponent(InTransformComponent);

		FString Content;
		switch(InTransformComponent)
		{
			case ESlateTransformComponent::Location:
			{
				const FVector Data = Transform.GetLocation();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				const FRotator Data = Transform.Rotator();
				TBaseStructure<FRotator>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				const FVector Data = Transform.GetScale3D();
				TBaseStructure<FVector>::Get()->ExportText(Content, &Data, &Data, nullptr, PPF_None, nullptr);
				break;
			}
			case ESlateTransformComponent::Max:
			default:
			{
				TBaseStructure<TransformType>::Get()->ExportText(Content, &Transform, &Transform, nullptr, PPF_None, nullptr);
				break;
			}
		}

		if(!Content.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*Content);
		}
	});

	WidgetArgs.OnPasteFromClipboard_Lambda([this, InPropertyHandle, SetTransformComponent](ESlateTransformComponent::Type InTransformComponent)
	{
		FString Content;
		FPlatformApplicationMisc::ClipboardPaste(Content);
	
		if(Content.IsEmpty())
		{
			return;
		}

		TOptional<FReal> Result;
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return;
		}

		URigVMController* Controller = nullptr;
		if (TStrongObjectPtr<UObject> Blueprint = BlueprintBeingCustomized.GetWeakObjectPtr().Pin(); GraphBeingCustomized.IsValid())
		{
			Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized.Get());
			Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InPropertyHandle->GetProperty()->GetName()));
		}

		// Apply the new value
		{
			class FRigPasteTransformWidgetErrorPipe : public FOutputDevice
			{
			public:
				
				int32 NumErrors;
				
				FRigPasteTransformWidgetErrorPipe()
					: FOutputDevice()
					, NumErrors(0)
				{
				}
				
				virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
				{
					UE_LOGF(LogRigVM, Error, "Error Pasting to Widget: %ls", V);
					NumErrors++;
				}
			};
				
			FRigPasteTransformWidgetErrorPipe ErrorPipe;

			TransformType Transform;
			switch(InTransformComponent)
			{
				case ESlateTransformComponent::Location:
				{
					FVector Data = FVector::ZeroVector;
					TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
					Transform.SetLocation(Data);
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					FRotator Data = FRotator::ZeroRotator;
					TBaseStructure<FRotator>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName(), true);
					FQuat Quat = Data.Quaternion();
					Transform.SetRotation(Quat);
							
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					FVector Data = FVector::OneVector;
					TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
					Transform.SetScale3D(Data);
					break;
				}
				case ESlateTransformComponent::Max:
				default:
				{
					TBaseStructure<TransformType>::Get()->ImportText(*Content, &Transform, nullptr, PPF_None, &ErrorPipe, TBaseStructure<TransformType>::Get()->GetName(), true);
					break;
				}
			}
			SetTransformComponent(InTransformComponent, Transform);
		}		
		
		if (Controller)
		{
			Controller->CloseUndoBracket();
		}
	});
}

template<typename TransformType>
void FRigVMGraphMathTypeDetailCustomization::MakeTransformHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TConstArrayView<FName> ComponentNames)
{
	typename SAdvancedTransformInputBox<TransformType>::FArguments WidgetArgs;
	ConfigureTransformWidgetArgs<TransformType>(InPropertyHandle, WidgetArgs, ComponentNames);

	SAdvancedTransformInputBox<TransformType>::ConfigureHeader(HeaderRow, InPropertyHandle->GetPropertyDisplayName(), InPropertyHandle->GetToolTipText(), WidgetArgs);
	SAdvancedTransformInputBox<TransformType>::ConfigureComponentWidgetRow(HeaderRow, ESlateTransformComponent::Max, WidgetArgs);
}
	
template<typename TransformType>
void FRigVMGraphMathTypeDetailCustomization::MakeTransformChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TConstArrayView<FName> ComponentNames)
{
	typename SAdvancedTransformInputBox<TransformType>::FArguments WidgetArgs;
	ConfigureTransformWidgetArgs<TransformType>(InPropertyHandle, WidgetArgs, ComponentNames);

	const FName TranslationName = GetTranslationPropertyName<TransformType>();
	const FName RotationName = GetRotationPropertyName<TransformType>();
	const FName ScaleName = GetScalePropertyName<TransformType>();
	
	FDetailWidgetRow* LocationRow = &StructBuilder.AddProperty(InPropertyHandle->GetChildHandle(TranslationName).ToSharedRef()).CustomWidget();
	FDetailWidgetRow* RotationRow = &StructBuilder.AddProperty(InPropertyHandle->GetChildHandle(RotationName).ToSharedRef()).CustomWidget();
	FDetailWidgetRow* ScaleRow = &StructBuilder.AddProperty(InPropertyHandle->GetChildHandle(ScaleName).ToSharedRef()).CustomWidget();
	
	SAdvancedTransformInputBox<TransformType>::ConfigureComponentWidgetRow(*LocationRow, ESlateTransformComponent::Location, WidgetArgs);
	SAdvancedTransformInputBox<TransformType>::ConfigureComponentWidgetRow(*RotationRow, ESlateTransformComponent::Rotation, WidgetArgs);
	SAdvancedTransformInputBox<TransformType>::ConfigureComponentWidgetRow(*ScaleRow, ESlateTransformComponent::Scale, WidgetArgs);
}

FRigVMGraphMathTypeDetailCustomization::FRigVMGraphMathTypeDetailCustomization()
	: BlueprintBeingCustomized(nullptr)
	, GraphBeingCustomized(nullptr)
{
}

void FRigVMGraphMathTypeDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle,
                                                           FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	for (UObject* Object : Objects)
	{
		if (Object == nullptr)
		{
			continue;
		}

		if(BlueprintBeingCustomized == nullptr)
		{
			BlueprintBeingCustomized = IRigVMEditorAssetInterface::GetInterfaceOuter(Object);
		}

		if(GraphBeingCustomized == nullptr)
		{
			GraphBeingCustomized = Object->GetTypedOuter<URigVMGraph>();
		}
	}

	const FProperty* Property = InPropertyHandle->GetProperty();
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	const UScriptStruct* ScriptStruct = StructProperty->Struct;

	if(ScriptStruct == TBaseStructure<FVector>::Get())
	{
		MakeVectorHeaderRow<FVector, 3>(InPropertyHandle, HeaderRow, StructCustomizationUtils);
	}
	else if(ScriptStruct == TBaseStructure<FVector2D>::Get())
	{
		MakeVectorHeaderRow<FVector2D, 2>(InPropertyHandle, HeaderRow, StructCustomizationUtils);
	}
	else if(ScriptStruct == TBaseStructure<FVector4>::Get())
	{
		MakeVectorHeaderRow<FVector4, 4>(InPropertyHandle, HeaderRow, StructCustomizationUtils);
	}
	else if(ScriptStruct == TBaseStructure<FRotator>::Get())
	{
		MakeRotationHeaderRow<FRotator>(InPropertyHandle, HeaderRow, StructCustomizationUtils);
	}
	else if(ScriptStruct == TBaseStructure<FQuat>::Get())
	{
		MakeRotationHeaderRow<FQuat>(InPropertyHandle, HeaderRow, StructCustomizationUtils);
	}
	else if(ScriptStruct == TBaseStructure<FTransform>::Get())
	{
		MakeTransformHeaderRow<FTransform>(InPropertyHandle, HeaderRow, StructCustomizationUtils, TransformComponentNames);
	}
	else if(ScriptStruct == TBaseStructure<FEulerTransform>::Get())
	{
		MakeTransformHeaderRow<FEulerTransform>(InPropertyHandle, HeaderRow, StructCustomizationUtils, EulerTransformComponentNames);
	}
}

void FRigVMGraphMathTypeDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle,
	IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if(!InPropertyHandle->IsValidHandle())
	{
		return;
	}
	
	TArray<UObject*> Objects;
	InPropertyHandle->GetOuterObjects(Objects);

	for (UObject* Object : Objects)
	{
		if (Object == nullptr)
		{
			continue;
		}
		if(BlueprintBeingCustomized == nullptr)
		{
			BlueprintBeingCustomized = IRigVMEditorAssetInterface::GetInterfaceOuter(Object);
		}

		if(GraphBeingCustomized == nullptr)
		{
			GraphBeingCustomized = Object->GetTypedOuter<URigVMGraph>();
		}
	}

	const FProperty* Property = InPropertyHandle->GetProperty();
	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	const UScriptStruct* ScriptStruct = StructProperty->Struct;

	if(ScriptStruct == TBaseStructure<FTransform>::Get())
	{
		MakeTransformChildren<FTransform>(InPropertyHandle, StructBuilder, StructCustomizationUtils, TransformComponentNames);
	}
	else if(ScriptStruct == TBaseStructure<FEulerTransform>::Get())
	{
		MakeTransformChildren<FEulerTransform>(InPropertyHandle, StructBuilder, StructCustomizationUtils, EulerTransformComponentNames);
	}
}

#undef LOCTEXT_NAMESPACE
