// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMGraphEditorTabSummoner.h"
#include "Editor/RigVMNewEditor.h"
#include "Engine/Blueprint.h"
#include "GraphEditor.h"
#include "RigVMEditorAsset.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "RigVMGraphEditorTabSummoner"

FText FRigVMLocalKismetCallbacks::GetGraphDisplayName(const UEdGraph* Graph)
{
	if (Graph)
	{
		if (const UEdGraphSchema* Schema = Graph->GetSchema())
		{
			FGraphDisplayInfo Info;
			Schema->GetGraphDisplayInformation(*Graph, /*out*/ Info);

			return Info.DisplayName;
		}
		else
		{
			// if we don't have a schema, we're dealing with a malformed (or incomplete graph)...
			// possibly in the midst of some transaction - here we return the object's outer path 
			// so we can at least get some context as to which graph we're referring
			return FText::FromString(Graph->GetPathName());
		}
	}

	return LOCTEXT("UnknownGraphName", "UNKNOWN");
}

FRigVMGraphEditorTabSummoner::FRigVMGraphEditorTabSummoner(const TSharedRef<FRigVMNewEditor>& InRigVMEditor, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback)
: FDocumentTabFactoryForObjects<UEdGraph>(TabID(), InRigVMEditor.Get().GetHostingApp())
, BlueprintEditorPtr(InRigVMEditor)
, OnCreateGraphEditorWidget(CreateGraphEditorWidgetCallback)
{
}

void FRigVMGraphEditorTabSummoner::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	BlueprintEditorPtr.Pin()->OnGraphEditorFocused(GraphEditor);
}

void FRigVMGraphEditorTabSummoner::OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	BlueprintEditorPtr.Pin()->OnGraphEditorBackgrounded(GraphEditor);
}

void FRigVMGraphEditorTabSummoner::SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const
{
	// Mirrors FGraphEditorSummoner::SaveState from Kismet: captures the graph tab and its
	// view/zoom into the asset's LastEditedDocuments so PostActivateMode -> RestoreEditedObjectState
	// can reopen it on the next editor session.
	TSharedPtr<FRigVMNewEditor> Editor = BlueprintEditorPtr.Pin();
	if (!Editor.IsValid() || !Tab.IsValid() || !Payload.IsValid() || !Payload->IsValid())
	{
		return;
	}

	FRigVMEditorAssetInterfacePtr AssetInterface = Editor->GetRigVMAssetInterface();
	if (!AssetInterface)
	{
		return;
	}

	UEdGraph* Graph = FTabPayload_UObject::CastChecked<UEdGraph>(Payload);

	// Don't save references to external graphs (matches FGraphEditorSummoner::SaveState's
	// use of FBlueprintEditor::IsGraphInCurrentBlueprint). That method is protected on
	// FRigVMNewEditor, so we inline the equivalent check here.
	TArray<UEdGraph*> OwnedGraphs;
	AssetInterface->GetAllEdGraphs(OwnedGraphs);
	if (!OwnedGraphs.Contains(Graph))
	{
		return;
	}

	TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
	FVector2f ViewLocation;
	float ZoomAmount;
	GraphEditor->GetViewLocation(ViewLocation, ZoomAmount);

	AssetInterface->GetLastEditedDocuments().Add(FEditedDocumentInfo(Graph, ViewLocation, ZoomAmount));
}

TSharedRef<SWidget> FRigVMGraphEditorTabSummoner::CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	check(Info.TabInfo.IsValid());
	return OnCreateGraphEditorWidget.Execute(Info.TabInfo.ToSharedRef(), DocumentID);
}

const FSlateBrush* FRigVMGraphEditorTabSummoner::GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const
{
	return FRigVMNewEditor::GetGlyphForGraph(DocumentID, false);
}

#undef LOCTEXT_NAMESPACE