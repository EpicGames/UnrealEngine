// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_RIGVMLEGACYEDITOR
#include "Editor/RigVMLegacyEditor.h"
#include "SBlueprintEditorToolbar.h"
#include "SGraphPanel.h"
#include "SMyBlueprint.h"
#include "RigVMEditorModule.h"
#include "SBlueprintEditorSelectedDebugObjectWidget.h"
#include "RigVMEditorCommands.h"
#include "SScribbleGraph.h"

struct FRigVMEditorZoomLevelsContainer;

FRigVMLegacyEditor::FRigVMLegacyEditor()
{
	DocumentManager = MakeShareable(new FDocumentTracker(NAME_None));
}

void FRigVMLegacyEditor::OnClose()
{
	FRigVMEditorBase::UnbindEditor();
	return FBlueprintEditor::OnClose();
}

void FRigVMLegacyEditor::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier,
                                         const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar,
                                         const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons,
                                         const TOptional<EAssetOpenMethod>& InOpenMethod)
{
	FBlueprintEditor::InitAssetEditor(Mode, InitToolkitHost, AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit, bInIsToolbarFocusable, bInUseSmallToolbarIcons, InOpenMethod);
}

void FRigVMLegacyEditor::CreateEditorToolbar()
{
	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShareable(new FBlueprintEditorToolbar(FBlueprintEditor::SharedThis<FBlueprintEditor>(this)));
	}
}

void FRigVMLegacyEditor::CommonInitialization(const TArray<FRigVMEditorAssetInterfacePtr>& InitBlueprints, bool bShouldOpenInDefaultsMode)
{
	TArray<UBlueprint*> Blueprints;
	Algo::Transform(InitBlueprints, Blueprints, [](FRigVMEditorAssetInterfacePtr In)
	{
		return Cast<UBlueprint>(In->GetObject());
	});
	FBlueprintEditor::CommonInitialization(Blueprints, bShouldOpenInDefaultsMode);
}

TSharedPtr<FApplicationMode> FRigVMLegacyEditor::CreateEditorMode()
{
	return MakeShareable(new FRigVMLegacyEditorMode(StaticCastSharedRef<FRigVMLegacyEditor>(FBlueprintEditor::SharedThis<FBlueprintEditor>(this))));
}

const FName FRigVMLegacyEditor::GetEditorAppName() const
{
	static const FLazyName AppName(TEXT("RigVMLegacyEditorApp"));
	return AppName;
}

void FRigVMLegacyEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	FBlueprintEditor::OnGraphEditorFocused(InGraphEditor);
	FRigVMEditorBase::OnGraphEditorFocused(InGraphEditor);
}

void FRigVMLegacyEditor::AddCompileWidget(FToolBarBuilder& ToolbarBuilder)
{
	if (UToolMenu* ToolMenu = RegisterModeToolbarIfUnregistered(FRigVMEditorModes::RigVMEditorMode))
	{
		GetToolbarBuilder()->AddCompileToolbar(ToolMenu);
	}
}

void FRigVMLegacyEditor::AddSelectedDebugObjectWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddWidget(SNew(SBlueprintEditorSelectedDebugObjectWidget, SharedThis(this)));
}

void FRigVMLegacyEditor::AddAutoCompileWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddToolBarButton(FRigVMEditorCommands::Get().AutoCompileGraph,
		NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FRigVMEditorStyle::Get().GetStyleSetName(), "RigVM.AutoCompileGraph"));
}

void FRigVMLegacyEditor::Tick(float DeltaTime)
{
	FBlueprintEditor::Tick(DeltaTime);
	FRigVMEditorBase::Tick(DeltaTime);
}

void FRigVMLegacyEditor::GetCustomDebugObjects(TArray<FCustomDebugObject>& DebugList) const
{
	TArray<FRigVMCustomDebugObject> RigVMList;
	RigVMList.Reserve(DebugList.Num());
	for (FCustomDebugObject Obj : DebugList)
	{
		FRigVMCustomDebugObject RigVMObj;
		RigVMObj.Object = Obj.Object;
		RigVMObj.NameOverride = Obj.NameOverride;
		RigVMList.Add(RigVMObj);
	}

	GetDebugObjects(RigVMList);

	DebugList.Reset();
	DebugList.Reserve(RigVMList.Num());
	for (FRigVMCustomDebugObject RigVMObj : RigVMList)
	{
		FCustomDebugObject Obj;
		Obj.Object = RigVMObj.Object;
		Obj.NameOverride = RigVMObj.NameOverride;
		DebugList.Add(Obj);
	}
}

void FRigVMLegacyEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FBlueprintEditor::InitToolMenuContext(MenuContext);
	FRigVMEditorBase::InitToolMenuContextImpl(MenuContext);
}

bool FRigVMLegacyEditor::TransactionObjectAffectsBlueprint(UObject* InTransactedObject)
{
	return FRigVMEditorBase::TransactionObjectAffectsBlueprintImpl(InTransactedObject)
	&& FBlueprintEditor::TransactionObjectAffectsBlueprint(InTransactedObject);
}

FEdGraphPinType FRigVMLegacyEditor::GetLastPinTypeUsed()
{
	return MyBlueprintWidget->GetLastPinTypeUsed();
}

void FRigVMLegacyEditor::JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename)
{
	if (FRigVMEditorBase::JumpToHyperlinkImpl(ObjectReference, bRequestRename))
	{
		return;
	}
	FBlueprintEditor::JumpToHyperlink(ObjectReference, bRequestRename);
}

void FRigVMLegacyEditor::PostUndo(bool bSuccess)
{
	FBlueprintEditor::PostUndo(bSuccess);
	FRigVMEditorBase::PostUndoImpl(bSuccess);
}

void FRigVMLegacyEditor::PostRedo(bool bSuccess)
{
	FBlueprintEditor::PostRedo(bSuccess);
	FRigVMEditorBase::PostRedoImpl(bSuccess);
}

void FRigVMLegacyEditor::CreateDefaultCommands()
{
	if (FBlueprintEditor::GetBlueprintObj())
	{
		FBlueprintEditor::CreateDefaultCommands();
	}

	FRigVMEditorBase::CreateDefaultCommandsImpl();
}

TSharedRef<SGraphEditor> FRigVMLegacyEditor::CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph)
{
	TSharedRef<SGraphEditor> GraphEditor = FBlueprintEditor::CreateGraphEditorWidget(InTabInfo, InGraph);
		GraphEditor->GetGraphPanel()->SetZoomLevelsContainer<FRigVMEditorZoomLevelsContainer>();

	TWeakObjectPtr<URigVMEdGraph> WeakGraph(Cast<URigVMEdGraph>(InGraph));
	if (WeakGraph.IsValid())
	{
		if (URigVMGraph* Model = WeakGraph->GetModel())
		{
			if (CVarScribbleEnabled.GetValueOnAnyThread())
			{
				TSharedRef<SWidget> GraphPanelWidget = GraphEditor->GetGraphPanel()->AsShared();
				if (TSharedPtr<SWidget> ParentWidget = GraphPanelWidget->GetParentWidget())
				{
					if (ParentWidget->GetWidgetClass().GetWidgetType() == SOverlay::StaticWidgetClass().GetWidgetType())
					{
						TSharedPtr<SOverlay> Overlay = StaticCastSharedPtr<SOverlay>(ParentWidget);
						if (Overlay)
						{
							Overlay->AddSlot()
							.Padding(0.f)
							.VAlign(VAlign_Fill)
							.HAlign(HAlign_Fill)
							[
								SNew(SScribbleGraph)
								.GraphData(Model->SetupScribbleGraph(true))
								.AllowNavigation(true)
								.ShouldDrawBackground(false)
								.ShouldDrawSurroundingShadow(false)
								.ToolbarPadding(FMargin(8,44,0,0))
								.PanelToSync(StaticCastWeakPtr<SGraphPanel,SWidget>(GraphPanelWidget.ToWeakPtr()))
							];
						}
					}
				}
			}
		}
		
		TWeakPtr<FRigVMEditorBase> WeakThisEditor = SharedThis(this).ToWeakPtr();
		TWeakPtr<SGraphEditor> WeakGraphEditor = GraphEditor.ToWeakPtr();
		
		GraphEditor->SetOnTick(FOnGraphEditorTickDelegate::CreateLambda([WeakThisEditor, WeakGraphEditor, WeakGraph]
			(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
			{
				if (WeakThisEditor.IsValid() && WeakGraphEditor.IsValid() && WeakGraph.IsValid())
				{
					WeakThisEditor.Pin()->OnGraphEditorTick(AllottedGeometry, InCurrentTime, InDeltaTime, WeakGraphEditor.Pin().ToSharedRef(), WeakGraph.Get());
				}
			}
		));
	}
	return GraphEditor;
}

void FRigVMLegacyEditor::RefreshEditorsImpl(ERefreshRigVMEditorReason::Type Reason)
{
	FBlueprintEditor::RefreshEditors((ERefreshBlueprintEditorReason::Type)Reason);
}

bool FRigVMLegacyEditor::NewDocument_IsVisibleForType(FBlueprintEditor::ECreatedDocumentType GraphType) const
{
	if (FRigVMEditorBase::NewDocument_IsVisibleForTypeImpl((FRigVMEditorBase::ECreatedDocumentType)GraphType))
	{
		return FBlueprintEditor::NewDocument_IsVisibleForType(GraphType);
	}
	return false;
}

bool FRigVMLegacyEditor::IsSectionVisible(NodeSectionID::Type InSectionID) const
{
	return FRigVMEditorBase::IsSectionVisibleImpl((RigVMNodeSectionID::Type)InSectionID);
}


FName FRigVMLegacyEditor::GetSelectedVariableName()
{
	FName VariableName;
	if (MyBlueprintWidget.IsValid())
	{
		if (FEdGraphSchemaAction_BlueprintVariableBase* VariableAcion = MyBlueprintWidget->SelectionAsBlueprintVariable())
		{
			return VariableAcion->GetVariableName();
		}
	}
	return VariableName;
}

#endif
