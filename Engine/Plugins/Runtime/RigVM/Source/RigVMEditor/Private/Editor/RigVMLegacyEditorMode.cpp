// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_RIGVMLEGACYEDITOR

#include "Editor/RigVMLegacyEditorMode.h" 
#include "BlueprintEditorTabs.h"
#include "SBlueprintEditorToolbar.h"
#include "Editor/RigVMExecutionStackTabSummoner.h"
#include "Editor/RigVMEditorGraphExplorerTabSummoner.h"
#include "ToolMenus.h"
#include "Editor/RigVMEditor.h"
#include "Editor/RigVMLegacyEditor.h"

FRigVMLegacyEditorMode::FRigVMLegacyEditorMode(const TSharedRef<FRigVMLegacyEditor>& InRigVMEditor)
	: FBlueprintEditorApplicationMode(InRigVMEditor, FRigVMEditorModes::RigVMEditorMode, FRigVMEditorModes::GetLocalizedMode, false, false)
{
	RigVMBlueprintPtr = CastChecked<URigVMBlueprint>(InRigVMEditor->GetRigVMAssetInterface().GetObject());

	TabFactories.RegisterFactory(MakeShared<FRigVMExecutionStackTabSummoner>(InRigVMEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMEditorGraphExplorerTabSummoner>(InRigVMEditor));

	TabLayout = FTabManager::NewLayout("Standalone_RigVMEditMode_Layout_v1.6")
		->AddArea
		(
			// Main application area
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					//	Left bottom - rig/hierarchy
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(FRigVMExecutionStackTabSummoner::TabID, ETabState::OpenedTab)
					->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					->AddTab(FRigVMEditorGraphExplorerTabSummoner::TabID(), ETabState::ClosedTab)
				)
				->Split
				(
					// Middle 
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.6f)
					->Split
					(
						// Middle top - document edit area
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->AddTab("GraphEditor", ETabState::ClosedTab)
					)
					->Split
					(
						// Middle bottom - compiler results & find
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
						->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
					)
				)
				->Split
				(
					// Right side
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						// Right top
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(1.f)
						->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
						->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
					)
				)
			)
		);

	if (UToolMenu* Toolbar = InRigVMEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InRigVMEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InRigVMEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InRigVMEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
	}
}

void FRigVMLegacyEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
	BP->PushTabFactories(TabFactories);
}
#endif