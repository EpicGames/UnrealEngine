// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMLocalVariableDetailCustomization.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "SPinTypeSelector.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMCore/RigVM.h"
#include "InstancedPropertyBagStructureDataProvider.h"
#include "RigVMDeveloperTypeUtils.h"

#define LOCTEXT_NAMESPACE "LocalVariableDetails"

void FRigVMLocalVariableDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ObjectsBeingCustomized.Reset();

	TArray<TWeakObjectPtr<UObject>> DetailObjects;
	DetailBuilder.GetObjectsBeingCustomized(DetailObjects);
	for(TWeakObjectPtr<UObject> DetailObject : DetailObjects)
	{
		URigVMDetailsViewWrapperObject* WrapperObject = CastChecked<URigVMDetailsViewWrapperObject>(DetailObject.Get());
		if (!WrapperObject->IsActive())
		{
			continue;
		}
		
		ObjectsBeingCustomized.Add(WrapperObject);
	}

	if (ObjectsBeingCustomized.Num() > 0 && ObjectsBeingCustomized[0].IsValid())
	{
		VariableDescription = ObjectsBeingCustomized[0]->GetContent<FRigVMGraphVariableDescription>();
		if (UObject* Subject = ObjectsBeingCustomized[0]->GetSubject())
		{
			GraphBeingCustomized = Cast<URigVMGraph>(ObjectsBeingCustomized[0]->GetSubject());
		}
		if (ensure(GraphBeingCustomized))
		{
			BlueprintBeingCustomized = IRigVMEditorAssetInterface::GetInterfaceOuter(GraphBeingCustomized);
		}

		NameValidator = FRigVMLocalVariableNameValidator(BlueprintBeingCustomized, GraphBeingCustomized, VariableDescription.Name);
	}

	DetailBuilder.HideCategory(TEXT("RigVMGraphVariableDescription"));
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("Local Variable"));
		
	NameHandle = DetailBuilder.GetProperty(TEXT("Name"));
	TypeHandle = DetailBuilder.GetProperty(TEXT("CPPType"));
	TypeObjectHandle = DetailBuilder.GetProperty(TEXT("CPPTypeObject"));
	DefaultValueHandle = DetailBuilder.GetProperty(TEXT("DefaultValue"));
	
	const UEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

	const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
	Category.AddCustomRow( LOCTEXT("LocalVariableName", "Variable Name") )
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("LocalVariableName", "Variable Name"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	[
		SNew(SEditableTextBox)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(this, &FRigVMLocalVariableDetailCustomization::GetName)
		.OnTextCommitted(this, &FRigVMLocalVariableDetailCustomization::SetName)
		.OnVerifyTextChanged(this, &FRigVMLocalVariableDetailCustomization::OnVerifyNameChanged)
	];

	Category.AddCustomRow(LOCTEXT("VariableTypeLabel", "Variable Type"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("VariableTypeLabel", "Variable Type"))
			.Font(DetailFontInfo)
		]
		.ValueContent()
		.MaxDesiredWidth(980.f)
		[
			SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
			.TargetPinType(this, &FRigVMLocalVariableDetailCustomization::OnGetPinInfo)
			.OnPinTypeChanged(this, &FRigVMLocalVariableDetailCustomization::HandlePinInfoChanged)
			.Schema(Schema)
			.TypeTreeFilter(ETypeTreeFilter::None)
			.Font(DetailFontInfo)
		];


	if (GraphBeingCustomized)
	{
		FString SourcePath = FString::Printf(TEXT("LocalVariableDefault::%s|%s::Const"), *GraphBeingCustomized->GetGraphName(), *VariableDescription.Name.ToString());

		// Here, we create a custom storage for the Details, so when it changes,
		// we extract the value and then update the default value handle with it,
		// which in turn will call URigVMDetailsViewWrapperObject::PostEditChangeChainProperty
		// and then the variable default value will be set through the graph controller
		// SetLocalVariableDefaultValue function
		TArray<FRigVMPropertyDescription> PropertyDescriptions;
		FRigVMPropertyDescription& PropertyDesc = PropertyDescriptions.AddDefaulted_GetRef();
		PropertyDesc.Name = *SourcePath;
		PropertyDesc.Property = nullptr;
		PropertyDesc.CPPType = VariableDescription.CPPType;
		PropertyDesc.CPPTypeObject = VariableDescription.CPPTypeObject;
		if (RigVMTypeUtils::IsArrayType(PropertyDesc.CPPType))
		{
			PropertyDesc.Containers.Add(EPinContainerType::Array);
		}
		PropertyDesc.DefaultValue = VariableDescription.DefaultValue;

		MemoryStorage = MakeShared<FRigVMMemoryStorageStruct>();
		MemoryStorage->AddProperties(PropertyDescriptions);

		FRigVMMemoryStorageStruct* Memory = MemoryStorage.Get();
		const FName SanitizedName = FInstancedPropertyBag::SanitizePropertyName(SourcePath);
		FProperty* Property = Memory->FindPropertyByName(SanitizedName);
		if (Property)
		{
			IDetailCategoryBuilder& DefaultValueCategory = DetailBuilder.EditCategory(TEXT("DefaultValueCategory"), LOCTEXT("DefaultValueCategoryHeading", "Default Value"));
			Property->ClearPropertyFlags(CPF_EditConst);
			Property->PropertyFlags |= CPF_Edit;
			
			if (IDetailPropertyRow* Row = DefaultValueCategory.AddExternalStructureProperty(MakeShared<FInstancePropertyBagStructureDataProvider>(*Memory), SanitizedName))
			{
				Row->DisplayName(FText::FromName(VariableDescription.Name));

				const FSimpleDelegate OnDefaultValueChanged = FSimpleDelegate::CreateLambda([this, Property]()
				{
					const int32 PropertyIndex = MemoryStorage->GetPropertyIndex(Property);
					if (MemoryStorage->IsValidIndex(PropertyIndex))
					{
						VariableDescription.DefaultValue = MemoryStorage->GetDataAsStringSafe(PropertyIndex);
						DefaultValueHandle->SetValue(VariableDescription.DefaultValue);
					}
				});

				TSharedPtr<IPropertyHandle> Handle = Row->GetPropertyHandle();
				Handle->SetOnPropertyValueChanged(OnDefaultValueChanged);
				Handle->SetOnChildPropertyValueChanged(OnDefaultValueChanged);
			}
		}
	}
}

FText FRigVMLocalVariableDetailCustomization::GetName() const
{
	return FText::FromName(VariableDescription.Name);
}

void FRigVMLocalVariableDetailCustomization::SetName(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if(InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (InNewText.ToString() == VariableDescription.Name.ToString())
	{
		return;
	}

	VariableDescription.Name =  *InNewText.ToString();
	NameHandle->SetValue(VariableDescription.Name);
}

bool FRigVMLocalVariableDetailCustomization::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	EValidatorResult Result = NameValidator.IsValid(InText.ToString(), false);
	OutErrorMessage = INameValidatorInterface::GetErrorText(InText.ToString(), Result);	

	return Result == EValidatorResult::Ok || Result == EValidatorResult::ExistingName;
}

FEdGraphPinType FRigVMLocalVariableDetailCustomization::OnGetPinInfo() const
{
	if (!VariableDescription.Name.IsNone())
	{
		const FEdGraphPinType EdPinType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(VariableDescription);
		return EdPinType;
	}
	return FEdGraphPinType();
}

void FRigVMLocalVariableDetailCustomization::HandlePinInfoChanged(const FEdGraphPinType& PinType)
{
	FString CPPType;
	UObject* CPPTypeObject = nullptr;
	RigVMTypeUtils::CPPTypeFromPinType(PinType, CPPType, &CPPTypeObject);
	VariableDescription.ChangeType(CPPType, CPPTypeObject);
	FRigVMBlueprintCompileScope CompileScope(BlueprintBeingCustomized);
	TypeHandle->SetValue(VariableDescription.CPPType);
	TypeObjectHandle->SetValue(VariableDescription.CPPTypeObject);	
}

ECheckBoxState FRigVMLocalVariableDetailCustomization::HandleBoolDefaultValueIsChecked() const
{
	return VariableDescription.DefaultValue == "1" ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FRigVMLocalVariableDetailCustomization::OnBoolDefaultValueChanged(ECheckBoxState InCheckBoxState)
{
	VariableDescription.DefaultValue = InCheckBoxState == ECheckBoxState::Checked ? "1" : "0";
	DefaultValueHandle->SetValue(VariableDescription.DefaultValue);
}

#undef LOCTEXT_NAMESPACE
