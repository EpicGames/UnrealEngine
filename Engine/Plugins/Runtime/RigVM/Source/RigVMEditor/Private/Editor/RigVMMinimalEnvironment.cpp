// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMMinimalEnvironment.h"
#include "RigVMBlueprintLegacy.h"
#include "Editor/RigVMEditorTools.h"

#define LOCTEXT_NAMESPACE "RigVMMinimalEnvironment"

FRigVMMinimalEnvironment::FRigVMMinimalEnvironment(const IRigVMAssetInterface* InAssetInterface)
{
	UObject* Outer = GetTransientPackage();
	if (InAssetInterface)
	{
		if (const UObject* AssetObject = InAssetInterface->GetObject())
		{
			Outer = const_cast<UObject*>(AssetObject);
		}
		WeakRigVMClientHost = InAssetInterface->GetRigVMClientHost();
	}
	
	InitGraphData(Outer);
	
	SetSchemata(InAssetInterface);
}

FRigVMMinimalEnvironment::FRigVMMinimalEnvironment(IRigVMClientHost* InRigVMClientHost, UObject* InAssetObject)
{
	UObject* Outer = InAssetObject ? InAssetObject : GetTransientPackage();
	WeakRigVMClientHost = InRigVMClientHost;
	
	InitGraphData(Outer);
	
	SetSchemata(InRigVMClientHost, InAssetObject);
}

void FRigVMMinimalEnvironment::InitGraphData(UObject* InOuter)
{
	check(InOuter);
	
	ModelController = TStrongObjectPtr<URigVMController>(NewObject<URigVMController>(InOuter));
	ModelGraph = TStrongObjectPtr<URigVMGraph>(NewObject<URigVMGraph>(ModelController.Get(), TEXT("PreviewGraph")));
	ModelController->SetGraph(ModelGraph.Get());
	
	EdGraphClass = URigVMEdGraph::StaticClass();
	EdGraphNodeClass = URigVMEdGraphNode::StaticClass();
}

URigVMGraph* FRigVMMinimalEnvironment::GetModel() const
{
	return ModelGraph.Get();
}

URigVMController* FRigVMMinimalEnvironment::GetController() const
{
	return ModelController.Get();
}

URigVMNode* FRigVMMinimalEnvironment::GetNode() const
{
	if(ModelNode.IsValid())
	{
		return ModelNode.Get();
	}
	return nullptr;
}

void FRigVMMinimalEnvironment::SetNode(URigVMNode* InModelNode)
{
	if(!ModelHandle.IsValid())
	{
		ModelHandle = ModelGraph->OnModified().AddSP(this, &FRigVMMinimalEnvironment::HandleModified);
	}
	if(URigVMNode* PreviousNode = GetNode())
	{
		GetController()->RemoveNode(PreviousNode);
	}
	
	ModelNode = InModelNode;

	if(GetEdGraphNode() == nullptr)
	{
		if(URigVMEdGraph* MyEdGraph = GetEdGraph())
		{
			EdGraphNode = NewObject<URigVMEdGraphNode>(MyEdGraph, EdGraphNodeClass);
			EdGraphNode->SyncGraphNodeNameWithModelNodeName(InModelNode);
			MyEdGraph->Nodes.Add(EdGraphNode.Get());
		}
	}

	HandleModified(ERigVMGraphNotifType::NodeAdded, GetModel(), GetNode());

	// process the queued up events
	Tick_GameThead(0.f);
}

URigVMNode* FRigVMMinimalEnvironment::SetFunctionReference(URigVMLibraryNode* InFunctionDefinition, const FString& InNodeName)
{
	URigVMNode* FunctionRefNode = GetController()->AddFunctionReferenceNode(InFunctionDefinition, FVector2D::ZeroVector, InNodeName);
	if (FunctionRefNode)
	{
		SetNode(FunctionRefNode);
	}
	return FunctionRefNode;
}

URigVMNode* FRigVMMinimalEnvironment::SetEmptyCollapseNode(URigVMCollapseNode* InSourceCollapseNode, const FString& InNodeName)
{
	URigVMCollapseNode* CollapseNode = NewObject<URigVMCollapseNode>(GetModel(), *InNodeName);
	if (CollapseNode)
	{
		FString ContainedGraphName = CollapseNode->GetName() + TEXT("_ContainedGraph");
		if (URigVMGraph* ContainedGraph = NewObject<URigVMGraph>(CollapseNode, *ContainedGraphName))
		{
			ContainedGraph->SetSchemaClass(GetModel()->GetSchemaClass());
			CollapseNode->ContainedGraph = ContainedGraph;
		}

		GetController()->AddGraphNode(CollapseNode, false);
		SetNode(CollapseNode);
		SynchronizeCollapseNode(InSourceCollapseNode);
	}
	return CollapseNode;
}

void FRigVMMinimalEnvironment::SynchronizeCollapseNode(URigVMCollapseNode* InSource)
{
	if (!InSource)
	{
		return;
	}

	URigVMCollapseNode* Target = Cast<URigVMCollapseNode>(GetNode());
	if (!Target)
	{
		return;
	}

	IRigVMClientHost* RigVMClientHost = WeakRigVMClientHost.Get();
	if (!RigVMClientHost)
	{
		return;
	}

	FRigVMClient* Client = RigVMClientHost->GetRigVMClient();
	if (!Client)
	{
		return;
	}

	URigVMController* SourceController = Client->GetOrCreateController(InSource->GetGraph());
	if (!SourceController)
	{
		return;
	}

	URigVMController* TargetController = GetController();
	if (!TargetController)
	{
		return;
	}

	TArray<URigVMController::FRepopulatePinsNodeData> SourceNodesPinData, TargetNodesPinData;
	// Only the top-level node's pin info is consumed below (SetNum(1, ...)) — pass bInRecurseIntoCollapseNodes=false
	// to skip the recursive walk into the collapse node's contained graph. The recursion would otherwise call
	// GetControllerForGraph on the source's contained graph and create a cached controller in the asset's
	// FRigVMClient::Controllers map as a side effect.
	SourceController->GenerateRepopulatePinsNodeData(SourceNodesPinData, InSource, false, false, false, false);
	TargetController->GenerateRepopulatePinsNodeData(TargetNodesPinData, Target, false, false, false, false);
	if (SourceNodesPinData.IsEmpty() || TargetNodesPinData.IsEmpty())
	{
		return;
	}
	SourceNodesPinData.SetNum(1, EAllowShrinking::No);
	TargetNodesPinData.SetNum(1, EAllowShrinking::No);
	
	TargetNodesPinData[0].NewPinInfos = SourceNodesPinData[0].NewPinInfos;
	{
		const FRigVMRegistry& Registry = FRigVMRegistry::Get();
		TargetController->GenerateRepopulatePinLists(Registry, TargetNodesPinData[0]);
	}
	TargetController->RepopulatePins(TargetNodesPinData);
}

void FRigVMMinimalEnvironment::SetFunctionNode(const FRigVMGraphFunctionIdentifier& InIdentifier)
{
	check(InIdentifier.IsValid());

	const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeader(InIdentifier);
	if(Header.IsValid())
	{
		const FAssetData AssetData = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(Header.LibraryPointer.GetLibraryNodePath(), true);
		if(AssetData.IsValid())
		{
			if(const UClass* Class = AssetData.GetClass())
			{
				if(Class->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()))
				{
					if (const URigVMNode* Node = GetNode())
					{
						SetSchemata(Node->GetImplementingOuter<IRigVMAssetInterface>());
					}
				}
			}
		}

		if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(GetNode()))
		{
			(void)GetController()->SwapFunctionReference(FunctionReferenceNode, InIdentifier, false, false, false);
		}
		else
		{
			URigVMNode* Node = GetController()->AddFunctionReferenceNodeFromDescription(Header, FVector2D::ZeroVector, FString(), false, false);
			SetNode(Node);
		}
	}
}

URigVMEdGraph* FRigVMMinimalEnvironment::GetEdGraph() const
{
	return EdGraph.Get();
}

URigVMEdGraphNode* FRigVMMinimalEnvironment::GetEdGraphNode() const
{
	if(EdGraphNode.IsValid())
	{
		return EdGraphNode.Get();
	}
	return nullptr;
}

void FRigVMMinimalEnvironment::SetSchemata(const IRigVMAssetInterface* InAssetInterface)
{
	if (!InAssetInterface)
	{
		return;
	}

	const TScriptInterface<IRigVMClientHost> ClientHost = InAssetInterface->GetRigVMClientHost();
	check(ClientHost);

	SetSchemata(ClientHost.GetInterface(), InAssetInterface->GetObject());
}

void FRigVMMinimalEnvironment::SetSchemata(const IRigVMClientHost* InRigVMClientHost, const UObject* InAssetObject)
{
	if (!InRigVMClientHost)
	{
		return;
	}
	
	EdGraphClass = InRigVMClientHost->GetRigVMEdGraphClass();
	EdGraphNodeClass = InRigVMClientHost->GetRigVMEdGraphNodeClass();
	
	if(!EdGraph.IsValid() || EdGraph->GetClass() != EdGraphClass)
	{
		EdGraph = TStrongObjectPtr<URigVMEdGraph>(NewObject<URigVMEdGraph>(ModelGraph.Get(), EdGraphClass));
		EdGraph->ModelNodePath = ModelGraph->GetNodePath();
	}
	
	ModelController->SetSchemaClass(InRigVMClientHost->GetRigVMSchemaClass());
	ModelGraph->SetSchemaClass(InRigVMClientHost->GetRigVMSchemaClass());
	if (InAssetObject)
	{
		EdGraph->SetBlueprintClass(InAssetObject->GetClass());
	}
	else
	{
		EdGraph->Schema = InRigVMClientHost->GetRigVMEdGraphSchemaClass();
	}
}

FSimpleDelegate& FRigVMMinimalEnvironment::OnChanged()
{
	return ChangedDelegate;
}

void FRigVMMinimalEnvironment::Tick_GameThead(float InDeltaTime)
{
	if(NumModifications.exchange(0) > 0)
	{
		// refresh the EdGraphNode
		if(URigVMNode* MyModelNode = GetNode())
		{
			if(URigVMEdGraphNode* MyEdGraphNode = GetEdGraphNode())
			{
				MyEdGraphNode->SetSubTitleEnabled(false);
				MyEdGraphNode->SetModelNode(MyModelNode);
				(void)ChangedDelegate.ExecuteIfBound();
			}
		}
	}
}

void FRigVMMinimalEnvironment::HandleModified(ERigVMGraphNotifType InNotification, URigVMGraph* InGraph, UObject* InSubject)
{
	if(InGraph != GetModel())
	{
		return;
	}

	switch(InNotification)
	{
		case ERigVMGraphNotifType::NodeSelected:
		case ERigVMGraphNotifType::NodeDeselected:
		case ERigVMGraphNotifType::PinExpansionChanged:
		case ERigVMGraphNotifType::InteractionBracketOpened:
		case ERigVMGraphNotifType::InteractionBracketClosed:
		case ERigVMGraphNotifType::InteractionBracketCanceled:
		case ERigVMGraphNotifType::PinCategoryChanged:
		case ERigVMGraphNotifType::PinCategoriesChanged:
		case ERigVMGraphNotifType::PinCategoryExpansionChanged:
		{
			break;
		}
		case ERigVMGraphNotifType::PinTypeChanged:
		default:
		{
			(void)NumModifications.fetch_add(1);
		}
	}
}

#undef LOCTEXT_NAMESPACE
