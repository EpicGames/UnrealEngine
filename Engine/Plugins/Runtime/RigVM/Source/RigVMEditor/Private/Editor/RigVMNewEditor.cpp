// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMNewEditor.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintEditorSettings.h"
#include "FileHelpers.h"
#include "BlueprintEditorTabs.h"
#include "IMessageLogListing.h"
#include "K2Node_Composite.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMSettings.h"
#include "SGraphPanel.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "SRigVMGraphTitleBar.h"
#include "UnrealEdGlobals.h"
#include "Editor/SRigVMActionMenu.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UObject/MetaData.h"
#include "Widgets/Docking/SDockTab.h"
#include "ScopedTransaction.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Stats/StatsHierarchical.h"
#include "Logging/MessageLog.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "MessageLogModule.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "RigVMEditorCommands.h"
#include "Editor/RigVMFindReferences.h"
#include "Editor/RigVMCompilerResultsTabSummoner.h"
#include "Editor/RigVMDetailsInspectorTabSummoner.h"
#include "Editor/RigVMFindReferencesTabSummoner.h"
#include "Editor/RigVMGraphEditorTabSummoner.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/UObjectToken.h"
#include "Widgets/SRigVMEditorSelectedDebugObjectWidget.h"


#define LOCTEXT_NAMESPACE "RigVMNewEditor"

const FName FRigVMNewEditor::SelectionState_GraphExplorer() {	static FName State = (TEXT("GraphExplorer")); return State; };
const FName FRigVMNewEditor::SelectionState_Graph() {			static FName State = (TEXT("Graph")); return State; };
const FName FRigVMNewEditor::SelectionState_ClassSettings() {	static FName State = (TEXT("ClassSettings")); return State; };
const FName FRigVMNewEditor::SelectionState_ClassDefaults() {	static FName State = (TEXT("ClassDefaults")); return State; };

namespace RigVMNewEditorImpl
{
	static const float InstructionFadeDuration = 0.5f;
	
	/**
	 * Searches through a blueprint, looking for the most severe error'ing node.
	 * 
	 * @param  Blueprint	The blueprint to search through.
	 * @param  Severity		Defines the severity of the error/warning to search for.
	 * @return The first node found with the specified error.
	 */
	static UEdGraphNode* FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity = EMessageSeverity::Error);

	/**
	 * Searches through an error log, looking for the most severe error'ing node.
	 * 
	 * @param  ErrorLog		The error log you want to search through.
	 * @param  Severity		Defines the severity of the error/warning to search for.
	 * @return The first node found with the specified error.
	 */
	static UEdGraphNode* FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity = EMessageSeverity::Error);

	/**
	 * Utility function that will check to see if the specified graph has any
	 * nodes that were default, pre-placed, in the graph.
	 *
	 * @param  InGraph  The graph to check.
	 * @return True if the graph has any pre-placed nodes, otherwise false.
	 */
	static bool GraphHasDefaultNode(UEdGraph const* InGraph);

	/**
	 * Utility function that will check to see if the specified graph has any 
	 * nodes other than those that come default, pre-placed, in the graph.
	 *
	 * @param  InGraph  The graph to check.
	 * @return True if the graph has any nodes added by the user, otherwise false.
	 */
	static bool GraphHasUserPlacedNodes(UEdGraph const* InGraph);
}

static UEdGraphNode* RigVMNewEditorImpl::FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	TArray<UEdGraph*> Graphs;
	Blueprint->GetAllGraphs(Graphs);

	UEdGraphNode* ChoiceNode = nullptr;
	for (UEdGraph* Graph : Graphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (Node && Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && (Node->ErrorType <= Severity))
			{
				if ((ChoiceNode == nullptr) || (ChoiceNode->ErrorType > Node->ErrorType))
				{
					ChoiceNode = Node;
					if (ChoiceNode->ErrorType == 0)
					{
						break;
					}
				}
			}
		}
	}
	return ChoiceNode;
}

static UEdGraphNode* RigVMNewEditorImpl::FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	UEdGraphNode* ChoiceNode = nullptr;
	for (TWeakObjectPtr<UEdGraphNode> NodePtr : ErrorLog.AnnotatedNodes)
	{
		UEdGraphNode* Node = NodePtr.Get();
		if ((Node != nullptr) && (Node->ErrorType <= Severity))
		{
			if ((ChoiceNode == nullptr) || (Node->ErrorType < ChoiceNode->ErrorType))
			{
				ChoiceNode = Node;
				if (ChoiceNode->ErrorType == 0)
				{
					break;
				}
			}
		}
	}

	return ChoiceNode;
}

static bool RigVMNewEditorImpl::GraphHasDefaultNode(UEdGraph const* InGraph)
{
	bool bHasDefaultNodes = false;

	for (UEdGraphNode const* Node : InGraph->Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (Node->GetPackage()->GetMetaData().HasValue(Node, FNodeMetadata::DefaultGraphNode) && Node->IsNodeEnabled())
		{
			bHasDefaultNodes = true;
			break;
		}
	}

	return bHasDefaultNodes;
}

static bool RigVMNewEditorImpl::GraphHasUserPlacedNodes(UEdGraph const* InGraph)
{
	bool bHasUserPlacedNodes = false;

	for (UEdGraphNode const* Node : InGraph->Nodes)
	{
		if (Node == nullptr)
		{
			continue;
		}

		if (!Node->GetPackage()->GetMetaData().HasValue(Node, FNodeMetadata::DefaultGraphNode))
		{
			bHasUserPlacedNodes = true;
			break;
		}
	}

	return bHasUserPlacedNodes;
}

FRigVMNewEditor::FRigVMNewEditor()
	: FRigVMEditorBase()
	, bIsActionMenuContextSensitive(true)
{
	DocumentManager = MakeShareable(new FDocumentTracker(NAME_None));
}

void FRigVMNewEditor::OnClose()
{
	FRigVMEditorBase::UnbindEditor();
	FWorkflowCentricApplication::OnClose();
}

void FRigVMNewEditor::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier,
                                      const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar,
                                      const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons,
                                      const TOptional<EAssetOpenMethod>& InOpenMethod)
{
	FWorkflowCentricApplication::InitAssetEditor(Mode, InitToolkitHost, AppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsToEdit, bInIsToolbarFocusable, bInUseSmallToolbarIcons, InOpenMethod);
}

void FRigVMNewEditor::HandlePreVariablesChanged(FName InName)
{
	if (Inspector && GraphExplorerWidget)
	{
		if (GraphExplorerWidget->GetSelectedType() == ERigVMExplorerElementType::Variable)
		{
			ElementInDetailsPanel = GraphExplorerWidget->GetSelectedVariableName();
			ClearDetailObject();
		}
	}
}

void FRigVMNewEditor::HandlePostVariablesChanged(FName InName)
{
	// Refresh Graph Explorer to show updated variable state
	if (GraphExplorerWidget.IsValid())
	{
		GraphExplorerWidget->Refresh();
	}

	// Refresh details panel
	if (Inspector)
	{
		SelectVariable(ElementInDetailsPanel);
		ElementInDetailsPanel = NAME_None;
	}
}

void FRigVMNewEditor::CommonInitialization(const TArray<FRigVMEditorAssetInterfacePtr>& InitBlueprints, bool bShouldOpenInDefaultsMode)
{
	TSharedPtr<FRigVMNewEditor> ThisPtr = SharedThis(this);

	// @todo TabManagement
	DocumentManager->Initialize(ThisPtr);

	// Register the document factories
	{
		// TODO sara-s
		//DocumentManager->RegisterDocumentFactory(MakeShareable(new FTimelineEditorSummoner(ThisPtr)));

		TSharedRef<FRigVMNewEditor> SharedRef = StaticCastSharedRef<FRigVMNewEditor>(SharedThis(this));
		TSharedRef<FDocumentTabFactory> GraphEditorFactory = MakeShareable(new FRigVMGraphEditorTabSummoner(SharedRef,
			FRigVMGraphEditorTabSummoner::FOnCreateGraphEditorWidget::CreateSP(SharedRef, &FRigVMNewEditor::CreateGraphEditorWidget)
			));

		// Also store off a reference to the grapheditor factory so we can find all the tabs spawned by it later.
		GraphEditorTabFactoryPtr = GraphEditorFactory;
		DocumentManager->RegisterDocumentFactory(GraphEditorFactory);
	}

	// Create a namespace helper to keep track of imports for all BPs being edited.
	//ImportedNamespaceHelper = MakeShared<FBlueprintNamespaceHelper>();

	// Add each Blueprint instance to be edited into the namespace helper's context.
	// for (const UBlueprint* BP : InitBlueprints)
	// {
	// 	ImportedNamespaceHelper->AddBlueprint(BP);
	// }

	// Create imported namespace type filters for value editing.
	// ImportedClassViewerFilter = MakeShared<BlueprintEditorImpl::FImportedClassViewerFilterProxy>(ImportedNamespaceHelper->GetClassViewerFilter());
	// ImportedPinTypeSelectorFilter = MakeShared<BlueprintEditorImpl::FImportedPinTypeSelectorFilterProxy>(ImportedNamespaceHelper->GetPinTypeSelectorFilter());
	// PermissionsPinTypeSelectorFilter = MakeShared<BlueprintEditorImpl::FPermissionsPinTypeSelectorFilter>(InitBlueprints);

	// Make sure we know when tabs become active to update details tab
	//OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe( FOnActiveTabChanged::FDelegate::CreateRaw(this, &FBlueprintEditor::OnActiveTabChanged) );

	if (InitBlueprints.Num() == 1)
	{
		if (!bShouldOpenInDefaultsMode)
		{
			// Load blueprint libraries
			// if (ShouldLoadBPLibrariesFromAssetRegistry())
			// {
			// 	LoadLibrariesFromAssetRegistry();
			// }

			// Init the action DB for the context menu/palette if not already constructed
			FBlueprintActionDatabase::Get();
		}

		//FLoadObjectsFromAssetRegistryHelper::Load<UUserDefinedEnum>(UserDefinedEnumerators);

		FRigVMEditorAssetInterfacePtr InitBlueprint = InitBlueprints[0];

		// Update the blueprint if required
		ERigVMAssetStatus OldStatus = InitBlueprint->GetAssetStatus();
		//EnsureBlueprintIsUpToDate(InitBlueprint); // TODO sara-s
		UPackage* BpPackage = InitBlueprint->GetObject()->GetOutermost();
		bBlueprintModifiedOnOpen = (InitBlueprint->GetAssetStatus() != OldStatus) && !BpPackage->HasAnyPackageFlags(PKG_NewlyCreated);

		// Flag the blueprint as having been opened
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InitBlueprint->GetObject()))
		{
			Blueprint->bIsNewlyCreated = false;
		}

		// When the blueprint that we are observing changes, it will notify this wrapper widget.
		InitBlueprint->OnChanged().AddSP(this, &FRigVMNewEditor::OnBlueprintChanged); // TODO sara-s
		// InitBlueprint->OnCompiled().AddSP(this, &FRigVMNewEditor::OnBlueprintCompiled); // TODO sara-s
		InitBlueprint->OnSetObjectBeingDebugged().AddSP(this, &FRigVMNewEditor::HandleSetObjectBeingDebugged);

		InitBlueprint->GetVMCompileSettings().ASTSettings.ReportDelegate.BindSPLambda(this,
			[InitBlueprint](EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
			{ InitBlueprint->HandleReportFromCompiler(InSeverity, InSubject, InMessage); });

		InitBlueprint->OnExternalVariablesChanged().AddSPLambda(this, [this](const TArray<FRigVMExternalVariable>& Variables)
			{
				if (GraphExplorerWidget.IsValid())
				{
					GraphExplorerWidget->Refresh();
				}
			});

		InitBlueprint->OnPreVariablesChanged().AddSP(this, &FRigVMNewEditor::HandlePreVariablesChanged);
		InitBlueprint->OnPostVariablesChanged().AddSP(this, &FRigVMNewEditor::HandlePostVariablesChanged);
	}

	bWasOpenedInDefaultsMode = bShouldOpenInDefaultsMode;

	CreateDefaultTabContents(InitBlueprints);

	// FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddSP(this, &FRigVMNewEditor::OnPreObjectPropertyChanged); // TODO sara-s
	// FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FRigVMNewEditor::OnPostObjectPropertyChanged); // TODO sara-s

	// FKismetEditorUtilities::OnBlueprintUnloaded.AddSP(this, &FRigVMNewEditor::OnBlueprintUnloaded); // TODO sara-s


	UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
	if (Editor != nullptr)
	{
		Editor->RegisterForUndo(this);
	}
}

void FRigVMNewEditor::OnBlueprintChanged(UObject* InBlueprint)
{
	if (InBlueprint)
	{
		// Notify that the blueprint has been changed (update Content browser, etc)
		InBlueprint->PostEditChange();

		// Call PostEditChange() on any Actors that are based on this Blueprint
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InBlueprint))
		{
			FBlueprintEditorUtils::PostEditChangeBlueprintActors(Blueprint);
		}
		else
		{
			checkf(false, TEXT("RigVM asset is not a blueprint"));
		}

		// Refresh the graphs
		//TODO sara-s
		ERefreshRigVMEditorReason::Type Reason = ERefreshRigVMEditorReason::UnknownReason;// = bIsJustBeingCompiled ? ERefreshBlueprintEditorReason::BlueprintCompiled : ERefreshBlueprintEditorReason::UnknownReason;
		ForceEditorRefresh(Reason);

		// In case objects were deleted, which should close the tab
		if (GetCurrentMode() == FRigVMNewEditorApplicationModes::StandardRigVMEditorMode())
		{
			SaveEditedObjectState();
		}
	}
}

void FRigVMNewEditor::SaveEditedObjectState()
{
	if (!IsEditingSingleBlueprint())
	{
		return;
	}

	// Save graph tabs in the order they were originally spawned (a stable proxy for the
	// user-visible tab strip in typical sessions) rather than via DocumentManager->SaveAllState().
	// SaveAllState saves background tabs first and foreground tabs last, which actively scrambles
	// any restore order. Spawn order is imperfect when the user has reordered tabs since spawn
	// but is strictly better than the foreground/background split.
	TArray<FEditedDocumentInfo>& Docs = GetRigVMAssetInterface()->GetLastEditedDocuments();

	// Snapshot the prior list so we can restore it if we fail to capture anything new - e.g.
	// the editor is tearing down without graph tabs, or FindAllTabsForFactory returns empty
	// because of a partially-torn-down state. We'd rather keep the old state than wipe it
	// to empty and lose the user's persisted tab set.
	TArray<FEditedDocumentInfo> PriorDocs = Docs;
	Docs.Empty();

	TSharedPtr<FDocumentTabFactory> GraphFactory = GraphEditorTabFactoryPtr.Pin();
	if (GraphFactory.IsValid())
	{
		TArray<TSharedPtr<SDockTab>> GraphTabs;
		DocumentManager->FindAllTabsForFactory(GraphEditorTabFactoryPtr, GraphTabs);
		for (const TSharedPtr<SDockTab>& Tab : GraphTabs)
		{
			if (!Tab.IsValid())
			{
				continue;
			}
			TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(Tab->GetContent());
			if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
			{
				TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(Graph);
				GraphFactory->SaveState(Tab, Payload);
			}
		}
	}

	if (Docs.IsEmpty())
	{
		Docs = MoveTemp(PriorDocs);
	}
}

void FRigVMNewEditor::SetCurrentMode(FName NewMode)
{
	// Clear the selection state when the mode changes.
	SetUISelectionState(NAME_None);

	// TODO sara-s: Replicate what blueprint editor is doing
	//OnModeSetData.Broadcast( NewMode );
	
	FWorkflowCentricApplication::SetCurrentMode(NewMode);
}

void FRigVMNewEditor::PostLayoutBlueprintEditorInitialization()
{
	// TODO sara-s: Replicate what blueprint editor is doing
	if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
	{
		// Refresh the graphs
		ForceEditorRefresh();
		
		// EnsureBlueprintIsUpToDate may have updated the blueprint so show notifications to user.
		if (bBlueprintModifiedOnOpen)
		{
			bBlueprintModifiedOnOpen = false;

			if (FocusedGraphEdPtr.IsValid())
			{
				FNotificationInfo Info( NSLOCTEXT("RigVM", "Blueprint Modified", "Blueprint requires updating. Please resave.") );
				Info.Image = FAppStyle::GetBrush(TEXT("Icons.Info"));
				Info.bFireAndForget = true;
				Info.bUseSuccessFailIcons = false;
				Info.ExpireDuration = 5.0f;

				FocusedGraphEdPtr.Pin()->AddNotification(Info, true);
			}

			// Fire log message
			FString BlueprintName;
			Blueprint->GetObject()->GetName(BlueprintName);

			FFormatNamedArguments Args;
			Args.Add( TEXT("BlueprintName"), FText::FromString( BlueprintName ) );
			LogSimpleMessage( FText::Format( LOCTEXT("Blueprint Modified Long", "Blueprint \"{BlueprintName}\" was updated to fix issues detected on load. Please resave."), Args ) );
		}

		// Determine if the current "mode" supports invoking the Compiler Results tab.
		const bool bCanInvokeCompilerResultsTab = TabManager->HasTabSpawner(FRigVMCompilerResultsTabSummoner::TabID());

		// If we have a warning/error, open output log if the current mode allows us to invoke it.
		if (bCanInvokeCompilerResultsTab)
		{
			TabManager->TryInvokeTab(FRigVMCompilerResultsTabSummoner::TabID());
		}
		else
		{
			// Toolkit modes that don't include this tab may have been incorrectly saved with layout information for restoring it
			// as an "unrecognized" tab, due to having previously invoked it above without checking to see if the layout can open
			// it first. To correct this, we check if the tab was restored from a saved layout here, and close it if not supported.
			TSharedPtr<SDockTab> TabPtr = TabManager->FindExistingLiveTab(FRigVMCompilerResultsTabSummoner::TabID());
			if (TabPtr.IsValid() && !bCanInvokeCompilerResultsTab)
			{
				TabPtr->RequestCloseTab();
			}
		}
	}
}

TSharedPtr<SGraphEditor> FRigVMNewEditor::OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus)
{
	if (!IsValid(Graph))
	{
		return TSharedPtr<SGraphEditor>();
	}

	// First, switch back to standard mode
	SetCurrentMode(FRigVMNewEditorApplicationModes::StandardRigVMEditorMode());

	// This will either reuse an existing tab or spawn a new one
	TSharedPtr<SDockTab> TabWithGraph = OpenDocument(Graph, FDocumentTracker::OpenNewDocument);
	if (TabWithGraph.IsValid())
	{

		// We know that the contents of the opened tabs will be a graph editor.
		TSharedRef<SGraphEditor> NewGraphEditor = StaticCastSharedRef<SGraphEditor>(TabWithGraph->GetContent());

		// Handover the keyboard focus to the new graph editor widget.
		if (bSetFocus)
		{
			NewGraphEditor->CaptureKeyboard();
		}

		return NewGraphEditor;
	}
	else
	{
		return TSharedPtr<SGraphEditor>();
	}
}

bool FRigVMNewEditor::FindOpenTabsContainingDocument(const UObject* DocumentID, TArray<TSharedPtr<SDockTab>>& Results)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	return false;
}

TSharedPtr<SDockTab> FRigVMNewEditor::OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	return DocumentManager->OpenDocument(Payload, Cause);
}

void FRigVMNewEditor::CloseDocumentTab(const UObject* DocumentID)
{
	TSharedRef<FTabPayload_UObject> Payload = FTabPayload_UObject::Make(DocumentID);
	DocumentManager->CloseTab(Payload);
}

TSharedPtr<FApplicationMode> FRigVMNewEditor::CreateEditorMode()
{
	return MakeShareable(new FRigVMNewEditorMode(SharedThis(this)));
}

const FName FRigVMNewEditor::GetEditorAppName() const
{
	static const FLazyName AppName(TEXT("RigVMNewEditorApp"));
	return AppName;
}

FText FRigVMNewEditor::GetToolkitName() const
{
	// Get the EditorAsset that's being edited
	if (const UObject* EditingObject = GetEditingObject())
	{
		// Cast to your EditorAsset type
		if (const URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(EditingObject))
		{
			// Get the RuntimeAsset
			if (URigVMRuntimeAsset* RuntimeAsset = EditorAsset->GetRuntimeAsset())
			{
				// Return the RuntimeAsset's name for the tab label
				return FText::FromString(RuntimeAsset->GetName());
			}
		}
	}

	// Fallback to default behavior
	return FAssetEditorToolkit::GetToolkitName();
}

FText FRigVMNewEditor::GetToolkitToolTipText() const
{
	const UObject* EditingObject = GetEditingObject();

	if (EditingObject)
	{
		if (const URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(EditingObject))
		{
			if (URigVMRuntimeAsset* RuntimeAsset = EditorAsset->GetRuntimeAsset())
			{
				// Return the RuntimeAsset's tooltip
				return FAssetEditorToolkit::GetToolTipTextForObject(RuntimeAsset);
			}
		}
	}

	// Fallback to default behavior
	return FAssetEditorToolkit::GetToolTipTextForObject(EditingObject);
}

void FRigVMNewEditor::OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	// Update the graph editor that is currently focused
	FocusedGraphEdPtr = InGraphEditor;

	// TODO sara-s
	//InGraphEditor->SetPinVisibility(PinVisibility);

	// Update the inspector as well, to show selection from the focused graph editor
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	FocusInspectorOnGraphSelection(SelectedNodes, /*bForceRefresh=*/ true);

	// During undo, garbage graphs can be temporarily brought into focus, ensure that before a refresh of the MyBlueprint window that the graph is owned by a Blueprint
	if ( FocusedGraphEdPtr.IsValid() && GraphExplorerWidget.IsValid() )
	{
		// The focused graph can be garbage as well
		TWeakObjectPtr< UEdGraph > FocusedGraphPtr = FocusedGraphEdPtr.Pin()->GetCurrentGraph();
		UEdGraph* FocusedGraph = FocusedGraphPtr.Get();

		if ( FocusedGraph != nullptr )
		{
			if ( FRigVMBlueprintUtils::FindAssetForGraph(FocusedGraph) )
			{
				GraphExplorerWidget->Refresh();
			}
		}
	}

	// TODO sara-s
	// if (bHideUnrelatedNodes && SelectedNodes.Num() <= 0)
	// {
	// 	ResetAllNodesUnrelatedStates();
	// }

	// TODO sara-sc
	// If the bookmarks view is active, check whether or not we're restricting the view to the current graph. If we are, update the tree to reflect the focused graph context.
	// if (BookmarksWidget.IsValid()
	// 	&& GetDefault<UBlueprintEditorSettings>()->bShowBookmarksForCurrentDocumentOnlyInTab)
	// {
	// 	BookmarksWidget->RefreshBookmarksTree();
	// }

	FRigVMEditorBase::OnGraphEditorFocused(InGraphEditor);
}

void FRigVMNewEditor::OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	// If the newly active document tab isn't a graph we want to make sure we clear the focused graph pointer.
	// Several other UI reads that, like the MyBlueprints view uses it to determine if it should show the "Local Variable" section.
	FocusedGraphEdPtr = nullptr;

	if ( GraphExplorerWidget.IsValid() == true )
	{
		GraphExplorerWidget->Refresh();
	}
}

FText FRigVMNewEditor::GetCompileStatusTooltip() const
{
	// Copied from FBlueprintEditorToolbar::GetStatusTooltip

	FRigVMEditorAssetInterfacePtr BlueprintObj = GetRigVMAssetInterface();
	ERigVMAssetStatus Status = BlueprintObj->GetAssetStatus();

	switch (Status)
	{
	case RVMA_Dirty:
		return LOCTEXT("Dirty_Status", "Dirty; needs to be recompiled");
	case RVMA_Error:
		return LOCTEXT("CompileError_Status", "There was an error during compilation, see the log for details");
	case RVMA_UpToDate:
		return LOCTEXT("GoodToGo_Status", "Good to go");
	case RVMA_UpToDateWithWarnings:
		return LOCTEXT("GoodToGoWarning_Status", "There was a warning during compilation, see the log for details");
	default:
		return LOCTEXT("Recompile_Status", "Unknown status; should recompile");
	}
}

FSlateIcon FRigVMNewEditor::GetCompileStatusImage() const
{
	// Copied from FBlueprintEditorToolbar::GetStatusImage

	FRigVMEditorAssetInterfacePtr BlueprintObj = GetRigVMAssetInterface();
	ERigVMAssetStatus Status = BlueprintObj->GetAssetStatus();

	
	static const FName CompileStatusBackground("Blueprint.CompileStatus.Background");
	static const FName CompileStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("Blueprint.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("Blueprint.CompileStatus.Overlay.Good");
	static const FName CompileStatusWarning("Blueprint.CompileStatus.Overlay.Warning");

	switch (Status)
	{
	case RVMA_Error:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
	case RVMA_UpToDate:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
	case RVMA_UpToDateWithWarnings:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
	default:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	}
}

void FRigVMNewEditor::MakeSaveOnCompileSubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection(TEXT("Section"));
	const FRigVMEditorCommands& Commands = FRigVMEditorCommands::Get();
	InMenuBuilder.AddMenuEntry(Commands.SaveOnCompile_Never);
	InMenuBuilder.AddMenuEntry(Commands.SaveOnCompile_SuccessOnly);
	InMenuBuilder.AddMenuEntry(Commands.SaveOnCompile_Always);
}

TSharedRef<SWidget> FRigVMNewEditor::GenerateCompileOptionsMenu()
{
	FMenuBuilder MenuBuilder(true, GetToolkitCommands());
	MenuBuilder.BeginSection(TEXT("Section"));
	const FRigVMEditorCommands& Commands = FRigVMEditorCommands::Get();

	// @TODO: disable the menu and change up the tooltip when all sub items are disabled
	MenuBuilder.AddSubMenu(
		LOCTEXT("SaveOnCompileSubMenu", "Save on Compile"),
		LOCTEXT("SaveOnCompileSubMenu_ToolTip", "Determines if the asset is saved whenever you compile it."),
		FNewMenuDelegate::CreateSP(this, &FRigVMNewEditor::MakeSaveOnCompileSubMenu)
	);

	MenuBuilder.AddMenuEntry(Commands.JumpToErrorNode);
	MenuBuilder.AddMenuEntry(Commands.AutoCompileGraph);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FRigVMNewEditor::AddCompileWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddToolBarButton(FRigVMEditorCommands::Get().Compile,
		NAME_None, TAttribute<FText>(),
		TAttribute<FText>::CreateLambda([this]()
		{
			return GetCompileStatusTooltip();
		}),
		TAttribute<FSlateIcon>::CreateLambda([this]()
		{
			return GetCompileStatusImage();
		}),
		"CompileBlueprint");

	ToolbarBuilder.AddComboButton(
			FUIAction(),
				FOnGetContent::CreateSP(this, &FRigVMNewEditor::GenerateCompileOptionsMenu),
				TAttribute<FText>(),
		LOCTEXT("BlueprintCompileOptions_ToolbarTooltip", "Options to customize how Blueprints compile"),
		TAttribute<FSlateIcon>(), true);
}

void FRigVMNewEditor::AddSettingsAndDefaultWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			ToggleFadeOutUnrelateNodes();
		}),
		FCanExecuteAction(),
		FGetActionCheckState::CreateLambda([this]() -> ECheckBoxState
		{
			return IsToggleFadeOutUnrelatedNodesChecked() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})),
		NAME_None,
		LOCTEXT("HideUnrelated", "Hide Unrelated"),
		LOCTEXT("HideUnrelatedTooltip", "Fades out unrelated nodes"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.ToggleHideUnrelatedNodes")
	);

	ToolbarBuilder.AddToolBarButton(FRigVMEditorCommands::Get().EditGlobalOptions, 
		NAME_None, 
		TAttribute<FText>(), 
		TAttribute<FText>(), 
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FullBlueprintEditor.EditGlobalOptions"));
	ToolbarBuilder.AddToolBarButton(FRigVMEditorCommands::Get().EditClassDefaults, 
		NAME_None, 
		TAttribute<FText>(), 
		TAttribute<FText>(), 
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "FullBlueprintEditor.EditClassDefaults"));
}

void FRigVMNewEditor::AddFindWidget(FToolBarBuilder& ToolbarBuilder)
{
	
	ToolbarBuilder.AddToolBarButton(
		FRigVMEditorCommands::Get().FindReferences,
		NAME_None,
		LOCTEXT("FindReferences_Label", "Find"),
			LOCTEXT("FindReferences_ToolTip", "Find all references to the selected variable"),
		TAttribute<FSlateIcon>::CreateLambda(
			[]() -> FSlateIcon
			{
				const FSlateIcon GlobalFindResultsIcon(FAppStyle::GetAppStyleSetName(), "BlueprintEditor.RigVMFindInBlueprints.MenuIcon");
				return GlobalFindResultsIcon;
			}
		),
		NAME_None,
		FNewMenuDelegate(),
		TAttribute<EVisibility>(),
		LOCTEXT("FindReferences_Label", "Find")
	);
}

void FRigVMNewEditor::AddSelectedDebugObjectWidget(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddWidget(SNew(SRigVMEditorSelectedDebugObjectWidget, SharedThis(this)));
}

void FRigVMNewEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	//@TODO: Can't we do this sooner?
	DocumentManager->SetTabManager(InTabManager);

	FWorkflowCentricApplication::RegisterTabSpawners(InTabManager);
}

void FRigVMNewEditor::Tick(float DeltaTime)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	FRigVMEditorBase::Tick(DeltaTime);
}

void FRigVMNewEditor::GetPinTypeSelectorFilters(TArray<TSharedPtr<IPinTypeSelectorFilter>>& OutFilters) const
{
	// TODO sara-s
	// OutFilters.Add(ImportedPinTypeSelectorFilter);
	// OutFilters.Add(PermissionsPinTypeSelectorFilter);
}

void FRigVMNewEditor::OnAddNewVariable()
{
	const FScopedTransaction Transaction( LOCTEXT("AddVariable", "Add Variable") );

	FName VarName = FRigVMBlueprintUtils::FindUniqueVariableName(GetRigVMAssetInterface().GetInterface(), TEXT("NewVar"));
	
	bool bSuccess = GraphExplorerWidget.IsValid() && GetRigVMAssetInterface()->AddAssetVariableFromPinType(VarName, GraphExplorerWidget->GetLastPinTypeUsed()) != NAME_None;

	if(!bSuccess)
	{
		LogSimpleMessage( LOCTEXT("AddVariable_Error", "Adding new variable failed.") );
	}
	else
	{
		RenameNewlyAddedAction(VarName);
		GraphExplorerWidget->Refresh();
	}
}

void FRigVMNewEditor::SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult)
{
	TSharedPtr<SRigVMFindReferences> FindResultsToUse = nullptr;

	FindResultsToUse = FindResults;
	TabManager->TryInvokeTab(FRigVMFindReferencesTabSummoner::TabID());
	
	if (FindResultsToUse.IsValid())
	{
		FindResultsToUse->FocusForUse(bSetFindWithinBlueprint, NewSearchTerms, bSelectFirstResult);
	}
}

void FRigVMNewEditor::ZoomToSelection_Clicked()
{
	if (SGraphEditor* GraphEd = FocusedGraphEdPtr.Pin().Get())
	{
		GraphEd->ZoomToFit(/*bOnlySelection=*/ true);
	}
}

void FRigVMNewEditor::FindInBlueprint_Clicked()
{
	SummonSearchUI(true, FString(), true);
}

void FRigVMNewEditor::RestoreEditedObjectState()
{
	check(IsEditingSingleBlueprint());

	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	
	TSet<FSoftObjectPath> PathsToRemove;
	for (int32 i = 0; i < Blueprint->GetLastEditedDocuments().Num(); i++)
	{
		if (UObject* Obj = Blueprint->GetLastEditedDocuments()[i].EditedObjectPath.ResolveObject())
		{
			if(UEdGraph* Graph = Cast<UEdGraph>(Obj))
			{
				struct LocalStruct
				{
					static TSharedPtr<SDockTab> OpenGraphTree(FRigVMNewEditor* InBlueprintEditor, UEdGraph* InGraph)
					{
						FDocumentTracker::EOpenDocumentCause OpenCause = FDocumentTracker::QuickNavigateCurrentDocument;

						for (UObject* OuterObject = InGraph->GetOuter(); OuterObject; OuterObject = OuterObject->GetOuter())
						{
							// A UBlueprint terminates the walk for BP-backed assets; for non-BP runtime
							// assets the graphs live under a URigVMEditorAsset (implements
							// IRigVMEditorAssetInterface), which serves the same role - treat it the same
							// way so we use RestorePreviousDocument (which spawns a fresh tab per graph)
							// instead of QuickNavigateCurrentDocument (which reuses the current tab and
							// collapses all restored tabs into one).
							if (OuterObject->IsA<UBlueprint>() || OuterObject->Implements<URigVMEditorAssetInterface>())
							{
								OpenCause = FDocumentTracker::RestorePreviousDocument;
								break;
							}
							else if(UEdGraph* OuterGraph = Cast<UEdGraph>(OuterObject))
							{
								// Found another graph, open it up
								OpenGraphTree(InBlueprintEditor, OuterGraph);
								break;
							}
						}

						return InBlueprintEditor->OpenDocument(InGraph, OpenCause);
					}
				};
				TSharedPtr<SDockTab> TabWithGraph = LocalStruct::OpenGraphTree(this, Graph);
				if (TabWithGraph.IsValid())
				{
					TSharedRef<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(TabWithGraph->GetContent());
					GraphEditor->SetViewLocation(Blueprint->GetLastEditedDocuments()[i].SavedViewOffset, Blueprint->GetLastEditedDocuments()[i].SavedZoomAmount);
				}
			}
			else
			{
				TSharedPtr<SDockTab> TabWithGraph = OpenDocument(Obj, FDocumentTracker::RestorePreviousDocument);
			}
		}
		else
		{
			PathsToRemove.Add(Blueprint->GetLastEditedDocuments()[i].EditedObjectPath);
		}
	}

	// Older assets may have neglected to clean up this array when referenced objects were deleted, so
	// we'll check for that now. This is done to ensure we don't store invalid object paths indefinitely.
	if (PathsToRemove.Num() > 0)
	{
		Blueprint->GetLastEditedDocuments().RemoveAll([&PathsToRemove](const FEditedDocumentInfo& Entry)
		{
			return PathsToRemove.Contains(Entry.EditedObjectPath);
		});
	}
}

void FRigVMNewEditor::SetupViewForBlueprintEditingMode()
{
	// Make sure the defaults tab is pointing to the defaults
	StartEditingDefaults(/*bAutoFocus=*/ true);

	// Make sure the inspector is always on top
	//@TODO: This is necessary right now because of a bug in restoring layouts not remembering which tab is on top (to get it right initially), but do we want this behavior always?
	TryInvokingDetailsTab();
}

void FRigVMNewEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	FWorkflowCentricApplication::InitToolMenuContext(MenuContext);
	FRigVMEditorBase::InitToolMenuContextImpl(MenuContext);
}

bool FRigVMNewEditor::TransactionObjectAffectsBlueprint(UObject* InTransactedObject)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	return FRigVMEditorBase::TransactionObjectAffectsBlueprintImpl(InTransactedObject);
}

FEdGraphPinType FRigVMNewEditor::GetLastPinTypeUsed()
{
	if (GraphExplorerWidget.IsValid())
	{
		return GraphExplorerWidget->GetLastPinTypeUsed();
	}
	return FEdGraphPinType();
}

void FRigVMNewEditor::LogSimpleMessage(const FText& MessageText)
{
	FNotificationInfo Info( MessageText );
	Info.ExpireDuration = 3.0f;
	Info.bUseLargeFont = false;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_Fail );
	}
}

void FRigVMNewEditor::RenameNewlyAddedAction(FName InActionName)
{
	// TODO sara-s: Replicate what blueprint editor is doing
}

FGraphPanelSelectionSet FRigVMNewEditor::GetSelectedNodes() const
{
	FGraphPanelSelectionSet CurrentSelection;
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		CurrentSelection = FocusedGraphEd->GetSelectedNodes();
	}
	return CurrentSelection;
}

void FRigVMNewEditor::SetUISelectionState(FName SelectionOwner)
{
	if ( SelectionOwner != CurrentUISelection )
	{
		ClearSelectionStateFor(CurrentUISelection);
	
		CurrentUISelection = SelectionOwner;
	}
}

void FRigVMNewEditor::AnalyticsTrackNodeEvent(IRigVMEditorAssetInterface* Blueprint, UEdGraphNode* GraphNode, bool bNodeDelete) const
{
	// TODO sara-s
	// if(Blueprint && GraphNode && FEngineAnalytics::IsAvailable())
	// {
	// 	// we'd like to see if this was happening in normal blueprint editor or persona 
	// 	//const FString EditorName = Cast<UAnimBlueprint>(Blueprint) != nullptr ? TEXT("Persona") : TEXT("BlueprintEditor");
	// 	const FString EditorName = TEXT("RigVMNewEditor");
	//
	// 	// Build Node Details
	// 	const UGeneralProjectSettings& ProjectSettings = *GetDefault<UGeneralProjectSettings>();
	// 	FString ProjectID = ProjectSettings.ProjectID.ToString();
	// 	TArray<FAnalyticsEventAttribute> NodeAttributes;
	// 	NodeAttributes.Add(FAnalyticsEventAttribute(TEXT("ProjectId"), ProjectID));
	// 	NodeAttributes.Add(FAnalyticsEventAttribute(TEXT("BlueprintId"), Blueprint->GetBlueprintGuid().ToString()));
	// 	TArray<TKeyValuePair<FString, FString>> Attributes;
	//
	// 	if (UK2Node* K2Node = Cast<UK2Node>(GraphNode))
	// 	{
	// 		K2Node->GetNodeAttributes(Attributes);
	// 	}
	// 	else if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(GraphNode))
	// 	{
	// 		Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Type"), TEXT("Comment")));
	// 		Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Class"), CommentNode->GetClass()->GetName()));
	// 		Attributes.Add(TKeyValuePair<FString, FString>(TEXT("Name"), CommentNode->GetName()));
	// 	}
	// 	if (Attributes.Num() > 0)
	// 	{
	// 		// Build Node Attributes
	// 		for (const TKeyValuePair<FString, FString>& Attribute : Attributes)
	// 		{
	// 			NodeAttributes.Add(FAnalyticsEventAttribute(Attribute.Key, Attribute.Value));
	// 		}
	// 		// Send Analytics event 
	// 		FString EventType = bNodeDelete ?	FString::Printf(TEXT("Editor.Usage.%s.NodeDeleted"), *EditorName) :
	// 											FString::Printf(TEXT("Editor.Usage.%s.NodeCreated"), *EditorName);
	// 		FEngineAnalytics::GetProvider().RecordEvent(EventType, NodeAttributes);
	// 	}
	// }
}

TSharedRef<SWidget> FRigVMNewEditor::GetCompilerResults() const
{
	if (CompilerResults.IsValid())
	{
		return CompilerResults.ToSharedRef();
	}
	return SNullWidget::NullWidget;
}

void FRigVMNewEditor::JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename)
{
	if (FRigVMEditorBase::JumpToHyperlinkImpl(ObjectReference, bRequestRename))
	{
		return;
	}
	
	SetCurrentMode(FRigVMNewEditorApplicationModes::StandardRigVMEditorMode());

	if (const UEdGraphNode* Node = Cast<const UEdGraphNode>(ObjectReference))
	{
		if (bRequestRename)
		{
			IsNodeTitleVisible(Node, bRequestRename);
		}
		else
		{
			JumpToNode(Node, false);
		}
	}
	else if (const UEdGraph* Graph = Cast<const UEdGraph>(ObjectReference))
	{
		// Navigating into things should re-use the current tab when it makes sense
		FDocumentTracker::EOpenDocumentCause OpenMode = FDocumentTracker::OpenNewDocument;
		if ((Graph->GetSchema()->GetGraphType(Graph) == GT_Ubergraph) || Cast<UK2Node>(Graph->GetOuter()) || Cast<UEdGraph>(Graph->GetOuter()))
		{
			// Ubergraphs directly reuse the current graph
			OpenMode = FDocumentTracker::NavigatingCurrentDocument;
		}
		else
		{
			// Walk up the outer chain to see if any tabs have a parent of this document open for edit, and if so
			// we should reuse that one and drill in deeper instead
			for (UObject* WalkPtr = const_cast<UEdGraph*>(Graph); WalkPtr != nullptr; WalkPtr = WalkPtr->GetOuter())
			{
				TArray< TSharedPtr<SDockTab> > TabResults;
				if (FindOpenTabsContainingDocument(WalkPtr, /*out*/ TabResults))
				{
					// See if the parent was active
					bool bIsActive = false;
					for (TSharedPtr<SDockTab> Tab : TabResults)
					{
						if (Tab->IsActive())
						{
							bIsActive = true;
							break;
						}
					}

					if (bIsActive)
					{
						OpenMode = FDocumentTracker::NavigatingCurrentDocument;
						break;
					}
				}
			}
		}

		// Force it to open in a new document if shift is pressed
		const bool bIsShiftPressed = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		if (bIsShiftPressed)
		{
			OpenMode = FDocumentTracker::ForceOpenNewDocument;
		}

		// Open the document
		OpenDocument(Graph, OpenMode);
	}
	else if ((ObjectReference != nullptr) && ObjectReference->IsAsset())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(const_cast<UObject*>(ObjectReference));
	}
	else
	{
		UE_LOGF(LogBlueprint, Warning, "Unknown type of hyperlinked object (%ls), cannot focus it", *GetNameSafe(ObjectReference));
	}

	//@TODO: Hacky way to ensure a message is seen when hitting an exception and doing intraframe debugging
	const FText ExceptionMessage = FKismetDebugUtilities::GetAndClearLastExceptionMessage();
	if (!ExceptionMessage.IsEmpty())
	{
		LogSimpleMessage( ExceptionMessage );
	}
}

void FRigVMNewEditor::PostUndo(bool bSuccess)
{
	FEditorUndoClient::PostUndo(bSuccess);
	FRigVMEditorBase::PostUndoImpl(bSuccess);
}

void FRigVMNewEditor::PostRedo(bool bSuccess)
{
	FEditorUndoClient::PostRedo(bSuccess);
	FRigVMEditorBase::PostRedoImpl(bSuccess);
}

UEdGraphPin* FRigVMNewEditor::GetCurrentlySelectedPin() const
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		return FocusedGraphEd->GetGraphPinForMenu();
	}

	return nullptr;
}

void FRigVMNewEditor::SetSaveOnCompileSetting(ERigVMSaveOnCompile NewSetting)
{
	URigVMEditorSettings* Settings = GetMutableDefault<URigVMEditorSettings>();
	Settings->SaveOnCompile = NewSetting;
	Settings->SaveConfig();
}

bool FRigVMNewEditor::IsSaveOnCompileOptionSet(ERigVMSaveOnCompile Option)
{
	const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>();
	return (Settings->SaveOnCompile == Option);
}

void FRigVMNewEditor::ToggleJumpToErrorNodeSetting()
{
	// Copied from BlueprintEditorImpl::ToggleJumpToErrorNodeSetting
	
	UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
	Settings->bJumpToNodeErrors = !Settings->bJumpToNodeErrors;
	Settings->SaveConfig();
}

bool FRigVMNewEditor::IsJumpToErrorNodeOptionSet()
{
	// Copied from BlueprintEditorImpl::IsJumpToErrorNodeOptionSet
	
	const UBlueprintEditorSettings* Settings = GetDefault<UBlueprintEditorSettings>();
	return Settings->bJumpToNodeErrors;
}

UEdGraphNode* FRigVMNewEditor::FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	// Copied from BlueprintEditorImpl::FindNodeWithError
	
	TArray<UEdGraph*> Graphs;
	Blueprint->GetAllGraphs(Graphs);

	UEdGraphNode* ChoiceNode = nullptr;
	for (UEdGraph* Graph : Graphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (Node && Node->bHasCompilerMessage && !Node->ErrorMsg.IsEmpty() && (Node->ErrorType <= Severity))
			{
				if ((ChoiceNode == nullptr) || (ChoiceNode->ErrorType > Node->ErrorType))
				{
					ChoiceNode = Node;
					if (ChoiceNode->ErrorType == 0)
					{
						break;
					}
				}
			}
		}
	}
	return ChoiceNode;
}

UEdGraphNode* FRigVMNewEditor::FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity/* = EMessageSeverity::Error*/)
{
	// Copied from BlueprintEditorImpl::FindNodeWithError
	
	UEdGraphNode* ChoiceNode = nullptr;
	for (TWeakObjectPtr<UEdGraphNode> NodePtr : ErrorLog.AnnotatedNodes)
	{
		UEdGraphNode* Node = NodePtr.Get();
		if ((Node != nullptr) && (Node->ErrorType <= Severity))
		{
			if ((ChoiceNode == nullptr) || (Node->ErrorType < ChoiceNode->ErrorType))
			{
				ChoiceNode = Node;
				if (ChoiceNode->ErrorType == 0)
				{
					break;
				}
			}
		}
	}

	return ChoiceNode;
}

void FRigVMNewEditor::CreateDefaultCommands()
{
	if (GetRigVMAssetInterface())
	{
		// TODO sara-s: Replicate what blueprint editor is doing
		//FBlueprintEditor::CreateDefaultCommands();

		GetToolkitCommands()->MapAction(
			FRigVMEditorCommands::Get().Compile,
			FExecuteAction::CreateSP(this, &FRigVMEditorBase::Compile),
			FCanExecuteAction::CreateSP(this, &FRigVMEditorBase::IsCompilingEnabled));

		// Save-on-compile commands - always available for runtime assets (no can-execute guard needed)
		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().SaveOnCompile_Never,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::SetSaveOnCompileSetting, ERigVMSaveOnCompile::Never),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsSaveOnCompileOptionSet, ERigVMSaveOnCompile::Never)
		);
		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().SaveOnCompile_SuccessOnly,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::SetSaveOnCompileSetting, ERigVMSaveOnCompile::SuccessOnly),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsSaveOnCompileOptionSet, ERigVMSaveOnCompile::SuccessOnly)
		);
		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().SaveOnCompile_Always,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::SetSaveOnCompileSetting, ERigVMSaveOnCompile::Always),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsSaveOnCompileOptionSet, ERigVMSaveOnCompile::Always)
		);

		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().JumpToErrorNode,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::ToggleJumpToErrorNodeSetting),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsJumpToErrorNodeOptionSet)
		);

		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().EditGlobalOptions,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::EditGlobalOptions_Clicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsDetailsPanelEditingGlobalOptions));

		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().EditClassDefaults,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::EditClassDefaults_Clicked),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FRigVMNewEditor::IsDetailsPanelEditingClassDefaults));
		
		ToolkitCommands->MapAction(
			FRigVMEditorCommands::Get().FindReferences,
			FExecuteAction::CreateSP(this, &FRigVMNewEditor::FindInBlueprint_Clicked),
			FCanExecuteAction()
		);
	}

	FRigVMEditorBase::CreateDefaultCommandsImpl();
}

TSharedRef<SGraphEditor> FRigVMNewEditor::CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph)
{
	check((InGraph != nullptr) && IsEditingSingleBlueprint());

	// No need to regenerate the commands.
	if(!GraphEditorCommands.IsValid())
	{
		GraphEditorCommands = MakeShareable( new FUICommandList );
		{
			
			// Alignment Commands
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesTop,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignTop )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesMiddle,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignMiddle )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesBottom,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignBottom )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesLeft,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignLeft )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesCenter,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignCenter )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().AlignNodesRight,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnAlignRight )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StraightenConnections,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnStraightenConnections )
				);
			
			// Distribution Commands
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DistributeNodesHorizontally,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnDistributeNodesH )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().DistributeNodesVertically,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnDistributeNodesV )
				);

			// Editing commands
			GraphEditorCommands->MapAction( FGenericCommands::Get().SelectAll,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::SelectAllNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMNewEditor::CanSelectAllNodes )
				);
			
			GraphEditorCommands->MapAction( FGenericCommands::Get().Delete,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::DeleteSelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanDeleteNodes )
				);
			
			GraphEditorCommands->MapAction( FGenericCommands::Get().Copy,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::CopySelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanCopyNodes )
				);
			
			GraphEditorCommands->MapAction( FGenericCommands::Get().Cut,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::CutSelectedNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanCutNodes )
				);
			
			GraphEditorCommands->MapAction( FGenericCommands::Get().Paste,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::PasteNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanPasteNodes )
				);
			
			GraphEditorCommands->MapAction( FGenericCommands::Get().Duplicate,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::DuplicateNodes ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanDuplicateNodes )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StartWatchingPin,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::OnStartWatchingPin ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanStartWatchingPin )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().StopWatchingPin,
				FExecuteAction::CreateSP( this, &FRigVMEditorBase::OnStopWatchingPin ),
				FCanExecuteAction::CreateSP( this, &FRigVMEditorBase::CanStopWatchingPin )
				);
			
			GraphEditorCommands->MapAction( FGraphEditorCommands::Get().CreateComment,
				FExecuteAction::CreateSP( this, &FRigVMNewEditor::OnCreateComment )
				);

			OnCreateGraphEditorCommands(GraphEditorCommands);
		}
	}

	// Create the title bar widget
	TSharedPtr<SWidget> TitleBarWidget = SNew(SRigVMGraphTitleBar)
		.EdGraphObj(InGraph)
		.Editor(SharedThis(this))
		.HistoryNavigationWidget(InTabInfo->CreateHistoryNavigationWidget());

	SGraphEditor::FGraphEditorEvents InEvents;
	SetupGraphEditorEvents(InGraph, InEvents);

	// Append play world commands
	GraphEditorCommands->Append( FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef() );

	TSharedRef<SGraphEditor> Editor = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		//.IsEditable(this, &FBlueprintEditor::IsEditable, InGraph)
		//.DisplayAsReadOnly(this, &FBlueprintEditor::IsGraphReadOnly, InGraph)
		.TitleBar(TitleBarWidget)
		//.Appearance(this, &FBlueprintEditor::GetGraphAppearance, InGraph)
		.GraphToEdit(InGraph)
		.GraphEvents(InEvents)
		//.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &FBlueprintEditor::NavigateTab, FDocumentTracker::NavigateBackwards))
		//.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &FBlueprintEditor::NavigateTab, FDocumentTracker::NavigateForwards))
		.AssetEditorToolkit(this->GetHostingApp());
		//@TODO: Crashes in command list code during the callback .OnGraphModuleReloaded(FEdGraphEvent::CreateSP(this, &FBlueprintEditor::ChangeOpenGraphInDocumentEditorWidget, WeakParent))
		;

	TWeakObjectPtr<URigVMEdGraph> WeakGraph(Cast<URigVMEdGraph>(InGraph));
	if (WeakGraph.IsValid())
	{
		TWeakPtr<FRigVMEditorBase> WeakThisEditor = SharedThis(this).ToWeakPtr();
		TWeakPtr<SGraphEditor> WeakGraphEditor = Editor.ToWeakPtr();
		
		Editor->SetOnTick(FOnGraphEditorTickDelegate::CreateLambda([WeakThisEditor, WeakGraphEditor, WeakGraph]
			(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
			{
				if (WeakThisEditor.IsValid() && WeakGraphEditor.IsValid() && WeakGraph.IsValid())
				{
					WeakThisEditor.Pin()->OnGraphEditorTick(AllottedGeometry, InCurrentTime, InDeltaTime, WeakGraphEditor.Pin().ToSharedRef(), WeakGraph.Get());
				}
			}
		));
	}
		
	//OnSetPinVisibility.AddSP(&Editor.Get(), &SGraphEditor::SetPinVisibility);

	FVector2f ViewOffset = FVector2f::ZeroVector;
	float ZoomAmount = INDEX_NONE;

	TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
	if(ActiveTab.IsValid())
	{
		// Check if the graph is already opened in the current tab, if it is we want to start at the same position to stop the graph from jumping around oddly
		TSharedPtr<SGraphEditor> GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());

		if(GraphEditor.IsValid() && GraphEditor->GetCurrentGraph() == InGraph)
		{
			GraphEditor->GetViewLocation(ViewOffset, ZoomAmount);
		}
	}

	Editor->SetViewLocation(ViewOffset, ZoomAmount);
	
	Editor->GetGraphPanel()->SetZoomLevelsContainer<FRigVMEditorZoomLevelsContainer>();
	return Editor;
}

void FRigVMNewEditor::CompileImpl()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	FRigVMEditorAssetInterfacePtr BlueprintObj = GetRigVMAssetInterface();
	if (BlueprintObj)
	{
		FMessageLog BlueprintLog("BlueprintLog");

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("BlueprintName"), FText::FromString(BlueprintObj->GetObject()->GetName()));
		BlueprintLog.NewPage(FText::Format(LOCTEXT("CompilationPageLabel", "Compile {BlueprintName}"), Arguments));

		FCompilerResultsLog LogResults = GetRigVMAssetInterface()->CompileBlueprint();

		if (CompilerResultsListing)
		{
			CompilerResultsListing->ClearMessages();

			// Note we dont mirror to the output log here as the compiler already does that
			CompilerResultsListing->AddMessages(LogResults.Messages, false);
		}

		const bool bForceMessageDisplay = (LogResults.NumWarnings > 0 || LogResults.NumErrors > 0) && !BlueprintObj->IsRegeneratingOnLoad();
		if (bForceMessageDisplay)
		{
			TabManager->TryInvokeTab(FRigVMCompilerResultsTabSummoner::TabID());
		}

		// send record when player clicks compile and send the result
		// this will make sure how the users activity is
		//AnalyticsTrackCompileEvent(BlueprintObj, LogResults.NumErrors, LogResults.NumWarnings);

		RefreshInspector();

		// Handle save-on-compile for runtime assets
		// This uses RigVM-specific settings rather than blueprint settings since runtime assets are blueprint-independent
		if (!BlueprintObj->IsRegeneratingOnLoad())
		{
			const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>();
			const ERigVMSaveOnCompile SaveOnCompileSetting = Settings->SaveOnCompile;
			const ERigVMAssetStatus AssetStatus = BlueprintObj->GetAssetStatus();

			const bool bShouldSave =
				(SaveOnCompileSetting == ERigVMSaveOnCompile::Always) ||
				(SaveOnCompileSetting == ERigVMSaveOnCompile::SuccessOnly && AssetStatus == ERigVMAssetStatus::RVMA_UpToDate);

			if (bShouldSave)
			{
				// Save the package containing the editing object
				// The editor asset (what we're editing) is a sub-object of the runtime asset,
				// both residing in the same package
				UPackage* Package = BlueprintObj->GetObject()->GetOutermost();
				if (Package)
				{
					TArray<UPackage*> PackagesToSave;
					PackagesToSave.Add(Package);
					FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty=*/true, /*bPromptToSave=*/false);
				}
			}
		}
	}
}

bool FRigVMNewEditor::IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const
{
	bool bEditable = true;

	FRigVMEditorAssetInterfacePtr EditingBP = GetRigVMAssetInterface();
	if(EditingBP)
	{
		TArray<UEdGraph*> Graphs;
		EditingBP->GetAllEdGraphs(Graphs);
		bEditable &= Graphs.Contains(InGraph);
	}

	return bEditable;
}

bool FRigVMNewEditor::IsEditableImpl(UEdGraph* InGraph) const
{
	return InEditingMode() && !FBlueprintEditorUtils::IsGraphReadOnly(InGraph);
}

UEdGraph* FRigVMNewEditor::GetFocusedGraph() const
{
	if (FocusedGraphEdPtr.IsValid())
	{
		if (UEdGraph* Graph = FocusedGraphEdPtr.Pin()->GetCurrentGraph())
		{
			if (IsValid(Graph))
			{
				return Graph;
			}
		}
	}
	return nullptr;
}

void FRigVMNewEditor::JumpToNode(const UEdGraphNode* Node, bool bRequestRename)
{
	TSharedPtr<SGraphEditor> GraphEditor;
	if(bRequestRename)
	{
		// If we are renaming, the graph will be open already, just grab the tab and it's content and jump to the node.
		TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
		check(ActiveTab.IsValid());
		GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
	}
	else
	{
		// Open a graph editor and jump to the node
		GraphEditor = OpenGraphAndBringToFront(Node->GetGraph());
	}

	if (GraphEditor.IsValid())
	{
		GraphEditor->JumpToNode(Node, bRequestRename);
	}
}

void FRigVMNewEditor::JumpToPin(const UEdGraphPin* Pin)
{
	if (!Pin->IsPendingKill())
	{
		// Open a graph editor and jump to the pin
		TSharedPtr<SGraphEditor> GraphEditor = OpenGraphAndBringToFront(Pin->GetOwningNode()->GetGraph());
		if (GraphEditor.IsValid())
		{
			GraphEditor->JumpToPin(Pin);
		}
	}
}

void FRigVMNewEditor::OnBlueprintChangedInnerImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled)
{
	if (InBlueprint)
	{
		// Notify that the blueprint has been changed (update Content browser, etc)
		InBlueprint->GetObject()->PostEditChange();

		// Call PostEditChange() on any Actors that are based on this Blueprint
		InBlueprint->PostEditChangeBlueprintActors();
		
		// Refresh the graphs
		ERefreshRigVMEditorReason::Type Reason = bIsJustBeingCompiled ? ERefreshRigVMEditorReason::BlueprintCompiled : ERefreshRigVMEditorReason::UnknownReason;
		ForceEditorRefresh(Reason);

		// In case objects were deleted, which should close the tab
		if (GetCurrentMode() == FRigVMNewEditorApplicationModes::StandardRigVMEditorMode())
		{
			// Copied from FBlueprintEditor::SaveEditedObjectState()
            {
            	check(IsEditingSingleBlueprint());
            
            	// Clear currently edited documents
            	GetRigVMAssetInterface()->GetLastEditedDocuments().Empty();
            
            	// Ask all open documents to save their state, which will update LastEditedDocuments
            	DocumentManager->SaveAllState();
            }
		}
	}
}

void FRigVMNewEditor::RefreshEditorsImpl(ERefreshRigVMEditorReason::Type Reason)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	bool bForceFocusOnSelectedNodes = false;

	// if (CurrentUISelection == SelectionState_MyBlueprint)
	// {
	// 	// Handled below, here to avoid tripping the ensure
	// }
	// else if (CurrentUISelection == SelectionState_Components)
	// {
	// 	if(SubobjectEditor.IsValid())
	// 	{
	// 		SubobjectEditor->RefreshSelectionDetails();
	// 	}
	// }
	// else if (CurrentUISelection == SelectionState_Graph)
	// {
	// 	bForceFocusOnSelectedNodes = true;
	// }
	// else if (CurrentUISelection == SelectionState_ClassSettings)
	// {
	// 	// No need for a refresh, the Blueprint object didn't change
	// }
	// else if (CurrentUISelection == SelectionState_ClassDefaults)
	// {
	// 	StartEditingDefaults(/*bAutoFocus=*/ false, true);
	// }

	// Remove any tabs are that are pending kill or otherwise invalid UObject pointers.
	DocumentManager->CleanInvalidTabs();

	//@TODO: Should determine when we need to do the invalid/refresh business and if the graph node selection change
	// under non-compiles is necessary (except when the selection mode is appropriate, as already detected above)
	if (Reason != ERefreshRigVMEditorReason::BlueprintCompiled)
	{
		DocumentManager->RefreshAllTabs();

		bForceFocusOnSelectedNodes = true;
	}

	if (bForceFocusOnSelectedNodes)
	{
		FocusInspectorOnGraphSelection(GetSelectedNodes(), /*bForceRefresh=*/ true);
	}

	// if (ReplaceReferencesWidget.IsValid())
	// {
	// 	ReplaceReferencesWidget->Refresh();
	// }

	if (GraphExplorerWidget.IsValid())
	{
		GraphExplorerWidget->Refresh();
	}

	// if(SubobjectEditor.IsValid())
	// {
	// 	SubobjectEditor->RefreshComponentTypesList();
	// 	SubobjectEditor->UpdateTree();
	//
	// 	// Note: Don't pass 'true' here because we don't want the preview actor to be reconstructed until after Blueprint modification is complete.
	// 	UpdateSubobjectPreview();
	// }
	//
	// if (BookmarksWidget.IsValid())
	// {
	// 	BookmarksWidget->RefreshBookmarksTree();
	// }

	// Note: There is an optimization inside of ShowDetailsForSingleObject() that skips the refresh if the object being selected is the same as the previous object.
	// The SKismetInspector class is shared between both Defaults mode and Components mode, but in Defaults mode the object selected is always going to be the CDO. Given
	// that the selection does not really change, we force it to refresh and skip the optimization. Otherwise, some things may not work correctly in Defaults mode. For
	// example, transform details are customized and the rotation value is cached at customization time; if we don't force refresh here, then after an undo of a previous
	// rotation edit, transform details won't be re-customized and thus the cached rotation value will be stale, resulting in an invalid rotation value on the next edit.
	//@TODO: Probably not always necessary
	//RefreshStandAloneDefaultsEditor();

	// Update associated controls like the function editor
	//BroadcastRefresh();
}

void FRigVMNewEditor::SetupGraphEditorEventsImpl(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	InEvents.OnSelectionChanged = SGraphEditor::FOnSelectionChanged::CreateSP( this, &FRigVMNewEditor::OnSelectedNodesChanged );
	// InEvents.OnVerifyTextCommit = FOnNodeVerifyTextCommit::CreateSP(this, &FRigVMNewEditor::OnNodeVerifyTitleCommit);
	InEvents.OnTextCommitted = FOnNodeTextCommitted::CreateSP(this, &FRigVMNewEditor::OnNodeTitleCommitted);
	InEvents.OnSpawnNodeByShortcutAtLocation = SGraphEditor::FOnSpawnNodeByShortcutAtLocation::CreateSP(this, &FRigVMNewEditor::OnSpawnGraphNodeByShortcut, InGraph);
	// InEvents.OnNodeSpawnedByKeymap = SGraphEditor::FOnNodeSpawnedByKeymap::CreateSP(this, &FRigVMNewEditor::OnNodeSpawnedByKeymap );
	// InEvents.OnDisallowedPinConnection = SGraphEditor::FOnDisallowedPinConnection::CreateSP(this, &FRigVMNewEditor::OnDisallowedPinConnection);
	// InEvents.OnDoubleClicked = SGraphEditor::FOnDoubleClicked::CreateSP(this, &FRigVMNewEditor::NavigateToParentGraphByDoubleClick);
	//

	// Direct Reroute Node Editing feature (URigVMEditorSettings::bDirectRerouteNodeEditing).
	// Bound only in the new editor - the legacy editor must preserve
	// FBlueprintEditor::NavigateToParentGraphByDoubleClick on its canvas double-click.
	InEvents.OnDoubleClicked = SGraphEditor::FOnDoubleClicked::CreateSP(this, &FRigVMNewEditor::HandleGraphCanvasDoubleClicked, InGraph);

	// Custom menu for K2 schemas
	if(InGraph->Schema != nullptr && InGraph->Schema->IsChildOf(UEdGraphSchema_K2::StaticClass()))
	{
		InEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &FRigVMNewEditor::OnCreateGraphActionMenu);
	}
}

void FRigVMNewEditor::HandleGraphCanvasDoubleClicked(UEdGraph* InGraph)
{
	URigVMEdGraph* RigEdGraph = Cast<URigVMEdGraph>(InGraph);
	URigVMController* Controller = RigEdGraph ? RigEdGraph->GetController() : nullptr;
	const URigVMGraph* RigGraph = RigEdGraph ? RigEdGraph->GetModel() : nullptr;
	const TSharedPtr<SGraphEditor> GraphEditor = GetGraphEditor(InGraph);
	SGraphPanel* Panel = GraphEditor.IsValid() ? GraphEditor->GetGraphPanel() : nullptr;
	if (!RigEdGraph ||
		!Controller ||
		!RigGraph ||
		!GraphEditor.IsValid() ||
		!Panel)
	{
		return;
	}

	const IRigVMClientHost* ClientHost = RigGraph->GetImplementingOuter<IRigVMClientHost>();
	const UClass* SettingsClass = ClientHost ? ClientHost->GetRigVMEditorSettingsClass() : URigVMEditorSettings::StaticClass();
	const URigVMEditorSettings* Settings = GetDefault<URigVMEditorSettings>(SettingsClass);
	if (!Settings->bDirectRerouteNodeEditing)
	{
		return;
	}

	// Adding reroute nodes can be slow, hence do only once per frame,
	// to avoid any oddities when clicking faster than frame time
	if (GFrameCounter == LastDoubleClickFrame)
	{
		return;
	}
	LastDoubleClickFrame = GFrameCounter;

	// Similar to Graph Editor's FConnectionDrawingPolicy::CheckSplineConnectionOverlapWithCursor
	float Zoom = 1.f;
	FVector2f IgnoredViewLoc(ForceInitToZero);
	GraphEditor->GetViewLocation(IgnoredViewLoc, Zoom);

	const FVector2D CursorScreen = FVector2D(FSlateApplication::Get().GetCursorPos());
	const FVector2D PanelLocalPos = Panel->GetCachedGeometry().AbsoluteToLocal(CursorScreen);
	const FVector2D GraphPos = FVector2D(Panel->PanelCoordToGraphCoord(PanelLocalPos));

	constexpr double HalfWidthGraph  = 100.0;
	constexpr double HalfHeightGraph = 100.0;
	const FBox2D SearchBox(
		FVector2D(GraphPos.X - HalfWidthGraph,  GraphPos.Y - HalfHeightGraph),
		FVector2D(GraphPos.X + HalfWidthGraph,  GraphPos.Y + HalfHeightGraph));

	const auto GetPinGraphPos =
		[&](URigVMPin* ModelPin, EEdGraphPinDirection Dir) -> TOptional<FVector2D>
		{
			const URigVMNode* Node = ModelPin ? ModelPin->GetNode() : nullptr;
			if (!ModelPin ||
				!Node)
			{
				return {};
			}

			UEdGraphNode* EdNode = RigEdGraph->FindNodeForModelNodeName(Node->GetFName());
			UEdGraphPin* EdPin = EdNode ? EdNode->FindPin(*ModelPin->GetPinPath(), Dir) : nullptr;
			if (!EdNode ||
				!EdPin)
			{
				return {};
			}

			const TSharedPtr<SGraphNode> NodeWidget = Panel->GetNodeWidgetFromGuid(EdNode->NodeGuid);
			const TSharedPtr<SGraphPin> PinWidget = NodeWidget.IsValid() ? NodeWidget->FindWidgetForPin(EdPin) : nullptr;
			if (!NodeWidget.IsValid() ||
				!PinWidget.IsValid())
			{
				return {};
			}

			return FVector2D(Node->GetPosition()) + FVector2D(PinWidget->GetNodeOffset());
		};

	// Detects reversed reroutes - matches the drawing policy's heuristic for tangent direction.
	const auto IsBackwardsReroute =
		[&](const URigVMNode* ModelNode) -> bool
		{
			if (!Cast<URigVMRerouteNode>(ModelNode))
			{
				return false;
			}

			UEdGraphNode* EdNode = RigEdGraph->FindNodeForModelNodeName(ModelNode->GetFName());
			if (!EdNode)
			{
				return false;
			}

			int32 InputIdx = 0;
			int32 OutputIdx = 0;
			if (!EdNode->ShouldDrawNodeAsControlPointOnly(InputIdx, OutputIdx))
			{
				return false;
			}

			const TArray<UEdGraphPin*>& Pins = EdNode->GetAllPins();
			if (!Pins.IsValidIndex(InputIdx) || !Pins.IsValidIndex(OutputIdx))
			{
				return false;
			}

			const auto AvgLinkedX = [](const UEdGraphPin* Pin) -> TOptional<double>
			{
				double Sum = 0.0;
				int32 Count = 0;
				for (UEdGraphPin* Linked : Pin->LinkedTo)
				{
					if (const UEdGraphNode* LinkedNode = Linked ? Linked->GetOwningNode() : nullptr)
					{
						Sum += LinkedNode->NodePosX;
						++Count;
					}
				}
				return Count > 0 ? TOptional<double>(Sum / Count) : TOptional<double>();
			};

			const TOptional<double> LeftAvg = AvgLinkedX(Pins[InputIdx]);
			const TOptional<double> RightAvg = AvgLinkedX(Pins[OutputIdx]);
			if (LeftAvg.IsSet() && RightAvg.IsSet())
			{
				return RightAvg.GetValue() < LeftAvg.GetValue();
			}

			const double CenterX = static_cast<double>(EdNode->NodePosX);
			if (LeftAvg.IsSet())
			{
				return CenterX < LeftAvg.GetValue();
			}
			if (RightAvg.IsSet())
			{
				return RightAvg.GetValue() < CenterX;
			}
			return false;
		};

	// Match FRigVMEdGraphConnectionDrawingPolicy::UseLowDetailConnections().
	const bool bLowDetail = Zoom <= 0.175f;

	const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();
	constexpr int32 NumSamples = 16;
	constexpr double MaximumTangentContribution = 4.0 / 27.0;

	URigVMLink* BestLink = nullptr;
	double BestDistSquared = TNumericLimits<double>::Max();

	for (URigVMLink* Link : RigGraph->GetLinks())
	{
		if (!Link)
		{
			continue;
		}

		const TOptional<FVector2D> OptionalPoint0 = GetPinGraphPos(Link->GetSourcePin(), EGPD_Output);
		const TOptional<FVector2D> OptionalPoint1 = GetPinGraphPos(Link->GetTargetPin(), EGPD_Input);
		if (!OptionalPoint0.IsSet() ||
			!OptionalPoint1.IsSet())
		{
			continue;
		}
		const FVector2D Point0 = OptionalPoint0.GetValue();
		const FVector2D Point1 = OptionalPoint1.GetValue();

		FVector2D P0Tangent = FVector2D::ZeroVector;
		FVector2D P1Tangent = FVector2D::ZeroVector;
		if (!bLowDetail)
		{
			const FVector2D SplineTangent = FVector2D(GraphEditorSettings->ComputeSplineTangent(FVector2f(Point0), FVector2f(Point1)));
			P0Tangent = IsBackwardsReroute(Link->GetSourcePin()->GetNode()) ? -SplineTangent : SplineTangent;
			P1Tangent = IsBackwardsReroute(Link->GetTargetPin()->GetNode()) ? -SplineTangent : SplineTangent;
		}

		// Stage 1: AABB pre-filter, skip if the spline can't reach the search box.
		FBox2D Bounds(ForceInit);
		Bounds += Point0;
		Bounds += Point1;
		if (!bLowDetail)
		{
			Bounds += Point0 + MaximumTangentContribution * P0Tangent;
			Bounds += Point1 - MaximumTangentContribution * P1Tangent;
		}
		if (!SearchBox.Intersect(Bounds))
		{
			continue;
		}

		// Stage 2: Sample the spline. Only count samples that land inside the box.
		double LinkMinDistSquared = TNumericLimits<double>::Max();
		if (bLowDetail)
		{
			const FVector2D Closest = FMath::ClosestPointOnSegment2D(GraphPos, Point0, Point1);
			if (SearchBox.IsInside(Closest))
			{
				LinkMinDistSquared = FVector2D::DistSquared(GraphPos, Closest);
			}
		}
		else
		{
			FVector2D Prev = FMath::CubicInterp(Point0, P0Tangent, Point1, P1Tangent, 0.0);
			for (int32 iSampleIndex = 0; iSampleIndex < NumSamples; ++iSampleIndex)
			{
				const double Alpha = static_cast<double>(iSampleIndex + 1) / NumSamples;
				const FVector2D Next = FMath::CubicInterp(Point0, P0Tangent, Point1, P1Tangent, Alpha);
				const FVector2D Closest = FMath::ClosestPointOnSegment2D(GraphPos, Prev, Next);
				if (SearchBox.IsInside(Closest))
				{
					const double DistSquared = FVector2D::DistSquared(GraphPos, Closest);
					if (DistSquared < LinkMinDistSquared)
					{
						LinkMinDistSquared = DistSquared;
					}
				}
				Prev = Next;
			}
		}

		if (LinkMinDistSquared < BestDistSquared)
		{
			BestDistSquared = LinkMinDistSquared;
			BestLink = Link;
		}
	}

	if (BestLink)
	{
		const FVector2D KnotHalfSize(21.f, 12.f);

		const FString NodeName;
		constexpr bool bSetupUndo = true;
		constexpr bool bPrintPythonCommand = true;
		Controller->AddRerouteNodeOnLink(BestLink, GraphPos - KnotHalfSize, NodeName, bSetupUndo, bPrintPythonCommand);
	}
}

void FRigVMNewEditor::OnClosedActionMenu(UEdGraph* InEdGraph) const
{
	if (TSharedPtr<SGraphEditor> GraphEditor = GetGraphEditor(InEdGraph))
	{
		if (SGraphPanel* GraphPanel = GraphEditor->GetGraphPanel())
		{
			GraphPanel->OnStopMakingConnection(true);
		}
	}
}

FActionMenuContent FRigVMNewEditor::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition,
                                                            const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	// TODO sara-s: Replicate what blueprint editor is doing
	HasOpenActionMenu = InGraph;
	// if (!BlueprintEditorImpl::GraphHasUserPlacedNodes(InGraph))
	// {
	// 	InstructionsFadeCountdown = BlueprintEditorImpl::InstructionFadeDuration;
	// }

	TSharedRef<SRigVMActionMenu> ActionMenu = 
		SNew(SRigVMActionMenu, SharedThis(this))
		.GraphObj(InGraph)
		.NewNodePosition(InNodePosition)
		.DraggedFromPins(InDraggedPins)
		.OnClosedCallback( SGraphEditor::FActionMenuClosed::CreateSP(this, &FRigVMNewEditor::OnClosedActionMenu, InGraph));

	return FActionMenuContent( ActionMenu, ActionMenu->GetFilterTextBox() );
}

void FRigVMNewEditor::AddReferencedObjectsImpl(FReferenceCollector& Collector)
{
	
}

bool FRigVMNewEditor::IsSectionVisible(RigVMNodeSectionID::Type InSectionID) const
{
	return FRigVMEditorBase::IsSectionVisibleImpl(InSectionID);
}

bool FRigVMNewEditor::NewDocument_IsVisibleForType(ECreatedDocumentType GraphType) const
{
	return FRigVMEditorBase::NewDocument_IsVisibleForTypeImpl(GraphType);
}

FGraphAppearanceInfo FRigVMNewEditor::GetGraphAppearanceImpl(UEdGraph* InGraph) const
{
	// Create the appearance info
	FGraphAppearanceInfo AppearanceInfo;

	
	auto Blueprint = (InGraph != nullptr) ? FRigVMBlueprintUtils::FindAssetForGraph(InGraph) : GetRigVMAssetInterface();
	if (Blueprint != nullptr)
	{
		AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Blueprint", "BLUEPRINT");
	}

	UEdGraph const* EditingGraph = GetFocusedGraph();
	if (InGraph && RigVMNewEditorImpl::GraphHasDefaultNode(InGraph))
	{
		AppearanceInfo.InstructionText = LOCTEXT("AppearanceInstructionText_DefaultGraph", "Drag Off Pins to Create/Connect New Nodes.");
	}
	else // if the graph is empty...
		{
		AppearanceInfo.InstructionText = LOCTEXT("AppearanceInstructionText_EmptyGraph", "Right-Click to Create New Nodes.");
		}
	auto InstructionOpacityDelegate = TAttribute<float>::FGetter::CreateSP(this, &FRigVMNewEditor::GetInstructionTextOpacity, InGraph);
	AppearanceInfo.InstructionFade.Bind(InstructionOpacityDelegate);

	// Copy from FBlueprintEditor::GetPIEStatus
	{
		FRigVMEditorAssetInterfacePtr CurrentBlueprint = GetRigVMAssetInterface();
		UWorld *DebugWorld = nullptr;
		ENetMode NetMode = NM_Standalone;
		if (CurrentBlueprint)
		{
			DebugWorld = CurrentBlueprint->GetWorldBeingDebugged();
			if (DebugWorld)
			{
				NetMode = DebugWorld->GetNetMode();
			}
			else
			{
				UObject* ObjOuter = CurrentBlueprint->GetObjectBeingDebugged();
				while(DebugWorld == nullptr && ObjOuter != nullptr)
				{
					ObjOuter = ObjOuter->GetOuter();
					DebugWorld = Cast<UWorld>(ObjOuter);
				}

				if (DebugWorld)
				{
					// Redirect through streaming levels to find the owning world; this ensures that we always use the appropriate NetMode for the context string below.
					if (DebugWorld->PersistentLevel != nullptr && DebugWorld->PersistentLevel->OwningWorld != nullptr)
					{
						DebugWorld = DebugWorld->PersistentLevel->OwningWorld;
					}

					NetMode = DebugWorld->GetNetMode();
				}
			}
		}

		if (NetMode == NM_ListenServer || NetMode == NM_DedicatedServer)
		{
			AppearanceInfo.PIENotifyText = LOCTEXT("PIEStatusServerSimulating", "SERVER - SIMULATING");
		}
		else if (NetMode == NM_Client)
		{
			FWorldContext* PIEContext = GEngine->GetWorldContextFromWorld(DebugWorld);
			if (PIEContext && PIEContext->PIEInstance > 1)
			{
				AppearanceInfo.PIENotifyText = FText::Format(LOCTEXT("PIEStatusClientSimulatingFormat", "CLIENT {0} - SIMULATING"), FText::AsNumber(PIEContext->PIEInstance - 1));
			}
		
			AppearanceInfo.PIENotifyText = LOCTEXT("PIEStatusClientSimulating", "CLIENT - SIMULATING");
		}

		AppearanceInfo.PIENotifyText = LOCTEXT("PIEStatusSimulating", "SIMULATING");
	}

	return AppearanceInfo;
}

void FRigVMNewEditor::NotifyPreChangeImpl(FProperty* PropertyAboutToChange)
{
	// this only delivers message to the "FOCUSED" one, not every one
	// internally it will only deliver the message to the selected node, not all nodes
	FString PropertyName = PropertyAboutToChange->GetName();
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->NotifyPrePropertyChange(PropertyName);
	}
}

void FRigVMNewEditor::NotifyPostChangeImpl(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FString PropertyName = PropertyThatChanged->GetName();
	if (FocusedGraphEdPtr.IsValid())
	{
		FocusedGraphEdPtr.Pin()->NotifyPostPropertyChange(PropertyChangedEvent, PropertyName);
	}
	
	if (IsEditingSingleBlueprint())
	{
		FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
		UPackage* BlueprintPackage = Blueprint->GetObject()->GetOutermost();

		// if any of the objects being edited are in our package, mark us as dirty
		bool bPropertyInBlueprint = false;
		for (int32 ObjectIndex = 0; ObjectIndex < PropertyChangedEvent.GetNumObjectsBeingEdited(); ++ObjectIndex)
		{
			const UObject* Object = PropertyChangedEvent.GetObjectBeingEdited(ObjectIndex);
			if (Object && Object->GetOutermost() == BlueprintPackage)
			{
				bPropertyInBlueprint = true;
				break;
			}
		}

		if (bPropertyInBlueprint)
		{
			// Note: if change type is "interactive," hold off on applying the change (e.g. this will occur if the user is scrubbing a spinbox value; we don't want to apply the change until the mouse is released, for performance reasons)
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				Blueprint->MarkAssetAsModified(PropertyChangedEvent);

				// Call PostEditChange() on any Actors that might be based on this Blueprint
				Blueprint->PostEditChangeBlueprintActors();
			}

			// TODO sara-s
			// Force updates to occur immediately during interactive mode (otherwise the preview won't refresh because it won't be ticking)
			//UpdateSubobjectPreview(PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive);
		}
	}
}

FName FRigVMNewEditor::GetSelectedVariableName()
{
	FName VariableName;
	if (GraphExplorerWidget.IsValid())
	{
		return GraphExplorerWidget->GetSelectedVariableName();
	}
	return VariableName;
}

bool FRigVMNewEditor::IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename)
{
	TSharedPtr<SGraphEditor> GraphEditor;
	if(bRequestRename)
	{
		// If we are renaming, the graph will be open already, just grab the tab and it's content and jump to the node.
		TSharedPtr<SDockTab> ActiveTab = DocumentManager->GetActiveTab();
		check(ActiveTab.IsValid());
		GraphEditor = StaticCastSharedRef<SGraphEditor>(ActiveTab->GetContent());
	}
	else
	{
		// Open a graph editor and jump to the node
		GraphEditor = OpenGraphAndBringToFront(Node->GetGraph());
	}

	bool bVisible = false;
	if (GraphEditor.IsValid())
	{
		bVisible = GraphEditor->IsNodeTitleVisible(Node, bRequestRename);
	}
	return bVisible;
}

void FRigVMNewEditor::EditClassDefaults_Clicked()
{
	StartEditingDefaults(true, true);
}

void FRigVMNewEditor::EditGlobalOptions_Clicked()
{
	SetUISelectionState(FRigVMNewEditor::SelectionState_ClassSettings());

	if (bWasOpenedInDefaultsMode)
	{
		RefreshStandAloneDefaultsEditor();
	}
	else
	{
		FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
		if (Blueprint != nullptr)
		{
			// Show details for the Blueprint instance we're editing
			if (Inspector.IsValid())
			{
				SRigVMDetailsInspector::FShowDetailsOptions Options(FText::FromString(TEXT("Settings")), true);
				Inspector->ShowDetailsForSingleObject(Blueprint->GetRuntimeAssetInterface().GetObject(), Options);
			}

			TryInvokingDetailsTab();
		}
	}
}

bool FRigVMNewEditor::IsDetailsPanelEditingGlobalOptions() const
{
	return CurrentUISelection == FRigVMNewEditor::SelectionState_ClassSettings();
}

bool FRigVMNewEditor::IsDetailsPanelEditingClassDefaults() const
{
	return CurrentUISelection == FRigVMNewEditor::SelectionState_ClassDefaults();
}

void FRigVMNewEditor::TryInvokingDetailsTab(bool bFlash)
{
	if ( TabManager->HasTabSpawner(FRigVMDetailsInspectorTabSummoner::TabID()) )
	{
		TSharedPtr<SDockTab> BlueprintTab = FGlobalTabmanager::Get()->GetMajorTabForTabManager(TabManager.ToSharedRef());

		// We don't want to force this tab into existence when the blueprint editor isn't in the foreground and actively
		// being interacted with.  So we make sure the window it's in is focused and the tab is in the foreground.
		if ( BlueprintTab.IsValid() && BlueprintTab->IsForeground() )
		{
			if ( !Inspector.IsValid() || !Inspector->GetOwnerTab().IsValid() || Inspector->GetOwnerTab()->GetDockArea().IsValid() )
			{
				// Show the details panel if it doesn't exist.
				TabManager->TryInvokeTab(FRigVMDetailsInspectorTabSummoner::TabID());

				if ( bFlash && Inspector.IsValid())
				{
					TSharedPtr<SDockTab> OwnerTab = Inspector->GetOwnerTab();
					if ( OwnerTab.IsValid() )
					{
						OwnerTab->FlashTab();
					}
				}
			}
		}
	}
}

void FRigVMNewEditor::RefreshInspector()
{
	if (Inspector.IsValid())
	{
	    Inspector->GetPropertyView()->ForceRefresh();
	}
}

void FRigVMNewEditor::RefreshStandAloneDefaultsEditor()
{
	// Update the details panel
	SRigVMDetailsInspector::FShowDetailsOptions Options(FText::GetEmpty(), true);

	TArray<UObject*> DefaultObjects;
	for ( int32 i = 0; i < GetEditingObjects().Num(); ++i )
	{
		UObject* Object = GetEditingObjects()[i];
		if (Object && Object->Implements<URigVMEditorAssetInterface>())
		{
			FRigVMEditorAssetInterfacePtr EditorAsset = Object;
			if (CurrentUISelection == FRigVMNewEditor::SelectionState_ClassSettings())
			{
				DefaultObjects.Add(EditorAsset.GetObject());
			}
			else if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = EditorAsset->GetRuntimeAssetInterface())
			{
				DefaultObjects.Add(RuntimeAsset.GetObject());
			}
		}
	}

	if ( DefaultObjects.Num() && DefaultEditor.IsValid() )
	{
		DefaultEditor->ShowDetailsForObjects(DefaultObjects);
	}
}

void FRigVMNewEditor::StartEditingDefaults(bool bAutoFocus, bool bForceRefresh)
{
	SetUISelectionState(FRigVMNewEditor::SelectionState_ClassDefaults());

	if (IsEditingSingleBlueprint())
	{
		if (UObject* DefaultObject = GetRigVMAssetInterface()->GetDefaultsObject())
		{
			// TODO sara-s
			// if (SubobjectEditor.IsValid() && GetBlueprintObj()->GeneratedClass->IsChildOf<AActor>())
			// {
			// 	SubobjectEditor->SelectRoot();
			// }
			// else
			{
				// Update the details panel
				FString Title;
				DefaultObject->GetName(Title);

				SRigVMDetailsInspector::FShowDetailsOptions Options(FText::FromString(Title), bForceRefresh);

				if (Inspector.IsValid())
				{
					Inspector->ShowDetailsForSingleObject(DefaultObject, Options);
				}

				if ( bAutoFocus )
				{
					TryInvokingDetailsTab();
				}
			}
		}
	}
	
	RefreshStandAloneDefaultsEditor();
}

float FRigVMNewEditor::GetInstructionTextOpacity(UEdGraph* InGraph) const
{
	bool bGraphReadOnly = true;
	if (InGraph)
	{
		bGraphReadOnly = !InGraph->bEditable;
	}
	
	URigVMEditorSettings const* Settings = GetDefault<URigVMEditorSettings>();
	if ((InGraph == nullptr) || !IsEditable(InGraph) || bGraphReadOnly)
	{
		return 0.0f;
	}
	else if ((InstructionsFadeCountdown > 0.0f) || (HasOpenActionMenu == InGraph))
	{
		return InstructionsFadeCountdown / RigVMNewEditorImpl::InstructionFadeDuration;
	}
	else if (RigVMNewEditorImpl::GraphHasUserPlacedNodes(InGraph))
	{
		return 0.0f;
	}
	return 1.0f;
}

void FRigVMNewEditor::ClearSelectionStateFor(FName SelectionOwner)
{
	if ( SelectionOwner == SelectionState_Graph() )
	{
		TArray< TSharedPtr<SDockTab> > GraphEditorTabs;
		DocumentManager->FindAllTabsForFactory(GraphEditorTabFactoryPtr, /*out*/ GraphEditorTabs);

		for (TSharedPtr<SDockTab>& GraphEditorTab : GraphEditorTabs)
		{
			TSharedRef<SGraphEditor> Editor = StaticCastSharedRef<SGraphEditor>((GraphEditorTab)->GetContent());

			Editor->ClearSelectionSet();
		}
	}
	else if ( SelectionOwner == SelectionState_GraphExplorer() )
	{
		if ( GraphExplorerWidget.IsValid() )
		{
			GraphExplorerWidget->ClearSelection();
		}
	}
}

void FRigVMNewEditor::OnLogTokenClicked(const TSharedRef<IMessageToken>& MessageToken)
{
	if (MessageToken->GetType() == EMessageToken::EdGraph)
	{
		const TSharedRef<FEdGraphToken> EdGraphToken = StaticCastSharedRef<FEdGraphToken>(MessageToken);
		const UEdGraphPin* PinBeingReferenced = EdGraphToken->GetPin();
		const UObject* ObjectBeingReferenced = EdGraphToken->GetGraphObject();
		if (PinBeingReferenced)
		{
			JumpToPin(PinBeingReferenced);
		}
		else if(ObjectBeingReferenced)
		{
			JumpToHyperlink(ObjectBeingReferenced, false);
		}
	}
}

void FRigVMNewEditor::DumpMessagesToCompilerLog(const TArray<TSharedRef<class FTokenizedMessage>>& Messages, bool bForceMessageDisplay)
{
	CompilerResultsListing->ClearMessages();

	// Note we dont mirror to the output log here as the compiler already does that
	CompilerResultsListing->AddMessages(Messages, false);
	
	if (bForceMessageDisplay)
	{
		TabManager->TryInvokeTab(FRigVMCompilerResultsTabSummoner::TabID());
	}
}

void FRigVMNewEditor::CreateDefaultTabContents(const TArray<FRigVMEditorAssetInterfacePtr> InBlueprints)
{
	FRigVMEditorAssetInterfacePtr InBlueprint = InBlueprints.Num() == 1 ? InBlueprints[0] : nullptr;

	// Cache off whether or not this is an interface, since it is used to govern multiple widget's behavior
	//const bool bIsInterface = (InBlueprint && InBlueprint->BlueprintType == BPTYPE_Interface);

	if (InBlueprint)
	{
		// TODO sara-s
		// this->BookmarksWidget =
		// 	SNew(SBlueprintBookmarks)
		// 		.EditorContext(SharedThis(this));
	}

	if (IsEditingSingleBlueprint())
	{
		//this->ReplaceReferencesWidget = SNew(SReplaceNodeReferences, SharedThis(this)); // TODO sara-s
	}

	if (UBlueprint* Blueprint = Cast<UBlueprint>(InBlueprint.GetObject()))
	{
		CompilerResultsListing = FCompilerResultsLog::GetBlueprintMessageLog(Blueprint);
		CompilerResultsListing->OnMessageTokenClicked().AddSP(this, &FRigVMNewEditor::OnLogTokenClicked);

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		CompilerResults = MessageLogModule.CreateLogListingWidget( CompilerResultsListing.ToSharedRef() );
	}
	else
	{
		// GetBlueprintMessageLog
		{
			FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

			FName LogName;
			//const FName LogName = GetBlueprintMessageLogName(InBlueprint);
			{
				if (InBlueprint != nullptr)
				{
					LogName = *FString::Printf(TEXT("%s_CompilerResultsLog"), *InBlueprint->GetObject()->GetName());
				}
				else
				{
					LogName = "BlueprintCompiler";
				}
			}

			// Reuse any existing log, or create a new one (that is not held onto bey the message log system)
			if(MessageLogModule.IsRegisteredLogListing(LogName))
			{
				CompilerResultsListing = MessageLogModule.GetLogListing(LogName);
			}
			else
			{
				FMessageLogInitializationOptions LogInitOptions;
				LogInitOptions.bShowInLogWindow = false;
				CompilerResultsListing = MessageLogModule.CreateLogListing(LogName, LogInitOptions);
			}
		}
		CompilerResultsListing->OnMessageTokenClicked().AddSP(this, &FRigVMNewEditor::OnLogTokenClicked);

		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		CompilerResults = MessageLogModule.CreateLogListingWidget( CompilerResultsListing.ToSharedRef() );
	}

	FindResults = SNew(SRigVMFindReferences, SharedThis(this));
	
	this->Inspector = 
		SNew(SRigVMDetailsInspector)
		.Editor(SharedThis(this))
		. HideNameArea(true)
		. ViewIdentifier(FName("BlueprintInspector"))
		. OnFinishedChangingProperties( FOnFinishedChangingProperties::FDelegate::CreateSP(this, &FRigVMNewEditor::OnFinishedChangingProperties) ); 
	;

	if ( InBlueprints.Num() > 0 )
	{
		// Don't show the object name in defaults mode.
		const bool bHideNameArea = bWasOpenedInDefaultsMode;

		this->DefaultEditor = 
			SNew(SRigVMDetailsInspector)
			.Editor(SharedThis(this))
			. ViewIdentifier(FName("BlueprintDefaults"))
			//. ShowPublicViewControl(this, &FBlueprintEditor::ShouldShowPublicViewControl) // TODO sara-s
			. ShowTitleArea(false)
			. HideNameArea(bHideNameArea)
			. OnFinishedChangingProperties( FOnFinishedChangingProperties::FDelegate::CreateSP( this, &FRigVMNewEditor::OnFinishedChangingProperties ) ); 
		;
	}

	// if (InBlueprint && 
	// 	InBlueprint->ParentClass &&
	// 	InBlueprint->ParentClass->IsChildOf(AActor::StaticClass()) && 
	// 	InBlueprint->SimpleConstructionScript )
	// {
	// 	//CreateSubobjectEditors(); // TODO sara-s
	// }
}

const FSlateBrush* FRigVMNewEditor::GetGlyphForGraph(const UEdGraph* Graph, bool bInLargeIcon)
{
	const FSlateBrush* ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Function_24x") : TEXT("GraphEditor.Function_16x") );

	check(Graph != nullptr);
	const UEdGraphSchema* Schema = Graph->GetSchema();
	if (Schema != nullptr)
	{
		const EGraphType GraphType = Schema->GetGraphType(Graph);
		switch (GraphType)
		{
		default:
		case GT_StateMachine:
		case GT_Macro:
		case GT_Animation:
			check(false);
			break;
		case GT_Function:
			{
				bool bIsSubGraph = false;
				{
					if( Graph->GetOuter() )
					{
						//Check whether the outer is a composite node
						if( Graph->GetOuter()->IsA( UK2Node_Composite::StaticClass() ) )
						{
							bIsSubGraph = true;
						}
					}
				}
				
				//Check for subgraph
				if( bIsSubGraph )
				{
					ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.SubGraph_24x") : TEXT("GraphEditor.SubGraph_16x") );
				}
				else
				{
					ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.Function_24x") : TEXT("GraphEditor.Function_16x") );
				}
			}
			break;
		case GT_Ubergraph:
			{
				ReturnValue = FAppStyle::GetBrush( bInLargeIcon ? TEXT("GraphEditor.EventGraph_24x") : TEXT("GraphEditor.EventGraph_16x") );
			}
			break;
		}
	}

	return ReturnValue;
}

void FRigVMNewEditor::OnSelectedNodesChanged(const FGraphPanelSelectionSet& NewSelection)
{
	OnSelectedNodesChangedImpl(NewSelection);
}

void FRigVMNewEditor::OnAlignTop()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignTop();
	}
}

void FRigVMNewEditor::OnAlignMiddle()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignMiddle();
	}
}

void FRigVMNewEditor::OnAlignBottom()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignBottom();
	}
}

void FRigVMNewEditor::OnAlignLeft()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignLeft();
	}
}

void FRigVMNewEditor::OnAlignCenter()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignCenter();
	}
}

void FRigVMNewEditor::OnAlignRight()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnAlignRight();
	}
}

void FRigVMNewEditor::OnStraightenConnections()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnStraightenConnections();
	}
}

void FRigVMNewEditor::OnDistributeNodesH()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnDistributeNodesH();
	}
}

void FRigVMNewEditor::OnDistributeNodesV()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->OnDistributeNodesV();
	}
}

void FRigVMNewEditor::SelectAllNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		FocusedGraphEd->SelectAllNodes();
	}
}

bool FRigVMNewEditor::CanSelectAllNodes() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE
