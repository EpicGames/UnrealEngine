// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMNewEditorMode.h"

#include "Editor/RigVMDetailsInspectorTabSummoner.h"
#include "Editor/RigVMCompilerResultsTabSummoner.h"
#include "Editor/RigVMEditor.h"
#include "Editor/RigVMExecutionStackTabSummoner.h"
#include "Editor/RigVMFindReferencesTabSummoner.h"
#include "Editor/RigVMGraphEditorTabSummoner.h"
#include "Editor/RigVMNewEditor.h"

const FName FRigVMNewEditorApplicationModes::StandardRigVMEditorMode() { static FName Mode = TEXT("GraphName"); return Mode; };
const FName FRigVMNewEditorApplicationModes::RigVMDefaultsMode() { static FName Mode = TEXT("DefaultsName"); return Mode; };

FRigVMNewEditorMode::FRigVMNewEditorMode(const TSharedRef<FRigVMNewEditor>& InRigVMEditor)
	: FApplicationMode(FRigVMEditorModes::RigVMEditorMode, FRigVMEditorModes::GetLocalizedMode)
{
	Editor = InRigVMEditor;
	
	TabFactories.RegisterFactory(MakeShared<FRigVMExecutionStackTabSummoner>(InRigVMEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMEditorGraphExplorerTabSummoner>(InRigVMEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMDetailsInspectorTabSummoner>(InRigVMEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMCompilerResultsTabSummoner>(InRigVMEditor));
	TabFactories.RegisterFactory(MakeShared<FRigVMFindReferencesTabSummoner>(InRigVMEditor));

	TabLayout = FTabManager::NewLayout("Standalone_RigVMEditMode_Layout_v1.7")
		->AddArea
		(
			// Main application area
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					//	Left bottom - rig/hierarchy
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(FRigVMExecutionStackTabSummoner::TabID, ETabState::OpenedTab)
					->AddTab(FRigVMEditorGraphExplorerTabSummoner::TabID(), ETabState::OpenedTab)
				)
				->Split
				(
					// Middle 
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.6f)
					->Split
					(
						// Middle top
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->AddTab(FRigVMGraphEditorTabSummoner::TabID(), ETabState::ClosedTab)
					)
					->Split
					(
						// Middle bottom - compiler results & find
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(FRigVMCompilerResultsTabSummoner::TabID(), ETabState::OpenedTab)
						->AddTab(FRigVMFindReferencesTabSummoner::TabID(), ETabState::OpenedTab)
					)
				)
				->Split
				(
					// Right side
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.2f)
					->Split
					(
						// Right top
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(1.f)
						->AddTab(FRigVMDetailsInspectorTabSummoner::TabID(), ETabState::OpenedTab)
						->SetForegroundTab(FRigVMDetailsInspectorTabSummoner::TabID())
					)
				)
			)
		);
}

void FRigVMNewEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FRigVMNewEditor> BP = Editor.Pin();

	// Mode-specific setup
	BP->PushTabFactories(TabFactories);
}

void FRigVMNewEditorMode::PostActivateMode()
{
	// Reopen any documents that were open when the blueprint was last saved
	TSharedPtr<FRigVMNewEditor> BP = Editor.Pin();
	BP->RestoreEditedObjectState();
	BP->SetupViewForBlueprintEditingMode();

	FApplicationMode::PostActivateMode();
}

void FRigVMNewEditorMode::PreDeactivateMode()
{
	FApplicationMode::PreDeactivateMode();

	// Capture which documents are open so the asset's LastEditedDocuments reflects the
	// current session; without this, tab state from the editing session is lost on close.
	// SaveEditedObjectState itself early-returns when IsEditingSingleBlueprint() is false,
	// so we don't gate it here (that method is protected on FRigVMEditorBase anyway).
	if (TSharedPtr<FRigVMNewEditor> BP = Editor.Pin())
	{
		BP->SaveEditedObjectState();
	}
}
