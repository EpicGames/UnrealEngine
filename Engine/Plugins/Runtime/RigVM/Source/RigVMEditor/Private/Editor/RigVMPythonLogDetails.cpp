// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMPythonLogDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPythonScriptPlugin.h"
#include "RigVMPythonUtils.h"
#include "Components/VerticalBox.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RigVMPythonDetails"

void SRigVMPythonLogDetails::Construct(const FArguments& InArgs, FRigVMEditorAssetInterfacePtr InAsset)
{
	BlueprintBeingCustomized = InAsset;
	
	ChildSlot
	[
		SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &SRigVMPythonLogDetails::OnCopyPythonScriptClicked)
				.ContentPadding(FMargin(2))
				.Content()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("CopyPythonScript", "Copy Python Script"))
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &SRigVMPythonLogDetails::OnRunPythonContextClicked)
				.ContentPadding(FMargin(2))
				.Content()
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("RunPythonContext", "Run Python Context"))
				]
			]
	];
}

FReply SRigVMPythonLogDetails::OnCopyPythonScriptClicked() const
{
	if (BlueprintBeingCustomized)
	{
		FString NewName = BlueprintBeingCustomized.GetObject()->GetPathName();
		int32 DotIndex = NewName.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (DotIndex != INDEX_NONE)
		{
			NewName = NewName.Left(DotIndex);
		}
		
		TArray<FString> Commands = BlueprintBeingCustomized->GeneratePythonContextCommands(NewName, true);
		Commands.Append(BlueprintBeingCustomized->GeneratePythonCommands());
		FString FullScript = FString::Join(Commands, TEXT("\n"));
		FPlatformApplicationMisc::ClipboardCopy(*FullScript);
	}
	return FReply::Handled();
}

FReply SRigVMPythonLogDetails::OnRunPythonContextClicked() const
{
	if (BlueprintBeingCustomized)
	{
		FString BlueprintName = BlueprintBeingCustomized.GetObject()->GetPathName();
		int32 DotIndex = BlueprintName.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (DotIndex != INDEX_NONE)
		{
			BlueprintName = BlueprintName.Left(DotIndex);
		}
		
		TArray<FString> PyCommands = BlueprintBeingCustomized->GeneratePythonContextCommands(BlueprintName, false);
		for (FString& Command : PyCommands)
		{
			RigVMPythonUtils::Print(BlueprintBeingCustomized.GetObject()->GetFName().ToString(), Command);
		
			// Run the Python commands
			IPythonScriptPlugin::Get()->ExecPythonCommand(*Command);
		}
	}
	return FReply::Handled();
}

void FRigVMPythonLogDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		InStructPropertyHandle->CreatePropertyValueWidget()
	];

	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);
	ensure(Objects.Num() == 1); // This is in here to ensure we are only showing the modifier details in the blueprint editor

	for (UObject* Object : Objects)
	{
		BlueprintBeingCustomized = Object;
	}
}

void FRigVMPythonLogDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (InStructPropertyHandle->IsValidHandle())
	{
		uint32 NumChildren = 0;
		InStructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef());
		}

		StructBuilder.AddCustomRow(LOCTEXT("Commands", "Python Commands"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Commands")))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SRigVMPythonLogDetails, BlueprintBeingCustomized)
			];
	}
}


#undef LOCTEXT_NAMESPACE
