// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "RigVMEditorAsset.h"
#include "Widgets/SCompoundWidget.h"


class SRigVMPythonLogDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMPythonLogDetails) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FRigVMEditorAssetInterfacePtr InAsset);
protected:

	FReply OnCopyPythonScriptClicked() const;
	FReply OnRunPythonContextClicked() const;

	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized;
};

class FRigVMPythonLogDetails : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMPythonLogDetails);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	UObject* BlueprintBeingCustomized = nullptr;
};
