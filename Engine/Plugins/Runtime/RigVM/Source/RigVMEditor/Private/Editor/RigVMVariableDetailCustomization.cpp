// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMVariableDetailCustomization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMHost.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDocumentation.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "RigVMBlueprintLegacy.h"
#include "Editor/RigVMEditor.h"
#if WITH_RIGVMLEGACYEDITOR
#include "Editor/RigVMLegacyEditor.h"
#endif
#include "InstancedPropertyBagStructureDataProvider.h"
#include "RigVMModel/RigVMController.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "SPinTypeSelector.h"
#include "SupportedRangeTypes.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SToolTip.h"
#include "RigVMDeveloperTypeUtils.h"


#define LOCTEXT_NAMESPACE "RigVMVariableDetailCustomization"

TSharedRef<IDetailCustomization> FRigVMVariableDetailCustomization::MakeInstance(TSharedPtr<IRigVMEditor> InEditor)
{
	const TArray<UObject*>* Objects = (InEditor.IsValid() ? InEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		if (FRigVMEditorAssetInterfacePtr Blueprint = (*Objects)[0])
		{
			return MakeShareable(new FRigVMVariableDetailCustomization(InEditor, Blueprint));
		}
		else if ((*Objects)[0]->Implements<URigVMRuntimeAssetInterface>())
		{
			TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = (*Objects)[0];
			return MakeShareable(new FRigVMVariableDetailCustomization(InEditor, RuntimeAsset->GetEditorOnlyData()));
		}
	}

	return MakeShareable(new FRigVMVariableDetailCustomization((TSharedPtr<IRigVMEditor>)nullptr, FRigVMEditorAssetInterfacePtr(nullptr)));
}

FRigVMVariableDetailCustomization::FRigVMVariableDetailCustomization(TSharedPtr<IRigVMEditor> InEditor, FRigVMEditorAssetInterfacePtr Blueprint)
		: EditorPtr(InEditor)
		, BlueprintPtr(Blueprint)
{}

#if WITH_RIGVMLEGACYEDITOR
TSharedPtr<IDetailCustomization> FRigVMVariableDetailCustomization::MakeLegacyInstance(TSharedPtr<IBlueprintEditor> InEditor)
{
	const TArray<UObject*>* Objects = (InEditor.IsValid() ? InEditor->GetObjectsCurrentlyBeingEdited() : nullptr);
	if (Objects && Objects->Num() == 1)
	{
		if (URigVMBlueprint* Blueprint = Cast<URigVMBlueprint>((*Objects)[0]))
		{
			if (Blueprint->ParentClass && Blueprint->ParentClass->IsChildOf(URigVMHost::StaticClass()))
			{
				return MakeShareable(new FRigVMVariableDetailCustomization(InEditor, Blueprint));
			}
		}
	}

	return MakeShareable(new FRigVMVariableDetailCustomization((TSharedPtr<IBlueprintEditor>)nullptr, nullptr));
}

FRigVMVariableDetailCustomization::FRigVMVariableDetailCustomization(TSharedPtr<IBlueprintEditor> RigVMigEditor, UBlueprint* Blueprint)
{
	EditorPtr = StaticCastSharedPtr<FRigVMLegacyEditor>(RigVMigEditor);
	BlueprintPtr = Cast<URigVMBlueprint>(Blueprint);
}
#endif

void FRigVMVariableDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor.IsValid())
	{
		return;
	}

	if (Editor->GetRigVMInspector())
	{
		FName VariableName = GetVariableName();
		CachedVariableProperty = BlueprintPtr->FindGeneratedPropertyByName(VariableName);

		if(!CachedVariableProperty.IsValid())
		{
			return;
		}

		Editor->OnRefresh().AddSP(this, &FRigVMVariableDetailCustomization::OnPostEditorRefresh);

		// Get an appropriate name validator
		TSharedPtr<INameValidatorInterface> NameValidator = nullptr;
		{
			const UEdGraphSchema* Schema = nullptr;
			if (BlueprintPtr.IsValid())
			{
				TArray<UEdGraph*> Graphs;
				BlueprintPtr->GetAllEdGraphs(Graphs);
				if (Graphs.Num() > 0)
				{
					Schema = Graphs[0]->GetSchema();
				}
			}

			if (const URigVMEdGraphSchema* RigVMSchema = Cast<URigVMEdGraphSchema>(Schema))
			{
				FRigVMEditorAssetInterfacePtr AssetPtr = BlueprintPtr.GetObject();
				NameValidator = RigVMSchema->GetNameValidator(AssetPtr, CachedVariableName, nullptr);
			}
		}

		const FProperty* VariableProperty = CachedVariableProperty.Get();

		IDetailCategoryBuilder& Category = DetailLayout.EditCategory("Variable", LOCTEXT("VariableDetailsCategory", "Variable"));
		const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();
		
		TSharedPtr<SToolTip> VarNameTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarNameTooltip", "The name of the variable."), NULL, "", TEXT("VariableName"));

		Category.AddCustomRow( LOCTEXT("BlueprintVarActionDetails_VariableNameLabel", "Variable Name") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlueprintVarActionDetails_VariableNameLabel", "Variable Name"))
			.ToolTip(VarNameTooltip)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MaxDesiredWidth(250.0f)
		[
			SAssignNew(VarNameEditableTextBox, SEditableTextBox)
			.Text(this, &FRigVMVariableDetailCustomization::OnGetVariableName)
			.ToolTip(VarNameTooltip)
			.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnVarNameCommitted)
			.OnVerifyTextChanged_Lambda([this, NameValidator](const FText& InNewText, FText& OutErrorMessage) -> bool
			{	
				if (NameValidator.IsValid())
				{
					EValidatorResult ValidatorResult = NameValidator->IsValid(InNewText.ToString());
					switch (ValidatorResult)
					{
						case EValidatorResult::Ok:
						case EValidatorResult::ExistingName:
							// These are fine, don't need to surface to the user, the rename can 'proceed' even if the name is the existing one
							return true;
						default:
							OutErrorMessage = INameValidatorInterface::GetErrorText(InNewText.ToString(), ValidatorResult);
							return false;
					}
				}
				
				return true;
			})
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	
		TSharedPtr<SToolTip> VarTypeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarTypeTooltip", "The type of the variable."), NULL, "", TEXT("VariableType"));

		TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
		Editor->GetPinTypeSelectorFilters(CustomPinTypeFilters);
		
		const UEdGraphSchema* Schema = GetDefault<UEdGraphSchema_K2>();
		if (Editor->GetFocusedGraph())
		{
			Schema = Editor->GetFocusedGraph()->GetSchema();
		}
		

		Category.AddCustomRow(LOCTEXT("VariableTypeLabel", "Variable Type"))
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("VariableTypeLabel", "Variable Type"))
				.ToolTip(VarTypeTooltip)
				.Font(DetailFontInfo)
			]
			.ValueContent()
			.MaxDesiredWidth(980.f)
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
				.TargetPinType(this, &FRigVMVariableDetailCustomization::OnGetVarType)
				.OnPinTypeChanged(this, &FRigVMVariableDetailCustomization::OnVarTypeChanged)
				.Schema(Schema)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.Font(DetailFontInfo)
				.ToolTip(VarTypeTooltip)
				.CustomFilters(CustomPinTypeFilters)
			]
			.AddCustomContextMenuAction(FUIAction(
				FExecuteAction::CreateRaw(this, &FRigVMVariableDetailCustomization::OnBrowseToVarType),
				FCanExecuteAction::CreateRaw(this, &FRigVMVariableDetailCustomization::CanBrowseToVarType)
				),
				LOCTEXT("BrowseToType", "Browse to Type"),
				LOCTEXT("BrowseToTypeToolTip", "Browse to this variable type in the Content Browser."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent")
			);



		TSharedPtr<SToolTip> ToolTipTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VarToolTipTooltip", "Extra information about this variable, shown when cursor is over it."), NULL, "", TEXT("Description"));

		Category.AddCustomRow( LOCTEXT("IsVariableToolTipLabel", "Description") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text( LOCTEXT("IsVariableToolTipLabel", "Description") )
			.ToolTip(ToolTipTooltip)
			.Font( DetailFontInfo )
		]
		.ValueContent()
		.MinDesiredWidth(250.f)
		.MaxDesiredWidth(250.f)
		[
			SNew(SMultiLineEditableTextBox)
			.Text( this, &FRigVMVariableDetailCustomization::OnGetTooltipText )
			.ToolTipText( this, &FRigVMVariableDetailCustomization::OnGetTooltipText )
			.OnTextCommitted( this, &FRigVMVariableDetailCustomization::OnTooltipTextCommitted, CachedVariableName )
			.Font( DetailFontInfo )
			.ModiferKeyForNewLine(EModifierKey::Shift)
		];


		TSharedPtr<SToolTip> ExposeOnSpawnTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableExposeToSpawn_Tooltip", "Should this variable be exposed as a pin when spawning this Blueprint?"), NULL, "", TEXT("ExposeOnSpawn"));

		Category.AddCustomRow( LOCTEXT("VariableExposeToSpawnLabel", "Expose on Spawn") )
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(ExposeOnSpawnTooltip)
			.Text( LOCTEXT("VariableExposeToSpawnLabel", "Expose on Spawn") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked( this, &FRigVMVariableDetailCustomization::OnGetExposedToSpawnCheckboxState )
			.OnCheckStateChanged( this, &FRigVMVariableDetailCustomization::OnExposedToSpawnChanged )
			.ToolTip(ExposeOnSpawnTooltip)
		];



		TSharedPtr<SToolTip> PublicTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariablePublic_Tooltip", "Should this variable be public?"), NULL, "", TEXT("Public"));

		Category.AddCustomRow(LOCTEXT("VariablePublic", "Public"))
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(PublicTooltip)
			.Text( LOCTEXT("VariablePublic", "Public") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked( this, &FRigVMVariableDetailCustomization::OnGetPublicCheckboxState )
			.OnCheckStateChanged( this, &FRigVMVariableDetailCustomization::OnPublicChanged )
			.ToolTip(PublicTooltip)
		];

		TSharedPtr<SToolTip> ExposeToCinematicsTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("VariableExposeToCinematics_Tooltip", "Should this variable be exposed for Sequencer to modify?"), NULL, "", TEXT("ExposeToCinematics"));

		Category.AddCustomRow( LOCTEXT("VariableExposeToCinematics", "Expose to Cinematics") )
		.NameContent()
		[
			SNew(STextBlock)
			.ToolTip(ExposeToCinematicsTooltip)
			.Text( LOCTEXT("VariableExposeToCinematics", "Expose to Cinematics") )
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked( this, &FRigVMVariableDetailCustomization::OnGetExposedToCinematicsCheckboxState )
			.OnCheckStateChanged( this, &FRigVMVariableDetailCustomization::OnExposedToCinematicsChanged )
			.ToolTip(ExposeToCinematicsTooltip)
		];

		PopulateCategories();
		TSharedPtr<SComboButton> NewComboButton;
		TSharedPtr<SListView<TSharedPtr<FText>>> NewListView;

		TSharedPtr<SToolTip> CategoryTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("EditCategoryName_Tooltip", "The category of the variable; editing this will place the variable into another category or create a new one."), NULL, "", TEXT("Category"));

		Category.AddCustomRow( LOCTEXT("CategoryLabel", "Category") )
		.NameContent()
		[
			SNew(STextBlock)
			.Text( LOCTEXT("CategoryLabel", "Category") )
			.ToolTip(CategoryTooltip)
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SAssignNew(NewComboButton, SComboButton)
			.ContentPadding(FMargin(0.0f, 0.0f, 5.0f, 0.0f))
			.ToolTip(CategoryTooltip)
			.ButtonContent()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder") )
				.Padding(FMargin(0, 0, 5, 0))
				[
					SNew(SEditableTextBox)
						.Text(this, &FRigVMVariableDetailCustomization::OnGetCategoryText)
						.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnCategoryTextCommitted, CachedVariableName )
						.OnVerifyTextChanged_Lambda([&](const FText& InNewText, FText& OutErrorMessage) -> bool
						{
							if (InNewText.IsEmpty())
							{
								OutErrorMessage = LOCTEXT("CategoryEmpty", "Cannot add a category with an empty string.");
								return false;
							}
							if (InNewText.EqualTo(FText::FromString(BlueprintPtr->GetObject()->GetName())))
							{
								OutErrorMessage = LOCTEXT("CategoryEqualsBlueprintName", "Cannot add a category with the same name as the blueprint.");
								return false;
							}
							return true;
						})
						.ToolTip(CategoryTooltip)
						.SelectAllTextWhenFocused(true)
						.RevertTextOnEscape(true)
						.Font( DetailFontInfo )
				]
			]
			.MenuContent()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(400.0f)
				[
					SAssignNew(NewListView, SListView<TSharedPtr<FText>>)
						.ListItemsSource(&CategorySource)
						.OnGenerateRow(this, &FRigVMVariableDetailCustomization::MakeCategoryViewWidget)
						.OnSelectionChanged(this, &FRigVMVariableDetailCustomization::OnCategorySelectionChanged)
				]
			]
		];

		CategoryComboButton = NewComboButton;
		CategoryListView = NewListView;


		TSharedPtr<SToolTip> SliderRangeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("SliderRange_Tooltip", "Allows setting the minimum and maximum values for the UI slider for this variable."), NULL, "", TEXT("SliderRange"));

		FName UIMin = TEXT("UIMin");
		FName UIMax = TEXT("UIMax");
		Category.AddCustomRow( LOCTEXT("SliderRangeLabel", "Slider Range") )
		.Visibility(TAttribute<EVisibility>(this, &FRigVMVariableDetailCustomization::RangeVisibility))
		.NameContent()
		[
			SNew(STextBlock)
			.Text( LOCTEXT("SliderRangeLabel", "Slider Range") )
			.ToolTip(SliderRangeTooltip)
			.Font( DetailFontInfo )
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.ToolTip(SliderRangeTooltip)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.Text(this, &FRigVMVariableDetailCustomization::OnGetMetaKeyValue, UIMin)
				.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnMetaKeyValueChanged, UIMin)
				.Font( DetailFontInfo )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( STextBlock )
				.Text( LOCTEXT("Min .. Max Separator", " .. ") )
				.Font(DetailFontInfo)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.Text(this, &FRigVMVariableDetailCustomization::OnGetMetaKeyValue, UIMax)
				.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnMetaKeyValueChanged, UIMax)
				.Font( DetailFontInfo )
			]
		];

		TSharedPtr<SToolTip> ValueRangeTooltip = IDocumentation::Get()->CreateToolTip(LOCTEXT("ValueRangeLabel_Tooltip", "The range of values allowed by this variable. Values outside of this will be clamped to the range."), NULL, "", TEXT("ValueRange"));

		FName ClampMin = TEXT("ClampMin");
		FName ClampMax = TEXT("ClampMax");
		Category.AddCustomRow(LOCTEXT("ValueRangeLabel", "Value Range"))
		.Visibility(TAttribute<EVisibility>(this, &FRigVMVariableDetailCustomization::RangeVisibility))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ValueRangeLabel", "Value Range"))
			.ToolTipText(LOCTEXT("ValueRangeLabel_Tooltip", "The range of values allowed by this variable. Values outside of this will be clamped to the range."))
			.ToolTip(ValueRangeTooltip)
			.Font(DetailFontInfo)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.Text(this, &FRigVMVariableDetailCustomization::OnGetMetaKeyValue, ClampMin)
				.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnMetaKeyValueChanged, ClampMin)
				.Font(DetailFontInfo)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Min .. Max Separator", " .. "))
				.Font(DetailFontInfo)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.Text(this, &FRigVMVariableDetailCustomization::OnGetMetaKeyValue, ClampMax)
				.OnTextCommitted(this, &FRigVMVariableDetailCustomization::OnMetaKeyValueChanged, ClampMax)
				.Font(DetailFontInfo)
			]
		];

		if (TStrongObjectPtr<UObject> StrongObj = BlueprintPtr.GetWeakObjectPtr().Pin())
		{
			FRigVMEditorAssetInterfacePtr Asset = StrongObj.Get();
			
			// Add in default value editing for properties that can be edited, local properties cannot be edited
			if (Asset->IsBlueprintAsset() && Asset->GetRuntimeAssetInterface() != nullptr)
			{
				URigVMBlueprint* Blueprint = Cast<URigVMBlueprint>(Asset.GetObject());
				bool bVariableRenamed = false;
				if (VariableProperty != nullptr)
				{
					const int32 VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(Blueprint, CachedVariableName);
					if (VarIndex != INDEX_NONE)
					{
						const FGuid VarGuid = Blueprint->NewVariables[VarIndex].VarGuid;
						if (UBlueprintGeneratedClass* AuthoritiveBPGC = Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass))
						{
							if (const FName* OldName = AuthoritiveBPGC->PropertyGuids.FindKey(VarGuid))
							{
								bVariableRenamed = CachedVariableName != *OldName;
							}
						}
					}
				
					const FProperty* OriginalProperty = nullptr;
					OriginalProperty = FindFProperty<FProperty>(Blueprint->GeneratedClass, VariableProperty->GetFName());

					if (OriginalProperty == nullptr || bVariableRenamed)
					{
						// Prevent editing the default value of a skeleton property
						VariableProperty = nullptr;
					}
					else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(OriginalProperty))
					{
						// Prevent editing the default value of a stale struct
						const UUserDefinedStruct* BGStruct = Cast<const UUserDefinedStruct>(StructProperty->Struct);
						if (BGStruct && (EUserDefinedStructureStatus::UDSS_UpToDate != BGStruct->Status))
						{
							VariableProperty = nullptr;
						}
					}
				}
			}
			else 
			{
				VariableProperty = Asset->FindGeneratedPropertyByName(CachedVariableName);
			}

			// Find the class containing the variable
			UClass* VariableClass = (VariableProperty ? VariableProperty->GetTypedOwner<UClass>() : nullptr);

			FText ErrorMessage;
			IDetailCategoryBuilder& DefaultValueCategory = DetailLayout.EditCategory(TEXT("DefaultValueCategory"), LOCTEXT("DefaultValueCategoryHeading", "Default Value"));

			if (VariableProperty == nullptr)
			{
				if (Asset->GetAssetStatus() != RVMA_UpToDate)
				{
					ErrorMessage = LOCTEXT("VariableMissing_DirtyBlueprint", "Please compile the blueprint");
				}
				else
				{
					ErrorMessage = LOCTEXT("VariableMissing_CleanBlueprint", "Failed to find variable property");
				}
			}
			// Show the error message if something went wrong
			if (!ErrorMessage.IsEmpty())
			{
				DefaultValueCategory.AddCustomRow( ErrorMessage )
				[
					SNew(STextBlock)
					.ToolTipText(ErrorMessage)
					.Text(ErrorMessage)
					.Font(DetailFontInfo)
				];
			}
			else 
			{
				TSharedPtr<IDetailsView> DetailsView;
				if (Editor.IsValid())
				{
					DetailsView = Editor->GetRigVMInspector()->GetPropertyView();
				}

				if (Asset->IsBlueprintAsset())
				{
					// Things are in order, show the property and allow it to be edited
					TArray<UObject*> ObjectList;
					if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeInterface = Asset->GetRuntimeAssetInterface())
					{
						if (UClass* GeneratedClass = Cast<UClass>(RuntimeInterface.GetObject()))
						{
							ObjectList.Add(GeneratedClass->GetDefaultObject());
						}
					}
					IDetailPropertyRow* Row = DefaultValueCategory.AddExternalObjectProperty(ObjectList, VariableProperty->GetFName());

				}
				else
				{
					// Things are in order, show the property and allow it to be edited
					if (FRigVMPropertyBag* InstancedPropertyBag = Asset->GetVariablesPropertyBag())
					{
						FStructView StructView = InstancedPropertyBag->GetMutableValue();
						if (StructView.IsValid())
						{
							TSharedRef<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(
								StructView.GetScriptStruct(),
								StructView.GetMemory()
								);
							FAddPropertyParams Params;
							Params.HideRootObjectNode(true);
							Params.ForceShowProperty();
							// Don't rely on the CachedVariableName because it is not sanitized (it allows spaces)
							DefaultValueCategory.AddExternalStructureProperty(StructOnScope, VariableProperty->GetFName(), EPropertyLocation::Default, Params);
						}
					}
				}

				if (DetailsView.IsValid())
				{
					DetailsView->OnFinishedChangingProperties().AddSP(this, &FRigVMVariableDetailCustomization::OnFinishedChangingVariable);
				}
			}
		}
	}
}

void FRigVMVariableDetailCustomization::PopulateCategories()
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	
	if (!Blueprint || !Editor)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	auto IsNewCategorySource = [&](const FText& NewCategory)
	{
		return !CategorySource.ContainsByPredicate([&NewCategory](const TSharedPtr<FText>& ExistingCategory)
		{
			return ExistingCategory->ToString().Equals(NewCategory.ToString(), ESearchCase::CaseSensitive);
		});
	};

	CategorySource.Reset();
	CategorySource.Add(MakeShared<FText>(UEdGraphSchema_K2::VR_DefaultCategory));


	TArray<FRigVMGraphVariableDescription> Variables = Interface->GetAssetVariables();
	
	for (const FRigVMGraphVariableDescription& Variable : Variables)
	{
		FText Category = Variable.Category;
		if (!Category.IsEmpty() && !Category.EqualTo(FText::FromString(Interface->GetObject()->GetName())))
		{
			if (IsNewCategorySource(Category))
			{
				CategorySource.Add(MakeShared<FText>(Category));
			}
		}
	}

	// Sort categories, but keep the default category listed first
	CategorySource.Sort([](const TSharedPtr <FText> &LHS, const TSharedPtr <FText> &RHS)
	{
		if (LHS.IsValid() && RHS.IsValid())
		{
			return (LHS->EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) || LHS->CompareToCaseIgnored(*RHS) <= 0);
		}
		return false;
	});
}

void FRigVMVariableDetailCustomization::OnFinishedChangingVariable(const FPropertyChangedEvent& PropertyChangedEvent)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}

	if (FRigVMEditorAssetInterfacePtr Asset = Blueprint.Get())
	{
		Asset->MarkAssetAsModified();
		Asset->SetAssetStatus(ERigVMAssetStatus::RVMA_Dirty);

		// If the changed property is the root property, autocompile
		// Otherwise wait until the user forces a compile, we do this in order to allow multi-parameter editing of structs like FVector (using tab)
		if (PropertyChangedEvent.Property == PropertyChangedEvent.MemberProperty)
		{
			Asset->RecompileVM();
		}
	}
	
}

void FRigVMVariableDetailCustomization::OnPostEditorRefresh()
{
	FName VariableName = GetVariableName();
	CachedVariableProperty = BlueprintPtr->FindGeneratedPropertyByName(VariableName);
}

FName FRigVMVariableDetailCustomization::GetVariableName() const
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
    if (!Editor.IsValid())
    {
    	return NAME_None;
    }
    
    CachedVariableName = Editor->GetGraphExplorerWidget()->GetSelectedVariableName();
	return CachedVariableName;
}

FText FRigVMVariableDetailCustomization::OnGetVariableName() const
{
	return FText::FromName(CachedVariableName);
}

void FRigVMVariableDetailCustomization::OnVarNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}

	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return;
	}
	
	ERigVMExplorerElementType::Type SelectedType = Explorer->GetSelectedType();
	if (SelectedType == ERigVMExplorerElementType::Variable)
	{
		FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
		Blueprint->Modify();
		Interface->RenameMemberVariable(CachedVariableName, *InNewName.ToString());
	}
}

FEdGraphPinType FRigVMVariableDetailCustomization::OnGetVarType() const
{
	FEdGraphPinType Type;
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return Type;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return Type;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();

	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return Type;
	}

	ERigVMExplorerElementType::Type SelectionVarType = Explorer->GetSelectedType();
	if (SelectionVarType == ERigVMExplorerElementType::Variable)
	{
		for (FRigVMGraphVariableDescription& Var : Interface->GetAssetVariables())
		{
			if (Var.Name == CachedVariableName)
			{
				return RigVMTypeUtils::PinTypeFromRigVMVariableDescription(Var);
			}
		}
		return Type;
	}
	return Type;
}

void FRigVMVariableDetailCustomization::OnVarTypeChanged(const FEdGraphPinType& NewPinType)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();

	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return;
	}
	
	ERigVMExplorerElementType::Type SelectedType = Explorer->GetSelectedType();
	if (SelectedType == ERigVMExplorerElementType::Variable)
	{
		for (FRigVMGraphVariableDescription& Variable : Interface->GetAssetVariables())
		{
			if (Variable.Name == CachedVariableName)
			{
				Interface->ChangeMemberVariableType(CachedVariableName, NewPinType);
				Explorer->SetLastPinTypeUsed(NewPinType);
				return;
			}
		}
	}
}

void FRigVMVariableDetailCustomization::OnBrowseToVarType() const
{
	FEdGraphPinType PinType = OnGetVarType();
	if (const UObject* Object = PinType.PinSubCategoryObject.Get())
	{
		if (Object->IsAsset())
		{
			FAssetData AssetData(Object, false);
			if (AssetData.IsValid())
			{
				TArray<FAssetData> AssetDataList = { AssetData };
				GEditor->SyncBrowserToObjects(AssetDataList);
			}
		}
	}
}

bool FRigVMVariableDetailCustomization::CanBrowseToVarType() const
{
	FEdGraphPinType PinType = OnGetVarType();
	if (const UObject* Object = PinType.PinSubCategoryObject.Get())
	{
		if (Object->IsAsset())
		{
			FAssetData AssetData(Object, false);
			if (AssetData.IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

FText FRigVMVariableDetailCustomization::OnGetTooltipText() const
{
	FText ToolTip;
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return ToolTip;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return ToolTip;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();

	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return ToolTip;
	}

	ERigVMExplorerElementType::Type SelectedType = Explorer->GetSelectedType();

	if (SelectedType == ERigVMExplorerElementType::Variable)
	{
		FString Result;
		for (FRigVMGraphVariableDescription& Var : Interface->GetAssetVariables())
		{
			if (Var.Name == CachedVariableName)
			{
				return Var.Tooltip;
			}
		}
	}

	return ToolTip;
}

void FRigVMVariableDetailCustomization::OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return;
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();

	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return;
	}

	ERigVMExplorerElementType::Type SelectedType = Explorer->GetSelectedType();

	if (SelectedType == ERigVMExplorerElementType::Variable)
	{
		Interface->SetVariableTooltip(CachedVariableName, NewText);
	}
}

EVisibility FRigVMVariableDetailCustomization::IsToolTipVisible() const
{
	EVisibility Visible = EVisibility::Collapsed;
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return Visible;
	}
	TSharedPtr<SRigVMEditorGraphExplorer> Explorer = Editor->GetGraphExplorerWidget();
	if (!Explorer)
	{
		return Visible;
	}
	ERigVMExplorerElementType::Type SelectedType = Explorer->GetSelectedType();
	if (SelectedType == ERigVMExplorerElementType::Variable)
	{
		Visible = EVisibility::Visible;
	}
	return Visible;
}

FText FRigVMVariableDetailCustomization::OnGetCategoryText() const
{
	TSharedPtr<IRigVMEditor> Editor = EditorPtr.Pin();
	if (!Editor)
	{
		return FText();
	}

	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return FText();
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();

	if (CachedVariableName != NAME_None)
	{
		FText Category;
		for (FRigVMGraphVariableDescription& Variable : Interface->GetAssetVariables())
		{
			if (Variable.Name == CachedVariableName)
			{
				Category = Variable.Category;
				break;
			}
		}

		// Older blueprints will have their name as the default category and whenever it is the same as the default category, display localized text
		if ( Category.EqualTo(FText::FromString(Blueprint->GetName())) || Category.EqualTo(UEdGraphSchema_K2::VR_DefaultCategory) )
		{
			return UEdGraphSchema_K2::VR_DefaultCategory;
		}
		else
		{
			return Category;
		}
	}
	return FText();
}

void FRigVMVariableDetailCustomization::OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		Interface->SetVariableCategory(CachedVariableName, NewText.ToString());
		PopulateCategories();
	}
}

TSharedRef<ITableRow> FRigVMVariableDetailCustomization::MakeCategoryViewWidget(TSharedPtr<FText> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(STextBlock) .Text(*Item.Get())
		];
}

void FRigVMVariableDetailCustomization::OnCategorySelectionChanged(TSharedPtr<FText> ProposedSelection, ESelectInfo::Type)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	if (ProposedSelection.IsValid() && CachedVariableName != NAME_None)
	{
		FText NewCategory = *ProposedSelection.Get();
		Interface->SetVariableCategory(CachedVariableName, NewCategory.ToString());
		CategoryListView.Pin()->ClearSelection();
		CategoryComboButton.Pin()->SetIsOpen(false);
	}
}

ECheckBoxState FRigVMVariableDetailCustomization::OnGetExposedToSpawnCheckboxState() const
{
	const FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && Property->GetBoolMetaData(FBlueprintMetadata::MD_ExposeOnSpawn) != false) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMVariableDetailCustomization::OnExposedToSpawnChanged(ECheckBoxState InNewState)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	if (CachedVariableName != NAME_None)
	{
		const bool bExposeOnSpawn = (InNewState == ECheckBoxState::Checked);
		Interface->SetVariableExposeOnSpawn(CachedVariableName, bExposeOnSpawn);
	}
}

ECheckBoxState FRigVMVariableDetailCustomization::OnGetPublicCheckboxState() const
{
	const FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return (Property && (Property->PropertyFlags & CPF_DisableEditOnInstance) == 0) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMVariableDetailCustomization::OnPublicChanged(ECheckBoxState InNewState)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	if (CachedVariableName != NAME_None)
	{
		const bool bPublic = (InNewState == ECheckBoxState::Checked);
		Interface->SetVariablePublic(CachedVariableName, bPublic);
	}
}

ECheckBoxState FRigVMVariableDetailCustomization::OnGetExposedToCinematicsCheckboxState() const
{
	const FProperty* Property = CachedVariableProperty.Get();
	if (Property)
	{
		return Property && Property->HasAnyPropertyFlags(CPF_Interp) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMVariableDetailCustomization::OnExposedToCinematicsChanged(ECheckBoxState InNewState)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	// Toggle the flag on the blueprint's version of the variable description, based on state
	const bool bExposeToCinematics = (InNewState == ECheckBoxState::Checked);
	
	const FName VarName = CachedVariableName;
	if (VarName != NAME_None)
	{
		Interface->SetVariableExposeToCinematics(VarName, bExposeToCinematics);
	}
}

FText FRigVMVariableDetailCustomization::OnGetMetaKeyValue(FName Key) const
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return FText();
	}
	FRigVMEditorAssetInterfacePtr Interface = Blueprint.Get();
	
	if (CachedVariableName != NAME_None)
	{
		FString Result = Interface->GetVariableMetadataValue(CachedVariableName, Key);
		
		return FText::FromString(Result);
	}
	return FText();
}

void FRigVMVariableDetailCustomization::OnMetaKeyValueChanged(const FText& NewMinValue, ETextCommit::Type CommitInfo, FName Key)
{
	TStrongObjectPtr<UObject> Blueprint = BlueprintPtr.GetWeakObjectPtr().Pin();
	if (!Blueprint)
	{
		return;
	}

	checkf(false, TEXT("Not implemented yet"));
	// if (CachedVariableName != NAME_None)
	// {
	// 	if ((CommitInfo == ETextCommit::OnEnter) || (CommitInfo == ETextCommit::OnUserMovedFocus))
	// 	{
	// 		FBlueprintEditorUtils::SetBlueprintVariableMetaData(Blueprint.Get(), CachedVariableName, nullptr, Key, NewMinValue.ToString());
	// 	}
	// }
}

EVisibility FRigVMVariableDetailCustomization::RangeVisibility() const
{
	const FProperty* VariableProperty = CachedVariableProperty.Get();
	if (VariableProperty)
	{
		const bool bIsInteger = VariableProperty->IsA(FIntProperty::StaticClass());
		const bool bIsNonEnumByte = (VariableProperty->IsA(FByteProperty::StaticClass()) && CastField<const FByteProperty>(VariableProperty)->Enum == nullptr);
		const bool bIsReal = (VariableProperty->IsA(FFloatProperty::StaticClass()) || VariableProperty->IsA(FDoubleProperty::StaticClass()));

		// If this is a struct property than we must check the name of the struct it points to, so we can check
		// if it supports the editing of the UIMin/UIMax metadata
		const FStructProperty* StructProp = CastField<FStructProperty>(VariableProperty);
		const UStruct* InnerStruct = StructProp ? StructProp->Struct.Get() : nullptr;
		const bool bIsSupportedStruct = InnerStruct ? RangeVisibilityUtils::StructsSupportingRangeVisibility.Contains(InnerStruct->GetFName()) : false;

		if (bIsInteger || bIsNonEnumByte || bIsReal || bIsSupportedStruct)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
