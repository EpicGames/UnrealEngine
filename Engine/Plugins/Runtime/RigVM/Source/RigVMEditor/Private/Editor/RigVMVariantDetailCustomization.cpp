// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/RigVMVariantDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Editor/RigVMEditorTools.h"
#include "Misc/UObjectToken.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SRigVMVariantWidget.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ScopedTransaction.h"
#include "RigVMSettings.h"

#define LOCTEXT_NAMESPACE "RigVMVariantDetailCustomization"

class FUObjectToken;

void FRigVMVariantDetailCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if(!CVarRigVMEnableVariants.GetValueOnAnyThread())
	{
		return;
	}
		
	TArray<UObject*> Objects;
	InStructPropertyHandle->GetOuterObjects(Objects);

	for (UObject* Object : Objects)
	{
		if (Object->Implements<URigVMEditorAssetInterface>())
		{
			BlueprintBeingCustomized = Object;
			break;
		}
		else if (Object->Implements<URigVMRuntimeAssetInterface>())
		{
			TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeInterface = Object;
			BlueprintBeingCustomized = RuntimeInterface->GetEditorOnlyData();
			break;
		}
	}

	FRigVMVariantWidgetContext VariantContext;
	if(BlueprintBeingCustomized)
	{
		VariantContext.ParentPath = BlueprintBeingCustomized->GetObject()->GetPathName();
	}

	HeaderRow
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SRigVMVariantWidget)
		.Context(VariantContext)
		.Variant(this, &FRigVMVariantDetailCustomization::GetVariant)
		.SubjectVariantRef(this, &FRigVMVariantDetailCustomization::GetSubjectVariantRef)
		.VariantRefs(this, &FRigVMVariantDetailCustomization::GetVariantRefs)
		.OnVariantChanged(this, &FRigVMVariantDetailCustomization::OnVariantChanged)
		.OnBrowseVariantRef(this, &FRigVMVariantDetailCustomization::OnBrowseVariantRef)
		.OnGetTags(this, &FRigVMVariantDetailCustomization::OnGetTags)
		.OnAddTag(this, &FRigVMVariantDetailCustomization::OnAddTag)
		.OnRemoveTag(this, &FRigVMVariantDetailCustomization::OnRemoveTag)
		.CanAddTags(true)
		.EnableTagContextMenu(true)
	];
}

void FRigVMVariantDetailCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// nothing to do here
}

FRigVMVariant FRigVMVariantDetailCustomization::GetVariant() const
{
	if (BlueprintBeingCustomized)
	{
		return BlueprintBeingCustomized->GetAssetVariant();
	}
	return FRigVMVariant();
}

FRigVMVariantRef FRigVMVariantDetailCustomization::GetSubjectVariantRef() const
{
	if (BlueprintBeingCustomized)
	{
		return BlueprintBeingCustomized->GetAssetVariantRef();
	}
	return FRigVMVariantRef();
}

TArray<FRigVMVariantRef> FRigVMVariantDetailCustomization::GetVariantRefs() const
{
	if (BlueprintBeingCustomized)
	{
		const FRigVMVariant& Variant = BlueprintBeingCustomized->GetAssetVariant();
		TArray<FRigVMVariantRef> Variants = URigVMBuildData::Get()->FindAssetVariantRefs(Variant.Guid);
		const FRigVMVariantRef MyVariantRef = BlueprintBeingCustomized->GetAssetVariantRef();
		Variants.RemoveAll([MyVariantRef](const FRigVMVariantRef& VariantRef) -> bool
		{
			return VariantRef == MyVariantRef;
		});
		return Variants;
	}
	return TArray<FRigVMVariantRef>();
}

void FRigVMVariantDetailCustomization::OnVariantChanged(const FRigVMVariant& InNewVariant)
{
	if(BlueprintBeingCustomized)
	{
		FScopedTransaction Transaction(LOCTEXT("ChangedVariantInfo", "Changed Blueprint Variant Information"));
		BlueprintBeingCustomized->GetObject()->Modify();
		BlueprintBeingCustomized->GetAssetVariant() = InNewVariant;
	}
}

void FRigVMVariantDetailCustomization::OnBrowseVariantRef(const FRigVMVariantRef& InVariantRef)
{
	const FAssetData AssetData = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(InVariantRef.ObjectPath.ToString(), true);
	if(AssetData.IsValid())
	{
		const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		ContentBrowserModule.Get().SyncBrowserToAssets({AssetData});
	}
}

TArray<FRigVMTag> FRigVMVariantDetailCustomization::OnGetTags() const
{
	if(BlueprintBeingCustomized)
	{
		return BlueprintBeingCustomized->GetAssetVariant().Tags;
	}
	return {};
}

void FRigVMVariantDetailCustomization::OnAddTag(const FName& InTagName)
{
	if(BlueprintBeingCustomized)
	{
		const URigVMProjectSettings* Settings = GetMutableDefault<URigVMProjectSettings>(URigVMProjectSettings::StaticClass());
		if (Settings)
		{
			if (const FRigVMTag* Tag = Settings->FindTag(InTagName))
			{
				if(!BlueprintBeingCustomized->GetAssetVariant().Tags.ContainsByPredicate([InTagName](const FRigVMTag& Tag) -> bool
				{
					return Tag.Name == InTagName;
				}))
				{
					FScopedTransaction Transaction(LOCTEXT("AddedBlueprintVariantTag", "Added Blueprint Variant Tag"));
					BlueprintBeingCustomized->GetObject()->Modify();
					BlueprintBeingCustomized->GetAssetVariant().Tags.Add(*Tag);
				}
			}
		}
	}
}

void FRigVMVariantDetailCustomization::OnRemoveTag(const FName& InTagName)
{
	if(BlueprintBeingCustomized)
	{
		if(BlueprintBeingCustomized->GetAssetVariant().Tags.ContainsByPredicate([InTagName](const FRigVMTag& Tag) -> bool
			{
				return Tag.Name == InTagName;
			}))
		{
			FScopedTransaction Transaction(LOCTEXT("RemovedBlueprintVariantTag", "Removed Blueprint Variant Tag"));
			BlueprintBeingCustomized->GetObject()->Modify();
			BlueprintBeingCustomized->GetAssetVariant().Tags.RemoveAll([InTagName](const FRigVMTag& Tag) -> bool
			{
				return Tag.Name == InTagName;
			});
		}
	}
}

#undef LOCTEXT_NAMESPACE
