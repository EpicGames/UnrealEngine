// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor/SRigVMDetailsInspector.h"

#include "Editor/RigVMEditor.h"
#include "Engine/SCS_Node.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "PropertyEditorModule.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Editor/RigVMVariableDetailCustomization.h"
#include "Editor/RigVMCommentNodeDetailCustomization.h"
#include "Framework/Application/SlateApplication.h"
#include "K2Node.h"
#include "Editor/RigVMClassSettingsDetailCustomization.h"

class FNotifyHook;
class FStructOnScope;
class IClassViewerFilter;
class IDetailLayoutBuilder;
class SDockTab;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "RigVMDetailsInspector"

//////////////////////////////////////////////////////////////////////////
// SRigVMDetailsInspector

void SRigVMDetailsInspector::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if(bRefreshOnTick)
	{
		// if struct is valid, update struct
		if (StructToDisplay.IsValid())
		{
			UpdateFromSingleStruct(StructToDisplay);
			StructToDisplay.Reset();
		}
		else
		{
			TArray<UObject*> SelectionInfo;
			RefreshPropertyObjects.Remove(nullptr);
			UpdateFromObjects(RefreshPropertyObjects, SelectionInfo, RefreshOptions);
			RefreshPropertyObjects.Empty();
		}

		bRefreshOnTick = false;
	}
}

TSharedRef<SWidget> SRigVMDetailsInspector::MakeContextualEditingWidget(TArray<UObject*>& SelectionInfo, const FShowDetailsOptions& Options)
{
	TSharedRef< SVerticalBox > ContextualEditingWidget = SNew( SVerticalBox );

	if(bShowTitleArea)
	{
		if (SelectedObjects.Num() == 0)
		{
			// Warning about nothing being selected
			ContextualEditingWidget->AddSlot()
			.AutoHeight()
			.HAlign( HAlign_Center )
			.Padding( 2.0f, 14.0f, 2.0f, 2.0f )
			[
				SNew( STextBlock )
				.Text( LOCTEXT("NoNodesSelected", "Select a node to edit details.") )
			];
		}
		else
		{
			// Title of things being edited
			ContextualEditingWidget->AddSlot()
			.AutoHeight()
			.Padding( 2.0f, 0.0f, 2.0f, 0.0f )
			[
				SNew(STextBlock)
				.Text(this, &SRigVMDetailsInspector::GetContextualEditingWidgetTitle)
			];
		}
	}

	// Show the property editor
	PropertyView->HideFilterArea(Options.bHideFilterArea);
	PropertyView->SetObjects(SelectionInfo, Options.bForceRefresh);

	if (SelectionInfo.Num())
	{
		ContextualEditingWidget->AddSlot()
		.FillHeight( 0.9f )
		.VAlign( VAlign_Top )
		[
			SNew( SBox )
			.Visibility(this, &SRigVMDetailsInspector::GetPropertyViewVisibility)
			[
				PropertyView.ToSharedRef()
			]
		];

		ContextualEditingWidget->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("TogglePublicView", "Toggle Public View"))
			.IsChecked(this, &SRigVMDetailsInspector::GetPublicViewCheckboxState)
			.OnCheckStateChanged(this, &SRigVMDetailsInspector::SetPublicViewCheckboxState)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PublicViewCheckboxLabel", "Public View"))
			]
			.Visibility_Lambda([bShowPublicView = this->bShowPublicView]()
			{
				return bShowPublicView.Get() ? EVisibility::Visible : EVisibility::Hidden;
			})
		];
	}

	return ContextualEditingWidget;
}

TSharedPtr<SDockTab> SRigVMDetailsInspector::GetOwnerTab() const
{
	return OwnerTab.Pin();
}

void SRigVMDetailsInspector::SetOwnerTab(TSharedRef<SDockTab> Tab)
{
	OwnerTab = Tab;
}

const TArray< TWeakObjectPtr<UObject> >& SRigVMDetailsInspector::GetSelectedObjects() const
{
	return SelectedObjects;
}

void SRigVMDetailsInspector::OnEditorClose(const IRigVMEditor* RigVMEditorBase, FRigVMEditorAssetInterfacePtr RigVMBlueprint)
{
	PropertyView->UnregisterInstancedCustomPropertyLayout(URigVMEdGraph::StaticClass());
	PropertyView->UnregisterInstancedCustomPropertyLayout(UPropertyWrapper::StaticClass());
	PropertyView->UnregisterInstancedCustomPropertyLayout(URigVMRuntimeAsset::StaticClass());
	PropertyView->UnregisterInstancedCustomPropertyLayout(URigVMCommentNode::StaticClass());
}

void SRigVMDetailsInspector::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(RefreshPropertyObjects);
}

FString SRigVMDetailsInspector::GetReferencerName() const
{
	return TEXT("SRigVMDetailsInspector");
}

FText SRigVMDetailsInspector::GetContextualEditingWidgetTitle() const
{
	FText Title = PropertyViewTitle;
	if (Title.IsEmpty())
	{
		if (SelectedObjects.Num() == 1 && SelectedObjects[0].IsValid())
		{
			UObject* Object = SelectedObjects[0].Get();

			if (UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
			{
				Title = Node->GetNodeTitle(ENodeTitleType::ListView);
			}
		}
		else if (SelectedObjects.Num() > 1)
		{
			UClass* BaseClass = nullptr;

			for (auto ObjectWkPtrIt = SelectedObjects.CreateConstIterator(); ObjectWkPtrIt; ++ObjectWkPtrIt)
			{
				TWeakObjectPtr<UObject> ObjectWkPtr = *ObjectWkPtrIt;
				if (ObjectWkPtr.IsValid())
				{
					UObject* Object = ObjectWkPtr.Get();
					UClass* ObjClass = Object->GetClass();

					if (UEdGraphNode* Node = Cast<UEdGraphNode>(Object))
					{
						// Hide any specifics of node types; they're all ed graph nodes
						ObjClass = UEdGraphNode::StaticClass();
					}

					// Keep track of the class of objects selected
					if (BaseClass == nullptr)
					{
						BaseClass = ObjClass;
						checkSlow(ObjClass);
					}
					while (!ObjClass->IsChildOf(BaseClass))
					{
						BaseClass = BaseClass->GetSuperClass();
					}
				}
			}

			if (BaseClass)
			{
				Title = FText::Format(LOCTEXT("MultipleObjectsSelectedFmt", "{0} {1} selected"), FText::AsNumber(SelectedObjects.Num()), FText::FromString(BaseClass->GetName() + TEXT("s")));
			}
		}
	}
	return Title;
}

void SRigVMDetailsInspector::Construct(const FArguments& InArgs)
{
	bShowInspectorPropertyView = true;
	PublicViewState = ECheckBoxState::Unchecked;
	bRefreshOnTick = false;

	WeakEditor = InArgs._Editor;
	bShowPublicView = InArgs._ShowPublicViewControl;
	bShowTitleArea = InArgs._ShowTitleArea;
	TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin();

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FNotifyHook* NotifyHook = nullptr;
	if(InArgs._SetNotifyHook)
	{
		NotifyHook = Editor->GetNotifyHook();
	}

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = InArgs._HideNameArea ? FDetailsViewArgs::HideNameArea : FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = NotifyHook;
	DetailsViewArgs.ViewIdentifier = InArgs._ViewIdentifier;
	DetailsViewArgs.ExternalScrollbar = InArgs._ExternalScrollbar;
	DetailsViewArgs.ScrollbarAlignment = InArgs._ScrollbarAlignment;
	DetailsViewArgs.bShowSectionSelector = InArgs._ShowSectionSelector;

	PropertyView = EditModule.CreateDetailView( DetailsViewArgs );

	PropertyView->SetIsPropertyVisibleDelegate( FIsPropertyVisible::CreateSP(this, &SRigVMDetailsInspector::IsPropertyVisible) );
	PropertyView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SRigVMDetailsInspector::IsPropertyEditingEnabled));

	IsPropertyEditingEnabledDelegate = InArgs._IsPropertyEditingEnabledDelegate;
	UserOnFinishedChangingProperties = InArgs._OnFinishedChangingProperties;

	const UClass* BlueprintClass = Editor->GetRigVMAssetInterface()->GetObject()->GetClass();
	FOnGetDetailCustomizationInstance FunctionDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMGraphDetailCustomization::MakeInstance, Editor, BlueprintClass);
	PropertyView->RegisterInstancedCustomPropertyLayout(URigVMEdGraph::StaticClass(), FunctionDetails);

	FOnGetDetailCustomizationInstance LayoutVariableDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMVariableDetailCustomization::MakeInstance, Editor);
	PropertyView->RegisterInstancedCustomPropertyLayout(UPropertyWrapper::StaticClass(), LayoutVariableDetails);

	FOnGetDetailCustomizationInstance LayoutClassSettingsDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMClassSettingsDetailCustomization::MakeInstance, Editor);
	PropertyView->RegisterInstancedCustomPropertyLayout(URigVMRuntimeAsset::StaticClass(), LayoutClassSettingsDetails);

	FOnGetDetailCustomizationInstance CommentNodeDetails = FOnGetDetailCustomizationInstance::CreateStatic(&FRigVMCommentNodeDetailCustomization::MakeInstance);
	PropertyView->RegisterInstancedCustomPropertyLayout(URigVMCommentNode::StaticClass(), CommentNodeDetails);

	Editor->OnEditorClosed().AddSP(this, &SRigVMDetailsInspector::OnEditorClose);

	// Create the border that all of the content will get stuffed into
	ChildSlot
	[
		SNew(SVerticalBox)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("RigVMInspector")))
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew( ContextualEditingBorderWidget, SBorder )
			.Padding(0.0f)
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
		]
	];

	// Update based on the current (empty) selection set
	TArray<UObject*> InitialSelectedObjects;
	TArray<UObject*> SelectionInfo;
	UpdateFromObjects(InitialSelectedObjects, SelectionInfo, SRigVMDetailsInspector::FShowDetailsOptions(FText::GetEmpty(), true));

	// create struct to display
	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;
	ViewArgs.NotifyHook = NotifyHook;

	StructureDetailsView = EditModule.CreateStructureDetailView(ViewArgs, StructureViewArgs, StructToDisplay, LOCTEXT("Struct", "Struct View"));
	StructureDetailsView->GetDetailsView()->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateSP(this, &SRigVMDetailsInspector::IsStructViewPropertyReadOnly));
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().Clear();
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().Add(UserOnFinishedChangingProperties);
}

/** Update the inspector window to show information on the supplied object */
void SRigVMDetailsInspector::ShowDetailsForSingleObject(UObject* Object, const FShowDetailsOptions& Options)
{
	TArray<UObject*> PropertyObjects;

	if (Object != nullptr)
	{
		PropertyObjects.Add(Object);
	}

	ShowDetailsForObjects(PropertyObjects, Options);
}

void SRigVMDetailsInspector::ShowDetailsForObjects(const TArray<UObject*>& PropertyObjects, const FShowDetailsOptions& Options)
{
	// Refresh is being deferred until the next tick, this prevents batch operations from bombarding the details view with calls to refresh
	RefreshPropertyObjects = PropertyObjects;
	RefreshOptions = Options;
	bRefreshOnTick = true;
}

/** Update the inspector window to show information on the supplied object */
void SRigVMDetailsInspector::ShowSingleStruct(TSharedPtr<FStructOnScope> InStructToDisplay)
{
	static bool bIsReentrant = false;
	if (!bIsReentrant)
	{
		bIsReentrant = true;
		// When the selection is changed, we may be potentially actively editing a property,
		// if this occurs we need need to immediately clear keyboard focus
		if (FSlateApplication::Get().HasFocusedDescendants(AsShared()))
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
		}
		bIsReentrant = false;
	}

	StructToDisplay = InStructToDisplay;
	// we don't defer this becasue StructDetailViews contains TSharedPtr to this sturct, 
	// not clearing until next tick causes crash
	// so will update struct view here, but updating widget will happen in the tick
	StructureDetailsView->SetStructureData(InStructToDisplay);
	bRefreshOnTick = true;
}

void SRigVMDetailsInspector::AddPropertiesRecursive(FProperty* Property)
{
	if (Property != nullptr)
	{
		// Add this property
		SelectedObjectProperties.Add(Property);

		// If this is a struct or an array of structs, recursively add the child properties
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if(	StructProperty != nullptr && StructProperty->Struct != nullptr)
		{
			for (TFieldIterator<FProperty> StructPropIt(StructProperty->Struct); StructPropIt; ++StructPropIt)
			{
				FProperty* InsideStructProperty = *StructPropIt;
				AddPropertiesRecursive(InsideStructProperty);
			}
		}
		else if( ArrayProperty && ArrayProperty->Inner->IsA<FStructProperty>() )
		{
			AddPropertiesRecursive(ArrayProperty->Inner);
		}
	}
}

void SRigVMDetailsInspector::UpdateFromSingleStruct(const TSharedPtr<FStructOnScope>& InStructToDisplay)
{
	if (StructureDetailsView.IsValid())
	{
		SelectedObjects.Empty();

		// Update our context-sensitive editing widget
		ContextualEditingBorderWidget->SetContent(StructureDetailsView->GetWidget().ToSharedRef());
	}
}

void SRigVMDetailsInspector::UpdateFromObjects(const TArray<UObject*>& PropertyObjects, TArray<UObject*>& SelectionInfo, const FShowDetailsOptions& Options)
{
	TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin();
	
	if (!Options.bForceRefresh)
	{
		// Early out if the PropertyObjects and the SelectedObjects are the same
		bool bEquivalentSets = (PropertyObjects.Num() == SelectedObjects.Num());
		if (bEquivalentSets)
		{
			// Verify the elements of the sets are equivalent
			for (int32 i = 0; i < PropertyObjects.Num(); i++)
			{
				if (PropertyObjects[i] != SelectedObjects[i].Get())
				{
					if (PropertyObjects[i] && !PropertyObjects[i]->IsValidLowLevel())
					{
						ensureMsgf(false, TEXT("Object in RigVMInspector is invalid, see TTP 281915"));
						continue;
					}

					bEquivalentSets = false;
					break;
				}
			}
		}

		if (bEquivalentSets)
		{
			return;
		}
	}

	PropertyView->OnFinishedChangingProperties().Clear();
	PropertyView->OnFinishedChangingProperties().Add(UserOnFinishedChangingProperties);
	PropertyView->OnFinishedChangingProperties().AddSP(this, &SRigVMDetailsInspector::OnFinishedChangingProperties);

	// Proceed to update
	SelectedObjects.Empty();

	for (auto ObjectIt = PropertyObjects.CreateConstIterator(); ObjectIt; ++ObjectIt)
	{
		if (UObject* Object = *ObjectIt)
		{
			if (!Object->IsValidLowLevel())
			{
				ensureMsgf(false, TEXT("Object in KismetInspector is invalid, see TTP 281915"));
				continue;
			}

			SelectedObjects.Add(Object);

			if (UK2Node* K2Node = Cast<UK2Node>(Object))
			{

				// See if we should edit properties of the node
				if (K2Node->ShouldShowNodeProperties())
				{
					SelectionInfo.Add(Object);
				}
			}
			else
			{
				// Editing any UObject*
				SelectionInfo.AddUnique(Object);
			}
		}
	}

	// By default, no property filtering
	SelectedObjectProperties.Empty();

	PropertyViewTitle = Options.ForcedTitle;

	// Update our context-sensitive editing widget
	ContextualEditingBorderWidget->SetContent( MakeContextualEditingWidget(SelectionInfo, Options) );
}

bool SRigVMDetailsInspector::IsStructViewPropertyReadOnly(const struct FPropertyAndParent& PropertyAndParent) const
{
	const FProperty& Property = PropertyAndParent.Property;
	if (Property.HasAnyPropertyFlags(CPF_EditConst))
	{
		return true;
	}

	return false;
}

bool SRigVMDetailsInspector::IsAnyParentOrContainerSelected(const FPropertyAndParent& PropertyAndParent) const
{
	for (const FProperty* CurrentProperty : PropertyAndParent.ParentProperties)
	{
		if (SelectedObjectProperties.Find(const_cast<FProperty*>(CurrentProperty)))
		{
			return true;
		}

		// the property might be the Inner property of an array (or Key/Value of a map), so check if the outer property is selected
		const FProperty* CurrentOuter = CurrentProperty->GetOwner<FProperty>();
		if (CurrentOuter != nullptr && SelectedObjectProperties.Find(const_cast<FProperty*>(CurrentOuter)))
		{
			return true;
		}
	}

	return false;
} 

bool SRigVMDetailsInspector::IsPropertyVisible( const FPropertyAndParent& PropertyAndParent ) const
{
	const FProperty& Property = PropertyAndParent.Property;

	// If we are in 'instance preview' - hide anything marked 'disabled edit on instance'
	if ((ECheckBoxState::Checked == PublicViewState) && Property.HasAnyPropertyFlags(CPF_DisableEditOnInstance))
	{
		return false;
	}

	// Only hide EditInstanceOnly properties if we are editing a CDO/archetype
	bool bIsEditingTemplate = true;
	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		UObject* Object = SelectedObject.Get();
		if (!Object || !Object->IsTemplate())
		{
			bIsEditingTemplate = false;
			break;
		}
	}

	if (bIsEditingTemplate)
	{
		// check if the property (or any of its parent properties) was added by this blueprint
		// this is necessary because of Instanced objects, which will have a different owning class yet are conceptually contained in this blueprint
		bool bVariableAddedInCurrentBlueprint = false;
		TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin();
		FRigVMEditorAssetInterfacePtr Blueprint = Editor.IsValid() ? Editor->GetRigVMAssetInterface() : nullptr;

		auto WasAddedInThisBlueprint = [Blueprint](const FProperty* Property)
		{
			if (const UClass* OwningClass = Property->GetOwnerClass())
			{
				return Blueprint && OwningClass->ClassGeneratedBy.Get() == Blueprint->GetObject();
			}
			return false;
		};
		
		bVariableAddedInCurrentBlueprint |= WasAddedInThisBlueprint(&Property);

		for (const FProperty* Parent : PropertyAndParent.ParentProperties)
		{
			bVariableAddedInCurrentBlueprint |= WasAddedInThisBlueprint(Parent);
		}

		// if this property wasn't added in this blueprint, we want to filter it out if it (or any of its parents) are marked EditInstanceOnly or private
		if (!bVariableAddedInCurrentBlueprint)
		{
			if (Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate) || Property.GetBoolMetaData(FBlueprintMetadata::MD_Private))
			{
				return false;
			}

			for (const FProperty* Parent : PropertyAndParent.ParentProperties)
			{
				if (Property.HasAnyPropertyFlags(CPF_DisableEditOnTemplate) || Parent->GetBoolMetaData(FBlueprintMetadata::MD_Private))
				{
					return false;
				}
			}
		}
	}
	
	// figure out if this Blueprint variable is an Actor variable
	const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(&Property);
	const FSetProperty* SetProperty = CastField<const FSetProperty>(&Property);
	const FMapProperty* MapProperty = CastField<const FMapProperty>(&Property);

	// Filter down to selected properties only if set.
	if (SelectedObjectProperties.Find(const_cast<FProperty*>(&Property)))
	{
		// If the current property is selected, it is visible.
		return true;
	}
	else if (PropertyAndParent.ParentProperties.Num() > 0 && SelectedObjectProperties.Num() > 0)
	{
		if (IsAnyParentOrContainerSelected(PropertyAndParent))
		{
			return true;
		}
	}
	else if (ArrayProperty || MapProperty || SetProperty)
	{
		// .Find won't work here because the items inside of the container properties are not FProperties
		for (const TWeakFieldPtr<FProperty>& CurProp : SelectedObjectProperties)
		{
			if ((ArrayProperty && (ArrayProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == ArrayProperty->GetFName()) ||
				(MapProperty && (MapProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == MapProperty->GetFName()) ||
				(SetProperty && (SetProperty->PropertyFlags & CPF_Edit) && CurProp->GetFName() == SetProperty->GetFName()))
			{
				return true;
			}
		}
	}

	return SelectedObjectProperties.Num() == 0;
}

EVisibility SRigVMDetailsInspector::GetPropertyViewVisibility() const
{
	return bShowInspectorPropertyView? EVisibility::Visible : EVisibility::Collapsed;
}

bool SRigVMDetailsInspector::IsPropertyEditingEnabled() const
{
	bool bIsEditable = true;

	if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
	{
		// This function is essentially for PIE use so if we are NOT doing PIE use the normal path
		if (GEditor->GetPIEWorldContext() == nullptr)
		{
			bIsEditable = Editor->InEditingMode();
		}
	}

	for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
	{
		if (UActorComponent* Component = Cast<UActorComponent>(SelectedObject.Get()))
		{
			if(!CastChecked<UActorComponent>(Component->GetArchetype())->IsEditableWhenInherited())
			{
				bIsEditable = false;
				break;
			}
		}
		else if(UEdGraphNode* EdGraphNode = Cast<UEdGraphNode>(SelectedObject.Get()))
		{
			if(UEdGraph* OuterGraph = EdGraphNode->GetGraph())
			{
				if(WeakEditor.IsValid() && !WeakEditor.Pin()->IsEditable(OuterGraph))
				{
					bIsEditable = false;
					break;
				}
			}
		}
	}
	
	return bIsEditable && (!IsPropertyEditingEnabledDelegate.IsBound() || IsPropertyEditingEnabledDelegate.Execute());
}

ECheckBoxState SRigVMDetailsInspector::GetPublicViewCheckboxState() const
{
	return PublicViewState;
}

void SRigVMDetailsInspector::SetPublicViewCheckboxState( ECheckBoxState InIsChecked )
{
	PublicViewState = InIsChecked;

	//reset the details view
	TArray<UObject*> Objs;
	for(auto It(SelectedObjects.CreateIterator());It;++It)
	{
		Objs.Add(It->Get());
	}
	SelectedObjects.Empty();
	
	if(Objs.Num() > 1)
	{
		ShowDetailsForObjects(Objs);
	}
	else if(Objs.Num() == 1)
	{
		ShowDetailsForSingleObject(Objs[0], FShowDetailsOptions(PropertyViewTitle));
	}
	
	//WeakEditor.Pin()->StartEditingDefaults();
}

void SRigVMDetailsInspector::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
