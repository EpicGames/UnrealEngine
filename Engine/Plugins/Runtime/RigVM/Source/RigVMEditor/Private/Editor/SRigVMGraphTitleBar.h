// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SCompoundWidget.h"

class FRigVMNewEditor;
class SFunctionEditor;
class SScrollBox;
class SWidget;
class UEdGraph;
class UObject;
struct FSlateBrush;

//////////////////////////////////////////////////////////////////////////
// SRigVMGraphTitleBar

class SRigVMGraphTitleBar : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam( FEdGraphEvent, class UEdGraph* );
	
public:
	SLATE_BEGIN_ARGS( SRigVMGraphTitleBar )
		: _EdGraphObj(nullptr)
		, _Editor()
		{}

		SLATE_ARGUMENT( UEdGraph*, EdGraphObj )
		SLATE_ARGUMENT( TWeakPtr<FRigVMNewEditor>, Editor )
		SLATE_ARGUMENT( TSharedPtr<SWidget>, HistoryNavigationWidget )
	SLATE_END_ARGS()

	/** SRigVMGraphTitleBar destructor */
	~SRigVMGraphTitleBar();

	void Construct(const FArguments& InArgs);

	/** Refresh the toolbar */
	void Refresh();

protected:
	/** Owning Kismet 2 */
	TWeakPtr<FRigVMNewEditor> EditorWeak;

	/** Edited graph */
	TObjectPtr<UEdGraph> EdGraphObj = nullptr;

	TSharedPtr<SScrollBox> BreadcrumbTrailScrollBox;

	/** Breadcrumb trail widget */
	TSharedPtr< SBreadcrumbTrail<UEdGraph*> > BreadcrumbTrail;

	/** Blueprint title being displayed for toolbar */
	FText BlueprintTitle;

protected:
	/** Get the icon to use */
	const FSlateBrush* GetTypeGlyph() const;

	/** Get the extra title text */
	FText GetTitleExtra() const;

	/** Helper methods */

	void RebuildBreadcrumbTrail();

	static FText GetTitleForOneCrumb(const UEdGraph* Graph);

	/** Function to fetch outer class which is of type UEGraph. */
	UEdGraph* GetOuterGraph( UObject* Obj );

	/** Helper method used to show blueprint title in breadcrumbs */
	FText GetBlueprintTitle() const;
};
