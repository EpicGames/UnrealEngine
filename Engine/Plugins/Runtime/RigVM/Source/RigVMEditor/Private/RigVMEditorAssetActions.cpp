// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorAssetActions.h"
#include "RigVMEditorAsset.h"
#include "RigVMRuntimeAsset.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"

UClass* FRigVMEditorAssetActions::GetSupportedClass() const
{
	return URigVMEditorAsset::StaticClass();
}

void FRigVMEditorAssetActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	if (IsEngineExitRequested())
	{
		return;
	}

	// Redirect to the runtime asset instead of opening the editor asset directly
	for (UObject* Obj : InObjects)
	{
		if (URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(Obj))
		{
			// The runtime asset is the outer of the editor asset
			if (URigVMRuntimeAsset* RuntimeAsset = EditorAsset->GetTypedOuter<URigVMRuntimeAsset>())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(RuntimeAsset);
			}
		}
	}
}

