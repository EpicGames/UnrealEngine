// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorBlueprintLibrary.h"
#include "RigVMDeveloperModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/UObjectIterator.h"
#include "ImageUtils.h"
#include "TextureResource.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/WidgetRenderer.h"
#include "RenderingThread.h"
#include "Misc/FileHelper.h"
#include "HAL/IConsoleManager.h"
#include "Editor/RigVMMinimalEnvironment.h"
#include "Widgets/SRigVMGraphNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigVMEditorBlueprintLibrary)

#define LOCTEXT_NAMESPACE "RigVMEditorBlueprintLibrary"

void URigVMEditorBlueprintLibrary::RecompileVM(URigVMBlueprint* InBlueprint)
{
	if(InBlueprint == nullptr)
	{
		return;
	}
	InBlueprint->RecompileVM();
}

void URigVMEditorBlueprintLibrary::RecompileVMIfRequired(URigVMBlueprint* InBlueprint)
{
	if(InBlueprint == nullptr)
	{
		return;
	}
	InBlueprint->RecompileVMIfRequired();
}

void URigVMEditorBlueprintLibrary::RequestAutoVMRecompilation(URigVMBlueprint* InBlueprint)
{
	if(InBlueprint == nullptr)
	{
		return;
	}
	InBlueprint->RequestAutoVMRecompilation();
}

URigVMGraph* URigVMEditorBlueprintLibrary::GetModel(URigVMBlueprint* InBlueprint)
{
	if(InBlueprint == nullptr)
	{
		return nullptr;
	}
	return InBlueprint->GetDefaultModel();
}

URigVMController* URigVMEditorBlueprintLibrary::GetController(URigVMBlueprint* InBlueprint)
{
	if(InBlueprint == nullptr)
	{
		return nullptr;
	}
	return InBlueprint->GetController();
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssets()
{
	return LoadAssetsByClass(URigVMBlueprint::StaticClass());
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsByClass(TSubclassOf<URigVMBlueprint> InClass)
{
	return LoadAssetsWithAssetDataAndBlueprintFilters(InClass, FRigVMAssetDataFilter(), FRigVMBlueprintFilter());
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithBlueprintFilter_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMBlueprintFilterDynamic InBlueprintFilter)
{
	return LoadAssetsWithAssetDataAndBlueprintFilters(InClass, FRigVMAssetDataFilter(), FRigVMBlueprintFilter::CreateLambda(
		[InBlueprintFilter](const URigVMBlueprint* Blueprint, const TArray<FRigVMBlueprintLoadLogEntry>& LogEntries) -> bool
		{
			return InBlueprintFilter.Execute(Blueprint, LogEntries);
		})
	);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithAssetDataFilter_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMAssetDataFilterDynamic InAssetDataFilter)
{
	return LoadAssetsWithAssetDataAndBlueprintFilters(InClass, FRigVMAssetDataFilter::CreateLambda(
		[InAssetDataFilter](const FAssetData& AssetData) -> bool
		{
			return InAssetDataFilter.Execute(AssetData);
		}),
		FRigVMBlueprintFilter()
	);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithNodeFilter_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMNodeFilterDynamic InNodeFilter)
{
	return LoadAssetsWithNodeFilter(InClass, FRigVMNodeFilter::CreateLambda(
		[InNodeFilter](const URigVMBlueprint* Blueprint, const URigVMNode* Node) -> bool
		{
			return InNodeFilter.Execute(Blueprint, Node);
		})
	);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithNodeFilter(
	TSubclassOf<URigVMBlueprint> InClass,
	FRigVMNodeFilter InNodeFilter)
{
	return LoadAssetsWithAssetDataAndNodeFilters(InClass, FRigVMAssetDataFilter(), InNodeFilter);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithAssetDataAndBlueprintFilters_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass,
	FRigVMAssetDataFilterDynamic InAssetDataFilter,
	FRigVMBlueprintFilterDynamic InBlueprintFilter)
{
	return LoadAssetsWithAssetDataAndBlueprintFilters(InClass,
		FRigVMAssetDataFilter::CreateLambda(
		[InAssetDataFilter](const FAssetData& AssetData) -> bool
			{
				return InAssetDataFilter.Execute(AssetData);
			}
		),
		FRigVMBlueprintFilter::CreateLambda(
		[InBlueprintFilter](const URigVMBlueprint* Blueprint, const TArray<FRigVMBlueprintLoadLogEntry>& LogEntries) -> bool
			{
				return InBlueprintFilter.Execute(Blueprint, LogEntries);
			}
		)
	);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithAssetDataAndBlueprintFilters(
	TSubclassOf<URigVMBlueprint> InClass, 
	FRigVMAssetDataFilter InAssetDataFilter, 
	FRigVMBlueprintFilter InBlueprintFilter
)
{
	TArray<FAssetData> AssetDataList = GetAssetsWithFilter(InClass, InAssetDataFilter);

	const int32 NumAssets = AssetDataList.Num();

	TArray<URigVMBlueprint*> LoadedAssets;
	{
		const FString Title = FString::Printf(TEXT("Load all %s assets..."), *InClass->GetName()); 
		FScopedSlowTask LoadAssetsTask(static_cast<float>(NumAssets), FText::FromString(Title));
		LoadAssetsTask.MakeDialog(true);

		for(int32 Index = 0; Index < NumAssets; Index++)
		{
			if (LoadAssetsTask.ShouldCancel())
			{
				break;
			}

			const FAssetData& AssetData = AssetDataList[Index];
			LoadAssetsTask.EnterProgressFrame(1, FText::FromName(AssetData.PackageName));		

			static constexpr TCHAR Format[] = TEXT("[%d/%d]: %s -> %s");
			UE_LOG(LogRigVM, Display, Format, Index, NumAssets, *AssetData.AssetName.ToString(), *AssetData.PackageName.ToString())

			// completely ignore exceptions during this scope
			FScopedScriptExceptionHandler ScopedScriptExceptionHandler([](ELogVerbosity::Type Verbosity, const TCHAR* ErrorMessage, const TCHAR* StackMessage)
			{
				FString Message;
				if(ErrorMessage)
				{
					Message += ErrorMessage;
				}
				if(StackMessage)
				{
					if(!Message.IsEmpty())
					{
						Message += TEXT("\n");
					}
					Message += StackMessage;
				}

				if(Verbosity == ELogVerbosity::Error || Verbosity == ELogVerbosity::Fatal)
				{
					UE_LOGF(LogRigVMDeveloper, Error, "%ls", *Message);
				}
				else if(Verbosity == ELogVerbosity::Warning)
				{
					UE_LOGF(LogRigVMDeveloper, Warning, "%ls", *Message);
				}
				else
				{
					UE_LOGF(LogRigVMDeveloper, Display, "%ls", *Message);
				}
			});

			// set up a lambda to record all error messages
			TArray<FRigVMBlueprintLoadLogEntry> LogEntries;

			if(InBlueprintFilter.IsBound())
			{
				URigVMBlueprint::QueueCompilerMessageDelegate(FOnRigVMReportCompilerMessage::FDelegate::CreateLambda(
					[&LogEntries](EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
					{
						ERigVMBlueprintLoadLogSeverity Severity = ERigVMBlueprintLoadLogSeverity::Display;
						if(InSeverity == EMessageSeverity::Warning)
						{
							Severity = ERigVMBlueprintLoadLogSeverity::Warning;
						}
						else if(InSeverity == EMessageSeverity::Error)
						{
							Severity = ERigVMBlueprintLoadLogSeverity::Error;
						}
						LogEntries.Emplace(Severity, InSubject, InMessage);
					})
				);
			}
			
			if(URigVMBlueprint* Blueprint = Cast<URigVMBlueprint>(AssetData.GetAsset()))
			{
				if(InBlueprintFilter.IsBound())
				{
					if(InBlueprintFilter.Execute(Blueprint, LogEntries))
					{
						LoadedAssets.Add(Blueprint);
					}
				}
				else
				{
					LoadedAssets.Add(Blueprint);
				}

				Blueprint->OnReportCompilerMessage().Clear();
			}

			if(InBlueprintFilter.IsBound())
			{
				URigVMBlueprint::ClearQueuedCompilerMessageDelegates();
			}
			LoadAssetsTask.ForceRefresh();
		}
	}

	return LoadedAssets;
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithAssetDataAndNodeFilters_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMAssetDataFilterDynamic InAssetDataFilter,
	FRigVMNodeFilterDynamic InNodeFilter)
{
	return LoadAssetsWithAssetDataAndNodeFilters(InClass,
		FRigVMAssetDataFilter::CreateLambda(
		[InAssetDataFilter](const FAssetData& AssetData) -> bool
		{
			return InAssetDataFilter.Execute(AssetData);
		}),
		FRigVMNodeFilter::CreateLambda(
		[InNodeFilter](const URigVMBlueprint* Blueprint, const URigVMNode* Node) -> bool
		{
			return InNodeFilter.Execute(Blueprint, Node);
		})
	);
}

TArray<URigVMBlueprint*> URigVMEditorBlueprintLibrary::LoadAssetsWithAssetDataAndNodeFilters(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMAssetDataFilter InAssetDataFilter, FRigVMNodeFilter InNodeFilter)
{
	return LoadAssetsWithAssetDataAndBlueprintFilters(InClass, InAssetDataFilter, FRigVMBlueprintFilter::CreateLambda(
		[InNodeFilter](const URigVMBlueprint* Blueprint, const TArray<FRigVMBlueprintLoadLogEntry>& LogEntries) -> bool
		{
			TArray<URigVMGraph*> Models = Blueprint->GetAllModels();
			int32 NumNodes = 0;
			for(const URigVMGraph* Model : Models)
			{
				NumNodes += Model->GetNodes().Num();
			}
			FScopedSlowTask FilterNodesTask(static_cast<float>(NumNodes), LOCTEXT("FilteringNodes", "Filtering Nodes"));
			for(const URigVMGraph* Model : Models)
			{
				for(const URigVMNode* Node : Model->GetNodes())
				{
					FilterNodesTask.EnterProgressFrame(1);		
					if(InNodeFilter.Execute(Blueprint, Node))
					{
						return true;
					}
				}
			}
			return false;
		})
	);
}

TArray<FAssetData> URigVMEditorBlueprintLibrary::GetAssetsWithFilter_ForBlueprint(
	TSubclassOf<URigVMBlueprint> InClass, FRigVMAssetDataFilterDynamic InAssetDataFilter)
{
	return GetAssetsWithFilter(InClass, FRigVMAssetDataFilter::CreateLambda(
		[InAssetDataFilter](const FAssetData& AssetData) -> bool
		{
			return InAssetDataFilter.Execute(AssetData);
		})
	);
}

TArray<FAssetData> URigVMEditorBlueprintLibrary::GetAssetsWithFilter(
		UClass* InClass,
		FRigVMAssetDataFilter InAssetDataFilter)
{
	FARFilter Filter;
	if(InClass == nullptr)
	{
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* Class = *It;

			// Skip abstract and non-blueprintable classes if needed
			if (Class->HasAnyClassFlags(CLASS_Abstract))
			{
				continue;
			}

			if (Class->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()))
			{
				Filter.ClassPaths.Add(Class->GetClassPathName());
			}
			else if (Class->ImplementsInterface(URigVMRuntimeAssetInterface::StaticClass()))
			{
				Filter.ClassPaths.Add(Class->GetClassPathName());
			}
			else if (Class->IsChildOf(URigVMBlueprintGeneratedClass::StaticClass()))
			{
				Filter.ClassPaths.Add(Class->GetClassPathName());
			}
		}
	}
	else
	{
		if (!InClass->IsChildOf(URigVMBlueprint::StaticClass()) &&
			!InClass->IsChildOf(URigVMBlueprintGeneratedClass::StaticClass()))
		{
			return TArray<FAssetData>();
		}
		
		Filter.ClassPaths.Add(InClass->GetClassPathName());
	}


	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Collect a full list of assets with the specified class
	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssets(Filter, AssetDataList);

	if(InAssetDataFilter.IsBound())
	{
		TArray<FAssetData> FilteredAssetDataList;
		for(const FAssetData& AssetData : AssetDataList)
		{
			if(InAssetDataFilter.Execute(AssetData))
			{
				FilteredAssetDataList.Add(AssetData);
			}			
		}
		Swap(FilteredAssetDataList, AssetDataList);
	}

	return AssetDataList;
}

FAutoConsoleCommand FCmdRigVMJsonRenderNodeToPNG
(
	TEXT("RigVM.JSON.RenderNodeToPNG"),
	TEXT("Renders a node to a PNG image."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		auto PrintHelp = []()
		{
			UE_LOGF(LogRigVMDeveloper, Warning, "Valid Arguments:\n"
				"subject=/Script/RigVM.RigVMFunction_MathBoolOr - the path name of the subject (struct or graph function). alternatively you can pass a template notation.\n"
				"filepath=c:/temp/NodePreview.png\n");
			return;					
		};
		if (Args.Num() != 2)
		{
			PrintHelp();
			return;
		}
		UObject* Subject = nullptr;
		FRigVMGraphFunctionIdentifier FunctionIdentifier;
		FName TemplateNotation = NAME_None;
		FString FilePath;

		for (const FString& Arg : Args)
		{
			if (Arg.StartsWith(TEXT("subject="), ESearchCase::IgnoreCase))
			{
				const FString SubjectPathName = Arg.Mid(8);
				Subject = FindObject<UObject>(nullptr, *SubjectPathName);
				if (!Subject)
				{
					const TChunkedArray<FRigVMTemplate>& Templates = FRigVMRegistry::Get().GetTemplates();
					for (const FRigVMTemplate& Template : Templates)
					{
						FString ExistingTemplateName = Template.GetName().ToString();
						ExistingTemplateName.ReplaceInline(TEXT(" "), TEXT("_"));
						if (ExistingTemplateName.Equals(SubjectPathName, ESearchCase::IgnoreCase))
						{
							TemplateNotation = Template.GetNotation();
							break;
						}
					}
					if (TemplateNotation.IsNone())
					{
						UE_LOGF(LogRigVMDeveloper, Error, "Invalid Subject '%ls'", *SubjectPathName);
						return;
					}
				}
			}
			else if (Arg.StartsWith(TEXT("filepath="), ESearchCase::IgnoreCase))
			{
				FilePath = Arg.Mid(9);
			}
			else
			{
				PrintHelp();
				return;
			}
		}

		if (Subject)
		{
			if (URigVMEditorBlueprintLibrary::RenderRigVMSubjectToPNG(Subject, FilePath))
			{
				UE_LOGF(LogRigVMDeveloper, Log, "Successfully rendered image of '%ls' to file '%ls'.", *Subject->GetPathName(), *FilePath);
			}
			else
			{
				UE_LOGF(LogRigVMDeveloper, Warning, "Error when rendering node of '%ls' to file '%ls'.", *Subject->GetPathName(), *FilePath);
			}
		}
		else if (!TemplateNotation.IsNone())
		{
			if (URigVMEditorBlueprintLibrary::RenderRigVMTemplateToPNG(TemplateNotation, FilePath))
			{
				UE_LOGF(LogRigVMDeveloper, Log, "Successfully rendered image of '%ls' to file '%ls'.", *TemplateNotation.ToString(), *FilePath);
			}
			else
			{
				UE_LOGF(LogRigVMDeveloper, Warning, "Error when rendering node of '%ls' to file '%ls'.", *TemplateNotation.ToString(), *FilePath);
			}
		}
		else
		{
			PrintHelp();
		}
	})
);

bool URigVMEditorBlueprintLibrary::RenderRigVMSubjectToPNG(UObject* InSubject, const FString& OutFileName)
{
	const URigVMSchema* Schema = nullptr;
	URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(MakeRigVMAssetForSubject(InSubject, Schema));
	if (!EditorAsset)
	{
		return false;
	}

	EditorAsset->SetAutoVMRecompile(false);

	URigVMGraph* Model = EditorAsset->GetRigVMClient()->GetDefaultModel();
	if (!Model)
	{
		return false;
	}

	Model->SetSchemaClass(Schema->GetClass());

	URigVMController* Controller = EditorAsset->GetRigVMClient()->GetOrCreateController(Model);
	if (!Controller)
	{
		return false;
	}

	Controller->SetSchemaClass(Schema->GetClass());

	URigVMNode* ModelNode = nullptr;
	if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InSubject))
	{
		if (ScriptStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()))
		{
			const FName FactoryName = FRigVMDispatchFactory::GetFactoryName(ScriptStruct);
			FName TemplateNotation = NAME_None;
			{
				FRigVMRegistryReadLock Registry;
				if (const FRigVMDispatchFactory* Factory = Registry->FindDispatchFactory_NoLock(FactoryName))
				{
					TemplateNotation = Factory->GetTemplateNotation_NoLock(Registry);
				}
			}
			if (!TemplateNotation.IsNone())
			{
				ModelNode = Controller->AddTemplateNode(TemplateNotation);
			}
		}
		else if (ScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()))
		{
			ModelNode = Controller->AddUnitNode(const_cast<UScriptStruct*>(ScriptStruct));
		}
	}
	else if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InSubject))
	{
		ModelNode = Controller->AddFunctionReferenceNode(LibraryNode);
	}

	return RenderModelNodeToPNG(EditorAsset, ModelNode, OutFileName);
}

bool URigVMEditorBlueprintLibrary::RenderRigVMTemplateToPNG(FName InTemplateNotation, const FString& OutFileName)
{
	const FRigVMTemplate* Template = FRigVMRegistry::Get().FindTemplate(InTemplateNotation);
	if (!Template)
	{
		return false;
	}

	UObject* Subject = nullptr;
	if (const FRigVMDispatchFactory* Factory = Template->GetDispatchFactory())
	{
		Subject = Factory->GetScriptStruct();
	}
	else if (const FRigVMFunction* Function = Template->GetPrimaryPermutation())
	{
		Subject = Function->Struct;
	}
	if (!Subject)
	{
		return false;
	}

	const URigVMSchema* Schema = nullptr;
	URigVMEditorAsset* EditorAsset = Cast<URigVMEditorAsset>(MakeRigVMAssetForSubject(Subject, Schema));
	if (!EditorAsset)
	{
		return false;
	}

	EditorAsset->SetAutoVMRecompile(false);

	URigVMGraph* Model = EditorAsset->GetRigVMClient()->GetDefaultModel();
	if (!Model)
	{
		return false;
	}

	Model->SetSchemaClass(Schema->GetClass());

	URigVMController* Controller = EditorAsset->GetRigVMClient()->GetOrCreateController(Model);
	if (!Controller)
	{
		return false;
	}

	Controller->SetSchemaClass(Schema->GetClass());

	URigVMNode* ModelNode = Controller->AddTemplateNode(Template->GetNotation());
	return RenderModelNodeToPNG(EditorAsset, ModelNode, OutFileName);
}

bool URigVMEditorBlueprintLibrary::RenderModelNodeToPNG(URigVMEditorAsset* EditorAsset, URigVMNode* InModelNode, const FString& OutFileName)
{
	if (!InModelNode)
	{
		return false;
	}

	//create directory if it doesn't exist
	if (!IFileManager::Get().MakeDirectory(*FPaths::GetPath(OutFileName), true))
	{
		UE_LOGF(LogRigVMDeveloper, Warning, "Cannot create directory for file '%ls'.", *OutFileName);
		return false;
	}

	// we only allow png extensions
	if (!OutFileName.EndsWith(TEXT(".png"), ESearchCase::IgnoreCase))
	{
		UE_LOGF(LogRigVMDeveloper, Warning, "Image file '%ls' is not a PNG.", *OutFileName);
		return false;
	}

	TSharedPtr<FRigVMMinimalEnvironment> MinimalEnvironment = MakeShared<FRigVMMinimalEnvironment>(EditorAsset);
	MinimalEnvironment->SetNode(InModelNode);

	URigVMEdGraphNode* EdGraphNode = MinimalEnvironment->GetEdGraphNode();
	if (!EdGraphNode)
	{
		return false;
	}

	TSharedRef<SRigVMGraphNode> GraphNode =
		SNew(SRigVMGraphNode)
		.Enabled(true)
		.GraphNodeObj(EdGraphNode);
	
	GraphNode->SlatePrepass();

	const FVector2D DesiredSize = GraphNode->GetDesiredSize();
	const uint32 SizeX = static_cast<uint32>(DesiredSize.X);
	const uint32 SizeY = static_cast<uint32>(DesiredSize.Y);
	if (SizeX <= 0 || SizeY <= 0)
	{
		return false;
	}

	// create a render target to render to
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(EditorAsset);
	RenderTarget->ClearColor = FLinearColor::Transparent;
	RenderTarget->InitCustomFormat(SizeX, SizeY, EPixelFormat::PF_B8G8R8A8, false);

	// render the widget into the render target
	FWidgetRenderer* WidgetRenderer = new FWidgetRenderer(true, false);
	check(WidgetRenderer);
	WidgetRenderer->DrawWidget(RenderTarget, GraphNode, GraphNode->GetCachedGeometry().Scale, DesiredSize, 0.f);
	FlushRenderingCommands();

	// read the pixels of the render target into a color array
	TArray<FColor> PixelColorData;
	ENQUEUE_RENDER_COMMAND(ReadScreenshotRTCmd)(
	[RenderTarget, &PixelColorData, SizeX, SizeY](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRenderTarget2DResource* RTResource =
				static_cast<FTextureRenderTarget2DResource*>(RenderTarget->GetRenderTargetResource());
			RHICmdList.ReadSurfaceData(
				RTResource->GetTextureRHI(),
				FIntRect(0, 0, static_cast<int32>(SizeX), static_cast<int32>(SizeY)),
				PixelColorData,
				FReadSurfaceDataFlags());
		}
	);
	FlushRenderingCommands();
	BeginCleanup(WidgetRenderer);

	// ReadPixels can return fewer pixels than expected
	if (PixelColorData.Num() != SizeX * SizeY)
	{
		return false;
	}
	
	// save the color array to png
	TArray64<uint8> CompressedBitmap;
	FImageUtils::PNGCompressImageArray(RenderTarget->SizeX, RenderTarget->SizeY, TArrayView64<const FColor>(PixelColorData.GetData(), PixelColorData.Num()), CompressedBitmap);
	return FFileHelper::SaveArrayToFile(CompressedBitmap, *OutFileName);
}

UObject* URigVMEditorBlueprintLibrary::MakeRigVMAssetForSubject(UObject* InUnitOrDispatchStructOrLibraryNode, const URigVMSchema *& OutSchema)
{
	if (!InUnitOrDispatchStructOrLibraryNode)
	{
		return  nullptr;
	}

	const URigVMSchema* SchemaToUse = nullptr;

	if (UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InUnitOrDispatchStructOrLibraryNode))
	{
		// find the schema that allows this struct
		for (TObjectIterator<UClass> ClassIterator; ClassIterator; ++ClassIterator)
		{
			if (!ClassIterator->IsChildOf(URigVMSchema::StaticClass()))
			{
				continue;
			}

			const URigVMSchema* Schema = ClassIterator->GetDefaultObject<URigVMSchema>();
			if (!Schema)
			{
				continue;
			}

			if (ScriptStruct->IsChildOf(FRigVMDispatchFactory::StaticStruct()))
			{
				const FName FactoryName = FRigVMDispatchFactory::GetFactoryName(ScriptStruct);
				FRigVMRegistryReadLock Registry;
				if (const FRigVMDispatchFactory* Factory = Registry->FindDispatchFactory_NoLock(FactoryName))
				{
					if (Schema->SupportsDispatchFactory_NoLock(nullptr, Factory, Registry))
					{
						SchemaToUse = Schema;
						break;
					}
				}
			}
			else if (ScriptStruct->IsChildOf(FRigVMStruct::StaticStruct()))
			{
				FRigVMRegistryReadLock Registry;
				if (const FRigVMFunction* Function = Registry->FindFunction_NoLock(ScriptStruct, *FRigVMStruct::ExecuteName.ToString()))
				{
					if (Schema->SupportsUnitFunction_NoLock(nullptr, Function, Registry))
					{
						SchemaToUse = Schema;
						break;
					}
				}
			}		
		}
	}
	else if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InUnitOrDispatchStructOrLibraryNode))
	{
		if (URigVMGraph* Graph = LibraryNode->GetGraph())
		{
			SchemaToUse = Graph->GetSchema();
		}
	}

	if (!SchemaToUse)
	{
		return nullptr;
	}
	OutSchema = SchemaToUse;

	UClass* RuntimeAssetClass = SchemaToUse->GetRuntimeAssetClass();
	if (!RuntimeAssetClass)
	{
		return nullptr;
	}

	UClass* EditorAssetClass = SchemaToUse->GetEditorAssetClass();
	if (!EditorAssetClass)
	{
		return nullptr;
	}

	URigVMRuntimeAsset* RuntimeAsset = NewObject<URigVMRuntimeAsset>(GetTransientPackage(), RuntimeAssetClass);
	RuntimeAsset->Initialize(EditorAssetClass);
	URigVMEditorAsset* EditorAsset = CastChecked<URigVMEditorAsset>(RuntimeAsset->GetEditorOnlyData());
	
	return EditorAsset; 
}

#undef LOCTEXT_NAMESPACE
