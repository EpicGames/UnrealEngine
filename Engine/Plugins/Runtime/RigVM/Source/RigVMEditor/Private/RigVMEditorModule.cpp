// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMEditor.cpp: Module implementation.
=============================================================================*/

#include "RigVMEditorModule.h"
#include "RigVMEditorAssetActions.h"
#include "AssetToolsModule.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/RigVMEditorCommands.h"
#include "Editor/RigVMExecutionStackCommands.h"
#include "Widgets/SRigVMEditorGraphExplorer.h"
#include "Editor/RigVMEditorStyle.h"
#include "Editor/RigVMGraphDetailCustomization.h"
#include "Editor/RigVMVariableDetailCustomization.h"
#include "Framework/Commands/GenericCommands.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "GraphEditorActions.h"
#include "RigVMBlueprintUtils.h"
#include "UserDefinedStructureCompilerUtils.h"
#include "EdGraph/RigVMEdGraphConnectionDrawingPolicy.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "ContentBrowserMenuContexts.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphUnitNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphVariableNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphTemplateNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphEnumNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphFunctionRefNodeSpawner.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphInvokeEntryNodeSpawner.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "Dialog/SCustomDialog.h"
#include "Widgets/SRigVMGraphChangePinType.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "RigVMFunctions/RigVMDispatch_Array.h"
#include "RigVMFunctions/RigVMDispatch_MakeStruct.h"
#include "RigVMFunctions/RigVMDispatch_Constant.h"
#include "RigVMFunctions/Simulation/RigVMFunction_AlphaInterp.h"
#include "RigVMFunctions/Debug/RigVMFunction_VisualDebug.h"
#include "ScopedTransaction.h"
#include "Editor/RigVMEditorTools.h"
#include "Editor/RigVMVariantDetailCustomization.h"
#include "UObject/UObjectIterator.h"
#include "Toolkits/ToolkitManager.h"
#include "ToolMenus.h"
#include "Editor/RigVMPythonLogDetails.h"
#include "RigVMFunctions/RigVMDispatch_Switch.h"
#include "RigVMFunctions/Execution/RigVMFunction_Sequence.h"
#include "RigVMModel/RigVMControllerActions.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "Widgets/SRigVMSwapFunctionsWidget.h"
#include "Widgets/SRigVMBulkEditDialog.h"
#include "Trace/RigVMTraceTrack.h"

DEFINE_LOG_CATEGORY(LogRigVMEditor);

#define LOCTEXT_NAMESPACE "RigVMEditorModule"

IMPLEMENT_MODULE(FRigVMEditorModule, RigVMEditor)

TAutoConsoleVariable<bool> CVarRigVMUseNewEditor(TEXT("RigVM.UseNewEditor"), 0, TEXT("Use editor which does not inherit from BlueprintEditor."));
TAutoConsoleVariable<bool> CVarRigVMBlueprintIndependentAssets(TEXT("RigVM.BlueprintIndependentAssets"), true, TEXT("Create assets that do not inherit from blueprint."));

UE::RigVMEditor::FRigVMEditorOnRequestFindNodeReferencesDelegate FRigVMEditorModule::OnRequestFindNodeReferences;

#if RIGVM_TRACE_ENABLED
FRewindDebuggerRigVMTrackCreator GRewindDebuggerRigVMTrackCreator;
#endif

FRigVMEditorModule& FRigVMEditorModule::Get()
{
	return FModuleManager::LoadModuleChecked< FRigVMEditorModule >(TEXT("RigVMEditor"));
}

void FRigVMEditorModule::StartupModule()
{
	if(IsRigVMEditorModuleBase())
	{
		FRigVMExecutionStackCommands::Register();
		FRigVMEditorGraphExplorerCommands::Register();
		FRigVMEditorCommands::Register();
		FRigVMEditorStyle::Register();

		EdGraphPanelNodeFactory = MakeShared<FRigVMEdGraphPanelNodeFactory>();
		EdGraphPanelPinFactory = MakeShared<FRigVMEdGraphPanelPinFactory>();
		FEdGraphUtilities::RegisterVisualNodeFactory(EdGraphPanelNodeFactory);
		FEdGraphUtilities::RegisterVisualPinFactory(EdGraphPanelPinFactory);

		ReconstructAllNodesDelegateHandle = FBlueprintEditorUtils::OnReconstructAllNodesEvent.AddStatic(&FRigVMBlueprintUtils::HandleReconstructAllBlueprintNodes);
		RefreshAllNodesDelegateHandle = FBlueprintEditorUtils::OnRefreshAllNodesEvent.AddStatic(&FRigVMBlueprintUtils::HandleRefreshAllBlueprintNodes);

#if WITH_RIGVMLEGACYEDITOR
		// Register Blueprint editor variable customization
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		BlueprintVariableCustomizationHandle = BlueprintEditorModule.RegisterVariableCustomization(FProperty::StaticClass(), FOnGetVariableCustomizationInstance::CreateStatic(&FRigVMVariableDetailCustomization::MakeLegacyInstance));
#endif

		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (ensure(AssetRegistry))
		{
			AssetRegistry->OnAssetRemoved().AddStatic(&FRigVMBlueprintUtils::HandleAssetDeleted);
		}

		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu"))
		{
			if (FToolMenuSection* Section = Menu->FindSection("CommonAssetActions"))
			{
				Section->AddDynamicEntry("CreateVariant", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
					if (Context)
					{
						if(CVarRigVMEnableVariants.GetValueOnAnyThread())
						{
							TArray<FAssetData> SelectedAssets = Context->SelectedAssets;
							for (FAssetData SelectedAssetData : SelectedAssets)
							{
								if (!SelectedAssetData.GetClass()->ImplementsInterface(URigVMAssetInterface::StaticClass()) &&
									!SelectedAssetData.GetClass()->ImplementsInterface(URigVMRuntimeAssetInterface::StaticClass()))
								{
									// We aren't dealing with a RigVMBlueprint derived type
									return;
								}
							}

							InSection.AddMenuEntry("CreateVariant", LOCTEXT("CreateVariant", "Create variant"), LOCTEXT("CreateVariant_ToolTip", "Create a variant for this asset"), FSlateIcon(FRigVMEditorStyle::Get().GetStyleSetName(), "RigVM", "RigVM.Unit"), FExecuteAction::CreateLambda([SelectedAssets]()
							{
								for (FAssetData SelectedAssetData : SelectedAssets)
								{
									FSoftObjectPath SoftObjectPath = SelectedAssetData.GetSoftObjectPath();
									// Perform the load from within our lambda since this can be expensive, and should not be done speculatively
									UObject* SelectedObject = SoftObjectPath.TryLoad();
									if (!SelectedObject)
									{
										return;
									}

									const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
									FString PathName = SoftObjectPath.GetLongPackageName();
									FString ObjectName = SelectedObject->GetName();
									FString PackageName;
									AssetToolsModule.Get().CreateUniqueAssetName(PathName, TEXT(""), PackageName, ObjectName);

									FString Extension;
									FPaths::Split(PackageName, PathName, ObjectName, Extension);

									UObject* DuplicateAsset = AssetToolsModule.Get().DuplicateAsset(ObjectName, PathName, SelectedObject);
									if (IRigVMEditorAssetInterface* DuplicateBlueprint = Cast<IRigVMEditorAssetInterface>(DuplicateAsset))
									{
										DuplicateBlueprint->GetAssetVariant() = Cast<IRigVMEditorAssetInterface>(SelectedObject)->GetAssetVariant();
									}
									else if (IRigVMRuntimeAssetInterface* DuplicateRuntimeBlueprint = Cast<IRigVMRuntimeAssetInterface>(DuplicateAsset))
									{
										DuplicateRuntimeBlueprint->GetAssetVariant() = Cast<IRigVMRuntimeAssetInterface>(SelectedObject)->GetAssetVariant();
									}
								}
							}));
							
						}
					}
				}));
			}
		}

#if RIGVM_TRACE_ENABLED
		IModularFeatures::Get().RegisterModularFeature( IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerForRigVM);
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &RigVMTraceModule);
		IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GRewindDebuggerRigVMTrackCreator);
#endif

		// Register asset type actions for EditorAsset redirection
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		TSharedRef<IAssetTypeActions> EditorAssetActions = MakeShareable(new FRigVMEditorAssetActions());
		RegisteredAssetTypeActions.Add(EditorAssetActions);
		AssetTools.RegisterAssetTypeActions(EditorAssetActions);
	}

	StartupModuleCommon();
}

void FRigVMEditorModule::StartupModuleCommon()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
    ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	const UClass* EdGraphSchemaClass = GetRigVMEdGraphSchemaClass();
	const URigVMEdGraphSchema* SchemaCDO = CastChecked<URigVMEdGraphSchema>(EdGraphSchemaClass->GetDefaultObject());
#if WITH_RIGVMLEGACYEDITOR
	const UClass* BlueprintClass = GetRigVMLegacyBlueprintClass();
	FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
	BlueprintEditorModule.RegisterGraphCustomization(SchemaCDO, FOnGetGraphCustomizationInstance::CreateStatic(&FRigVMGraphDetailCustomization::MakeLegacyInstance, BlueprintClass));
#endif

	// Register to fixup newly created BPs
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, URigVMHost::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FRigVMEditorModule::HandleNewBlueprintCreated));
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
    PropertiesToUnregisterOnShutdown.Reset();

	PropertiesToUnregisterOnShutdown.Add(FRigVMVariant::StaticStruct()->GetFName());
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(PropertiesToUnregisterOnShutdown.Last(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRigVMVariantDetailCustomization::MakeInstance));
	
	PropertiesToUnregisterOnShutdown.Add(FRigVMPythonSettings::StaticStruct()->GetFName());
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(PropertiesToUnregisterOnShutdown.Last(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRigVMPythonLogDetails::MakeInstance));

	AssetsPreDeleteDelegate = FEditorDelegates::OnAssetsPreDelete.AddStatic(&FRigVMEditorModule::HandleAssetsPreDelete);
}

void FRigVMEditorModule::ShutdownModule()
{
	if (!IsEngineExitRequested())
	{
		if(IsRigVMEditorModuleBase())
		{
			FRigVMEditorStyle::Unregister();
			FEdGraphUtilities::UnregisterVisualNodeFactory(EdGraphPanelNodeFactory);
			FEdGraphUtilities::UnregisterVisualPinFactory(EdGraphPanelPinFactory);
			FRigVMEditorGraphExplorerCommands::Unregister();

			FBlueprintEditorUtils::OnRefreshAllNodesEvent.Remove(RefreshAllNodesDelegateHandle);
			FBlueprintEditorUtils::OnReconstructAllNodesEvent.Remove(ReconstructAllNodesDelegateHandle);

			// Unregister Blueprint editor variable customization
#if WITH_RIGVMLEGACYEDITOR
			FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
			BlueprintEditorModule.UnregisterVariableCustomization(FProperty::StaticClass(), BlueprintVariableCustomizationHandle);
#endif

#if RIGVM_TRACE_ENABLED
			IModularFeatures::Get().UnregisterModularFeature( IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerForRigVM);
			IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &RigVMTraceModule);
			IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &GRewindDebuggerRigVMTrackCreator);
#endif

			// Unregister asset type actions
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			for (const TSharedRef<IAssetTypeActions>& Action : RegisteredAssetTypeActions)
			{
				AssetTools.UnregisterAssetTypeActions(Action);
			}
			RegisteredAssetTypeActions.Empty();
		}
	}

	ShutdownModuleCommon();
}

void FRigVMEditorModule::ShutdownModuleCommon()
{
	if (!IsEngineExitRequested())
	{
		const UClass* EdGraphSchemaClass = GetRigVMEdGraphSchemaClass();
		const URigVMEdGraphSchema* SchemaCDO = CastChecked<URigVMEdGraphSchema>(EdGraphSchemaClass->GetDefaultObject());
#if WITH_RIGVMLEGACYEDITOR
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		BlueprintEditorModule.UnregisterGraphCustomization(SchemaCDO);
#endif
	}
	
	// Clear all instances in the class pools of the details wrapper objects
	URigVMDetailsViewWrapperObject::ClearAllClassPools();

	// Unregister to fixup newly created BPs
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);
	
	FEditorDelegates::OnAssetsPreDelete.Remove(AssetsPreDeleteDelegate);
}

void FRigVMEditorModule::GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar)
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the class (so if the class 
	// type disappears, then the action should go with it)
	UClass* ActionKey = RigVMBlueprint->GetObject()->GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (!ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		return;
	}

	TArray<TSharedRef<URigVMEdGraphNodeSpawner>> NodeSpawners;
	GetTypeActions(RigVMBlueprint, NodeSpawners);

	for (TSharedRef<URigVMEdGraphNodeSpawner>& NodeSpawner : NodeSpawners)
	{
		URigVMEdGraphNodeBlueprintSpawner* BlueprintSpawner = URigVMEdGraphNodeBlueprintSpawner::CreateFromRigVMSpawner(NodeSpawner);
		ActionRegistrar.AddBlueprintAction(ActionKey, BlueprintSpawner);
	}
}

void FRigVMEditorModule::GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners)
{
	// Only register actions for ourselves
	UClass* InClass = RigVMBlueprint.GetObject()->GetClass();
	if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = RigVMBlueprint->GetRuntimeAssetInterface())
	{
		InClass = RuntimeAsset.GetObject()->GetClass();
	}
	UClass* BlueprintClass = GetRigVMAssetClass();
	if (RigVMBlueprint->IsBlueprintAsset() && CVarRigVMBlueprintIndependentAssets->GetBool())
	{
		BlueprintClass = GetRigVMLegacyBlueprintClass();
	}

	if (BlueprintClass->GetDefaultObject()->IsA<UInterface>())
	{
		if (!InClass->ImplementsInterface(BlueprintClass))
		{
			return;
		}
	}
	else if(InClass != BlueprintClass)
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the class (so if the class 
	// type disappears, then the action should go with it)
	UClass* ActionKey = RigVMBlueprint->GetObject()->GetClass();

	FRigVMRegistry& Registry = FRigVMRegistry::Get();
	Registry.RefreshEngineTypes();

	for (const FRigVMTemplate& Template : Registry.GetTemplates())
	{
		// factories are registered below
		if(Template.UsesDispatch())
		{
			continue;
		}

		// ignore templates that have only one permutation
		if (Template.NumPermutations() <= 1)
		{
			continue;
		}

		// ignore templates which don't have a function backing it up
		if(Template.GetPermutation(0) == nullptr)
		{
			continue;
		}

		if(!Template.SupportsExecuteContextStruct(GetRigVMExecuteContextStruct()))
		{
			continue;
		}

		ensure (!Template.GetNotation().ToString().Contains(TEXT("AnimNext")));

		FText NodeCategory = FText::FromString(Template.GetCategory());
		FText MenuDesc = FText::FromName(Template.GetName());
		FText ToolTip = Template.GetTooltipText();

		TSharedRef<URigVMEdGraphTemplateNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphTemplateNodeSpawner>(Template.GetNotation(), MenuDesc, NodeCategory, ToolTip);
		NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(NodeSpawner);
	};

	for (const FRigVMDispatchFactory* Factory : Registry.GetFactories())
	{
		if(!Factory->SupportsExecuteContextStruct(GetRigVMExecuteContextStruct()))
		{
			continue;
		}

		const FRigVMTemplate* Template = Factory->GetTemplate();
		if(Template == nullptr)
		{
			continue;
		}

		FText NodeCategory = FText::FromString(Factory->GetCategory());
		FText MenuDesc = FText::FromString(Factory->GetNodeTitle(FRigVMTemplateTypeMap()));
		FText ToolTip = Factory->GetNodeTooltip(FRigVMTemplateTypeMap());

		TSharedRef<URigVMEdGraphTemplateNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphTemplateNodeSpawner>(Template->GetNotation(), MenuDesc, NodeCategory, ToolTip);
		NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(NodeSpawner);
	};

	// Add all rig units
	for(const FRigVMFunction& Function : Registry.GetFunctions())
	{
		UScriptStruct* Struct = Function.Struct;
		if (Struct == nullptr || !Struct->IsChildOf(FRigVMStruct::StaticStruct()))
		{
			continue;
		}

		if(!Function.SupportsExecuteContextStruct(GetRigVMExecuteContextStruct()))
		{
			continue;
		}

		// skip rig units which have a template
		if (Function.GetTemplate())
		{
			continue;
		}

		// skip deprecated units
		if(Function.Struct->HasMetaData(FRigVMStruct::DeprecatedMetaName))
		{
			continue;
		}

		FString CategoryMetadata, DisplayNameMetadata, MenuDescSuffixMetadata;
		Struct->GetStringMetaDataHierarchical(FRigVMStruct::CategoryMetaName, &CategoryMetadata);
		Struct->GetStringMetaDataHierarchical(FRigVMStruct::DisplayNameMetaName, &DisplayNameMetadata);
		Struct->GetStringMetaDataHierarchical(FRigVMStruct::MenuDescSuffixMetaName, &MenuDescSuffixMetadata);

		if(DisplayNameMetadata.IsEmpty())
		{
			DisplayNameMetadata = Struct->GetDisplayNameText().ToString();
		}
		if (!MenuDescSuffixMetadata.IsEmpty())
		{
			MenuDescSuffixMetadata = TEXT(" ") + MenuDescSuffixMetadata;
		}
		FText NodeCategory = FText::FromString(CategoryMetadata);
		FText MenuDesc = FText::FromString(DisplayNameMetadata + MenuDescSuffixMetadata);
		FText ToolTip = Struct->GetToolTipText();

		TSharedRef<URigVMEdGraphUnitNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphUnitNodeSpawner>(Struct, Function.GetMethodName(), MenuDesc, NodeCategory, ToolTip);
		NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(NodeSpawner);
	};

	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		UEnum* EnumToConsider = (*EnumIt);

		if (EnumToConsider->HasMetaData(TEXT("Hidden")))
		{
			continue;
		}

		if (EnumToConsider->IsEditorOnly())
		{
			continue;
		}

		if(EnumToConsider->IsNative())
		{
			continue;
		}

		FText NodeCategory = FText::FromString(TEXT("Enum"));
		FText MenuDesc = FText::FromString(FString::Printf(TEXT("Enum %s"), *EnumToConsider->GetName()));
		FText ToolTip = MenuDesc;

		TSharedRef<URigVMEdGraphEnumNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphEnumNodeSpawner>(EnumToConsider, MenuDesc, NodeCategory, ToolTip);
		NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(NodeSpawner);
	}

	FArrayProperty* PublicGraphFunctionsProperty = CastField<FArrayProperty>(URigVMBlueprint::StaticClass()->FindPropertyByName(TEXT("PublicGraphFunctions")));
	FArrayProperty* PublicFunctionsProperty = CastField<FArrayProperty>(URigVMBlueprint::StaticClass()->FindPropertyByName(TEXT("PublicFunctions")));
	if(PublicGraphFunctionsProperty || PublicFunctionsProperty)
	{
		// find all control rigs in the project
		TMap<FRigVMGraphFunctionIdentifier, FAssetData> ControlRigAssetDatas;
		const TArray<FRigVMGraphFunctionIdentifier> FunctionIds = URigVMBuildData::Get()->GetAllFunctionIdentifiers();
		for (const FRigVMGraphFunctionIdentifier& FunctionId : FunctionIds)
		{
			// Starting from a library node path
			const FString LibraryNodePath = FunctionId.GetLibraryNodePath();

			FAssetData AssetData = FunctionId.GetAssetData();
			if (AssetData.IsValid())  // Found in registry = exists on disk
			{
				ControlRigAssetDatas.FindOrAdd(FunctionId) = AssetData;
			}
		}
		
		
		FARFilter ControlRigAssetFilter;

		auto AddClassToFilter = [&ControlRigAssetFilter](const UClass* InClass)
		{
			const UClass* Class = InClass;
			while(Class)
			{
				ControlRigAssetFilter.ClassPaths.Add(Class->GetClassPathName());
				if (Class == URigVMBlueprint::StaticClass())
				{
					ControlRigAssetFilter.ClassPaths.Add(URigVMRuntimeAsset::StaticClass()->GetClassPathName());
				}
				else if (Class == URigVMRuntimeAsset::StaticClass())
				{
					ControlRigAssetFilter.ClassPaths.Add(URigVMBlueprint::StaticClass()->GetClassPathName());
				}
				else if (Class->GetClassPathName() == FTopLevelAssetPath(TEXT("/Script/ControlRig"), TEXT("ControlRigRuntimeAsset")))
				{
					ControlRigAssetFilter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/ControlRigDeveloper"), TEXT("ControlRigBlueprint")));
				}
				else if (Class->GetClassPathName() == FTopLevelAssetPath(TEXT("/Script/ControlRigDeveloper"), TEXT("ControlRigBlueprint")))
				{
					ControlRigAssetFilter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/ControlRig"), TEXT("ControlRigRuntimeAsset")));
				}
				Class = Class->GetSuperClass();
				if(Class == nullptr)
				{
					break;
				}
				if(Class == UBlueprint::StaticClass() ||
					Class == UBlueprintGeneratedClass::StaticClass() ||
					Class == UObject::StaticClass())
				{
					break;
				}
			}
		};
		AddClassToFilter(RigVMBlueprint->GetAssetObjectForEditor()->GetClass());
		AddClassToFilter(RigVMBlueprint->GetRigVMGeneratedClassPrototype());

		TArray<FRigVMGraphFunctionHeader> PublicFunctions;
		for(TPair<FRigVMGraphFunctionIdentifier, FAssetData> Pair : ControlRigAssetDatas)
		{
			if (!ControlRigAssetFilter.ClassPaths.Contains(Pair.Value.AssetClassPath))
			{
				continue;
			}

			if (!AssetsPublicFunctionsAllowed(Pair.Value))
			{
				continue;
			}

			const FRigVMGraphFunctionHeader& Header = URigVMBuildData::Get()->FindFunctionHeader(Pair.Key);
			if (Header.IsValid())
			{
				PublicFunctions.Add(Header);
			}
		}
		for(FRigVMGraphFunctionHeader& PublicFunction : PublicFunctions)
		{
			if (PublicFunction.LibraryPointer.IsValid())
			{
				TSharedRef<URigVMEdGraphFunctionRefNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphFunctionRefNodeSpawner>(ControlRigAssetDatas.FindChecked(PublicFunction.LibraryPointer), PublicFunction);
				NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(NodeSpawner);
			}
		}
	}
}

void FRigVMEditorModule::GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar)
{
	if (UClass* GeneratedClass = Cast<UClass>(RigVMBlueprint->GetRuntimeAssetInterface().GetObject()))
	{
		if (!ActionRegistrar.IsOpenForRegistration(GeneratedClass))
		{
			return;
		}

		TArray<TSharedRef<URigVMEdGraphNodeSpawner>> NodeSpawners;
		GetInstanceActions(RigVMBlueprint, NodeSpawners);

		for (TSharedRef<URigVMEdGraphNodeSpawner>& NodeSpawner : NodeSpawners)
		{
			URigVMEdGraphNodeBlueprintSpawner* BlueprintNodeSpawner = URigVMEdGraphNodeBlueprintSpawner::CreateFromRigVMSpawner(NodeSpawner);
			ActionRegistrar.AddBlueprintAction(GeneratedClass, BlueprintNodeSpawner);
		}
	}
}

void FRigVMEditorModule::GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners)
{
	// Only register actions for ourselves
	UClass* InClass = RigVMBlueprint.GetObject()->GetClass();
	if (!RigVMBlueprint->IsBlueprintAsset())
	{
		if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = RigVMBlueprint->GetRuntimeAssetInterface())
		{
			InClass = RuntimeAsset.GetObject()->GetClass();
		}
	}
	UClass* BlueprintClass = GetRigVMAssetClass();
	if (RigVMBlueprint->IsBlueprintAsset() && CVarRigVMBlueprintIndependentAssets->GetBool())
	{
		BlueprintClass = GetRigVMLegacyBlueprintClass();
	}

	if (BlueprintClass->GetDefaultObject()->IsA<UInterface>())
	{
		if (!InClass->ImplementsInterface(BlueprintClass))
		{
			return;
		}
	}
	else if(InClass != BlueprintClass)
	{
		return;
	}

	static const FText NodeCategory = LOCTEXT("Variables", "Variables");

	TArray<FRigVMGraphVariableDescription> Variables = RigVMBlueprint->GetAssetVariables();
	for (const FRigVMGraphVariableDescription& Variable : Variables)
	{
		const FRigVMExternalVariable ExternalVariable = Variable.ToExternalVariable();
		FText MenuDesc = FText::FromName(ExternalVariable.GetName());
		FText ToolTip = FText::FromString(FString::Printf(TEXT("Get the value of variable %s"), *ExternalVariable.GetName().ToString()));
		TSharedRef<URigVMEdGraphVariableNodeSpawner> GetNodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, ExternalVariable, true, MenuDesc, NodeCategory, ToolTip);
		GetNodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(GetNodeSpawner);

		ToolTip = FText::FromString(FString::Printf(TEXT("Set the value of variable %s"), *ExternalVariable.GetName().ToString()));
		TSharedRef<URigVMEdGraphVariableNodeSpawner> SetNodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, ExternalVariable, false, MenuDesc, NodeCategory, ToolTip);
		SetNodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
		OutGraphNodeSpawners.Add(SetNodeSpawner);
	}

	if (URigVMFunctionLibrary* LocalFunctionLibrary = RigVMBlueprint->GetLocalFunctionLibrary())
	{
		TArray<URigVMLibraryNode*> Functions = LocalFunctionLibrary->GetFunctions();
		const FSoftObjectPath LocalLibrarySoftPath = LocalFunctionLibrary->GetFunctionHostObjectPath();
		for (URigVMLibraryNode* Function : Functions)
		{
			// Avoid adding functions that are already added by the GetTypeActions functions (public functions that are already saved into the blueprint tag)
			if (RigVMBlueprint->GetPublicGraphFunctions().ContainsByPredicate([LocalLibrarySoftPath, Function](const FRigVMGraphFunctionHeader& Header) -> bool
			{
				return FRigVMGraphFunctionIdentifier(LocalLibrarySoftPath, Function->GetPathName()) == Header.LibraryPointer;
			}))
			{
				continue;
			}
			
			TSharedRef<URigVMEdGraphFunctionRefNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphFunctionRefNodeSpawner>(Function);
			NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
			OutGraphNodeSpawners.Add(NodeSpawner);
		}

		static const FText LocalNodeCategory = LOCTEXT("LocalVariables", "Local Variables");
		for (URigVMLibraryNode* Function : Functions)
		{
			for (const FRigVMGraphVariableDescription& LocalVariable : Function->GetContainedGraph()->GetLocalVariables())
			{
				FText MenuDesc = FText::FromName(LocalVariable.Name);
				FText ToolTip = FText::FromString(FString::Printf(TEXT("Get the value of variable %s"), *LocalVariable.Name.ToString()));
				TSharedRef<URigVMEdGraphVariableNodeSpawner> GetNodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, Function->GetContainedGraph(), LocalVariable, true, MenuDesc, LocalNodeCategory, ToolTip);
				GetNodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(GetNodeSpawner);

				ToolTip = FText::FromString(FString::Printf(TEXT("Set the value of variable %s"), *LocalVariable.Name.ToString()));
				TSharedRef<URigVMEdGraphVariableNodeSpawner> SetNodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, Function->GetContainedGraph(), LocalVariable, false, MenuDesc, LocalNodeCategory, ToolTip);
				SetNodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(SetNodeSpawner);
			}
		}
	}

	for (URigVMGraph* Graph : RigVMBlueprint->GetAllModels())
	{
		if (Graph->GetEntryNode())
		{
			static const FText InputNodeCategory = LOCTEXT("InputArguments", "Input Arguments");
			for (const FRigVMGraphVariableDescription& InputArgument : Graph->GetInputArguments())
			{
				FText MenuDesc = FText::FromName(InputArgument.Name);
				FText ToolTip = FText::FromString(FString::Printf(TEXT("Get the value of input %s"), *InputArgument.Name.ToString()));
				TSharedRef<URigVMEdGraphVariableNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, Graph, InputArgument, true, MenuDesc, InputNodeCategory, ToolTip);
				NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(NodeSpawner);
			}
			
			static const FText InputVariableCategory = LOCTEXT("InputVariables", "Input Variables");
			for (const FRigVMGraphVariableDescription& InputVariable : Graph->GetInputVariables())
			{
				FText MenuDesc = FText::FromName(InputVariable.Name);
				FText ToolTip = FText::FromString(FString::Printf(TEXT("Get the value of input variable %s"), *InputVariable.Name.ToString()));
				TSharedRef<URigVMEdGraphVariableNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, Graph, InputVariable, true, MenuDesc, InputVariableCategory, ToolTip);
				NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(NodeSpawner);

				ToolTip = FText::FromString(FString::Printf(TEXT("Set the value of input variable %s"), *InputVariable.Name.ToString()));
				NodeSpawner = MakeShared<URigVMEdGraphVariableNodeSpawner>(RigVMBlueprint, Graph, InputVariable, false, MenuDesc, InputVariableCategory, ToolTip);
				NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
				OutGraphNodeSpawners.Add(NodeSpawner);
			}			
		}
	}

	const TArray<FName> EntryNames = RigVMBlueprint->GetRigVMClient()->GetEntryNames();
	if(!EntryNames.IsEmpty())
	{
		static const FText EventNodeCategory = LOCTEXT("Events", "Events");
		for (const FName& EntryName : EntryNames)
		{
			static constexpr TCHAR EventStr[] = TEXT("Event");
			static const FString EventSuffix = FString::Printf(TEXT(" %s"), EventStr);
			FString Suffix = EntryName.ToString().EndsWith(EventStr) ? FString() : EventSuffix;
			FText MenuDesc = FText::FromString(FString::Printf(TEXT("Run %s%s"), *EntryName.ToString(), *Suffix));
			FText ToolTip = FText::FromString(FString::Printf(TEXT("Runs the %s%s"), *EntryName.ToString(), *Suffix));
			TSharedRef<URigVMEdGraphInvokeEntryNodeSpawner> NodeSpawner = MakeShared<URigVMEdGraphInvokeEntryNodeSpawner>(RigVMBlueprint, EntryName, MenuDesc, EventNodeCategory, ToolTip);
			NodeSpawner->SetRelatedBlueprintClass(BlueprintClass);
			OutGraphNodeSpawners.Add(NodeSpawner);
		}
	}
}

void FRigVMEditorModule::GetNodeContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	// Only register menu actions for ourselves if we are a blueprint
	if(FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject())
	{
		UClass* BPClass = RigVMBlueprint->GetObject()->GetClass();
		UClass* BaseClass = GetRigVMAssetClass();
		if (RigVMBlueprint->IsBlueprintAsset() && CVarRigVMBlueprintIndependentAssets->GetBool())
		{
			BaseClass = GetRigVMLegacyBlueprintClass();
		}
		else if (BaseClass->IsChildOf(URigVMRuntimeAsset::StaticClass()))
		{
			BPClass = RigVMBlueprint->GetRuntimeAssetInterface().GetObject()->GetClass();
		}
		if(BPClass != BaseClass && !BPClass->IsChildOf(BaseClass))
		{
			return;
		}
	}

	GetNodeActionsContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeWorkflowContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeEventsContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeDefaultValueContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeConversionContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeDebugContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeVariablesContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeTemplatesContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeOrganizationContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeVariantContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeTestContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
	GetNodeDisplayContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
}

void FRigVMEditorModule::GetNodeActionsContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	const TScriptInterface<IRigVMEditorAssetInterface> RigVMAsset = RigVMClientHost.GetObject();
	if (!EdGraphNode || 
		!ModelNode || 
		!Menu || 
		!RigVMAsset)
	{
		return;
	}

	const TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakRigVMAsset = RigVMAsset;
	const TWeakObjectPtr<URigVMNode> WeakModelNode = ModelNode;
	const TWeakObjectPtr<const URigVMEdGraphNode> WeakEdGraphNode = EdGraphNode;

	const TSharedRef<FUICommandList> FindNodeFreferencesCommandList = MakeShared<FUICommandList>();

	FindNodeFreferencesCommandList->MapAction(
		FRigVMEditorCommands::Get().FindReferences,
		FExecuteAction::CreateLambda(
			[WeakRigVMAsset, WeakEdGraphNode]()
			{
				constexpr bool bSearchInAllBlueprints = false;

				const UE::RigVMEditor::FRigVMEditorFindNodeReferencesParams FindNodeReferencesParams(
					WeakRigVMAsset,
					WeakEdGraphNode,
					bSearchInAllBlueprints
				);

				OnRequestFindNodeReferences.Broadcast(FindNodeReferencesParams);
			})
	);

	FindNodeFreferencesCommandList->MapAction(
		FRigVMEditorCommands::Get().FindReferencesByNameLocal,
		FExecuteAction::CreateLambda(
			[WeakRigVMAsset, WeakEdGraphNode]()
			{
				constexpr bool bSearchInAllBlueprints = false;

				const UE::RigVMEditor::FRigVMEditorFindNodeReferencesParams FindNodeReferencesParams(
					WeakRigVMAsset,
					WeakEdGraphNode,
					bSearchInAllBlueprints
				);

				OnRequestFindNodeReferences.Broadcast(FindNodeReferencesParams);
			})
	);


	FindNodeFreferencesCommandList->MapAction(
		FRigVMEditorCommands::Get().FindReferencesByNameGlobal,
		FExecuteAction::CreateLambda(
			[WeakRigVMAsset, WeakEdGraphNode]()
			{
				constexpr bool bSearchInAllBlueprints = true;

				const UE::RigVMEditor::FRigVMEditorFindNodeReferencesParams FindNodeReferencesParams(
					WeakRigVMAsset,
					WeakEdGraphNode,
					bSearchInAllBlueprints
				);

				OnRequestFindNodeReferences.Broadcast(FindNodeReferencesParams);
			})
	);

	// This section displays actions that are to be found under 'Node Actions' in other blueprint editors as well
	FToolMenuSection& NodeActionsSection = Menu->AddSection(
		"RigVMEditorContextMenuNodeActions",
		LOCTEXT("NodeActionsHeader", "Node Actions"));

	const bool bShowSubMenu = [EdGraphNode]()
		{
			if (const URigVMNode* Node = EdGraphNode->GetModelNode())
			{
				return
					Node->IsA(URigVMVariableNode::StaticClass()) ||
					Node->IsA(URigVMUnitNode::StaticClass()) ||
					Node->IsA(URigVMFunctionReferenceNode::StaticClass());
			}

			return false;
		}();

	const FText FindReferencesLabel = LOCTEXT("FindReferencesLabel", "Find References");
	const FText FindReferencesTooltip = LOCTEXT("FindReferencesTooltip", "Options for finding references to class members");

	if (bShowSubMenu)
	{
		NodeActionsSection.AddSubMenu(
			"FindReferenceSubMenu",
			FindReferencesLabel,
			FindReferencesTooltip,
			FNewToolMenuDelegate::CreateLambda([FindNodeFreferencesCommandList](UToolMenu* InMenu)
				{
					if (InMenu)
					{
						FToolMenuSection& FindReferencesSection = InMenu->AddSection(
							"FindReferencesSection",
							FText::GetEmpty());

						FindReferencesSection.AddMenuEntry(FRigVMEditorCommands::Get().FindReferencesByNameLocal)
							.SetCommandList(FindNodeFreferencesCommandList);

						FindReferencesSection.AddMenuEntry(FRigVMEditorCommands::Get().FindReferencesByNameGlobal)
							.SetCommandList(FindNodeFreferencesCommandList);
					}
				})
		);		
	}
	else
	{
		NodeActionsSection.AddMenuEntry(
			FRigVMEditorCommands::Get().FindReferences,
			FindReferencesLabel,
			FindReferencesTooltip)
			.SetCommandList(FindNodeFreferencesCommandList);
	}

	NodeActionsSection.AddMenuEntry(FGenericCommands::Get().Cut);
	NodeActionsSection.AddMenuEntry(FGenericCommands::Get().Copy);
	NodeActionsSection.AddMenuEntry(FGenericCommands::Get().Paste);
	NodeActionsSection.AddMenuEntry(FGenericCommands::Get().Delete);
}

void FRigVMEditorModule::GetNodeWorkflowContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	const TArray<FRigVMUserWorkflow> Workflows = ModelNode->GetSupportedWorkflows(ERigVMUserWorkflowType::NodeContext, ModelNode);
	if(!Workflows.IsEmpty())
	{
		FToolMenuSection& SettingsSection = Menu->AddSection("RigVMEditorContextMenuWorkflow", LOCTEXT("WorkflowHeader", "Workflow"));
		URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelNode->GetGraph());

		for(const FRigVMUserWorkflow& Workflow : Workflows)
		{
			SettingsSection.AddMenuEntry(
				*Workflow.GetTitle(),
				FText::FromString(Workflow.GetTitle()),
				FText::FromString(Workflow.GetTooltip()),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, Controller, Workflow, ModelNode]()
				{
					URigVMUserWorkflowOptions* Options = Controller->MakeOptionsForWorkflow(ModelNode, Workflow);

					bool bPerform = true;
					if(Options->RequiresDialog())
					{
						bPerform = ShowWorkflowOptionsDialog(Options);
					}
					if(bPerform)
					{
						Controller->PerformUserWorkflow(Workflow, Options);
					}
				}))
			);
		}
	}
}

void FRigVMEditorModule::GetNodeEventsContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject();
	if(RigVMBlueprint && ModelNode->IsEvent())
	{
		const FName& EventName = ModelNode->GetEventName();
		const bool bCanRunOnce = !CastChecked<URigVMEdGraphSchema>(EdGraphNode->GetSchema())->IsRigVMDefaultEvent(EventName);

		FToolMenuSection& EventsSection = Menu->AddSection("RigVMEditorContextMenuEvents", LOCTEXT("EventsHeader", "Events"));

		EventsSection.AddMenuEntry(
			"Switch to Event",
			LOCTEXT("SwitchToEvent", "Switch to Event"),
			LOCTEXT("SwitchToEvent_Tooltip", "Switches the Control Rig Editor to run this event permanently."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([RigVMBlueprint, EventName]()
			{
				if(URigVMHost* Host = Cast<URigVMHost>(RigVMBlueprint->GetObjectBeingDebugged()))
				{
					Host->SetEventQueue({EventName});
				}
			}))
		);

		EventsSection.AddMenuEntry(
			"Run Event Once",
			LOCTEXT("RuntEventOnce", "Run Event Once"),
			LOCTEXT("RuntEventOnce_Tooltip", "Runs the event once (for testing)"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([RigVMBlueprint, EventName]()
			{
				if(URigVMHost* Host = Cast<URigVMHost>(RigVMBlueprint->GetObjectBeingDebugged()))
				{
					const TArray<FName> PreviousEventQueue = Host->GetEventQueue();
					TArray<FName> NewEventQueue = PreviousEventQueue;
					NewEventQueue.Add(EventName);
					Host->SetEventQueue(NewEventQueue);

					TSharedPtr<FDelegateHandle> RunOnceHandle = MakeShareable(new FDelegateHandle);
					*(RunOnceHandle.Get()) = Host->OnExecuted_AnyThread().AddLambda(
						[RunOnceHandle, EventName, PreviousEventQueue](URigVMHost* InRig, const FName& InEventName)
						{
							if(InEventName == EventName)
							{
								InRig->SetEventQueue(PreviousEventQueue);
								InRig->OnExecuted_AnyThread().Remove(*RunOnceHandle.Get());
							}
						}
					);
				}
			}),
			FCanExecuteAction::CreateLambda([bCanRunOnce](){ return bCanRunOnce; }))
		);
	}
}

void FRigVMEditorModule::GetNodeDefaultValueContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	if(!CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		return;
	}
	
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelNode->GetGraph());
	FToolMenuSection& DefaultValuesSection = Menu->AddSection("RigVMEditorContextMenuDefaultValues", LOCTEXT("DefaultValuesHeader", "Pin Values"));

	DefaultValuesSection.AddMenuEntry(
		"Override All",
		LOCTEXT("OverrideAll", "Override All"),
		LOCTEXT("OverrideAll_Tooltip", "Overrides all values"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Controller]()
		{
			const TArray<FName> SelectedNodeNames = Controller->GetGraph()->GetSelectNodes();
			if(!SelectedNodeNames.IsEmpty())
			{
				FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Override);
				Controller->OpenUndoBracket(TEXT("Override all pin values"));
				for(const FName& SelectedNodeName : SelectedNodeNames)
				{
					if(const URigVMNode* Node = Controller->GetGraph()->FindNodeByName(SelectedNodeName))
					{
						for(const URigVMPin* Pin : Node->GetPins())
						{
							if(Pin->CanProvideDefaultValue())
							{
								FString DefaultValue = Pin->GetDefaultValue();
								if(DefaultValue.IsEmpty())
								{
									DefaultValue = Pin->GetOriginalDefaultValue();
								}
								if(!DefaultValue.IsEmpty())
								{
									Controller->SetPinDefaultValue(Pin->GetPinPath(), DefaultValue);
								}
							}
						}
					}
				}
				Controller->CloseUndoBracket();
			}
		}))
	);

	DefaultValuesSection.AddMenuEntry(
		"Reset Unchanged",
		LOCTEXT("ResetUnchanged", "Reset Unchanged"),
		LOCTEXT("ResetUnchanged_Tooltip", "Resets pin values that match the default"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Controller]()
		{
			const TArray<FName> SelectedNodeNames = Controller->GetGraph()->GetSelectNodes();
			if(!SelectedNodeNames.IsEmpty())
			{
				FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Unset);
				Controller->OpenUndoBracket(TEXT("Reset unchanged pin values"));
				for(const FName& SelectedNodeName : SelectedNodeNames)
				{
					if(const URigVMNode* Node = Controller->GetGraph()->FindNodeByName(SelectedNodeName))
					{
						for(const URigVMPin* Pin : Node->GetPins())
						{
							if(Pin->CanProvideDefaultValue())
							{
								FString DefaultValue = Pin->GetDefaultValue();
								const FString OriginalDefaultValue = Pin->GetOriginalDefaultValue();
								if(!OriginalDefaultValue.IsEmpty())
								{
									if(DefaultValue.IsEmpty())
									{
										DefaultValue = OriginalDefaultValue;
									}
									if(DefaultValue.Equals(OriginalDefaultValue, ESearchCase::CaseSensitive))
									{
										Controller->SetPinDefaultValue(Pin->GetPinPath(), OriginalDefaultValue);
									}
								}
							}
						}
					}
				}
				Controller->CloseUndoBracket();
			}
		}))
	);
}

void FRigVMEditorModule::GetNodeConversionContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	if(URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
	{
		if(DispatchNode->GetFactory()->GetFactoryName() == FRigVMDispatch_Constant().GetFactoryName())
		{
			if(const URigVMPin* ValuePin = DispatchNode->FindPin(TEXT("Value")))
			{
				// if the value pin has only links on the root pin
				if((ValuePin->GetSourceLinks(false).Num() > 0) && (ValuePin->GetTargetLinks(false).Num() > 0))
				{
					URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelNode->GetGraph());

					FToolMenuSection& ConversionSection = Menu->AddSection("RigVMEditorContextMenuConversion", LOCTEXT("ConversionHeader", "Conversion"));
					ConversionSection.AddMenuEntry(
						"Convert Constant to Reroute",
						LOCTEXT("ConvertConstantToReroute", "Convert Constant to Reroute"),
						LOCTEXT("ConvertConstantToReroute_Tooltip", "Converts the Constant node to a to Reroute node and sustains the value"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, ValuePin, DispatchNode]()
						{
							Controller->OpenUndoBracket(TEXT("Replace Constant with Reroute"));
							TArray<URigVMController::FLinkedPath> LinkedPaths = Controller->GetLinkedPaths(DispatchNode);
							Controller->FastBreakLinkedPaths(LinkedPaths, true);
							const FVector2D Position = DispatchNode->GetPosition();
							const FString NodeName = DispatchNode->GetName();
							const FString CPPType = ValuePin->GetCPPType();
							FName CPPTypeObjectPath = NAME_None;
							if(const UObject* CPPTypeObject = ValuePin->GetCPPTypeObject())
							{
								CPPTypeObjectPath = *CPPTypeObject->GetPathName();
							}
							
							Controller->RemoveNode(DispatchNode, true, true);
							Controller->AddFreeRerouteNode(CPPType, CPPTypeObjectPath, false, NAME_None, FString(), Position, NodeName, true);
							Controller->RestoreLinkedPaths(LinkedPaths, URigVMController::FRestoreLinkedPathSettings(), true);
							Controller->CloseUndoBracket();
						}
					)));
				}
			}
		}
	}
}

void FRigVMEditorModule::GetNodeDebugContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	if(FRigVMEditorAssetInterfacePtr RigVMAsset = RigVMClientHost.GetObject())
	{
		FToolMenuSection& DebugSection = Menu->AddSection("RigVMEditorContextMenuDebug", LOCTEXT("DebugHeader", "Debug"));
		bool bAnyNodeIsEarlyExit = false;

		const URigVMGraph* Model = ModelNode->GetGraph();

		TSharedPtr<FUICommandList> PreviewHereCommandList = MakeShared<FUICommandList>();
		
		PreviewHereCommandList->MapAction(
			FRigVMEditorCommands::Get().TogglePreviewHere,
			FExecuteAction::CreateLambda([RigVMAsset, Model]()
			{
				if (const FRigVMClient* Client = RigVMAsset->GetRigVMClient())
				{
					RigVMAsset->TogglePreviewHere(Model);
				}
			}),
			FCanExecuteAction()
		);

		PreviewHereCommandList->MapAction(
			FRigVMEditorCommands::Get().PreviewHereStepForward,
			FExecuteAction::CreateLambda([RigVMAsset]()
			{
				RigVMAsset->PreviewHereStepForward();
			}),
			FCanExecuteAction::CreateLambda([RigVMAsset]() -> bool
			{
				return RigVMAsset->CanPreviewHereStepForward();
			})
		);
		
		if (ModelNode->HasEarlyExitMarker())
		{
			DebugSection.AddMenuEntry(
				FRigVMEditorCommands::Get().TogglePreviewHere,
				LOCTEXT("StopPreviewHere", "Stop Preview"),
				LOCTEXT("StopPreviewHere_Tooltip", "Stops the preview of this node and run the full graph")
			).SetCommandList(PreviewHereCommandList);
			
			DebugSection.AddMenuEntry(FRigVMEditorCommands::Get().PreviewHereStepForward).SetCommandList(PreviewHereCommandList);
		}
		else
		{
			DebugSection.AddMenuEntry(
				FRigVMEditorCommands::Get().TogglePreviewHere,
				LOCTEXT("PreviewHere", "Preview Here"),
				LOCTEXT("PreviewHere_Tooltip", "Run the graph up to this node")
			).SetCommandList(PreviewHereCommandList);
		}
	}
}

void FRigVMEditorModule::GetNodeVariablesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject())
	{
		const URigVMGraph* Model = ModelNode->GetGraph();
		URigVMController* Controller = RigVMBlueprint->GetController(Model);

		if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(EdGraphNode->GetModelNode()))
		{
			FToolMenuSection& VariablesSection = Menu->AddSection("RigVMEditorContextMenuVariables", LOCTEXT("VariablesSettingsHeader", "Variables"));
			VariablesSection.AddMenuEntry(
				"MakePindingsFromVariableNode",
				LOCTEXT("MakeBindingsFromVariableNode", "Make Bindings From Node"),
				LOCTEXT("MakeBindingsFromVariableNode_Tooltip", "Turns the variable node into one ore more variable bindings on the pin(s)"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([RigVMBlueprint, Controller, VariableNode]() {
					Controller->MakeBindingsFromVariableNode(VariableNode->GetFName());
				})
			));
		}

		if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(EdGraphNode->GetModelNode()))
		{
			if(!FunctionReferenceNode->IsFullyRemapped() && FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer.GetNodeSoftPath().ResolveObject())
			{
				FToolMenuSection& VariablesSection = Menu->AddSection("RigVMEditorContextMenuVariables", LOCTEXT("Variables", "Variables"));
				VariablesSection.AddMenuEntry(
					"MakeVariablesFromFunctionReferenceNode",
					LOCTEXT("MakeVariablesFromFunctionReferenceNode", "Create required variables"),
					LOCTEXT("MakeVariablesFromFunctionReferenceNode_Tooltip", "Creates all required variables for this function and binds them"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Controller, FunctionReferenceNode, RigVMBlueprint]() {

						const TArray<FRigVMExternalVariable> ExternalVariables = FunctionReferenceNode->GetExternalVariables();
						if(!ExternalVariables.IsEmpty())
						{
							FScopedTransaction Transaction(LOCTEXT("MakeVariablesFromFunctionReferenceNode", "Create required variables"));
							RigVMBlueprint->GetObject()->Modify();

							if (URigVMLibraryNode* LibraryNode = FunctionReferenceNode->LoadReferencedNode())
							{
								FRigVMAssetInterfacePtr ReferencedBlueprint = IRigVMAssetInterface::GetInterfaceOuter(LibraryNode);
							   // ReferencedBlueprint != RigVMBlueprint - since only FunctionReferenceNodes from other assets have the potential to be unmapped
                        
								for(const FRigVMExternalVariable& ExternalVariable : ExternalVariables)
								{
									URigVMPin* InterfacePin = FunctionReferenceNode->FindRootPinByName(ExternalVariable.GetName());
									if (!InterfacePin)
									{
										continue;
									}
									if (!InterfacePin->IsDefinedAsInputVariable())
									{
										continue;
									}
									
									FString DefaultValue;
									if(ReferencedBlueprint)
									{
										for(const FRigVMGraphVariableDescription& NewVariable : ReferencedBlueprint->GetAssetVariables())
										{
											if(NewVariable.Name == ExternalVariable.GetName())
											{
											   DefaultValue = NewVariable.DefaultValue;
											   break;
											}
										}
									}

									FName NewVariableName = RigVMBlueprint->AddHostMemberVariableFromExternal(ExternalVariable, DefaultValue);
									if (!NewVariableName.IsNone())
									{
										Controller->BindPinToVariable(InterfacePin->GetPinPath(), NewVariableName.ToString());
									}
								}
							}

							RigVMBlueprint->MarkAssetAsModified();
						}
                    
					})
				));
			}
		}
	}
}

void FRigVMEditorModule::GetNodeTemplatesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(EdGraphNode->GetModelNode()))
	{
		if (!TemplateNode->IsSingleton())
		{
			const URigVMGraph* Model = ModelNode->GetGraph();
			URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(Model);
			
			FToolMenuSection& TemplatesSection = Menu->AddSection("RigVMEditorContextMenuTemplates", LOCTEXT("TemplatesHeader", "Templates"));
			TemplatesSection.AddMenuEntry(
				"Unresolve Template Node",
				LOCTEXT("UnresolveTemplateNode", "Unresolve Template Node"),
				LOCTEXT("UnresolveTemplateNode_Tooltip", "Removes any type information from the template node"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Controller, Model]() {
					const TArray<FName> Nodes = Model->GetSelectNodes();
					Controller->UnresolveTemplateNodes(Nodes, true, true);
				})
			));
		}
	}
}

void FRigVMEditorModule::GetNodeOrganizationContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject();
	URigVMGraph* Model = ModelNode->GetGraph();
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(Model);

	FToolMenuSection& NavigationSection = Menu->AddSection("RigVMEditorContextMenuNavigation", LOCTEXT("NavigationHeader", "Navigation"));
	FToolMenuSection& OrganizationSection = Menu->AddSection("RigVMEditorContextMenuOrganization", LOCTEXT("OrganizationHeader", "Organization"));

	check(EdGraphNode);
	check(ModelNode);

	if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
	{
		NavigationSection.AddMenuEntry(FGraphEditorCommands::Get().OpenInNewTab);
	}

	if(ModelNode->SupportsRenaming())
	{
		OrganizationSection.AddMenuEntry(
			"Rename",
			LOCTEXT("RenameNode", "Rename Node"),
			LOCTEXT("RenameNode_Tooltip", "Constant nodes can be renamed"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([EdGraphNode]() {
				const_cast<URigVMEdGraphNode*>(EdGraphNode)->RequestRename();
			})
		));
	}
	
	OrganizationSection.AddMenuEntry(
		"Collapse Nodes",
		LOCTEXT("CollapseNodes", "Collapse Nodes"),
		LOCTEXT("CollapseNodes_Tooltip", "Turns the selected nodes into a single Collapse node"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Model, Controller]() {
			TArray<FName> Nodes = Model->GetSelectNodes();
			Controller->CollapseNodes(Nodes, FString(), true, true);
		})
	));
	OrganizationSection.AddMenuEntry(
		"Collapse to Function",
		LOCTEXT("CollapseNodesToFunction", "Collapse to Function"),
		LOCTEXT("CollapseNodesToFunction_Tooltip", "Turns the selected nodes into a new Function"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Model, Controller]() {
			TArray<FName> Nodes = Model->GetSelectNodes();
			Controller->OpenUndoBracket(TEXT("Collapse to Function"));
			URigVMCollapseNode* CollapseNode = Controller->CollapseNodes(Nodes, TEXT("New Function"), true, true);
			if(CollapseNode)
			{
				if (Controller->PromoteCollapseNodeToFunctionReferenceNode(CollapseNode->GetFName(), true, true).IsNone())
				{
					Controller->CancelUndoBracket();
				}
				else
				{
					Controller->CloseUndoBracket();
				}
			}
			else
			{
				Controller->CancelUndoBracket();
			}
		})
	));

	if (Model->AreSectionsEnabled() && !Model->GetSectionsMatchingTheSelection().IsEmpty())
	{
		OrganizationSection.AddMenuEntry(
			"Collapse to Function (All Matches)",
			LOCTEXT("CollapseNodesToFunctionAllMatches", "Collapse to Function (All Matches)"),
			LOCTEXT("CollapseNodesToFunctionAllMatches_Tooltip", "Turns the selected nodes into a new Function while refactoring all matches"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Model, Controller]() {
				// todo: this is doing the same thing as collapse to function right now
				TArray<FName> Nodes = Model->GetSelectNodes();
				Controller->OpenUndoBracket(TEXT("Collapse to Function"));
				URigVMCollapseNode* CollapseNode = Controller->CollapseNodes(Nodes, TEXT("New Function"), true, true);
				if(CollapseNode)
				{
					if (Controller->PromoteCollapseNodeToFunctionReferenceNode(CollapseNode->GetFName(), true, true).IsNone())
					{
						Controller->CancelUndoBracket();
					}
					else
					{
						Controller->CloseUndoBracket();
					}
				}
				else
				{
					Controller->CancelUndoBracket();
				}
			})
		));
	}

	if (URigVMCollapseNode* CollapseNode = Cast<URigVMCollapseNode>(ModelNode))
	{
		OrganizationSection.AddMenuEntry(
			"Promote To Function",
			LOCTEXT("PromoteToFunction", "Promote To Function"),
			LOCTEXT("PromoteToFunction_Tooltip", "Turns the Collapse Node into a Function"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, CollapseNode]() {
				Controller->PromoteCollapseNodeToFunctionReferenceNode(CollapseNode->GetFName(), true, true);
			})
		));
	}

	if (URigVMFunctionInterfaceNode* FunctionInterfaceNode = Cast<URigVMFunctionInterfaceNode>(ModelNode))
	{
		TArray<FName> PinsToRemove;
		for (const URigVMPin* Pin : FunctionInterfaceNode->GetPins())
		{
			if (!FunctionInterfaceNode->IsInterfacePinUsed(Pin->GetFName()))
			{
				if (FunctionInterfaceNode->IsA<URigVMFunctionEntryNode>() && Pin->GetDirection() == ERigVMPinDirection::Output)
				{
					PinsToRemove.Add(Pin->GetFName());
				}
				else if (FunctionInterfaceNode->IsA<URigVMFunctionReturnNode>() && Pin->GetDirection() == ERigVMPinDirection::Input)
				{
					PinsToRemove.Add(Pin->GetFName());
				}
			}
		}

		OrganizationSection.AddMenuEntry(
			"RemoveUnusedExposedPins",
			LOCTEXT("RemoveUnusedExposedPins", "Remove unused pins"),
				LOCTEXT("RemoveUnusedExposedPins_Tooltip", "Removes all unused exposed pins from this node"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, PinsToRemove]() {
				Controller->OpenUndoBracket(TEXT("Remove Unused Exposed Pins"));
				for (const FName& ExposedPinName : PinsToRemove)
				{
					Controller->RemoveExposedPin(ExposedPinName);
				}
				Controller->CloseUndoBracket();
			}),
			FCanExecuteAction::CreateLambda([PinsToRemove]()
			{
				return !PinsToRemove.IsEmpty();
			})
		));
	}

	if(RigVMBlueprint)
	{
		if (URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
		{
			TSoftObjectPtr<URigVMFunctionReferenceNode> RefPtr(FunctionReferenceNode);
			if(RefPtr.GetLongPackageName() != FunctionReferenceNode->GetReferencedFunctionHeader().LibraryPointer.GetNodeSoftPath().GetLongPackageName())
			{
				OrganizationSection.AddMenuEntry(
				   "Localize Function",
				   LOCTEXT("LocalizeFunction", "Localize Function"),
				   LOCTEXT("LocalizeFunction_Tooltip", "Creates a local copy of the function backing the node."),
				   FSlateIcon(),
				   FUIAction(FExecuteAction::CreateLambda([RigVMBlueprint, FunctionReferenceNode]() {
					   RigVMBlueprint->BroadcastRequestLocalizeFunctionDialog(FunctionReferenceNode->GetFunctionIdentifier(), true);
				   })
				));
			}
		}

		if (URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(ModelNode))
		{
			OrganizationSection.AddMenuEntry(
				"Promote To Collapse Node",
				LOCTEXT("PromoteToCollapseNode", "Promote To Collapse Node"),
				LOCTEXT("PromoteToCollapseNode_Tooltip", "Turns the Function Ref Node into a Collapse Node"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Controller, FunctionRefNode]() {
					Controller->PromoteFunctionReferenceNodeToCollapseNode(FunctionRefNode->GetFName());
					})
				));
		}
	}

	if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(ModelNode))
	{
		OrganizationSection.AddMenuEntry(
			"Expand Node",
			LOCTEXT("ExpandNode", "Expand Node"),
			LOCTEXT("ExpandNode_Tooltip", "Expands the contents of the node into this graph"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, LibraryNode, RigVMBlueprint]() {
				Controller->OpenUndoBracket(TEXT("Expand node"));

				URigVMFunctionLibrary* FunctionLibrary = LibraryNode->GetGraph()->GetDefaultFunctionLibrary();
				TGuardValue<FRigVMController_RequestLocalizeFunctionDelegate> RequestLocalizeDelegateGuard(
					Controller->RequestLocalizeFunctionDelegate,
					FRigVMController_RequestLocalizeFunctionDelegate::CreateLambda([RigVMBlueprint, FunctionLibrary](FRigVMGraphFunctionIdentifier& InFunctionToLocalize)
						{
							if (RigVMBlueprint)
							{
								RigVMBlueprint->BroadcastRequestLocalizeFunctionDialog(InFunctionToLocalize, true);
							}
							const URigVMLibraryNode* LocalizedFunctionNode = FunctionLibrary->FindPreviouslyLocalizedFunction(InFunctionToLocalize);
							return LocalizedFunctionNode != nullptr;
						})
				);

				TArray<URigVMNode*> ExpandedNodes = Controller->ExpandLibraryNode(LibraryNode->GetFName(), true, true);
				if (ExpandedNodes.Num() > 0)
				{
					TArray<FName> ExpandedNodeNames;
					for (URigVMNode* ExpandedNode : ExpandedNodes)
					{
						ExpandedNodeNames.Add(ExpandedNode->GetFName());
					}
					Controller->SetNodeSelection(ExpandedNodeNames);
				}
				Controller->CloseUndoBracket();
			})
		));
	}

	if (!Model->AreSectionsEnabled())
	{
		OrganizationSection.AddMenuEntry(
			"Highlight Occurrences",
			LOCTEXT("HighlightOccurrences", "Highlight Occurrences"),
			LOCTEXT("HighlightOccurrences_Tooltip", "Highlights all occurrences of the selected section in this graph"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Model]() {
				Model->UpdateSections(true /* force */);
			})
		));
	}

	OrganizationSection.AddMenuEntry(
		"Select Occurrences",
		LOCTEXT("SelectOccurrences", "Select Occurrences"),
		LOCTEXT("SelectOccurrences_Tooltip", "Selects all occurrences of the selected sections in this graph"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Model, Controller]() {

			const TArray<FName>& SelectedNodes = Model->GetSelectNodes();
			if (SelectedNodes.IsEmpty())
			{
				return;
			}

			const TArray<FRigVMGraphSection> SelectedSections = FRigVMGraphSection::GetSectionsFromSelection(Model);
			TArray<FRigVMGraphSection> AllSections = SelectedSections;

			TArray<URigVMNode*> AvailableNodes = Model->GetNodes();
			AvailableNodes.RemoveAll([SelectedNodes]( const URigVMNode* Node) -> bool
			{
				return SelectedNodes.Contains(Node->GetFName());
			});

			for (const FRigVMGraphSection& SelectedSection : SelectedSections)
			{
				AllSections.Append(SelectedSection.FindMatches(AvailableNodes));
			}

			TArray<FName> SimilarNodes;
			for (const FRigVMGraphSection& Section : AllSections)
			{
				SimilarNodes.Append(Section.Nodes);
			}

			Controller->SetNodeSelection(SimilarNodes, true);
		})
	));

	OrganizationSection.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
	{
		{
			FToolMenuSection& InSection = AlignmentMenu->AddSection("RigVMEditorContextMenuAlignment", LOCTEXT("AlignHeader", "Align"));
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
		}

		{
			FToolMenuSection& InSection = AlignmentMenu->AddSection("RigVMEditorContextMenuDistribution", LOCTEXT("DistributionHeader", "Distribution"));
			InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
			InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
		}
	}));
}

void FRigVMEditorModule::GetNodeVariantContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	const URigVMGraph* Model = ModelNode->GetGraph();
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(Model);

	bool bCanNodeBeUpgraded = false;
	const bool bIsFunctionReference = ModelNode->IsA<URigVMFunctionReferenceNode>();
	TArray<FName> SelectedNodeNames = Model->GetSelectNodes();
	SelectedNodeNames.AddUnique(ModelNode->GetFName());

	for(const FName& SelectedNodeName : SelectedNodeNames)
	{
		if (URigVMNode* FoundNode = Model->FindNodeByName(SelectedNodeName))
		{
			bCanNodeBeUpgraded = bCanNodeBeUpgraded || FoundNode->CanBeUpgraded();
		}
	}
	
	if(bCanNodeBeUpgraded || bIsFunctionReference)
	{
		FToolMenuSection& VariantSection = Menu->AddSection("RigVMEditorContextMenuVariant", LOCTEXT("VariantHeader", "Variants"));

		if(bCanNodeBeUpgraded)
		{
			VariantSection.AddMenuEntry(
				"Upgrade Nodes",
				LOCTEXT("UpgradeNodes", "Upgrade Nodes"),
				LOCTEXT("UpgradeNodes_Tooltip", "Upgrades selected outdated nodes to their current implementation"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Model, Controller]() {
					TArray<FName> Nodes = Model->GetSelectNodes();
					Controller->UpgradeNodes(Nodes, true, true);
				})
			));
		}

		if(bIsFunctionReference)
		{
			VariantSection.AddMenuEntry(
				"Swap function for selected nodes",
				LOCTEXT("SwapSelectedFunction", "Swap function for selected nodes"),
				LOCTEXT("SwapSelectedFunction_Tooltip", "Swaps this function for another one for all nodes matching within the selection"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Model, RigVMClientHost]() {
					TArray<FName> NodeNames = Model->GetSelectNodes();
					TArray<URigVMFunctionReferenceNode*> FunctionReferenceNodes;
					FRigVMGraphFunctionIdentifier Identifier;
					for(const FName& NodeName : NodeNames)
					{
						if(URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Model->FindNodeByName(NodeName)))
						{
							if(Identifier.IsValid())
							{
								if(FunctionReferenceNode->GetFunctionIdentifier() != Identifier)
								{
									continue;
								}
							}
							else
							{
								Identifier = FunctionReferenceNode->GetFunctionIdentifier();
							}
							FunctionReferenceNodes.Add(FunctionReferenceNode);
						}
					}
					if(!FunctionReferenceNodes.IsEmpty())
					{
						SRigVMSwapFunctionsWidget::FArguments WidgetArgs;
						WidgetArgs
							.Source(Identifier)
							.FunctionReferenceNodes(FunctionReferenceNodes)
							.Assets({FAssetData(RigVMClientHost->GetRigVMGraphFunctionHost().GetObject())})
							.SkipPickingFunctionRefs(true)
							.EnableUndo(true)
							.CloseOnSuccess(true);

						const TSharedRef<SRigVMBulkEditDialog<SRigVMSwapFunctionsWidget>> SwapFunctionsDialog =
							SNew(SRigVMBulkEditDialog<SRigVMSwapFunctionsWidget>)
							.WindowSize(FVector2D(800.0f, 640.0f))
							.WidgetArgs(WidgetArgs);

						SwapFunctionsDialog->ShowNormal();
					}
				})
			));
		}
	}
}

void FRigVMEditorModule::GetNodeTestContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	// this struct is only available in EngineTest for now
	static const FString TraitObjectPath = TEXT("/Script/EngineTestEditor.EngineTestRigVM_SimpleTrait");
	const UScriptStruct* SimpleTraitStruct = Cast<UScriptStruct>(RigVMTypeUtils::FindObjectFromCPPTypeObjectPath(TraitObjectPath));
	if(SimpleTraitStruct == nullptr)
	{
		return;
	}

	const URigVMGraph* Model = ModelNode->GetGraph();
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(Model);

	FToolMenuSection& EngineTestSection = Menu->AddSection("RigVMEditorContextMenuEngineTest", LOCTEXT("EngineTestHeader", "EngineTest"));
	EngineTestSection.AddMenuEntry(
		"Add simple trait",
		LOCTEXT("AddSimpleTrait", "Add simple trait"),
		LOCTEXT("AddSimpleTrait_Tooltip", "Adds a simple test trait to the node"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Controller, ModelNode, SimpleTraitStruct]()
		{
			(void)Controller->AddTrait(
				ModelNode->GetFName(),
				*SimpleTraitStruct->GetPathName(),
				TEXT("Trait"),
				FString(), INDEX_NONE, true, true);
		}))
	);
}

void FRigVMEditorModule::GetNodeDisplayContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const
{
	FRigVMEditorAssetInterfacePtr Blueprint = RigVMClientHost.GetObject();
	if(Blueprint == nullptr)
	{
		return;
	}
	
	FToolMenuSection& DisplaySection = Menu->AddSection("RigVMEditorContextMenuDisplay", LOCTEXT("DisplayHeader", "Display"));

	DisplaySection.AddMenuEntry(
		TEXT("EnableProfiler"),
		LOCTEXT("EnableProfiler", "Enable Profiler"),
		LOCTEXT("EnableProfiler_Tooltip", "Enables the heat map profiler"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Blueprint]()
		{
			FScopedTransaction Transaction(LOCTEXT("ToggleProfiler", "Toggle Profiler"));
			Blueprint.GetObject()->Modify();
			Blueprint->SetProfilingEnabled(!Blueprint->GetVMRuntimeSettings().bEnableProfiling);
		}),
		FCanExecuteAction(),
		FGetActionCheckState::CreateLambda([Blueprint]() -> ECheckBoxState
		{
			return Blueprint->GetVMRuntimeSettings().bEnableProfiling ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})),
		EUserInterfaceActionType::ToggleButton
	);

	DisplaySection.AddMenuEntry(
		TEXT("ShowAllTags"),
		LOCTEXT("ShowAllTags", "Show all tags"),
		LOCTEXT("ShowAllTags_Tooltip", "Shows all of the tags on nodes. If turned off this will show the deprecation tags only."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Blueprint]()
		{
			FScopedTransaction Transaction(LOCTEXT("ToggleTagDisplayMode", "Toggle Tag Display Mode"));
			Blueprint->GetObject()->Modify();

			if(Blueprint->GetRigGraphDisplaySettings().TagDisplayMode == ERigVMTagDisplayMode::All)
			{
				Blueprint->GetRigGraphDisplaySettings().TagDisplayMode = ERigVMTagDisplayMode::DeprecationOnly;
			}
			else
			{
				Blueprint->GetRigGraphDisplaySettings().TagDisplayMode = ERigVMTagDisplayMode::All;
			}

			// this causes all nodes to refresh
			Blueprint->PropagateRuntimeSettingsFromBPToInstances();
		}),
		FCanExecuteAction(),
		FGetActionCheckState::CreateLambda([Blueprint]() -> ECheckBoxState
		{
			return Blueprint->GetRigGraphDisplaySettings().TagDisplayMode == ERigVMTagDisplayMode::All ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})),
		EUserInterfaceActionType::ToggleButton
	);
}

void FRigVMEditorModule::GetPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	GetExposedPinContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinWorkflowContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinDebugContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinArrayContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinSequenceContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinAggregateContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinTemplateContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinConversionContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinVariableContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinResetDefaultContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
	GetPinInjectedNodesContextMenuActions(RigVMClientHost, EdGraphPin, ModelPin, Menu);
}

void FRigVMEditorModule::GetExposedPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (!ModelPin->IsRootPin() || !ModelPin->GetNode()->IsA<URigVMFunctionInterfaceNode>())
	{
		return;
	}

	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
	if (Controller == nullptr)
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuExposedPin", LOCTEXT("ExposedPinHeader", "Exposed Pin"));

	Section.AddMenuEntry(
		"RemoveExposedPin",
		LOCTEXT("RemoveExposedPin", "Remove Pin"),
		LOCTEXT("RemoveExposedPin_Tooltip", "Removes the exposed pin from the node"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
			Controller->RemoveExposedPin(ModelPin->GetFName());
		})
	));
}

void FRigVMEditorModule::GetPinWorkflowContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	const TArray<FRigVMUserWorkflow> Workflows = ModelPin->GetNode()->GetSupportedWorkflows(ERigVMUserWorkflowType::PinContext, ModelPin);
	if(!Workflows.IsEmpty())
	{
		URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());

		FToolMenuSection& SettingsSection = Menu->AddSection("RigVMEditorContextMenuWorkflow", LOCTEXT("WorkflowHeader", "Workflow"));

		for(const FRigVMUserWorkflow& Workflow : Workflows)
		{
			SettingsSection.AddMenuEntry(
				*Workflow.GetTitle(),
				FText::FromString(Workflow.GetTitle()),
				FText::FromString(Workflow.GetTooltip()),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, Controller, Workflow, ModelPin]()
				{
					URigVMUserWorkflowOptions* Options = Controller->MakeOptionsForWorkflow(ModelPin, Workflow);

					bool bPerform = true;
					if(Options->RequiresDialog())
					{
						bPerform = ShowWorkflowOptionsDialog(Options);
					}
					if(bPerform)
					{
						Controller->PerformUserWorkflow(Workflow, Options);
					}
				}))
			);
		}
	}
}

void FRigVMEditorModule::GetPinDebugContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject())
	{
		const bool bIsEditablePin = !ModelPin->IsExecuteContext() && !ModelPin->IsWildCard();

		if(Cast<URigVMEdGraphNode>(EdGraphPin->GetOwningNode()))
		{
			if(bIsEditablePin)
			{
				// Add the watch pin / unwatch pin menu items
				FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuWatches", LOCTEXT("WatchesHeader", "Watches"));
				if (RigVMBlueprint->IsPinBeingWatched(EdGraphPin))
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().StopWatchingPin);
				}
				else
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().StartWatchingPin);
				}
			}
		}
	}
}

void FRigVMEditorModule::GetPinArrayContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelPin->GetNode()))
	{
		if (DispatchNode->GetFactory()->GetScriptStruct() == FRigVMDispatch_SwitchEnum::StaticStruct())
		{
			return;
		}
	}
	
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
	if (ModelPin->IsArray() && !ModelPin->IsExecuteContext())
	{
		FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuPinArrays", LOCTEXT("PinArrays", "Arrays"));
		Section.AddMenuEntry(
			"ClearPinArray",
			LOCTEXT("ClearPinArray", "Clear Array"),
			LOCTEXT("ClearPinArray_Tooltip", "Removes all elements of the array."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
				Controller->ClearArrayPin(ModelPin->GetPinPath());
			})
		));
	}
	
	if(ModelPin->IsArrayElement())
	{
		FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuPinArrays", LOCTEXT("PinArrays", "Arrays"));
		Section.AddMenuEntry(
			"RemoveArrayPin",
			LOCTEXT("RemoveArrayPin", "Remove Array Element"),
			LOCTEXT("RemoveArrayPin_Tooltip", "Removes the selected element from the array"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
				Controller->RemoveArrayPin(ModelPin->GetPinPath(), true, true);
			})
		));
		Section.AddMenuEntry(
			"DuplicateArrayPin",
			LOCTEXT("DuplicateArrayPin", "Duplicate Array Element"),
			LOCTEXT("DuplicateArrayPin_Tooltip", "Duplicates the selected element"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
				Controller->DuplicateArrayPin(ModelPin->GetPinPath(), true, true);
			})
		));
	}
}

void FRigVMEditorModule::GetPinSequenceContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (ModelPin->GetDirection() == ERigVMPinDirection::Output)
	{
		URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
		if (URigVMAggregateNode* AggregateNode = Cast<URigVMAggregateNode>(ModelPin->GetNode()))
		{
			if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(AggregateNode->GetFirstInnerNode()))
			{
				if (UnitNode->GetScriptStruct() == FRigVMFunction_Sequence::StaticStruct())
				{
					FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuSequencePin", LOCTEXT("SequencePin", "Sequence"));
					Section.AddMenuEntry(
						"InsertSequencePin",
						LOCTEXT("InsertSequencePin", "Insert Pin Above"),
						LOCTEXT("InsertSequencePin_Tooltip", "Inserts a new output pin"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin, AggregateNode]() {

							Controller->GetActionStack()->OpenUndoBracket(Controller, TEXT("Insert Sequence Pin"));
							if (!Controller->AddAggregatePin(AggregateNode, FString(), FString(), true, true).IsEmpty())
							{
								TArray<URigVMPin*> Pins = AggregateNode->GetPins();
								for (int32 i=Pins.Num()-1; i > 0; i--)
								{
									if (Pins[i] == ModelPin)
									{
										break;
									}
	
									if (Pins.IsValidIndex(i-1))
									{
										TArray<URigVMPin*> TargetPins = Pins[i-1]->GetLinkedTargetPins();
										
										Controller->BreakAllLinks(Pins[i-1], false);
										for (URigVMPin* TargetPin : TargetPins)
										{
											Controller->AddLink(Pins[i], TargetPin);
										}
									}
								}
							}
							else
							{
								Controller->GetActionStack()->CancelUndoBracket(Controller);
							}
							Controller->GetActionStack()->CloseUndoBracket(Controller);
						})
					));
				}
			}
		}
	}
}

void FRigVMEditorModule::GetPinAggregateContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
	if (Cast<URigVMAggregateNode>(ModelPin->GetNode()))
	{
		FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuAggregatePin", LOCTEXT("AggregatePin", "Aggregates"));
		Section.AddMenuEntry(
			"RemoveAggregatePin",
			LOCTEXT("RemoveAggregatePin", "Remove Aggregate Element"),
			LOCTEXT("RemoveAggregatePin_Tooltip", "Removes the selected element from the aggregate"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
				Controller->RemoveAggregatePin(ModelPin->GetPinPath(), true, true);
			})
		));
	}
}

void FRigVMEditorModule::GetPinTemplateContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelPin->GetNode()))
	{
		if (!TemplateNode->IsSingleton())
		{
			URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
			FToolMenuSection& TemplatesSection = Menu->AddSection("RigVMEditorContextMenuTemplates", LOCTEXT("TemplatesHeader", "Templates"));

			if(const FRigVMTemplate* Template = TemplateNode->GetTemplate())
			{
				if(!ModelPin->IsExecuteContext())
				{
					URigVMPin* RootPin = ModelPin->GetRootPin();
					if(const FRigVMTemplateArgument* Argument = Template->FindArgument(RootPin->GetFName()))
					{
						if(!Argument->IsSingleton())
						{
							TArray<TRigVMTypeIndex> ResolvedTypeIndices = Argument->GetSupportedTypeIndices(TemplateNode->GetResolvedPermutationIndices(true));
							TSharedRef<SRigVMGraphChangePinType> ChangePinTypeWidget =
							SNew(SRigVMGraphChangePinType)
							.Types(ResolvedTypeIndices)
							.OnTypeSelected_Lambda([RigVMClientHost, RootPin](const TRigVMTypeIndex& TypeSelected)
							{
								if (URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(RootPin->GetGraph()))
								{
									Controller->ResolveWildCardPin(RootPin, TypeSelected, true, true);
								}
							});

							TemplatesSection.AddEntry(FToolMenuEntry::InitWidget("ChangePinTypeWidget", ChangePinTypeWidget, FText(), true));
						}
					}
				}
							
				TemplatesSection.AddMenuEntry(
					"Unresolve Template Node",
					LOCTEXT("UnresolveTemplateNode", "Unresolve Template Node"),
					LOCTEXT("UnresolveTemplateNode_Tooltip", "Removes any type information from the template node"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
						const TArray<FName> Nodes = ModelPin->GetGraph()->GetSelectNodes();
						Controller->UnresolveTemplateNodes(Nodes, true, true);
					})
				));
			}
		}
	}
}

void FRigVMEditorModule::GetPinConversionContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (URigVMRerouteNode* RerouteNode = Cast<URigVMRerouteNode>(ModelPin->GetNode()))
	{
		if(RerouteNode->GetLinkedSourceNodes().Num() == 0)
		{
			if(const URigVMPin* ValuePin = RerouteNode->FindPin(TEXT("Value")))
			{
				if(!ValuePin->IsExecuteContext())
				{
					URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());
					FToolMenuSection& ConversionSection = Menu->AddSection("RigVMEditorContextMenuConversion", LOCTEXT("ConversionHeader", "Conversion"));

					if(ValuePin->IsArray())
					{
						ConversionSection.AddMenuEntry(
							"Convert Reroute to Make Array",
							LOCTEXT("ConvertReroutetoMakeArray", "Convert Reroute to Make Array"),
							LOCTEXT("ConvertReroutetoMakeArray_Tooltip", "Converts the Reroute node to a to Make Array node and sustains the value"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateLambda([Controller, RerouteNode]()
							{
								Controller->ConvertRerouteNodeToDispatch(RerouteNode, FRigVMDispatch_ArrayMake().GetTemplateNotation(), true, true);
							}
						)));
					}
					else if(ValuePin->IsStruct())
					{
						ConversionSection.AddMenuEntry(
							"Convert Reroute to Make Struct",
							LOCTEXT("ConvertReroutetoMakeStruct", "Convert Reroute to Make Struct"),
							LOCTEXT("ConvertReroutetoMakeStruct_Tooltip", "Converts the Reroute node to a to Make Struct node and sustains the value"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateLambda([Controller, RerouteNode]()
							{
								Controller->ConvertRerouteNodeToDispatch(RerouteNode, FRigVMDispatch_MakeStruct().GetTemplateNotation(), true, true);
							}
						)));
					}
					else
					{
						ConversionSection.AddMenuEntry(
							"Convert Reroute to Constant",
							LOCTEXT("ConvertReroutetoConstant", "Convert Reroute to Constant"),
							LOCTEXT("ConvertReroutetoConstant_Tooltip", "Converts the Reroute node to a to Constant node and sustains the value"),
							FSlateIcon(),
							FUIAction(FExecuteAction::CreateLambda([Controller, RerouteNode]()
							{
								Controller->ConvertRerouteNodeToDispatch(RerouteNode, FRigVMDispatch_Constant().GetTemplateNotation(), true, true);
							}
						)));
					}
				}
			}
		}
	}
}

void FRigVMEditorModule::GetPinVariableContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if(FRigVMEditorAssetInterfacePtr RigVMBlueprint = RigVMClientHost.GetObject())
	{
		const bool bIsEditablePin = !ModelPin->IsExecuteContext() && !ModelPin->IsWildCard();

		if (ModelPin->GetDirection() == ERigVMPinDirection::Input && bIsEditablePin)
		{
			const UEdGraphNode* EdGraphNode = EdGraphPin->GetOwningNode();
			URigVMController* Controller = RigVMBlueprint->GetController(ModelPin->GetGraph());

			if (ModelPin->IsBoundToVariable())
			{
				FVector2D NodePosition = FVector2D(EdGraphNode->NodePosX - 200.0, EdGraphNode->NodePosY);

				FToolMenuSection& VariablesSection = Menu->AddSection("RigVMEditorContextMenuVariables", LOCTEXT("Variables", "Variables"));
				VariablesSection.AddMenuEntry(
					"MakeVariableNodeFromBinding",
					LOCTEXT("MakeVariableNodeFromBinding", "Make Variable Node"),
					LOCTEXT("MakeVariableNodeFromBinding_Tooltip", "Turns the variable binding on the pin to a variable node"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin, NodePosition]() {
						Controller->MakeVariableNodeFromBinding(ModelPin->GetPinPath(), NodePosition, true, true);
					})
				));
			}
			else
			{
				FVector2D NodePosition = FVector2D(EdGraphNode->NodePosX - 200.0, EdGraphNode->NodePosY);

				FToolMenuSection& VariablesSection = Menu->AddSection("RigVMEditorContextMenuVariables", LOCTEXT("Variables", "Variables"));
				VariablesSection.AddMenuEntry(
					"PromotePinToVariable",
					LOCTEXT("PromotePinToVariable", "Promote Pin To Variable"),
					LOCTEXT("PromotePinToVariable_Tooltip", "Turns the pin into a variable"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin, NodePosition]() {

						FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
						bool bCreateVariableNode = !KeyState.IsAltDown();

						Controller->PromotePinToVariable(ModelPin->GetPinPath(), bCreateVariableNode, NodePosition, true, true);
					})
				));
			}
		}

		if (Cast<URigVMUnitNode>(ModelPin->GetNode()) != nullptr || 
			Cast<URigVMDispatchNode>(ModelPin->GetNode()) != nullptr || 
			Cast<URigVMLibraryNode>(ModelPin->GetNode()) != nullptr)
		{
			if (ModelPin->GetDirection() == ERigVMPinDirection::Input &&
				ModelPin->IsRootPin() &&
				bIsEditablePin)
			{
				if (!ModelPin->IsBoundToVariable())
				{
					FToolMenuSection& VariablesSection = Menu->FindOrAddSection(TEXT("Variables"));

					TSharedRef<SRigVMGraphVariableBinding> VariableBindingWidget =
						SNew(SRigVMGraphVariableBinding)
						.Asset(RigVMBlueprint)
						.ModelPins({ModelPin})
						.CanRemoveBinding(false);

					VariablesSection.AddEntry(FToolMenuEntry::InitWidget("BindPinToVariableWidget", VariableBindingWidget, FText(), true));
				}
			}
		}
	}
}

void FRigVMEditorModule::GetPinResetDefaultContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (Cast<URigVMUnitNode>(ModelPin->GetNode()) != nullptr || 
		Cast<URigVMDispatchNode>(ModelPin->GetNode()) != nullptr || 
		Cast<URigVMLibraryNode>(ModelPin->GetNode()) != nullptr)
	{
		const bool bIsEditablePin = !ModelPin->IsExecuteContext() && !ModelPin->IsWildCard();
		
		if (ModelPin->GetDirection() == ERigVMPinDirection::Input &&
			ModelPin->IsRootPin() &&
			bIsEditablePin)
		{
			FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuPinDefaults", LOCTEXT("PinDefaults", "Pin Defaults"));
			URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());

			Section.AddMenuEntry(
				"ResetPinDefaultValue",
				LOCTEXT("ResetPinDefaultValue", "Reset Pin Value"),
				LOCTEXT("ResetPinDefaultValue_Tooltip", "Resets the pin's value to its default."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
					Controller->ResetPinDefaultValue(ModelPin->GetPinPath());
				})
			));

			Section.AddMenuEntry(
				"OverridePinDefaultValue",
				LOCTEXT("OverridePinDefaultValue", "Override Value"),
				LOCTEXT("OverridePinDefaultValue_Tooltip", "Marks the pin value as an override."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
					FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::Override);
					Controller->SetPinDefaultValue(ModelPin->GetPinPath(), ModelPin->GetDefaultValue());
				})
			));
		}
	}
}

void FRigVMEditorModule::GetPinInjectedNodesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const
{
	if (ModelPin->GetRootPin() == ModelPin && (
	Cast<URigVMUnitNode>(ModelPin->GetNode()) != nullptr ||
	Cast<URigVMLibraryNode>(ModelPin->GetNode()) != nullptr))
	{
		URigVMController* Controller = RigVMClientHost->GetRigVMClient()->GetController(ModelPin->GetGraph());

		if (ModelPin->HasInjectedNodes())
		{
			FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuNodeEjectionInterp", LOCTEXT("NodeEjectionInterp", "Eject"));

			Section.AddMenuEntry(
				"EjectLastNode",
				LOCTEXT("EjectLastNode", "Eject Last Node"),
				LOCTEXT("EjectLastNode_Tooltip", "Eject the last injected node"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([Controller, ModelPin]() {
					Controller->OpenUndoBracket(TEXT("Eject node from pin"));
					URigVMNode* Node = Controller->EjectNodeFromPin(ModelPin->GetPinPath(), true, true);
					Controller->SelectNode(Node, true, true, true);
					Controller->CloseUndoBracket();
				})
			));
		}

		if (ModelPin->GetCPPType() == TEXT("float") ||
			ModelPin->GetCPPType() == TEXT("double") ||
			ModelPin->GetCPPType() == TEXT("FVector"))
		{
			FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuNodeInjectionInterp", LOCTEXT("NodeInjectionInterp", "Interpolate"));
			URigVMNode* InterpNode = nullptr;
			bool bBoundToVariable = false;
			for (URigVMInjectionInfo* Injection : ModelPin->GetInjectedNodes())
			{
				FString TemplateName;
				if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Injection->Node))
				{
					if (UnitNode->GetScriptStruct()->GetStringMetaDataHierarchical(FRigVMRegistry::TemplateNameMetaName, &TemplateName))
					{
						if (TemplateName == TEXT("AlphaInterp"))
						{
							InterpNode = Injection->Node;
							break;
						}
					}
				}
				else if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Injection->Node))
				{
					bBoundToVariable = true;
					break;
				}
			}

			if(!bBoundToVariable)
			{
				if (InterpNode == nullptr)
				{
					UScriptStruct* ScriptStruct = nullptr;

					if ((ModelPin->GetCPPType() == TEXT("float")) || (ModelPin->GetCPPType() == TEXT("double")))
					{
						ScriptStruct = FRigVMFunction_AlphaInterp::StaticStruct();
					}
					else if (ModelPin->GetCPPType() == TEXT("FVector"))
					{
						ScriptStruct = FRigVMFunction_AlphaInterpVector::StaticStruct();
					}
					else
					{
						checkNoEntry();
					}

					Section.AddMenuEntry(
						"AddAlphaInterp",
						LOCTEXT("AddAlphaInterp", "Add Interpolate"),
						LOCTEXT("AddAlphaInterp_Tooltip", "Injects an interpolate node"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, EdGraphPin, ModelPin, ScriptStruct]() {
							Controller->OpenUndoBracket(TEXT("Add injected node"));
							URigVMInjectionInfo* Injection = Controller->AddInjectedNode(ModelPin->GetPinPath(), ModelPin->GetDirection() != ERigVMPinDirection::Output, ScriptStruct, FRigVMStruct::ExecuteName, TEXT("Value"), TEXT("Result"), FString(), true, true);
							if (Injection)
							{
								TArray<FName> NodeNames;
								NodeNames.Add(Injection->Node->GetFName());
								Controller->SetNodeSelection(NodeNames);
							}
							Controller->CloseUndoBracket();
						})
					));
				}
				else
				{
					Section.AddMenuEntry(
						"EditAlphaInterp",
						LOCTEXT("EditAlphaInterp", "Edit Interpolate"),
						LOCTEXT("EditAlphaInterp_Tooltip", "Edit the interpolate node"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([RigVMClientHost, InterpNode]() {
						TArray<FName> NodeNames;
						NodeNames.Add(InterpNode->GetFName());
						RigVMClientHost->GetRigVMClient()->GetController(InterpNode->GetGraph())->SetNodeSelection(NodeNames);
					})
						));
					Section.AddMenuEntry(
						"RemoveAlphaInterp",
						LOCTEXT("RemoveAlphaInterp", "Remove Interpolate"),
						LOCTEXT("RemoveAlphaInterp_Tooltip", "Removes the interpolate node"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, EdGraphPin, ModelPin, InterpNode]() {
							Controller->RemoveInjectedNode(ModelPin->GetPinPath(), ModelPin->GetDirection() != ERigVMPinDirection::Output, true);
						})
					));
				}
			}
		}

		if (ModelPin->GetCPPType() == TEXT("FVector") ||
			ModelPin->GetCPPType() == TEXT("FQuat") ||
			ModelPin->GetCPPType() == TEXT("FTransform"))
		{
			FToolMenuSection& Section = Menu->AddSection("RigVMEditorContextMenuNodeInjectionVisualDebug", LOCTEXT("NodeInjectionVisualDebug", "Visual Debug"));

			URigVMNode* VisualDebugNode = nullptr;
			bool bBoundToVariable = false;
			for (URigVMInjectionInfo* Injection : ModelPin->GetInjectedNodes())
			{
				FString TemplateName;
				if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Injection->Node))
				{
					if (UnitNode->GetScriptStruct()->GetStringMetaDataHierarchical(FRigVMRegistry::TemplateNameMetaName, &TemplateName))
					{
						if (TemplateName == TEXT("VisualDebug"))
						{
							VisualDebugNode = Injection->Node;
							break;
						}
					}
				}
				else if (URigVMVariableNode* VariableNode = Cast<URigVMVariableNode>(Injection->Node))
				{
					bBoundToVariable = true;
					break;
				}
			}

			if (!bBoundToVariable)
			{
				if (VisualDebugNode == nullptr)
				{
					UScriptStruct* ScriptStruct = nullptr;

					if (ModelPin->GetCPPType() == TEXT("FVector"))
					{
						ScriptStruct = FRigVMFunction_VisualDebugVectorNoSpace::StaticStruct();
					}
					else if (ModelPin->GetCPPType() == TEXT("FQuat"))
					{
						ScriptStruct = FRigVMFunction_VisualDebugQuatNoSpace::StaticStruct();
					}
					else if (ModelPin->GetCPPType() == TEXT("FTransform"))
					{
						ScriptStruct = FRigVMFunction_VisualDebugTransformNoSpace::StaticStruct();
					}
					else
					{
						checkNoEntry();
					}

					Section.AddMenuEntry(
						"AddVisualDebug",
						LOCTEXT("AddVisualDebug", "Add Visual Debug"),
						LOCTEXT("AddVisualDebug_Tooltip", "Injects a visual debugging node"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([RigVMClientHost, Controller, EdGraphPin, ModelPin, ScriptStruct]() {
							URigVMInjectionInfo* Injection = Controller->AddInjectedNode(ModelPin->GetPinPath(), ModelPin->GetDirection() != ERigVMPinDirection::Output, ScriptStruct, FRigVMStruct::ExecuteName, TEXT("Value"), TEXT("Value"), FString(), true, true);
							if (Injection)
							{
								TArray<FName> NodeNames;
								NodeNames.Add(Injection->Node->GetFName());
								Controller->SetNodeSelection(NodeNames);

								/*
								 * todo
								if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelPin->GetNode()))
								{
									if (TSharedPtr<FStructOnScope> DefaultStructScope = UnitNode->ConstructStructInstance())
									{
										if(DefaultStructScope->GetStruct()->IsChildOf(FRigUnit::StaticStruct()))
										{
											FRigUnit* DefaultStruct = (FRigUnit*)DefaultStructScope->GetStructMemory();

											FString PinPath = ModelPin->GetPinPath();
											FString Left, Right;

											FRigElementKey SpaceKey;
											if (URigVMPin::SplitPinPathAtStart(PinPath, Left, Right))
											{
												SpaceKey = DefaultStruct->DetermineSpaceForPin(Right, RigVMBlueprint->Hierarchy);
											}

											if (SpaceKey.IsValid())
											{
												if (URigVMPin* SpacePin = Injection->Node->FindPin(TEXT("Space")))
												{
													if(URigVMPin* SpaceTypePin = SpacePin->FindSubPin(TEXT("Type")))
													{
														FString SpaceTypeStr = StaticEnum<ERigElementType>()->GetDisplayNameTextByValue((int64)SpaceKey.Type).ToString();
														Controller->SetPinDefaultValue(SpaceTypePin->GetPinPath(), SpaceTypeStr, true, true, false, true);
													}
													if(URigVMPin* SpaceNamePin = SpacePin->FindSubPin(TEXT("Name")))
													{
														Controller->SetPinDefaultValue(SpaceNamePin->GetPinPath(), SpaceKey.Name.ToString(), true, true, false, true);
													}
												}
											}
										}
									}
								}
								*/
							}
						})
					));
				}
				else
				{
					Section.AddMenuEntry(
						"EditVisualDebug",
						LOCTEXT("EditVisualDebug", "Edit Visual Debug"),
						LOCTEXT("EditVisualDebug_Tooltip", "Edit the visual debugging node"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, VisualDebugNode]() {
							TArray<FName> NodeNames;
							NodeNames.Add(VisualDebugNode->GetFName());
							Controller->SetNodeSelection(NodeNames);
						})
					));
					Section.AddMenuEntry(
						"ToggleVisualDebug",
						LOCTEXT("ToggleVisualDebug", "Toggle Visual Debug"),
						LOCTEXT("ToggleVisualDebug_Tooltip", "Toggle the visibility the visual debugging"),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([Controller, VisualDebugNode]() {
							URigVMPin* EnabledPin = VisualDebugNode->FindPin(TEXT("bEnabled"));
							check(EnabledPin);
							Controller->SetPinDefaultValue(EnabledPin->GetPinPath(), EnabledPin->GetDefaultValue() == TEXT("True") ? TEXT("False") : TEXT("True"), false, true, false, true);
						})
					));
					Section.AddMenuEntry(
						"RemoveVisualDebug",
						LOCTEXT("RemoveVisualDebug", "Remove Visual Debug"),
						LOCTEXT("RemoveVisualDebug_Tooltip", "Removes the visual debugging node"),
						FSlateIcon(),
							FUIAction(FExecuteAction::CreateLambda([Controller, EdGraphPin, ModelPin, VisualDebugNode]() {
							Controller->RemoveNodeByName(VisualDebugNode->GetFName(), true, false);
						})
					));
				}
			}
		}
	}
}

void FRigVMEditorModule::GetContextMenuActions(const URigVMEdGraphSchema* Schema, UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Menu && Context)
	{
		Schema->UEdGraphSchema::GetContextMenuActions(Menu, Context);

		if (const UEdGraphPin* InGraphPin = (UEdGraphPin*)Context->Pin)
		{
			if(const UEdGraph* Graph = Context->Graph)
			{
				// Can't get the UObject with GetImplementingOuter, which I need to create a TScriptInterface<IRigVMClientHost>
				UObject* Outer = Graph->GetOuter();
				while (Outer && !Outer->Implements<URigVMClientHost>())
				{
					Outer = Outer->GetOuter();
				}
				if (Outer)
				{
					if(TScriptInterface<IRigVMClientHost> RigVMClientHost = Outer)
					{
						if (URigVMPin* ModelPin = RigVMClientHost->GetRigVMClient()->GetModel(Graph)->FindPin(InGraphPin->GetName()))
						{
							GetPinContextMenuActions(RigVMClientHost, InGraphPin, ModelPin, Menu);
						}
					}
				}
			}
		}
		else if(const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(Context->Node))
		{
			if (const UEdGraph* Graph = Context->Graph)
			{
				// Can't get the UObject with GetImplementingOuter, which I need to create a TScriptInterface<IRigVMClientHost>
				UObject* Outer = Graph->GetOuter();
				while (Outer && !Outer->Implements<URigVMClientHost>())
				{
					Outer = Outer->GetOuter();
				}
				if (Outer)
				{
					if (TScriptInterface<IRigVMClientHost> RigVMClientHost = Outer)
					{
						if(URigVMNode* ModelNode = EdGraphNode->GetModelNode())
						{
							GetNodeContextMenuActions(RigVMClientHost, EdGraphNode, ModelNode, Menu);
						}
					}
				}
			}
		}
	}
}

void FRigVMEditorModule::PreChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType)
{
	if(!IsRigVMEditorModuleBase())
	{
		return;
	}
	
	// the following is similar to
	// FUserDefinedStructureCompilerInner::ReplaceStructWithTempDuplicate()
	// it is necessary since existing rigs need to be kept valid until after PreBPCompile
	// there are other systems, such as sequencer, that might need to evaluate the rig
	// for one last time during PreBPCompile
	// Overall sequence of events
	// PreStructChange --1--> PostStructChange
	//                              --2--> PreBPCompile --3--> PostBPCompile
	
	UUserDefinedStruct* StructureToReinstance = (UUserDefinedStruct*)Changed;

	FUserDefinedStructureCompilerUtils::ReplaceStructWithTempDuplicateByPredicate(
		StructureToReinstance,
		[](FStructProperty* InStructProperty)
		{
			// make sure variable properties on the BP is patched
			// since active rig instance still references it
			if (URigVMBlueprintGeneratedClass* BPClass = Cast<URigVMBlueprintGeneratedClass>(InStructProperty->GetOwnerClass()))
			{
				if (BPClass->ClassGeneratedBy->IsA<URigVMBlueprint>())
				{
					return true;
				}
			}
			// similar story, VM instructions reference properties on the GeneratorClass
			else if ((InStructProperty->GetOwnerStruct())->IsA<URigVMMemoryStorageGeneratorClass>())
			{
				return true;
			}
			else if (URigVMDetailsViewWrapperObject::IsValidClass(InStructProperty->GetOwnerClass()))
			{
				return true;
			}
			
			return false;
		},
		[](UStruct* InStruct)
		{
			// refresh these since VM caching references them
			if (URigVMMemoryStorageGeneratorClass* GeneratorClass = Cast<URigVMMemoryStorageGeneratorClass>(InStruct))
			{
				GeneratorClass->RefreshLinkedProperties();
				GeneratorClass->RefreshPropertyPaths();	
			}
			else if (InStruct->IsChildOf(URigVMDetailsViewWrapperObject::StaticClass()))
			{
				URigVMDetailsViewWrapperObject::MarkOutdatedClass(Cast<UClass>(InStruct));
			}
		});
	
	// in the future we could only invalidate caches on affected rig instances, it shouldn't make too much of a difference though
	for (TObjectIterator<URigVMHost> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
	{
		URigVMHost* Host = *It;
		// rebuild property list and property path list
		Host->RecreateCachedMemory();
	}
}

void FRigVMEditorModule::PostChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType)
{
	if(!IsRigVMEditorModuleBase())
	{
		return;
	}

	TArray<FRigVMEditorAssetInterfacePtr> BlueprintsToRefresh;
	TArray<UEdGraph*> EdGraphsToRefresh;

	for (TObjectIterator<URigVMPin> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
	{
		const URigVMPin* Pin = *It;
		// GetCPPTypeObject also makes sure the pin's type information is update to date
		if (Pin && Pin->GetCPPTypeObject() == Changed)
		{
			if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(Pin))
			{
				BlueprintsToRefresh.AddUnique(RigVMBlueprint);
				
				// this pin is part of a function definition
				// update all BP that uses this function
				if (Pin->GetGraph() == RigVMBlueprint->GetLocalFunctionLibrary())
				{
					TArray< TSoftObjectPtr<URigVMFunctionReferenceNode> > References;
					References = RigVMBlueprint->GetRigVMClient()->GetOrCreateFunctionLibrary(false)->GetReferencesForFunction(Pin->GetNode()->GetFName());
					
					for (const TSoftObjectPtr<URigVMFunctionReferenceNode>& Reference : References)
					{
						const URigVMFunctionReferenceNode* RefNode = Reference.LoadSynchronous();
						if (!RefNode)
						{
							continue;
						}
						
						if (FRigVMEditorAssetInterfacePtr FunctionUserBlueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(RefNode))
						{
							BlueprintsToRefresh.AddUnique(FunctionUserBlueprint);
						}
					}	
				}

				if (URigVMGraph* RigVMGraph = Pin->GetNode()->GetGraph())
				{
					EdGraphsToRefresh.AddUnique(Cast<UEdGraph>(RigVMBlueprint->GetEditorObjectForRigVMGraph(RigVMGraph)));
				}
			}
		}
	}

	for (FRigVMEditorAssetInterfacePtr RigVMBlueprint : BlueprintsToRefresh)
	{
		RigVMBlueprint->OnRigVMRegistryChanged();
		(void)RigVMBlueprint.GetObject()->MarkPackageDirty();
	}

	// Avoid slate crashing after pins get repopulated
	for (UEdGraph* Graph : EdGraphsToRefresh)
	{
		Graph->NotifyGraphChanged();
	}

	for (FRigVMEditorAssetInterfacePtr RigVMBlueprint : BlueprintsToRefresh)
	{
		// this should make sure variables in BP are updated with the latest struct object
		// otherwise RigVMCompiler validation would complain about variable type - pin type mismatch
		FCompilerResultsLog	ResultsLog = RigVMBlueprint->CompileBlueprint();
		
		// BP compiler always initialize the new CDO by copying from the old CDO,
		// however, in case that a BP variable type has changed, the data old CDO would be invalid because
		// while the old memory container still references the temp duplicated struct we created during PreChange()
		// registers that reference the BP variable would be referencing the new struct as a result of
		// FKismetCompilerContext::CompileClassLayout, so type mismatch would invalidate relevant copy operations
		// so to simplify things, here we just reset all rigs upon error
		if (ResultsLog.NumErrors > 0)
		{
			if (URigVM* VM = RigVMBlueprint->GetVM(false))
			{
				if (FRigVMExtendedExecuteContext* Context = RigVMBlueprint->GetRigVMExtendedExecuteContext())
				{
					VM->Reset(*Context);
				}
			}
			TArray<UObject*> Instances = RigVMBlueprint->GetArchetypeInstances(true, true);
			for (UObject* Instance : Instances)
			{
				if (URigVMHost* RigVMHost = Cast<URigVMHost>(Instance))
				{
					if (RigVMHost->GetVM())
					{
						RigVMHost->GetVM()->Reset(RigVMHost->GetRigVMExtendedExecuteContext());
					}
				}
			}
		}
	}
}

TArray<FRigVMEditorModule::FRigVMEditorToolbarExtender>& FRigVMEditorModule::GetAllRigVMEditorToolbarExtenders()
{
	return RigVMEditorToolbarExtenders;
}

void FRigVMEditorModule::CreateRootGraphIfRequired(FRigVMEditorAssetInterfacePtr InBlueprint) const
{
	if(InBlueprint == nullptr)
	{
		return;
	}

	
	UClass* EdGraphClass = InBlueprint->GetRigVMClientHost()->GetRigVMEdGraphClass();

	for(const UEdGraph* EdGraph : InBlueprint->GetUberGraphs())
	{
		if(EdGraph->IsA(EdGraphClass))
		{
			return;
		}
	}

	UClass* EdGraphSchemaClass = InBlueprint->GetRigVMClientHost()->GetRigVMEdGraphSchemaClass();
	const URigVMEdGraphSchema* SchemaCDO = CastChecked<URigVMEdGraphSchema>(EdGraphSchemaClass->GetDefaultObject());
	
	// add an initial graph for us to work in
	
	URigVMEdGraph* ControlRigGraph = FRigVMBlueprintUtils::CreateNewGraph(InBlueprint->GetObject(), SchemaCDO->GetRootGraphName(), EdGraphClass, EdGraphSchemaClass);
	ControlRigGraph->bAllowDeletion = false;
	InBlueprint->AddUbergraphPage(ControlRigGraph);
	InBlueprint->GetLastEditedDocuments().AddUnique(ControlRigGraph);
	InBlueprint->PostLoad();
}

void FRigVMEditorModule::HandleNewBlueprintCreated(UBlueprint* InBlueprint)
{
	CreateRootGraphIfRequired(FRigVMEditorAssetInterfacePtr(InBlueprint));
}

bool FRigVMEditorModule::ShowWorkflowOptionsDialog(URigVMUserWorkflowOptions* InOptions) const
{
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;
	DetailsViewArgs.bAllowSearch = false;

	const TSharedRef<class IDetailsView> PropertyView = EditModule.CreateDetailView( DetailsViewArgs );
	PropertyView->SetObject(InOptions);

	TSharedRef<SCustomDialog> OptionsDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("ControlRigWorkflowOptions", "Options")))
		.Content()
		[
			PropertyView
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("OK", "OK")),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
	});
	return OptionsDialog->ShowModal() == 0;
}

FConnectionDrawingPolicy* FRigVMEditorModule::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FRigVMEdGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj);
}

bool FRigVMEditorModule::IsRigVMEditorModuleBase() const
{
	UClass* Class = GetRigVMAssetClass();
	if (Class == URigVMEditorAssetInterface::StaticClass() || Class->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()))
	{
		return true;
	}
	return false;
}

void FRigVMEditorModule::HandleAssetsPreDelete(const TArray<UObject*>& InAssetsToDelete)
{
	for (UObject* AssetToDelete : InAssetsToDelete)
	{
		if (URigVMRuntimeAsset* Asset = Cast<URigVMRuntimeAsset>(AssetToDelete))
		{
			Asset->MarkInstancesAsGarbage();
		}
	}
}

#undef LOCTEXT_NAMESPACE
