// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTraceAnalyzer.h"

#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "RigVMTraceProvider.h"
#include "TraceServices/Utils.h"

#if RIGVM_TRACE_ENABLED

FRigVMTraceAnalyzer::FRigVMTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FRigVMTraceProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FRigVMTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_RigVMExecute, "RigVM", "Execute");
	Builder.RouteEvent(RouteId_RigVMLiterals, "RigVM", "Literals");
}

bool FRigVMTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FRigVMTraceAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_RigVMLiterals:
		{
			const uint64 HostId = EventData.GetValue<uint64>("HostId");
			if (Provider.HasConstantData(HostId))
			{
				break;
			}
				
			FRigVMTraceConstantData ConstantData;
			ConstantData.Cycles = EventData.GetValue<uint64>("Cycles");
			ConstantData.ProfileTime = Context.EventTime.AsSeconds(ConstantData.Cycles);
			ConstantData.RecordingTime = EventData.GetValue<double>("RecordingTime");
			ConstantData.RigVMObjectVersion = EventData.GetValue<int32>("RigVMObjectVersion");
			ConstantData.HostId = HostId;
			ConstantData.BoundOuterId = EventData.GetValue<uint64>("BoundOuterId");;
			ConstantData.VMHash = EventData.GetValue<uint32>("VMHash");
			ConstantData.ByteCodeHash = EventData.GetValue<uint32>("ByteCodeHash");
			ConstantData.LiteralMemory.SetOverallBuffer(EventData.GetArrayView<uint8>("LiteralMemory"));
			ConstantData.HostConstantData.SetOverallBuffer(EventData.GetArrayView<uint8>("HostConstantData"));
				
			Provider.AppendLiterals(ConstantData);
			break;
		}
		case RouteId_RigVMExecute:
		{
			FRigVMTraceExecuteData ExecuteData;
			ExecuteData.Cycles = EventData.GetValue<uint64>("Cycles");
			ExecuteData.ProfileTime = Context.EventTime.AsSeconds(ExecuteData.Cycles);
			ExecuteData.RecordingTime = EventData.GetValue<double>("RecordingTime");
			ExecuteData.AbsoluteTime = EventData.GetValue<double>("AbsoluteTime");
			ExecuteData.DeltaTime = EventData.GetValue<double>("DeltaTime");
			ExecuteData.HostId = EventData.GetValue<uint64>("HostId");
			ExecuteData.WorldId = EventData.GetValue<uint64>("WorldId");
			ExecuteData.RigVMObjectVersion = EventData.GetValue<int32>("RigVMObjectVersion");
			ExecuteData.WorkMemory.SetOverallBuffer(EventData.GetArrayView<uint8>("WorkMemory"));
			ExecuteData.DebugMemory.SetOverallBuffer(EventData.GetArrayView<uint8>("DebugMemory"));
			ExecuteData.ExternalMemory.SetOverallBuffer(EventData.GetArrayView<uint8>("ExternalMemory"));
			(void)EventData.GetString("Entries", ExecuteData.Entries);
			ExecuteData.VisitedInstructions = EventData.GetArrayView<int32>("VisitedInstructions");
			ExecuteData.OverallCycles = EventData.GetValue<uint64>("OverallCycles");
			ExecuteData.InstructionCycles = EventData.GetArrayView<uint64>("InstructionCycles");
			ExecuteData.DrawInterface.SetOverallBuffer(EventData.GetArrayView<uint8>("DrawInterface"));
			ExecuteData.HostExecuteData.SetOverallBuffer(EventData.GetArrayView<uint8>("HostExecuteData"));
				
			Provider.AppendExecute(ExecuteData);
			break;
		}
	}

	return true;
}

#endif
