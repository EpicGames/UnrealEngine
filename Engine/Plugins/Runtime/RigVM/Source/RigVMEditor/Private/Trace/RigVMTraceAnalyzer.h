// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "RigVMTrace.h"

#if RIGVM_TRACE_ENABLED

class FRigVMTraceProvider;
namespace TraceServices { class IAnalysisSession; }

/**
 * The analyzer is used to turn unstructured data from an analytics session
 * into useful bits that can be send to the corresponding provider class.
 */
class FRigVMTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FRigVMTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FRigVMTraceProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_RigVMLiterals,
		RouteId_RigVMExecute,
	};

	TraceServices::IAnalysisSession& Session;
	FRigVMTraceProvider& Provider;
};

#endif