// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/RigVMTraceModule.h"

#include "IGameplayProvider.h"
#include "RigVMTraceProvider.h"
#include "RigVMTraceAnalyzer.h"
#include "TraceServices/Model/Frames.h"
#include "ObjectTrace.h"
#include "RigVMHost.h"
#include "Common/ProviderLock.h"
#include "Engine/Level.h"

#if RIGVM_TRACE_ENABLED

#define LOCTEXT_NAMESPACE "RigVMTraceModule"

FName FRigVMTraceModule::ModuleName("RigVM");

void FRigVMTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("RigVM");
}

void FRigVMTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FRigVMTraceProvider> RigVMTraceProvider = MakeShared<FRigVMTraceProvider>(InSession);
	InSession.AddProvider(FRigVMTraceProvider::ProviderName, RigVMTraceProvider);
	InSession.AddAnalyzer(new FRigVMTraceAnalyzer(InSession, *RigVMTraceProvider));
}

void FRigVMTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("RigVM"));
}

void FRigVMTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
}

void FRewindDebuggerForRigVM::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	if (RewindDebugger->IsPIESimulating() || RewindDebugger->GetRecordingDuration() == 0.0)
	{
		return;
	}

	if (!RewindDebugger->GetWorldToVisualize() || !RewindDebugger->GetWorldToVisualize()->GetCurrentLevel())
	{
		return;
	}
	
	if (const TraceServices::IAnalysisSession* Session = RewindDebugger->GetAnalysisSession())
	{
		// Session scope required only for RigVMProvider
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session);

		const IGameplayProvider* GameplayProvider = Session->ReadProvider<IGameplayProvider>("GameplayProvider");
		if (!GameplayProvider)
		{
			return;
		}

		if (const FRigVMTraceProvider* Provider = Session->ReadProvider<FRigVMTraceProvider>(FRigVMTraceProvider::ProviderName))
		{
			// remove obsolete hosts
			TArray<uint64> HostIdsToRemove;
			for (const TPair<uint64,TWeakObjectPtr<URigVMHost>>& Pair : AffectedHosts)
			{
				if (!Provider->HasConstantData(Pair.Key))
				{
					HostIdsToRemove.Add(Pair.Key);
				}
				else if (!Pair.Value.IsValid())
				{
					HostIdsToRemove.Add(Pair.Key);
				}
			}
			for (const uint64& HostIdToRemove : HostIdsToRemove)
			{
				const TWeakObjectPtr<URigVMHost> WeakHostPtr = AffectedHosts.FindChecked(HostIdToRemove);
				if (URigVMHost* Host = WeakHostPtr.Get())
				{
					Host->StopPlayingRewindDebugTrace();
					Host->MarkAsGarbage();
				}
				AffectedHosts.Remove(HostIdToRemove);
			}

			const double CurrentTraceTime = RewindDebugger->CurrentTraceTime();

			// find missing hosts
			Provider->EnumerateConstantData([this, RewindDebugger, GameplayProvider, CurrentTraceTime](uint64 HostId, const FRigVMTraceConstantData& ConstantData)
			{
				if (ConstantData.HostId == HostId)
				{
					if (AffectedHosts.Contains(ConstantData.HostId))
					{
						return;
					}

					const uint64 LevelClassId = FObjectTrace::GetObjectId(ULevel::StaticClass());
					const uint64 ActorClassId = FObjectTrace::GetObjectId(AActor::StaticClass());

					UWorld* World = RewindDebugger->GetWorldToVisualize();
					ULevel* Level = World->GetCurrentLevel();

					AActor* Actor = nullptr;
					TArray<FString> Names, Paths;

					uint64 ObjectId = HostId;
					uint64 HostInfoClassId = 0;
					const TCHAR* HostInfoName = nullptr;
					const TCHAR* HostInfoPathName = nullptr;
					{
						TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);
						do
						{
							const FObjectInfo* ObjectInfo = GameplayProvider->FindObjectInfo(ObjectId);
							if (!ObjectInfo)
							{
								break;
							}

							bool bIsLevel = false;

							uint64 ClassId = ObjectInfo->ClassId;
							do
							{
								if (ClassId == ActorClassId)
								{
									// find actor based on name
									TObjectPtr<AActor>* FoundActor = Level->Actors.FindByPredicate([ObjectInfo](const TObjectPtr<AActor>& ExistingActor) -> bool
										{
											if (ExistingActor)
											{
												FString ActorLabel = ExistingActor->GetActorLabel();
												if (ActorLabel.StartsWith("RewindDebugger: "))
												{
													ActorLabel.RemoveFromStart("RewindDebugger: ");
												}
												return ActorLabel == ObjectInfo->Name;
											}
											return false;
										});

									if (FoundActor)
									{
										Actor = FoundActor->Get();
									}
									break;
								}
								if (ClassId == LevelClassId)
								{
									bIsLevel = true;
									break;
								}

								const FClassInfo* ClassInfo = GameplayProvider->FindClassInfo(ClassId);
								if (!ClassInfo)
								{
									break;
								}

								ClassId = ClassInfo->SuperId;
							} while (ClassId);

							if (Actor || bIsLevel)
							{
								break;
							}

							if (ObjectId != HostId)
							{
								Names.Insert(ObjectInfo->Name, 0);
								Paths.Insert(ObjectInfo->PathName, 0);
							}
							ObjectId = ObjectInfo->GetOuterId().GetMainId();
						} while (ObjectId);

						const FObjectInfo* FoundInfo = GameplayProvider->FindObjectInfo(HostId);
						if (!FoundInfo)
						{
							return;
						}

						HostInfoClassId = FoundInfo->ClassId;
						HostInfoName = FoundInfo->Name;
						HostInfoPathName = FoundInfo->PathName;
					}

					const bool NotYetLoggedAtThisTime = CurrentTraceTime != LastLogTime;
					LastLogTime = CurrentTraceTime;
					
					if (!Actor)
					{
						if(NotYetLoggedAtThisTime)
						{
							UE_LOGF(LogRigVM, Warning, "Cannot find actor for Host '%ls' in visualize world.", HostInfoPathName);
						}
						return;
					}

					// find all sub elements under the actor.
					UObject* Outer = Actor;
					for (int32 NameIndex = 0; NameIndex < Names.Num(); ++NameIndex)
					{
						Outer = FindObject<UObject>(Outer, *Names[NameIndex]);
						if (!Outer)
						{
							if(NotYetLoggedAtThisTime)
							{
								UE_LOGF(LogRigVM, Warning, "Cannot find expected child '%ls' under actor '%ls' in visualize world.", *Paths[NameIndex], *Actor->GetPathName());
							}
							return;
						}
					}

					URigVMHost* Host = FindObject<URigVMHost>(Outer, HostInfoName);
					if (!Host)
					{
						UClass* HostClass = Cast<UClass>(FObjectTrace::GetObjectFromId(HostInfoClassId));
						if (!HostClass)
						{
							if(NotYetLoggedAtThisTime)
							{
								UE_LOGF(LogRigVM, Warning, "Cannot recreate host '%ls' in visualize world - class not found.", HostInfoPathName);
							}
							return;
						}
						
						Host = NewObject<URigVMHost>(Outer, HostClass, HostInfoName);
						Host->Initialize(true);

						// add this here to that GC doesn't destroy the host while the visualize world exist
						World->ExtraReferencedObjects.Add(Host);
					}
					
					if (Host)
					{
						if (URigVMHost::IsGarbageOrDestroyed(Host))
						{
							return;
						}

						FRigVMTraceConstantData& MutableConstantData = const_cast<FRigVMTraceConstantData&>(ConstantData);
						check(!MutableConstantData.HostConstantData.IsPayloadEmpty());

						FRigVMTraceArchiveReader HostConstantDataReader(MutableConstantData.HostConstantData);
						if (!Host->MatchesTracedConstantData(HostConstantDataReader))
						{
							if(NotYetLoggedAtThisTime)
							{
								UE_LOGF(LogRigVM, Warning, "Host '%ls' for id '%llu' was not able to play back the rewind debugger data. Mismatch in constant data / hashes.", *Host->GetPathName(), HostId);
							}
							return;
						}

						if (FRigVMTrace::RestoreRigVMConstantData(Host->GetRigVMExtendedExecuteContext(), ConstantData))
						{
							AffectedHosts.Add(HostId, Host);
						}
					}
					else
					{
						if(NotYetLoggedAtThisTime)
						{
							UE_LOGF(LogRigVM, Warning, "Host for id '%llu' was not found in world.", HostId);
						}
					}
				}
			});

			const bool ChangedTraceTime = CurrentTraceTime != LastTraceTime;

			const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*Session);
			TraceServices::FFrame Frame;

			if(FrameProvider.GetFrameFromTime(ETraceFrameType::TraceFrameType_Game, CurrentTraceTime, Frame))
			{
				Provider->EnumerateExecuteTimelines([this, ChangedTraceTime, Frame](uint64 HostId, const FRigVMTraceProvider::ExecuteTimeline& TimelineData)
				{
					TimelineData.EnumerateEvents(Frame.StartTime, Frame.EndTime,
						[this, ChangedTraceTime, HostId](double InStartTime, double InEndTime, uint32 InDepth, const FRigVMTraceExecuteData& ExecuteData)
						{
							if (ExecuteData.HostId == HostId)
							{
								if (TWeakObjectPtr<URigVMHost>* WeakHostPtr = AffectedHosts.Find(HostId))
								{
									if (URigVMHost* Host = WeakHostPtr->Get())
									{
										if (ChangedTraceTime)
										{
											(void)FRigVMTrace::RestoreRigVMExecuteData(Host->GetRigVMExtendedExecuteContext(), ExecuteData);
										}
										Host->Evaluate_AnyThread();
									}
								}
								return TraceServices::EEventEnumerate::Stop;
							}
							return TraceServices::EEventEnumerate::Continue;
						}
					);
				});
			}

			LastTraceTime = CurrentTraceTime;
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif