// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMTraceProvider.h"
#include "ObjectTrace.h"

#if RIGVM_TRACE_ENABLED

FName FRigVMTraceProvider::ProviderName("RigVMTraceProvider");

#define LOCTEXT_NAMESPACE "RigVMTraceProvider"

FRigVMTraceProvider::FRigVMTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, ConstantData(InSession.GetLinearAllocator(), 256)
{
}

bool FRigVMTraceProvider::HasConstantData(uint64 InObjectId) const
{
	Session.ReadAccessCheck();
	
	return ObjectIdToConstantData.Contains(InObjectId);
}

bool FRigVMTraceProvider::ReadConstantData(uint64 InObjectId, TFunctionRef<void(const FRigVMTraceConstantData&)> Callback) const
{
	Session.ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToConstantData.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < static_cast<uint32>(ConstantData.Num()))
		{
			Callback(ConstantData[*IndexPtr]);
			return true;
		}
	}

	return false;
}

void FRigVMTraceProvider::EnumerateConstantData(TFunctionRef<void(uint64 OwnerId, const FRigVMTraceConstantData&)> Callback) const
{
	Session.ReadAccessCheck();

	for (TTuple<uint64, uint32> Entry : ObjectIdToConstantData)
	{
		if (Entry.Value < static_cast<uint32>(ConstantData.Num()))
		{
			Callback(Entry.Key, ConstantData[Entry.Value]);
		}
	}
}

bool FRigVMTraceProvider::ReadExecuteTimeline(uint64 InObjectId, TFunctionRef<void(const ExecuteTimeline&)> Callback) const
{
	Session.ReadAccessCheck();

	const uint32* IndexPtr = ObjectIdToExecuteTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		if (*IndexPtr < uint32(ExecuteTimelines.Num()))
		{
			Callback(*ExecuteTimelines[*IndexPtr]);
			return true;
		}
	}

	return false;
}

void FRigVMTraceProvider::EnumerateExecuteTimelines(TFunctionRef<void(uint64 OwnerId, const ExecuteTimeline&)> Callback) const
{
	Session.ReadAccessCheck();

	for (TTuple<uint64, uint32> Entry : ObjectIdToExecuteTimelines)
	{
		if (ExecuteTimelines.IsValidIndex(Entry.Value))
		{
			Callback(Entry.Key, *ExecuteTimelines[Entry.Value]);
		}
	}
}

uint32 FRigVMTraceProvider::GetExecuteTimelineIndex(uint64 InObjectId)
{
	uint32* IndexPtr = ObjectIdToExecuteTimelines.Find(InObjectId);
	if(IndexPtr != nullptr)
	{
		return *IndexPtr;
	}

	uint32 Index = ExecuteTimelines.Num();
	ObjectIdToExecuteTimelines.Add(InObjectId, Index);
	ExecuteTimelines.Add( MakeShared<TraceServices::TPointTimeline<FRigVMTraceExecuteData>>(Session.GetLinearAllocator()));
	return Index;
}

void FRigVMTraceProvider::AppendLiterals(const FRigVMTraceConstantData& InConstantData)
{
	Session.WriteAccessCheck();

	if (ObjectIdToConstantData.Contains(InConstantData.HostId))
	{
		return;
	}

	ObjectIdToConstantData.Add(InConstantData.HostId, static_cast<uint32>(ConstantData.Num()));
	ConstantData.PushBack() = InConstantData;
}

void FRigVMTraceProvider::AppendExecute(const FRigVMTraceExecuteData& InExecuteData)
{
	Session.WriteAccessCheck();

	TSharedPtr<TraceServices::TPointTimeline<FRigVMTraceExecuteData>> Timeline = ExecuteTimelines[GetExecuteTimelineIndex(InExecuteData.HostId)];
	Timeline->AppendEvent(InExecuteData.ProfileTime, InExecuteData);
}

#undef LOCTEXT_NAMESPACE

#endif 