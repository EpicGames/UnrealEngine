// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineBaseTypes.h"
#include "Model/PointTimeline.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "RigVMTrace.h"

#if RIGVM_TRACE_ENABLED

namespace TraceServices { class IAnalysisSession; }

/**
 * The provider is used to provide data to the world given a corresponding analyzer.
 * The analyzer receives unstructured data from the session and forwards the data to the provider.
 * The provider can then provide the data to the target RigVMHost during playback.
 */
class FRigVMTraceProvider : public TraceServices::IProvider
{
public:
	typedef TraceServices::ITimeline<FRigVMTraceExecuteData> ExecuteTimeline;
	
	static FName ProviderName;

	FRigVMTraceProvider(TraceServices::IAnalysisSession& InSession);

	bool HasConstantData(uint64 InObjectId) const;
	bool ReadConstantData(uint64 InObjectId, TFunctionRef<void(const FRigVMTraceConstantData&)> Callback) const;
	void EnumerateConstantData(TFunctionRef<void(uint64 OwnerId, const FRigVMTraceConstantData&)> Callback) const;

	bool ReadExecuteTimeline(uint64 InObjectId, TFunctionRef<void(const ExecuteTimeline&)> Callback) const;
	void EnumerateExecuteTimelines(TFunctionRef<void(uint64 OwnerId, const ExecuteTimeline&)> Callback) const;

	void AppendLiterals(const FRigVMTraceConstantData& InConstantData);
	void AppendExecute(const FRigVMTraceExecuteData& InExecuteData);

private:
	uint32 GetExecuteTimelineIndex(uint64 ObjectId);
	
	TraceServices::IAnalysisSession& Session;

	TMap<uint64, uint32> ObjectIdToConstantData;
	TraceServices::TPagedArray<FRigVMTraceConstantData> ConstantData;
	TMap<uint64, uint32> ObjectIdToExecuteTimelines;
	TArray<TSharedRef<TraceServices::TPointTimeline<FRigVMTraceExecuteData>>> ExecuteTimelines;
};

#endif