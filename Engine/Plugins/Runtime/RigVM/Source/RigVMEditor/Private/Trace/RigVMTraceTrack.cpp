// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/RigVMTraceTrack.h"
#include "RigVMTraceProvider.h"
#include "RigVMTraceAnalyzer.h"
#include "DetailLayoutBuilder.h"
#include "RigVMHost.h"
#include "TraceServices/Model/Frames.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SNumericEntryBox.h"

#if RIGVM_TRACE_ENABLED

#define LOCTEXT_NAMESPACE "RigVMTraceTrack"

static const FLazyName RigVMName("RigVM");

FRewindDebuggerRigVMExecuteTrack::FRewindDebuggerRigVMExecuteTrack(uint64 InHostId)
	: HostId(InHostId)
{
	Icon = FSlateIcon(TEXT("RigVMEditorStyle"), "ExecutionStack.TabIcon");
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();

	// this is temporary
	SAssignNew(DetailsVerticalBox, SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.Padding(8)
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Top)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1)
			
			+ SGridPanel::Slot(0, 0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AbsoluteTime", "Absolute Time"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SGridPanel::Slot(1, 0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(2)
			[
				SNew(SNumericEntryBox<double>)
				.MaxFractionalDigits(3)
				.IsEnabled(false)
				.Value_Lambda([this]() { return AbsoluteTime; })
			]
			+ SGridPanel::Slot(0, 1)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DeltaTime", "Delta Time"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			+ SGridPanel::Slot(1, 1)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(2)
			[
				SNew(SNumericEntryBox<double>)
				.MaxFractionalDigits(3)
				.IsEnabled(false)
				.Value_Lambda([this]() { return DeltaTime; })
			]
		]
	];
}

bool FRewindDebuggerRigVMExecuteTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerRigVMExecuteTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	const TRange<double> RecordingTimeRange = RewindDebugger->GetCurrentViewRange();
	const double RecordingStartTime = RecordingTimeRange.GetLowerBoundValue();
	const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	const double TraceStartTime = TraceTimeRange.GetLowerBoundValue();
	const double TraceEndTime = TraceTimeRange.GetUpperBoundValue();

	auto TraceToRecordingTime = [RecordingStartTime, TraceStartTime](const double& InTraceTime) -> double
	{
		return (InTraceTime - TraceStartTime) + RecordingStartTime;
	};
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FRigVMTraceProvider* Provider = AnalysisSession->ReadProvider<FRigVMTraceProvider>(FRigVMTraceProvider::ProviderName);
	
	if(EventUpdateRequested > 0 && Provider)
	{
		EventUpdateRequested = 0;
	
		EventData->EventPoints.SetNum(0,EAllowShrinking::No);
		EventData->EventWindows.SetNum(1,EAllowShrinking::No);

		EventData->EventWindows[0].Color = FLinearColor::White;
		EventData->EventWindows[0].TimeStart = DBL_MAX;
		EventData->EventWindows[0].TimeEnd = DBL_MIN;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	
		Provider->ReadExecuteTimeline(HostId, [this, TraceStartTime, TraceEndTime, TraceToRecordingTime](const FRigVMTraceProvider::ExecuteTimeline& InTimeline)
		{
			InTimeline.EnumerateEvents(TraceStartTime, TraceEndTime, [this, TraceToRecordingTime](double InStartTime, double InEndTime, uint32 InDepth, const FRigVMTraceExecuteData& ExecuteData)
			{
				if (ExecuteData.HostId == HostId)
				{
					const double StartRecordingTime = TraceToRecordingTime(InStartTime);
					EventData->EventWindows[0].TimeStart = FMath::Min(StartRecordingTime, EventData->EventWindows[0].TimeStart);
					EventData->EventWindows[0].TimeEnd = FMath::Max(StartRecordingTime, EventData->EventWindows[0].TimeEnd);
					//EventData->EventPoints.Add({TraceToRecordingTime(InStartTime), FText(), FText(), FLinearColor::White});
				}
				return TraceServices::EEventEnumerate::Continue;
			});
		});
	}

	double CurrentScrubTime = IRewindDebugger::Instance()->CurrentTraceTime();
	if (PreviousScrubTime != CurrentScrubTime && Provider)
	{
		PreviousScrubTime = CurrentScrubTime;

		const TraceServices::IFrameProvider& FramesProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		TraceServices::FFrame MarkerFrame;
		if (FramesProvider.GetFrameFromTime(TraceFrameType_Game, CurrentScrubTime, MarkerFrame))
		{
			Provider->ReadExecuteTimeline(HostId,
				[this, &MarkerFrame](const FRigVMTraceProvider::ExecuteTimeline& InTimeline)
				{
					InTimeline.EnumerateEvents(MarkerFrame.StartTime, MarkerFrame.EndTime,
						[this](double InStartTime, double InEndTime, uint32 InDepth, const FRigVMTraceExecuteData& InExecuteData)
						{
							AbsoluteTime = InExecuteData.AbsoluteTime;
							DeltaTime = InExecuteData.DeltaTime;
							return TraceServices::EEventEnumerate::Stop;
						}
					);
				}
			);
		}

	}

	EventUpdateRequested++;
	
	constexpr bool bChanged = false;
	return bChanged;
}

TSharedPtr<SWidget> FRewindDebuggerRigVMExecuteTrack::GetDetailsViewInternal()
{
	return DetailsVerticalBox;
}

TSharedPtr<SWidget> FRewindDebuggerRigVMExecuteTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FRewindDebuggerRigVMExecuteTrack::GetEventData);
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FRewindDebuggerRigVMExecuteTrack::GetEventData() const
{
	if (!EventData.IsValid())
	{
		EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	}

	EventUpdateRequested++;

	return EventData;
}

FRewindDebuggerRigVMTrack::FRewindDebuggerRigVMTrack(uint64 InObjectId)
: HostId(InObjectId)
{
	Icon = FSlateIcon(TEXT("RigVMEditorStyle"), "ExecutionStack.TabIcon");
}

TSharedPtr<SWidget> FRewindDebuggerRigVMTrack::GetDetailsViewInternal() 
{
	return nullptr;
}

bool FRewindDebuggerRigVMTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerRigVMTrack::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	const TRange<double> TraceTimeRange = RewindDebugger->GetCurrentTraceRange();
	const double TraceStartTime = TraceTimeRange.GetLowerBoundValue();
	const double TraceEndTime = TraceTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FRigVMTraceProvider* Provider = AnalysisSession->ReadProvider<FRigVMTraceProvider>(FRigVMTraceProvider::ProviderName);
	
	bool bChanged = false;
	
	if(Provider)
	{
		TArray<uint64> UniqueTrackIds;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);

		Provider->ReadExecuteTimeline(HostId, [this, TraceStartTime, TraceEndTime, Provider, AnalysisSession, &UniqueTrackIds](const FRigVMTraceProvider::ExecuteTimeline& InTimeline)
			{
				InTimeline.EnumerateEvents(TraceStartTime, TraceEndTime, [&UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FRigVMTraceExecuteData& InExecuteData)
					{
						UniqueTrackIds.AddUnique(InExecuteData.HostId);
						return TraceServices::EEventEnumerate::Continue;
					}
				);
			}
		);

		UniqueTrackIds.StableSort();
		const int32 TrackCount = UniqueTrackIds.Num();

		if (Children.Num() != TrackCount)
		{
			bChanged = true;
		}

		Children.SetNum(UniqueTrackIds.Num());
		for (int i = 0; i < TrackCount; i++)
		{
			if (!Children[i].IsValid() || Children[i].Get()->GetAssociatedObjectId().GetMainId() != UniqueTrackIds[i])
			{
				Children[i] = MakeShared<FRewindDebuggerRigVMExecuteTrack>(UniqueTrackIds[i]);
				bChanged = true;
			}

			if (Children[i]->Update())
			{
				bChanged = true;
			}
		}
	}
	
	return bChanged;
}

TConstArrayView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> FRewindDebuggerRigVMTrack::GetChildrenInternal(
	TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(Children.GetData(), Children.Num());
}

FName FRewindDebuggerRigVMTrackCreator::GetTargetTypeNameInternal() const
{
	return URigVMHost::StaticClass()->GetFName();
}
	
FName FRewindDebuggerRigVMTrackCreator::GetNameInternal() const
{
	return RigVMName;
}

void FRewindDebuggerRigVMTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({RigVMName, LOCTEXT("RigVM", "RigVM")});
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FRewindDebuggerRigVMTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FRewindDebuggerRigVMTrack>(InObjectId.GetMainId());
}

bool FRewindDebuggerRigVMTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRewindDebuggerRigVMTrack::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();
	
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	bool bHasData = false;
	if (const FRigVMTraceProvider* Provider = AnalysisSession->ReadProvider<FRigVMTraceProvider>(FRigVMTraceProvider::ProviderName))
	{
		Provider->ReadExecuteTimeline(InObjectId.GetMainId(), [&bHasData](const FRigVMTraceProvider::ExecuteTimeline& InTimeline)
	 	{
	 		bHasData = true;
	 	});
	}
	return bHasData;
}

#undef LOCTEXT_NAMESPACE

#endif