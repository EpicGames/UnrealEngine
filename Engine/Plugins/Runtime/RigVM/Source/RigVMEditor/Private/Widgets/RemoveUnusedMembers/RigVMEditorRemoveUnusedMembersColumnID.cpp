// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorRemoveUnusedMembersColumnID.h"

namespace UE::RigVMEditor
{
	const FName FRigVMEditorRemoveUnusedMembersColumnID::LabelRow = "Label";
	const FName FRigVMEditorRemoveUnusedMembersColumnID::SelectedCheckBox = "SelectedCheckBox";
}
