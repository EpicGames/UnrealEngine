// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::RigVMEditor
{
	struct FRigVMEditorRemoveUnusedMembersColumnID
	{
		/** The column to display the member name label */
		static const FName LabelRow;

		/** The column to display a checkbox to chose if the member should be removed */
		static const FName SelectedCheckBox;
	};
}
