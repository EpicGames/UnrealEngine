// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigVMEditorRemoveUnusedMembersListItem.h"

namespace UE::RigVMEditor
{
	FRigVMEditorRemoveUnusedMembersListItem::FRigVMEditorRemoveUnusedMembersListItem(
		const FName& InMemberName, 
		const bool bInInitiallySelected)
		: MemberName(InMemberName)
		, bSelected(bInInitiallySelected)
	{}
}
