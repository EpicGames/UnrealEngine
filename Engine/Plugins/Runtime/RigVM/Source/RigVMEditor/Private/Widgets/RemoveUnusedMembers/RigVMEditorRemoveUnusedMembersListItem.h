// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::RigVMEditor
{
	/** An item in the unused members list */
	struct FRigVMEditorRemoveUnusedMembersListItem
	{
		FRigVMEditorRemoveUnusedMembersListItem() = default;
		FRigVMEditorRemoveUnusedMembersListItem(const FName& InMemberName, const bool bInInitiallySelected);

		/** Returns the member name this item stands for */
		const FName& GetMemberName() const { return MemberName; }

		/** Returns true if the item is selected */
		bool IsSelected() const { return bSelected; }

		/** Sets the itetm selected */
		void SetSelected(const bool bInSelected) { bSelected = bInSelected; }

	private:
		/** The member name this item stands for */
		FName MemberName;

		/** If true the item is selected */
		bool bSelected = false;
	};
}
