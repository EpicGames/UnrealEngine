// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigVMEditorRemoveUnusedMembersDialog.h"

#include "SRigVMEditorRemoveUnusedMembersList.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "SRigVMEditorRemoveUnusedMembersDialog"

namespace UE::RigVMEditor
{
	SRigVMEditorRemoveUnusedMembersDialog::~SRigVMEditorRemoveUnusedMembersDialog()
	{
		if (OnDialogEndedDelegate.IsBound())
		{
			OnDialogEndedDelegate.Execute(EAppReturnType::Cancel, TArray<FName>());
		}
	}

	void SRigVMEditorRemoveUnusedMembersDialog::Construct(
		const FArguments& InArgs,
		const TSharedRef<SWindow>& InOwningWindow,
		const TMap<FRigVMUnusedMemberCategory, TArray<FName>>& CategoryToUnusedMemberNamesMap)
	{
		WeakOwningWindow = InOwningWindow;

		OnDialogEndedDelegate = InArgs._OnDialogEnded;

		const TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox);
		for (const TTuple<FRigVMUnusedMemberCategory, TArray<FName>>& CategoryToUnusedMemberNamesPair : CategoryToUnusedMemberNamesMap)
		{
			ScrollBox->AddSlot()
				[
					CreateCategory(CategoryToUnusedMemberNamesPair.Key, CategoryToUnusedMemberNamesPair.Value)
				];
		}

		ChildSlot
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(4.f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					ScrollBox
				]

				+ SVerticalBox::Slot()

				.AutoHeight()
				.HAlign(HAlign_Right)
				[
					CreateDialogButtons()
				]
			]
		];
	}

	TSharedRef<SWidget> SRigVMEditorRemoveUnusedMembersDialog::CreateCategory(
		const FRigVMUnusedMemberCategory& Category,
		const TArray<FName>& UnusedMemberNames)
	{
		const TSharedRef<SRigVMEditorRemoveUnusedMembersList> UnusedMembersList = 
			SNew(SRigVMEditorRemoveUnusedMembersList)
			.Category(Category)
			.MemberNames(UnusedMemberNames);

		UnusedMembersListViews.Add(UnusedMembersList);
		
		return UnusedMembersList;
	}

	TSharedRef<SWidget> SRigVMEditorRemoveUnusedMembersDialog::CreateDialogButtons()
	{
		return
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MinWidth(60.f)
			.Padding(4.f, 0.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("OkButtonText", "Ok"))
				.OnClicked(this, &SRigVMEditorRemoveUnusedMembersDialog::OnOkClicked)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MinWidth(60.f)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("CancelButtonText", "Cancel"))
				.OnClicked(this, &SRigVMEditorRemoveUnusedMembersDialog::OnCancelClicked)
			];
	}

	FReply SRigVMEditorRemoveUnusedMembersDialog::OnOkClicked()
	{
		if (OnDialogEndedDelegate.IsBound())
		{
			TArray<FName> SelectedMemberNames;
			for (const TSharedPtr<SRigVMEditorRemoveUnusedMembersList>& UnusedMembersListView : UnusedMembersListViews)
			{
				SelectedMemberNames.Append(UnusedMembersListView->GetSelectedMemberNames());
			}

			OnDialogEndedDelegate.Execute(EAppReturnType::Ok, SelectedMemberNames);

			// Only raise once
			OnDialogEndedDelegate.Unbind();
		}

		if (WeakOwningWindow.IsValid())
		{
			WeakOwningWindow.Pin()->RequestDestroyWindow();
		}

		return FReply::Handled();
	}

	FReply SRigVMEditorRemoveUnusedMembersDialog::OnCancelClicked()
	{
		if (OnDialogEndedDelegate.IsBound())
		{
			OnDialogEndedDelegate.Execute(EAppReturnType::Cancel, TArray<FName>());

			// Only raise once
			OnDialogEndedDelegate.Unbind();
		}

		if (WeakOwningWindow.IsValid())
		{
			WeakOwningWindow.Pin()->RequestDestroyWindow();
		}

		return FReply::Handled();
	}
}

#undef LOCTEXT_NAMESPACE
