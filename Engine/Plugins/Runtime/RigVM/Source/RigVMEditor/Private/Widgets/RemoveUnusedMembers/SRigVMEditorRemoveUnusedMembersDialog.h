// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Editor/RemoveUnusedMembers/RigVMEditorRemoveUnusedMembersCategory.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

template <typename ItemType> class SListView;

namespace UE::RigVMEditor
{
	class SRigVMEditorRemoveUnusedMembersList;

	DECLARE_DELEGATE_TwoParams(FRigVMEditorOnRemoveUnusedMembersDialogEnded, const EAppReturnType::Type /** AppReturnType */, const TArray<FName>& /** SelectedMemberNames*/);

	/** The dialog displayed when removing unused members */
	class SRigVMEditorRemoveUnusedMembersDialog
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRigVMEditorRemoveUnusedMembersDialog)
		{}

			/** Delegate executed when the dialog is closing */
			SLATE_EVENT(FRigVMEditorOnRemoveUnusedMembersDialogEnded, OnDialogEnded)

		SLATE_END_ARGS()

		virtual ~SRigVMEditorRemoveUnusedMembersDialog();

		/** Constructs this widget */
		void Construct(
			const FArguments& InArgs, 
			const TSharedRef<SWindow>& InOwningWindow, 
			const TMap<FRigVMUnusedMemberCategory, TArray<FName>>& CategoryToUnusedMemberNamesMap);

	private:
		/** Creates a category with its unused members */
		TSharedRef<SWidget> CreateCategory(
			const FRigVMUnusedMemberCategory& Category, 
			const TArray<FName>& UnusedMemberNames);

		/** Creates the widget that displays the dialog buttons */
		TSharedRef<SWidget> CreateDialogButtons();

		/** Called when the ok button was clicked */
		FReply OnOkClicked();
		
		/** Called when the cancel button was clicked */
		FReply OnCancelClicked();

		/** The widget that owns this dialog */
		TWeakPtr<SWindow> WeakOwningWindow;

		/** The list views of unused members displayed in this dialog */
		TArray<TSharedPtr<SRigVMEditorRemoveUnusedMembersList>> UnusedMembersListViews;

		/** Delegate executed when this dialog ended */
		FRigVMEditorOnRemoveUnusedMembersDialogEnded OnDialogEndedDelegate;
	};
}
