// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigVMEditorRemoveUnusedMembersList.h"

#include "RigVMEditorRemoveUnusedMembersColumnID.h"
#include "RigVMEditorRemoveUnusedMembersListItem.h"
#include "SRigVMEditorRemoveUnusedMembersRow.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRigVMEditorRemoveUnusedMembersList"

namespace UE::RigVMEditor
{
	void SRigVMEditorRemoveUnusedMembersList::Construct(const FArguments& InArgs)
	{
		if (InArgs._MemberNames.IsEmpty())
		{
			return;
		}

		Category = InArgs._Category;

		const bool bInitiallySelectItems = Category.bSafeToRemove;
		for (const FName& MemberName : InArgs._MemberNames)
		{
			Items.Add(MakeShared<FRigVMEditorRemoveUnusedMembersListItem>(MemberName, bInitiallySelectItems));
		}

		ChildSlot
		[
			SAssignNew(ListView, SListView<TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>>)
			.SelectionMode(ESelectionMode::None)
			.ListItemsSource(&Items)
			.HeaderRow(GenerateHeaderRow())
			.OnGenerateRow(this, &SRigVMEditorRemoveUnusedMembersList::GenerateRow)
			.Visibility(this, &SRigVMEditorRemoveUnusedMembersList::GetVisibility)
		];
	}

	TSharedRef<SHeaderRow> SRigVMEditorRemoveUnusedMembersList::GenerateHeaderRow()
	{
		const TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

		// Label Column
		{
			HeaderRow->AddColumn(
				SHeaderRow::FColumn::FArguments()
				.ColumnId(FRigVMEditorRemoveUnusedMembersColumnID::LabelRow)
				.MinSize(40.f)
				.FillWidth(0.9f)
				.InitialSortMode(EColumnSortMode::Ascending)
				.SortMode(this, &SRigVMEditorRemoveUnusedMembersList::GetColumnSortMode, FRigVMEditorRemoveUnusedMembersColumnID::LabelRow)
				.OnSort(this, &SRigVMEditorRemoveUnusedMembersList::SortByColumnID)
				.HeaderContent()
				[
					GenerateHeaderRowLabel()
				]
			);

			ColumnSortMode = EColumnSortMode::Ascending;
			SortedByColumnID = FRigVMEditorRemoveUnusedMembersColumnID::LabelRow;
		}

		// Select column
		{
			HeaderRow->AddColumn(
				SHeaderRow::FColumn::FArguments()
				.ColumnId(FRigVMEditorRemoveUnusedMembersColumnID::SelectedCheckBox)
				.MinSize(40.f)
				.FillWidth(0.5f)
				.VAlignHeader(VAlign_Center)
				.HeaderContent()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.HAlign(HAlign_Center)
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Center)
						.IsChecked(this, &SRigVMEditorRemoveUnusedMembersList::IsHeaderRowChecked)
						.OnCheckStateChanged(this, &SRigVMEditorRemoveUnusedMembersList::OnHeaderRowCheckStateChanged)
					]
				]
			);
		}

		return HeaderRow;
	}

	TSharedRef<ITableRow> SRigVMEditorRemoveUnusedMembersList::GenerateRow(
		TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem> Item, 
		const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SRigVMEditorRemoveUnusedMembersRow, Item, OwnerTable);
	}

	TSharedRef<SWidget> SRigVMEditorRemoveUnusedMembersList::GenerateHeaderRowLabel() const
	{
		if (Category.bSafeToRemove)
		{
			const FText LabelText = FText::Format(
				LOCTEXT("RemoveUnusedPrivateMemberNameLabel", "Unused {0}"),
				Category.CategoryNameText);

			return 
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LabelText)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				];
		}
		else
		{
			const FText LabelText = FText::Format(
				LOCTEXT("ForceRemoveUnusedPublicMemberNameLabel", "Force Remove Unused {0}"),
				Category.CategoryNameText);

			const FText WarningToolTip = FText::Format(
				LOCTEXT("PublicMemberWarningTooltip", "Warning! {0} might be referenced in other Rig Assets."),
				Category.CategoryNameText);

			const TSharedRef<SWidget> WarningIcon = 
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentYellow"))
				.ToolTipText(WarningToolTip);

			const TSharedRef<SWidget> Label =
				SNew(STextBlock)
				.Text(LabelText)
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				.ToolTipText(WarningToolTip);

			return 
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						WarningIcon
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(4.f, 0.f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						Label
					]
				];
		}
	}

	TArray<FName> SRigVMEditorRemoveUnusedMembersList::GetSelectedMemberNames() const
	{
		TArray<FName> SelectedMemberNames;

		Algo::TransformIf(Items, SelectedMemberNames,
			[](const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& Item)
			{
				return Item.IsValid() && Item->IsSelected();
			},
			[](const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& Item)
			{
				return Item->GetMemberName();
			});

		return SelectedMemberNames;
	}

	ECheckBoxState SRigVMEditorRemoveUnusedMembersList::IsHeaderRowChecked() const
	{
		bool bSelected = false;
		for (const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& Item : Items)
		{
			if (!bSelected && Item->IsSelected())
			{
				bSelected = true;
			}

			if (bSelected && !Item->IsSelected())
			{
				return ECheckBoxState::Undetermined;
			}
		}

		if (bSelected)
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}

	void SRigVMEditorRemoveUnusedMembersList::OnHeaderRowCheckStateChanged(ECheckBoxState NewState)
	{
		const bool bSelect = NewState == ECheckBoxState::Checked;

		for (const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& Item : Items)
		{
			Item->SetSelected(bSelect);
		}
	}

	EColumnSortMode::Type SRigVMEditorRemoveUnusedMembersList::GetColumnSortMode(const FName ColumnID) const
	{
		if (SortedByColumnID != ColumnID)
		{
			return EColumnSortMode::None;
		}

		return ColumnSortMode;
	}

	void SRigVMEditorRemoveUnusedMembersList::SortByColumnID(
		const EColumnSortPriority::Type SortPriority, 
		const FName& ColumnID, 
		const EColumnSortMode::Type InSortMode)
	{
		ColumnSortMode = InSortMode;
		SortedByColumnID = ColumnID;

		const bool bAscending = ColumnSortMode == EColumnSortMode::Ascending ? true : false;

		if (ColumnID == FRigVMEditorRemoveUnusedMembersColumnID::LabelRow)
		{
			Algo::StableSort(Items, 
				[bAscending](
					const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& ItemA,
					const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& ItemB)
				{
					if (bAscending)
					{
						return ItemA->GetMemberName().ToString() > ItemB->GetMemberName().ToString();
					}
					else
					{
						return ItemA->GetMemberName().ToString() < ItemB->GetMemberName().ToString();
					}
				});

			if (ListView.IsValid())
			{
				ListView->RequestListRefresh();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
