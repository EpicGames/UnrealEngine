// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/RemoveUnusedMembers/RigVMEditorRemoveUnusedMembersCategory.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

class ITableRow;
class SHeaderRow;
class STableViewBase;
template <typename ItemType> class SListView;

namespace UE::RigVMEditor
{
	struct FRigVMEditorRemoveUnusedMembersListItem;

	/** A list displaying unused members along with an option to remove each item in the list */
	class SRigVMEditorRemoveUnusedMembersList
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRigVMEditorRemoveUnusedMembersList)
			{}

			/** The category to display in the header */
			SLATE_ARGUMENT(FRigVMUnusedMemberCategory, Category)

			/** The member names to display */
			SLATE_ARGUMENT(TArray<FName>, MemberNames)

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs);

		/** Returns the selected member names in this list */
		TArray<FName> GetSelectedMemberNames() const;

	private:
		/** Generates the header row for the list */
		TSharedRef<SHeaderRow> GenerateHeaderRow();

		/** Generates a row in the list */
		TSharedRef<ITableRow> GenerateRow(TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

		/** Generates the header row label */
		TSharedRef<SWidget> GenerateHeaderRowLabel() const;

		/** Returns true if the checkbox in the header row is checked */
		ECheckBoxState IsHeaderRowChecked() const;

		/** Called when the check state of the checkbox in the header row changed */
		void OnHeaderRowCheckStateChanged(ECheckBoxState NewState);

		/** Returns the sort mode for the column */
		EColumnSortMode::Type GetColumnSortMode(const FName ColumnID) const;

		/** Sorts the list by column ID */
		void SortByColumnID(const EColumnSortPriority::Type SortPriority, const FName& ColumnID, const EColumnSortMode::Type InSortMode);

		/** The category to display in the header */
		FRigVMUnusedMemberCategory Category;

		/** The actual list view */
		TSharedPtr<SListView<TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>>> ListView;

		/** The member names in this list */
		TArray<TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>> Items;

		/** The collumn ID by which this list is sorted */
		FName SortedByColumnID;

		/** The sort mode by which sorted by column ID is currently sorted */
		EColumnSortMode::Type ColumnSortMode = EColumnSortMode::None;
	};
}
