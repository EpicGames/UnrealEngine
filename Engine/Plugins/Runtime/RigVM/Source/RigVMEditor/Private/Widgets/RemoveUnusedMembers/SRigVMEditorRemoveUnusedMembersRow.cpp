// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRigVMEditorRemoveUnusedMembersRow.h"

#include "RigVMEditorRemoveUnusedMembersColumnID.h"
#include "RigVMEditorRemoveUnusedMembersListItem.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::RigVMEditor
{
	void SRigVMEditorRemoveUnusedMembersRow::Construct(
		const FArguments& InArgs, 
		const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& InItem,
		const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		WeakItem = InItem;

		SMultiColumnTableRow<TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>>::Construct(
			FSuperRowType::FArguments()
			.Padding(FMargin(8.f, 0.f, 0.f, 0.f)),
			InOwnerTableView);
	}

	TSharedRef<SWidget> SRigVMEditorRemoveUnusedMembersRow::GenerateWidgetForColumn(const FName& ColumnID)
	{
		if (WeakItem.IsValid())
		{
			const TSharedRef<FRigVMEditorRemoveUnusedMembersListItem> Item = WeakItem.Pin().ToSharedRef();

			if (ColumnID == FRigVMEditorRemoveUnusedMembersColumnID::LabelRow)
			{
				return
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromName(Item->GetMemberName()))
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					];
			}
			else if (ColumnID == FRigVMEditorRemoveUnusedMembersColumnID::SelectedCheckBox)
			{
				return
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.HAlign(HAlign_Center)
					[
						SNew(SCheckBox)
						.HAlign(HAlign_Center)
						.IsChecked(this, &SRigVMEditorRemoveUnusedMembersRow::IsChecked)
						.OnCheckStateChanged(this, &SRigVMEditorRemoveUnusedMembersRow::OnCheckStateChanged)
					];
			}
			else
			{
				ensureMsgf(0, TEXT("Unhandled column ID"));
			}
		}

		return SNullWidget::NullWidget;
	}

	ECheckBoxState SRigVMEditorRemoveUnusedMembersRow::IsChecked() const
	{
		const bool bSelected = WeakItem.IsValid() && WeakItem.Pin()->IsSelected() ? true : false;
		
		return bSelected ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SRigVMEditorRemoveUnusedMembersRow::OnCheckStateChanged(ECheckBoxState NewState)
	{
		if (WeakItem.IsValid())
		{
			if (NewState == ECheckBoxState::Checked)
			{
				WeakItem.Pin()->SetSelected(true);
			}
			else if (NewState == ECheckBoxState::Unchecked)
			{
				WeakItem.Pin()->SetSelected(false);
			}
		}
	}
}
