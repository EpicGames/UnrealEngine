// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::RigVMEditor
{
	struct FRigVMEditorRemoveUnusedMembersListItem;

	/** A row displaying a member along with a checkbox to select if it should be removed */
	class SRigVMEditorRemoveUnusedMembersRow
		: public SMultiColumnTableRow<TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>>
	{
	public:
		SLATE_BEGIN_ARGS(SRigVMEditorRemoveUnusedMembersRow)
		{}

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(
			const FArguments& InArgs, 
			const TSharedPtr<FRigVMEditorRemoveUnusedMembersListItem>& InItem,
			const TSharedRef<STableViewBase>& InOwnerTableView);

	private:
		/** Generates a widget for this column ID */
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnID) override;

		/** Returns true if the checkbox in this row is checked */
		ECheckBoxState IsChecked() const;

		/** Called when the check state of the checkbox in this row changed */
		void OnCheckStateChanged(ECheckBoxState NewState);

		/** The item this row displays */
		TWeakPtr<FRigVMEditorRemoveUnusedMembersListItem> WeakItem;
	};
}
