// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RigVMTreeToolkitContext.h"
#include "Widgets/RigVMTreeToolkitNode.h"
#include "Editor/RigVMEditorTools.h"

#define LOCTEXT_NAMESPACE "RigVMTreeToolkitContext"

uint32 FRigVMTreeContext::GetVisibleChildrenHash() const
{
	uint32 Hash = HashOffset;
	for(const TSharedRef<FRigVMTreeFilter>& Filter : Filters)
	{
		Hash = HashCombine(Hash, Filter->GetVisibleChildrenHash());
	}
	return Hash;
}

bool FRigVMTreeContext::FiltersNode(TSharedRef<FRigVMTreeNode> InNode) const
{
	const TSharedRef<FRigVMTreeContext> SharedRef = ToSharedRef();
	if(!InNode->GetVisibleChildren(SharedRef).IsEmpty())
	{
		return false;
	}
	for(const TSharedRef<FRigVMTreeFilter>& Filter : Filters)
	{
		if(Filter->IsEnabled())
		{
			if(Filter->Filters(InNode, SharedRef))
			{
				return true;
			}
		}
	}
	return false;
}

FAssetData FRigVMTreeContext::FindAssetFromAnyPath(const FString& InPartialOrFullPath, bool bConvertToRootPath)
{
	return UE::RigVM::Editor::Tools::FindAssetFromAnyPath(InPartialOrFullPath, bConvertToRootPath);
}

void FRigVMTreeContext::LogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const
{
	OnLogTokenizedMessage.Broadcast(InMessage);
}

void FRigVMTreeContext::LogMessage(const FText& InText) const
{
	return LogMessage(FTokenizedMessage::Create(EMessageSeverity::Info, InText));
}

void FRigVMTreeContext::LogMessage(const FString& InString) const
{
	return LogMessage(FText::FromString(InString));
}

void FRigVMTreeContext::LogWarning(const FText& InText) const
{
	return LogMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, InText));
}

void FRigVMTreeContext::LogWarning(const FString& InString) const
{
	return LogWarning(FText::FromString(InString));
}

void FRigVMTreeContext::LogError(const FText& InText) const
{
	return LogMessage(FTokenizedMessage::Create(EMessageSeverity::Error, InText));
}

void FRigVMTreeContext::LogError(const FString& InString) const
{
	return LogError(FText::FromString(InString));
}

FRigVMTreePhase::FRigVMTreePhase(int32 InID, const FString& InName, const TSharedRef<FRigVMTreeContext>& InContext)
	: ID(InID)
	, Name(InName)
	, bIsActive(false)
	, bAllowsMultiSelection(false)
	, Context(InContext)
	, RootNode(FRigVMTreeRootNode::Create())
{
}

void FRigVMTreePhase::IncrementContextHash()
{
	Context->HashOffset++;
}

TArray<TSharedRef<FRigVMTreeNode>> FRigVMTreePhase::GetAllNodes() const
{
	return RootNode->GetChildren(Context);
}

const TArray<TSharedRef<FRigVMTreeNode>>& FRigVMTreePhase::GetVisibleNodes() const
{
	return RootNode->GetVisibleChildren(Context);
}

void FRigVMTreePhase::AddNode(const TSharedRef<FRigVMTreeNode>& InNode)
{
	RootNode->AddChild(InNode);
}

void FRigVMTreePhase::RemoveNode(const TSharedRef<FRigVMTreeNode>& InNode)
{
	RootNode->RemoveChild(InNode);
}

void FRigVMTreePhase::SetNodes(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes)
{
	RootNode->SetChildren(InNodes);
}

TSharedPtr<FRigVMTreeNode> FRigVMTreePhase::FindVisibleNode(const FString& InPath) const
{
	return FRigVMTreeNode::FindVisibleNodeInSet(GetVisibleNodes(), InPath, Context);
}

#undef LOCTEXT_NAMESPACE
