// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RigVMTreeToolkitFilter.h"
#include "Widgets/RigVMTreeToolkitNode.h"

#define LOCTEXT_NAMESPACE "RigVMTreeToolkitFilter"

bool FRigVMTreeFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	return false;
}

uint32 FRigVMTreeFilter::GetVisibleChildrenHash() const
{
	return HashCombine(GetTypeHash(Type.ToString()), GetTypeHash(bEnabled));
}

bool FRigVMTreePathFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(FilterText.IsEmpty())
	{
		return false;
	}
	return !InNode->GetPath().Contains(FilterText, ESearchCase::IgnoreCase);
}

uint32 FRigVMTreePathFilter::GetVisibleChildrenHash() const
{
	return HashCombine(FRigVMTreeFilter::GetVisibleChildrenHash(), GetTypeHash(FilterText));
}

FText FRigVMTreeEngineContentFilter::GetLabel() const
{
	return LOCTEXT("ShowEngineContent", "Show Engine Content");
}

bool FRigVMTreeEngineContentFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	static const FString Prefix = TEXT("/Game/");
	return !InNode->GetPath().StartsWith(Prefix, ESearchCase::IgnoreCase);
}

FText FRigVMTreeDeveloperContentFilter::GetLabel() const
{
	return LOCTEXT("ShowDeveloperContent", "Show Developer Content");
}

bool FRigVMTreeDeveloperContentFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	static const FString Prefix = TEXT("/Game/Developers");
	return InNode->GetPath().StartsWith(Prefix, ESearchCase::IgnoreCase);
}

#undef LOCTEXT_NAMESPACE
