// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RigVMTreeToolkitNode.h"
#include "Widgets/RigVMTreeToolkitContext.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateIconFinder.h"

FAssetData FRigVMTreeNode::GetAssetData() const
{
	return FRigVMTreeContext::FindAssetFromAnyPath(GetPath(), false);
}

IRigVMEditorAssetInterface* FRigVMTreeNode::GetAsset() const
{
	const FAssetData AssetData = FRigVMTreeContext::FindAssetFromAnyPath(GetPath(), true);
	if(AssetData.IsAssetLoaded())
	{
		UObject* Asset = AssetData.GetAsset();
		return Cast<IRigVMEditorAssetInterface>(Asset);
	}
	return nullptr;
}

FText FRigVMTreeNode::GetLabel() const
{
	FString Label = GetPath();
	if(Parent)
	{
		if(Label.StartsWith(Parent->GetPath(), ESearchCase::IgnoreCase))
		{
			Label = Label.Mid(Parent->GetPath().Len() + 1);
		}
	}
	return FText::FromString(Label);
}

ECheckBoxState FRigVMTreeNode::GetCheckState() const
{
	if(CheckState.IsSet())
	{
		return CheckState.GetValue();
	}

	if(Parent)
	{
		if(Parent->GetCheckState() != ECheckBoxState::Unchecked)
		{
			return ECheckBoxState::Undetermined;
		}
	}
	return ECheckBoxState::Unchecked;
}

void FRigVMTreeNode::SetCheckState(ECheckBoxState InNewState)
{
	CheckState = InNewState;
}

void FRigVMTreeNode::ResetCheckState()
{
	CheckState.Reset();
}

int32 FRigVMTreeNode::GetDepth() const
{
	if(Parent && !Parent->IsA<FRigVMTreeRootNode>())
	{
		return Parent->GetDepth() + 1;
	}
	return 0;
}

TSharedRef<FRigVMTreeNode> FRigVMTreeNode::GetRoot() const
{
	if(Parent && !Parent->IsA<FRigVMTreeRootNode>())
	{
		return Parent->GetRoot(); 
	}
	return ToSharedRef();
}

TSharedPtr<FRigVMTreeNode> FRigVMTreeNode::FindVisibleChild(const FString& InFullPath, const TSharedRef<FRigVMTreeContext>& InContext) const
{
	return FindVisibleNodeInSet(GetVisibleChildren(InContext), InFullPath, InContext);
}

void FRigVMTreeNode::DirtyChildren()
{
	Children.Reset();
	DirtyVisibleChildren();
}

void FRigVMTreeNode::DirtyVisibleChildren()
{
	VisibleChildren.Reset();
	VisibleChildrenHash = UINT32_MAX;
	DirtyVisibleParent();
	RequestRefresh();
}

void FRigVMTreeNode::DirtyVisibleParent() const
{
	if(bIsDirtyingParent || (Parent==nullptr))
	{
		return;
	}
	const TGuardValue<bool> ReEntryGuard(bIsDirtyingParent, true);
	GetParent()->DirtyVisibleChildren();
}

void FRigVMTreeNode::RequestRefresh(bool bForce)
{
	(void)RefreshDelegate.ExecuteIfBound(bForce);
}

const FSlateBrush* FRigVMTreeNode::GetBackgroundImage(bool bIsHovered, bool bIsSelected) const
{
	return nullptr;
}

FSlateColor FRigVMTreeNode::GetBackgroundColor(bool bIsHovered, bool bIsSelected) const
{
	return FSlateColor(EStyleColor::Background);
}

bool FRigVMTreeNode::IsLoaded() const
{
	if(Parent)
	{
		if(!Parent->IsLoaded())
		{
			return false;
		}
	}
	return true;
}

void FRigVMTreeNode::AddChildImpl(const TSharedRef<FRigVMTreeNode>& InChild) const
{
	if(!Children.IsSet())
	{
		Children = TArray<TSharedRef<FRigVMTreeNode>>();
	}
	Children.GetValue().Add(InChild);
	InChild->Parent = this;
}

void FRigVMTreeNode::UpdateChildren(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	TArray<TSharedRef<FRigVMTreeNode>> NewChildren = GetChildrenImpl(InContext);
	if(NewChildren.IsEmpty())
	{
		Children.Reset();
		return;
	}

	bool bRequiresSort = false;
	bool bIsIdentical = false;
	const TArray<TSharedRef<FRigVMTreeNode>> OldChildren = Children.Get(TArray<TSharedRef<FRigVMTreeNode>>());
	
	// remove missing children
	if(Children.IsSet())
	{
		bIsIdentical = OldChildren.Num() == NewChildren.Num();
		for(const TSharedRef<FRigVMTreeNode>& OldChild : OldChildren)
		{
			if(!NewChildren.Contains(OldChild))
			{
				Children.GetValue().Remove(OldChild);
				bIsIdentical = false;
			}
		}
	}

	// add new children
	if(!bIsIdentical)
	{
		for(TSharedRef<FRigVMTreeNode> NewChild : NewChildren)
		{
			if(!OldChildren.Contains(NewChild))
			{
				AddChildImpl(NewChild);
				bRequiresSort = true;
			}
		}
	}

	if(bRequiresSort && Children.IsSet())
	{
		Algo::Sort(Children.GetValue(), [](const TSharedPtr<FRigVMTreeNode>& A, const TSharedPtr<FRigVMTreeNode>& B)
		{
			return FCString::Strcmp(*A->GetPath(),*B->GetPath()) < 0;
		});
	}
}

void FRigVMTreeNode::UpdateVisibleChildren(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	const uint32 ExpectedHash = InContext->GetVisibleChildrenHash();
	if(VisibleChildrenHash == ExpectedHash)
	{
		return;
	}
	VisibleChildrenHash = ExpectedHash;
	
	const TArray<TSharedRef<FRigVMTreeNode>> NewChildren = GetChildren(InContext);
	if(NewChildren.IsEmpty())
	{
		VisibleChildren.Reset();
		return;
	}

	VisibleChildren = NewChildren.FilterByPredicate([InContext](const TSharedRef<FRigVMTreeNode>& Node)
	{
		return !InContext->FiltersNode(Node);
	});

	Algo::Sort(VisibleChildren, [](const TSharedPtr<FRigVMTreeNode>& A, const TSharedPtr<FRigVMTreeNode>& B)
	{
		return FCString::Strcmp(*A->GetPath(),*B->GetPath()) < 0;
	});
}

bool FRigVMTreeNode::ContainsAnyVisibleCheckedNode() const
{
	// operating on the visible children directly here to avoid pulling
	for(const TSharedRef<FRigVMTreeNode>& VisibleChild : VisibleChildren)
	{
		if(VisibleChild->GetCheckState() != ECheckBoxState::Unchecked)
		{
			return true;
		}
		if(VisibleChild->ContainsAnyVisibleCheckedNode())
		{
			return true;
		}
	}
	return false;
}

TSharedPtr<FRigVMTreeNode> FRigVMTreeNode::FindVisibleNodeInSet(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes, const FString& InPath, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(!InPath.IsEmpty())
	{
		for(const TSharedRef<FRigVMTreeNode>& Node : InNodes)
		{
			if(Node->GetPath() == InPath)
			{
				return Node->ToSharedPtr();
			}
		}

		// find the node with the longest matching start path
		int32 MatchingPathLength = 0;
		TSharedPtr<FRigVMTreeNode> MatchingNode;

		for(const TSharedRef<FRigVMTreeNode>& Node : InNodes)
		{
			if(InPath.StartsWith(Node->GetPath()))
			{
				if(Node->GetPath().Len() > MatchingPathLength)
				{
					MatchingPathLength = Node->GetPath().Len();
					MatchingNode = Node->ToSharedPtr();
				}
			}
		}

		if(MatchingNode)
		{
			return MatchingNode->FindVisibleChild(InPath, InContext);
		}
	}
	return nullptr;
}

FRigVMTreeRootNode::FRigVMTreeRootNode()
	: FRigVMTreeNode(TEXT("Root"))
{
}

void FRigVMTreeRootNode::AddChild(const TSharedRef<FRigVMTreeNode>& InNode)
{
	check(!InNode->GetParent().IsValid() || InNode->GetParent().Get() == this);
	AddChildImpl(InNode);
	DirtyVisibleChildren();
}

void FRigVMTreeRootNode::RemoveChild(const TSharedRef<FRigVMTreeNode>& InNode)
{
	if(Children.IsSet())
	{
		if(Children.GetValue().Remove(InNode) > 0)
		{
			DirtyVisibleChildren();
		}
	}
}

void FRigVMTreeRootNode::SetChildren(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes)
{
	if(InNodes.IsEmpty())
	{
		return;
	}
	for(const TSharedRef<FRigVMTreeNode>& Node : InNodes)
	{
		check(!Node->GetParent().IsValid() || Node->GetParent().Get() == this);
		AddChildImpl(Node);
	}
	DirtyVisibleChildren();
}

const FSlateBrush* FRigVMTreeCategoryNode::GetBackgroundImage(bool bIsHovered, bool bIsSelected) const
{
	return FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle");
}

FSlateColor FRigVMTreeCategoryNode::GetBackgroundColor(bool bIsHovered, bool bIsSelected) const
{
	if(bIsSelected)
	{
		return FSlateColor(EStyleColor::Select); 
	}
	if(bIsHovered)
	{
		return FSlateColor(EStyleColor::Hover); 
	}
	return FAppStyle::Get().GetSlateColor("Colors.Header"); 
}

FRigVMTreePackageNode::FRigVMTreePackageNode(const FAssetData& InAssetData)
	: FRigVMTreeCategoryNode(InAssetData.GetObjectPathString())
	, bRetrievedTags(false)
{
	SoftObjectPath = InAssetData.GetSoftObjectPath();
	bIsLoaded = InAssetData.IsAssetLoaded();
}

void FRigVMTreePackageNode::Initialize()
{
	FCoreUObjectDelegates::OnAssetLoaded.AddSP(this, &FRigVMTreePackageNode::HandleAssetLoaded);
}

FText FRigVMTreePackageNode::GetLabel() const
{
	FText Label = FRigVMTreeCategoryNode::GetLabel();
	const FString LabelString = Label.ToString();
	FString Right;
	if(LabelString.Split(TEXT("."), nullptr, &Right, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		if(LabelString.EndsWith(FString::Printf(TEXT("/%s.%s"), *Right, *Right), ESearchCase::CaseSensitive))
		{
			Label = FText::FromString(LabelString.LeftChop(Right.Len() + 1));
		}
	}
	return Label;
}

bool FRigVMTreePackageNode::IsLoaded() const
{
	if(!FRigVMTreeCategoryNode::IsLoaded())
	{
		return false;
	}

	if(!bIsLoaded.IsSet())
	{
		const TSoftObjectPtr<UObject> SoftObject(SoftObjectPath);
		return SoftObject.Get() != nullptr;

	}
	
	return bIsLoaded.GetValue();
}

const TArray<FRigVMTag>& FRigVMTreePackageNode::GetTags() const
{
	if(!bRetrievedTags)
	{
		if(IsLoaded())
		{
			if(const IRigVMEditorAssetInterface* Blueprint = GetAsset())
			{
				Tags = Blueprint->GetAssetVariant().Tags;
			}
			else
			{
				const FAssetData AssetData = GetAssetData();
				if(AssetData.FindTag(TEXT("AssetVariant")))
				{
					//const FRigVMVariant& Variant = AssetData.GetTagValueRef<FRigVMVariant>(GET_MEMBER_NAME_CHECKED(URigVMBlueprint, AssetVariant));
					//Tags = Variant.Tags;
				}
			}
		}
		bRetrievedTags = true;
	}
	return FRigVMTreeCategoryNode::GetTags();
}

const FSlateBrush* FRigVMTreePackageNode::GetIconAndTint(FLinearColor& OutColor) const
{
	if(!IconBrush.IsSet())
	{
		const FAssetData AssetData = GetAssetData();
		if(AssetData.IsValid())
		{
			if(UClass* Class = AssetData.GetClass())
			{
				Icon = FSlateIconFinder::FindIconForClass(Class);
				IconBrush = Icon.GetIcon();
			}
		}
	}

	return IconBrush.Get(nullptr);
}

void FRigVMTreePackageNode::HandleAssetLoaded(UObject* InAsset)
{
	if(InAsset)
	{
		if(InAsset->GetPathName() == SoftObjectPath.ToString())
		{
			DirtyChildren();
			bIsLoaded = true;
		}
	}
}
