// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RigVMTreeToolkitTask.h"
#include "Widgets/RigVMTreeToolkitNode.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/RigVMTreeToolkitContext.h"
#include "Kismet2/KismetEditorUtilities.h"

#define LOCTEXT_NAMESPACE "RigVMTreeToolkitTask"

FRigVMTreeLoadPackageForNodeTask::FRigVMTreeLoadPackageForNodeTask(const TSharedRef<FRigVMTreeNode>& InNode)
{
	static const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetData = AssetRegistry.GetAssetByObjectPath(InNode->GetPath());
}

bool FRigVMTreeLoadPackageForNodeTask::Execute(const TSharedRef<FRigVMTreePhase>& InPhase)
{
	if(AssetData.IsValid())
	{
		if(!AssetData.IsAssetLoaded())
		{
			const TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
			Message->AddText(LOCTEXT("Loading", "Loading"));
			Message->AddToken(FAssetNameToken::Create(AssetData.PackageName.ToString()));
			InPhase->GetContext()->LogMessage(Message);

			// load synchronously
			if(AssetData.GetAsset() == nullptr)
			{
				InPhase->GetContext()->LogError(FString::Printf(TEXT("Asset '%s' cannot be loaded."), *AssetData.GetObjectPathString()));
				return false;
			}
		}
		return true;
	}
	InPhase->GetContext()->LogError(TEXT("Provided AssetData is not valid."));
	return false;
}

FRigVMCompileBlueprintTask::FRigVMCompileBlueprintTask(const TSharedRef<FRigVMTreeNode>& InNode)
{
	ObjectPath = InNode->GetPath();
}

bool FRigVMCompileBlueprintTask::Execute(const TSharedRef<FRigVMTreePhase>& InPhase)
{
	if(const UObject* Object = ObjectPath.TryLoad())
	{
		if(UBlueprint* Blueprint = Object->GetTypedOuter<UBlueprint>())
		{
			const TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
			Message->AddText(LOCTEXT("Compiling", "Compiling"));
			Message->AddToken(FAssetNameToken::Create(Blueprint->GetOutermost()->GetPathName()));
			InPhase->GetContext()->LogMessage(Message);

			FCompilerResultsLog CompilerResults;
			CompilerResults.SetSourcePath(Blueprint->GetPathName());
			CompilerResults.bSilentMode = true;
			CompilerResults.BeginEvent(TEXT("Compile"));
			FKismetEditorUtilities::CompileBlueprint(Blueprint, EBlueprintCompileOptions::SkipGarbageCollection | EBlueprintCompileOptions::SkipSave, &CompilerResults);
			CompilerResults.EndEvent();

			for(const TSharedRef<FTokenizedMessage>& CompilerMessage : CompilerResults.Messages)
			{
				InPhase->GetContext()->LogMessage(CompilerMessage);
			}
			return true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
