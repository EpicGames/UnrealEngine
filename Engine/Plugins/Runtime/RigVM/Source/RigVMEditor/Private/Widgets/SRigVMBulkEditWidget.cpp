// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMBulkEditWidget.h"
#include "RigVMDeveloperModule.h"
#include "Widgets/SWindow.h"
#include "Widgets/SRigVMLogWidget.h"
#include "Widgets/Input/SButton.h"
#include "SPrimaryButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Styling/AppStyle.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RigVMEditorModule.h"
#include "Misc/UObjectToken.h"
#include "Dialogs/Dialogs.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor/RigVMNewEditor.h"

#define LOCTEXT_NAMESPACE "SRigVMBulkEditWidget"

//////////////////////////////////////////////////////////////////////////
// SRigVMBulkEditWidget

SRigVMBulkEditWidget::~SRigVMBulkEditWidget()
{
}

void SRigVMBulkEditWidget::Construct(const FArguments& InArgs)
{
	Phases = InArgs._Phases;
	bEnableUndo = InArgs._EnableUndo;
	bCloseOnSuccess = InArgs._CloseOnSuccess;

	OnPhaseActivated = InArgs._OnPhaseActivated;

	BulkEditTitle = InArgs._BulkEditTitle;
	BulkEditConfirmMessage = InArgs._BulkEditConfirmMessage;
	BulkEditConfirmIniField = InArgs._BulkEditConfirmIniField;

	if(BulkEditTitle.Get().IsEmpty())
	{
		BulkEditTitle = LOCTEXT("BulkEdit", "Bulk Edit");
	}
	if(BulkEditConfirmMessage.Get().IsEmpty())
	{
		BulkEditConfirmMessage = LOCTEXT("ConfirmBulkEditWithoutUndo", "This Bulk Edit will run with support for Undo. Are you sure?");
	}

	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);
	TSharedRef<SHorizontalBox> MainHorizontalBox = SNew(SHorizontalBox);
	if(InArgs._LeftWidget)
	{
		MainHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		[
			InArgs._LeftWidget.ToSharedRef()
		];
	}
	
	MainHorizontalBox->AddSlot()
	.FillWidth(1)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(TreeView, SRigVMChangesTreeView)
		.Visibility_Lambda([this]()
		{
			return !bShowLog ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.Phase(this, &SRigVMBulkEditWidget::GetActivePhasePtr)
		.OnNodeSelected(InArgs._OnNodeSelected)
		.OnNodeDoubleClicked(InArgs._OnNodeDoubleClicked)
	];

	MainHorizontalBox->AddSlot()
	.FillWidth(1)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(BulkEditLogWidget, SRigVMLogWidget)
		.LogName(TEXT("RigVMBulkEditLog"))
		.LogLabel(LOCTEXT("RigVMBulkEditLog", "Bulk Edit Log"))
		.Visibility_Lambda([this]()
		{
			return bShowLog ? EVisibility::Visible : EVisibility::Collapsed;
		})
	];
	
	BulkEditLogWidget->GetListing()->ClearMessages();

	if(InArgs._RightWidget)
	{
		MainHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		[
			InArgs._RightWidget.ToSharedRef()
		];
	}

	if(InArgs._HeaderWidget)
	{
		MainVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			InArgs._HeaderWidget.ToSharedRef()
		];
	}
	
	MainVerticalBox->AddSlot()
	.VAlign(VAlign_Fill)
	.FillHeight(1.0f)		// This is required to make the scrollbar work, as content overflows Slate containers by default
	[
		MainHorizontalBox
	];

	if(InArgs._FooterWidget)
	{
		MainVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Bottom)
		[
			InArgs._FooterWidget.ToSharedRef()
		];
	}

	MainVerticalBox->AddSlot()
	.AutoHeight()
	.Padding(0, 2)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				SNew(SProgressBar)
				.Visibility(this, &SRigVMBulkEditWidget::GetTasksProgressVisibility)
				.Percent(this, &SRigVMBulkEditWidget::GetTasksProgressPercentage)
			]
		
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 4, 0, 0)
			.HAlign(HAlign_Fill)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				[
					SNew(SButton)
					.Visibility(this, &SRigVMBulkEditWidget::GetBackButtonVisibility)
					.IsEnabled(this, &SRigVMBulkEditWidget::IsBackButtonEnabled)
					.Text(LOCTEXT("Back", "Back"))
					.OnClicked(this, &SRigVMBulkEditWidget::OnBackButtonClicked)
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SSpacer)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Visibility(this, &SRigVMBulkEditWidget::GetCancelButtonVisibility)
					.IsEnabled(this, &SRigVMBulkEditWidget::IsCancelButtonEnabled)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SRigVMBulkEditWidget::OnCancelButtonClicked)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				[
					SNew(SPrimaryButton)
					.Visibility(this, &SRigVMBulkEditWidget::GetPrimaryButtonVisibility)
					.IsEnabled(this, &SRigVMBulkEditWidget::IsPrimaryButtonEnabled)
					.Text(this, &SRigVMBulkEditWidget::GetPrimaryButtonText)
					.OnClicked(this, &SRigVMBulkEditWidget::OnPrimaryButtonClicked)
				]
			]
		]
	];

	ChildSlot
	[
		MainVerticalBox
	];

	BulkEditLogWidget->BindLog(LogRigVMDeveloper.GetCategoryName());
	BulkEditLogWidget->BindLog(LogRigVMEditor.GetCategoryName());
	ActivatePhase(InArgs._PhaseToActivate);
}

void SRigVMBulkEditWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// update the tree view even if it is hidden
	if(bShowLog)
	{
		TreeView->RefreshFilteredNodesIfRequired();
	}

	FScopeLock _(&TasksCriticalSection);
	if(!RemainingTasks.IsEmpty())
	{
		if(bEnableUndo && !Transaction.IsValid())
		{
			Transaction = MakeShareable(new FScopedTransaction(LOCTEXT("BulkEdit", "Bulk Edit")));
		}
		
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]()
		{
			TSharedPtr<FRigVMTreeTask> Task;
			{
				FScopeLock _(&TasksCriticalSection);
				if(RemainingTasks.IsEmpty())
				{
					return;
				}
				Task = RemainingTasks[0].ToSharedPtr();
			}

			if(Task.IsValid())
			{
				Task->SetEnableUndo(bEnableUndo);
				
				FScopedScriptExceptionHandler ScopedScriptExceptionHandler(
					[this](ELogVerbosity::Type InVerbosity, const TCHAR* InMessage, const TCHAR* InStackMessage)
					{
						OnScriptException(InVerbosity, InMessage, InStackMessage);
					});

				const TSharedRef<FRigVMTreePhase>& Phase = GetActivePhase();

				TSharedPtr<TGuardValue<FRigVMReportDelegate>> ReportDelegateGuard; 
				if(FRigVMEditorAssetInterfacePtr Blueprint = Task->GetRigVMAsset(Phase))
				{
					ReportDelegateGuard = MakeShareable(new TGuardValue<FRigVMReportDelegate>(
						Blueprint->GetVMCompileSettings().ASTSettings.ReportDelegate,
						FRigVMReportDelegate::CreateLambda(
						[this](EMessageSeverity::Type InSeverity, UObject* InSubject, const FString& InMessage)
						{
							TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity); 
							if(InSubject)
							{
								FString Left, Right;
								if(InMessage.Split(TEXT("@@"), &Left, &Right))
								{
									Left.TrimStartAndEndInline();
									Right.TrimStartAndEndInline();
									Message->AddText(FText::FromString(Left));

									const TSharedRef<FUObjectToken> SubjectToken = FUObjectToken::Create(InSubject);
									TWeakObjectPtr<UObject> WeakSubject(InSubject);
									SubjectToken->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([WeakSubject](const TSharedRef<IMessageToken>&)
									{
										if(WeakSubject.IsValid())
										{
											if (FRigVMEditorAssetInterfacePtr EditorAsset = IRigVMEditorAssetInterface::GetInterfaceOuter(WeakSubject.Get()))
											{
												GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorAsset->GetAssetObjectForEditor());
														
												if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(EditorAsset.GetObject(), true))
												{
													if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
													{
														RigVMEditor->HandleJumpToHyperlink(WeakSubject.Get());
													}
												}
											}
										}
									}));
									Message->AddToken(SubjectToken);
									Message->AddText(FText::FromString(Right));
								}
							}
							else
							{
								Message->AddText(FText::FromString(InMessage));
							}
							BulkEditLogWidget->GetListing()->AddMessage(Message);
						})
					));
				}

				if(Task->Execute(Phase))
				{
					FScopeLock _(&TasksCriticalSection);
					RemainingTasks.Remove(Task.ToSharedRef());
					CompletedTasks.Add(Task.ToSharedRef());

					const TArray<FString> AffectedPaths = Task->GetAffectedNodes();
					for(const FString& AffectedPath : AffectedPaths)
					{
						if(const TSharedPtr<FRigVMTreeNode> Node = Phase->FindVisibleNode(AffectedPath))
						{
							if(const TSharedPtr<FRigVMTreeNode> Parent = Node->GetParent())
							{
								Parent->DirtyChildren();
							}
							else
							{
								Node->DirtyChildren();
							}
						}
					}

					if(Task->RequiresRefresh() || !AffectedPaths.IsEmpty())
					{
						TreeView->RequestRefresh_AnyThread(true);
					}
				}
                else
                {
                    CancelTasks();
                }

				ReportDelegateGuard.Reset();
			}
		});		
	}
	else if(!CompletedTasks.IsEmpty())
	{
		CompletedTasks.Reset();
		Transaction.Reset();
		bTasksSucceeded = true;
		TreeView->RequestRefresh_AnyThread(true);
	}
}

FText SRigVMBulkEditWidget::GetDialogTitle() const
{
	if(bShowLog)
	{
		return LOCTEXT("ProcessingTasks", "Performing Bulk Edit...");
	}
	return FText::FromString(GetActivePhase()->GetName());
}

bool SRigVMBulkEditWidget::AreTasksInProgress() const
{
	FScopeLock _(&TasksCriticalSection);
	return !RemainingTasks.IsEmpty();
}

EVisibility SRigVMBulkEditWidget::GetTasksProgressVisibility() const
{
	return AreTasksInProgress() ? EVisibility::Visible : EVisibility::Collapsed;
}

TOptional<float> SRigVMBulkEditWidget::GetTasksProgressPercentage() const
{
	if(!AreTasksInProgress())
	{
		return TOptional<float>();
	}

	FScopeLock _(&TasksCriticalSection);
	const int32 NumTotalTasks = CompletedTasks.Num() + RemainingTasks.Num();
	if(NumTotalTasks == 0)
	{
		return TOptional<float>();
	}
	return float(CompletedTasks.Num()) / float(NumTotalTasks);
}

void SRigVMBulkEditWidget::OnLogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const
{
	BulkEditLogWidget->GetListing()->AddMessage( InMessage );
}

void SRigVMBulkEditWidget::OnScriptException(ELogVerbosity::Type InVerbosity, const TCHAR* InMessage, const TCHAR* InStackMessage)
{
	if(InMessage)
	{
		switch(InVerbosity)
		{
			case ELogVerbosity::Fatal:
			case ELogVerbosity::Error:
			{
				OnLogMessage(FTokenizedMessage::Create(EMessageSeverity::Error, FText::FromString(InMessage)));
				break;
			}
			case ELogVerbosity::Warning:
			{
				OnLogMessage(FTokenizedMessage::Create(EMessageSeverity::Warning, FText::FromString(InMessage)));
				break;
			}
			default:
			{
				OnLogMessage(FTokenizedMessage::Create(EMessageSeverity::Info, FText::FromString(InMessage)));
				break;
			}
		}
	}
}

void SRigVMBulkEditWidget::CloseDialog()
{
	const TSharedPtr<SWindow> OwningWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (OwningWindow)
	{
		OwningWindow->RequestDestroyWindow();
	}
}

EVisibility SRigVMBulkEditWidget::GetBackButtonVisibility() const
{
	if(IsReadyToClose())
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

bool SRigVMBulkEditWidget::IsBackButtonEnabled() const
{
	if(IsReadyToClose())
	{
		return false;
	}
	if(bShowLog)
	{
		return true;
	}
	if(AreTasksInProgress())
	{
		return false;
	}
	return ActivatedPhaseIDs.Num() > 1;
}

FReply SRigVMBulkEditWidget::OnBackButtonClicked()
{
	if(bShowLog)
	{
		bShowLog = false;
		return FReply::Handled();
	}
	
	if(ActivatedPhaseIDs.Num() > 1)
	{
		bShowLog = false;
		ActivatedPhaseIDs.Pop();
		const int32 PreviousID = ActivatedPhaseIDs.Pop();
		if(ActivatePhase(PreviousID))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

EVisibility SRigVMBulkEditWidget::GetCancelButtonVisibility() const
{
	if(IsReadyToClose())
	{
		return EVisibility::Collapsed;
	}
	if(AreTasksInProgress())
	{
		return EVisibility::Visible;
	}
	return GetActivePhase()->IsCancelButtonVisible().Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SRigVMBulkEditWidget::IsCancelButtonEnabled() const
{
	if(IsReadyToClose())
	{
		return false;
	}
	if(AreTasksInProgress())
	{
		return true;
	}
	return GetActivePhase()->IsCancelButtonEnabled().Get(true);
}

FReply SRigVMBulkEditWidget::OnCancelButtonClicked()
{
	if(AreTasksInProgress())
	{
		CancelTasks();
		return FReply::Handled();
	}
	if(!GetActivePhase()->OnCancel().IsBound())
	{
		CloseDialog();
		return FReply::Handled();
	}
	FReply Reply = GetActivePhase()->Cancel();
	TreeView->RequestRefresh_AnyThread(true);
	return Reply;
}

EVisibility SRigVMBulkEditWidget::GetPrimaryButtonVisibility() const
{
	if(IsReadyToClose())
	{
		return EVisibility::Visible;
	}
	return GetActivePhase()->IsPrimaryButtonVisible().Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SRigVMBulkEditWidget::IsPrimaryButtonEnabled() const
{
	if(IsReadyToClose())
	{
		return true;
	}
	if(AreTasksInProgress())
	{
		return false;
	}
	return GetActivePhase()->IsPrimaryButtonEnabled().Get(true);
}

FText SRigVMBulkEditWidget::GetPrimaryButtonText() const
{
	if(IsReadyToClose())
	{
		return LOCTEXT("Close", "Close");
	}
	return GetActivePhase()->PrimaryButtonText().Get(LOCTEXT("Ok", "Ok"));
}

bool SRigVMBulkEditWidget::IsReadyToClose() const
{
	return bTasksSucceeded && bCloseOnSuccess;
}

FReply SRigVMBulkEditWidget::OnPrimaryButtonClicked()
{
	if(IsReadyToClose())
	{
		CloseDialog();
		return FReply::Handled();
	}
	
	FReply Reply = GetActivePhase()->PrimaryAction();
	if(Reply.IsEventHandled())
	{
		if(GetActivePhase() == Phases.Last())
        {
			FScopeLock _(&TasksCriticalSection);
        	bShowLog = !RemainingTasks.IsEmpty();
        }
	}
	TreeView->RequestRefresh_AnyThread(true);
	return Reply;
}

void SRigVMBulkEditWidget::QueueTasks(const TArray<TSharedRef<FRigVMTreeTask>>& InTasks)
{
	FScopeLock _(&TasksCriticalSection);

	if(!bEnableUndo)
	{
		const bool bRequiresUndo = InTasks.ContainsByPredicate([](const TSharedRef<FRigVMTreeTask>& Task)
		{
			return Task->RequiresUndo();
		});

		if(bRequiresUndo)
		{
			FSuppressableWarningDialog::FSetupInfo Info(BulkEditConfirmMessage.Get(), BulkEditTitle.Get(), BulkEditConfirmIniField.Get());
			Info.ConfirmText = LOCTEXT("Yes", "Yes");
			Info.CancelText = LOCTEXT("No", "No");

			const FSuppressableWarningDialog ConfirmationDialog(Info);
			if(!ConfirmationDialog.ShowModal())
			{
				return;
			}
		}
	}
	
	RemainingTasks.Append(InTasks);
}

void SRigVMBulkEditWidget::CancelTasks()
{
	FScopeLock _(&TasksCriticalSection);
	RemainingTasks.Reset();
	CompletedTasks.Reset();
	if(Transaction.IsValid())
	{
		Transaction->Cancel();
		Transaction.Reset();
	}
	bTasksSucceeded = false;
}

TSharedPtr<FRigVMTreePhase> SRigVMBulkEditWidget::GetActivePhasePtr() const
{
	for(const TSharedRef<FRigVMTreePhase>& Phase : Phases)
	{
		if(Phase->IsActive())
		{
			return Phase;
		}
	}
	return nullptr;
}

TSharedRef<FRigVMTreePhase> SRigVMBulkEditWidget::GetActivePhase() const
{
	for(const TSharedRef<FRigVMTreePhase>& Phase : Phases)
	{
		if(Phase->IsActive())
		{
			return Phase;
		}
	}
	static const TSharedRef<FRigVMTreeContext> EmptyContext = FRigVMTreeContext::Create();
	static const TSharedRef<FRigVMTreePhase> EmptyPhase = FRigVMTreePhase::Create(INDEX_NONE, TEXT("Default"), EmptyContext);
	return EmptyPhase;
}

TSharedPtr<FRigVMTreePhase> SRigVMBulkEditWidget::FindPhase(int32 InID) const
{
	for(const TSharedRef<FRigVMTreePhase>& Phase : Phases)
	{
		if(Phase->GetID() == InID)
		{
			return Phase.ToSharedPtr();
		}
	}
	return nullptr;
}

bool SRigVMBulkEditWidget::ActivatePhase(int32 InID)
{
	FString PreviouslySelectedPath;
	TArray<TSharedRef<FRigVMTreeNode>> PreviousSelection = GetSelectedNodes();
	if(!PreviousSelection.IsEmpty())
	{
		PreviouslySelectedPath = PreviousSelection[0]->GetPath();
	}
	
	for(const TSharedRef<FRigVMTreePhase>& Phase : Phases)
	{
		Phase->bIsActive = false;
	}

	bool bResult = false;
	if(const TSharedPtr<FRigVMTreePhase> Phase = FindPhase(InID))
	{
		Phase->bIsActive = true;
		Phase->GetContext()->OnLogTokenizedMessage.RemoveAll(this);
		Phase->GetContext()->OnLogTokenizedMessage.AddSP(this, &SRigVMBulkEditWidget::OnLogMessage);;
		OnPhaseActivated.Execute(Phase.ToSharedRef());
		ActivatedPhaseIDs.Add(Phase->GetID());
		Phase->OnQueueTasks().BindSP(this, &SRigVMBulkEditWidget::QueueTasks);
		bResult = true;
	}

	TreeView->OnPhaseChanged();

	if(!PreviouslySelectedPath.IsEmpty())
	{
		if(const TSharedPtr<FRigVMTreeNode> Node = GetActivePhase()->FindVisibleNode(PreviouslySelectedPath))
		{
			TreeView->SetSelection(Node, true);
		}
	}
	
	return bResult;
}

TSharedRef<FRigVMTreeContext> SRigVMBulkEditWidget::GetContext() const
{
	return GetActivePhase()->GetContext();
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMBulkEditWidget::GetSelectedNodes() const
{
	return TreeView->GetSelectedNodes();
}

bool SRigVMBulkEditWidget::HasAnyVisibleCheckedNode() const
{
	return TreeView->HasAnyVisibleCheckedNode();
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMBulkEditWidget::GetCheckedNodes() const
{
	return TreeView->GetCheckedNodes();
}

#undef LOCTEXT_NAMESPACE
