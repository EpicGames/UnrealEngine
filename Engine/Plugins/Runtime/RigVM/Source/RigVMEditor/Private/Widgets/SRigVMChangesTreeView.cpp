// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMChangesTreeView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Styling/AppStyle.h"
#include "Misc/UObjectToken.h"
#include "Layout/WidgetPath.h"
#include "Framework/MultiBox//MultiBoxBuilder.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Widgets/SRigVMVariantWidget.h"
#define LOCTEXT_NAMESPACE "SRigVMChangesTreeView"

SRigVMChangesTreeRow::~SRigVMChangesTreeRow()
{
	if(Node)
	{
		Node->RefreshDelegate.Unbind();
	}
}

void SRigVMChangesTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Node = InArgs._Node;
	OwningWidget = InArgs._OwningWidget.Get();

	Node->RefreshDelegate.BindRaw(this, &SRigVMChangesTreeRow::RequestRefresh);
	
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &SRigVMChangesTreeRow::GetBackgroundImage)
		.BorderBackgroundColor(this, &SRigVMChangesTreeRow::GetBackgroundColor)
		.Padding(3.0f, 2.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			[
				SNew(SBox)
				.WidthOverride(this, &SRigVMChangesTreeRow::GetIndentWidth)
			]
			
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(0)
			.AutoWidth()
			[
				SNew( SImage )
				.Image(this, &SRigVMChangesTreeRow::GetExpanderImage) 
				.Visibility( this, &SRigVMChangesTreeRow::GetExpanderVisibility )
				.OnMouseButtonDown(this, &SRigVMChangesTreeRow::OnExpanderMouseButtonDown)
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2.f, 0.f, 3.f, 0.f)
			.AutoWidth()
			[
				SNew( SImage )
				.Image(this, &SRigVMChangesTreeRow::GetIcon)
				.ColorAndOpacity(this, &SRigVMChangesTreeRow::GetIconColor)
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 4.0f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Node->GetLabel())
				.ColorAndOpacity(this, &SRigVMChangesTreeRow::GetTextColor)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8, 0, 0, 0)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SRigVMVariantTagWidget)
				.Visibility(GetVariantTags().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
				.CanAddTags(false)
				.EnableContextMenu(false)
				.EnableTick(false)
				.Orientation(EOrientation::Orient_Horizontal)
				.OnGetTags(this, &SRigVMChangesTreeRow::GetVariantTags)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SSpacer)
				.Size(FVector2D(10000,0))
			]
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 4.0f)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.Visibility(this, &SRigVMChangesTreeRow::GetCheckBoxVisibility)
				.IsChecked(this, &SRigVMChangesTreeRow::GetCheckBoxState)
				.OnCheckStateChanged(this, &SRigVMChangesTreeRow::OnCheckBoxStateChanged)
			]
		]
	];
	
	STableRow< TSharedRef<FRigVMTreeNode> >::ConstructInternal(
		STableRow<TSharedRef<FRigVMTreeNode>>::FArguments(), InOwnerTableView);
}

const FSlateBrush* SRigVMChangesTreeRow::GetBackgroundImage() const
{
	return Node->GetBackgroundImage(IsHovered(), IsSelected());
}

FSlateColor SRigVMChangesTreeRow::GetBackgroundColor() const
{
	return Node->GetBackgroundColor(IsHovered(), IsSelected());
}

FOptionalSize SRigVMChangesTreeRow::GetIndentWidth() const
{
	return float(Node->GetDepth()) * 16.0f;
}

FSlateColor SRigVMChangesTreeRow::GetTextColor() const
{
	if(!Node->IsLoaded())
	{
		return FLinearColor(0.3f, 0.3f, 0.3f, 1.f);
	}
	return FSlateColor::UseForeground();
}

const FSlateBrush* SRigVMChangesTreeRow::GetIcon() const
{
	FLinearColor Color = FLinearColor::White;
	return Node->GetIconAndTint(Color);
}

FSlateColor SRigVMChangesTreeRow::GetIconColor() const
{
	FLinearColor Color = FLinearColor::White;
	(void)Node->GetIconAndTint(Color);
	if(!Node->IsLoaded())
	{
		Color = Color * FLinearColor(0.1f, 0.1f, 0.1f, 1.f);
	}
	return Color;
}

const FSlateBrush* SRigVMChangesTreeRow::GetExpanderImage() const
{
	if(IsItemExpanded())
	{
		if(IsHovered())
		{
			return FAppStyle::GetBrush("TreeArrow_Expanded_Hovered");
		}
		return FAppStyle::GetBrush("TreeArrow_Expanded");
	}
	if(IsHovered())
	{
		return FAppStyle::GetBrush("TreeArrow_Collapsed_Hovered");
	}
	return FAppStyle::GetBrush("TreeArrow_Collapsed");
}

FReply SRigVMChangesTreeRow::OnExpanderMouseButtonDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent)
{
	if(MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedRef< ITypedTableView< TSharedRef< FRigVMTreeNode> > > OwnerTable = OwnerTablePtr.Pin().ToSharedRef();
		OwnerTable->Private_SetItemExpansion(Node.ToSharedRef(), !IsItemExpanded());
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

EVisibility SRigVMChangesTreeRow::GetExpanderVisibility() const
{
	return Node->HasVisibleChildren() ? EVisibility::Visible : EVisibility::Collapsed;
}

TArray<FRigVMTag> SRigVMChangesTreeRow::GetVariantTags() const
{
	if(!Tags.IsSet())
	{
		Tags = Node->GetTags();
	}
	return Tags.GetValue();
}

void SRigVMChangesTreeRow::RequestRefresh(bool bForce)
{
	if(OwningWidget)
	{
		OwningWidget->RequestRefresh_AnyThread(bForce);
	}
}

EVisibility SRigVMChangesTreeRow::GetCheckBoxVisibility() const
{
	return Node->IsCheckable() ? EVisibility::Visible : EVisibility::Collapsed;
}

ECheckBoxState SRigVMChangesTreeRow::GetCheckBoxState() const
{
	return Node->GetCheckState();
}

void SRigVMChangesTreeRow::OnCheckBoxStateChanged(ECheckBoxState InNewState)
{
	const TArray<TSharedRef<FRigVMTreeNode>>& SelectedNodes = OwningWidget->GetSelectedNodes();
	if(SelectedNodes.Contains(Node))
	{
		for(TSharedRef<FRigVMTreeNode> SelectedNode : SelectedNodes)
		{
			SelectedNode->SetCheckState(InNewState);
		}
	}
	else
	{
		Node->SetCheckState(InNewState);
	}
}

//////////////////////////////////////////////////////////////////////////
// SRigVMChangesTreeView

SRigVMChangesTreeView::~SRigVMChangesTreeView()
{
}

void SRigVMChangesTreeView::Construct(const FArguments& InArgs)
{
	RequestRefreshCount = 0;
	RequestRefreshForceCount = 0;
	PhaseAttribute = InArgs._Phase;

	OnNodeSelected = InArgs._OnNodeSelected;
	OnNodeDoubleClicked = InArgs._OnNodeDoubleClicked;

	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	TSharedRef<SHorizontalBox> PathFilterHorizontalBox = SNew(SHorizontalBox);
	PathFilterHorizontalBox->AddSlot()
	.FillWidth(1)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(PathFilterBox, SSearchBox)
		.Visibility(this, &SRigVMChangesTreeView::GetPathFilterVisibility)
		.SelectAllTextWhenFocused(true)
		.InitialText(GetPathFilterText())
		.OnTextChanged(this, &SRigVMChangesTreeView::OnPathFilterTextChanged)
		.OnTextCommitted(this, &SRigVMChangesTreeView::OnPathFilterTextCommitted)
	];

	PathFilterHorizontalBox->AddSlot()
	.AutoWidth()
	[
		SNew(SButton)
		.ContentPadding(FMargin(1, 1))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Refresh"))
			.ToolTipText(LOCTEXT("RefreshToolTip", "Refresh the contents of the view"))
		]
		.OnClicked_Lambda([this]()
		{
			RequestRefresh_AnyThread(true);
			return FReply::Handled();
		})
	];

	PathFilterHorizontalBox->AddSlot()
	.AutoWidth()
	[
		SNew(SButton)
		.Visibility(this, &SRigVMChangesTreeView::GetSettingsButtonVisibility)
		.ContentPadding(FMargin(1, 1))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			.ToolTipText(LOCTEXT("SettingsToolTip", "Change filtering settings here"))
		]
		.OnClicked(this, &SRigVMChangesTreeView::OnSettingsButtonClicked)
	];

	TSharedRef<SHorizontalBox> MainHorizontalBox = SNew(SHorizontalBox);
	
	MainHorizontalBox->AddSlot()
	.FillWidth(1)
	.HAlign(HAlign_Fill)
	[
		SAssignNew(TreeView, STreeView<TSharedRef<FRigVMTreeNode>>)
		.TreeItemsSource(&FilteredNodes)
		.OnGenerateRow(this, &SRigVMChangesTreeView::MakeTreeRowWidget)
		.OnGetChildren(this, &SRigVMChangesTreeView::GetChildrenForNode)
		.OnSelectionChanged(this, &SRigVMChangesTreeView::OnSelectionChanged)
		.OnMouseButtonDoubleClick(this, &SRigVMChangesTreeView::OnTreeElementDoubleClicked)
		.SelectionMode(this, &SRigVMChangesTreeView::GetSelectionMode)
		.OnContextMenuOpening(this, &SRigVMChangesTreeView::OnGetNodeContextMenuContent)
	];

	MainVerticalBox->AddSlot()
	.AutoHeight()
	.Padding(0, 2)
	[
		PathFilterHorizontalBox
	];

	MainVerticalBox->AddSlot()
	.VAlign(VAlign_Fill)
	.FillHeight(1.0f)		// This is required to make the scrollbar work, as content overflows Slate containers by default
	[
		MainHorizontalBox
	];

	ChildSlot
	[
		MainVerticalBox
	];
}

void SRigVMChangesTreeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	RefreshFilteredNodesIfRequired();
}

TSharedRef<ITableRow> SRigVMChangesTreeView::MakeTreeRowWidget(TSharedRef<FRigVMTreeNode> InNode, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRigVMChangesTreeRow, OwnerTable)
		.Node(InNode)
		.OwningWidget(SharedThis(this));
}

void SRigVMChangesTreeView::GetChildrenForNode(TSharedRef<FRigVMTreeNode> InNode, TArray< TSharedRef<FRigVMTreeNode> >& OutChildren)
{
	OutChildren = InNode->GetVisibleChildren(GetContext());
}

void SRigVMChangesTreeView::OnSelectionChanged(TSharedPtr<FRigVMTreeNode> Selection, ESelectInfo::Type SelectInfo)
{
	if(Selection.IsValid())
	{
		if(OnNodeSelected.IsBound())
		{
			if(OnNodeSelected.Execute(Selection.ToSharedRef()).IsEventHandled())
			{
				return;
			}
		}
	}
}

void SRigVMChangesTreeView::OnTreeElementDoubleClicked(TSharedRef<FRigVMTreeNode> InNode)
{
	if(OnNodeDoubleClicked.IsBound())
	{
		if(OnNodeDoubleClicked.Execute(InNode).IsEventHandled())
		{
			return;
		}
	}

	if(const TSharedPtr<FRigVMTreePackageNode> PackageNode = Cast<FRigVMTreePackageNode>(InNode))
	{
		const FAssetData AssetData = PackageNode->GetAssetData();
		if(AssetData.IsValid())
		{
			const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			TArray<FAssetData> Assets;
			Assets.Add(AssetData);
			ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
		}			
	}
}

ESelectionMode::Type SRigVMChangesTreeView::GetSelectionMode() const
{
	return GetPhase()->AllowsMultiSelection() ? ESelectionMode::Multi : ESelectionMode::Single;
}

TSharedPtr<SWidget> SRigVMChangesTreeView::OnGetNodeContextMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, TSharedPtr<const FUICommandList>());

	TArray< TSharedRef< FRigVMTreeNode > > SelectedNodes = TreeView->GetSelectedItems();
	TArray< TSharedRef< FRigVMTreeNode > > RootNodes;

	for(const TSharedRef< FRigVMTreeNode >& Node : SelectedNodes)
	{
		if(!Node->IsLoaded())
		{
			RootNodes.AddUnique(Node->GetRoot());
		}
	}
	
	MenuBuilder.BeginSection("Assets", LOCTEXT( "Assets", "Assets" ) );
	for(const TSharedRef< FRigVMTreeNode >& Node : RootNodes)
	{
		if(!Node->IsLoaded())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("LoadAssets", "Load Assets"),
				LOCTEXT("LoadAssets_ToolTip", "Loads all of the assets backing up the selection"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, RootNodes]()
				{
					TArray<TSharedRef<FRigVMTreeTask>> Tasks;
					for(const TSharedRef<FRigVMTreeNode>& Node : RootNodes)
					{
						Tasks.Add(FRigVMTreeLoadPackageForNodeTask::Create(Node));
					}
					GetPhase()->QueueTasks(Tasks);
				})));
			break;
		}
	}
	for(const TSharedRef< FRigVMTreeNode >& Node : RootNodes)
	{
		if(const TSharedPtr<FRigVMTreePackageNode> PackageNode = Cast<FRigVMTreePackageNode>(Node))
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ShowInContentBrowser", "Locate in Content Browser"),
				LOCTEXT("ShowInContentBrowser_ToolTip", "Locates the asset in the content browser"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([PackageNode]()
				{
					const FAssetData AssetData = PackageNode->GetAssetData();
					if(AssetData.IsValid())
					{
						const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
						TArray<FAssetData> Assets;
						Assets.Add(AssetData);
						ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
					}
				})));
			break;
		}
	}
	MenuBuilder.EndSection();

	for(const TSharedRef<FRigVMTreeNode>& SelectedNode : SelectedNodes)
	{
		if(SelectedNode->IsCheckable())
		{
			MenuBuilder.BeginSection("Marking", LOCTEXT( "Marking", "Marking" ) );
			MenuBuilder.AddMenuEntry(
				LOCTEXT("MarkAllSelected", "Mark Selection"),
				LOCTEXT("MarkAllSelected_ToolTip", "Marks all checkboes for the selection"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]()
				{
					TArray< TSharedRef< FRigVMTreeNode > > SelectedNodes = TreeView->GetSelectedItems();
					for(const TSharedRef<FRigVMTreeNode>& SelectedNode : SelectedNodes)
					{
						SelectedNode->SetCheckState(ECheckBoxState::Checked);
					}
				})));
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UnmarkAllSelected", "Unmark Selection"),
				LOCTEXT("UnmarkAllSelected_ToolTip", "Unmarks all checkboes for the selection"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this]()
				{
					TArray< TSharedRef< FRigVMTreeNode > > SelectedNodes = TreeView->GetSelectedItems();
					for(const TSharedRef<FRigVMTreeNode>& SelectedNode : SelectedNodes)
					{
						SelectedNode->ResetCheckState();
					}
				})));
			MenuBuilder.EndSection();
			break;
		}
	}

	if(!SelectedNodes.IsEmpty())
	{
		SelectedNodes[0]->GetContextMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

EVisibility SRigVMChangesTreeView::GetSettingsButtonVisibility() const
{
	const bool bHasFilterSettings = GetContext()->Filters.ContainsByPredicate([](const TSharedPtr<FRigVMTreeFilter>& Filter) -> bool
	{
		return Filter->CanBeToggledInUI();
	});
	return bHasFilterSettings ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SRigVMChangesTreeView::OnSettingsButtonClicked()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for(const TSharedRef<FRigVMTreeFilter>& Filter : GetContext()->Filters)
	{
		if(!Filter->CanBeToggledInUI())
		{
			 continue;
		}
		
		FUIAction ToggleSettingAction(FExecuteAction::CreateLambda([this, Filter]()
		{
			Filter->SetEnabled(!Filter->IsEnabled());
			RefreshFilteredNodes();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([Filter]()
		{
			return Filter->IsInvertedInUI() ? !Filter->IsEnabled() : Filter->IsEnabled();
		}));

		MenuBuilder.AddMenuEntry(
			Filter->GetLabel(),
			Filter->GetToolTip(),
			FSlateIcon(),
			ToggleSettingAction,
			NAME_None,
			EUserInterfaceActionType::Check);
	}

	MenuBuilder.MakeWidget();

	FSlateApplication::Get().PushMenu(SharedThis(this), FWidgetPath(), MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	
	return FReply::Handled();
}

EVisibility SRigVMChangesTreeView::GetPathFilterVisibility() const
{
	return GetPathFilter().IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SRigVMChangesTreeView::OnPathFilterTextChanged(const FText& SearchText)
{
	const TSharedPtr<FRigVMTreePathFilter> PathFilter = GetPathFilter();
	if(PathFilter.IsValid())
	{
		if (GetPathFilterText().CompareToCaseIgnored(SearchText) != 0)
		{
			PathFilter->SetFilterText(SearchText.ToString());
			RefreshFilteredNodes();
		}
	}
}

void SRigVMChangesTreeView::OnPathFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo)
{
	OnPathFilterTextChanged(SearchText);
}

FText SRigVMChangesTreeView::GetPathFilterText()
{
	TSharedPtr<FRigVMTreePathFilter> PathFilter = GetPathFilter();
	if(PathFilter.IsValid())
	{
		return FText::FromString(PathFilter->GetFilterText());
	}
	static const FText EmptyText;
	return EmptyText;
}

void SRigVMChangesTreeView::RequestRefresh_AnyThread(bool bForce)
{
	(void)RequestRefreshCount++;
	if(bForce)
	{
		(void)RequestRefreshForceCount++;
	}
}

void SRigVMChangesTreeView::RefreshFilteredNodes(bool bForce)
{
	const TSharedRef<FRigVMTreePhase> ActivePhase = GetPhase();
	if(bForce)
	{
		ActivePhase->IncrementContextHash();
	}
	FilteredNodes = ActivePhase->GetVisibleNodes();
	TreeView->RebuildList();
}

void SRigVMChangesTreeView::RefreshFilteredNodesIfRequired()
{
	if(RequestRefreshCount.exchange(0) > 0)
	{
		const bool bForce = RequestRefreshForceCount.exchange(0) > 0;
		RefreshFilteredNodes(bForce);
	}
}

void SRigVMChangesTreeView::OnPhaseChanged()
{
	RefreshFilteredNodes(true);

	if(const TSharedPtr<FRigVMTreePathFilter> PathFilter = GetPathFilter())
	{
		PathFilterBox->SetText(FText::FromString(PathFilter->GetFilterText()));
	}

	for(const TSharedRef<FRigVMTreeNode>& RootNode : FilteredNodes)
	{
		if(RootNode->ShouldExpandByDefault())
		{
			TreeView->SetItemExpansion(RootNode, true);
		}
	}
}

void SRigVMChangesTreeView::SetSelection(const TSharedPtr<FRigVMTreeNode>& InNode, bool bRequestScrollIntoView)
{
	if(InNode.IsValid())
	{
		TreeView->SetSelection(InNode.ToSharedRef());
		if(bRequestScrollIntoView)
		{
			TreeView->RequestScrollIntoView(InNode.ToSharedRef());
		}
	}
	else
	{
		TreeView->ClearSelection();
	}
}

TSharedRef<FRigVMTreePhase> SRigVMChangesTreeView::GetPhase() const
{
	if(const TSharedPtr<FRigVMTreePhase> Phase = PhaseAttribute.Get())
	{
		return Phase.ToSharedRef();
	}
	static const TSharedRef<FRigVMTreeContext> EmptyContext = FRigVMTreeContext::Create();
	static const TSharedRef<FRigVMTreePhase> EmptyPhase = FRigVMTreePhase::Create(INDEX_NONE, TEXT("Default"), EmptyContext);
	return EmptyPhase;
}

TSharedRef<FRigVMTreeContext> SRigVMChangesTreeView::GetContext() const
{
	return GetPhase()->GetContext();
}

TSharedPtr<FRigVMTreePathFilter> SRigVMChangesTreeView::GetPathFilter() const
{
	const TSharedRef<FRigVMTreeContext> Context = GetContext();
	for(const TSharedRef<FRigVMTreeFilter>& Filter : Context->Filters)
	{
		if(Filter->IsA<FRigVMTreePathFilter>())
		{
			return Cast<FRigVMTreePathFilter>(Filter);
		}
	}
	return nullptr;
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMChangesTreeView::GetSelectedNodes() const
{
	return TreeView->GetSelectedItems();
}

bool SRigVMChangesTreeView::HasAnyVisibleCheckedNode() const
{
	TArray<TSharedRef<FRigVMTreeNode>> Nodes = GetPhase()->GetVisibleNodes();
	for(const TSharedRef<FRigVMTreeNode>& Node : Nodes)
	{
		if(Node->GetCheckState() != ECheckBoxState::Unchecked)
		{
			return true;
		}
		if(Node->ContainsAnyVisibleCheckedNode())
		{
			return true;
		}
	}
	return false;
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMChangesTreeView::GetCheckedNodes() const
{
	TArray<TSharedRef<FRigVMTreeNode>> Nodes = GetPhase()->GetVisibleNodes();
	for(int32 Index = 0; Index < Nodes.Num(); Index++)
	{
		Nodes.Append(Nodes[Index]->GetVisibleChildren(GetContext()));
	}

	Nodes = Nodes.FilterByPredicate([](const TSharedRef<FRigVMTreeNode>& Node)
	{
		return Node->GetCheckState() != ECheckBoxState::Unchecked;
	});

	return Nodes;
}

#undef LOCTEXT_NAMESPACE
