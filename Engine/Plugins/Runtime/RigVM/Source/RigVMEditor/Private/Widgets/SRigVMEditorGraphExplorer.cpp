// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMEditorGraphExplorer.h"

#include "Editor/RemoveUnusedMembers/RigVMEditorRemoveUnusedMembers.h"
#include "Editor/RigVMEdGraphNodeRegistry.h"
#include "Editor/RigVMEditorCommands.h"
#include "Editor/RigVMNewEditor.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SRigVMEditorGraphExplorerTreeView.h"

#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Input/SSearchBox.h"
#include "SPositiveActionButton.h"
#include "EdGraphSchema_K2_Actions.h"

// Ick.
#include "GraphActionNode.h"
#include "GraphEditorActions.h"
#include "RigVMBlueprintLegacy.h"
#include "SPinTypeSelector.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SScaleBox.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"
#include "ScopedTransaction.h"
#if WITH_RIGVMLEGACYEDITOR
#include "SKismetInspector.h"
#include "Editor/RigVMLegacyEditor.h"
#else
#include "Editor/SRigVMDetailsInspector.h"
#endif
#include "EdGraph/RigVMEdGraph.h"
#include "Editor/SRigVMDetailsInspector.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "RigVMDeveloperTypeUtils.h"

#define LOCTEXT_NAMESPACE "RigVMGraphExplorer"

TSharedRef<FRigVMGraphExplorerDragDropOp> FRigVMGraphExplorerDragDropOp::New(const FRigVMExplorerElementKey& InElement, TObjectPtr<URigVMBlueprint> InBlueprint)
{
	const FRigVMEditorAssetInterfacePtr Asset(InBlueprint);
	return New(InElement, Asset);
}

TSharedRef<FRigVMGraphExplorerDragDropOp> FRigVMGraphExplorerDragDropOp::New(const ::FRigVMExplorerElementKey& InElement, FRigVMEditorAssetInterfacePtr InBlueprint)
{
	TSharedRef<FRigVMGraphExplorerDragDropOp> Operation = MakeShared<FRigVMGraphExplorerDragDropOp>();
	Operation->Element = InElement;
	Operation->SourceBlueprint = InBlueprint;
	Operation->Construct();
	return Operation;
}

void FRigVMGraphExplorerDragDropOp::Construct()
{
	// Create the drag-drop decorator window
	CursorDecoratorWindow = SWindow::MakeCursorDecorator();
	const bool bShowImmediately = false;
	FSlateApplication::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), bShowImmediately);

	const FSlateBrush* PrimarySymbol = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.NewNode"));
	const FSlateBrush* SecondarySymbol = nullptr;
	FSlateColor PrimaryColor = FLinearColor::White;
	FSlateColor SecondaryColor = FLinearColor::White;

	//Create feedback message with the function name.
	TSharedRef<SWidget> TypeImage = SPinTypeSelector::ConstructPinTypeImage(PrimarySymbol, PrimaryColor, SecondarySymbol, SecondaryColor, TSharedPtr<SToolTip>());

	CursorDecoratorWindow->ShowWindow();
	CursorDecoratorWindow->SetContent
	(
		SNew(SBorder)
		. BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3.0f)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					TypeImage
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3.0f)
			.MaxWidth(500)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.WrapTextAt( 480 )
				.Text( FText::FromString(GetElement().Name ))
			]
		]
	);
}

FReply FRigVMGraphExplorerDragDropOp::DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph)
{
	if (URigVMEdGraph* TargetRigGraph = Cast<URigVMEdGraph>(&Graph))
	{
		if(FRigVMEditorAssetInterfacePtr Blueprint = TargetRigGraph->GetAsset())
		{
			//  Find the appropriate asset editor where the drop is happening
			UObject* AssetObj = Blueprint.GetObject();
			
			const TArray<IAssetEditorInstance*> AssetEditors = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorsForAsset(AssetObj);
			IAssetEditorInstance* FocusedAssetEditor = nullptr;
			for (IAssetEditorInstance* AssetEditor : AssetEditors)
			{
				if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(AssetEditor))
				{
					TWeakPtr<SGraphEditor> GraphEditor = RigVMEditor->GetFocusedGraphEditor();
					TSharedPtr<SWidget> Widget = Panel->GetParentWidget();
					while (Widget)
					{
						if (Widget == GraphEditor)
						{
							break;
						}
						Widget = Widget->GetParentWidget();
					}
					if (Widget)
					{
						FocusedAssetEditor = AssetEditor;
						break;
					}
				}
			}

			if (!FocusedAssetEditor)
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint->GetAssetObjectForEditor());
				FocusedAssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AssetObj, /*bFocusIfOpen =*/true);
			}
			if (FocusedAssetEditor)
			{
				if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(FocusedAssetEditor))
				{
					RigVMEditor->OnGraphNodeDropToPerform(SharedThis(this), TargetRigGraph, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition));
				}
			}
		}
	}
	return FGraphEditorDragDropAction::DroppedOnPanel(Panel, ScreenPosition, GraphPosition, Graph);
}

FReply FRigVMGraphExplorerDragDropOp::DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition)
{
	UEdGraph* TargetGraph = GetHoveredGraph();
	UEdGraphPin* TargetPin = GetHoveredPin();
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	
	const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();

	if (Element.Type == ERigVMExplorerElementType::Variable)
	{
		if (FProperty* Property = Blueprint->FindGeneratedPropertyByName(*GetElement().Name))
		{
			if (EdSchema->RequestVariableDropOnPin(TargetGraph, Property, TargetPin, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition)))
			{
				return FReply::Handled();
			}
		}
	}
	else if (Element.Type == ERigVMExplorerElementType::LocalVariable)
	{
		for (FRigVMGraphVariableDescription& Variable : Blueprint->GetFocusedModel()->GetLocalVariables())
		{
			if (Variable.Name == *Element.Name)
			{
				if (EdSchema->RequestVariableDropOnPin(TargetGraph, Variable.ToExternalVariable(), TargetPin, FDeprecateSlateVector2D(GraphPosition), FDeprecateSlateVector2D(ScreenPosition)))
				{
					return FReply::Handled();
				}
				break;
			}
		}
	}
	
	return FGraphEditorDragDropAction::DroppedOnPin(ScreenPosition, GraphPosition);
}

FRigVMEditorGraphExplorerCommands::FRigVMEditorGraphExplorerCommands()
	: TCommands<FRigVMEditorGraphExplorerCommands>(
          TEXT("RigVMEditorGraphExplorer"), NSLOCTEXT("Contexts", "Explorer", "Explorer"),
          NAME_None, FAppStyle::GetAppStyleSetName())
{
}


void FRigVMEditorGraphExplorerCommands::RegisterCommands()
{
	UI_COMMAND(OpenGraph, "Open Graph", "Opens up this graph in the editor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenGraphInNewTab, "Open Graph In New Tab", "Opens up this graph in a new tab.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateGraph, "Add New Graph", "Create a new graph and show it in the editor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateFunction, "Add New Function", "Create a new function and show it in the editor.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateVariable, "Add New Variable", "Create a new member variable.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateLocalVariable, "Add New Local Variable", "Create a new local variable to the function.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddFunctionVariant, "Add Variant", "Creates a new variant of a function.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteFunction, "Paste Function", "Pastes the function.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteVariable, "Paste Variable", "Pastes the variable.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PasteLocalVariable, "Paste Local Variable", "Pastes the local variable.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveUnusedFunctions, "Remove Unused Functions", "Removes unsued functions.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveUnusedVariables, "Remove Unused Variables", "Removes unused variables.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FindReferences, "Find references", "Find references.", EUserInterfaceActionType::Button, FInputChord(EKeys::F, EModifierKey::Control));
}


void SRigVMEditorGraphExplorer::Construct(const FArguments& InArgs, TWeakPtr<IRigVMEditor> InRigVMEditor)
{
	bNeedsRefresh = false;
	RigVMEditor = InRigVMEditor;
	UICommandList = MakeShared<FUICommandList>();

	RegisterCommands();

	CreateWidgets();

	LastPinType.ResetToDefaults();
	LastPinType.PinCategory = TEXT("bool");

	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (Editor)
	{
		Editor->OnRefresh().AddSP(this, &SRigVMEditorGraphExplorer::Refresh);
		Editor->GetKeyDownDelegate().BindSP(this, &SRigVMEditorGraphExplorer::OnKeyDown);

		// Acquire a registry for function and variable nodes
		if (const TScriptInterface<IRigVMEditorAssetInterface> AssetInterface = Editor->GetRigVMAssetInterface())
		{
			EdGraphNodeFunctionRegistry = FRigVMEdGraphNodeRegistry::GetOrCreateRegistry(AssetInterface, URigVMFunctionReferenceNode::StaticClass());
			EdGraphNodeVariableRegistry = FRigVMEdGraphNodeRegistry::GetOrCreateRegistry(AssetInterface, URigVMVariableNode::StaticClass());
		}
	}
}

void SRigVMEditorGraphExplorer::Refresh()
{
	bNeedsRefresh = true;
}

void SRigVMEditorGraphExplorer::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if(bNeedsRefresh)
	{
		TreeView->RefreshTreeView(true);
		bNeedsRefresh = false;
	}
}

FName SRigVMEditorGraphExplorer::GetSelectedVariableName() const
{
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	if (Selected.Num() != 1)
	{
		return NAME_None;
	}

	if (Selected[0].Type != ERigVMExplorerElementType::Variable && Selected[0].Type != ERigVMExplorerElementType::LocalVariable)
	{
		return NAME_None;
	}

	return *Selected[0].Name;
}

ERigVMExplorerElementType::Type SRigVMEditorGraphExplorer::GetSelectedType() const
{
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	if (Selected.Num() != 1)
	{
		return ERigVMExplorerElementType::Invalid;
	}

	return Selected[0].Type;
}

void SRigVMEditorGraphExplorer::ClearSelection()
{
	TreeView->ClearSelection();
}

void SRigVMEditorGraphExplorer::HandleGraphModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	switch (InNotifType)
	{
		case ERigVMGraphNotifType::GraphChanged:
		case ERigVMGraphNotifType::VariableAdded:
		case ERigVMGraphNotifType::VariableRenamed:
		case ERigVMGraphNotifType::VariableRemoved:
		case ERigVMGraphNotifType::LocalVariableAdded:
		case ERigVMGraphNotifType::LocalVariableRenamed:
		case ERigVMGraphNotifType::LocalVariableRemoved:
		case ERigVMGraphNotifType::LocalVariableTypeChanged:
			{
				Refresh();
				break;
			}
		case ERigVMGraphNotifType::NodeAdded:
		case ERigVMGraphNotifType::NodeRemoved:
		case ERigVMGraphNotifType::NodeRenamed:
			{
				if (URigVMCollapseNode* FunctionLibrary = Cast<URigVMCollapseNode>(InSubject))
				{
					Refresh();
				}
				if (URigVMUnitNode* Node = Cast<URigVMUnitNode>(InSubject))
				{
					if (Node->IsEvent())
					{
						Refresh();
					}
				}
				break;
			}
	}
}

FReply SRigVMEditorGraphExplorer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SRigVMEditorGraphExplorer::RegisterCommands()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().OpenGraph,
		    FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnOpenGraph, false),
			FCanExecuteAction(), FGetActionCheckState(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanOpenGraph));

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().OpenGraphInNewTab,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnOpenGraph, true),
			FCanExecuteAction(), FGetActionCheckState(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanOpenGraph));

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().CreateGraph,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateGraph),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCreateGraph)
			);

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().CreateFunction,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateFunction),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCreateFunction)
			);

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().CreateVariable,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateVariable),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCreateVariable)
			);

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().CreateLocalVariable,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateLocalVariable),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCreateLocalVariable)
			);

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().AddFunctionVariant,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnAddFunctionVariant),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanAddFunctionVariant)
			);

		UICommandList->MapAction(FGenericCommands::Get().Rename,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnRenameEntry),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanRenameEntry));

		UICommandList->MapAction(FGenericCommands::Get().Copy,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCopy),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCopy));
		
		UICommandList->MapAction(FGenericCommands::Get().Cut,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCut),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanCut));

		UICommandList->MapAction(FGenericCommands::Get().Duplicate,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnDuplicate),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanDuplicate));

		UICommandList->MapAction(FGenericCommands::Get().Paste,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnPasteGeneric),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanPasteGeneric));

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().PasteFunction,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnPasteFunction),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanPasteFunction));

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().PasteVariable,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnPasteVariable),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanPasteVariable));

		UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().PasteLocalVariable,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnPasteLocalVariable),
			FCanExecuteAction(), FIsActionChecked(),
			FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanPasteLocalVariable));

		UICommandList->MapAction(FGenericCommands::Get().Delete,
			FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnDeleteEntry),
			FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanDeleteEntry));
		
		// Bind widget-level command for Ctrl+F (only active when this widget has focus)
		{
			constexpr bool bSearchAllBlueprints = false;

			UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().FindReferences,
				FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnFindReferences, bSearchAllBlueprints),
				FCanExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::CanFindReferences));
		}

		// Find references local
		{
			constexpr bool bSearchAllBlueprints = false;

			UICommandList->MapAction(FRigVMEditorCommands::Get().FindReferencesByNameLocal,
				FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnFindReferences, bSearchAllBlueprints),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanFindReferences));
		}

		// Find references global
		{
			constexpr bool bSearchAllBlueprints = true;

			UICommandList->MapAction(FRigVMEditorCommands::Get().FindReferencesByNameGlobal,
				FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnFindReferences, bSearchAllBlueprints),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanFindReferences));
		}

		// Remove unused members
		{
			UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedFunctions,
				FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnRemoveUnusedFunctions),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanRemoveUnusedFunctions));

			UICommandList->MapAction(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedVariables,
				FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnRemoveUnusedVariables),
				FCanExecuteAction(),
				FIsActionChecked(),
				FIsActionButtonVisible::CreateSP(this, &SRigVMEditorGraphExplorer::CanRemoveUnusedVariables));
		}
	}
}

void SRigVMEditorGraphExplorer::CreateWidgets()
{
	TSharedPtr<SWidget> AddNewMenu = SNew(SPositiveActionButton)
		.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new item."))
		.OnGetMenuContent(this, &SRigVMEditorGraphExplorer::CreateAddNewMenuWidget)
		.Icon(FAppStyle::GetBrush("Plus"))
		.Text(LOCTEXT("AddNew", "Add"));

	FMenuBuilder ViewOptions(true, nullptr);

	ViewOptions.AddMenuEntry(
	    LOCTEXT("ShowEmptySections", "Show Empty Sections"),
	    LOCTEXT("ShowEmptySectionsTooltip", "Should we show empty sections? eg. Graphs, Functions...etc."),
	    FSlateIcon(),
	    FUIAction(
	        FExecuteAction::CreateSP(this, &SRigVMEditorGraphExplorer::OnToggleShowEmptySections),
	        FCanExecuteAction(),
	        FIsActionChecked::CreateSP(this, &SRigVMEditorGraphExplorer::IsShowingEmptySections)),
	    NAME_None,
	    EUserInterfaceActionType::ToggleButton,
	    TEXT("RigVMGraphExplorer_ShowEmptySections"));

	SAssignNew(FilterBox, SSearchBox)
	    .OnTextChanged_Lambda([this](const FText& InFilterText) { TreeView->FilterText = InFilterText; TreeView->RefreshTreeView(); } );


	FRigVMEditorGraphExplorerTreeDelegates Delegates;
	Delegates.OnGetRootGraphs = FRigVMGraphExplorer_OnGetRootGraphs::CreateSP(this, &SRigVMEditorGraphExplorer::GetRootGraphs);
	Delegates.OnGetChildrenGraphs = FRigVMGraphExplorer_OnGetChildrenGraphs::CreateSP(this, &SRigVMEditorGraphExplorer::GetChildrenGraphs);
	Delegates.OnGetEventNodesInGraph = FRigVMGraphExplorer_OnGetEventNodesInGraph::CreateSP(this, &SRigVMEditorGraphExplorer::GetEventNodesInGraph);
	Delegates.OnGetFunctions = FRigVMGraphExplorer_OnGetFunctions::CreateSP(this, &SRigVMEditorGraphExplorer::GetFunctions);
	Delegates.OnGetVariables = FRigVMGraphExplorer_OnGetVariables::CreateSP(this, &SRigVMEditorGraphExplorer::GetVariables);
	Delegates.OnGetLocalVariables = FRigVMGraphExplorer_OnGetVariables::CreateSP(this, &SRigVMEditorGraphExplorer::GetLocalVariables);
	Delegates.OnGetGraphDisplayName = FRigVMGraphExplorer_OnGetGraphDisplayName::CreateSP(this, &SRigVMEditorGraphExplorer::GetGraphDisplayName);
	Delegates.OnGetEventDisplayName = FRigVMGraphExplorer_OnGetEventDisplayName::CreateSP(this, &SRigVMEditorGraphExplorer::GetEventDisplayName);
	Delegates.OnGetGraphIcon = FRigVMGraphExplorer_OnGetGraphIcon::CreateSP(this, &SRigVMEditorGraphExplorer::GetGraphIcon);
	Delegates.OnGetGraphTooltip = FRigVMGraphExplorer_OnGetGraphTooltip::CreateSP(this, &SRigVMEditorGraphExplorer::GetGraphTooltip);
	Delegates.OnGraphClicked = FRigVMGraphExplorer_OnGraphClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnGraphClicked);
	Delegates.OnEventClicked = FRigVMGraphExplorer_OnEventClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnEventClicked);
	Delegates.OnFunctionClicked = FRigVMGraphExplorer_OnFunctionClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnFunctionClicked);
	Delegates.OnVariableClicked = FRigVMGraphExplorer_OnVariableClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnVariableClicked);
	Delegates.OnGraphDoubleClicked = FRigVMGraphExplorer_OnGraphDoubleClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnGraphDoubleClicked);
	Delegates.OnEventDoubleClicked = FRigVMGraphExplorer_OnEventDoubleClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnEventDoubleClicked);
	Delegates.OnFunctionDoubleClicked = FRigVMGraphExplorer_OnFunctionDoubleClicked::CreateSP(this, &SRigVMEditorGraphExplorer::OnFunctionDoubleClicked);
	Delegates.OnCreateGraph = FRigVMGraphExplorer_OnCreateGraph::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateGraph);
	Delegates.OnCreateFunction = FRigVMGraphExplorer_OnCreateFunction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateFunction);
	Delegates.OnCreateVariable = FRigVMGraphExplorer_OnCreateVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateVariable);
	Delegates.OnCreateLocalVariable = FRigVMGraphExplorer_OnCreateVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnCreateLocalVariable);
	Delegates.OnRenameGraph = FRigVMGraphExplorer_OnRenameGraph::CreateSP(this, &SRigVMEditorGraphExplorer::OnRenameGraph);
	Delegates.OnRenameFunction = FRigVMGraphExplorer_OnRenameFunction::CreateSP(this, &SRigVMEditorGraphExplorer::OnRenameFunction);
	Delegates.OnCanRenameGraph = FRigVMGraphExplorer_OnCanRenameGraph::CreateSP(this, &SRigVMEditorGraphExplorer::OnCanRenameGraph);
	Delegates.OnCanRenameFunction = FRigVMGraphExplorer_OnCanRenameFunction::CreateSP(this, &SRigVMEditorGraphExplorer::OnCanRenameFunction);
	Delegates.OnRenameVariable = FRigVMGraphExplorer_OnRenameVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnRenameVariable);
	Delegates.OnCanRenameVariable = FRigVMGraphExplorer_OnCanRenameVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnCanRenameVariable);
	Delegates.OnSetFunctionCategory = FRigVMGraphExplorer_OnSetFunctionCategory::CreateSP(this, &SRigVMEditorGraphExplorer::OnSetFunctionCategory);
	Delegates.OnGetFunctionCategory = FRigVMGraphExplorer_OnGetFunctionCategory::CreateSP(this, &SRigVMEditorGraphExplorer::OnGetFunctionCategory);
	Delegates.OnGetFunctionTooltip = FRigVMGraphExplorer_OnGetFunctionTooltip::CreateSP(this, &SRigVMEditorGraphExplorer::OnGetFunctionTooltip);
	Delegates.OnGetVariableTooltip = FRigVMGraphExplorer_OnGetFunctionTooltip::CreateSP(this, &SRigVMEditorGraphExplorer::OnGetVariableTooltip);
	Delegates.OnSetVariableCategory = FRigVMGraphExplorer_OnSetVariableCategory::CreateSP(this, &SRigVMEditorGraphExplorer::OnSetVariableCategory);
	Delegates.OnGetVariableCategory = FRigVMGraphExplorer_OnGetVariableCategory::CreateSP(this, &SRigVMEditorGraphExplorer::OnGetVariableCategory);
	Delegates.OnRequestContextMenu = FRigVMGraphExplorer_OnRequestContextMenu::CreateSP(this, &SRigVMEditorGraphExplorer::OnContextMenuOpening);
	Delegates.OnDragDetected = FOnDragDetected::CreateSP(this, &SRigVMEditorGraphExplorer::OnDragDetected);
	Delegates.OnGetVariablePinType = FRigVMGraphExplorer_OnGetVariablePinType::CreateSP(this, &SRigVMEditorGraphExplorer::OnGetVariablePinType);
	Delegates.OnSetVariablePinType = FRigVMGraphExplorer_OnSetVariablePinType::CreateSP(this, &SRigVMEditorGraphExplorer::OnSetVariablePinType);
	Delegates.OnIsVariablePublic = FRigVMGraphExplorer_OnIsVariablePublic::CreateSP(this, &SRigVMEditorGraphExplorer::OnIsVariablePublic);
	Delegates.OnToggleVariablePublic = FRigVMGraphExplorer_OnToggleVariablePublic::CreateSP(this, &SRigVMEditorGraphExplorer::OnToggleVariablePublic);
	Delegates.OnIsFunctionFocused = FRigVMGraphExplorer_OnIsFunctionFocused::CreateSP(this, &SRigVMEditorGraphExplorer::IsFunctionFocused);
	Delegates.OnGetCustomPinFilters = FRigVMGraphExplorer_OnGetCustomPinFilters::CreateSP(this, &SRigVMEditorGraphExplorer::GetCustomPinFilters);
	Delegates.OnReorderVariable = FRigVMGraphExplorer_OnReorderVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnReorderVariable);
	Delegates.OnReorderLocalVariable = FRigVMGraphExplorer_OnReorderVariable::CreateSP(this, &SRigVMEditorGraphExplorer::OnReorderLocalVariable);
	
	SAssignNew(TreeView, SRigVMEditorGraphExplorerTreeView, SharedThis(this))
		.RigTreeDelegates(Delegates)
	;

	// now piece together all the content for this widget
	ChildSlot
	[
		SNew(SVerticalBox)
	    + SVerticalBox::Slot()
	    .AutoHeight()
	    [
			SNew(SBorder)
	        .Padding(4.0f)
	        .BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	        [
				SNew(SVerticalBox)
	            + SVerticalBox::Slot()
	            .AutoHeight()
	            [
					SNew(SHorizontalBox)
	                + SHorizontalBox::Slot()
	                .AutoWidth()
	                .Padding(0, 0, 2, 0)
	                [
						AddNewMenu.ToSharedRef()
					]
	                + SHorizontalBox::Slot()
	                .FillWidth(1.0f)
	                .VAlign(VAlign_Center)
	                [
						FilterBox.ToSharedRef()
					]
	                + SHorizontalBox::Slot()
	                .AutoWidth()
	                .Padding(2, 0, 0, 0)
	                [
						SNew(SComboButton)
		                .ButtonStyle(FAppStyle::Get(), "SimpleButton")
	                    .ComboButtonStyle(FAppStyle::Get(), "ToolbarComboButton")
	                    .ForegroundColor(FSlateColor::UseForeground())
	                    .HasDownArrow(false)
	                    .ContentPadding(0.f)
	                    .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ViewOptions")))
	                    .MenuContent()
	                    [
							ViewOptions.MakeWidget()
						]
	                    .ButtonContent()
	                    [
	                    	SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Settings"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			]
		]
	    + SVerticalBox::Slot()
	    .FillHeight(1.0f)
	    [
			TreeView.ToSharedRef()
		]
	];

	Refresh();
}


TSharedRef<SWidget> SRigVMEditorGraphExplorer::CreateAddNewMenuWidget()
{
	FMenuBuilder MenuBuilder(/* bShouldCloseWindowAfterMenuSelection= */true, UICommandList);

	BuildAddNewMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}


void SRigVMEditorGraphExplorer::BuildAddNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AddNewItem", LOCTEXT("AddOperations", "Add New"));
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().CreateGraph);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().CreateFunction);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().PasteFunction);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().CreateVariable);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().PasteVariable);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().CreateLocalVariable);
	MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().PasteLocalVariable);
	
	MenuBuilder.EndSection();
}

TArray<const URigVMGraph*> SRigVMEditorGraphExplorer::GetRootGraphs() const
{
	TArray<const URigVMGraph*> Graphs;
	
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Graphs;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	for (URigVMGraph* Graph : RigVMClient->GetModels())
	{
		Graphs.Add(Graph);
	}

	return Graphs;
}

TArray<const URigVMGraph*> SRigVMEditorGraphExplorer::GetChildrenGraphs(const FString& InParentGraphPath) const
{
	TArray<const URigVMGraph*> Children;
	
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Children;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	URigVMGraph* ParentGraph = RigVMClient->GetModel(InParentGraphPath);
	if (!ParentGraph)
	{
		return Children;
	}

	TArray<URigVMGraph*> ContainedGraphs = ParentGraph->GetContainedGraphs();
	Children.Reserve(ContainedGraphs.Num());
	for (URigVMGraph* Child : ContainedGraphs)
	{
		// Do not show contained graphs of aggregate nodes
		if (URigVMAggregateNode* CollapseNode = Cast<URigVMAggregateNode>(Child->GetOuter()))
		{
			continue;
		}
		Children.Add(Child);
	}
	
	return Children;
}

TArray<URigVMNode*> SRigVMEditorGraphExplorer::GetEventNodesInGraph(const FString& InParentGraphPath) const
{
	TArray<URigVMNode*> Events;
	
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Events;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	URigVMGraph* ParentGraph = RigVMClient->GetModel(InParentGraphPath);
	if (!ParentGraph)
	{
		return Events;
	}

	if (!ParentGraph->IsTopLevelGraph())
	{
		return Events;
	}
	
	if (ParentGraph->IsA<URigVMFunctionLibrary>())
	{
		return Events;
	}

	for (URigVMNode* Node : ParentGraph->GetNodes())
	{
		if (Node->IsEvent())
		{
			Events.Add(Node);
		}
	}

	return Events;
}

TArray<URigVMLibraryNode*> SRigVMEditorGraphExplorer::GetFunctions() const
{
	TArray<URigVMLibraryNode*> Functions;
	
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Functions;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (URigVMFunctionLibrary* Library = RigVMClient->GetFunctionLibrary())
	{
		return Library->GetFunctions();
	}

	return Functions;
}

TArray<FRigVMGraphVariableDescription> SRigVMEditorGraphExplorer::GetVariables() const
{
	TArray<FRigVMGraphVariableDescription> Variables;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Variables;
	}

	return Editor->GetRigVMAssetInterface()->GetAssetVariables();
}

TArray<FRigVMGraphVariableDescription> SRigVMEditorGraphExplorer::GetLocalVariables() const
{
	TArray<FRigVMGraphVariableDescription> Variables;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Variables;
	}

	if (FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface())
	{
		if (URigVMGraph* Function = Blueprint->GetFocusedModel())
		{
			return Function->GetLocalVariables(false);
		}
	}

	return Variables;
}

FText SRigVMEditorGraphExplorer::GetGraphDisplayName(const FString& InGraphPath) const
{
	FText DisplayName;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return DisplayName;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (const URigVMGraph* Graph = RigVMClient->GetModel(InGraphPath))
	{
		const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
		FGraphDisplayInfo DisplayInfo;
		if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(Graph))
		{
			EdSchema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);
		}

		DisplayName = DisplayInfo.DisplayName;
	}
	
	return DisplayName;
}

FText SRigVMEditorGraphExplorer::GetEventDisplayName(const FString& InNodePath) const
{
	FText DisplayName;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return DisplayName;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (const URigVMNode* Node = RigVMClient->FindNode(InNodePath))
	{
		DisplayName = FText::FromName(Node->GetEventName());
	}
	
	return DisplayName;
}

const FSlateBrush* SRigVMEditorGraphExplorer::GetGraphIcon(const FString& InGraphPath) const
{
	const FSlateBrush* Brush = nullptr;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Brush;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (const URigVMGraph* Graph = RigVMClient->GetModel(InGraphPath))
	{
		if (Graph->IsRootGraph())
		{
			return FAppStyle::GetBrush(TEXT("GraphEditor.EventGraph_16x"));
		}
		else
		{
			return FAppStyle::GetBrush(TEXT("GraphEditor.SubGraph_16x"));
		}
	}
	return nullptr;
}

FText SRigVMEditorGraphExplorer::GetGraphTooltip(const FString& InGraphPath) const
{
	FText Tooltip;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return Tooltip;
	}

	FRigVMEditorAssetInterfacePtr Asset = Editor->GetRigVMAssetInterface();
	FRigVMClient* RigVMClient = Asset->GetRigVMClient();
	if (const URigVMGraph* Graph = RigVMClient->GetModel(InGraphPath))
	{
		const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
		FGraphDisplayInfo DisplayInfo;
		if (UEdGraph* EdGraph = Asset->GetEdGraph(Graph))
		{
			EdSchema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);
		}
		else
		{
			ensure(false);
		}

		Tooltip = DisplayInfo.Tooltip;
	}
	
	return Tooltip;
}

void SRigVMEditorGraphExplorer::OnGraphClicked(const FString& InGraphPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (const URigVMGraph* Graph = RigVMClient->GetModel(InGraphPath))
	{
		const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
		if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(Graph))
		{
			FGraphDisplayInfo DisplayInfo;
			const UEdGraphSchema* Schema = EdGraph->GetSchema();
			check(Schema != nullptr);
			Schema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);
#if WITH_RIGVMLEGACYEDITOR
			if (TSharedPtr<SKismetInspector> KismetInspector = Editor->GetKismetInspector())
			{
				KismetInspector->ShowDetailsForSingleObject(EdGraph, SKismetInspector::FShowDetailsOptions(DisplayInfo.PlainName));
			}
#endif
			if(TSharedPtr<SRigVMDetailsInspector> RigVMInspector = Editor->GetRigVMInspector())
			{
				RigVMInspector->ShowDetailsForSingleObject(EdGraph, SRigVMDetailsInspector::FShowDetailsOptions(DisplayInfo.PlainName));
			}
		}
	}
}

void SRigVMEditorGraphExplorer::OnEventClicked(const FString& InEventPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}
#if WITH_RIGVMLEGACYEDITOR
	if (TSharedPtr<SKismetInspector> KismetInspector = Editor->GetKismetInspector())
	{
		if (KismetInspector.IsValid())
		{
			KismetInspector->ShowDetailsForObjects(TArray<UObject*>());
		}
	}
#endif
	if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = Editor->GetRigVMInspector())
	{
		if (RigVMInspector.IsValid())
		{
			RigVMInspector->ShowDetailsForObjects(TArray<UObject*>());
		}
	}
}

void SRigVMEditorGraphExplorer::OnFunctionClicked(const FString& InFunctionPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (URigVMFunctionLibrary* FunctionLibrary = RigVMClient->GetFunctionLibrary())
	{
		if (const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(*InFunctionPath))
		{
			const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
			if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(FunctionNode->GetContainedGraph()))
			{
				FGraphDisplayInfo DisplayInfo;
				const UEdGraphSchema* Schema = EdGraph->GetSchema();
				check(Schema != nullptr);
				Schema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);
#if WITH_RIGVMLEGACYEDITOR
				if (TSharedPtr<SKismetInspector> KismetInspector = Editor->GetKismetInspector())
				{
					KismetInspector->ShowDetailsForSingleObject(EdGraph, SKismetInspector::FShowDetailsOptions(DisplayInfo.PlainName));
				}
#endif
				if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = Editor->GetRigVMInspector())
				{
					RigVMInspector->ShowDetailsForSingleObject(EdGraph, SRigVMDetailsInspector::FShowDetailsOptions(DisplayInfo.PlainName));
				}
			}
		}
	}
}

void SRigVMEditorGraphExplorer::OnVariableClicked(const FRigVMExplorerElementKey& InVariable)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	if (InVariable.Type == ERigVMExplorerElementType::Variable)
	{
		FProperty* Prop = Blueprint->FindGeneratedPropertyByName(*InVariable.Name);
		UPropertyWrapper* PropWrap = (Prop ? Prop->GetUPropertyWrapper() : nullptr);
#if WITH_RIGVMLEGACYEDITOR
		if (TSharedPtr<SKismetInspector> KismetInspector = Editor->GetKismetInspector())
		{
			SKismetInspector::FShowDetailsOptions Options(FText::FromString(InVariable.Name));
			Options.bForceRefresh = true;
			if (KismetInspector.IsValid())
			{
				KismetInspector->ShowDetailsForSingleObject(PropWrap, Options);
			}
		}
#endif
		if (TSharedPtr<SRigVMDetailsInspector> RigVMInspector = Editor->GetRigVMInspector())
		{
			SRigVMDetailsInspector::FShowDetailsOptions Options(FText::FromString(InVariable.Name));
			Options.bForceRefresh = true;
			
			if (RigVMInspector.IsValid())
			{
				RigVMInspector->ShowDetailsForSingleObject(PropWrap, Options);
			}
		}
	}
	else if (InVariable.Type == ERigVMExplorerElementType::LocalVariable)
	{
		URigVMGraph* Graph = Blueprint->GetFocusedModel();
		UEdGraph* EdGraph = Blueprint->GetEdGraph(Graph);
		Editor->SelectLocalVariable(EdGraph, *InVariable.Name);
	}
}

void SRigVMEditorGraphExplorer::OnGraphDoubleClicked(const FString& InGraphPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (const URigVMGraph* Graph = RigVMClient->GetModel(InGraphPath))
	{
		const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
		if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(Graph))
		{
			Editor->JumpToHyperlink(EdGraph);
		}
	}
}

void SRigVMEditorGraphExplorer::OnEventDoubleClicked(const FString& InEventPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	FRigVMClient* RigVMClient = Blueprint->GetRigVMClient();

	if(const URigVMNode* ModelNode = RigVMClient->FindNode(InEventPath))
	{
		Blueprint->OnRequestJumpToHyperlink().Execute(ModelNode);
	}
}
void SRigVMEditorGraphExplorer::OnFunctionDoubleClicked(const FString& InFunctionPath)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FRigVMClient* RigVMClient = Editor->GetRigVMAssetInterface()->GetRigVMClient();
	if (URigVMFunctionLibrary* FunctionLibrary = RigVMClient->GetFunctionLibrary())
	{
		if (const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(*InFunctionPath))
		{
			const URigVMEdGraphSchema* EdSchema = GetDefault<URigVMEdGraphSchema>();
			if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(FunctionNode->GetContainedGraph()))
			{
				Editor->JumpToHyperlink(EdGraph);
			}
		}
	}
}

bool SRigVMEditorGraphExplorer::OnSetFunctionCategory(const FString& InFunctionPath, const FString& InCategory)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetFunctionCategory", "Set Function Category"));

	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	if (URigVMFunctionLibrary* Library = Editor->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
	{
		if (URigVMLibraryNode* Function = Library->FindFunction(*InFunctionPath))
		{
			if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(Function->GetContainedGraph()))
			{
				Schema->TrySetGraphCategory(EdGraph, FText::FromString(InCategory));
				return true;
			}
		}
	}
	
	return false;
}

FString SRigVMEditorGraphExplorer::OnGetFunctionCategory(const FString& InFunctionPath) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return FString();
	}

	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	if (URigVMFunctionLibrary* Library = Editor->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
	{
		if (URigVMLibraryNode* Function = Library->FindFunction(*InFunctionPath))
		{
			return Function->GetNodeCategory();
		}
	}
	return FString();
}

FText SRigVMEditorGraphExplorer::OnGetFunctionTooltip(const FString& InFunctionPath) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return FText();
	}

	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	if (URigVMFunctionLibrary* Library = Editor->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
	{
		if (URigVMLibraryNode* Function = Library->FindFunction(*InFunctionPath))
		{
			return Function->GetToolTipText();
		}
	}
	return FText();
}

FText SRigVMEditorGraphExplorer::OnGetVariableTooltip(const FString& InVariable) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return FText();
	}

	return Editor->GetRigVMAssetInterface()->GetVariableTooltip(*InVariable);
}

bool SRigVMEditorGraphExplorer::OnSetVariableCategory(const FString& InVariable, const FString& InCategory)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetVariableCategory", "Set Variable Category"));
	Editor->GetRigVMAssetInterface()->SetVariableCategory(*InVariable, InCategory);
	return true;
}

FString SRigVMEditorGraphExplorer::OnGetVariableCategory(const FString& InVariable) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return FString();
	}

	return Editor->GetRigVMAssetInterface()->GetVariableCategory(*InVariable);
}

FEdGraphPinType SRigVMEditorGraphExplorer::OnGetVariablePinType(const FRigVMExplorerElementKey& InVariable)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return FEdGraphPinType();
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	if (InVariable.Type == ERigVMExplorerElementType::Variable)
	{
		for (FRigVMGraphVariableDescription& Variable : Blueprint->GetAssetVariables())
		{
			if (Variable.Name == *InVariable.Name)
			{
				return RigVMTypeUtils::PinTypeFromRigVMVariableDescription(Variable);
			}
		}
	}
	else if(InVariable.Type == ERigVMExplorerElementType::LocalVariable)
	{
		if (URigVMGraph* Function = Blueprint->GetFocusedModel())
		{
			for (FRigVMGraphVariableDescription& Variable : Function->GetLocalVariables())
			{
				if (Variable.Name == *InVariable.Name)
				{
					return RigVMTypeUtils::PinTypeFromRigVMVariableDescription(Variable);
				}
			}
		}
	}

	return FEdGraphPinType();
}

bool SRigVMEditorGraphExplorer::OnSetVariablePinType(const FRigVMExplorerElementKey& InVariable, const FEdGraphPinType& InType)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();

	if (InVariable.Type == ERigVMExplorerElementType::Variable)
	{
		for (FRigVMGraphVariableDescription& Variable : Blueprint->GetAssetVariables())
		{
			if (Variable.Name == *InVariable.Name)
			{
				FString CPPType;
				UObject* CPPTypeObject = nullptr;
				RigVMTypeUtils::CPPTypeFromPinType(InType, CPPType, &CPPTypeObject);
				Blueprint->ChangeMemberVariableType(*InVariable.Name, CPPType, Variable.bPublic);
				SetLastPinTypeUsed(InType);
				Refresh();
				return true;
			}
		}
	}
	else if(InVariable.Type == ERigVMExplorerElementType::LocalVariable)
	{
		if (URigVMGraph* Function = Blueprint->GetFocusedModel())
		{
			if (URigVMController* Controller = Blueprint->GetRigVMClient()->GetController(Function))
			{
				FString NewCPPType;
				UObject* NewCPPTypeObject = nullptr;
				RigVMTypeUtils::CPPTypeFromPinType(InType, NewCPPType, &NewCPPTypeObject);
				Controller->SetLocalVariableType(*InVariable.Name, NewCPPType, NewCPPTypeObject, true, true);
				SetLastPinTypeUsed(InType);
				Refresh();
				Editor->SelectLocalVariable(Blueprint->GetEdGraph(Function), *InVariable.Name);
				return true;
			}
		}
	}

	return false;
}

bool SRigVMEditorGraphExplorer::OnIsVariablePublic(const FString& InVariableName) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	if (const FProperty* Property = Blueprint->FindGeneratedPropertyByName(*InVariableName))
	{
		return !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);
	}
	
	return false;
}

bool SRigVMEditorGraphExplorer::OnToggleVariablePublic(const FString& InVariableName) const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	if (FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface())
	{
		// Toggle the flag on the blueprint's version of the variable description, based on state
	   const bool bVariableIsExposed = OnIsVariablePublic(InVariableName);
	   Blueprint->SetVariablePublic(*InVariableName, !bVariableIsExposed);
	}
	return true;
}

bool SRigVMEditorGraphExplorer::IsFunctionFocused() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid())
	{
		return false;
	}

	if (URigVMGraph* Graph = Editor->GetRigVMAssetInterface()->GetFocusedModel())
	{
		if (URigVMCollapseNode* FunctionNode = Cast<URigVMCollapseNode>(Graph->GetOuter()))
		{
			if (URigVMFunctionLibrary* Library = Cast<URigVMFunctionLibrary>(FunctionNode->GetOuter()))
			{
				return true;
			}
		}
	}
	return false;
}

TArray<TSharedPtr<IPinTypeSelectorFilter>> SRigVMEditorGraphExplorer::GetCustomPinFilters() const
{
	TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters;
	if (RigVMEditor.IsValid())
	{
		RigVMEditor.Pin()->GetPinTypeSelectorFilters(CustomPinTypeFilters);
	}
	return CustomPinTypeFilters;
}

TSharedPtr<SWidget> SRigVMEditorGraphExplorer::OnContextMenuOpening()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor.IsValid() || 
		!TreeView.IsValid())
	{
		return TSharedPtr<SWidget>();
	}

	FMenuBuilder MenuBuilder(/*Close after selection*/true, UICommandList);
	
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();

	if (Selection.Num() == 1)
	{
		const FText FindReferencesLabel = LOCTEXT("FindReferencesLabel", "Find References");
		const FText FindReferencesTooltip = LOCTEXT("FindReferencesTooltip", "Options for finding references to class members");

		switch (Selection[0].Type)
		{
			case ERigVMExplorerElementType::Section:
			{
				const ERigVMGraphExplorerSectionType SectionType = TreeView->GetSectionType(Selection[0]);
				if (SectionType == ERigVMGraphExplorerSectionType::Functions)
				{
					MenuBuilder.BeginSection("BasicOperations");
					MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedFunctions);
					MenuBuilder.EndSection();
				}
				else if (SectionType == ERigVMGraphExplorerSectionType::Variables ||
					SectionType == ERigVMGraphExplorerSectionType::LocalVariables)
				{
					MenuBuilder.BeginSection("BasicOperations");
					MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedVariables);
					MenuBuilder.EndSection();
				}

				break;
			}
			case ERigVMExplorerElementType::Graph:
			{
				MenuBuilder.BeginSection("BasicOperations");
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().OpenGraph);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().OpenGraphInNewTab);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().FindReferences);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
				MenuBuilder.EndSection();
				break;
			}
			case ERigVMExplorerElementType::Event:
			{
				return nullptr;
			}
			case ERigVMExplorerElementType::Function:
			{
				MenuBuilder.BeginSection("BasicOperations");
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().OpenGraph);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().OpenGraphInNewTab);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
				MenuBuilder.AddSubMenu(
					FindReferencesLabel,
					FindReferencesTooltip,
					FNewMenuDelegate::CreateStatic(&FRigVMEditorCommands::BuildFindReferencesMenu));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().AddFunctionVariant);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedFunctions);
				MenuBuilder.EndSection();
				break;
			}
			case ERigVMExplorerElementType::Variable:
			case ERigVMExplorerElementType::LocalVariable:
			{
				MenuBuilder.BeginSection("BasicOperations");
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
				MenuBuilder.AddSubMenu(
					FindReferencesLabel,
					FindReferencesTooltip,
					FNewMenuDelegate::CreateStatic(&FRigVMEditorCommands::BuildFindReferencesMenu));
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Duplicate);
				MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
				MenuBuilder.AddMenuEntry(FRigVMEditorGraphExplorerCommands::Get().RemoveUnusedVariables);
				MenuBuilder.EndSection();
				break;
			}
		}
		
	}
	else
	{
		BuildAddNewMenu(MenuBuilder);
	}

	return MenuBuilder.MakeWidget();
}

FReply SRigVMEditorGraphExplorer::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TArray<FRigVMExplorerElementKey> DraggedElements = TreeView->GetSelectedKeys();
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && DraggedElements.Num() > 0)
	{
		if (RigVMEditor.IsValid() && DraggedElements.Num() == 1)
		{
			TSharedRef<FRigVMGraphExplorerDragDropOp> DragDropOp = FRigVMGraphExplorerDragDropOp::New(MoveTemp(DraggedElements[0]), RigVMEditor.Pin()->GetRigVMAssetInterface());
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

void SRigVMEditorGraphExplorer::OnCopy()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
	if (Selection.Num() != 1)
	{
		return;
	}

	if (Selection[0].Type != ERigVMExplorerElementType::Function && Selection[0].Type != ERigVMExplorerElementType::Variable && Selection[0].Type != ERigVMExplorerElementType::LocalVariable)
	{
		return;
	}

	FString OutputString;
	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();

	switch (Selection[0].Type)
	{
		case ERigVMExplorerElementType::Function:
		{
			if (URigVMFunctionLibrary* FunctionLibrary = Blueprint->GetLocalFunctionLibrary())
			{
				if (const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(*Selection[0].Name))
				{
					if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(FunctionNode->GetContainedGraph()))
					{
						Blueprint->ExportGraphToText(EdGraph, OutputString);
					}
				}
			}
			break;
		}
		case ERigVMExplorerElementType::Variable:
		{
			OutputString = Blueprint->OnCopyVariable(*Selection[0].Name);
			break;
		}
		case ERigVMExplorerElementType::LocalVariable:
		{
			if (URigVMGraph* Graph = Blueprint->GetFocusedModel())
			{
				if (const UEdGraph* FocusedGraph = Blueprint->GetEdGraph(Graph))
				{
					TArray<FBPVariableDescription> LocalVariables;
					Schema->GetLocalVariables(FocusedGraph, LocalVariables);
					for (const FBPVariableDescription& VariableDescription : LocalVariables)
					{
						if (VariableDescription.VarName == *Selection[0].Name)
						{
							FBPVariableDescription::StaticStruct()->ExportText(OutputString, &VariableDescription, &VariableDescription, nullptr, 0, nullptr, false);
							OutputString = TEXT("BPVar") + OutputString;
							break;
						}
					}
				}
			}
			break;
		}
	}
	

	if (!OutputString.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(OutputString.GetCharArray().GetData());
	}
}

bool SRigVMEditorGraphExplorer::CanCopy() const
{
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
	if (Selection.Num() == 1)
	{
		if (Selection[0].Type == ERigVMExplorerElementType::Function || Selection[0].Type == ERigVMExplorerElementType::Variable || Selection[0].Type == ERigVMExplorerElementType::LocalVariable)
		{
			return true;
		}
	}
	return false;
}

void SRigVMEditorGraphExplorer::OnCut()
{
	OnCopy();
	OnDeleteEntry();
}

bool SRigVMEditorGraphExplorer::CanCut() const
{
	return CanCopy() && CanDeleteEntry();
}

void SRigVMEditorGraphExplorer::OnDuplicate()
{
	OnCopy();
	OnPasteGeneric();
}

bool SRigVMEditorGraphExplorer::CanDuplicate() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}
	
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
	if (Selection.Num() != 1)
	{
		return false;
	}

	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	switch (Selection[0].Type)
	{
		case ERigVMExplorerElementType::Function:
		case ERigVMExplorerElementType::Variable:
		case ERigVMExplorerElementType::LocalVariable:
		{
			return true;
		}
	}
	
	return false;
}

void SRigVMEditorGraphExplorer::OnPasteGeneric()
{
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	if (Selected.Num() == 1)
	{
		if (Selected[0].Type == ERigVMExplorerElementType::Variable && CanPasteVariable())
		{
			OnPasteVariable();
			return;
		}
		if (Selected[0].Type == ERigVMExplorerElementType::LocalVariable && CanPasteLocalVariable())
		{
			OnPasteLocalVariable();
			return;
		}
	}

	// try any of the other options

	// prioritize pasting as a member variable if possible
	if (CanPasteVariable())
	{
		OnPasteVariable();
	}
	else if (CanPasteLocalVariable())
	{
		OnPasteLocalVariable();
	}
	else if (CanPasteFunction())
	{
		OnPasteFunction();
	}
}

bool SRigVMEditorGraphExplorer::CanPasteGeneric() const
{
	return CanPasteVariable() 
	|| CanPasteLocalVariable() 
	|| CanPasteFunction() ;
}

void SRigVMEditorGraphExplorer::OnPasteFunction()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	Editor->GetRigVMAssetInterface()->TryImportGraphFromText(ClipboardText);
}

bool SRigVMEditorGraphExplorer::CanPasteFunction() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	return Editor->GetRigVMAssetInterface()->CanImportGraphFromText(ClipboardText);
}

void SRigVMEditorGraphExplorer::OnPasteVariable()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	Blueprint->OnPasteVariable(ClipboardText);
	Refresh();
}

bool SRigVMEditorGraphExplorer::CanPasteVariable() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (ClipboardText.StartsWith(TEXT("BPVar"), ESearchCase::CaseSensitive))
	{
		FBPVariableDescription Description;
		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(TEXT("BPVar"));
		FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, 0, &Errors, FBPVariableDescription::StaticStruct()->GetName());

		return Errors.IsEmpty();
	}

	if (ClipboardText.StartsWith(TEXT("RigVMVar"), ESearchCase::CaseSensitive))
	{
		FRigVMGraphVariableDescription Description;

		FStringOutputDevice Errors;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(TEXT("RigVMVar"));
		FRigVMGraphVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, 0, &Errors, FRigVMGraphVariableDescription::StaticStruct()->GetName());

		return Errors.IsEmpty();
	}

	return false;
}

void SRigVMEditorGraphExplorer::OnPasteLocalVariable()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
	if (!ensure(ClipboardText.StartsWith(TEXT("BPVar"), ESearchCase::CaseSensitive) ||
							ClipboardText.StartsWith(TEXT("RigVMVar"), ESearchCase::CaseSensitive)))
	{
		return;
	}

	if (ClipboardText.StartsWith(TEXT("BPVar"), ESearchCase::CaseSensitive))
	{
		FStringOutputDevice Errors;
		FBPVariableDescription Description;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(TEXT("BPVar"));
		FBPVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, PPF_None, &Errors, FBPVariableDescription::StaticStruct()->GetName());
		if (Errors.IsEmpty())
		{
			Editor->OnPasteNewLocalVariable(Description);
			Refresh();
		}
	}
	else
	{
		FStringOutputDevice Errors;
		FRigVMGraphVariableDescription Description;
		const TCHAR* Import = ClipboardText.GetCharArray().GetData() + FCString::Strlen(TEXT("RigVMVar"));
		FRigVMGraphVariableDescription::StaticStruct()->ImportText(Import, &Description, nullptr, PPF_None, &Errors, FRigVMGraphVariableDescription::StaticStruct()->GetName());
		if (Errors.IsEmpty())
		{
			if (URigVMGraph* FocusedGraph = Editor->GetFocusedModel())
			{
				if (URigVMController* Controller = Blueprint->GetRigVMClient()->GetController(FocusedGraph))
				{
					Controller->AddLocalVariable(Description.Name, Description.CPPType, Description.CPPTypeObject, Description.DefaultValue, true, true);
				}
			}
			Refresh();
		}
	}
}

bool SRigVMEditorGraphExplorer::CanPasteLocalVariable() const
{
	return CanPasteVariable();
}

void SRigVMEditorGraphExplorer::OnToggleShowEmptySections()
{
	// FIXME: Move to preferences
	bShowEmptySections = !bShowEmptySections;

	Refresh();
}


bool SRigVMEditorGraphExplorer::IsShowingEmptySections() const
{
	return bShowEmptySections;
}


void SRigVMEditorGraphExplorer::OnOpenGraph(bool bOnNewTab)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		UEdGraph* GraphToOpen = nullptr;
		TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
		if (Selection.Num() == 1 && Selection[0].Type == ERigVMExplorerElementType::Graph)
		{
			const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
			GraphToOpen = Editor->GetRigVMAssetInterface()->GetEdGraph(Selection[0].Name);
		}

		if (GraphToOpen)
		{
			const FDocumentTracker::EOpenDocumentCause Cause =bOnNewTab
			? FDocumentTracker::ForceOpenNewDocument
			: FDocumentTracker::OpenNewDocument;
			
			Editor->OpenDocument(GraphToOpen, Cause);
		}
	}
}


bool SRigVMEditorGraphExplorer::CanOpenGraph() const
{
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
	return RigVMEditor.IsValid() && Selection.Num() == 1 && Selection[0].Type == ERigVMExplorerElementType::Graph;
}


void SRigVMEditorGraphExplorer::OnCreateGraph()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (Editor)
	{
		Editor->OnNewDocumentClicked(FRigVMEditorBase::CGT_NewEventGraph);
	}
}


bool SRigVMEditorGraphExplorer::CanCreateGraph() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (Editor)
	{
		return Editor->InEditingMode();
	}

	return false;
}

void SRigVMEditorGraphExplorer::OnCreateFunction()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (Editor)
	{
		Editor->OnNewDocumentClicked(FRigVMEditorBase::CGT_NewFunctionGraph);
	}
}

bool SRigVMEditorGraphExplorer::CanCreateFunction() const
{
	return true;
}

void SRigVMEditorGraphExplorer::OnCreateVariable()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (Editor)
	{
		Editor->OnAddNewVariable();
	}
}

bool SRigVMEditorGraphExplorer::CanCreateVariable() const
{
	return true;
}

void SRigVMEditorGraphExplorer::OnCreateLocalVariable()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return;
	}

	Editor->OnAddNewLocalVariable();
	bNeedsRefresh = true;
}

bool SRigVMEditorGraphExplorer::CanCreateLocalVariable() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	return Editor->CanAddNewLocalVariable();
}

void SRigVMEditorGraphExplorer::OnAddFunctionVariant()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (!Editor)
	{
		return;
	}
	
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	if (Selected.Num() != 1 || Selected[0].Type != ERigVMExplorerElementType::Function)
	{
		return;
	}

	if (URigVMFunctionLibrary* FunctionLibrary = Editor->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
	{
		if (const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(*Selected[0].Name))
		{
			if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(FunctionNode->GetContainedGraph()))
			{
				Editor->AddNewFunctionVariant(EdGraph);
			}
		}
	}
}

bool SRigVMEditorGraphExplorer::CanAddFunctionVariant() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (!Editor)
	{
		return false;
	}
	
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	if (Selected.Num() != 1 || Selected[0].Type != ERigVMExplorerElementType::Function)
	{
		return false;
	}

	return true;
}

void SRigVMEditorGraphExplorer::OnDeleteEntry()
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
		TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		for (const FRigVMExplorerElementKey& Key : Selected)
		{
			switch (Key.Type)
			{
				case ERigVMExplorerElementType::Graph:
				{
					if (UEdGraph* EdGraph = Blueprint->GetEdGraph(Key.Name))
					{
						Schema->TryDeleteGraph(EdGraph);
					}
					break;
				}
				case ERigVMExplorerElementType::Function:
				{
					if (URigVMFunctionLibrary* Library = Blueprint->GetLocalFunctionLibrary())
					{
						if (URigVMLibraryNode* Function = Library->FindFunction(*Key.Name))
						{
							if (UEdGraph* EdGraph = Blueprint->GetEdGraph(Function->GetContainedGraph()))
							{
								Schema->TryDeleteGraph(EdGraph);
							}
						}
					}
					break;
				}
				case ERigVMExplorerElementType::Variable:
				{
					const FScopedTransaction Transaction( LOCTEXT( "RemoveVariable", "Remove Variable" ) );

					Blueprint->GetObject()->Modify();
					Blueprint->RemoveMemberVariable(*Key.Name);
					break;
				}
				case ERigVMExplorerElementType::LocalVariable:
				{
					if (URigVMGraph* FocusedGraph = Editor->GetFocusedModel())
					{
						if (URigVMController* Controller = Blueprint->GetRigVMClient()->GetController(FocusedGraph))
						{
							Controller->RemoveLocalVariable(*Key.Name, true, true);
						}
					}
					break;
				}
			}
		}
	}

	Refresh();
}


bool SRigVMEditorGraphExplorer::CanDeleteEntry() const
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (!Editor)
	{
		return false;
	}
	
	TArray<FRigVMExplorerElementKey> Selected = TreeView->GetSelectedKeys();
	for (const FRigVMExplorerElementKey& Key : Selected)
	{
		switch (Key.Type)
		{
			case ERigVMExplorerElementType::Section:
			case ERigVMExplorerElementType::Event:
			case ERigVMExplorerElementType::FunctionCategory:
			case ERigVMExplorerElementType::VariableCategory:
				{
					return false;
				}
			case ERigVMExplorerElementType::Graph:
			case ERigVMExplorerElementType::Function:
			case ERigVMExplorerElementType::Variable:
			case ERigVMExplorerElementType::LocalVariable:
				{
					return true;
				}
		}
	}
	
	return false;
}


void SRigVMEditorGraphExplorer::OnRenameEntry()
{
	TArray<FRigVMExplorerElementKey> Selection = TreeView->GetSelectedKeys();
	if (Selection.Num() != 1)
	{
		return;
	}

	TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Element = TreeView->FindElement(Selection[0]);
	if (Element.IsValid())
	{
		Element->RequestRename();
	}
}


bool SRigVMEditorGraphExplorer::CanRenameEntry() const
{
	return true;
}

bool SRigVMEditorGraphExplorer::OnRenameGraph(const FString& InOldPath, const FString& InNewPath)
{
	bool bResult = false;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(InOldPath))
		{
			bResult = Schema->TryRenameGraph(EdGraph, *InNewPath);
		}
	}

	Refresh();
	return bResult;
}

bool SRigVMEditorGraphExplorer::OnCanRenameGraph(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const
{
	bool bResult = false;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		int32 Index = INDEX_NONE;
		FString Prefix, NewPath = InNewPath;
		if(InOldPath.FindLastChar(TEXT('|'), Index))
		{
			Prefix = InOldPath.Left(Index+1);
		}
		if (!Prefix.IsEmpty())
		{
			NewPath = FString::Printf(TEXT("%s%s"), *Prefix, *InNewPath);
		}
		if (Editor->GetRigVMAssetInterface()->GetRigVMClient()->GetModel(NewPath))
		{
			OutErrorMessage = FText::FromString(TEXT("Name already in use."));
			bResult = false;
		}
		else
		{
			bResult = true;
		}
	}

	return bResult;
}

bool SRigVMEditorGraphExplorer::OnRenameFunction(const FString& InOldPath, const FString& InNewPath)
{
	bool bResult = false;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		if (URigVMFunctionLibrary* FunctionLibrary = Editor->GetRigVMAssetInterface()->GetLocalFunctionLibrary())
		{
			if (const URigVMLibraryNode* FunctionNode = FunctionLibrary->FindFunction(*InOldPath))
			{
				if (UEdGraph* EdGraph = Editor->GetRigVMAssetInterface()->GetEdGraph(FunctionNode->GetContainedGraph()))
				{
					bResult = Schema->TryRenameGraph(EdGraph, *InNewPath);
				}
			}
		}
	}

	Refresh();
	return bResult;
}

bool SRigVMEditorGraphExplorer::OnCanRenameFunction(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const
{
	bool bResult = false;
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();

	if (Editor)
	{
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		TSharedPtr<INameValidatorInterface> NameValidator = Schema->GetNameValidator(Editor->GetRigVMAssetInterface(), *InOldPath, nullptr);
		const EValidatorResult Result = NameValidator->IsValid(InNewPath);
		switch (Result)
		{
			case EValidatorResult::AlreadyInUse:
			case EValidatorResult::LocallyInUse:
				{
					OutErrorMessage = FText::FromString(TEXT("Name already in use."));
					bResult = false;
					break;
				}
			case EValidatorResult::ContainsInvalidCharacters:
				{
					OutErrorMessage = FText::FromString(TEXT("Name contains invalid characters."));
					bResult = false;
					break;
				}
			case EValidatorResult::TooLong:
				{
					OutErrorMessage = FText::FromString(TEXT("Name is too long."));
					bResult = false;
					break;
				}
			case EValidatorResult::EmptyName:
				{
					OutErrorMessage = FText::FromString(TEXT("Name is empty."));
					bResult = false;
					break;
				}
			default:
				{
					bResult = true;
				}
		}
	}

	return bResult;
}

bool SRigVMEditorGraphExplorer::OnRenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}
	
	// Check if the name is unchanged
	if (InNewName.Equals(InOldKey.Name, ESearchCase::CaseSensitive))
	{
		return false;
	}

	const FScopedTransaction Transaction( LOCTEXT( "RenameVariable", "Rename Variable" ));

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();

	if (InOldKey.Type == ERigVMExplorerElementType::Variable)
	{
		Blueprint->GetObject()->Modify();
		Blueprint->RenameMemberVariable(*InOldKey.Name, *InNewName);
	}
	else if (InOldKey.Type == ERigVMExplorerElementType::LocalVariable)
	{
		if (URigVMGraph* Graph = Blueprint->GetFocusedModel())
		{
			if (URigVMController* Controller = Blueprint->GetRigVMClient()->GetController(Graph))
			{
				Controller->RenameLocalVariable(*InOldKey.Name, *InNewName, true, true);
			}
		}
	}
	Refresh();
	return true;
}

bool SRigVMEditorGraphExplorer::OnReorderVariable(const FRigVMExplorerElementKey& InVariable, int32 NewIndex)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();

	const FScopedTransaction Transaction(LOCTEXT("ReorderVariable", "Reorder Variable"));

	// Call asset interface to reorder using name (will be converted to GUID internally)
	bool bSuccess = Blueprint->SetVariableIndex(*InVariable.Name, NewIndex);

	if (bSuccess)
	{
		Refresh();
	}

	return bSuccess;
}

bool SRigVMEditorGraphExplorer::OnReorderLocalVariable(const FRigVMExplorerElementKey& InVariable, int32 NewIndex)
{
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();

	// Find local variable in the focused model
	URigVMGraph* Graph = Blueprint->GetFocusedModel();
	if (!Graph)
	{
		return false;
	}

	URigVMController* Controller = Blueprint->GetRigVMClient()->GetController(Graph);
	if (!Controller)
	{
		return false;
	}

	// Find the local variable by name
	const TArray<FRigVMGraphVariableDescription>& LocalVars = Graph->GetLocalVariables();
	const FRigVMGraphVariableDescription* LocalVar = LocalVars.FindByPredicate(
		[&InVariable](const FRigVMGraphVariableDescription& Var) {
			return Var.Name.ToString() == InVariable.Name;
		});

	if (!LocalVar)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("ReorderLocalVariable", "Reorder Local Variable"));
	bool bSuccess = Controller->SetLocalVariableIndex(LocalVar->Guid, NewIndex, true, true);
	if (bSuccess)
	{
		Refresh();
	}
	return bSuccess;
}

bool SRigVMEditorGraphExplorer::OnCanRenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName, FText& OutErrorMessage) const
{
	if (InNewName.Equals(InOldKey.Name, ESearchCase::CaseSensitive))
	{
		return true;
	}
	
	TSharedPtr<IRigVMEditor> Editor = RigVMEditor.Pin();
	if (!Editor)
	{
		return false;
	}

	FRigVMEditorAssetInterfacePtr Blueprint = Editor->GetRigVMAssetInterface();
	if (!Blueprint)
	{
		OutErrorMessage = LOCTEXT("RenameVariable_NoAsset", "No valid asset context for rename.");
		return false;
	}

	EValidatorResult Result;
	if (InOldKey.Type == ERigVMExplorerElementType::LocalVariable)
	{
		FRigVMLocalVariableNameValidator Validator(Blueprint, Blueprint->GetFocusedModel(), *InOldKey.Name);
		Result = Validator.IsValid(InNewName, false);
	}
	else
	{
		FRigVMNameValidator Validator(Blueprint, nullptr, *InOldKey.Name);
		Result = Validator.IsValid(InNewName, false);
	}

	if (Result != EValidatorResult::Ok && Result != EValidatorResult::ExistingName)
	{
		OutErrorMessage = INameValidatorInterface::GetErrorText(InNewName, Result);
		return false;
	}
	
	return true;
}

void SRigVMEditorGraphExplorer::OnFindReferences(bool bSearchInAllBlueprints)
{
	if (!RigVMEditor.IsValid() ||
		!TreeView.IsValid())
	{
		return;
	}
	const TSharedRef<IRigVMEditor> PinnedEditor = RigVMEditor.Pin().ToSharedRef();

	const TArray<FRigVMExplorerElementKey> SelectedKeys = TreeView->GetSelectedKeys();

	// Only searching for a single keys is supported for the time being
	if (!CanFindReferences() ||
		SelectedKeys.Num() != 1)
	{
		PinnedEditor->SummonSearchUI(true);
		return;
	}

	FString SearchTerm;

	const TScriptInterface<IRigVMEditorAssetInterface> Asset = PinnedEditor->GetRigVMAssetInterface();
	const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
	if (!Asset || !Schema)
	{
		return;
	}

	switch (SelectedKeys[0].Type)
	{
	case ERigVMExplorerElementType::Graph:
	{
		if (UEdGraph* EdGraph = Asset->GetEdGraph(SelectedKeys[0].Name))
		{
			FGraphDisplayInfo DisplayInfo;
			Schema->GetGraphDisplayInformation(*EdGraph, DisplayInfo);
			SearchTerm = DisplayInfo.DisplayName.ToString();
		}
		break;
	}
	case ERigVMExplorerElementType::Event:
	case ERigVMExplorerElementType::Function:
	case ERigVMExplorerElementType::Variable:
	case ERigVMExplorerElementType::LocalVariable:
	{
		SearchTerm = SelectedKeys[0].Name;
		break;
	}
	default:
		ensureMsgf(0, TEXT("Unsupported element type"));
		break;
	}

	if (!SearchTerm.IsEmpty())
	{
		const bool bSetFindWithinBlueprint = !bSearchInAllBlueprints;
		PinnedEditor->SummonSearchUI(bSetFindWithinBlueprint, SearchTerm);
	}
}

bool SRigVMEditorGraphExplorer::CanFindReferences() const
{	
	// Only searching for a single keys is supported for the time being
	const TArray<FRigVMExplorerElementKey> SelectedKeys = TreeView.IsValid() ? TreeView->GetSelectedKeys() : TArray<FRigVMExplorerElementKey>();
	return SelectedKeys.Num() == 1;
}

void SRigVMEditorGraphExplorer::OnRemoveUnusedFunctions()
{
	const TScriptInterface<IRigVMEditorAssetInterface> Asset = RigVMEditor.IsValid() ? RigVMEditor.Pin()->GetRigVMAssetInterface() : nullptr;
	if (Asset)
	{
		UE::RigVMEditor::RemoveUnusedMembersFromAsset<URigVMFunctionReferenceNode>(
			Asset,
			LOCTEXT("RemoveUnusedFunctions", "Remove Unused Functions"), 
			LOCTEXT("AllFunctionsReferencedNotification", "All functions are referenced"));

		Refresh();
	}
}

bool SRigVMEditorGraphExplorer::CanRemoveUnusedFunctions() const
{
	return true;
}

void SRigVMEditorGraphExplorer::OnRemoveUnusedVariables()
{
	const TScriptInterface<IRigVMEditorAssetInterface> Asset = RigVMEditor.IsValid() ? RigVMEditor.Pin()->GetRigVMAssetInterface() : nullptr;
	if (Asset)
	{
		UE::RigVMEditor::RemoveUnusedMembersFromAsset<URigVMVariableNode>(
			Asset,
			LOCTEXT("RemoveUnusedVariables", "Remove Unused Variables"),
			LOCTEXT("AllVariablesReferencedNotification", "All variables are referenced"));

		Refresh();
	}
}

bool SRigVMEditorGraphExplorer::CanRemoveUnusedVariables() const
{
	return true;
}



#undef LOCTEXT_NAMESPACE
