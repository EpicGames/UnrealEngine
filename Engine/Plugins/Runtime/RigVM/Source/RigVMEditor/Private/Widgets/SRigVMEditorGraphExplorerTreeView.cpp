// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMEditorGraphExplorerTreeView.h"

#include "SPinTypeSelector.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Editor/RigVMNewEditor.h"
#include "Editor/RigVMEdGraphNodeRegistry.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "Widgets/SRigVMEditorGraphExplorer.h"

#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRigVMEditorGraphExplorerTreeView"

static const FText GraphsSectionName = LOCTEXT("Graphs", "Graphs");
static const FText FunctionsSectionName = LOCTEXT("Functions", "Functions");
static const FText VariablesSectionName = LOCTEXT("Variables", "Variables");
static const FText LocalVariablesSectionName = LOCTEXT("LocalVariables", "Local Variables");

void SRigVMEditorGraphExplorerItem::Construct(const FArguments& InArgs,
                                              const TSharedRef<STableViewBase>& InOwnerTable,
                                              TSharedRef<FRigVMEditorGraphExplorerTreeElement> InElement,
                                              TSharedPtr<SRigVMEditorGraphExplorerTreeView> InTreeView)
{
	using namespace UE::RigVMEditor;

	WeakExplorerElement = InElement;
	WeakTreeView = InTreeView;
	Delegates = InTreeView->GetRigTreeDelegates();
	OnAddClickedOnSection = InArgs._OnAddClickedOnSection;

	if (const TSharedPtr<UE::RigVMEditor::FRigVMEdGraphNodeRegistry> EdGraphNodeRegistry = GetEdGraphNodeRegistry())
	{
		UpdateIsUsedInGraph();

		EdGraphNodeRegistry->OnPostRegistryUpdated.AddSP(this, &SRigVMEditorGraphExplorerItem::UpdateIsUsedInGraph);
	}

	if (InElement->Key.Type == ERigVMExplorerElementType::Section)
	{
		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::Construct(
		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FArguments()
		.ShowWires(false)
		.Padding(FMargin(0.0f, 2.0f, .0f, 0.0f))
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header")) // SCategoryHeaderTableRow::GetBackgroundImage
			.Padding(FMargin(3.0f, 5.0f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SRichTextBlock)
					.Text(FText::FromString(InElement->Key.Name))
					.TransformPolicy(ETextTransformPolicy::ToUpper)
					.DecoratorStyleSet(&FAppStyle::Get())
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(FMargin(0,0,2,0))
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ContentPadding(FMargin(1, 0))
					.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
					.OnClicked(this, &SRigVMEditorGraphExplorerItem::OnAddButtonClickedOnSection, InElement)
					// .IsEnabled(this, &SRigVMEditorGraphExplorer::CanAddNewElementToSection, InSectionID)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.ToolTipText(LOCTEXT("AddNewGraphTooltip", "Create a new graph"))
					]
				]
			]
			
		], InOwnerTable);
	}
	else if(InElement->Key.Type == ERigVMExplorerElementType::FunctionCategory
		|| InElement->Key.Type == ERigVMExplorerElementType::VariableCategory)
	{
		int32 LastSeparator;
		InElement->Key.Name.FindLastChar('|', LastSeparator);
		FString CategoryName = InElement->Key.Name.RightChop(LastSeparator+1);
		CategoryName[0] = FChar::ToUpper(CategoryName[0]);
		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::Construct(
		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FArguments()
		.ShowWires(false)
		.Padding(FMargin(0.0f, 2.0f, .0f, 0.0f))
		.OnDragDetected(Delegates.OnDragDetected)
		.OnCanAcceptDrop(this, &SRigVMEditorGraphExplorerItem::OnCanAcceptDrop)
		.OnAcceptDrop(this, &SRigVMEditorGraphExplorerItem::OnAcceptDrop)
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
				.Font( FCoreStyle::GetDefaultFontStyle("Bold", 9) )
				.Text( FText::FromString(CategoryName ))
				.ToolTipText( FText::FromString(CategoryName ))
				.HighlightText( InTreeView->FilterText )
				.OnVerifyTextChanged(this, &SRigVMEditorGraphExplorerItem::OnVerifyNameChanged)
				.OnTextCommitted(this, &SRigVMEditorGraphExplorerItem::OnNameCommitted)
			]
		], InOwnerTable);
	}
	else if (InElement->Key.Type == ERigVMExplorerElementType::Variable || InElement->Key.Type == ERigVMExplorerElementType::LocalVariable)
	{
		TArray<TSharedPtr<IPinTypeSelectorFilter>> CustomPinTypeFilters = Delegates.GetCustomPinFilters();

		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::Construct(
			STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FArguments()
			.ShowWires(false)
			.OnDragDetected(Delegates.OnDragDetected)
			.OnCanAcceptDrop(this, &SRigVMEditorGraphExplorerItem::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SRigVMEditorGraphExplorerItem::OnAcceptDrop)
			.Content()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.FillWidth(0.6f)
					.VAlign(VAlign_Center)
					.Padding(3.0f, 0.0f)
					[
						CreateTextSlotWidget(InElement->Key, InTreeView->FilterText)
					]

				+SHorizontalBox::Slot()
					.FillWidth(0.4f)
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SPinTypeSelector, FGetPinTypeTree::CreateUObject(GetDefault<UEdGraphSchema_K2>(), &UEdGraphSchema_K2::GetVariableTypeTree))
						.Schema(GetDefault<URigVMEdGraphSchema>())
						.TargetPinType_Lambda([this, InElement]() { return Delegates.GetVariablePinType(InElement->Key); })
						.OnPinTypeChanged_Lambda([this, InElement](const FEdGraphPinType& InType) { Delegates.SetVariablePinType(InElement->Key, InType); })
						.TypeTreeFilter(ETypeTreeFilter::None)
						.SelectorType(SPinTypeSelector::ESelectorType::Partial)
						.CustomFilters(CustomPinTypeFilters)
					]

				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(6.0f, 0.0f, 3.0f, 0.0f))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SNew(SBorder)
						.Padding(0.0f)
						.Visibility(InElement->Key.Type == ERigVMExplorerElementType::Variable ? EVisibility::Visible : EVisibility::Collapsed)
						.BorderImage(FStyleDefaults::GetNoBrush())
						[
							SNew(SCheckBox)
							.ToolTipText_Lambda([this, InElement]()
							{
								FText ToolTipText = FText::GetEmpty();
								if (Delegates.IsVariablePublic(InElement->Key.Name))
								{
									ToolTipText = LOCTEXT("VariablePrivacy_is_public_Tooltip", "Variable is public and is editable on each instance of this Blueprint.");
								}
								else
								{
									ToolTipText = LOCTEXT("VariablePrivacy_not_public_Tooltip", "Variable is not public and will not be editable on an instance of this Blueprint.");
								}
								return ToolTipText;
							})
							.OnCheckStateChanged_Lambda([this, InElement](ECheckBoxState) { Delegates.ToggleVariablePublic(InElement->Key.Name); })
							.IsChecked_Lambda([this, InElement]() { return Delegates.IsVariablePublic(InElement->Key.Name) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.Style(FAppStyle::Get(), "TransparentCheckBox")
							[
								SNew(SImage)
								.Image_Lambda([this, InElement]()
								{
									if (Delegates.IsVariablePublic(InElement->Key.Name))
									{
										return FAppStyle::GetBrush( "Kismet.VariableList.ExposeForInstance" );
									}
									return FAppStyle::GetBrush( "Kismet.VariableList.HideForInstance" );
								})
								.ColorAndOpacity(FSlateColor::UseForeground())
						   ]
					   ]
				   ]

		   ], InOwnerTable);
	}
	else
	{
		STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::Construct(
		   STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FArguments()
		   .ShowWires(false)
		   .OnDragDetected(Delegates.OnDragDetected)
		   .Content()
		   [
			   SNew(SHorizontalBox)
			   + SHorizontalBox::Slot()
			   .AutoWidth()
			   .VAlign(VAlign_Center)
			   [
				   CreateIconWidget(InElement->Key)
			   ]
			   + SHorizontalBox::Slot()
			   .FillWidth(1.f)
			   .VAlign(VAlign_Center)
			   .Padding(/* horizontal */ 3.0f, /* vertical */ 3.0f)
			   [
				   CreateTextSlotWidget(InElement->Key, InTreeView->FilterText)
			   ]
		   ], InOwnerTable);
	}

	if(WeakExplorerElement.IsValid() && InlineRenameWidget.IsValid())
	{
		WeakExplorerElement.Pin()->OnRenameRequested.BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}
}

TSharedRef<SWidget> SRigVMEditorGraphExplorerItem::CreateIconWidget(const FRigVMExplorerElementKey& Key)
{
	TSharedPtr<SWidget> IconWidget;

	switch (Key.Type)
	{
		case ERigVMExplorerElementType::Graph:
			{
				IconWidget = SNew(SImage)
							.Image(Delegates.GetGraphIcon(Key.Name));
				break;
			}
		case ERigVMExplorerElementType::Event:
			{
				static FSlateIcon EventIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
				IconWidget = SNew(SImage)
							.Image(EventIcon.GetIcon());
				break;
			}
		case ERigVMExplorerElementType::Function:
			{
				static FSlateIcon FunctionIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
				IconWidget = SNew(SImage)
							.Image(FunctionIcon.GetIcon());
				break;
			}
	}

	if (IconWidget.IsValid())
	{
		return IconWidget.ToSharedRef();
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

TSharedRef<SWidget> SRigVMEditorGraphExplorerItem::CreateTextSlotWidget(
	const FRigVMExplorerElementKey& Key, 
	const FText& InHighlightText)
{
	InlineRenameWidget = SNew(SInlineEditableTextBlock)
		.Text(this, &SRigVMEditorGraphExplorerItem::GetDisplayText)
		.HighlightText(InHighlightText)
		.ToolTipText(this, &SRigVMEditorGraphExplorerItem::GetItemTooltip)
		.OnVerifyTextChanged(this, &SRigVMEditorGraphExplorerItem::OnVerifyNameChanged)
		.OnTextCommitted(this, &SRigVMEditorGraphExplorerItem::OnNameCommitted)
		.OnExitEditingMode_Lambda([this]()
			{
				if (TSharedPtr<SRigVMEditorGraphExplorerTreeView> TreeView = WeakTreeView.Pin())
				{
					TreeView->bWasAddNewItemClicked = false;
				}
			})
		.ColorAndOpacity_Lambda([this]()
			{
				if (OptionalIsUsedInGraph.IsSet())
				{
					return OptionalIsUsedInGraph.GetValue() ?
						FLinearColor::White :
						FLinearColor::White.CopyWithNewOpacity(.35f);
				}
				else
				{
					return FLinearColor::White;
				}
			});

	return InlineRenameWidget.ToSharedRef();
}

FText SRigVMEditorGraphExplorerItem::GetDisplayText() const
{
	FText DisplayText;
	if (!WeakExplorerElement.IsValid())
	{
		return DisplayText;
	}
	TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Element = WeakExplorerElement.Pin();

	switch (Element->Key.Type)
	{
		case ERigVMExplorerElementType::Graph:
			{
				return Delegates.GetGraphDisplayName(Element->Key.Name);
			}
		case ERigVMExplorerElementType::Event:
			{
				return Delegates.GetEventDisplayName(Element->Key.Name);
			}
		case ERigVMExplorerElementType::Function:
		case ERigVMExplorerElementType::Variable:
		case ERigVMExplorerElementType::LocalVariable:
			{
				return FText::FromString(Element->Key.Name);
			}
	}
	
	return DisplayText;
}

FText SRigVMEditorGraphExplorerItem::GetItemTooltip() const
{
	FText Tooltip;
	if (TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Element = WeakExplorerElement.Pin())
	{
		switch (Element->Key.Type)
		{
		case ERigVMExplorerElementType::Graph:
			{
				return Delegates.GetGraphTooltip(Element->Key.Name);
			}
		case ERigVMExplorerElementType::Function:
			{
				return Delegates.GetFunctionTooltip(Element->Key.Name);
			}
		case ERigVMExplorerElementType::Variable:
			{
				return Delegates.GetVariableTooltip(Element->Key.Name);
			}
		case ERigVMExplorerElementType::Event:
			{
				return Delegates.GetEventDisplayName(Element->Key.Name);
			}
		}
	}
	
	return Tooltip;
}

bool SRigVMEditorGraphExplorerItem::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	FString NewName = InText.ToString();
	const FRigVMExplorerElementKey OldKey = WeakExplorerElement.Pin()->Key;

	switch (OldKey.Type)
	{
		case ERigVMExplorerElementType::Section:
			{
				break;
			}
		case ERigVMExplorerElementType::FunctionCategory:
		case ERigVMExplorerElementType::VariableCategory:
			{
				return true;
			}
		case ERigVMExplorerElementType::Graph:
			{
				return Delegates.CanRenameGraph(OldKey.Name, *NewName, OutErrorMessage);
			}
		case ERigVMExplorerElementType::Function:
			{
				return Delegates.CanRenameFunction(OldKey.Name, *NewName, OutErrorMessage);
			}
		case ERigVMExplorerElementType::Variable:
		case ERigVMExplorerElementType::LocalVariable:
			{
				return Delegates.CanRenameVariable(OldKey, *NewName, OutErrorMessage);
			}
	}
	return false;
}

void SRigVMEditorGraphExplorerItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	if (!WeakExplorerElement.IsValid())
	{
		return;
	}
	
	// for now only allow enter
	if (InCommitType == ETextCommit::OnEnter)
	{
		FString NewName = InText.ToString();
		TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Element = WeakExplorerElement.Pin();
		const FRigVMExplorerElementKey OldKey = Element->Key;

		switch (OldKey.Type)
		{
			case ERigVMExplorerElementType::Section:
				{
					break;
				}
			case ERigVMExplorerElementType::FunctionCategory:
			case ERigVMExplorerElementType::VariableCategory:
				{
					int32 LastSeparator;
					FString OldCategoryPath = Element->Key.Name;
					Element->Key.Name.FindLastChar('|', LastSeparator);
					FString Prefix;
					if (LastSeparator != INDEX_NONE)
					{
						Prefix = Element->Key.Name.LeftChop(Element->Key.Name.Len() - LastSeparator);
					}
					if (!Prefix.IsEmpty() && !InText.IsEmpty())
					{
						Prefix.Append(TEXT("|"));
					}
					FString NewCategoryPath = FString::Printf(TEXT("%s%s"), *Prefix, *InText.ToString());
					TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> SubElements = Element->Children;
					for (int32 i=0; i<SubElements.Num(); ++i)
					{
						TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SubElement = SubElements[i];
						if (SubElement->Key.Type == ERigVMExplorerElementType::Function || SubElement->Key.Type == ERigVMExplorerElementType::Variable || SubElement->Key.Type == ERigVMExplorerElementType::LocalVariable)
						{
							FString Category = OldKey.Type == ERigVMExplorerElementType::FunctionCategory ?
								Delegates.GetFunctionCategory(SubElement->Key.Name)
									: Delegates.GetVariableCategory(SubElement->Key.Name);
							Category.RemoveFromStart(OldCategoryPath);
							Category.InsertAt(0, NewCategoryPath);
							Category.RemoveFromStart(TEXT("|"));
							Category.RemoveFromEnd(TEXT("|"));
							if (OldKey.Type == ERigVMExplorerElementType::FunctionCategory)
							{
								Delegates.SetFunctionCategory(SubElement->Key.Name, Category);
							}
							else if (OldKey.Type == ERigVMExplorerElementType::VariableCategory)
							{
								Delegates.SetVariableCategory(SubElement->Key.Name, Category);
							}
						}
						SubElements.Append(SubElement->Children);
					}
					break;
				}
			case ERigVMExplorerElementType::Graph:
				{
					Delegates.RenameGraph(OldKey.Name, *NewName);
					break;
				}
			case ERigVMExplorerElementType::Function:
				{
					Delegates.RenameFunction(OldKey.Name, *NewName);
					break;
				}
			case ERigVMExplorerElementType::Variable:
			case ERigVMExplorerElementType::LocalVariable:
				{
					Delegates.RenameVariable(OldKey, *NewName);
					break;
				}
		}
	}
}

FReply SRigVMEditorGraphExplorerItem::OnAddButtonClickedOnSection(TSharedRef<FRigVMEditorGraphExplorerTreeElement> InElement)
{
	if (OnAddClickedOnSection.IsBound())
	{
		return OnAddClickedOnSection.Execute(InElement->Key);
	}
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SRigVMEditorGraphExplorerItem::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigVMEditorGraphExplorerTreeElement> TargetItem)
{
	static const TOptional<EItemDropZone> InvalidDropZone;

	TSharedPtr<FRigVMGraphExplorerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRigVMGraphExplorerDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return InvalidDropZone;
	}

	const FRigVMExplorerElementKey& DraggedElement = DragDropOp->GetElement();
	const ERigVMExplorerElementType::Type DraggedType = DraggedElement.Type;
	const ERigVMExplorerElementType::Type TargetType = TargetItem->Key.Type;

	// Only Variable, LocalVariable, VariableCategory, and FunctionCategory can be dropped
	if (DraggedType != ERigVMExplorerElementType::Variable &&
		DraggedType != ERigVMExplorerElementType::LocalVariable &&
		DraggedType != ERigVMExplorerElementType::VariableCategory &&
		DraggedType != ERigVMExplorerElementType::FunctionCategory)
	{
		return InvalidDropZone;
	}

	// Cannot drop on self
	if (DraggedElement.Name == TargetItem->Key.Name && DraggedType == TargetType)
	{
		return InvalidDropZone;
	}

	// Check same section - items can only drop within their own section
	bool bSameSection = false;
	if (DraggedType == TargetType)
	{
		// Same type always same section
		bSameSection = true;
	}
	else if (TargetType == ERigVMExplorerElementType::VariableCategory)
	{
		// Need to verify the category is in the correct section
		// Walk up the parent hierarchy to find which section this category belongs to
		TSharedPtr<SRigVMEditorGraphExplorerTreeView> TreeView = WeakTreeView.Pin();
		if (TreeView.IsValid())
		{
			FRigVMExplorerElementKey CurrentKey = TargetItem->Key;
			while (const FRigVMExplorerElementKey* ParentKey = TreeView->ParentMap.Find(CurrentKey))
			{
				if (ParentKey->Type == ERigVMExplorerElementType::Section)
				{
					// Found the section - check if it matches the dragged type
					if ((DraggedType == ERigVMExplorerElementType::Variable || DraggedType == ERigVMExplorerElementType::VariableCategory) &&
						ParentKey->Name == VariablesSectionName.ToString())
					{
						bSameSection = true;
					}
					else if ((DraggedType == ERigVMExplorerElementType::LocalVariable) &&
						ParentKey->Name == LocalVariablesSectionName.ToString())
					{
						bSameSection = true;
					}
					break;
				}
				CurrentKey = *ParentKey;
			}
		}
	}
	else if (DraggedType == ERigVMExplorerElementType::VariableCategory || DraggedType == ERigVMExplorerElementType::FunctionCategory)
	{
		// When dragging a category, allow dropping onto same category type or compatible variable type
		if (TargetType == ERigVMExplorerElementType::Variable && DraggedType == ERigVMExplorerElementType::VariableCategory)
		{
			bSameSection = true;
		}
		else if (TargetType == ERigVMExplorerElementType::LocalVariable && DraggedType == ERigVMExplorerElementType::VariableCategory)
		{
			// Need to check if the category is in the local variables section
			TSharedPtr<SRigVMEditorGraphExplorerTreeView> TreeView = WeakTreeView.Pin();
			if (TreeView.IsValid())
			{
				FRigVMExplorerElementKey CurrentKey = DraggedElement;
				while (const FRigVMExplorerElementKey* ParentKey = TreeView->ParentMap.Find(CurrentKey))
				{
					if (ParentKey->Type == ERigVMExplorerElementType::Section)
					{
						if (ParentKey->Name == LocalVariablesSectionName.ToString())
						{
							bSameSection = true;
						}
						break;
					}
					CurrentKey = *ParentKey;
				}
			}
		}
	}

	if (!bSameSection)
	{
		return InvalidDropZone;
	}

	// Allow onto category to nest categories (category onto category)
	// Must check this BEFORE "variable onto category" to ensure cycle detection runs
	// Must be same category type: VariableCategory onto VariableCategory, or FunctionCategory onto FunctionCategory
	if (DraggedType == TargetType &&
		(DraggedType == ERigVMExplorerElementType::VariableCategory || DraggedType == ERigVMExplorerElementType::FunctionCategory) &&
		DropZone == EItemDropZone::OntoItem)
	{
		// Prevent dropping a category onto itself or its own descendants (would create a cycle)
		// Allow dropping a descendant onto its ancestor (promotes/un-nests the category)
		FString DraggedCategoryPath = DraggedElement.Name.TrimStartAndEnd();
		FString TargetCategoryPath = TargetItem->Key.Name.TrimStartAndEnd();

		// Don't allow dropping onto self
		if (DraggedCategoryPath.Equals(TargetCategoryPath, ESearchCase::IgnoreCase))
		{
			return InvalidDropZone;
		}

		// Don't allow dropping an ancestor onto its own descendant (e.g., dropping "aaa" onto "aaa|bbb|ccc")
		// This would create a cycle where "aaa" contains itself
		FString DraggedWithSeparator = DraggedCategoryPath + TEXT("|");
		if (TargetCategoryPath.StartsWith(DraggedWithSeparator, ESearchCase::IgnoreCase))
		{
			return InvalidDropZone;
		}

		// ALLOW dropping a descendant onto its own ancestor (e.g., dropping "aaa|bbb|ccc" onto "aaa")
		// This promotes/un-nests the category: "aaa|bbb|ccc" becomes "aaa|ccc"
		// No cycle is created because we're moving up the hierarchy

		return EItemDropZone::OntoItem;
	}

	// Allow onto category to change category (variable onto category)
	// Check this AFTER category-onto-category to ensure only variables (not categories) are accepted
	if ((DraggedType == ERigVMExplorerElementType::Variable || DraggedType == ERigVMExplorerElementType::LocalVariable) &&
		TargetType == ERigVMExplorerElementType::VariableCategory &&
		DropZone == EItemDropZone::OntoItem)
	{
		return EItemDropZone::OntoItem;
	}

	// Allow above/below same variable type for reordering
	if (TargetType == DraggedType &&
		(DropZone == EItemDropZone::AboveItem || DropZone == EItemDropZone::BelowItem))
	{
		return DropZone;
	}

	return InvalidDropZone;
}

FReply SRigVMEditorGraphExplorerItem::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigVMEditorGraphExplorerTreeElement> TargetItem)
{
	if (!OnCanAcceptDrop(DragDropEvent, DropZone, TargetItem).IsSet())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<FRigVMGraphExplorerDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRigVMGraphExplorerDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}

	const FRigVMExplorerElementKey& DraggedElement = DragDropOp->GetElement();
	const ERigVMExplorerElementType::Type DraggedType = DraggedElement.Type;
	const ERigVMExplorerElementType::Type TargetType = TargetItem->Key.Type;

	TSharedPtr<SRigVMEditorGraphExplorerTreeView> TreeView = WeakTreeView.Pin();
	if (!TreeView.IsValid())
	{
		return FReply::Unhandled();
	}

	// Handle category nesting (category onto category)
	if ((DraggedType == ERigVMExplorerElementType::VariableCategory || DraggedType == ERigVMExplorerElementType::FunctionCategory) &&
		TargetType == DraggedType &&
		DropZone == EItemDropZone::OntoItem)
	{

		FString DraggedCategoryPath = DraggedElement.Name;
		FString TargetCategoryPath = TargetItem->Key.Name;

		// Extract the leaf category name (last part after the last pipe)
		int32 LastPipeIndex;
		FString LeafCategoryName = DraggedCategoryPath;
		if (DraggedCategoryPath.FindLastChar('|', LastPipeIndex))
		{
			LeafCategoryName = DraggedCategoryPath.RightChop(LastPipeIndex + 1);
		}

		// New nested category path: TargetPath|LeafName
		FString NewCategoryPath = TargetCategoryPath + TEXT("|") + LeafCategoryName;

		// Find all items that belong to the dragged category (or its subcategories) and update them
		if (DraggedType == ERigVMExplorerElementType::VariableCategory)
		{
			TArray<FRigVMGraphVariableDescription> Variables = Delegates.GetVariables();
			for (const FRigVMGraphVariableDescription& Var : Variables)
			{
				FString VarCategory = Delegates.GetVariableCategory(Var.Name.ToString());

				// Check if this variable belongs to the dragged category or any of its subcategories
				if (VarCategory == DraggedCategoryPath || VarCategory.StartsWith(DraggedCategoryPath + TEXT("|")))
				{
					// Replace the dragged category path with the new nested path
					FString UpdatedCategory = NewCategoryPath;
					if (VarCategory.Len() > DraggedCategoryPath.Len())
					{
						// Preserve subcategory structure: "B|C" -> "A|B|C"
						UpdatedCategory = NewCategoryPath + VarCategory.RightChop(DraggedCategoryPath.Len());
					}
					Delegates.SetVariableCategory(Var.Name.ToString(), UpdatedCategory);
				}
			}

			// Also handle local variables
			TArray<FRigVMGraphVariableDescription> LocalVariables = Delegates.GetLocalVariables();
			for (const FRigVMGraphVariableDescription& Var : LocalVariables)
			{
				FString VarCategory = Delegates.GetVariableCategory(Var.Name.ToString());

				if (VarCategory == DraggedCategoryPath || VarCategory.StartsWith(DraggedCategoryPath + TEXT("|")))
				{
					FString UpdatedCategory = NewCategoryPath;
					if (VarCategory.Len() > DraggedCategoryPath.Len())
					{
						UpdatedCategory = NewCategoryPath + VarCategory.RightChop(DraggedCategoryPath.Len());
					}
					Delegates.SetVariableCategory(Var.Name.ToString(), UpdatedCategory);
				}
			}
		}
		else if (DraggedType == ERigVMExplorerElementType::FunctionCategory)
		{
			TArray<URigVMLibraryNode*> Functions = Delegates.GetFunctions();
			for (URigVMLibraryNode* Function : Functions)
			{
				FString FunctionCategory = Delegates.GetFunctionCategory(Function->GetPathName());

				if (FunctionCategory == DraggedCategoryPath || FunctionCategory.StartsWith(DraggedCategoryPath + TEXT("|")))
				{
					FString UpdatedCategory = NewCategoryPath;
					if (FunctionCategory.Len() > DraggedCategoryPath.Len())
					{
						UpdatedCategory = NewCategoryPath + FunctionCategory.RightChop(DraggedCategoryPath.Len());
					}
					Delegates.SetFunctionCategory(Function->GetPathName(), UpdatedCategory);
				}
			}
		}

		TreeView->RefreshTreeView();
		return FReply::Handled();
	}

	// Handle category assignment (variable onto category)
	if (TargetType == ERigVMExplorerElementType::VariableCategory && DropZone == EItemDropZone::OntoItem)
	{

		// Use the full category path from the category key
		FString CategoryPath = TargetItem->Key.Name;

		Delegates.SetVariableCategory(DraggedElement.Name, CategoryPath);
		TreeView->RefreshTreeView();
		return FReply::Handled();
	}

	// Handle reordering within same variable type
	if (TargetType == DraggedType &&
		(DropZone == EItemDropZone::AboveItem || DropZone == EItemDropZone::BelowItem))
	{
		// Get all variables of this type to find source and target indices
		TArray<FRigVMGraphVariableDescription> Variables;
		if (DraggedType == ERigVMExplorerElementType::Variable)
		{
			Variables = Delegates.GetVariables();
		}
		else if (DraggedType == ERigVMExplorerElementType::LocalVariable)
		{
			Variables = Delegates.GetLocalVariables();
		}

		// Find source index (where the dragged item currently is)
		int32 SourceIndex = INDEX_NONE;
		for (int32 i = 0; i < Variables.Num(); ++i)
		{
			if (Variables[i].Name.ToString() == DraggedElement.Name)
			{
				SourceIndex = i;
				break;
			}
		}

		// Find target index (where we want to drop)
		int32 TargetIndex = INDEX_NONE;
		for (int32 i = 0; i < Variables.Num(); ++i)
		{
			if (Variables[i].Name.ToString() == TargetItem->Key.Name)
			{
				TargetIndex = i;
				// If dropping below, insert after the target
				if (DropZone == EItemDropZone::BelowItem)
				{
					TargetIndex++;
				}
				break;
			}
		}

		if (SourceIndex != INDEX_NONE && TargetIndex != INDEX_NONE)
		{
			// When dropping onto a variable, adopt that variable's category
			// This allows both categorizing (if target has category) and uncategorizing (if target has no category)
			FString TargetCategory = Delegates.GetVariableCategory(TargetItem->Key.Name);
			FString CurrentCategory = Delegates.GetVariableCategory(DraggedElement.Name);
			if (TargetCategory != CurrentCategory)
			{
				Delegates.SetVariableCategory(DraggedElement.Name, TargetCategory);
			}

			// Adjust target index if we're moving down
			// When we remove the source item, all items after it shift down by one
			if (SourceIndex < TargetIndex)
			{
				TargetIndex--;
			}

			bool bSuccess = false;
			if (DraggedType == ERigVMExplorerElementType::Variable)
			{
				bSuccess = Delegates.OnReorderVariable.Execute(DraggedElement, TargetIndex);
			}
			else if (DraggedType == ERigVMExplorerElementType::LocalVariable)
			{
				bSuccess = Delegates.OnReorderLocalVariable.Execute(DraggedElement, TargetIndex);
			}

			if (bSuccess)
			{
				TreeView->RefreshTreeView();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

void SRigVMEditorGraphExplorerItem::UpdateIsUsedInGraph()
{
	using namespace UE::RigVMEditor;

	TRACE_CPUPROFILER_EVENT_SCOPE(SRigVMEditorGraphExplorerTreeView::UpdateIsUsedInGraph);

	if (!WeakExplorerElement.IsValid())
	{
		return;
	}

	if (const TSharedPtr<UE::RigVMEditor::FRigVMEdGraphNodeRegistry> EdGraphNodeRegistry = GetEdGraphNodeRegistry())
	{
		OptionalIsUsedInGraph.Reset();

		const FRigVMExplorerElementKey& Key = WeakExplorerElement.Pin()->Key;

		OptionalIsUsedInGraph = EdGraphNodeRegistry->GetConnectedEdGrapNodes().ContainsByPredicate(
				[&Key](const TWeakObjectPtr<URigVMEdGraphNode>& WeakEdGraphNode)
				{
					const URigVMVariableNode* VariableNode = WeakEdGraphNode.IsValid() ? 
						Cast<URigVMVariableNode>(WeakEdGraphNode->GetModelNode()) : 
						nullptr;

					const URigVMFunctionReferenceNode* FunctionReferenceNode = 
						WeakEdGraphNode.IsValid() ? Cast<URigVMFunctionReferenceNode>(WeakEdGraphNode->GetModelNode()) : 
						nullptr;

					if (VariableNode)
					{
						return VariableNode->GetVariableName() == Key.Name;
					}
					else if (FunctionReferenceNode)
					{
						return FunctionReferenceNode->GetFunctionIdentifier().GetFunctionName() == Key.Name;
					}

					return false;
				});
	}
}

TSharedPtr<UE::RigVMEditor::FRigVMEdGraphNodeRegistry> SRigVMEditorGraphExplorerItem::GetEdGraphNodeRegistry() const
{
	const TSharedPtr<SRigVMEditorGraphExplorer> GraphExplorer = WeakTreeView.IsValid() ? WeakTreeView.Pin()->GetGraphExplorer() : nullptr;
	if (!WeakExplorerElement.IsValid() ||
		!GraphExplorer.IsValid())
	{
		return nullptr;
	}

	const FRigVMExplorerElementKey& Key = WeakExplorerElement.Pin()->Key;

	if (Key.Type == ERigVMExplorerElementType::Variable ||
		Key.Type == ERigVMExplorerElementType::LocalVariable)
	{
		return GraphExplorer->GetEdGraphNodeVariableRegistry();
	}
	else if (Key.Type == ERigVMExplorerElementType::Function)
	{
		return GraphExplorer->GetEdGraphNodeFunctionRegistry();
	}
	else
	{
		return nullptr;
	}
}


TSharedRef<ITableRow> FRigVMEditorGraphExplorerTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FRigVMEditorGraphExplorerTreeElement> InElement, TSharedPtr<SRigVMEditorGraphExplorerTreeView> InTreeView)
{
	return SNew(SRigVMEditorGraphExplorerItem, InOwnerTable, InElement, InTreeView)
	.OnAddClickedOnSection(InTreeView.Get(), &SRigVMEditorGraphExplorerTreeView::OnAddButtonClickedOnSection);
}

void FRigVMEditorGraphExplorerTreeElement::RequestRename()
{
	// Defer to next tick
	GEditor->GetTimerManager()->SetTimerForNextTick([this]()
	{
		OnRenameRequested.ExecuteIfBound();
	});
}

void SRigVMEditorGraphExplorerTreeView::Construct(const FArguments& InArgs, const TSharedRef<SRigVMEditorGraphExplorer>& InGraphExplorer)
{
	WeakGraphExplorer = InGraphExplorer;

	Delegates = InArgs._RigTreeDelegates;
	
	STreeView<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FArguments SuperArgs;
	SuperArgs.SelectionMode(ESelectionMode::Single);
	SuperArgs.TreeItemsSource(&RootElements);
	SuperArgs.OnGenerateRow(this, &SRigVMEditorGraphExplorerTreeView::MakeTableRowWidget);
	SuperArgs.OnGetChildren(this, &SRigVMEditorGraphExplorerTreeView::HandleGetChildrenForTree);
	SuperArgs.HighlightParentNodesForSelection(true);
	SuperArgs.AllowInvisibleItemSelection(true);  //without this we deselect everything when we filter or we collapse
	SuperArgs.OnGeneratePinnedRow(this, &SRigVMEditorGraphExplorerTreeView::MakeTableRowWidget);
	SuperArgs.OnMouseButtonClick(this, &SRigVMEditorGraphExplorerTreeView::OnItemClicked);
	SuperArgs.OnMouseButtonDoubleClick(this, &SRigVMEditorGraphExplorerTreeView::OnItemDoubleClicked);
	SuperArgs.OnSelectionChanged(FRigVMGraphExplorer_OnSelectionChanged::CreateRaw(&Delegates, &FRigVMEditorGraphExplorerTreeDelegates::HandleSelectionChanged));
	SuperArgs.OnContextMenuOpening(FRigVMGraphExplorer_OnRequestContextMenu::CreateRaw(&Delegates, &FRigVMEditorGraphExplorerTreeDelegates::RequestContextMenu));


	STreeView<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::Construct(SuperArgs);
}

void SRigVMEditorGraphExplorerTreeView::RefreshTreeView(bool bRebuildContent)
{
	TArray<FRigVMExplorerElementKey> Selection;
	FString FilterTextStr = FilterText.ToString();
	
	TSharedPtr<FRigVMEditorGraphExplorerTreeElement> NewElement;
	if(bRebuildContent)
	{
		// store expansion state
		TMap<FRigVMExplorerElementKey, bool> ExpansionState;
		for (TPair<FRigVMExplorerElementKey, TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> Pair : ElementMap)
		{
			ExpansionState.FindOrAdd(Pair.Key) = IsItemExpanded(Pair.Value);
		}

		Selection = GetSelectedKeys();

		// internally save expansion states before rebuilding the tree, so the states can be restored later
		SaveAndClearSparseItemInfos();
		
		RootElements.Reset();
		ElementMap.Reset();
		ParentMap.Reset();

		// Add Elements
		{
			// GRAPHS section
			{
				FRigVMExplorerElementKey SectionKey(ERigVMExplorerElementType::Section, GraphsSectionName.ToString());
				TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SectionItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(SectionKey, SharedThis(this));
				ElementMap.Add(SectionKey, SectionItem);
				RootElements.Add(SectionItem);
				SetItemExpansion(SectionItem, true);

				TArray<const URigVMGraph*> Graphs = Delegates.GetRootGraphs();
				for (const URigVMGraph* Graph : Graphs)
				{
					FRigVMExplorerElementKey Key(ERigVMExplorerElementType::Graph, Graph->GetNodePath());
					ParentMap.Add(Key, SectionKey);
				}
			
				// Add Graphs to section
				for (int32 i=0; i<Graphs.Num(); ++i)
				{
					const URigVMGraph* Graph = Graphs[i];
				
					FRigVMExplorerElementKey Key(ERigVMExplorerElementType::Graph, Graph->GetNodePath());
				
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> NewItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(Key, SharedThis(this));
					ElementMap.Add(Key, NewItem);
					SetItemExpansion(NewItem, true);

					TArray<const URigVMGraph*> ChildrenGraphs = Delegates.GetChildrenGraphs(Key.Name);
					for (const URigVMGraph* Child : ChildrenGraphs)
					{
						FRigVMExplorerElementKey ChildKey(ERigVMExplorerElementType::Graph, Child->GetNodePath());
						ParentMap.Add(ChildKey, Key);
						Graphs.Add(Child);
					}

					TArray<URigVMNode*> EventNodes = Delegates.GetEventNodesInGraph(Key.Name);
					for (const URigVMNode* EventNode : EventNodes)
					{
					
						FRigVMExplorerElementKey EventKey(ERigVMExplorerElementType::Event, EventNode->GetNodePath());
						ParentMap.Add(EventKey, Key);
						TSharedPtr<FRigVMEditorGraphExplorerTreeElement> EventItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(EventKey, SharedThis(this));
						ElementMap.Add(EventKey, EventItem);
						NewItem->Children.Add(EventItem);
					}

					FRigVMExplorerElementKey ParentKey = ParentMap.FindChecked(Key);
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> ParentItem = ElementMap.FindChecked(ParentKey);

					ParentItem->Children.Add(NewItem);
				}
			}

			// FUNCTIONS section
			{
				FRigVMExplorerElementKey SectionKey(ERigVMExplorerElementType::Section, FunctionsSectionName.ToString());
				TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SectionItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(SectionKey, SharedThis(this));
				ElementMap.Add(SectionKey, SectionItem);
				RootElements.Add(SectionItem);
				SetItemExpansion(SectionItem, true);

				TArray<URigVMLibraryNode*> Functions = Delegates.GetFunctions();
				for (const URigVMLibraryNode* Function : Functions)
				{
					const FString Category = Function->GetNodeCategory();
					TArray<FString> SingleCategories;
					Category.ParseIntoArray(SingleCategories, TEXT("|"));
					FString CategoryParentPath;
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> PreviousParentItem = SectionItem;
					for (const FString& SingleCategory : SingleCategories)
					{
						if (!CategoryParentPath.IsEmpty())
						{
							CategoryParentPath.Append(TEXT("|"));
						}
						CategoryParentPath.Append(SingleCategory);
						FRigVMExplorerElementKey ParentKey(ERigVMExplorerElementType::FunctionCategory, CategoryParentPath);
						TSharedPtr<FRigVMEditorGraphExplorerTreeElement> ParentItem = FindElement(ParentKey);
						if (!ParentItem)
						{
							ParentItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(ParentKey, SharedThis(this));
							ElementMap.Add(ParentKey, ParentItem);
							ParentMap.Add(ParentKey, PreviousParentItem->Key);
							PreviousParentItem->Children.Add(ParentItem);
							SetItemExpansion(ParentItem, true);
						}
						PreviousParentItem = ParentItem;
					}
					
					FRigVMExplorerElementKey Key(ERigVMExplorerElementType::Function, Function->GetNodePath());
				
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> NewItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(Key, SharedThis(this));
					ElementMap.Add(Key, NewItem);
					ParentMap.Add(Key, PreviousParentItem->Key);
					PreviousParentItem->Children.Add(NewItem);
					SetItemExpansion(NewItem, true);
				}
			}

			// VARIABLES section
			{
				FRigVMExplorerElementKey SectionKey(ERigVMExplorerElementType::Section, VariablesSectionName.ToString());
				TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SectionItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(SectionKey, SharedThis(this));
				ElementMap.Add(SectionKey, SectionItem);
				RootElements.Add(SectionItem);
				SetItemExpansion(SectionItem, true);

				TArray<FRigVMGraphVariableDescription> Variables = Delegates.GetVariables();
				for (const FRigVMGraphVariableDescription& Variable : Variables)
				{
					const FString Category = Variable.Category.ToString();
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> PreviousParentItem = SectionItem;
					if (!Category.Equals(UEdGraphSchema_K2::VR_DefaultCategory.ToString()))
					{
						TArray<FString> SingleCategories;
						Category.ParseIntoArray(SingleCategories, TEXT("|"));
						FString CategoryParentPath;
						for (const FString& SingleCategory : SingleCategories)
						{
							if (!CategoryParentPath.IsEmpty())
							{
								CategoryParentPath.Append(TEXT("|"));
							}
							CategoryParentPath.Append(SingleCategory);
							FRigVMExplorerElementKey ParentKey(ERigVMExplorerElementType::VariableCategory, CategoryParentPath);
							TSharedPtr<FRigVMEditorGraphExplorerTreeElement> ParentItem = FindElement(ParentKey);
							if (!ParentItem)
							{
								ParentItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(ParentKey, SharedThis(this));
								ElementMap.Add(ParentKey, ParentItem);
								ParentMap.Add(ParentKey, PreviousParentItem->Key);
								PreviousParentItem->Children.Add(ParentItem);
								SetItemExpansion(ParentItem, true);
							}
							PreviousParentItem = ParentItem;
						}
					}
					
					FRigVMExplorerElementKey Key(ERigVMExplorerElementType::Variable, Variable.Name.ToString());
				
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> NewItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(Key, SharedThis(this));
					ElementMap.Add(Key, NewItem);
					ParentMap.Add(Key, PreviousParentItem->Key);
					PreviousParentItem->Children.Add(NewItem);
					SetItemExpansion(NewItem, true);
				}
			}

			// LOCAL VARIABLES section
			if (Delegates.IsFunctionFocused())
			{
				FRigVMExplorerElementKey SectionKey(ERigVMExplorerElementType::Section, LocalVariablesSectionName.ToString());
				TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SectionItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(SectionKey, SharedThis(this));
				ElementMap.Add(SectionKey, SectionItem);
				RootElements.Add(SectionItem);
				SetItemExpansion(SectionItem, true);

				TArray<FRigVMGraphVariableDescription> LocalVariables = Delegates.GetLocalVariables();
				for (const FRigVMGraphVariableDescription& Variable : LocalVariables)
				{
					FRigVMExplorerElementKey Key(ERigVMExplorerElementType::LocalVariable, Variable.Name.ToString());
				
					TSharedPtr<FRigVMEditorGraphExplorerTreeElement> NewItem = MakeShared<FRigVMEditorGraphExplorerTreeElement>(Key, SharedThis(this));
					ElementMap.Add(Key, NewItem);
					ParentMap.Add(Key, SectionItem->Key);
					SectionItem->Children.Add(NewItem);
					SetItemExpansion(NewItem, true);
				}
			}
		}

		// Filter Items
		if (!FilterText.IsEmpty())
		{
			for (TSharedPtr<FRigVMEditorGraphExplorerTreeElement> RootElement : RootElements)
			{
				struct Local
				{
					static bool ChildNameSatisfiesFilter(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement, const FString& InFilterText, FRigVMEditorGraphExplorerTreeDelegates& Delegates)
					{
						FText DisplayName;
						switch (InElement->Key.Type)
						{
						case ERigVMExplorerElementType::Graph:
							{
								DisplayName = Delegates.GetGraphDisplayName(InElement->Key.Name);
								break;
							}
						case ERigVMExplorerElementType::Event:
							{
								DisplayName = Delegates.GetEventDisplayName(InElement->Key.Name);
								break;
							}
						case ERigVMExplorerElementType::Function:
						case ERigVMExplorerElementType::Variable:
						case ERigVMExplorerElementType::LocalVariable:
						case ERigVMExplorerElementType::FunctionCategory:
						case ERigVMExplorerElementType::VariableCategory:
							{
								DisplayName = FText::FromString(InElement->Key.Name);
								break;
							}
						}
						
						if (InFilterText.IsEmpty() || DisplayName.ToString().Contains(InFilterText))
						{
							return true;
						}
						return false;
					}
					
					static bool HasVisibleChildren(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement, const FString& InFilterText, FRigVMEditorGraphExplorerTreeDelegates& Delegates)
					{
						TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> NewChildren;
						NewChildren.Reserve(InElement->Children.Num());
						for (TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Child : InElement->Children)
						{
							if (HasVisibleChildren(Child, InFilterText, Delegates) || ChildNameSatisfiesFilter(Child, InFilterText, Delegates))
							{
								NewChildren.Add(Child);
							}
						}
						InElement->Children = NewChildren;
						return !NewChildren.IsEmpty();
					}
				};
				TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> NewChildren;
				NewChildren.Reserve(RootElement->Children.Num());
				for (TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Child : RootElement->Children)
				{
					if (Local::HasVisibleChildren(Child, FilterTextStr, Delegates) || Local::ChildNameSatisfiesFilter(Child, FilterTextStr, Delegates))
					{
						NewChildren.Add(Child);
					}
				}
				RootElement->Children = NewChildren;
			}
		}
		
		// Find new element added
		if (bWasAddNewItemClicked)
		{
			for (TPair<FRigVMExplorerElementKey, TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& Element : ElementMap)
			{
				if (!ExpansionState.Contains(Element.Key))
				{
					NewElement = Element.Value;
					break;
				}
			}
		}


		// expand all elements upon the initial construction of the tree
		if (ExpansionState.Num() == 0)
		{
			for (TSharedPtr<FRigVMEditorGraphExplorerTreeElement> RootElement : RootElements)
			{
				SetExpansionRecursive(RootElement, false, true);
			}
		}
		// expand any new items
		else if (ExpansionState.Num() < ElementMap.Num())
		{
			for (const TPair<FRigVMExplorerElementKey, TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& Element : ElementMap)
			{
				if (!ExpansionState.Contains(Element.Key))
				{
					SetItemExpansion(Element.Value, true);
				}
			}
		}

		// restore infos
		for (const TPair<FRigVMExplorerElementKey, TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& Pair : ElementMap)
		{
			RestoreSparseItemInfos(Pair.Value);
		}
	}


	RequestTreeRefresh();
	
	if (!bWasAddNewItemClicked || !NewElement.IsValid())
	{
		TGuardValue<bool> Guard(Delegates.bSuspendSelectionDelegate, true);
		ClearSelection();

		if(!Selection.IsEmpty())
		{
			TArray<FRigVMExplorerElementKey> SelectedElements;
			for(const FRigVMExplorerElementKey& SelectedPath : Selection)
			{
				SelectedElements.Add(SelectedPath);
			}
			if(!SelectedElements.IsEmpty())
			{
				SetSelection(SelectedElements);
			}
		}

		Selection = GetSelectedKeys();
	}
	else
	{
		NewElement->RequestRename();
	}
}

TSharedRef<ITableRow> SRigVMEditorGraphExplorerTreeView::MakeTableRowWidget(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InItem,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
}

void SRigVMEditorGraphExplorerTreeView::SetExpansionRecursive(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement, bool bTowardsParent, bool bShouldBeExpanded)
{
	SetItemExpansion(InElement, bShouldBeExpanded);

	if (bTowardsParent)
	{
		if (const FRigVMExplorerElementKey* ParentKey = ParentMap.Find(InElement->Key))
		{
			if (TSharedPtr<FRigVMEditorGraphExplorerTreeElement>* ParentItem = ElementMap.Find(*ParentKey))
			{
				SetExpansionRecursive(*ParentItem, bTowardsParent, bShouldBeExpanded);
			}
		}
	}
	else
	{
		for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
		{
			SetExpansionRecursive(InElement->Children[ChildIndex], bTowardsParent, bShouldBeExpanded);
		}
	}
}

void SRigVMEditorGraphExplorerTreeView::HandleGetChildrenForTree(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InItem,
	TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& OutChildren)
{
	OutChildren = InItem.Get()->Children;
}

TArray<FRigVMExplorerElementKey> SRigVMEditorGraphExplorerTreeView::GetKeys() const
{
	TArray<FRigVMExplorerElementKey> Keys;
	ElementMap.GenerateKeyArray(Keys);
	
	return Keys;
}

TArray<FRigVMExplorerElementKey> SRigVMEditorGraphExplorerTreeView::GetSelectedKeys() const
{
	TArray<FRigVMExplorerElementKey> Keys;
	TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> SelectedElements = GetSelectedItems();
	for(const TSharedPtr<FRigVMEditorGraphExplorerTreeElement>& SelectedElement : SelectedElements)
	{
		if (SelectedElement.IsValid())
		{
			Keys.AddUnique(SelectedElement->Key);
		}
	}
	return Keys;
}

void SRigVMEditorGraphExplorerTreeView::SetSelection(TArray<FRigVMExplorerElementKey>& InSelectedKeys)
{
	ClearSelection();
	TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> Selection;
	for (const FRigVMExplorerElementKey& Key : InSelectedKeys)
	{
		if (TSharedPtr<FRigVMEditorGraphExplorerTreeElement>* Element = ElementMap.Find(Key))
		{
			Selection.Add(*Element);
		}
	}
	SetItemSelection(Selection, true, ESelectInfo::Direct);
}

TSharedPtr<FRigVMEditorGraphExplorerTreeElement> SRigVMEditorGraphExplorerTreeView::FindElement(const FRigVMExplorerElementKey& Key)
{
	if (TSharedPtr<FRigVMEditorGraphExplorerTreeElement>* Element = ElementMap.Find(Key))
	{
		return *Element;
	}
	return nullptr;
}

void SRigVMEditorGraphExplorerTreeView::OnItemClicked(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement)
{
	switch(InElement->Key.Type)
	{
		case ERigVMExplorerElementType::Section:
			{
				FRigVMExplorerElementKey SectionKey(ERigVMExplorerElementType::Section, InElement->Key.Name);
				TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Section = ElementMap.FindChecked(SectionKey);
				SetItemExpansion(Section, !IsItemExpanded(Section));
				break;
			}
		case ERigVMExplorerElementType::Graph:
			{
				Delegates.GraphClicked(InElement->Key.Name);
				break;
			}
		case ERigVMExplorerElementType::Event:
			{
				Delegates.EventClicked(InElement->Key.Name);
				break;
			}
		case ERigVMExplorerElementType::Function:
			{
				Delegates.FunctionClicked(InElement->Key.Name);
				break;
			}
		case ERigVMExplorerElementType::Variable:
		case ERigVMExplorerElementType::LocalVariable:
			{
				Delegates.VariableClicked(InElement->Key);
				break;
			}
	}
}

void SRigVMEditorGraphExplorerTreeView::OnItemDoubleClicked(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement)
{
	switch(InElement->Key.Type)
	{
		case ERigVMExplorerElementType::Section:
		case ERigVMExplorerElementType::Variable:
		case ERigVMExplorerElementType::LocalVariable:
			{
				break;
			}
		case ERigVMExplorerElementType::Graph:
			{
				Delegates.GraphDoubleClicked(InElement->Key.Name);
				break;
			}
		case ERigVMExplorerElementType::Event:
			{
				Delegates.EventDoubleClicked(InElement->Key.Name);
			}
		case ERigVMExplorerElementType::Function:
			{
				Delegates.FunctionDoubleClicked(InElement->Key.Name);
			}
	}
}

FReply SRigVMEditorGraphExplorerTreeView::OnAddButtonClickedOnSection(const FRigVMExplorerElementKey& InSectionKey)
{
	bWasAddNewItemClicked = true;
	if (InSectionKey.Name == GraphsSectionName.ToString())
	{
		Delegates.CreateGraph();
		return FReply::Handled();
	}
	else if (InSectionKey.Name == FunctionsSectionName.ToString())
	{
		Delegates.CreateFunction();
		return FReply::Handled();
	}
	else if (InSectionKey.Name == VariablesSectionName.ToString())
	{
		Delegates.CreateVariable();
		return FReply::Handled();
	}
	else if (InSectionKey.Name == LocalVariablesSectionName.ToString())
	{
		Delegates.CreateLocalVariable();
		return FReply::Handled();
	}
	bWasAddNewItemClicked = false;
	return FReply::Unhandled();
}

ERigVMGraphExplorerSectionType SRigVMEditorGraphExplorerTreeView::GetSectionType(const FRigVMExplorerElementKey& Key) const
{
	if (Key.Type == ERigVMExplorerElementType::Section)
	{
		if (Key.Name == GraphsSectionName.ToString())
		{
			return ERigVMGraphExplorerSectionType::Graphs;
		}
		else if (Key.Name == FunctionsSectionName.ToString())
		{
			return ERigVMGraphExplorerSectionType::Functions;
		}
		else if (Key.Name == VariablesSectionName.ToString())
		{
			return ERigVMGraphExplorerSectionType::Variables;
		}
		else if (Key.Name == LocalVariablesSectionName.ToString())
		{
			return ERigVMGraphExplorerSectionType::LocalVariables;
		}
	}

	return ERigVMGraphExplorerSectionType::None;
}

TSharedPtr<SRigVMEditorGraphExplorer> SRigVMEditorGraphExplorerTreeView::GetGraphExplorer() const
{
	return WeakGraphExplorer.IsValid() ? WeakGraphExplorer.Pin() : nullptr;
}

#undef LOCTEXT_NAMESPACE
