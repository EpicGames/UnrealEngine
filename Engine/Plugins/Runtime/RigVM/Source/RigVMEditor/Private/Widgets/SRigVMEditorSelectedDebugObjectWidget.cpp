// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMEditorSelectedDebugObjectWidget.h"

#include "UObject/UObjectIterator.h"
#include "PreviewScene.h"
#include "IDocumentation.h"
#include "PropertyCustomizationHelpers.h"
#include "RigVMBlueprintLegacy.h"
#include "SLevelOfDetailBranchNode.h"
#include "UnrealEdGlobals.h"
#include "Components/Widget.h"
#include "Editor/UnrealEdEngine.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/SToolTip.h"
#include "Components/Widget.h"

#define LOCTEXT_NAMESPACE "RigVMToolbar"

//////////////////////////////////////////////////////////////////////////
// SRigVMEditorSelectedDebugObjectWidget

void SRigVMEditorSelectedDebugObjectWidget::Construct(const FArguments& InArgs, TSharedPtr<IRigVMEditor> InEditor)
{
	Editor = InEditor;

	GenerateDebugWorldNames(false);
	GenerateDebugObjectInstances(false);

	LastObjectObserved = nullptr;

	DebugWorldsComboBox = SNew(STextComboBox)
		.ToolTip(IDocumentation::Get()->CreateToolTip(
			LOCTEXT("RigVMDebugWorldTooltip", "Select a world to debug, will filter what to debug if no specific object selected"),
			nullptr, FString(),
			TEXT("DebugWorld")))
		.OptionsSource(&DebugWorldNames)
		.InitiallySelectedItem(GetDebugWorldName())
		.Visibility(this, &SRigVMEditorSelectedDebugObjectWidget::IsDebugWorldComboVisible)
		.OnComboBoxOpening(this, &SRigVMEditorSelectedDebugObjectWidget::GenerateDebugWorldNames, true)
		.OnSelectionChanged(this, &SRigVMEditorSelectedDebugObjectWidget::DebugWorldSelectionChanged)
		.ContentPadding(FMargin(0.f, 4.f));

	DebugObjectsComboBox = SNew(SComboBox<TSharedPtr<FRigVMDebugObjectInstance>>)
		.ToolTip(IDocumentation::Get()->CreateToolTip(
			LOCTEXT("RigVMDebugObjectTooltip", "Select an object to debug, if set to none will debug any object"),
			nullptr,
			FString(),
			TEXT("DebugObject")))
		.OptionsSource(&DebugObjects)
		.InitiallySelectedItem(GetDebugObjectInstance())
		.OnComboBoxOpening(this, &SRigVMEditorSelectedDebugObjectWidget::GenerateDebugObjectInstances, true)
		.OnSelectionChanged(this, &SRigVMEditorSelectedDebugObjectWidget::DebugObjectSelectionChanged)
		.OnGenerateWidget(this, &SRigVMEditorSelectedDebugObjectWidget::CreateDebugObjectItemWidget)
		.ContentPadding(FMargin(0.f, 4.f))
		.AddMetaData<FTagMetaData>(TEXT("SelectDebugObjectCobmo"))
		[
			SNew(STextBlock)
			.Text(this, &SRigVMEditorSelectedDebugObjectWidget::GetSelectedDebugObjectTextLabel)
		];

	const TSharedRef<SWidget> BrowseButton = PropertyCustomizationHelpers::MakeBrowseButton(
		FSimpleDelegate::CreateSP(this, &SRigVMEditorSelectedDebugObjectWidget::SelectedDebugObject_OnClicked),
		LOCTEXT("DebugSelectActor", "Select and frame the debug actor in the Level Editor."),
		TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SRigVMEditorSelectedDebugObjectWidget::IsDebugObjectSelected))
	);


	TSharedRef<SWidget> DebugObjectSelectionWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
		[
			DebugObjectsComboBox.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(4.0f)
		[
			BrowseButton
		];

	ChildSlot
	[
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f)
			.AutoWidth()
			[
				DebugWorldsComboBox.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(0.0f)
			.AutoWidth()
			[
				DebugObjectSelectionWidget
			]
	];
}

void SRigVMEditorSelectedDebugObjectWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
	{
		if (UObject* Object = Blueprint->GetObjectBeingDebugged())
		{
			if (Object != LastObjectObserved.Get())
			{
				// bRestoreSelection attempts to restore the selection by name, 
				// this ensures that if the last object we had selected was 
				// regenerated (spawning a new object), then we select that  
				// again, even if it is technically a different object
				GenerateDebugObjectInstances(/*bRestoreSelection =*/true);

				LastObjectObserved = Object;
			}
		}
		else
		{
			LastObjectObserved = nullptr;

			// If the object name is a name (rather than the 'No debug selected' string then regenerate the names (which will reset the combo box) as the object is invalid.
			TSharedPtr<FRigVMDebugObjectInstance> CurrentSelection = DebugObjectsComboBox->GetSelectedItem();
			if (CurrentSelection.IsValid() && CurrentSelection->IsEditorObject())
			{
				GenerateDebugObjectInstances(false);
			}
		}
	}
}

const FString& SRigVMEditorSelectedDebugObjectWidget::GetNoDebugString() const
{
	return NSLOCTEXT("BlueprintEditor", "DebugObjectNothingSelected", "No debug object selected").ToString();
}

const FString& SRigVMEditorSelectedDebugObjectWidget::GetDebugAllWorldsString() const
{
	return NSLOCTEXT("BlueprintEditor", "DebugWorldNothingSelected", "All Worlds").ToString();
}

void SRigVMEditorSelectedDebugObjectWidget::OnRefresh()
{
	if (GetRigVMAssetInterface())
	{
		GenerateDebugWorldNames(false);
		GenerateDebugObjectInstances(false);

		if (DebugObjectsComboBox.IsValid())
		{
			DebugWorldsComboBox->SetSelectedItem(GetDebugWorldName());
			DebugObjectsComboBox->SetSelectedItem(GetDebugObjectInstance());
		}
	}
}

void SRigVMEditorSelectedDebugObjectWidget::GenerateDebugWorldNames(bool bRestoreSelection)
{
	DebugWorldNames.Empty();
	DebugWorlds.Empty();

	DebugWorlds.Add(nullptr);
	DebugWorldNames.Add(MakeShareable(new FString(GetDebugAllWorldsString())));

	for (TObjectIterator<UWorld> It; It; ++It)
	{
		UWorld *TestWorld = *It;

		// Include only PIE and worlds that own the persistent level (i.e. non-streaming levels).
		const bool bIsValidDebugWorld = (TestWorld != nullptr)
			&& TestWorld->WorldType == EWorldType::PIE
			&& TestWorld->PersistentLevel != nullptr
			&& TestWorld->PersistentLevel->OwningWorld == TestWorld;

		if (!bIsValidDebugWorld)
		{
			continue;
		}

		ENetMode NetMode = TestWorld->GetNetMode();

		FString WorldName;

		switch (NetMode)
		{
		case NM_Standalone:
			WorldName = NSLOCTEXT("BlueprintEditor", "DebugWorldStandalone", "Standalone").ToString();
			break;

		case NM_ListenServer:
			WorldName = NSLOCTEXT("BlueprintEditor", "DebugWorldListenServer", "Listen Server").ToString();
			break;

		case NM_DedicatedServer:
			WorldName = NSLOCTEXT("BlueprintEditor", "DebugWorldDedicatedServer", "Dedicated Server").ToString();
			break;

		case NM_Client:
			if (FWorldContext* PieContext = GEngine->GetWorldContextFromWorld(TestWorld))
			{
				WorldName = FString::Printf(TEXT("%s %d"), *NSLOCTEXT("BlueprintEditor", "DebugWorldClient", "Client").ToString(), PieContext->PIEInstance - 1);
			}
			break;
		};

		if (!WorldName.IsEmpty())
		{
			if (FWorldContext* PieContext = GEngine->GetWorldContextFromWorld(TestWorld))
			{
				if (!PieContext->CustomDescription.IsEmpty())
				{
					WorldName += TEXT(" ") + PieContext->CustomDescription;
				}
			}

			// DebugWorlds & DebugWorldNames need to be the same size (we expect
			// an index in one to correspond to the other) - DebugWorldNames is
			// what populates the dropdown, so it is the authority (if there's 
			// no name to present, they can't select from DebugWorlds)
			DebugWorlds.Add(TestWorld);
			DebugWorldNames.Add( MakeShareable(new FString(WorldName)) );
		}
	}

	if (DebugWorldsComboBox.IsValid())
	{
		// Attempt to restore the old selection
		if (bRestoreSelection)
		{
			TSharedPtr<FString> CurrentDebugWorld = GetDebugWorldName();
			if (CurrentDebugWorld.IsValid())
			{
				DebugWorldsComboBox->SetSelectedItem(CurrentDebugWorld);
			}
		}

		// Finally ensure we have a valid selection
		TSharedPtr<FString> CurrentSelection = DebugWorldsComboBox->GetSelectedItem();
		if (DebugWorldNames.Find(CurrentSelection) == INDEX_NONE)
		{
			if (DebugWorldNames.Num() > 0)
			{
				DebugWorldsComboBox->SetSelectedItem(DebugWorldNames[0]);
			}
			else
			{
				DebugWorldsComboBox->ClearSelection();
			}
		}

		DebugWorldsComboBox->RefreshOptions();
	}
}

void SRigVMEditorSelectedDebugObjectWidget::GenerateDebugObjectInstances(bool bRestoreSelection)
{
	TSharedPtr<IRigVMEditor> EditorPtr = Editor.Pin();
	if (!EditorPtr)
	{
		return;
	}
	
	// Cache the current selection as we may need to restore it
	TSharedPtr<FRigVMDebugObjectInstance> LastSelection = GetDebugObjectInstance();

	// Empty the lists of actors and regenerate them
	DebugObjects.Empty();
	DebugObjects.Add(MakeShareable(new FRigVMDebugObjectInstance(nullptr, GetNoDebugString())));

	// Grab custom objects that should always be visible, regardless of the world
	TArray<FRigVMCustomDebugObject> CustomDebugObjects;
	EditorPtr->GetDebugObjects(/*inout*/ CustomDebugObjects);

	for (const FRigVMCustomDebugObject& Entry : CustomDebugObjects)
	{
		AddDebugObject(Entry.Object, Entry.NameOverride);
	}

	// Check for a specific debug world. If DebugWorld=nullptr we take that as "any PIE world"
	UWorld* DebugWorld = nullptr;
	if (DebugWorldsComboBox.IsValid())
	{
		TSharedPtr<FString> CurrentWorldSelection = DebugWorldsComboBox->GetSelectedItem();
		int32 SelectedIndex = INDEX_NONE;
		for (int32 WorldIdx = 0; WorldIdx < DebugWorldNames.Num(); ++WorldIdx)
		{
			if (DebugWorldNames[WorldIdx].IsValid() && CurrentWorldSelection.IsValid()
				&& (*DebugWorldNames[WorldIdx] == *CurrentWorldSelection))
			{
				SelectedIndex = WorldIdx;
				break;
			}
		}
		if (SelectedIndex > 0 && DebugWorldNames.IsValidIndex(SelectedIndex))
		{
			DebugWorld = DebugWorlds[SelectedIndex].Get();
		}
	}

	UWorld* PreviewWorld = nullptr;
	if (FPreviewScene* PreviewScene = EditorPtr->GetPreviewScene())
	{
		PreviewWorld = PreviewScene->GetWorld();
	}

	if (!EditorPtr->OnlyShowCustomDebugObjects())
	{
		//if (BlueprintClass)
		{
			// Experimental new path for debug object discovery
			bool bIncludeDerivedClasses = true;
			TArray<UObject*> BlueprintInstances = GetRigVMAssetInterface()->GetArchetypeInstances(false, bIncludeDerivedClasses);
			
			for (auto It = BlueprintInstances.CreateIterator(); It; ++It)
			{
				UObject* TestObject = *It;
				// Skip Blueprint preview objects (don't allow them to be selected for debugging)
				if (PreviewWorld != nullptr && TestObject->IsIn(PreviewWorld))
				{
					continue;
				}

				// check outer chain for pending kill objects
				bool bPendingKill = false;
				UObject* ObjOuter = TestObject;
				do
				{
					bPendingKill = !IsValid(ObjOuter);
					ObjOuter = ObjOuter->GetOuter();
				} while (!bPendingKill && ObjOuter != nullptr);

				if (!TestObject->HasAnyFlags(RF_ClassDefaultObject) && !bPendingKill)
				{
					ObjOuter = TestObject;
					UWorld *ObjWorld = nullptr;
					static bool bUseNewWorldCode = false;
					do		// Run through at least once in case the TestObject is a UGameInstance
					{
						UGameInstance *ObjGameInstance = Cast<UGameInstance>(ObjOuter);

						ObjOuter = ObjOuter->GetOuter();
						ObjWorld = ObjGameInstance ? ObjGameInstance->GetWorld() : Cast<UWorld>(ObjOuter);
					} while (ObjWorld == nullptr && ObjOuter != nullptr);

					if (ObjWorld)
					{
						// Make check on owning level (not streaming level)
						if (ObjWorld->PersistentLevel && ObjWorld->PersistentLevel->OwningWorld)
						{
							ObjWorld = ObjWorld->PersistentLevel->OwningWorld;
						}

						// We have a specific debug world and the object isn't in it
						if (DebugWorld && ObjWorld != DebugWorld)
						{
							continue;
						}

						if ((ObjWorld->WorldType == EWorldType::Editor) && (GUnrealEd->GetPIEViewport() == nullptr))
						{
							AddDebugObject(TestObject);
						}
						else if (ObjWorld->WorldType == EWorldType::PIE)
						{
							AddDebugObject(TestObject);
						}
					}
				}
			}
		}
		// this else doesn't make sense to me, since BlueprintClass doesn't exist, but we check it anyway
		// else
		// {
		// 	for (TObjectIterator<UObject> It; It; ++It)
		// 	{
		// 		UObject* TestObject = *It;
		//
		// 		// Skip Blueprint preview objects (don't allow them to be selected for debugging)
		// 		if (PreviewWorld != nullptr && TestObject->IsIn(PreviewWorld))
		// 		{
		// 			continue;
		// 		}
		//
		// 		const bool bPassesFlags = !TestObject->HasAnyFlags(RF_ClassDefaultObject) && IsValid(TestObject);
		// 		const bool bGeneratedByAnyBlueprint = TestObject->GetClass()->ClassGeneratedBy != nullptr;
		// 		const bool bGeneratedByThisBlueprint = bGeneratedByAnyBlueprint && GetBlueprintObj()->GetRigVMGeneratedClass() && TestObject->IsA(GetBlueprintObj()->GetRigVMGeneratedClass());
		//
		// 		if (bPassesFlags && bGeneratedByThisBlueprint)
		// 		{
		// 			UObject *ObjOuter = TestObject;
		// 			UWorld *ObjWorld = nullptr;
		// 			do		// Run through at least once in case the TestObject is a UGameInstance
		// 			{
		// 				UGameInstance *ObjGameInstance = Cast<UGameInstance>(ObjOuter);
		//
		// 				ObjOuter = ObjOuter->GetOuter();
		// 				ObjWorld = ObjGameInstance ? ObjGameInstance->GetWorld() : Cast<UWorld>(ObjOuter);
		// 			} while (ObjWorld == nullptr && ObjOuter != nullptr);
		//
		// 			if (ObjWorld)
		// 			{
		// 				// Make check on owning level (not streaming level)
		// 				if (ObjWorld->PersistentLevel && ObjWorld->PersistentLevel->OwningWorld)
		// 				{
		// 					ObjWorld = ObjWorld->PersistentLevel->OwningWorld;
		// 				}
		//
		// 				// We have a specific debug world and the object isn't in it
		// 				if (DebugWorld && ObjWorld != DebugWorld)
		// 				{
		// 					continue;
		// 				}
		//
		// 				if ((ObjWorld->WorldType == EWorldType::Editor) && (GUnrealEd->GetPIEViewport() == nullptr))
		// 				{
		// 					AddDebugObject(TestObject);
		// 				}
		// 				else if (ObjWorld->WorldType == EWorldType::PIE)
		// 				{
		// 					AddDebugObject(TestObject);
		// 				}
		// 			}
		// 		}
		// 	}
		// }
	}

	if (DebugObjectsComboBox.IsValid())
	{
		if (bRestoreSelection)
		{
			TSharedPtr<FRigVMDebugObjectInstance> NewSelection = GetDebugObjectInstance();
			if (NewSelection.IsValid() && !NewSelection->IsEmptyObject())
			{
				// If our new selection matches the actual debug object, set it
				DebugObjectsComboBox->SetSelectedItem(NewSelection);
			}
			else if (LastSelection.IsValid() && !LastSelection->IsEditorObject() && !LastSelection->IsEmptyObject())
			{
				// Re-add the desired runtime object if needed, even though it is currently null
				DebugObjects.Add(LastSelection);
				DebugObjectsComboBox->SetSelectedItem(LastSelection);
			}
		}

		// Finally ensure we have a valid selection, this will set to all objects as a backup
		TSharedPtr<FRigVMDebugObjectInstance> CurrentSelection = DebugObjectsComboBox->GetSelectedItem();
		if (DebugObjects.Find(CurrentSelection) == INDEX_NONE)
		{
			if (DebugObjects.Num() > 0)
			{
				DebugObjectsComboBox->SetSelectedItem(DebugObjects[0]);
			}
			else
			{
				DebugObjectsComboBox->ClearSelection();
			}
		}

		DebugObjectsComboBox->RefreshOptions();
	}
}

TSharedPtr<FRigVMDebugObjectInstance> SRigVMEditorSelectedDebugObjectWidget::GetDebugObjectInstance() const
{
	check(GetRigVMAssetInterface());
	const FString& PathToDebug = GetRigVMAssetInterface()->GetObjectBeingDebuggedPath();
	if (!PathToDebug.IsEmpty())
	{
		for (int32 ObjectIndex = 0; ObjectIndex < DebugObjects.Num(); ++ObjectIndex)
		{
			if (DebugObjects[ObjectIndex].IsValid() && PathToDebug.Equals(DebugObjects[ObjectIndex]->ObjectPath))
			{
				return DebugObjects[ObjectIndex];
			}
		}
	}

	if (DebugObjects.Num() > 0)
	{
		return DebugObjects[0];
	}
	
	return nullptr;
}

TSharedPtr<FString> SRigVMEditorSelectedDebugObjectWidget::GetDebugWorldName() const
{
	check(GetRigVMAssetInterface());
	if (ensure(DebugWorlds.Num() == DebugWorldNames.Num()))
	{
		UWorld* DebugWorld = GetRigVMAssetInterface()->GetWorldBeingDebugged();
		if (DebugWorld != nullptr)
		{
			for (int32 WorldIndex = 0; WorldIndex < DebugWorlds.Num(); ++WorldIndex)
			{
				if (DebugWorlds[WorldIndex].IsValid() && (DebugWorlds[WorldIndex].Get() == DebugWorld))
				{
					return DebugWorldNames[WorldIndex];
				}
			}
		}
	}


	if (DebugWorldNames.Num() > 0)
	{
		return DebugWorldNames[0];
	}

	return nullptr;
}

void SRigVMEditorSelectedDebugObjectWidget::DebugWorldSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (NewSelection != GetDebugWorldName())
	{
		check(DebugWorlds.Num() == DebugWorldNames.Num());
		for (int32 WorldIdx = 0; WorldIdx < DebugWorldNames.Num(); ++WorldIdx)
		{
			if (DebugWorldNames[WorldIdx] == NewSelection)
			{
				GetRigVMAssetInterface()->SetWorldBeingDebugged(DebugWorlds[WorldIdx].Get());

				GetRigVMAssetInterface()->SetObjectBeingDebugged(nullptr);
				LastObjectObserved.Reset();

				GenerateDebugObjectInstances(false);
				break;
			}
		}
	}
}

void SRigVMEditorSelectedDebugObjectWidget::DebugObjectSelectionChanged(TSharedPtr<FRigVMDebugObjectInstance> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (NewSelection != GetDebugObjectInstance() && NewSelection.IsValid())
	{
		UObject* DebugObj = NewSelection->ObjectPtr.Get();
		GetRigVMAssetInterface()->SetObjectBeingDebugged(DebugObj);

		if (TSharedPtr<IRigVMEditor> SharedBlueprintEditor = Editor.Pin())
		{
			SharedBlueprintEditor->ForceEditorRefresh();
		}
		
		LastObjectObserved = DebugObj;
	}
}

bool SRigVMEditorSelectedDebugObjectWidget::IsDebugObjectSelected() const
{
	check(GetRigVMAssetInterface());
	if (UObject* DebugObj = GetRigVMAssetInterface()->GetObjectBeingDebugged())
	{
		if (AActor* Actor = Cast<AActor>(DebugObj))
		{
			return true;
		}
	}
	return false;
}

void SRigVMEditorSelectedDebugObjectWidget::SelectedDebugObject_OnClicked()
{
	if (UObject* DebugObj = GetRigVMAssetInterface()->GetObjectBeingDebugged())
	{
		if (AActor* Actor = Cast<AActor>(DebugObj))
		{
			GEditor->SelectNone(false, true, false);
			GEditor->SelectActor(Actor, true, true, true);
			GUnrealEd->Exec(Actor->GetWorld(), TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY"));
		}
	}
}

EVisibility SRigVMEditorSelectedDebugObjectWidget::IsDebugWorldComboVisible() const
{
	if (GEditor->PlayWorld != nullptr)
	{
		int32 LocalWorldCount = 0;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE && Context.World() != nullptr)
			{
				++LocalWorldCount;
			}
		}

		if (LocalWorldCount > 1)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FString SRigVMEditorSelectedDebugObjectWidget::MakeDebugObjectLabel(UObject* TestObject, bool bAddContextIfSelectedInEditor, bool bAddSpawnedContext) const
{
	FString CustomLabelFromEditor = Editor.Pin()->GetCustomDebugObjectLabel(TestObject);
	if (!CustomLabelFromEditor.IsEmpty())
	{
		return CustomLabelFromEditor;
	}

	auto GetActorLabelStringLambda = [](AActor* InActor, bool bIncludeNetModeSuffix, bool bIncludeSelectedSuffix, bool bIncludeSpawnedContext)
	{
		FString Label = InActor->GetActorLabel();

		FString Context;

		if (bIncludeNetModeSuffix)
		{
			switch (InActor->GetNetMode())
			{
			case ENetMode::NM_Client:
			{
				Context = NSLOCTEXT("BlueprintEditor", "DebugWorldClient", "Client").ToString();

				FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(InActor->GetWorld());
				if (WorldContext != nullptr && WorldContext->PIEInstance > 1)
				{
					Context += TEXT(" ");
					Context += FText::AsNumber(WorldContext->PIEInstance - 1).ToString();
				}
			}
			break;

			case ENetMode::NM_ListenServer:
			case ENetMode::NM_DedicatedServer:
				Context = NSLOCTEXT("BlueprintEditor", "DebugWorldServer", "Server").ToString();
				break;
			}
		}

		if (bIncludeSpawnedContext)
		{
			if (!Context.IsEmpty())
			{
				Context += TEXT(", ");
			}

			Context += NSLOCTEXT("BlueprintEditor", "DebugObjectSpawned", "spawned").ToString();
		}

		if (bIncludeSelectedSuffix && InActor->IsSelected())
		{
			if (!Context.IsEmpty())
			{
				Context += TEXT(", ");
			}

			Context += NSLOCTEXT("BlueprintEditor", "DebugObjectSelected", "selected").ToString();
		}

		if (!Context.IsEmpty())
		{
			Label = FString::Printf(TEXT("%s (%s)"), *Label, *Context);
		}

		return Label;
	};

	// Include net mode suffix when "All worlds" is selected.
	const bool bIncludeNetModeSuffix = *GetDebugWorldName() == GetDebugAllWorldsString();

	FString Label;
	if (AActor* Actor = Cast<AActor>(TestObject))
	{
		Label = GetActorLabelStringLambda(Actor, bIncludeNetModeSuffix, bAddContextIfSelectedInEditor, bAddSpawnedContext);
	}
	else
	{
		if (AActor* ParentActor = TestObject->GetTypedOuter<AActor>())
		{
			// We don't need the full path because it's in the tooltip
			const FString RelativePath = TestObject->GetName();
			Label = FString::Printf(TEXT("%s in %s"), *RelativePath, *GetActorLabelStringLambda(ParentActor, bIncludeNetModeSuffix, bAddContextIfSelectedInEditor, bAddSpawnedContext));
		}
		else
		{
			Label = TestObject->GetName();
		}
	}

	return Label;
}

void SRigVMEditorSelectedDebugObjectWidget::FillDebugObjectInstance(TSharedPtr<FRigVMDebugObjectInstance> Instance)
{
	check(Instance.IsValid());
	FRigVMDebugObjectInstance& Ref = *Instance.Get();

	if (Ref.ObjectPtr.IsValid())
	{
		Ref.ObjectPath = Ref.ObjectPtr->GetPathName();

		// Compute non-PIE path
		FString OriginalPath = UWorld::RemovePIEPrefix(Ref.ObjectPath);

		// Look for original object
		UObject* OriginalObject = FindObjectSafe<UObject>(nullptr, *OriginalPath);

		if (OriginalObject)
		{
			Ref.EditorObjectPath = OriginalPath;
		}
		else
		{
			// No editor path, was dynamically spawned
			Ref.EditorObjectPath = FString();
		}
	}
	else
	{
		Ref.ObjectPath = Ref.EditorObjectPath = FString();
	}
}

void SRigVMEditorSelectedDebugObjectWidget::AddDebugObject(UObject* TestObject, const FString& TestObjectName)
{
	TSharedPtr<FRigVMDebugObjectInstance> NewInstance = MakeShareable(new FRigVMDebugObjectInstance(TestObject, TestObjectName));
	FillDebugObjectInstance(NewInstance);

	if (TestObjectName.IsEmpty())
	{
		NewInstance->ObjectLabel = MakeDebugObjectLabel(TestObject, true, NewInstance->IsSpawnedObject());
	}

	if (UWidget* DebugWidget = Cast<UWidget>(TestObject))
	{
		if (!DebugWidget->IsConstructed())
		{
			NewInstance->ObjectLabel += " (No Slate Widget)";
		}
	}
	
	DebugObjects.Add(NewInstance);
}

FRigVMEditorAssetInterfacePtr SRigVMEditorSelectedDebugObjectWidget::GetRigVMAssetInterface() const
{
	return Editor.Pin()->GetRigVMAssetInterface();
}

TSharedRef<SWidget> SRigVMEditorSelectedDebugObjectWidget::CreateDebugObjectItemWidget(TSharedPtr<FRigVMDebugObjectInstance> InItem)
{
	FString ItemString;
	FString ItemTooltip;

	if (InItem.IsValid())
	{
		ItemString = InItem->ObjectLabel;
		ItemTooltip = InItem->ObjectPath;
	}

	return SNew(STextBlock)
		.Text(FText::FromString(*ItemString))
		.ToolTipText(FText::FromString(*ItemTooltip));
}

FText SRigVMEditorSelectedDebugObjectWidget::GetSelectedDebugObjectTextLabel() const
{
	FString Label;

	TSharedPtr<FRigVMDebugObjectInstance> DebugInstance = GetDebugObjectInstance();
	if (DebugInstance.IsValid())
	{
		Label = DebugInstance->ObjectLabel;

		FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
		if (Blueprint != nullptr)
		{
			UObject* DebugObj = Blueprint->GetObjectBeingDebugged();
			if (DebugObj != nullptr)
			{
				// Exclude the editor selection suffix for the combo button's label.
				Label = MakeDebugObjectLabel(DebugObj, false, DebugInstance->IsSpawnedObject());
			}
		}
	}

	return FText::FromString(Label);
}

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
