// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMExecutionStackView.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Editor/RigVMNewEditor.h"
#include "Editor/RigVMExecutionStackCommands.h"
#include "RigVMHost.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "Widgets/Input/SSearchBox.h"
#include "Dialog/SCustomDialog.h"
#include "Async/TaskGraphInterfaces.h"

#define LOCTEXT_NAMESPACE "SRigVMExecutionStackView"

TAutoConsoleVariable<bool> CVarRigVMExecutionStackDetailedLabels(TEXT("RigVM.StackDetailedLabels"), false, TEXT("Set to true to turn on detailed labels for the execution stack widget"));

//////////////////////////////////////////////////////////////
/// FRigStackEntry
///////////////////////////////////////////////////////////
FRigStackEntry::FRigStackEntry(int32 InEntryIndex, ERigStackEntry::Type InEntryType, int32 InSubjectIndex, ERigVMOpCode InOpCode, const FString& InLabel, const FRigVMASTProxy& InProxy)
	: EntryIndex(InEntryIndex)
	, EntryType(InEntryType)
	, SubjectIndex(InSubjectIndex)
	, CallPath(InProxy.GetCallstack().GetCallPath())
	, Callstack(InProxy.GetCallstack())
	, OpCode(InOpCode)
	, Label(InLabel)
	, MicroSeconds(0.0)
{

}

TSharedRef<ITableRow> FRigStackEntry::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigStackEntry> InEntry,
	TSharedRef<FUICommandList> InCommandList, TSharedPtr<SRigVMExecutionStackView> InStackView, TWeakObjectPtr<UObject> InBlueprint)
{
	return SNew(SRigStackItem, InOwnerTable, InEntry, InCommandList, InBlueprint.Get());
}

TSharedRef<ITableRow> FRigStackEntry::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigStackEntry> InEntry, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SRigVMExecutionStackView> InStackView, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint)
{
	return SNew(SRigStackItem, InOwnerTable, InEntry, InCommandList, InBlueprint);
}

void FRigStackEntry::FilterChildren(const FString& InSearchText)
{
	if (FilteredChildren.IsSet())
	{
		return;
	}

	FilteredChildren = Children;
	FilteredChildren.GetValue().RemoveAll([InSearchText](const TSharedPtr<FRigStackEntry>& InEntry) -> bool
	{
		return !InEntry->MatchesFilter(InSearchText);
	});
}

bool FRigStackEntry::MatchesFilter(const FString& InSearchText)
{
	if (Label.Contains(InSearchText))
	{
		return true;
	}
	if (Children.IsEmpty())
	{
		return false;
	}
	
	FilterChildren(InSearchText);
	check(FilteredChildren.IsSet());
	return !FilteredChildren.GetValue().IsEmpty();
}

//////////////////////////////////////////////////////////////
/// SRigStackItem
///////////////////////////////////////////////////////////
void SRigStackItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FRigStackEntry> InStackEntry, TSharedRef<FUICommandList> InCommandList, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint)
{
	WeakStackEntry = InStackEntry;
	WeakBlueprint = InBlueprint;
	WeakCommandList = InCommandList;

	TSharedPtr< STextBlock > NumberWidget;
	TSharedPtr< STextBlock > TextWidget;

	const FSlateBrush* Icon = nullptr;
	switch (InStackEntry->EntryType)
	{
		case ERigStackEntry::Instruction:
		{
			Icon = FSlateIcon(TEXT("RigVMEditor"), "RigVM.Unit").GetIcon();
			break;
		}
		case ERigStackEntry::Info:
		{
			Icon = FAppStyle::GetBrush("Icons.Info");
			break;
		}
		case ERigStackEntry::Warning:
		{
			Icon = FAppStyle::GetBrush("Icons.Warning");
			break;
		}
		case ERigStackEntry::Error:
		{
			Icon = FAppStyle::GetBrush("Icons.Error");
			break;
		}
		case ERigStackEntry::Callable:
		{
			Icon = FAppStyle::GetBrush("Kismet.AllClasses.FunctionIcon");
			break;
		}
		default:
		{
			break;
		}
	}

	const EVisibility InstructionVisibility =
		(
			InStackEntry->EntryType == ERigStackEntry::Instruction
		) ? EVisibility::Visible : EVisibility::Collapsed;

	const EVisibility InstrumentationVisibility =
		(
			InStackEntry->EntryType == ERigStackEntry::Instruction ||
			InStackEntry->EntryType == ERigStackEntry::Callable
		) ? EVisibility::Visible : EVisibility::Collapsed;

	STableRow<TSharedPtr<FRigStackEntry>>::Construct(
		STableRow<TSharedPtr<FRigStackEntry>>::FArguments()
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.MaxWidth(35.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SAssignNew(NumberWidget, STextBlock)
					.Visibility(InstructionVisibility)
					.Text(this, &SRigStackItem::GetIndexText)
					.ColorAndOpacity(this, &SRigStackItem::GetLabelColorAndOpacity)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				+ SHorizontalBox::Slot()
				.MaxWidth(22.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(Icon)
				]
				
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SAssignNew(TextWidget, STextBlock)
					.Text(this, &SRigStackItem::GetLabelText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(this, &SRigStackItem::GetLabelColorAndOpacity)
					.ToolTipText(this, &SRigStackItem::GetTooltip)
					.Justification(ETextJustify::Left)
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
	            .HAlign(HAlign_Left)
	            [
	                SNew(STextBlock)
	                .Visibility(InstrumentationVisibility)
					.Text(this, &SRigStackItem::GetVisitedCountText)
					.ColorAndOpacity(this, &SRigStackItem::GetLabelColorAndOpacity)
	                .Font(IDetailLayoutBuilder::GetDetailFont())
	            ]
	            
	            + SHorizontalBox::Slot()
	            .Padding(20, 0, 0, 0)
				.AutoWidth()
	            .VAlign(VAlign_Center)
	            .HAlign(HAlign_Left)
	            [
	                SNew(STextBlock)
					.Visibility(InstrumentationVisibility)
	                .Text(this, &SRigStackItem::GetDurationText)
					.ColorAndOpacity(this, &SRigStackItem::GetLabelColorAndOpacity)
	                .Font(IDetailLayoutBuilder::GetDetailFont())
	            ]
	        ]
        ], OwnerTable);
}

FText SRigStackItem::GetIndexText() const
{
	const FString IndexStr = FString::FromInt(WeakStackEntry.Pin()->SubjectIndex) + TEXT(".");
	return FText::FromString(IndexStr);
}

FText SRigStackItem::GetLabelText() const
{
	return (FText::FromString(WeakStackEntry.Pin()->Label));
}

FSlateColor SRigStackItem::GetLabelColorAndOpacity() const
{
	if (TSharedPtr<FRigStackEntry> StackEntry = WeakStackEntry.Pin())
	{
		if(StackEntry->ShowAsFadedOut.Get(false))
		{
			return FSlateColor::UseSubduedForeground();
		}
	}
	return FSlateColor::UseForeground();
}

FText SRigStackItem::GetTooltip() const
{
	return FText::FromString(WeakStackEntry.Pin()->Callstack.GetCallPath(true));
}

FText SRigStackItem::GetVisitedCountText() const
{
	if (TSharedPtr<FRigStackEntry> StackEntry = WeakStackEntry.Pin())
	{
		return StackEntry->VisitedCountText.Get(FText());
	}
	return FText();
}

FText SRigStackItem::GetDurationText() const
{
	if (TSharedPtr<FRigStackEntry> StackEntry = WeakStackEntry.Pin())
	{
		return StackEntry->DurationText.Get(FText());
	}
	return FText();
}

//////////////////////////////////////////////////////////////
/// SRigVMExecutionStackView
///////////////////////////////////////////////////////////

SRigVMExecutionStackView::~SRigVMExecutionStackView()
{
	UnbindHostDelegates();
	UnbindAssetDelegates();
}

void SRigVMExecutionStackView::Construct( const FArguments& InArgs, TSharedRef<IRigVMEditor> InRigVMEditor)
{
	WeakEditor = InRigVMEditor;
	WeakRigVMBlueprint = InRigVMEditor->GetRigVMAssetInterface();
	CommandList = MakeShared<FUICommandList>();
	bSuspendModelNotifications = false;
	bSuspendControllerSelection = false;

	BindCommands();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SNew(SHorizontalBox)
					//.Visibility(this, &SRigHierarchy::IsSearchbarVisible)
					+SHorizontalBox::Slot()
					.AutoWidth()
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(3.0f, 1.0f)
					[
						SAssignNew(FilterBox, SSearchBox)
						.OnTextChanged(this, &SRigVMExecutionStackView::OnFilterTextChanged)
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
			[
				SAssignNew(TreeView, SRigVMExecutionStackTreeView)
				.TreeItemsSource(&FilteredRootEntries)
				.SelectionMode(ESelectionMode::Multi)
				.OnGenerateRow(this, &SRigVMExecutionStackView::MakeTableRowWidget, WeakRigVMBlueprint)
				.OnGetChildren(this, &SRigVMExecutionStackView::HandleGetChildrenForTree)
				.OnSelectionChanged(this, &SRigVMExecutionStackView::OnSelectionChanged)
				.OnContextMenuOpening(this, &SRigVMExecutionStackView::CreateContextMenu)
				.OnMouseButtonDoubleClick(this, &SRigVMExecutionStackView::HandleItemMouseDoubleClick)
			]
		]
	];

	RefreshTreeView(nullptr, nullptr, true);

	if (TStrongObjectPtr<UObject> StrongPtr = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
	{
		IRigVMEditorAssetInterface* RigVMBlueprint = WeakRigVMBlueprint.Get();
		if (OnVMCompiledHandle.IsValid())
		{
			RigVMBlueprint->OnVMCompiled().Remove(OnVMCompiledHandle);
		}
		if (OnModelModified.IsValid())
		{
			RigVMBlueprint->OnModified().Remove(OnModelModified);
		}
		OnVMCompiledHandle = RigVMBlueprint->OnVMCompiled().AddSP(this, &SRigVMExecutionStackView::OnVMCompiled);
		OnObjectBeingDebuggedChangedHandle = WeakRigVMBlueprint->OnSetObjectBeingDebugged().AddSP(this, &SRigVMExecutionStackView::OnObjectBeingDebuggedChanged);

		OnModelModified = RigVMBlueprint->OnModified().AddSP(this, &SRigVMExecutionStackView::HandleModifiedEvent);

		BindHostDelegates();
		
		URigVMHost* RigVMHost = PreviouslyDebuggedHost.Get();
		OnObjectBeingDebuggedChanged(RigVMHost);
	}

	if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
	{
		OnPreviewHostUpdatedHandle = Editor->OnPreviewHostUpdated().AddSP(this, &SRigVMExecutionStackView::HandlePreviewHostUpdated);
	}
}

void SRigVMExecutionStackView::OnSelectionChanged(TSharedPtr<FRigStackEntry> Selection, ESelectInfo::Type SelectInfo)
{
	if (bSuspendModelNotifications || bSuspendControllerSelection)
	{
		return;
	}
	TGuardValue<bool> SuspendNotifs(bSuspendModelNotifications, true);

	TArray<TSharedPtr<FRigStackEntry>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		if (TStrongObjectPtr<UObject> StrongPtr = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
		{
			IRigVMEditorAssetInterface* RigVMBlueprint = WeakRigVMBlueprint.Get();
			URigVMHost* Host = PreviouslyDebuggedHost.Get();
			if (Host == nullptr || Host->GetVM() == nullptr)
			{
				return;
			}

			const FRigVMByteCode& ByteCode = Host->GetVM()->GetByteCode();

			TMap<URigVMGraph*, TArray<FName>> SelectedNodesPerGraph;
			for (TSharedPtr<FRigStackEntry>& Entry : SelectedItems)
			{
				for(int32 StackIndex = 0; StackIndex < Entry->Callstack.Num(); StackIndex++)
				{
					const UObject* Subject = Entry->Callstack[StackIndex];
					URigVMGraph* SubjectGraph = nullptr;

					FName NodeName = NAME_None;
					if (const URigVMNode* Node = Cast<URigVMNode>(Subject))
					{
						NodeName = Node->GetFName();
						SubjectGraph = Node->GetGraph();
					}
					else if (const URigVMPin* Pin = Cast<URigVMPin>(Subject))
					{
						NodeName = Pin->GetNode()->GetFName();
						SubjectGraph = Pin->GetGraph();
					}

					if (NodeName.IsNone() || SubjectGraph == nullptr)
					{
						continue;
					}

					SelectedNodesPerGraph.FindOrAdd(SubjectGraph).AddUnique(NodeName);
				}
			}

			for (const TPair< URigVMGraph*, TArray<FName> >& Pair : SelectedNodesPerGraph)
			{
				RigVMBlueprint->GetOrCreateController(Pair.Key)->SetNodeSelection(Pair.Value);
			}
		}
	}

	UpdateTargetItemHighlighting();
}

FReply SRigVMExecutionStackView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FInputChord KeyEventAsInputChord = FInputChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SRigVMExecutionStackView::BindCommands()
{
	// create new command
	const FRigVMExecutionStackCommands& Commands = FRigVMExecutionStackCommands::Get();
	CommandList->MapAction(Commands.FocusOnSelection, FExecuteAction::CreateSP(this, &SRigVMExecutionStackView::HandleFocusOnSelectedGraphNode));
	CommandList->MapAction(Commands.GoToInstruction, FExecuteAction::CreateSP(this, &SRigVMExecutionStackView::HandleGoToInstruction));
	CommandList->MapAction(Commands.SelectTargetInstructions, FExecuteAction::CreateSP(this, &SRigVMExecutionStackView::HandleSelectTargetInstructions));
	CommandList->MapAction(Commands.ToggleEarlyExitInstruction, FExecuteAction::CreateSP(this, &SRigVMExecutionStackView::HandleToggleEarlyExitInstruction));
	CommandList->MapAction(Commands.StepEarlyExitInstruction, FExecuteAction::CreateSP(this, &SRigVMExecutionStackView::HandleStepEarlyExitInstruction));
}

TSharedRef<ITableRow> SRigVMExecutionStackView::MakeTableRowWidget(TSharedPtr<FRigStackEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint)
{
	return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), CommandList.ToSharedRef(), SharedThis(this), InBlueprint);
}

void SRigVMExecutionStackView::HandleGetChildrenForTree(TSharedPtr<FRigStackEntry> InItem, TArray<TSharedPtr<FRigStackEntry>>& OutChildren)
{
	if (InItem->FilteredChildren.IsSet())
	{
		OutChildren = InItem->FilteredChildren.GetValue();
	}
	else
	{
		OutChildren = InItem->Children;
	}
}

void SRigVMExecutionStackView::PopulateStackView(URigVM* InVM, FRigVMExtendedExecuteContext* InVMContext)
{
	if (InVM && InVMContext)
	{
		const FRigVMInstructionArray Instructions = InVM->GetInstructions();
		if (Instructions.Num() == 0)
		{
			return;
		}
		
		const FRigVMByteCode& ByteCode = InVM->GetByteCode();

		TArray<int32> CallableIndexPerInstruction;
		CallableIndexPerInstruction.AddZeroed(Instructions.Num());
		for (int32& CallableIndex : CallableIndexPerInstruction)
		{
			CallableIndex = INDEX_NONE;
		}

		TArray<int32> BlockIndexPerInstruction;
		BlockIndexPerInstruction.AddZeroed(Instructions.Num());
		for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
		{
			BlockIndexPerInstruction[InstructionIndex] = INDEX_NONE;
		}
		for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
		{
			if (Instructions[InstructionIndex].OpCode == ERigVMOpCode::RunInstructions)
			{
				const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instructions[InstructionIndex]);
				if (Op.EndInstruction < Op.StartInstruction)
				{
					continue;
				}
				const TTuple<int32,int32> Block = {Op.StartInstruction, FMath::Max(Op.StartInstruction, Op.EndInstruction)};
				int32 BlockIndex = Blocks.AddUnique(Block);
				for (int32 ChildInstructionIndex = Block.Get<0>(); ChildInstructionIndex <= Block.Get<1>(); ChildInstructionIndex++)
				{
					if (BlockIndexPerInstruction[ChildInstructionIndex] == INDEX_NONE)
					{
						BlockIndexPerInstruction[ChildInstructionIndex] = BlockIndex;
					}
					else
					{
						const TTuple<int32,int32> OldBlock = Blocks[BlockIndexPerInstruction[ChildInstructionIndex]];
						const int32 NewLen = Block.Get<1>() - Block.Get<0>(); 
						const int32 OldLen = OldBlock.Get<1>() - OldBlock.Get<0>();
						if (NewLen < OldLen)
						{
							BlockIndexPerInstruction[ChildInstructionIndex] = BlockIndex;
						}
					}
				}
			}
		}

		for (int32 CallableIndex = 0; CallableIndex < ByteCode.NumCallables(); CallableIndex++)
		{
			const FRigVMCallableInfo* Callable = ByteCode.GetCallable(CallableIndex);

			FRigVMASTProxy Proxy;
			FString CallableLabel = Callable->Name.ToString();
			int32 LastIndex = INDEX_NONE;
			if (CallableLabel.FindLastChar(TEXT(':'), LastIndex))
			{
				CallableLabel = CallableLabel.Mid(LastIndex + 1);
			}
			TSharedPtr<FRigStackEntry> NewEntry = MakeEntry(ERigStackEntry::Callable, CallableIndex, ERigVMOpCode::Invalid, CallableLabel, Proxy);
			RootEntries.Add(NewEntry);
			CallableToEntry.Add(CallableIndex, NewEntry);

			for (int32 InstructionIndex = Callable->FirstInstruction; InstructionIndex <= Callable->LastInstruction; InstructionIndex++)
			{
				check(CallableIndexPerInstruction.IsValidIndex(InstructionIndex));
				CallableIndexPerInstruction[InstructionIndex] = CallableIndex;
			}
		}

		for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); BlockIndex++)
		{
			const TTuple<int32,int32> Block = Blocks[BlockIndex];
			const FString BlockLabel = FString::Printf(TEXT("Block %04d - %04d"), Block.Get<0>(), Block.Get<1>());

			FRigVMASTProxy Proxy;
			TSharedPtr<FRigStackEntry> NewEntry = MakeEntry(ERigStackEntry::Block, BlockIndex, ERigVMOpCode::Invalid, BlockLabel, Proxy);
			BlockToEntry.Add(Block, NewEntry);
		}

		TArray<URigVMGraph*> RootGraphs;
		RootGraphs.AddZeroed(Instructions.Num());

		const bool bUseSimpleLabels = !CVarRigVMExecutionStackDetailedLabels.GetValueOnAnyThread();
		if(bUseSimpleLabels)
		{
			for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
			{
				URigVMNode* Node = Cast<URigVMNode>(ByteCode.GetSubjectForInstruction(InstructionIndex));

				FString DisplayName;

				if(Node)
				{
					DisplayName = Node->GetName();
					RootGraphs[InstructionIndex] = Node->GetRootGraph();
					
					// only unit nodes among all nodes has StaticExecute() that generates actual instructions
					if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
					{
						DisplayName = UnitNode->GetNodeTitle();
	#if WITH_EDITOR
						UScriptStruct* Struct = UnitNode->GetScriptStruct();
						FString MenuDescSuffixMetadata;
						if (Struct)
						{
							Struct->GetStringMetaDataHierarchical(FRigVMStruct::MenuDescSuffixMetaName, &MenuDescSuffixMetadata);
						}
						if (!MenuDescSuffixMetadata.IsEmpty())
						{
							DisplayName = FString::Printf(TEXT("%s %s"), *UnitNode->GetNodeTitle(), *MenuDescSuffixMetadata);
						}

						if(UnitNode->IsEvent())
						{
							DisplayName = Node->GetEventName().ToString();
							if(!DisplayName.EndsWith(TEXT("Event")))
							{
								DisplayName += TEXT(" Event");
							}
						}
	#endif
					}
				}

				FString Label;

				switch (Instructions[InstructionIndex].OpCode)
				{
					case ERigVMOpCode::Execute:
					{
						const FRigVMExecuteOp& Op = ByteCode.GetOpAt<FRigVMExecuteOp>(Instructions[InstructionIndex]);

						if(Node)
						{
							Label = DisplayName;
						}
						else
						{
							Label = InVM->GetRigVMFunctionName(Op.CallableIndex);
						}
						break;
					}
					case ERigVMOpCode::InvokeCallable:
					{
						const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instructions[InstructionIndex]);
						if (const FRigVMCallableInfo* CallableInfo = ByteCode.GetCallable(Op.CallableIndex))
						{
							FString CallableLabel = CallableInfo->Name.ToString();
							int32 LastIndex = INDEX_NONE;
							if (CallableLabel.FindLastChar(TEXT(':'), LastIndex))
							{
								CallableLabel = CallableLabel.Mid(LastIndex + 1);
							}
							Label = FString::Printf(TEXT("Invoke %s"), *CallableLabel);
						}
						break;
					}
					case ERigVMOpCode::Copy:
					case ERigVMOpCode::Zero:
					case ERigVMOpCode::BoolFalse:
					case ERigVMOpCode::BoolTrue:
					case ERigVMOpCode::Increment:
					case ERigVMOpCode::Decrement:
					case ERigVMOpCode::Equals:
					case ERigVMOpCode::NotEquals:
					case ERigVMOpCode::BeginBlock:
					case ERigVMOpCode::EndBlock:
					case ERigVMOpCode::ChangeType:
					case ERigVMOpCode::ArrayReset:
					case ERigVMOpCode::ArrayGetNum: 
					case ERigVMOpCode::ArraySetNum:
					case ERigVMOpCode::ArrayGetAtIndex:  
					case ERigVMOpCode::ArraySetAtIndex:
					case ERigVMOpCode::ArrayAdd:
					case ERigVMOpCode::ArrayInsert:
					case ERigVMOpCode::ArrayRemove:
					case ERigVMOpCode::ArrayFind:
					case ERigVMOpCode::ArrayAppend:
					case ERigVMOpCode::ArrayClone:
					case ERigVMOpCode::ArrayIterator:
					case ERigVMOpCode::ArrayUnion:
					case ERigVMOpCode::ArrayDifference:
					case ERigVMOpCode::ArrayIntersection:
					case ERigVMOpCode::ArrayReverse:
					{
						const FText OpCodeText = StaticEnum<ERigVMOpCode>()->GetDisplayNameTextByValue((int32)Instructions[InstructionIndex].OpCode);
						if(Node)
						{
							Label = DisplayName + TEXT(" - ") + OpCodeText.ToString();
						}
						else
						{
							Label = OpCodeText.ToString();
						}
						break;
					}
					case ERigVMOpCode::InvokeEntry:
					{
						const FRigVMInvokeEntryOp& Op = ByteCode.GetOpAt<FRigVMInvokeEntryOp>(Instructions[InstructionIndex]);
						Label = FString::Printf(TEXT("Run %s Event"), *Op.EntryName.ToString());
						break;
					}
					case ERigVMOpCode::JumpAbsolute:
					case ERigVMOpCode::JumpForward:
					case ERigVMOpCode::JumpBackward:
					{
						const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instructions[InstructionIndex]);
						Label = FString::Printf(TEXT("Jump to %d"), FRigVMJumpOp::GetTargetInstruction(Op.OpCode, InstructionIndex, Op.InstructionIndex));
						break;
					}
					case ERigVMOpCode::JumpAbsoluteIf:
					case ERigVMOpCode::JumpForwardIf:
					case ERigVMOpCode::JumpBackwardIf:
					{
						const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instructions[InstructionIndex]);
						Label = FString::Printf(TEXT("Maybe jump to %d"), FRigVMJumpOp::GetTargetInstruction(Op.OpCode, InstructionIndex, Op.InstructionIndex));
						break;
					}
					case ERigVMOpCode::JumpToBranch:
					{
						const FRigVMJumpToBranchOp& Op = ByteCode.GetOpAt<FRigVMJumpToBranchOp>(Instructions[InstructionIndex]);
						Label = TEXT("Jump To Branch");
						break;
					}
					case ERigVMOpCode::RunInstructions:
					{
						const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instructions[InstructionIndex]);
						Label = FString::Printf(TEXT("Run Block %d-%d"), Op.StartInstruction, FMath::Max(Op.StartInstruction, Op.EndInstruction));
						break;
					}
					case ERigVMOpCode::SetupTraits:
					{
						Label = TEXT("Setup Traits");
						break;
					}
					case ERigVMOpCode::Exit:
					{
						Label = TEXT("Exit");
						break;
					}

					default:
					{
						ensure(false);
						break;
					}
				}
				
				// add the entry with the new label to the stack view
				const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
				FRigVMASTProxy Proxy;
				if(const TArray<TWeakObjectPtr<UObject>>* Callstack = ByteCode.GetCallstackForInstruction(InstructionIndex))
				{
					Proxy = FRigVMASTProxy::MakeFromCallstack(Callstack);
				}
				TSharedPtr<FRigStackEntry> NewEntry = MakeEntry(ERigStackEntry::Instruction, InstructionIndex, Instruction.OpCode, Label, Proxy);
				InstructionToEntry.Add(InstructionIndex, NewEntry);
			}
		}
		else
		{
			// 1. cache information about instructions/nodes, which will be used later 
			TMap<FString, FString> OperandFormatMap;
			for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
			{
				const FRigVMASTProxy Proxy = FRigVMASTProxy::MakeFromCallPath(ByteCode.GetCallPathForInstruction(InstructionIndex), RootGraphs[InstructionIndex]);
				if(URigVMNode* Node = Proxy.GetSubject<URigVMNode>())
				{
					FString DisplayName = Node->GetName();

					// only unit nodes among all nodes has StaticExecute() that generates actual instructions
					if (URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(Node))
					{
						DisplayName = UnitNode->GetNodeTitle();
#if WITH_EDITOR
						UScriptStruct* Struct = UnitNode->GetScriptStruct();
						FString MenuDescSuffixMetadata;
						if (Struct)
						{
							Struct->GetStringMetaDataHierarchical(FRigVMStruct::MenuDescSuffixMetaName, &MenuDescSuffixMetadata);
						}
						if (!MenuDescSuffixMetadata.IsEmpty())
						{
							DisplayName = FString::Printf(TEXT("%s %s"), *UnitNode->GetNodeTitle(), *MenuDescSuffixMetadata);
						}
#endif
					}

					// this is needed for name replacement later
					OperandFormatMap.Add(Node->GetName(), DisplayName);
				}
			}

			// 2. replace raw operand names with NodeTitle.PinName/PropertyName.OffsetName
			TArray<FString> Labels = InVM->DumpByteCodeAsTextArray(*InVMContext, TArray<int32>(), false, [OperandFormatMap](const FString& RegisterName, const FString& RegisterOffsetName)
			{
				FString NewRegisterName = RegisterName;
				FString NodeName;
				FString PinName;
				if (RegisterName.Split(TEXT("."), &NodeName, &PinName))
				{
					const FString* NodeTitle = OperandFormatMap.Find(NodeName);
					NewRegisterName = FString::Printf(TEXT("%s.%s"), NodeTitle ? **NodeTitle : *NodeName, *PinName);
				}
				FString OperandLabel;
				OperandLabel = NewRegisterName;
				if (!RegisterOffsetName.IsEmpty())
				{
					OperandLabel = FString::Printf(TEXT("%s.%s"), *OperandLabel, *RegisterOffsetName);
				}
				return OperandLabel;
			});

			if (!ensure(Labels.Num() == Instructions.Num()))
			{
				return;
			}
		
			// 3. replace instruction names with node titles
			for (int32 InstructionIndex = 0; InstructionIndex < Labels.Num(); InstructionIndex++)
			{
				FString Label = Labels[InstructionIndex];
				const FRigVMASTProxy Proxy = FRigVMASTProxy::MakeFromCallPath(ByteCode.GetCallPathForInstruction(InstructionIndex), RootGraphs[InstructionIndex]);

				if(URigVMNode* Node = Proxy.GetSubject<URigVMNode>())
				{
					FString Suffix;
					switch(Instructions[InstructionIndex].OpCode)
					{
						case ERigVMOpCode::Copy:
						case ERigVMOpCode::Zero:
						case ERigVMOpCode::BoolFalse:
						case ERigVMOpCode::BoolTrue:
						case ERigVMOpCode::Increment:
						case ERigVMOpCode::Decrement:
						case ERigVMOpCode::Equals:
						case ERigVMOpCode::NotEquals:
						case ERigVMOpCode::JumpAbsolute:
						case ERigVMOpCode::JumpForward:
						case ERigVMOpCode::JumpBackward:
						case ERigVMOpCode::JumpAbsoluteIf:
						case ERigVMOpCode::JumpForwardIf:
						case ERigVMOpCode::JumpBackwardIf:
						case ERigVMOpCode::BeginBlock:
						case ERigVMOpCode::EndBlock:
						{
							const FText OpCodeText = StaticEnum<ERigVMOpCode>()->GetDisplayNameTextByValue((int32)Instructions[InstructionIndex].OpCode);
							Suffix = FString::Printf(TEXT(" - %s"), *OpCodeText.ToString());
							break;
						}
						default:
						{
							break;
						}
					}
					
					Label =  OperandFormatMap[Node->GetName()] + Suffix;
				}

				// add the entry with the new label to the stack view
				const FRigVMInstruction& Instruction = Instructions[InstructionIndex];
				TSharedPtr<FRigStackEntry> NewEntry = MakeEntry(ERigStackEntry::Instruction, InstructionIndex, Instruction.OpCode, Label, Proxy);
				InstructionToEntry.Add(InstructionIndex, NewEntry);
			}
		}

		for (int32 InstructionIndex = 0; InstructionIndex < Instructions.Num(); InstructionIndex++)
		{
			TSharedPtr<FRigStackEntry>& NewEntry = InstructionToEntry.FindChecked(InstructionIndex); 
			if (BlockIndexPerInstruction[InstructionIndex] != INDEX_NONE)
			{
				BlockToEntry.FindChecked(Blocks[BlockIndexPerInstruction[InstructionIndex]])->Children.Add(NewEntry);
			}
			else if (CallableIndexPerInstruction[InstructionIndex] != INDEX_NONE)
			{
				CallableToEntry.FindChecked(CallableIndexPerInstruction[InstructionIndex])->Children.Add(NewEntry);
			}
			else
			{
				RootEntries.Add(NewEntry);
			}
		}

		TArray<TTuple<int32, int32>> SortedBlocks = Blocks;
		Algo::Sort(SortedBlocks, [](const TTuple<int32,int32>& A, const TTuple<int32,int32>& B) -> bool
		{
			const int32 LengthA = FMath::Max(A.Get<0>(), A.Get<1>()) - A.Get<0>();
			const int32 LengthB = FMath::Max(B.Get<0>(), B.Get<1>()) - B.Get<0>();
			if (LengthA < LengthB)
			{
				return true;
			}
			if (LengthA > LengthB)
			{
				return false;
			}
			if (A.Get<0>() == B.Get<0>())
			{
				return A.Get<1>() < B.Get<1>();
			}
			return A.Get<0>() < B.Get<0>();
		});

		for (int32 BlockIndexA = 0; BlockIndexA < SortedBlocks.Num(); BlockIndexA++)
		{
			const TTuple<int32, int32>& BlockA = SortedBlocks[BlockIndexA];
			TTuple<int32, int32> ParentBlock = {INDEX_NONE, INDEX_NONE};

			for (int32 BlockIndexB = BlockIndexA + 1; BlockIndexB < SortedBlocks.Num(); BlockIndexB++)
			{
				const TTuple<int32, int32>& BlockB = SortedBlocks[BlockIndexB];
				if (BlockB.Get<0>() <= BlockA.Get<0>() &&
					FMath::Max(BlockB.Get<0>(), BlockB.Get<1>()) >= FMath::Max(BlockA.Get<0>(), BlockA.Get<1>()))
				{
					BlockToEntry.FindChecked(BlockB)->Children.Add(BlockToEntry.FindChecked(BlockA));
					break;
				}
			}

			TSharedPtr<FRigStackEntry> NewEntry = BlockToEntry.FindChecked(BlockA);
			if (ParentBlock.Get<0>() != INDEX_NONE)
			{
				BlockToEntry.FindChecked(ParentBlock)->Children.Add(NewEntry);
			}
			else if (CallableIndexPerInstruction[BlockA.Get<0>()] != INDEX_NONE)
			{
				CallableToEntry.FindChecked(CallableIndexPerInstruction[BlockA.Get<0>()])->Children.Add(NewEntry);
			}
			else
			{
				RootEntries.Add(NewEntry);
			}
		}

		// sort all children again.
		for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
		{
			if (Entry->Children.IsEmpty())
			{
				continue;
			}

			Algo::Sort(Entry->Children, [this](const TSharedPtr<FRigStackEntry>& A, const TSharedPtr<FRigStackEntry>& B) -> bool
			{
				const ERigStackEntry::Type TypeA = A->EntryType;
				const ERigStackEntry::Type TypeB = B->EntryType;

				if (TypeA == ERigStackEntry::Instruction && TypeB == ERigStackEntry::Block)
				{
					const int32 InstructionIndexA = A->SubjectIndex;
					const TTuple<int32,int32>& BlockB = Blocks[B->SubjectIndex];
					if (InstructionIndexA < BlockB.Get<0>())
					{
						return true;
					}
					if (InstructionIndexA > FMath::Max(BlockB.Get<0>(), BlockB.Get<1>()))
					{
						return false;
					}
					// we should never get here since then the instruction should be inside the block
					checkNoEntry();
				}

				if (TypeA == ERigStackEntry::Block && TypeB == ERigStackEntry::Instruction)
				{
					const TTuple<int32,int32>& BlockA = Blocks[A->SubjectIndex];
					const int32 InstructionIndexB = B->SubjectIndex;
					if (InstructionIndexB < BlockA.Get<0>())
					{
						return false;
					}
					if (InstructionIndexB > FMath::Max(BlockA.Get<0>(), BlockA.Get<1>()))
					{
						return true;
					}
					// we should never get here since then the instruction should be inside the block
					checkNoEntry();
				}

				if (TypeA != TypeB)
				{
					return TypeA < TypeB;
				}

				if (TypeA == ERigStackEntry::Block)
				{
					const TTuple<int32,int32>& BlockA = Blocks[A->SubjectIndex];
					const TTuple<int32,int32>& BlockB = Blocks[B->SubjectIndex];
					return BlockA.Get<0>() < BlockB.Get<0>();
				}
					
				if (TypeA == ERigStackEntry::Error || TypeA == ERigStackEntry::Warning || TypeA == ERigStackEntry::Info)
				{
					return FCString::Strcmp(*(A->Label), *(B->Label)) < 0;
				}

				return A->SubjectIndex < B->SubjectIndex;
			});
		}
	}
}

void SRigVMExecutionStackView::BindHostDelegates()
{
	UnbindHostDelegates();
	
	if (TStrongObjectPtr<UObject> Blueprint = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
	{
		BindHostDelegates(WeakRigVMBlueprint->GetObjectBeingDebugged());
	}
}

void SRigVMExecutionStackView::BindHostDelegates(UObject* InObjectBeingDebugged)
{
	UnbindHostDelegates();

	if (URigVMHost* DebuggedHost = Cast<URigVMHost>(InObjectBeingDebugged))
	{
		PreviouslyDebuggedHost = DebuggedHost;
		OnHostInitializedHandle = DebuggedHost->OnInitialized_AnyThread().AddSP(this, &SRigVMExecutionStackView::HandleHostInitializedEvent);
		OnHostExecutedHandle = DebuggedHost->OnExecuted_AnyThread().AddSP(this, &SRigVMExecutionStackView::HandleHostExecutedEvent);
	}
}

void SRigVMExecutionStackView::UnbindHostDelegates()
{
	if (URigVMHost* Host = PreviouslyDebuggedHost.Get())
	{
		if (OnHostInitializedHandle.IsValid())
		{
			Host->OnInitialized_AnyThread().Remove(OnHostInitializedHandle);
		}
		if (OnHostExecutedHandle.IsValid())
		{
			Host->OnExecuted_AnyThread().Remove(OnHostExecutedHandle);
		}
	}
	OnHostInitializedHandle.Reset();
	OnHostExecutedHandle.Reset();
	PreviouslyDebuggedHost.Reset();
}

void SRigVMExecutionStackView::UnbindAssetDelegates()
{
	if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
	{
		if (OnModelModified.IsValid() && Editor->GetRigVMAssetInterface())
		{
			Editor->GetRigVMAssetInterface()->OnModified().Remove(OnModelModified);
		}
	}
	if (TStrongObjectPtr<UObject> Blueprint = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
	{
		if (OnVMCompiledHandle.IsValid())
		{
			WeakRigVMBlueprint->OnVMCompiled().Remove(OnVMCompiledHandle);
		}
		if (OnObjectBeingDebuggedChangedHandle.IsValid())
		{
			WeakRigVMBlueprint->OnSetObjectBeingDebugged().Remove(OnObjectBeingDebuggedChangedHandle);
		}
	}
}

TSharedPtr<FRigStackEntry> SRigVMExecutionStackView::MakeEntry(ERigStackEntry::Type InEntryType,
	int32 InSubjectIndex, ERigVMOpCode InOpCode, const FString& InLabel, const FRigVMASTProxy& InProxy)
{
	TSharedPtr<FRigStackEntry> Entry = MakeShared<FRigStackEntry>(INDEX_NONE, InEntryType, InSubjectIndex, InOpCode, InLabel, InProxy);
	Entry->EntryIndex = AllEntries.Add(Entry);
	return Entry;
}

void SRigVMExecutionStackView::RefreshTreeView(URigVM* InVM, FRigVMExtendedExecuteContext* InVMContext, bool bRebuildEntries)
{
	TreeView->SaveAndClearSparseItemInfos();

	if (bRebuildEntries)
	{
		RootEntries.Reset();
		FilteredRootEntries.Reset();
		AllEntries.Reset();
		InstructionToEntry.Reset();
		CallableToEntry.Reset();
		Blocks.Reset();
		BlockToEntry.Reset();

		// populate the stack with node names/instruction names
		PopulateStackView(InVM, InVMContext);
		
		if (InVM && InVMContext)
		{
			// fill the children from the log
			if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
			{
				URigVMHost* Host = PreviouslyDebuggedHost.Get();
				if(Host && Host->GetLog())
				{
					const TArray<FRigVMLog::FLogEntry>& LogEntries = Host->GetLog()->Entries;
					for (const FRigVMLog::FLogEntry& LogEntry : LogEntries)
					{
						TSharedPtr<FRigStackEntry>* StackEntryPtr = InstructionToEntry.Find(LogEntry.InstructionIndex);
						if (!StackEntryPtr)
						{
							continue;
						}
						TSharedPtr<FRigStackEntry>& StackEntry = *StackEntryPtr;
						check(StackEntry.IsValid());
						
						int32 ChildIndex = StackEntry->Children.Num();
						switch (LogEntry.Severity)
						{
							case EMessageSeverity::Info:
							{
								StackEntry->Children.Add(MakeEntry(ERigStackEntry::Info, LogEntry.InstructionIndex, ERigVMOpCode::Invalid, LogEntry.Message, FRigVMASTProxy()));
								break;
							}
							case EMessageSeverity::Warning:
							case EMessageSeverity::PerformanceWarning:
							{
								StackEntry->Children.Add(MakeEntry(ERigStackEntry::Warning, LogEntry.InstructionIndex, ERigVMOpCode::Invalid, LogEntry.Message, FRigVMASTProxy()));
								break;
							}
							case EMessageSeverity::Error:
							{
								StackEntry->Children.Add(MakeEntry(ERigStackEntry::Error, LogEntry.InstructionIndex, ERigVMOpCode::Invalid, LogEntry.Message, FRigVMASTProxy()));
								break;
							}
							default:
							{
								break;
							}
						}
					}
				}
			}
		}
	}

	FilteredRootEntries = RootEntries;
	for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
	{
		Entry->FilteredChildren.Reset();
	}
	
	if (!FilterText.IsEmpty())
	{
		const FString FilterTextString = FilterText.ToString();
		FilteredRootEntries.RemoveAll([FilterTextString](TSharedPtr<FRigStackEntry>& Entry) -> bool
		{
			return !Entry->MatchesFilter(FilterTextString);
		});
	}

	for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
	{
		TreeView->RestoreSparseItemInfos(Entry);
	}
	
	TreeView->RequestTreeRefresh();
}

TSharedPtr< SWidget > SRigVMExecutionStackView::CreateContextMenu()
{
	TArray<TSharedPtr<FRigStackEntry>> SelectedItems = TreeView->GetSelectedItems();
	if(SelectedItems.Num() == 0)
	{
		return nullptr;
	}

	const FRigVMExecutionStackCommands& Actions = FRigVMExecutionStackCommands::Get();

	const bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);
	{
		MenuBuilder.BeginSection("RigStackToolsAction", LOCTEXT("ToolsAction", "Tools"));
		MenuBuilder.AddMenuEntry(Actions.FocusOnSelection);
		MenuBuilder.AddMenuEntry(Actions.GoToInstruction);

		if(SelectedItems.ContainsByPredicate([](const TSharedPtr<FRigStackEntry>& InEntry) -> bool
		{
			return InEntry->OpCode == ERigVMOpCode::JumpAbsolute ||
				InEntry->OpCode == ERigVMOpCode::JumpBackward || 
				InEntry->OpCode == ERigVMOpCode::JumpForward ||
				InEntry->OpCode == ERigVMOpCode::JumpAbsoluteIf || 
				InEntry->OpCode == ERigVMOpCode::JumpBackwardIf || 
				InEntry->OpCode == ERigVMOpCode::JumpForwardIf ||
				InEntry->OpCode == ERigVMOpCode::RunInstructions;
		}))
		{
			MenuBuilder.AddMenuEntry(Actions.SelectTargetInstructions);
		}
		
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SRigVMExecutionStackView::HandleFocusOnSelectedGraphNode()
{
	OnSelectionChanged(TSharedPtr<FRigStackEntry>(), ESelectInfo::Direct);

	TArray<TSharedPtr<FRigStackEntry>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
		{
			if (TStrongObjectPtr<UObject> StrongPtr = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
			{
				IRigVMEditorAssetInterface* RigVMBlueprint = WeakRigVMBlueprint.Get();
				URigVMHost* Host = PreviouslyDebuggedHost.Get();
				if (Host == nullptr || Host->GetVM() == nullptr)
				{
					return;
				}

				const FRigVMByteCode& ByteCode = Host->GetVM()->GetByteCode();
				UObject* Subject = ByteCode.GetSubjectForInstruction(SelectedItems[0]->SubjectIndex);
				if (URigVMNode* SelectedNode = Cast<URigVMNode>(Subject))
				{
					URigVMGraph* GraphToFocus = SelectedNode->GetGraph();
					if (GraphToFocus && GraphToFocus->GetTypedOuter<URigVMAggregateNode>())
					{
						if(URigVMGraph* ParentGraph = GraphToFocus->GetParentGraph())
						{
							GraphToFocus = ParentGraph;
						} 
					}
				
					if (UEdGraph* EdGraph = RigVMBlueprint->GetEdGraph(GraphToFocus))
					{
						Editor->OpenGraphAndBringToFront(EdGraph, true);
						Editor->ZoomToSelection_Clicked();
						Editor->HandleModifiedEvent(ERigVMGraphNotifType::NodeSelected, GraphToFocus, SelectedNode);
					}
				}
			}
		}
	}
}

void SRigVMExecutionStackView::HandleGoToInstruction()
{
	// figure out the current instruction's index
	int32 Index = 0;
	TArray<TSharedPtr<FRigStackEntry>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		if (SelectedItems[0]->EntryType == ERigStackEntry::Instruction)
		{
			Index = SelectedItems[0]->SubjectIndex;
		}
	}
	
	const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();
	
	SWindow::FArguments WindowArguments;
	WindowArguments.ScreenPosition(MouseCursorLocation);
	WindowArguments.AutoCenter(EAutoCenter::None);
	WindowArguments.FocusWhenFirstShown(true);

	TSharedPtr<SNumericEntryBox<int32>> NumericBox;
	const TSharedRef<SCustomDialog> OptionsDialog = SNew(SCustomDialog)
	.Title(FText(LOCTEXT("GoToInstructionDialog", "Go to...")))
	.WindowArguments(WindowArguments)
	.Content()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SAssignNew(NumericBox, SNumericEntryBox<int32>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Value(Index)
			.MinDesiredValueWidth(250)
			.MinValue(0)
			.MaxValue(RootEntries.Num() - 1)
			.OnValueChanged_Lambda([&Index](const int32& InIndex)
			{
				Index = InIndex;
			})
			.IsEnabled(true)
		]
	]
	.Buttons({
		SCustomDialog::FButton(LOCTEXT("OK", "OK")),
		SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
	});

	if(OptionsDialog->ShowModal() == 0)
	{
		if (const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(Index))
		{
			TreeView->SetSelection(*EntryPtr);
			TreeView->SetScrollOffset(static_cast<float>(FMath::Max(Index-5, 0)));
		}
	}
}

void SRigVMExecutionStackView::HandleSelectTargetInstructions()
{
	const TArray<TSharedPtr<FRigStackEntry>> TargetItems = GetTargetItems(TreeView->GetSelectedItems());
	if(!TargetItems.IsEmpty())
	{
		TreeView->ClearSelection();
		for(const TSharedPtr<FRigStackEntry>& TargetItem : TargetItems)
		{
			TreeView->SetItemSelection(TargetItem, true, ESelectInfo::Direct);
		}
		TreeView->RequestScrollIntoView(TargetItems[0]);
	}
}

void SRigVMExecutionStackView::HandleToggleEarlyExitInstruction()
{
	const TArray<TSharedPtr<FRigStackEntry>> TargetItems = TreeView->GetSelectedItems();
	if(TargetItems.IsEmpty())
	{
		return;
	}

	int32 InstructionIndex = TargetItems[0]->SubjectIndex;

	TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin();
	if (!Editor)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr RigBlueprint = Editor->GetRigVMAssetInterface();
	if (!RigBlueprint)
	{
		return;
	}

	URigVMHost* RigVMHost = PreviouslyDebuggedHost.Get();
	if (!RigVMHost)
	{
		return;
	}

	if (RigVMHost->GetDebugInfo().GetStepCondition().OriginInstruction == INDEX_NONE)
	{
		URigVM* VM = RigVMHost->GetVM();
		check(VM);

		const FRigVMInstructionArray Instructions = VM->GetByteCode().GetInstructions();
		while (Instructions.IsValidIndex(InstructionIndex))
		{
			if (URigVMNode* Node = Cast<URigVMNode>(VM->GetByteCode().GetSubjectForInstruction(InstructionIndex)))
			{
				RigBlueprint->SetEarlyExitInstruction(Node, InstructionIndex, false);
				return;
			}
			InstructionIndex++;
		}
	}
	
	RigBlueprint->ResetEarlyExitInstruction(true);
}

void SRigVMExecutionStackView::HandleStepEarlyExitInstruction()
{
	TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin();
	if (!Editor)
	{
		return;
	}
	FRigVMEditorAssetInterfacePtr RigBlueprint = Editor->GetRigVMAssetInterface();
	if (!RigBlueprint)
	{
		return;
	}

	URigVMHost* RigVMHost = PreviouslyDebuggedHost.Get();
	if (!RigVMHost)
	{
		return;
	}

	RigVMHost->GetRigVMExtendedExecuteContext().StepInto();
}

TArray<TSharedPtr<FRigStackEntry>> SRigVMExecutionStackView::GetTargetItems(const TArray<TSharedPtr<FRigStackEntry>>& InItems) const
{
	TArray<TSharedPtr<FRigStackEntry>> TargetItems;

	URigVMHost* Host = PreviouslyDebuggedHost.Get();
	if (Host == nullptr || Host->GetVM() == nullptr)
	{
		return TargetItems;
	}

	const FRigVMByteCode& ByteCode = Host->GetVM()->GetByteCode();
	const FRigVMInstructionArray Instructions = ByteCode.GetInstructions();

	TArray<TSharedPtr<FRigStackEntry>> TempTargetItems;
	const TArray<TSharedPtr<FRigStackEntry>> SelectedItems = TreeView->GetSelectedItems();
	for(const TSharedPtr<FRigStackEntry>& SelectedItem : SelectedItems)
	{
		if(!Instructions.IsValidIndex(SelectedItem->SubjectIndex))
		{
			continue;
		}
		
		const FRigVMInstruction& Instruction = Instructions[SelectedItem->SubjectIndex];
		switch(SelectedItem->OpCode)
		{
			case ERigVMOpCode::JumpAbsolute:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::JumpAbsoluteIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::JumpForward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(SelectedItem->SubjectIndex + Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::JumpForwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(SelectedItem->SubjectIndex + Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::JumpBackward:
			{
				const FRigVMJumpOp& Op = ByteCode.GetOpAt<FRigVMJumpOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(SelectedItem->SubjectIndex - Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::JumpBackwardIf:
			{
				const FRigVMJumpIfOp& Op = ByteCode.GetOpAt<FRigVMJumpIfOp>(Instruction);
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(SelectedItem->SubjectIndex - Op.InstructionIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			case ERigVMOpCode::RunInstructions:
			{
				const FRigVMRunInstructionsOp& Op = ByteCode.GetOpAt<FRigVMRunInstructionsOp>(Instruction);
				for(int32 Index = Op.StartInstruction; Index <= FMath::Max(Op.StartInstruction, Op.EndInstruction); Index++)
				{
					if(const TSharedPtr<FRigStackEntry>* EntryPtr = InstructionToEntry.Find(Index))
					{
						TargetItems.Add(*EntryPtr);
					}
				}
				for (int32 BlockIndex = 0; BlockIndex < Blocks.Num(); BlockIndex++)
				{
					if (Blocks[BlockIndex].Get<0>() == Op.StartInstruction && Blocks[BlockIndex].Get<1>() == Op.EndInstruction)
					{
						if(const TSharedPtr<FRigStackEntry>* EntryPtr = BlockToEntry.Find(Blocks[BlockIndex]))
						{
							TargetItems.Add(*EntryPtr);
						}
					}
				}
				break;
			}
			case ERigVMOpCode::InvokeCallable:
			{
				const FRigVMInvokeCallableOp& Op = ByteCode.GetOpAt<FRigVMInvokeCallableOp>(Instruction);
				check(ByteCode.IsValidCallableIndex(Op.CallableIndex));
				if(const TSharedPtr<FRigStackEntry>* EntryPtr = CallableToEntry.Find(Op.CallableIndex))
				{
					TargetItems.Add(*EntryPtr);
				}
				break;
			}
			default:
			{
				break;
			}
		}
	}

	return TargetItems;
}

void SRigVMExecutionStackView::UpdateTargetItemHighlighting()
{
	TreeView->ClearHighlightedItems();
	
	const TArray<TSharedPtr<FRigStackEntry>> TargetItems = GetTargetItems(TreeView->GetSelectedItems());
	for(const TSharedPtr<FRigStackEntry>& TargetItem : TargetItems)
	{
		if(!TreeView->IsItemSelected(TargetItem))
		{
			TreeView->SetItemHighlighted(TargetItem, true);
		}
	}
}

void SRigVMExecutionStackView::OnVMCompiled(UObject* InCompiledObject, URigVM* InCompiledVM, FRigVMExtendedExecuteContext& InVMContext)
{
	BindHostDelegates();
	RefreshTreeView(InCompiledVM, &InVMContext, true);
}

void SRigVMExecutionStackView::OnObjectBeingDebuggedChanged(UObject* InObjectBeingDebugged)
{
	BindHostDelegates(InObjectBeingDebugged);
	if (URigVMHost* DebuggedHost = Cast<URigVMHost>(InObjectBeingDebugged))
	{
		RefreshTreeView(DebuggedHost->GetVM(), &DebuggedHost->GetRigVMExtendedExecuteContext(), true);
	}
}

void SRigVMExecutionStackView::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;

	if (URigVMHost* RigVMHost = PreviouslyDebuggedHost.Get())
	{
		RefreshTreeView(RigVMHost->GetVM(), &RigVMHost->GetRigVMExtendedExecuteContext(), false);
	}
}

void SRigVMExecutionStackView::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject)
{
	if (bSuspendModelNotifications)
	{
		return;
	}

	URigVMHost* Host = PreviouslyDebuggedHost.Get();
	if (Host == nullptr || Host->GetVM() == nullptr)
	{
		return;
	}

	const FRigVMByteCode& ByteCode = Host->GetVM()->GetByteCode();

	switch (InNotifType)
	{
		case ERigVMGraphNotifType::NodeSelectionChanged:
		{
			const TGuardValue<bool> SuspendNotifs(bSuspendModelNotifications, true);
			TreeView->ClearSelection();
			TArray<const URigVMNode*> SelectedNodes;
			const TArray<FName>& SelectedNames = InGraph->GetSelectNodes();
			for (const FName& Selected : SelectedNames)
			{
				if (const URigVMNode* Node = InGraph->FindNodeByName(Selected))
				{
					SelectedNodes.Add(Node);
				}
			}		

			TArray<TSharedPtr<FRigStackEntry>> SelectedItems;
			for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
			{
				for (const URigVMNode* Node : SelectedNodes)
				{
					if(Entry->Callstack.Contains(Node))
					{
						SelectedItems.Add(Entry);
						break;
					}
				}
			}

			if(!SelectedItems.IsEmpty())
			{
				TreeView->SetItemSelection(SelectedItems, true, ESelectInfo::Direct);
				TreeView->RequestScrollIntoView(SelectedItems[0]);
			}

			UpdateTargetItemHighlighting();
			break;
		}
		default:
		{
			break;
		}
	}
}

void SRigVMExecutionStackView::HandleHostInitializedEvent(URigVMHost* InHost, const FName& InEventName)
{
	auto HandleHostInitializedEventLambda = [this, InHost, InEventName]
	{
		TGuardValue<bool> SuspendControllerSelection(bSuspendControllerSelection, true);

		RefreshTreeView(InHost->GetVM(), &InHost->GetRigVMExtendedExecuteContext(), true);
		OnSelectionChanged(TSharedPtr<FRigStackEntry>(), ESelectInfo::Direct);

		for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
		{
			for (TSharedPtr<FRigStackEntry>& Child : Entry->Children)
			{
				if (Child->EntryType == ERigStackEntry::Warning || Child->EntryType == ERigStackEntry::Error || Child->EntryType == ERigStackEntry::Info)
				{
					TreeView->SetItemExpansion(Entry, true);
					break;
				}
			}
		}

		if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
		{
			InHost->OnInitialized_AnyThread().Remove(OnHostInitializedHandle);
			OnHostInitializedHandle.Reset();

			if (FRigVMEditorAssetInterfacePtr RigBlueprint = Editor->GetRigVMAssetInterface())
			{
				TArray<URigVMGraph*> Models = RigBlueprint->GetAllModels();
				for (URigVMGraph* Model : Models)
				{
					for (URigVMNode* ModelNode : Model->GetNodes())
					{
						if (ModelNode->IsSelected())
						{
							HandleModifiedEvent(ERigVMGraphNotifType::NodeSelected, Model, ModelNode);
						}
					}
				}
			}
		}
	};
	
	if(IsInGameThread())
	{
		HandleHostInitializedEventLambda();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, HandleHostInitializedEventLambda);
	}
}

void SRigVMExecutionStackView::HandleHostExecutedEvent(URigVMHost* InHost, const FName& InEventName)
{
	auto HandleHostExecutedEventLambda = [this, InHost, InEventName]()
	{
		for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
		{
			Entry->ShowAsFadedOut.Reset();
			Entry->VisitedCountText.Reset();
			Entry->DurationText.Reset();
		}

		if (WeakRigVMBlueprint.IsValid())
		{
			IRigVMEditorAssetInterface* Blueprint = WeakRigVMBlueprint.Get();
			check(Blueprint);
			
			if(Blueprint->GetRigGraphDisplaySettings().bShowNodeRunCounts)
			{
				if(URigVM* VM = InHost->GetVM())
				{
					for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
					{
						if(Entry->EntryType == ERigStackEntry::Instruction)
						{
							const int32 Count = VM->GetInstructionVisitedCount(InHost->GetRigVMExtendedExecuteContext(), Entry->SubjectIndex);
							if(Count > 0)
							{
								Entry->VisitedCountText = FText::FromString(FString::FromInt(Count));
							}
							else
							{
								Entry->ShowAsFadedOut = true;
							}
						}
						else if(Entry->EntryType == ERigStackEntry::Callable)
						{
							const int32 Count = VM->GetCallableVisitedCount(InHost->GetRigVMExtendedExecuteContext(), Entry->SubjectIndex);
							if(Count > 0)
							{
								Entry->VisitedCountText = FText::FromString(FString::FromInt(Count));
							}
							else
							{
								Entry->ShowAsFadedOut = true;
							}
						}
					}
				}
			}

			if(Blueprint->GetVMRuntimeSettings().bEnableProfiling)
			{
				if(URigVM* VM = InHost->GetVM())
				{
					for (TSharedPtr<FRigStackEntry>& Entry : AllEntries)
					{
						if(Entry->EntryType == ERigStackEntry::Instruction)
						{
							const double CurrentMicroSeconds = VM->GetInstructionMicroSeconds(InHost->GetRigVMExtendedExecuteContext(), Entry->SubjectIndex);
							Entry->MicroSeconds = Blueprint->GetRigGraphDisplaySettings().AggregateAverage(Entry->MicroSecondsFrames, Entry->MicroSeconds, CurrentMicroSeconds);
							if(Entry->MicroSeconds > 0.0)
							{
								Entry->DurationText = FText::FromString(FString::Printf(TEXT("%.02f µs"), (float)Entry->MicroSeconds));
							}
							else
							{
								Entry->ShowAsFadedOut = true;
							}
						}
						else if(Entry->EntryType == ERigStackEntry::Callable)
						{
							const double CurrentMicroSeconds = VM->GetCallableMicroSeconds(InHost->GetRigVMExtendedExecuteContext(), Entry->SubjectIndex);
							Entry->MicroSeconds = Blueprint->GetRigGraphDisplaySettings().AggregateAverage(Entry->MicroSecondsFrames, Entry->MicroSeconds, CurrentMicroSeconds);
							if(Entry->MicroSeconds > 0.0)
							{
								Entry->DurationText = FText::FromString(FString::Printf(TEXT("%.02f µs"), (float)Entry->MicroSeconds));
							}
							else
							{
								Entry->ShowAsFadedOut = true;
							}
						}
					}
				}
			}
		}
	};

	if(IsInGameThread())
	{
		HandleHostExecutedEventLambda();
	}
	else
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, HandleHostExecutedEventLambda);
	}

}

void SRigVMExecutionStackView::HandlePreviewHostUpdated(IRigVMEditor* InEditor)
{
	if (URigVMHost* RigVMHost = PreviouslyDebuggedHost.Get())
	{
		RefreshTreeView(RigVMHost->GetVM(), &RigVMHost->GetRigVMExtendedExecuteContext(), true);
	}
}

void SRigVMExecutionStackView::HandleItemMouseDoubleClick(TSharedPtr<FRigStackEntry> InItem)
{
	if (TSharedPtr<IRigVMEditor> Editor = WeakEditor.Pin())
	{
		if (TStrongObjectPtr<UObject> StrongPtr = WeakRigVMBlueprint.GetWeakObjectPtr().Pin())
		{
			IRigVMEditorAssetInterface* RigVMBlueprint = WeakRigVMBlueprint.Get();
			URigVMHost* Host = PreviouslyDebuggedHost.Get();
			if (!Host || !Host->GetVM())
			{
				return;
			}

			const FRigVMByteCode& ByteCode = Host->GetVM()->GetByteCode();
			URigVMNode* Subject = Cast<URigVMNode>(ByteCode.GetSubjectForInstruction(InItem->SubjectIndex));
			if (!Subject)
			{
				if (InItem->Callstack.Num() > 0)
				{
					Subject = Cast<URigVMNode>(const_cast<UObject*>(InItem->Callstack.Last()));
				}
			}

			if (!Subject)
			{
				if (InItem->EntryType == ERigStackEntry::Callable)
				{
					if (ByteCode.IsValidCallableIndex(InItem->SubjectIndex))
					{
						const FRigVMCallableInfo* Callable = ByteCode.GetCallable(InItem->SubjectIndex);
						check(Callable);
						const FRigVMGraphFunctionHeader Header = FRigVMGraphFunctionHeader::FindGraphFunctionHeaderFromHash(Callable->Name.ToString());
						if (Header.IsValid())
						{
							if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(Header.LibraryPointer.GetNodeSoftPath().TryLoad()))
							{
								Subject = LibraryNode->GetEntryNode();
							}
						}
					}
				}
			}
			
			if (Subject)
			{
				URigVMGraph* GraphToFocus = Subject->GetGraph();
				if (GraphToFocus && GraphToFocus->GetTypedOuter<URigVMAggregateNode>())
				{
					if(URigVMGraph* ParentGraph = GraphToFocus->GetParentGraph())
					{
						GraphToFocus = ParentGraph;
					} 
				}
		
				if(URigVMEdGraph* EdGraph = Cast<URigVMEdGraph>(RigVMBlueprint->GetEdGraph(GraphToFocus)))
				{
					if(const UEdGraphNode* Node = EdGraph->FindNodeForModelNodeName(Subject->GetFName()))
					{
						Editor->JumpToHyperlink(Node, false);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
