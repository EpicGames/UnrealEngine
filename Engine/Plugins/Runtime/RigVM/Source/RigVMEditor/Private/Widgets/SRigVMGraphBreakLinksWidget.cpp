// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphBreakLinksWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "HAL/ConsoleManager.h"
#include "RigVMModel/RigVMController.h"
#include "Editor/RigVMNewEditor.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphBreakLinksWidget"

class SRigVMGraphBreakLinksWidgetRow : public SMultiColumnTableRow< URigVMLink* >
{
	SLATE_BEGIN_ARGS(SRigVMGraphBreakLinksWidgetRow) { }
	SLATE_ARGUMENT(URigVMLink*, Item)
	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Link = InArgs._Item;

		SMultiColumnTableRow<URigVMLink*>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if(InColumnName == TEXT("Source"))
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Link->GetSourcePin()->GetPinPath()))
				];
		}
		else if (InColumnName == TEXT("Target") )
		{
			return SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(Link->GetTargetPin()->GetPinPath()))
				];
		}
		else
		{
			ensure(false);
			return SNew(SBorder);
		}
	}

private:
	URigVMLink* Link;
};

//////////////////////////////////////////////////////////////
/// SRigVMGraphBreakLinksWidget
///////////////////////////////////////////////////////////

void SRigVMGraphBreakLinksWidget::Construct(const FArguments& InArgs, TArray<URigVMLink*> InLinks)
{
	Links = InLinks;
	OnFocusOnLink = InArgs._OnFocusOnLink;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(4.0f, 4.0f, 0.f, 0.f)
		.AutoHeight()
		.MaxHeight(600.f)
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[			
			   SNew( SListView< URigVMLink* > )
			     .ListItemsSource( &Links )
			     .ListViewStyle( FAppStyle::Get(), "SimpleListView" )
			     .OnMouseButtonDoubleClick(this, &SRigVMGraphBreakLinksWidget::HandleItemMouseDoubleClick)
			     .HeaderRow(SNew(SHeaderRow)
			     	+SHeaderRow::Column(FName(TEXT("Source")))
			     	.DefaultLabel(LOCTEXT("SourceColumnHeader", "Source"))
			     	.FillWidth(0.5f)
			     	+SHeaderRow::Column(FName(TEXT("Target")))
			     	.DefaultLabel(LOCTEXT("TargetColumnHeader", "Target"))
			     	.FillWidth(0.5f)
			     )
			     .OnGenerateRow(this, &SRigVMGraphBreakLinksWidget::GenerateItemRow)
			]
		]
	];
}

TSharedRef<ITableRow> SRigVMGraphBreakLinksWidget::GenerateItemRow(URigVMLink* Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRigVMGraphBreakLinksWidgetRow, OwnerTable)
			.Item(Item);
}

void SRigVMGraphBreakLinksWidget::HandleItemMouseDoubleClick(URigVMLink* InItem)
{
	OnFocusOnLink.ExecuteIfBound(InItem);		
}

void SRigVMGraphBreakLinksDialog::Construct(const FArguments& InArgs)
{
	UserResponse = EAppReturnType::Cancel;

	static const FString DialogText = TEXT(
        "This action will break links." 
    );

	const FString DialogTitle = TEXT("Break Links"); 
	
	SWindow::Construct(SWindow::FArguments()
        .Title(FText::FromString(DialogTitle))
        .SizingRule( ESizingRule::Autosized )
        //.ClientSize(FVector2D(450, 450))
        [
	        SNew(SHorizontalBox)

	        +SHorizontalBox::Slot()
	        .FillWidth(1.f)
	        .HAlign(HAlign_Left)
	        .VAlign(VAlign_Top)
	        //.Padding(2)
	        [
				SNew(SVerticalBox)

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                //.Padding(8)
                [

                    SNew(STextBlock)
                    .Text(FText::FromString(DialogText))
                    .Font(IDetailLayoutBuilder::GetDetailFont())
                ]
				
                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                .Padding(2)
                [
                    SAssignNew(BreakLinksWidget, SRigVMGraphBreakLinksWidget, InArgs._Links)
                    .OnFocusOnLink(InArgs._OnFocusOnLink)
                ]

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Right)
                .Padding(5)
                [
                    SNew(SUniformGridPanel)
                    .SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
                    .MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
                    .MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
                    +SUniformGridPanel::Slot(0, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("OK", "OK"))
                        .OnClicked(this, &SRigVMGraphBreakLinksDialog::OnButtonClick, EAppReturnType::Ok)
                    ]
                    +SUniformGridPanel::Slot(1, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("Cancel", "Cancel"))
                        .OnClicked(this, &SRigVMGraphBreakLinksDialog::OnButtonClick, EAppReturnType::Cancel)
                    ]
                ]
			]
		]);
}

FReply SRigVMGraphBreakLinksDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	RequestDestroyWindow();
	return FReply::Handled();
}

EAppReturnType::Type SRigVMGraphBreakLinksDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

#undef LOCTEXT_NAMESPACE
