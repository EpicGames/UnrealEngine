// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphChangePinType.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "DetailLayoutBuilder.h"
#include "RigVMDeveloperTypeUtils.h"
#include "RigVMModel/RigVMController.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphChangePinType"

static const FText RigVMEdGraphChangePinTypeMultipleValues = LOCTEXT("MultipleValues", " - ");

void SRigVMGraphChangePinType::Construct(const FArguments& InArgs)
{
	this->Types = InArgs._Types;
	this->OnTypeSelected = InArgs._OnTypeSelected;

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	BindingArgs.CurrentBindingText.BindRaw(this, &SRigVMGraphChangePinType::GetBindingText);
	BindingArgs.CurrentBindingImage.BindRaw(this, &SRigVMGraphChangePinType::GetBindingImage);
	BindingArgs.CurrentBindingColor.BindRaw(this, &SRigVMGraphChangePinType::GetBindingColor);

	BindingArgs.OnCanBindPropertyWithBindingChain.BindLambda([](const FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain) -> bool { return true; });
	BindingArgs.OnCanBindToClass.BindLambda([](UClass* InClass) -> bool { return false; });

	BindingArgs.bGeneratePureBindings = true;
	BindingArgs.bAllowNewBindings = true;
	BindingArgs.bAllowArrayElementBindings = false;
	BindingArgs.bAllowStructMemberBindings = false;
	BindingArgs.bAllowUObjectFunctions = false;

	BindingArgs.MenuExtender = MakeShareable(new FExtender);
	BindingArgs.MenuExtender->AddMenuExtension(
		"Properties",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateSP(this, &SRigVMGraphChangePinType::FillPinTypeMenu));

	this->ChildSlot
	[
		PropertyAccessEditor.MakePropertyBindingWidget(nullptr, BindingArgs)
	];
}

FText SRigVMGraphChangePinType::GetBindingText(const FRigVMTemplateArgumentType& InType)
{
	return RigVMTypeUtils::GetDisplayTextForArgumentType(InType);
}

FText SRigVMGraphChangePinType::GetBindingText() const
{
	if(Types.Num() > 0)
	{
		const FRigVMTemplateArgumentType FirstType = FRigVMRegistry::Get().GetType(Types[0]);
		const FText FirstText = GetBindingText(FirstType);
		for(int32 Index = 1; Index < Types.Num(); Index++)
		{
			const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(Types[Index]);
			if(!GetBindingText(Type).EqualTo(FirstText))
			{
				return RigVMEdGraphChangePinTypeMultipleValues;
			}
		}
		return FirstText;
	}
	return FText();
}

const FSlateBrush* SRigVMGraphChangePinType::GetBindingImage() const
{
	static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
	static const FLazyName ArrayTypeIcon(TEXT("Kismet.VariableList.ArrayTypeIcon"));

	if(Types.Num() > 0)
	{
		const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(Types[0]);
		if(Type.IsArray())
		{
			return FAppStyle::GetBrush(ArrayTypeIcon);
		}
	}
	return FAppStyle::GetBrush(TypeIcon);
}

FLinearColor SRigVMGraphChangePinType::GetBindingColor() const
{
	if(Types.Num() > 0)
	{
		const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(Types[0]);
		const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(Type.CPPType, Type.CPPTypeObject);
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		return Schema->GetPinTypeColor(PinType);
	}
	return FLinearColor::White;
}

void SRigVMGraphChangePinType::FillPinTypeMenu(FMenuBuilder& MenuBuilder)
{
	if(Types.Num() == 0)
	{
		return;
	}

	struct FArgumentInfo
	{
		static FArgumentInfo Make(const FProperty* InProperty)
		{
			FArgumentInfo Info;
			Info.Property = InProperty;
			return Info;
		}

		static FArgumentInfo Make(const FRigVMTemplateArgumentType& InType)
		{
			FArgumentInfo Info;
			Info.Property = nullptr;
			Info.Type = InType;
			return Info;
		}

		const FProperty* Property;
		FRigVMTemplateArgumentType Type;
	};

	typedef TPair<TRigVMTypeIndex, FArgumentInfo> FTypePair;
	TMap<TRigVMTypeIndex, FArgumentInfo> TypesToInfo;
	for(TRigVMTypeIndex& TypeIndex : Types)
	{
		if (TypeIndex == INDEX_NONE)
		{
			// skip invalid permutations
			continue;
		}

		if(TypeIndex == RigVMTypeUtils::TypeIndex::Float)
		{
			TypeIndex = RigVMTypeUtils::TypeIndex::Double;
		}
		else if(TypeIndex == RigVMTypeUtils::TypeIndex::FloatArray)
		{
			TypeIndex = RigVMTypeUtils::TypeIndex::DoubleArray;
		}
		
		if(TypesToInfo.Contains(TypeIndex))
		{
			continue;
		}

		const FRigVMTemplateArgumentType ArgumentType = FRigVMRegistry::Get().GetType(TypeIndex);
		TypesToInfo.Add(TypeIndex, FArgumentInfo::Make(ArgumentType));
	}

	// sort the types and put them into an array
	static const TArray<FString> SortOrder = {
		TEXT("bool"),
		TEXT("int32"),
		TEXT("float"),
		TEXT("double"),
		TEXT("FName"),
		TEXT("FString"),
		TEXT("FVector"),
		TEXT("FRotator"),
		TEXT("FQuat"),
		TEXT("FTransform"),
		TEXT("FEulerTransform"),
		TEXT("FMatrix")
	};
	TArray<FTypePair> SortedTypes;
	for(const FTypePair& Pair : TypesToInfo)
	{
		SortedTypes.Add(Pair);
	}
	SortedTypes.Sort([](const FTypePair& A, const FTypePair& B) -> bool
	{
		const FRigVMTemplateArgumentType TypeA = FRigVMRegistry::Get().GetType(A.Key);
		const FRigVMTemplateArgumentType TypeB = FRigVMRegistry::Get().GetType(B.Key);
		const FString BaseTypeA = TypeA.GetBaseCPPType();
		const FString BaseTypeB = TypeB.GetBaseCPPType();

		const int32 IndexA = SortOrder.Find(BaseTypeA);  
		const int32 IndexB = SortOrder.Find(BaseTypeB);
		if((IndexA == INDEX_NONE) != (IndexB == INDEX_NONE))
		{
			return IndexA != INDEX_NONE;
		}
		else if((IndexA != INDEX_NONE) && (IndexB != INDEX_NONE))
		{
			return IndexA < IndexB;
		}

		return BaseTypeA.Compare(BaseTypeB) < 0;
	});
	
	MenuBuilder.BeginSection("PinTypes", LOCTEXT("PinTypes", "Pin Types"));
	{
		static const FLazyName PropertyIcon(TEXT("Kismet.VariableList.TypeIcon"));
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

		const bool bHasAllTypes =
			SortedTypes.Num() >=
				FRigVMRegistry::Get().GetTypesForCategory(FRigVMTemplateArgument::ETypeCategory_SingleAnyValue).Num();
		
		for(int32 SortedIndex=0; SortedIndex < SortedTypes.Num(); SortedIndex++)
		{
			const TRigVMTypeIndex& TypeIndex = SortedTypes[SortedIndex].Key;
			const FRigVMTemplateArgumentType Type = FRigVMRegistry::Get().GetType(TypeIndex);
			if (Type.CPPTypeObject != nullptr && !IsValid(Type.CPPTypeObject))
			{
				continue;
			}
			
			const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromCPPType(Type.CPPType, Type.CPPTypeObject);

			if(bHasAllTypes && Type.CPPTypeObject)
			{
				if(Type.CPPTypeObject->IsA<UEnum>())
				{
					continue;
				}

				if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
				{
					const FRigVMTemplateArgumentType MathType(*RigVMTypeUtils::GetUniqueStructTypeName(ScriptStruct), ScriptStruct);
					const TRigVMTypeIndex MathTypeIndex = FRigVMRegistry::Get().GetTypeIndex(MathType);
					
					if(!FRigVMRegistry::Get().GetTypesForCategory(FRigVMTemplateArgument::ETypeCategory_SingleMathStructValue).Contains(MathTypeIndex))
					{
						continue;
					}
				}
			}

			if(UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Type.CPPTypeObject))
			{
				if(ScriptStruct->IsChildOf(FRigVMExecutePin::StaticStruct()))
				{
					continue;
				}
			}

			MenuBuilder.AddMenuEntry(
				FUIAction(FExecuteAction::CreateSP(this, &SRigVMGraphChangePinType::HandlePinTypeChanged, Type)),
				SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(18.0f, 0.0f))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FBlueprintEditorUtils::GetIconFromPin(PinType, true))
						.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(GetBindingText(Type))
						.ColorAndOpacity(FLinearColor::White)
						.ToolTipText(LOCTEXT("WildcardAvailableFilterTypeTooltip", "Available filtered type"))
					]);
		}
	}
	MenuBuilder.EndSection(); // Local Variables
}

void SRigVMGraphChangePinType::HandlePinTypeChanged(FRigVMTemplateArgumentType InType)
{
	if (OnTypeSelected.IsBound())
	{
		OnTypeSelected.Execute(FRigVMRegistry::Get().GetTypeIndex(InType));
		return;
	}
}


#undef LOCTEXT_NAMESPACE
