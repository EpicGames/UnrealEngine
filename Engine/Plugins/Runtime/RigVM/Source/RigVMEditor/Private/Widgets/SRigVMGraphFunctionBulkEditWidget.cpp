// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphFunctionBulkEditWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "HAL/ConsoleManager.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/RigVMController.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphFunctionBulkEditWidget"

//////////////////////////////////////////////////////////////
/// SRigVMGraphFunctionBulkEditWidget
///////////////////////////////////////////////////////////

void SRigVMGraphFunctionBulkEditWidget::Construct(const FArguments& InArgs, FRigVMEditorAssetInterfacePtr InBlueprint, URigVMController* InController, URigVMLibraryNode* InFunction, ERigVMControllerBulkEditType InEditType)
{
	Blueprint = InBlueprint;
	Controller = InController;
	Function = InFunction;
	EditType = InEditType;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(4.0f, 4.0f, 0.f, 0.f)
		.AutoHeight()
		.MaxHeight(600.f)
		[
			SNew(SBorder)
			.Visibility(EVisibility::Visible)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				MakeAssetViewForReferencedAssets()
			]
		]
	];
}

TSharedPtr<SWidget> SRigVMGraphFunctionBulkEditWidget::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	return nullptr;
}

void SRigVMGraphFunctionBulkEditWidget::OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets,
	EAssetTypeActivationMethod::Type ActivationMethod)
{
	// nothing to do here
}

void SRigVMGraphFunctionBulkEditWidget::LoadAffectedAssets()
{
	TArray<FAssetData> FirstLevelReferenceAssets = Controller->GetAffectedAssets(EditType, false);

	{
		FScopedSlowTask SlowTask((float)FirstLevelReferenceAssets.Num(), LOCTEXT("LoadingAffectedAssets", "Loading Affected Assets"));
		Controller->OnBulkEditProgressDelegate.BindLambda([&SlowTask](
            TSoftObjectPtr<URigVMFunctionReferenceNode> InReference,
            ERigVMControllerBulkEditType InEditType,
            ERigVMControllerBulkEditProgress InProgress,
            int32 InIndex,
            int32 InCount)
        {
            if(!InReference.IsValid())
            {
                return;
            }

			const FString PackageName = InReference.ToSoftObjectPath().GetLongPackageName();
			const FText Message = FText::FromString(FString::Printf(TEXT("Loading '%s' ..."), *PackageName));

			if (InIndex == 0)
			{
				SlowTask.TotalAmountOfWork += (float)InCount;
			}
			SlowTask.EnterProgressFrame(1.f, Message);
        });
		
		SlowTask.MakeDialog();

		Controller->GetAffectedAssets(EditType, true);
	}

	Controller->OnBulkEditProgressDelegate.Unbind();
}

TSharedRef<SWidget> SRigVMGraphFunctionBulkEditWidget::MakeAssetViewForReferencedAssets()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	FAssetPickerConfig AssetPickerConfig;

	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.bCanShowClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bShowBottomToolbar = false;
	AssetPickerConfig.bAutohideSearchBar = true;
	AssetPickerConfig.Filter.ClassPaths.Add(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("InvalidClass")));
	AssetPickerConfig.Filter.bIncludeOnlyOnDiskAssets = false;
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SRigVMGraphFunctionBulkEditWidget::OnAssetsActivated);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP( this, &SRigVMGraphFunctionBulkEditWidget::OnGetAssetContextMenu );

	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.OnGetCustomSourceAssets.BindLambda([this](const FARFilter& SourceFilter, TArray<FAssetData>& AddedAssets)
	{
		AddedAssets.Append(Controller->GetAffectedAssets(EditType, false));
	});
	
	return ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);
}


void SRigVMGraphFunctionBulkEditDialog::Construct(const FArguments& InArgs)
{
	UserResponse = EAppReturnType::Cancel;

	static const FString DialogText = TEXT(
        "This action will affect other assets.\n\n" 
        "Note that this action is not undoable.\n\n"
        "All affected assets will be marked as modified\n"
        "and require re-saving."
    );

	const FString DialogTitle = FString::Printf(TEXT("Preparing Bulk-Edit - %s"), *StaticEnum<ERigVMControllerBulkEditType>()->GetDisplayNameTextByValue((int64)InArgs._EditType).ToString()); 
	
	SWindow::Construct(SWindow::FArguments()
        .Title(FText::FromString(DialogTitle))
        .SupportsMinimize(false)
        .SupportsMaximize(false)
        .SizingRule( ESizingRule::Autosized )
        .ClientSize(FVector2D(450, 450))
        [
	        SNew(SHorizontalBox)

	        +SHorizontalBox::Slot()
	        .AutoWidth()
	        .HAlign(HAlign_Left)
	        .VAlign(VAlign_Top)
	        .Padding(2)
	        [
				SNew(SVerticalBox)

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                .Padding(8)
                [

                    SNew(STextBlock)
                    .Text(FText::FromString(DialogText))
                    .Font(IDetailLayoutBuilder::GetDetailFont())
                ]
				
                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                .Padding(2)
                [
                    SAssignNew(BulkEditWidget, SRigVMGraphFunctionBulkEditWidget, InArgs._Blueprint, InArgs._Controller, InArgs._Function, InArgs._EditType)
                ]

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Right)
                .Padding(5)
                [
                    SNew(SUniformGridPanel)
                    .SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
                    .MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
                    .MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
                    +SUniformGridPanel::Slot(0, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("OK", "OK"))
                        .OnClicked(this, &SRigVMGraphFunctionBulkEditDialog::OnButtonClick, EAppReturnType::Ok)
                    ]
                    +SUniformGridPanel::Slot(1, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("Cancel", "Cancel"))
                        .OnClicked(this, &SRigVMGraphFunctionBulkEditDialog::OnButtonClick, EAppReturnType::Cancel)
                    ]
                ]
			]
		]);
}

FReply SRigVMGraphFunctionBulkEditDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	if(UserResponse == EAppReturnType::Ok)
	{
		BulkEditWidget->LoadAffectedAssets();
	}
	
	RequestDestroyWindow();

	return FReply::Handled();
}

EAppReturnType::Type SRigVMGraphFunctionBulkEditDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

#undef LOCTEXT_NAMESPACE
