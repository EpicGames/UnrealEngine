// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphFunctionLocalizationWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "HAL/ConsoleManager.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphFunctionLocalizationWidget"

//////////////////////////////////////////////////////////////
/// SRigVMGraphFunctionLocalizationItem
///////////////////////////////////////////////////////////

SRigVMGraphFunctionLocalizationItem::SRigVMGraphFunctionLocalizationItem(const FRigVMGraphFunctionIdentifier& InFunction)
    : Function(InFunction)
{
	FString OuterName;
	FString FunctionName;
	bool bIsPublic = false;

	if (FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(InFunction, &bIsPublic))
	{
		OuterName = InFunction.GetNodeSoftPath().GetAssetPathString();
		FunctionName = FunctionData->Header.Name.ToString();
		DisplayText = FText::FromString(FString::Printf(TEXT("%s :: %s"), *OuterName, *FunctionName));
		ToolTipText = bIsPublic ?
			FText::FromString(FString::Printf(TEXT("%s :: %s is public. Check this to create a local copy."), *OuterName, *FunctionName)) :
			FText::FromString(FString::Printf(TEXT("%s :: %s is private. A local copy has to be created. To avoid this you can turn it public in the source rig vm."), *OuterName, *FunctionName));
	}
}

//////////////////////////////////////////////////////////////
/// SRigVMGraphFunctionLocalizationTableRow
///////////////////////////////////////////////////////////

void SRigVMGraphFunctionLocalizationTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, SRigVMGraphFunctionLocalizationWidget* InLocalizationWidget, TSharedRef<SRigVMGraphFunctionLocalizationItem> InFunctionItem)
{
	STableRow<TSharedPtr<SRigVMGraphFunctionLocalizationItem>>::Construct(
		STableRow<TSharedPtr<SRigVMGraphFunctionLocalizationItem>>::FArguments()
		.Content()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 12.f, 0.f)
				[
					SNew(STextBlock)
					.Text(InFunctionItem->DisplayText)
					.ToolTipText(InFunctionItem->ToolTipText)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(InLocalizationWidget, &SRigVMGraphFunctionLocalizationWidget::IsFunctionEnabled, InFunctionItem->Function)
					.OnCheckStateChanged(InLocalizationWidget, &SRigVMGraphFunctionLocalizationWidget::SetFunctionEnabled, InFunctionItem->Function)
					.IsEnabled(InLocalizationWidget, &SRigVMGraphFunctionLocalizationWidget::IsFunctionPublic, InFunctionItem->Function)
					.ToolTipText(InFunctionItem->ToolTipText)
				]
			]
		]
		, OwnerTable
	);
}

//////////////////////////////////////////////////////////////
/// SRigVMGraphFunctionLocalizationWidget
///////////////////////////////////////////////////////////

void SRigVMGraphFunctionLocalizationWidget::Construct(const FArguments& InArgs, const FRigVMGraphFunctionIdentifier& InFunctionToLocalize, IRigVMGraphFunctionHost* InTargetFunctionHost)
{
	FunctionsToLocalize.Add(InFunctionToLocalize);
	FunctionItems.Reset();

	TArray<FRigVMGraphFunctionIdentifier> NodesToVisit;
	TArray<FRigVMGraphFunctionData*> FunctionsForTable;

	{
		FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(InFunctionToLocalize);
		FunctionsForTable.Add(FunctionData);
		NodesToVisit.Add(InFunctionToLocalize);
	}

	for(int32 NodeToVisitIndex=0; NodeToVisitIndex<NodesToVisit.Num(); NodeToVisitIndex++)
	{
		const FRigVMGraphFunctionIdentifier NodeToVisit = NodesToVisit[NodeToVisitIndex];

		bool bIsPublic;
		FRigVMGraphFunctionData* FunctionData = FRigVMGraphFunctionData::FindFunctionData(NodeToVisit, &bIsPublic);
		if (!FunctionData)
		{
			continue;
		}

		for (TPair<FRigVMGraphFunctionIdentifier, uint32>& Pair : FunctionData->Header.Dependencies)
		{
			NodesToVisit.AddUnique(Pair.Key);
		}
		
		if (NodeToVisit.HostObject != Cast<UObject>(InTargetFunctionHost))
		{
			FunctionsForTable.AddUnique(FunctionData);
			if (bIsPublic)
			{
				FunctionsToLocalize.AddUnique(NodeToVisit);
			}
		}
	} 

	// sort the functions to localize based on their nesting
	Algo::Sort(FunctionsForTable, [](FRigVMGraphFunctionData* A, FRigVMGraphFunctionData* B) -> bool
    {
		check(A);
		check(B);
		return A->Header.Dependencies.Contains(B->Header.LibraryPointer);
    });

	for (FRigVMGraphFunctionData* FunctionForTable : FunctionsForTable)
	{
		FunctionItems.Add(MakeShared<SRigVMGraphFunctionLocalizationItem>(FunctionForTable->Header.LibraryPointer));
	}

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.0f, 4.0f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::GetBrush("Menu.Background"))
				[
					SNew(SListView<TSharedPtr<SRigVMGraphFunctionLocalizationItem>>)
					.ListItemsSource(&FunctionItems)
					.OnGenerateRow(this, &SRigVMGraphFunctionLocalizationWidget::GenerateFunctionListRow)
					.SelectionMode(ESelectionMode::None)
				]
			]
		];
}

TSharedRef<ITableRow> SRigVMGraphFunctionLocalizationWidget::GenerateFunctionListRow(TSharedPtr<SRigVMGraphFunctionLocalizationItem> InItem, const TSharedRef<STableViewBase>& InOwningTable)
{
	TSharedRef<SRigVMGraphFunctionLocalizationTableRow> TableRow = SNew(SRigVMGraphFunctionLocalizationTableRow, InOwningTable, this, InItem.ToSharedRef());
	TableRows.Add(InItem->Function, TableRow);
	return TableRow;
}

ECheckBoxState SRigVMGraphFunctionLocalizationWidget::IsFunctionEnabled(const FRigVMGraphFunctionIdentifier InFunction) const
{
	return FunctionsToLocalize.Contains(InFunction) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SRigVMGraphFunctionLocalizationWidget::SetFunctionEnabled(ECheckBoxState NewState, const FRigVMGraphFunctionIdentifier InFunction)
{
	if(NewState == ECheckBoxState::Checked)
	{
		FunctionsToLocalize.AddUnique(InFunction);
	}
	else
	{
		FunctionsToLocalize.Remove(InFunction);
	}
}

bool SRigVMGraphFunctionLocalizationWidget::IsFunctionPublic(const FRigVMGraphFunctionIdentifier InFunction) const
{
	IRigVMGraphFunctionHost* FunctionHost = nullptr;
	if (UObject* FunctionHostObj = InFunction.HostObject.TryLoad())
	{
		FunctionHost = Cast<IRigVMGraphFunctionHost>(FunctionHostObj);									
	}
	if (FunctionHost)
	{
		return FunctionHost->GetRigVMGraphFunctionStore()->IsFunctionPublic(InFunction);
	}
			
	return false;	
}

void SRigVMGraphFunctionLocalizationDialog::Construct(const FArguments& InArgs)
{
	UserResponse = EAppReturnType::Cancel;

	static const FString DialogText = TEXT(
        "This action requires a function to be localized.\n\n" 
        "Check the functions you want to create copies of below.\n\n"
        "Note that private functions have to be copied,\n"
        "while public functions can be referenced.\n\n"
        "Checking all functions removes the dependencies completely."
    );
	
	SWindow::Construct(SWindow::FArguments()
        .Title(LOCTEXT("SRigVMGraphFunctionLocalizationDialog_Title", "Select function(s) to localize"))
        .SupportsMinimize(false)
        .SupportsMaximize(false)
        .SizingRule( ESizingRule::Autosized )
        .ClientSize(FVector2D(450, 450))
        [
	        SNew(SHorizontalBox)

	        +SHorizontalBox::Slot()
	        .AutoWidth()
	        .HAlign(HAlign_Left)
	        .VAlign(VAlign_Top)
	        .Padding(2)
	        [
				SNew(SVerticalBox)

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                .Padding(8)
                [

                    SNew(STextBlock)
                    .Text(FText::FromString(DialogText))
                    .Font(IDetailLayoutBuilder::GetDetailFont())
                ]
				
                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Left)
                .Padding(2)
                [
                    SAssignNew(FunctionsWidget, SRigVMGraphFunctionLocalizationWidget, InArgs._Function, InArgs._GraphFunctionHost)
                ]

                +SVerticalBox::Slot()
                .AutoHeight()
                .HAlign(HAlign_Right)
                .Padding(5)
                [
                    SNew(SUniformGridPanel)
                    .SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
                    .MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
                    .MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
                    +SUniformGridPanel::Slot(0, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("OK", "OK"))
                        .OnClicked(this, &SRigVMGraphFunctionLocalizationDialog::OnButtonClick, EAppReturnType::Ok)
                        .IsEnabled(this, &SRigVMGraphFunctionLocalizationDialog::IsOkButtonEnabled)
                    ]
                    +SUniformGridPanel::Slot(1, 0)
                    [
                        SNew(SButton)
                        .HAlign(HAlign_Center)
                        .ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
                        .Text(LOCTEXT("Cancel", "Cancel"))
                        .OnClicked(this, &SRigVMGraphFunctionLocalizationDialog::OnButtonClick, EAppReturnType::Cancel)
                    ]
                ]
			]
		]);
}

FReply SRigVMGraphFunctionLocalizationDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();

	return FReply::Handled();
}

bool SRigVMGraphFunctionLocalizationDialog::IsOkButtonEnabled() const
{
	return FunctionsWidget->FunctionsToLocalize.Num() > 0;
}

EAppReturnType::Type SRigVMGraphFunctionLocalizationDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

TArray<FRigVMGraphFunctionIdentifier>& SRigVMGraphFunctionLocalizationDialog::GetFunctionsToLocalize()
{
	return FunctionsWidget->FunctionsToLocalize;
}

#undef LOCTEXT_NAMESPACE
