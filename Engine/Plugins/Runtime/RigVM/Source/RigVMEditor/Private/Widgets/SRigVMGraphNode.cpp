// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphNode.h"
#include "RigVMHost.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "EdGraph/RigVMEdGraph.h"
#include "SGraphPin.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "SLevelOfDetailBranchNode.h"
#include "SGraphPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
#include "GraphEditorSettings.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Engine/Engine.h"
#include "KismetNodes/KismetNodeInfoContext.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "PropertyPathHelpers.h"
#include "UObject/PropertyPortFlags.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMCore/RigVM.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMCompiler/RigVMCompiler.h"
#include "IDocumentation.h"
#include "DetailLayoutBuilder.h"
#include "RigVMBlueprintUtils.h"
#include "Styling/AppStyle.h"
#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "RigVMModel/Nodes/RigVMAggregateNode.h"
#include "RigVMModel/Nodes/RigVMDispatchNode.h"
#include "Slate/SlateTextures.h"
#include "RigVMFunctions/RigVMDispatch_If.h"
#include "RigVMFunctions/RigVMDispatch_Select.h"
#include "Algo/Copy.h"
#include "Brushes/SlateColorBrush.h"
#include "Widgets/SRigVMGraphPinCategory.h"
#include "Widgets/SRigVMVariantTagWidget.h"
#include "Widgets/SToolTip.h"
#include "Overrides/OverrideStatusWidgetMenuBuilder.h"
#include "RigVMFunctions/RigVMDispatch_Switch.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EdGraphSchema_K2.h"
#endif

#define LOCTEXT_NAMESPACE "SRigVMGraphNode"

const FSlateBrush* SRigVMGraphNode::CachedImg_CR_Pin_Connected = nullptr;
const FSlateBrush* SRigVMGraphNode::CachedImg_CR_Pin_Disconnected = nullptr;

class FRigVMGraphPinReorderDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRigVMGraphPinReorderDragDropOperation, FDragDropOperation)

	TWeakObjectPtr<URigVMPin> Pin;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}
};

void SRigVMGraphPinReorderHandle::Construct(const FArguments& InArgs)
{
	bIsInput = InArgs._bIsInput;
	Pin = InArgs._Pin;
	LineExtension = InArgs._LineExtension;
	CanReorderDelegate = InArgs._CanReorder;
	ReorderDelegate = InArgs._OnReorder;
	WeakClientHost = InArgs._WeakClientHost;

	SetCursor(EMouseCursor::GrabHand);

	ChildSlot
	[
		SNew(SBox)
		.Padding(0.0f)
		.HAlign(bIsInput ? HAlign_Left : HAlign_Right)
		.VAlign(VAlign_Center)
		.WidthOverride(InArgs._Width)
		.HeightOverride(InArgs._Height)
		[
			SNew(SImage)
			.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
		]
	];
}

FReply SRigVMGraphPinReorderHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SRigVMGraphPinReorderHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IsEnabled() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && Pin.IsValid())
	{
		if (Pin->IsExecuteContext())
		{
			return FReply::Unhandled();
		}
		
		TSharedPtr<FRigVMGraphPinReorderDragDropOperation> DragDropOp = MakeShared<FRigVMGraphPinReorderDragDropOperation>();
		if (DragDropOp.IsValid())
		{
			DragDropOp->Pin = Pin;
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
	}

	return FReply::Unhandled();
}

FReply SRigVMGraphPinReorderHandle::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	UE::Slate::FDeprecateVector2DResult LocalPosition = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
	const bool bIsAboveItem = LocalPosition.Y < MyGeometry.Size.Y * 0.5f;
	const EItemDropZone DesiredDropZone = bIsAboveItem ? EItemDropZone::AboveItem : EItemDropZone::BelowItem;

	if (TSharedPtr<FRigVMGraphPinReorderDragDropOperation> ReorderDragDropOp = DragDropEvent.GetOperationAs<FRigVMGraphPinReorderDragDropOperation>())
	{
		// only allow drag & drop of pins on the same node
		if (Pin.IsValid() && ReorderDragDropOp->Pin.IsValid() && Pin->GetNode() == ReorderDragDropOp->Pin->GetNode())
		{
			DropZone = DesiredDropZone;

			if (CanReorderDelegate.IsBound())
			{
				if (CanReorderDelegate.Execute(ReorderDragDropOp->Pin.Get(), Pin.Get(), DropZone.GetValue()) == INDEX_NONE)
				{
					DropZone.Reset();
				}
			}

			
			if (DropZone.IsSet())
			{
				return FReply::Handled();
			}
		}
	}
	else if (TSharedPtr<FGraphEditorDragDropAction> PinDragDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>())
	{
		if (PinDragDropOp->IsOfTypeImpl(TEXT("FDragConnection")))
		{
			DropZone = DesiredDropZone;
		}
	}
	return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
}

void SRigVMGraphPinReorderHandle::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragLeave(DragDropEvent);
	DropZone.Reset();
}

FReply SRigVMGraphPinReorderHandle::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (DropZone.IsSet())
	{
		if (TSharedPtr<FRigVMGraphPinReorderDragDropOperation> ReorderDragDropOp = DragDropEvent.GetOperationAs<FRigVMGraphPinReorderDragDropOperation>())
		{
			// only allow drag & drop of pins on the same node
			if (Pin.IsValid() && ReorderDragDropOp->Pin.IsValid() && Pin->GetNode() == ReorderDragDropOp->Pin->GetNode())
			{
				if (ReorderDelegate.IsBound())
				{
					ReorderDelegate.Execute(ReorderDragDropOp->Pin.Get(), Pin.Get(), DropZone.GetValue());
				}
			}
		}
		else if (TSharedPtr<FGraphEditorDragDropAction> PinDragDropOp = DragDropEvent.GetOperationAs<FGraphEditorDragDropAction>())
		{
			if (PinDragDropOp->IsOfTypeImpl(TEXT("FDragConnection")) && Pin.IsValid() && WeakClientHost.IsValid())
			{
				int32 NewIndex = Pin->GetPinIndex();
				if (DropZone.GetValue() == EItemDropZone::BelowItem)
				{
					NewIndex++;
				}

				if (URigVMController* Controller = WeakClientHost.Get()->GetOrCreateController(Pin->GetGraph()))
				{
					FName ExposedPinName = NAME_None;

					// attach a listener here to determine the exposed pin being added.
					// the code path to access the pins dragged etc is closed down within private
					// code in GraphEditor - so relying on listening to changes coming from RigVM here instead.
					const FDelegateHandle Handle = Controller->OnModified().AddLambda(
						[Controller, &ExposedPinName](ERigVMGraphNotifType InNotification, URigVMGraph* InGraph, UObject* InSubject)
						{
							if (InGraph != Controller->GetGraph())
							{
								return;
							}

							if (InNotification == ERigVMGraphNotifType::PinAdded)
							{
								if (const URigVMPin* Pin = Cast<URigVMPin>(InSubject))
								{
									if (Pin->IsRootPin())
									{
										ExposedPinName = Pin->GetFName();
									}
								}
							}
						}
					);
					
					FReply Reply = PinDragDropOp->DroppedOnNode(DragDropEvent.GetScreenSpacePosition(), FVector2f::ZeroVector);

					// remove the handle immediately to clean up the listener
					Controller->OnModified().Remove(Handle);
					
					if (Reply.IsEventHandled())
					{
						// now since we have a new exposed pin let's move it to the right spot
						Controller->SetExposedPinIndex(ExposedPinName, NewIndex);
						return Reply;
					}
				}
			}
		}

		DropZone.Reset();
		return FReply::Handled();
	}
	return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
}

int32 SRigVMGraphPinReorderHandle::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 LastLayerId = LayerId;

	if (IsHovered())
	{
		LastLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	if (DropZone.IsSet())
	{
		TArray<FVector2D> ClampLine;
		ClampLine.Emplace(0, DropZone.GetValue() == EItemDropZone::AboveItem ? 0 : AllottedGeometry.Size.Y - 1.f);
		ClampLine.Emplace(AllottedGeometry.Size.X, ClampLine.Last().Y);

		if (LineExtension.IsSet() || LineExtension.IsBound())
		{
			const float Extension = LineExtension.Get();
			if (bIsInput)
			{
				ClampLine[0].X -= Extension;
				ClampLine[1].X -= Extension;
			}
			else
			{
				ClampLine[0].X += Extension;
				ClampLine[1].X += Extension;
			}
		}

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ClampLine,
			ESlateDrawEffect::None,
			FSlateColor(EStyleColor::AccentBlue).GetSpecifiedColor(),
			false,
			3
		);
	}
	
	return LastLayerId;
}

void SRigVMGraphNode::Construct( const FArguments& InArgs )
{
	if (CachedImg_CR_Pin_Connected == nullptr)
	{
		static const FLazyName NAME_CR_Pin_Connected("ControlRig.Bug.Solid");
		static const FLazyName NAME_CR_Pin_Disconnected("ControlRig.Bug.Open");
		CachedImg_CR_Pin_Connected = FSlateIcon(TEXT("ControlRigEditorStyle"), NAME_CR_Pin_Connected).GetIcon();
		CachedImg_CR_Pin_Disconnected = FSlateIcon(TEXT("ControlRigEditorStyle"), NAME_CR_Pin_Disconnected).GetIcon();
	}

	check(InArgs._GraphNodeObj);
	this->GraphNode = InArgs._GraphNodeObj;
	this->SetCursor( EMouseCursor::CardinalCross );

	SetEnabled(InArgs._Enabled);
	PinDragDropEnabledAttribute = InArgs._PinDragDropEnabled;

 	URigVMEdGraphNode* EdGraphNode = InArgs._GraphNodeObj;
	ModelNode = EdGraphNode->GetModelNode();
	if (!ModelNode.IsValid())
	{
		return;
	}
	URigVMController* Controller = EdGraphNode->GetController();
	if (!Controller)
	{
		return;
	}

	Blueprint = FRigVMBlueprintUtils::FindAssetForNode(this->GraphNode);

	NodeErrorType = int32(EMessageSeverity::Info) + 1;
	this->UpdateGraphNode();

	bPreviewHereIndicatorHovered = false;
	
	SetIsEditable(IsEnabled());

	Controller->OnModified().AddSP(this, &SRigVMGraphNode::HandleModifiedEvent);

	UpdatePinTreeView();

	const FSlateBrush* ImageBrush = FSlateIcon(TEXT("RigVMEditorStyle"), TEXT("RigVM.Bug.Dot")).GetIcon();

	if (const URigVMEdGraphSchema* Schema = Cast<URigVMEdGraphSchema>(EdGraphNode->GetSchema()))
	{
		if (Schema->SupportsPreviewFromHereOnNode(EdGraphNode))
		{
			PreviewHereIndicatorWidget =
				SNew(SButton)
				.Visibility(EVisibility::Visible)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText(LOCTEXT("PreviewHereTooltip", "Run the graph up to this node [F9]"))
				.ContentPadding(0.0f)
				.OnHovered_Lambda([this]
				{
					bPreviewHereIndicatorHovered = true;
				})
				.OnUnhovered_Lambda([this]
				{
					bPreviewHereIndicatorHovered = false;
				})
				.OnClicked_Lambda([this, EdGraphNode]() -> FReply
				{
					if (ModelNode.IsValid() && Blueprint.IsValid())
					{
						if (ModelNode->HasEarlyExitMarker())
						{
							Blueprint->ResetEarlyExitInstruction();
						}
						else
						{
							Blueprint->SetEarlyExitInstruction(ModelNode.Get());
						}
						EdGraphNode->InvalidateNodeTitle();
						return FReply::Handled();
					}
					return FReply::Unhandled();
				})
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Icons.Preview")))
					.DesiredSizeOverride(FVector2D(10, 10))
					.ColorAndOpacity(this, &SRigVMGraphNode::GetPreviewHereColor)
					.Visibility(EVisibility::Visible)
				];
		}
	}
	VisualDebugIndicatorWidget =
		SNew(SImage)
		.Image(ImageBrush)
		.Visibility(EVisibility::Visible);

	OverrideStatusWidget =
		SNew(SOverrideStatusWidget)
		.Visibility(EVisibility::Visible)
		.MenuContent(this, &SRigVMGraphNode::OnOverrideWidgetMenu);

	static const FSlateColorBrush WhiteBrush(FLinearColor::White);
	
	SAssignNew(InstructionCountTextBlockWidget, STextBlock)
	.Margin(FMargin(2.0f, 2.0f, 2.0f, 1.0f))
	.Text(this, &SRigVMGraphNode::GetInstructionCountText)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.ColorAndOpacity(FLinearColor::White)
	.ShadowColorAndOpacity(FLinearColor(0.1f, 0.1f, 0.1f, 1.f))
	.Visibility(EVisibility::Visible)
	.ToolTipText(LOCTEXT("NodeHitCountToolTip", "This number represents the number of instructions hit for a node.\nIf the node has auxiliary instructions (such as Copies) you'll first see the primary instructions followed by the overall instructions in braces.\nFor functions / collapse nodes it represents the sum of all hit instructions of contained nodes.\n\nYou can enable / disable the display of the number in the Class Settings\n(Rig Graph Display Settings -> Show Node Run Counts)"));

	SAssignNew(InstructionDurationTextBlockWidget, STextBlock)
	.Margin(FMargin(2.0f, 2.0f, 2.0f, 1.0f))
	.Text(this, &SRigVMGraphNode::GetInstructionDurationText)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.ColorAndOpacity(FLinearColor::White)
	.ShadowColorAndOpacity(FLinearColor(0.1f, 0.1f, 0.1f, 1.f))
	.Visibility(EVisibility::Visible)
	.ToolTipText(LOCTEXT("NodeDurationToolTip", "This number represents the duration in microseconds for a node.\nFor functions / collapse nodes it represents the accumulated time of contained nodes.\n\nIf you have more than one node selected you'll see also the overall summed up time of the selection.\n\nYou can enable / disable the display of the number in the Class Settings\n(VM Runtime Settings -> Enable Profiling)"));

	EdGraphNode->OnNodeTitleDirtied().AddSP(this, &SRigVMGraphNode::HandleNodeTitleDirtied);
	EdGraphNode->OnNodePinsChanged().AddSP(this, &SRigVMGraphNode::HandleNodePinsChanged);
	EdGraphNode->OnNodeBeginRemoval().AddSP(this, &SRigVMGraphNode::HandleNodeBeginRemoval);

	LastHighDetailSize = FVector2D::ZeroVector;
	
	EdGraphNode->RequestRenameDelegate.BindSP(this, &SRigVMGraphNode::RequestRename);

	if(EdGraphNode->bRenameIsPending)
	{
		TWeakObjectPtr<URigVMEdGraphNode> WeakNode = EdGraphNode;
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakNode]()
		{
			if(WeakNode.IsValid())
			{
				(void)WeakNode->RequestRename();
			}
		});
	}
}

TSharedRef<SWidget> SRigVMGraphNode::CreateNodeContentArea()
{
	return SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SRigVMGraphNode::UseLowDetailNodeContent)
		.LowDetail()
		[
			SNew(SSpacer)
			.Size(this, &SRigVMGraphNode::GetLowDetailDesiredSize)
		]
		.HighDetail()
		[
			SAssignNew(LeftNodeBox, SVerticalBox)
		];
}

bool SRigVMGraphNode::UseLowDetailPinNames() const
{
	if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
	{
		return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail);
	}
	return false;
}

void SRigVMGraphNode::UpdateGraphNode()
{
	if(URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		if(RigGraphNode->DrawAsCompactNode())
		{
			UpdateCompactNode();
			return;
		}
	}
	UpdateStandardNode();
}

void SRigVMGraphNode::UpdateStandardNode()
{
	// call super
	SGraphNode::UpdateGraphNode();
}

void SRigVMGraphNode::UpdateCompactNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	SetupErrorReporting();
	
	TSharedRef<SOverlay> NodeOverlay = SNew(SOverlay);
	
	// add optional node specific widget to the overlay:
	TSharedPtr<SWidget> OverlayWidget = GraphNode->CreateNodeImage();
	if(OverlayWidget.IsValid())
	{
		NodeOverlay->AddSlot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew( SBox )
			.WidthOverride( 70.f )
			.HeightOverride( 70.f )
			[
				OverlayWidget.ToSharedRef()
			]
		];
	}

	TSharedRef<SVerticalBox> InnerVerticalBox =
	SNew(SVerticalBox)
	+SVerticalBox::Slot()
	[
		// NODE CONTENT AREA
		SNew( SOverlay)
		+SOverlay::Slot()
		[
			SNew(SImage)
			.Image( FAppStyle::GetBrush("Graph.VarNode.Body") )
		]
		+ SOverlay::Slot()
		[
			SNew(SImage)
			.Image( FAppStyle::GetBrush("Graph.VarNode.Gloss") )
		]
		+SOverlay::Slot()
		.Padding( FMargin(0,3) )
		[
			NodeOverlay
		]
	];

	NodeOverlay->AddSlot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(/* left */ 0.f, 0.f, 0.f, /* bottom */ 0.f)
	[
		CreateNodeContentArea()
	];

	InnerVerticalBox->AddSlot()
		.AutoHeight()
		.Padding( FMargin(5.0f, 1.0f) )
		[
			ErrorReporting->AsWidget()
		];

	this->GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		InnerVerticalBox
	];

	if (!SWidget::GetToolTip().IsValid())
	{
		TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip( TAttribute< FText >( this, &SGraphNode::GetNodeTooltip ), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
		SetToolTip(DefaultToolTip);
	}

	CreatePinWidgets();
}

TOptional<FSlateColor> SRigVMGraphNode::GetPinTextColor(const SGraphPin* InGraphPin) const
{
	TOptional<FSlateColor> Result = SGraphNode::GetPinTextColor(InGraphPin); 

	const UEdGraphPin* PinObj = InGraphPin->GetPinObj();
	if (PinObj == nullptr)
	{
		return Result;
	}
	
	if (!ModelNode.IsValid())
	{
		return Result;
	}

	const URigVMFunctionInterfaceNode* FunctionInterfaceNode = Cast<URigVMFunctionInterfaceNode>(ModelNode);
	const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(ModelNode);
	if (!FunctionInterfaceNode && !FunctionReferenceNode)
	{
		return Result;
	}

	FString NodeName, PinPath;
	if (!URigVMPin::SplitPinPathAtStart(PinObj->GetName(), NodeName, PinPath))
	{
		return Result;
	}

	if (const URigVMPin* ModelPin = ModelNode->FindPin(PinPath))
	{
		if (FunctionInterfaceNode && !ModelPin->IsLinked() &&
			!FunctionInterfaceNode->IsInterfacePinUsed(ModelPin->GetRootPin()->GetFName()))
		{
			// we are other an output argument without any links to it
			// or an input argument without links and without being bound within the graph
			return FSlateColor::UseSubduedForeground();
		}

		if (FunctionReferenceNode && 
			ModelPin->GetDirection() == ERigVMPinDirection::Input &&
			ModelPin->IsDefinedAsInputVariable() &&
			ModelPin->GetBoundVariableName().IsEmpty())
		{
			return FSlateColor(FLinearColor::Red);
		}
	}

	return Result;
}

int32 SRigVMGraphNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FWidgetStyle WidgetStyle = InWidgetStyle;
	
	if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		WidgetStyle.BlendOpacity(RigVMEdGraphNode->GetFadedOutState());
	}
		
	return SGraphNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, WidgetStyle, bParentEnabled);
}

void SRigVMGraphNode::CreateAddPinButton()
{
	if (LeftNodeBox.IsValid() && ModelNode.IsValid())
	{
		TSharedPtr<SWidget> AddPinButton;
		FMargin AddPinPadding = Settings->GetInputPinPadding();
		AddPinPadding.Top += 2.0f;
		AddPinPadding.Left -= 2.0f;
		EHorizontalAlignment HorizontalAlignment = HAlign_Left;
		
		if (URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode.Get()))
		{
			if (DispatchNode->GetFactory()->GetScriptStruct() == FRigVMDispatch_SwitchEnum::StaticStruct())
			{
				return;
			}
		}
		
		if(ModelNode->IsAggregate() || ModelNode->IsA<URigVMAggregateNode>())
		{
			const bool bInputAggregate = ModelNode->IsInputAggregate();
			AddPinButton = AddPinButtonContent(
			   LOCTEXT("RigVMGraphNodeAggregateNodeAddPinButton", "Add pin"),
			   bInputAggregate ? 
				LOCTEXT("RigVMGraphNodeAggregateNodeAddInputPinButton_Tooltip", "Adds an input pin to the node") :
				LOCTEXT("RigVMGraphNodeAggregateNodeAddOutputPinButton_Tooltip", "Adds an output pin to the node"),
			   !bInputAggregate);
		
			AddPinPadding = bInputAggregate ? Settings->GetInputPinPadding() : Settings->GetOutputPinPadding();
			AddPinPadding.Top += 2.0f;
			AddPinPadding.Left -= bInputAggregate ? 2.f : 0.f;
			AddPinPadding.Right -= bInputAggregate ? 0.f : 2.f;

			HorizontalAlignment = bInputAggregate ? HAlign_Left : HAlign_Right;
		}

		else if(ModelNode->GetPins().ContainsByPredicate([](const URigVMPin* Pin) { return Pin->IsFixedSizeArray(); }))
		{
			AddPinButton = AddPinButtonContent(
			   LOCTEXT("RigVMGraphNodeFixedArrayAddPinButton", "Add pin"),
				LOCTEXT("RigVMGraphNodeFixedArrayAddPinButton_Tooltip", "Adds an input pin to the node"),
			   false);
		}

		if(AddPinButton.IsValid())
		{
			LeftNodeBox->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.HAlign(HorizontalAlignment)
				.Padding(AddPinPadding)
				[
					AddPinButton.ToSharedRef()
				];
		}
	}
}

FReply SRigVMGraphNode::OnAddPin()
{
	if (ModelNode.IsValid())
	{
		if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
		{
			if (ModelNode->IsA<URigVMAggregateNode>() || ModelNode->IsAggregate())
			{
				RigVMEdGraphNode->HandleAddAggregateElement(ModelNode->GetNodePath());
			}
			else
			{
				// we assume the model node has a fixed size array pin
				for(URigVMPin* Pin : ModelNode->GetPins())
				{
					if(Pin->IsFixedSizeArray())
					{
						RigVMEdGraphNode->HandleAddArrayElement(Pin->GetPinPath());
						break;
					}
				}
			}
		}
	}
	return FReply::Handled();
}

void SRigVMGraphNode::CreateWorkflowWidgets()
{
	if (!LeftNodeBox.IsValid() || !ModelNode.IsValid())
	{
		return;
	}

	URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelNode.Get());
	if(UnitNode == nullptr)
	{
		return;
	}

	URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(GraphNode);
	if(EdGraphNode == nullptr)
	{
		return;
	}

	URigVMController* Controller = EdGraphNode->GetController();
	if(Controller == nullptr)
	{
		return;
	}

	TArray<FRigVMUserWorkflow> Workflows = UnitNode->GetSupportedWorkflows(ERigVMUserWorkflowType::NodeContextButton, UnitNode);
	for(const FRigVMUserWorkflow& Workflow : Workflows)
	{
		TSharedRef<SButton> WorkflowButton = StaticCastSharedRef<SButton>(AddPinButtonContent(
			FText::FromString(Workflow.GetTitle()),
			FText::FromString(Workflow.GetTooltip())));

		WorkflowButton->SetOnClicked(FOnClicked::CreateLambda([UnitNode, Controller, Workflow]()
		{
			URigVMUserWorkflowOptions* Options = Controller->MakeOptionsForWorkflow(UnitNode, Workflow);
			Workflow.Perform(Options, Controller);
			return FReply::Handled();
		}));

		WorkflowButton->SetToolTipText(FText::FromString(Workflow.GetTooltip()));

		FMargin Padding = Settings->GetInputPinPadding();
		Padding.Top += 2.0f;
		Padding.Left -= 2.0f;

		LeftNodeBox->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(Padding)
			[
				WorkflowButton
			];
	}
}

bool SRigVMGraphNode::UseLowDetailNodeContent() const
{
	if(LastHighDetailSize.IsNearlyZero())
	{
		return false;
	}
	
	if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
	{
		return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowestDetail);
	}
	return false;
}

FVector2D SRigVMGraphNode::GetLowDetailDesiredSize() const
{
	return LastHighDetailSize;
}

void SRigVMGraphNode::EndUserInteraction() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	if (GraphNode)
	{
		if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
		{
			RigSchema->EndGraphNodeInteraction(GraphNode);
		}
	}

	SGraphNode::EndUserInteraction();
}

void SRigVMGraphNode::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (!NodeFilter.Find(SharedThis(this)))
	{
		if (GraphNode && !RequiresSecondPassLayout())
		{
			if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
			{
				RigSchema->SetNodePosition(GraphNode, FVector2D(NewPosition), false);
			}
		}
	}
}

void SRigVMGraphNode::AddPin(const TSharedRef<SGraphPin>& PinToAdd) 
{
	if(ModelNode.IsValid())
	{
		const UEdGraphPin* EdPinObj = PinToAdd->GetPinObj();

		// Customize the look for pins with injected nodes
		FString NodeName, PinPath;
		if (URigVMPin::SplitPinPathAtStart(EdPinObj->GetName(), NodeName, PinPath))
		{
			if (const URigVMPin* ModelPin = ModelNode->FindPin(PinPath))
			{
				if (ModelPin->HasInjectedUnitNodes())
				{
					PinToAdd->SetCustomPinIcon(CachedImg_CR_Pin_Connected, CachedImg_CR_Pin_Disconnected);
				}

				const FText ToolTipText = ModelPin->GetToolTipText();
				TWeakObjectPtr<const URigVMPin> WeakPin = ModelPin;
				PinToAdd->SetToolTip(
					FSlateApplicationBase::Get().MakeToolTip(
						TAttribute<FText>::CreateLambda([ToolTipText, WeakPin]() -> FText
						{
							if(WeakPin.IsValid())
							{
								return WeakPin->GetToolTipText();
							}
							return ToolTipText;
						})
					)
				);

				// If the pin belongs to a template node that does not own an argument for that pin, make it transparent
				if (!ModelPin->IsExecuteContext())
				{
					if (const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelPin->GetNode()))
					{
						if (const FRigVMTemplate* Template = TemplateNode->GetTemplate())
						{
							const URigVMPin* RootPin = ModelPin->GetRootPin();
							FLinearColor PinColorAndOpacity = PinToAdd->GetColorAndOpacity();

							FRigVMDispatchContext DispatchContext;
							if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
							{
								DispatchContext = DispatchNode->GetDispatchContext();
							}

							if (Template->FindArgument(RootPin->GetFName()) == nullptr && Template->FindExecuteArgument(RootPin->GetFName(), DispatchContext) == nullptr)
							{
								PinColorAndOpacity.A = 0.2f;
							}
							else
							{
								PinColorAndOpacity.A = 1.0f;
							}
							PinToAdd->SetColorAndOpacity(PinColorAndOpacity);
						}
					}
				}
			}
		}

		if(!PinsToKeep.Contains(EdPinObj))
		{
			// reformat the pin by
			// 1. taking out the swrapbox widget
			// 2. re-inserting all widgets from the label and value wrap box back in the horizontal box
			TSharedPtr<SHorizontalBox> FullPinHorizontalRowWidget = PinToAdd->GetFullPinHorizontalRowWidget().Pin();
			TSharedPtr<SWrapBox> LabelAndValueWidget = PinToAdd->GetLabelAndValue();
			if(FullPinHorizontalRowWidget.IsValid() && LabelAndValueWidget.IsValid())
			{
				int32 LabelAndValueWidgetIndex = INDEX_NONE;
				for(int32 ChildIndex = 0; ChildIndex < FullPinHorizontalRowWidget->GetChildren()->Num(); ChildIndex++)
				{
					TSharedRef<SWidget> ChildWidget = FullPinHorizontalRowWidget->GetChildren()->GetChildAt(ChildIndex);
					if(ChildWidget == LabelAndValueWidget)
					{
						LabelAndValueWidgetIndex = ChildIndex;
						break;
					}
				}
				check(LabelAndValueWidgetIndex != INDEX_NONE);
				
				FullPinHorizontalRowWidget->RemoveSlot(LabelAndValueWidget.ToSharedRef());
				
				for(int32 ChildIndex = 0; ChildIndex < LabelAndValueWidget->GetChildren()->Num(); ChildIndex++)
				{
					TSharedRef<SWidget> ChildWidget = LabelAndValueWidget->GetChildren()->GetChildAt(ChildIndex);
					if(ChildWidget != SNullWidget::NullWidget)
					{
						ChildWidget->AssignParentWidget(FullPinHorizontalRowWidget.ToSharedRef());
						
						FullPinHorizontalRowWidget->InsertSlot(LabelAndValueWidgetIndex + ChildIndex)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						.Padding(EdPinObj->Direction == EGPD_Input ? 0.f : 2.f, 0.f, EdPinObj->Direction == EGPD_Input ? 2.f : 0.f, 0.f)
						.AutoWidth()
						[
							ChildWidget
						];
					}
				}
			}

			PinToAdd->SetOwner(SharedThis(this));
		}
		
		if(EdPinObj->Direction == EGPD_Input)
		{
			InputPins.Add(PinToAdd);
		}
		else
		{
			OutputPins.Add(PinToAdd);
		}

		const bool bPinDragDropEnabled = PinDragDropEnabledAttribute.Get(true);
		PinToAdd->EnableDragAndDrop(bPinDragDropEnabled);
	}
}

void SRigVMGraphNode::CreateStandardPinWidget(UEdGraphPin* CurPin)
{
	bool bShowPin = true;
	bool bIsFixedArray = false;
	bool bIsInputVariable = false;
	FString CPPType;
	FString BoundVariableName;
	if(const URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		if(const URigVMPin* ModelPin = RigGraphNode->FindModelPinFromGraphPin(CurPin))
		{
			bShowPin =
				ModelPin->GetDirection() == ERigVMPinDirection::Visible ||
				ModelPin->GetDirection() == ERigVMPinDirection::Input ||
				ModelPin->GetDirection() == ERigVMPinDirection::Output ||
				ModelPin->GetDirection() == ERigVMPinDirection::IO;
			
			CPPType = ModelPin->GetCPPType();
			BoundVariableName = ModelPin->GetBoundVariableName();
			bIsFixedArray = ModelPin->IsFixedSizeArray();
			bIsInputVariable = ModelPin->IsDefinedAsInputVariable();
		}
	}
	
	if (bShowPin)
	{
		// Do we have this pin in our list of pins to keep?
		TSharedPtr<SGraphPin> NewPin;
		const TSharedRef<SGraphPin>* RecycledPinPtr = PinsToKeep.Find(CurPin);
		if (RecycledPinPtr)
		{
			const TSharedRef<SGraphPin>& RecycledPin = *RecycledPinPtr;
			const TSharedPtr<FPinInfoMetaData> PinInfoMetaData = RecycledPin->GetMetaData<FPinInfoMetaData>();
			if(PinInfoMetaData.IsValid())
			{
				if(PinInfoMetaData->CPPType == CPPType &&
					PinInfoMetaData->BoundVariableName == BoundVariableName &&
					PinInfoMetaData->bIsInputVariable == bIsInputVariable)
				{
					NewPin = RecycledPin;
				}
			}
		}

		if(!NewPin.IsValid())
		{
			if(RecycledPinPtr)
			{
				(*RecycledPinPtr)->InvalidateGraphData();
			}
			NewPin = CreatePinWidget(CurPin);
			NewPin->AddMetadata(MakeShared<FPinInfoMetaData>(CPPType, BoundVariableName, bIsInputVariable));
			check(NewPin.IsValid());
			PinsToKeep.Remove(CurPin);

			if(bIsFixedArray)
			{
				if(TSharedPtr<SWidget> PinImage = NewPin->GetPinImageWidget())
				{
					PinImage->SetVisibility(EVisibility::Collapsed);
				}
			}
		}
		
		if(const URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
		{
			if(RigGraphNode->DrawAsCompactNode())
			{
				NewPin->SetShowLabel(false);
			}
		}

		AddPin(NewPin.ToSharedRef());
	}
}

const FSlateBrush * SRigVMGraphNode::GetNodeBodyBrush() const
{
	static const FSlateBrush* Brush = FAppStyle::GetBrush("Graph.Node.Body");
	static const FSlateBrush* TintedBrush = FAppStyle::GetBrush("Graph.Node.TintedBody");
	
	if(const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GetNodeObj()))
	{
		if(RigNode->ProfilingColor.IsSet() ||
			RigNode->HighlightingColor.IsSet())
		{
			return TintedBrush;
		}
	}
	return Brush;
}

FReply SRigVMGraphNode::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = SGraphNode::OnMouseButtonDown(MyGeometry, MouseEvent);

	if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		if (URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(RigNode->GetGraph()))
		{
			RigGraph->OnGraphNodeClicked.Broadcast(RigNode, MyGeometry, MouseEvent);
		}
	}

	return Reply;
}

FReply SRigVMGraphNode::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!InMouseEvent.GetModifierKeys().AnyModifiersDown())
	{
		if (ModelNode.IsValid())
		{
			if(Blueprint.IsValid())
			{
				Blueprint->BroadcastNodeDoubleClicked(ModelNode.Get());
				return FReply::Handled();
			}
		}
	}
	return SGraphNode::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

EVisibility SRigVMGraphNode::GetTitleVisibility() const
{
	return UseLowDetailNodeTitles() ? EVisibility::Hidden : EVisibility::Visible;
}

EVisibility SRigVMGraphNode::GetArrayPlusButtonVisibility(URigVMPin* InModelPin) const
{
	if(InModelPin)
	{
		if (InModelPin->IsDefinedAsInputVariable())
		{
			return EVisibility::Hidden;
		}
		if (Cast<URigVMFunctionReturnNode>(InModelPin->GetNode()))
		{
			return EVisibility::Hidden;
		}
		
		if(InModelPin->GetSourceLinks().Num() == 0)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Hidden;
}

TSharedRef<SWidget> SRigVMGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle)
{
	NodeTitle = InNodeTitle;

	TSharedRef<SWidget> WidgetRef = SGraphNode::CreateTitleWidget(NodeTitle);
	WidgetRef->SetVisibility(MakeAttributeSP(this, &SRigVMGraphNode::GetTitleVisibility));
	if (NodeTitle.IsValid())
	{
		NodeTitle->SetVisibility(MakeAttributeSP(this, &SRigVMGraphNode::GetTitleVisibility));
	}

	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(0.0f)
		[
			WidgetRef
		];
}

FText SRigVMGraphNode::GetPinLabel(TWeakPtr<SGraphPin> GraphPin) const
{
	if(GraphPin.IsValid())
	{
		if (GraphNode)
		{
			return GraphNode->GetPinDisplayName(GraphPin.Pin()->GetPinObj());
		}
	}

	return FText();
}

TOptional<FSlateColor> SRigVMGraphNode::GetHighlightColor(const SGraphPin* InGraphPin) const
{
	if(IsCategoryPin(InGraphPin))
	{
		return FSlateColor(EStyleColor::Hover);
	}
	return SGraphNode::GetHighlightColor(InGraphPin);
}

FSlateColor SRigVMGraphNode::GetVariableLabelTextColor(
	TWeakObjectPtr<URigVMFunctionReferenceNode> FunctionReferenceNode, FName InVariableName) const
{
	if(FunctionReferenceNode.IsValid())
	{
		if(FunctionReferenceNode->GetOuterVariableName(InVariableName).IsNone())
		{
			return FLinearColor::Red;
		}
	}
	return FLinearColor::White;
}

FText SRigVMGraphNode::GetVariableLabelTooltipText(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint,
	FName InVariableName) const
{
	if(InBlueprint.IsValid())
	{
		for(const FRigVMGraphVariableDescription& Variable : InBlueprint->GetAssetVariables())
		{
			if(Variable.Name == InVariableName)
			{
				FString Message = FString::Printf(TEXT("Variable from %s"), *InBlueprint->GetObject()->GetPathName()); 
				if(Variable.Tooltip.IsEmpty())
				{
					const FString Tooltip = Variable.Tooltip.ToString();
					Message = FString::Printf(TEXT("%s\n%s"), *Message, *Tooltip);
				}
				return FText::FromString(Message);
			}
		}
	}
	return FText();
}

FReply SRigVMGraphNode::HandleAddArrayElement(FString InModelPinPath)
{
	if(!InModelPinPath.IsEmpty())
	{
		if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
		{
			RigVMEdGraphNode->HandleAddArrayElement(InModelPinPath);
		}
	}
	return FReply::Handled();
}

void SRigVMGraphNode::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	FKismetNodeInfoContext* K2Context = (FKismetNodeInfoContext*)Context;

	const FLinearColor LatentBubbleColor(1.f, 0.5f, 0.25f);
	const FLinearColor PinnedWatchColor(0.35f, 0.25f, 0.25f);

	URigVMHost* ActiveObject = Cast<URigVMHost>(K2Context->ActiveObjectBeingDebugged);
	URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode);
	FRigVMEditorAssetInterfacePtr RigAsset = K2Context->SourceBlueprint;

	if (!RigAsset && RigGraphNode)
	{
		RigAsset = RigGraphNode->GetAsset();
	}

	if (!ActiveObject && RigAsset)
	{
		ActiveObject = RigAsset->GetDebuggedRigVMHost();
	}

	// Display any pending latent actions
	if (ActiveObject && RigAsset && RigGraphNode)
	{
		// Display pinned watches
		if (RigAsset->NodeContainsWatchedPins(GraphNode))
		{
			const UEdGraphSchema* Schema = GraphNode->GetSchema();

			FString PinnedWatchText;
			int32 ValidWatchCount = 0;
			int32 InvalidWatchCount = 0;
			for (int32 PinIndex = 0; PinIndex < GraphNode->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* WatchPin = GraphNode->Pins[PinIndex];
				if (RigAsset->IsPinBeingWatched(WatchPin))
				{
					if (URigVMPin* ModelPin = RigGraphNode->GetModel()->FindPin(WatchPin->GetName()))
					{
						if (ValidWatchCount > 0)
						{
							PinnedWatchText += TEXT("\n");
						}

						FString PinName = Schema->GetPinDisplayName(WatchPin).ToString();
						PinName += TEXT(" (");
						PinName += UEdGraphSchema_K2::TypeToText(WatchPin->PinType).ToString();
						PinName += TEXT(")");

						TArray<FString> DefaultValues;
					
						if(ModelPin->GetDirection() == ERigVMPinDirection::Input ||
							ModelPin->GetDirection() == ERigVMPinDirection::Visible)
						{
							if(ModelPin->GetSourceLinks(true).Num() == 0)
							{
								const FString DefaultValue = ModelPin->GetDefaultValue();
								if(ModelPin->IsArray())
								{
									DefaultValues = URigVMPin::SplitDefaultValue(DefaultValue);
								}
								else
								{
									DefaultValues.Add(DefaultValue);
								}
							}
						}

						if(DefaultValues.IsEmpty() && ModelNode.IsValid())
						{
							// calling on the impl here directly to avoid the ensure / exception around functions
							const FString PinHash = URigVMCompiler::GetPinHashImpl(ModelPin, nullptr);
							
							if (const FRigVMOperand* PinOperand = RigAsset->PinToOperandMap.Find(PinHash))
							{
								if (const FRigVMOperand* DebugOperand = ActiveObject->GetDebugOperandForOperand(*PinOperand))
								{
									FRigVMMemoryStorageStruct* Memory = ActiveObject->GetDebugMemory();
									// We mark PPF_ExternalEditor so that default values are also printed
									const FString DebugValue = Memory->GetDataAsStringSafe(DebugOperand->GetRegisterIndex(), PPF_ExternalEditor | STRUCT_ExportTextItemNative);
									if(!DebugValue.IsEmpty())
									{
										DefaultValues = URigVMPin::SplitDefaultValue(DebugValue);
									}
								}
							}
						}

						FString WatchText;
						if (DefaultValues.Num() == 1)
						{
							// Fixing the order of values in the rotator to match the order in the pins (x, y, z)
							if (ModelPin->GetCPPType() == TEXT("FRotator"))
							{
								TArray<FString> Values;
								FString TrimmedText = DefaultValues[0].LeftChop(1).RightChop(1); //Remove ()
								TrimmedText.ParseIntoArray(Values, TEXT(","));
								if (Values.Num() == 3)
								{
									Values.Swap(0, 1);
									Values.Swap(0, 2);
								}
								WatchText = FString(TEXT("(")) + FString::Join(Values, TEXT(",")) + FString(TEXT(")"));
								UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
								for (TFieldIterator<FProperty> It(RotatorStruct); It; ++It)
								{
									FName PropertyName = It->GetFName();
									WatchText.ReplaceInline(*PropertyName.ToString(), *It->GetDisplayNameText().ToString());
								}
							}
							else
							{								
								WatchText = DefaultValues[0];
							}
						}
						else if (DefaultValues.Num() > 1)
						{
							// todo: Fix order of values in rotators within other structures
							WatchText = FString::Printf(TEXT("%s"), *FString::Join(DefaultValues, TEXT("\n")));
						}

						if (!WatchText.IsEmpty())
						{
							PinnedWatchText += FText::Format(LOCTEXT("WatchingAndValidFmt", "{0}\n\t{1}"), FText::FromString(PinName), FText::FromString(WatchText)).ToString();//@TODO: Print out object being debugged name?
						}
						else
						{
							PinnedWatchText += FText::Format(LOCTEXT("InvalidPropertyFmt", "No watch found for {0}"), Schema->GetPinDisplayName(WatchPin)).ToString();//@TODO: Print out object being debugged name?
							InvalidWatchCount++;
						}
						ValidWatchCount++;
					}
				}
			}

			if (ValidWatchCount)
			{
				if (InvalidWatchCount && ModelNode.IsValid())
				{
					if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
					{
						const int32 Count = ModelNode->GetInstructionVisitedCount(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
						if(Count == 0)
						{
							PinnedWatchText = FString::Printf(TEXT("Node is not running - wrong event?\n%s"), *PinnedWatchText);
						}
					}
				}

				new (Popups) FGraphInformationPopupInfo(NULL, PinnedWatchColor, PinnedWatchText);
			}
		}
	}
}

TArray<FOverlayWidgetInfo> SRigVMGraphNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets = SGraphNode::GetOverlayWidgets(bSelected, WidgetSize);

	if (ModelNode.IsValid())
	{
		float HorizontalOffset = 0;

		if (ModelNode->IsMutable() && PreviewHereIndicatorWidget.IsValid())
		{
			const FVector2f ImageSize = PreviewHereIndicatorWidget->GetDesiredSize();

			FOverlayWidgetInfo Info;
			Info.OverlayOffset = FVector2f(WidgetSize.X - ImageSize.X - 6.f - HorizontalOffset, 8.f);
			HorizontalOffset += ImageSize.X + 6.f;
			Info.Widget = PreviewHereIndicatorWidget;

			Widgets.Add(Info);
		}
		
		if(CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
		{
			EOverrideWidgetStatus::Type Status = GetOverrideStatus();
			OverrideStatusWidget->SetStatus(Status);
			
			const FVector2f ImageSize = OverrideStatusWidget->GetDesiredSize();

			FOverlayWidgetInfo Info;
			Info.OverlayOffset = FVector2f(WidgetSize.X - ImageSize.X - 6.f - HorizontalOffset, 6.f);
			HorizontalOffset += ImageSize.X + 6.f;
			Info.Widget = OverrideStatusWidget;

			Widgets.Add(Info);
		}

		bool bSetColor = false;
		FLinearColor Color = FLinearColor::Black;
		int32 PreviousNumWidgets = Widgets.Num();
		VisualDebugIndicatorWidget->SetColorAndOpacity(Color);

		for (URigVMPin* ModelPin : ModelNode->GetPins())
		{
			if (ModelPin->HasInjectedUnitNodes())
			{
				for (URigVMInjectionInfo* Injection : ModelPin->GetInjectedNodes())
				{
					if (URigVMUnitNode* VisualDebugNode = Cast<URigVMUnitNode>(Injection->Node))
					{
						FString TemplateName;
					   if (VisualDebugNode->GetScriptStruct()->GetStringMetaDataHierarchical(FRigVMRegistry::TemplateNameMetaName, &TemplateName))
					   {
						   if (TemplateName == TEXT("VisualDebug"))
						   {
							   if (!bSetColor)
							   {
								   if (VisualDebugNode->FindPin(TEXT("bEnabled"))->GetDefaultValue() == TEXT("True"))
								   {
									   if (URigVMPin* ColorPin = VisualDebugNode->FindPin(TEXT("Color")))
									   {
										   TBaseStructure<FLinearColor>::Get()->ImportText(*ColorPin->GetDefaultValue(), &Color, nullptr, PPF_None, nullptr, TBaseStructure<FLinearColor>::Get()->GetName());
									   }
									   else
									   {
										   Color = FLinearColor::White;
									   }

									   VisualDebugIndicatorWidget->SetColorAndOpacity(Color);
									   bSetColor = true;
								   }
							   }

							   if (Widgets.Num() == PreviousNumWidgets)
							   {
								   const FVector2f ImageSize = VisualDebugIndicatorWidget->GetDesiredSize();

								   FOverlayWidgetInfo Info;
								   Info.OverlayOffset = FVector2f(WidgetSize.X - ImageSize.X - 6.f - HorizontalOffset, 6.f);
							   	   HorizontalOffset += ImageSize.X + 6.f;
								   Info.Widget = VisualDebugIndicatorWidget;

								   Widgets.Add(Info);
							   }
						   }
					   }
					}
				}
			}
		}

		if(Blueprint.IsValid())
		{
			const bool bShowInstructionIndex = Blueprint->GetRigGraphDisplaySettings().bShowNodeInstructionIndex;
			const bool bShowNodeCounts = Blueprint->GetRigGraphDisplaySettings().bShowNodeRunCounts;
			const bool bEnableProfiling = Blueprint->GetVMRuntimeSettings().bEnableProfiling;
			
			if(bShowNodeCounts || bShowInstructionIndex || bEnableProfiling)
			{
				if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
				{
					if(bShowNodeCounts || bShowInstructionIndex)
					{
						const int32 Count = ModelNode->GetInstructionVisitedCount(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
						if((Count > Blueprint->GetRigGraphDisplaySettings().NodeRunLowerBound) || bShowInstructionIndex)
						{
							const float VOffset = bSelected ? -2.f : 2.f;
							const FVector2f TextSize = InstructionCountTextBlockWidget->GetDesiredSize();
							FOverlayWidgetInfo Info;
							Info.OverlayOffset = FVector2f(WidgetSize.X - TextSize.X - 8.0f, VOffset - TextSize.Y);
							Info.Widget = InstructionCountTextBlockWidget;
							Widgets.Add(Info);
						}
					}

					if(bEnableProfiling)
					{
						const double MicroSeconds = ModelNode->GetInstructionMicroSeconds(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
						if(MicroSeconds >= 0.0)
						{
							const float VOffset = bSelected ? -2.f : 2.f;
							const FVector2f TextSize = InstructionDurationTextBlockWidget->GetDesiredSize();
							FOverlayWidgetInfo Info;
							Info.OverlayOffset = FVector2f(8.f, VOffset - TextSize.Y);
							Info.Widget = InstructionDurationTextBlockWidget;
							Widgets.Add(Info);
						}
					}
					
				}
			}
		}
	}

	return Widgets;
}

void SRigVMGraphNode::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	SGraphNode::OnFocusLost(InFocusEvent);
	CancelRename();
}

void SRigVMGraphNode::RefreshErrorInfo()
{
	if (GraphNode)
	{
		// if the node has no further errors, check for array reference issues
		if(const URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
		{
			if(!GraphNode->bHasCompilerMessage &&
				(GraphNode->ErrorType == int32(EMessageSeverity::Info) + 1))
			{
				if(const URigVMNode* RigModelNode = RigGraphNode->GetModelNode())
				{
					bool bShowCopyWarning = RigModelNode->IsA<UDEPRECATED_RigVMIfNode>() || RigModelNode->IsA<UDEPRECATED_RigVMSelectNode>();
					if(!bShowCopyWarning)
					{
						if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(RigModelNode))
						{
							if(const UScriptStruct* FactoryStruct = DispatchNode->GetScriptStruct())
							{
								if(FactoryStruct == FRigVMDispatch_If::StaticStruct() ||
									FactoryStruct == FRigVMDispatch_SelectInt32::StaticStruct())
								{
									bShowCopyWarning = true;
								}
							}
						}
					}
					if(bShowCopyWarning)
					{
						for(const URigVMPin* Pin : RigModelNode->GetPins())
						{
							if(Pin->IsArray() && !Pin->IsFixedSizeArray())
							{
								GraphNode->bHasCompilerMessage = true;
								if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
								{
									static const FString ArrayWarning = TEXT("This node creates a copy of the array.\nThis may cause side effects.");
									RigVMEdGraphNode->SetErrorInfo(EMessageSeverity::Info, ArrayWarning);
								}
								
								break;
							}
						}
					}
				}
			}
		}

		if (NodeErrorType != GraphNode->ErrorType)
		{
			SGraphNode::RefreshErrorInfo();
			NodeErrorType = GraphNode->ErrorType;
		}
	}
}

void SRigVMGraphNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!ModelNode.IsValid())
	{
		return;
	}
	
	SGraphNode::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (GraphNode)
	{
		GraphNode->NodeWidth = (int32)AllottedGeometry.Size.X;
		GraphNode->NodeHeight = (int32)AllottedGeometry.Size.Y;
		RefreshErrorInfo();

		// These will be deleted on the next tick.
		for (UEdGraphPin *PinToDelete: PinsToDelete)
		{
			PinToDelete->MarkAsGarbage();
		}
		PinsToDelete.Reset();

		if (URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
		{
			RigVMEdGraphNode->UpdateVisualSettings();
		}
	}

	if((!UseLowDetailNodeContent()) && (LeftNodeBox != nullptr))
	{
		LastHighDetailSize = LeftNodeBox->GetTickSpaceGeometry().Size;
	}
}

void SRigVMGraphNode::HandleNodeTitleDirtied()
{
	if (NodeTitle.IsValid())
	{
		NodeTitle->MarkDirty();
	}
}

void SRigVMGraphNode::HandleNodePinsChanged()
{
	URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode);
	if(RigVMEdGraphNode == nullptr)
	{
		return;
	}

	// Collect graph pins to delete. We do this here because this widget is the only entity
	// that's aware of the lifetime requirements for the graph pins (SGraphPanel uses Slate 
	// timers to trigger a delete, which makes deleting them from a non-widget setting). 
	TSet<UEdGraphPin *> LocalPinsToDelete;
	LocalPinsToDelete.Reserve(InputPins.Num() + OutputPins.Num());
	
	for (const TSharedRef<SGraphPin>& GraphPin: InputPins)
	{
		if (UEdGraphPin* PinObj = GraphPin->GetPinObj())
		{
			LocalPinsToDelete.Add(PinObj);
		}
	}
	for (const TSharedRef<SGraphPin>& GraphPin: OutputPins)
	{
		if (UEdGraphPin* PinObj = GraphPin->GetPinObj())
		{
			LocalPinsToDelete.Add(PinObj);
		}
	}

	check(PinsToKeep.IsEmpty());

	for (const UEdGraphPin* LivePin: RigVMEdGraphNode->Pins)
	{
		const FString PinPath = LivePin->GetName();
		
		const FPinInfo* PinInfoPtr = PinInfos.FindByPredicate([PinPath](const FPinInfo& PinInfo) -> bool
		{
			return PinInfo.Identifier == PinPath;
		});
		
		if (PinInfoPtr)
		{
			if (LivePin->Direction == EGPD_Input && PinInfoPtr->InputPinWidget.IsValid())
			{
				PinsToKeep.Add(LivePin, PinInfoPtr->InputPinWidget.ToSharedRef());
				LocalPinsToDelete.Remove(PinInfoPtr->InputPinWidget->GetPinObj());
			}
			if (LivePin->Direction == EGPD_Output && PinInfoPtr->OutputPinWidget.IsValid())
			{
				PinsToKeep.Add(LivePin, PinInfoPtr->OutputPinWidget.ToSharedRef());
				LocalPinsToDelete.Remove(PinInfoPtr->OutputPinWidget->GetPinObj());
			}
		}
		LocalPinsToDelete.Remove(LivePin);
	}

	for (const UEdGraphPin* DeletingPin: LocalPinsToDelete)
	{
		const FString PinPath = DeletingPin->GetName();
		
		const FPinInfo* PinInfoPtr = PinInfos.FindByPredicate([PinPath](const FPinInfo& PinInfo) -> bool
		{
			return PinInfo.Identifier == PinPath;
		});
		
		if (PinInfoPtr)
		{
			if (DeletingPin->Direction == EGPD_Input && PinInfoPtr->InputPinWidget.IsValid())
			{
				// Ensure that this pin widget can no longer depend on the soon-to-be-deleted
				// graph pin.
				PinInfoPtr->InputPinWidget->InvalidateGraphData();
			}

			if (DeletingPin->Direction == EGPD_Output && PinInfoPtr->OutputPinWidget.IsValid())
			{
				// Ensure that this pin widget can no longer depend on the soon-to-be-deleted
				// graph pin.
				PinInfoPtr->OutputPinWidget->InvalidateGraphData();
			}
		}
	}

	// only store those pins if not already marked as garbage
	Algo::CopyIf(LocalPinsToDelete, PinsToDelete, [](const UEdGraphPin* PinToDelete)
	{
		return !PinToDelete->IsPendingKill();
	});

	// Reconstruct the pin widgets. This could be done more surgically but will do for now.
	InputPins.Reset();
	OutputPins.Reset();
	PinInfos.Reset();

	CreatePinWidgets();

	// Nix any pins left in this map. They're most likely hidden sub-pins.
	PinsToKeep.Reset();

	UpdatePinTreeView();
}

void SRigVMGraphNode::HandleNodeBeginRemoval()
{
	if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		RigNode->OnNodeTitleDirtied().RemoveAll(this);
		RigNode->OnNodePinsChanged().RemoveAll(this);
		RigNode->OnNodeBeginRemoval().RemoveAll(this);
		
		if(URigVMController* Controller = RigNode->GetController())
		{
			Controller->OnModified().RemoveAll(this);
		}
	}
	
	for (const TSharedRef<SGraphPin>& GraphPin: InputPins)
	{
		GraphPin->SetPinObj(nullptr);
	}
	for (const TSharedRef<SGraphPin>& GraphPin: OutputPins)
	{
		GraphPin->SetPinObj(nullptr);
	}

	InputPins.Reset();
	OutputPins.Reset();
	PinInfos.Reset();
	PinsToKeep.Reset();
	
	InvalidateGraphData();
}

FText SRigVMGraphNode::GetInstructionCountText() const
{
	if(Blueprint.IsValid())
	{
		bool bShowInstructionIndex = Blueprint->GetRigGraphDisplaySettings().bShowNodeInstructionIndex;
		bool bShowNodeRunCount = Blueprint->GetRigGraphDisplaySettings().bShowNodeRunCounts;
		if(bShowInstructionIndex || bShowNodeRunCount)
		{
			if (ModelNode.IsValid())
			{
				if(URigVMHost* DebuggedHost = Cast<URigVMHost>(Blueprint->GetObjectBeingDebugged()))
				{
					int32 RunCount = 0;
					int32 FirstInstructionIndex = INDEX_NONE;
					if(bShowNodeRunCount)
					{
						RunCount = ModelNode->GetInstructionVisitedCount(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM(), FRigVMASTProxy());
						bShowNodeRunCount = RunCount > Blueprint->GetRigGraphDisplaySettings().NodeRunLowerBound;

						// toodoo here we want to differentiate between primary and secondary instructions. (call extern vs copy).
						// const TArray<int32>& Instructions = ModelNode->GetInstructionsForVM(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM());
					}

					if(bShowInstructionIndex)
					{
						const TArray<int32>& Instructions = ModelNode->GetInstructionsForVM(DebuggedHost->GetRigVMExtendedExecuteContext(), DebuggedHost->GetVM());
						bShowInstructionIndex = Instructions.Num() > 0;
						if(bShowInstructionIndex)
						{
							FirstInstructionIndex = Instructions[0];
						}
					}

					if(bShowInstructionIndex || bShowNodeRunCount)
					{
						FText NodeRunCountText, NodeInstructionIndexText;
						if(bShowNodeRunCount)
						{
							NodeRunCountText = FText::FromString(FString::FromInt(RunCount));
							if(!bShowInstructionIndex)
							{
								return NodeRunCountText;
							}
						}

						if(bShowInstructionIndex)
						{
							NodeInstructionIndexText = FText::FromString(FString::FromInt(FirstInstructionIndex));
							if(!bShowNodeRunCount)
							{
								return NodeInstructionIndexText;
							}
						}

						return FText::Format(LOCTEXT("SRigVMGraphNodeCombinedNodeCountText", "{0}: {1}"), NodeInstructionIndexText, NodeRunCountText);
					}
				}
			}
		}
	}

	return FText();
}

FText SRigVMGraphNode::GetInstructionDurationText() const
{
	if(Blueprint.IsValid())
	{
		if(Blueprint->GetVMRuntimeSettings().bEnableProfiling)
		{
			if(const URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
			{
				const double MicroSeconds = RigGraphNode->MicroSeconds.Get(0);
				if(MicroSeconds >= 0)
				{
					if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
					{
						const TArray<UEdGraphNode*> SelectedNodes = MyOwnerPanel->GetSelectedGraphNodes();
						if((SelectedNodes.Num() > 1) && SelectedNodes.Contains(RigGraphNode))
						{
							double OverallMicroSeconds = 0;
							for(const UEdGraphNode* SelectedNode : SelectedNodes)
							{
								if(const URigVMEdGraphNode* SelectedRigGraphNode = Cast<URigVMEdGraphNode>(SelectedNode))
								{
									OverallMicroSeconds += SelectedRigGraphNode->MicroSeconds.Get(0);
								}
							}
							
							return FText::FromString(FString::Printf(TEXT("%.02f µs of %.02f µs"), (float)MicroSeconds, (float)OverallMicroSeconds));
						}
					}

					return FText::FromString(FString::Printf(TEXT("%.02f µs"), (float)MicroSeconds));
				}
			}
		}
	}

	return FText();
}

TSharedRef<SWidget> SRigVMGraphNode::OnOverrideWidgetMenu() const
{
	if(const URigVMNode* Node = ModelNode.Get())
	{
		TSharedPtr<FOverrideStatusWidgetMenuBuilder> MenuBuilder = MakeShared<FOverrideStatusWidgetMenuBuilder>(Node, nullptr /* property path */);
		MenuBuilder->GetStatusAttribute().Set(GetOverrideStatus());
		MenuBuilder->OnAddOverride().BindLambda([this](const FOverrideStatusSubject&) -> FReply
		{
			if(const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
			{
				if(URigVMController* Controller = EdGraphNode->GetController())
				{
					if(Controller->AddOverrideToAllPinsOnNode(GraphNode->GetFName()))
					{
						return FReply::Handled();
					}
				}
			}
			return FReply::Unhandled();
		});
		MenuBuilder->OnClearOverride().BindLambda([this](const FOverrideStatusSubject&) -> FReply
		{
			if(const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
			{
				if(URigVMController* Controller = EdGraphNode->GetController())
				{
					if(Controller->ClearOverrideOnAllPinsOnNode(GraphNode->GetFName()))
					{
						return FReply::Handled();
					}
				}
			}
			return FReply::Unhandled();
		});
		MenuBuilder->OnResetToDefault().BindLambda([this](const FOverrideStatusSubject&) -> FReply
		{
			if(const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
			{
				if(URigVMController* Controller = EdGraphNode->GetController())
				{
					FRigVMDefaultValueTypeGuard _(Controller, ERigVMPinDefaultValueType::KeepValueType, true);
					if(Controller->ResetDefaultValueForAllPinsOnNode(GraphNode->GetFName()))
					{
						return FReply::Handled();
					}
				}
			}
			return FReply::Unhandled();
		});
		MenuBuilder->OnValueDiffersFromDefault().BindLambda([this](const FOverrideStatusSubject&)
		{
			if(const URigVMNode* Node = Cast<URigVMNode>(ModelNode.Get()))
			{
				for(const URigVMPin* Pin : Node->GetPins())
				{
					if(Pin->CanProvideDefaultValue())
					{
						if(!Pin->HasOriginalDefaultValue())
						{
							return true;
						}
					}
				}
			}
			return false;
		});

		if(const TSharedPtr<SWidget> MenuWidget = MenuBuilder->GenerateWidget())
		{
			return MenuWidget.ToSharedRef();
		}
	}
	return SNullWidget::NullWidget;
}

int32 SRigVMGraphNode::GetNodeTopologyVersion() const
{
	if(URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		return RigVMEdGraphNode->GetNodeTopologyVersion();
	}
	return INDEX_NONE;
}

EVisibility SRigVMGraphNode::GetPinVisibility(int32 InPinInfoIndex, bool bAskingForSubPin) const
{
	if(PinInfos.IsValidIndex(InPinInfoIndex))
	{
		if(PinInfos[InPinInfoIndex].bShowOnlySubPins)
		{
			return bAskingForSubPin ? EVisibility::Visible : EVisibility::Collapsed;
		}
		
		const int32 ParentPinIndex = PinInfos[InPinInfoIndex].ParentIndex;
		if(ParentPinIndex != INDEX_NONE)
		{
			const EVisibility ParentPinVisibility = GetPinVisibility(ParentPinIndex, true); 
			if(ParentPinVisibility != EVisibility::Visible)
			{
				return ParentPinVisibility;
			}

			if(!PinInfos[InPinInfoIndex].bShowOnlySubPins)
			{
				if(!PinInfos[ParentPinIndex].bExpanded)
				{
					return EVisibility::Collapsed;
				}
			}
		}
	}

	return EVisibility::Visible;
}

const FSlateBrush* SRigVMGraphNode::GetExpanderImage(int32 InPinInfoIndex, bool bLeft, bool bHovered) const
{
	static const FSlateBrush* ExpandedHoveredLeftBrush = nullptr;
	static const FSlateBrush* ExpandedHoveredRightBrush = nullptr;
	static const FSlateBrush* ExpandedLeftBrush = nullptr;
	static const FSlateBrush* ExpandedRightBrush = nullptr;
	static const FSlateBrush* CollapsedHoveredLeftBrush = nullptr;
	static const FSlateBrush* CollapsedHoveredRightBrush = nullptr;
	static const FSlateBrush* CollapsedLeftBrush = nullptr;
	static const FSlateBrush* CollapsedRightBrush = nullptr;
	
	if(ExpandedHoveredLeftBrush == nullptr)
	{
		static const TCHAR* ExpandedHoveredLeftName = TEXT("RigVM.Node.PinTree.Arrow_Expanded_Hovered_Left");
		static const TCHAR* ExpandedHoveredRightName = TEXT("RigVM.Node.PinTree.Arrow_Expanded_Hovered_Right");
		static const TCHAR* ExpandedLeftName = TEXT("RigVM.Node.PinTree.Arrow_Expanded_Left");
		static const TCHAR* ExpandedRightName = TEXT("RigVM.Node.PinTree.Arrow_Expanded_Right");
		static const TCHAR* CollapsedHoveredLeftName = TEXT("RigVM.Node.PinTree.Arrow_Collapsed_Hovered_Left");
		static const TCHAR* CollapsedHoveredRightName = TEXT("RigVM.Node.PinTree.Arrow_Collapsed_Hovered_Right");
		static const TCHAR* CollapsedLeftName = TEXT("RigVM.Node.PinTree.Arrow_Collapsed_Left");
		static const TCHAR* CollapsedRightName = TEXT("RigVM.Node.PinTree.Arrow_Collapsed_Right");
		
		ExpandedHoveredLeftBrush = FSlateIcon(TEXT("RigVMEditorStyle"), ExpandedHoveredLeftName).GetIcon();
		ExpandedHoveredRightBrush = FSlateIcon(TEXT("RigVMEditorStyle"), ExpandedHoveredRightName).GetIcon();
		ExpandedLeftBrush = FSlateIcon(TEXT("RigVMEditorStyle"), ExpandedLeftName).GetIcon();
		ExpandedRightBrush = FSlateIcon(TEXT("RigVMEditorStyle"), ExpandedRightName).GetIcon();
		CollapsedHoveredLeftBrush = FSlateIcon(TEXT("RigVMEditorStyle"), CollapsedHoveredLeftName).GetIcon();
		CollapsedHoveredRightBrush = FSlateIcon(TEXT("RigVMEditorStyle"), CollapsedHoveredRightName).GetIcon();
		CollapsedLeftBrush = FSlateIcon(TEXT("RigVMEditorStyle"), CollapsedLeftName).GetIcon();
		CollapsedRightBrush = FSlateIcon(TEXT("RigVMEditorStyle"), CollapsedRightName).GetIcon();
	}

	if(PinInfos.IsValidIndex(InPinInfoIndex))
	{
		if(PinInfos[InPinInfoIndex].bExpanded)
		{
			if(bHovered)
			{
				return bLeft ? ExpandedHoveredLeftBrush : ExpandedHoveredRightBrush;
			}
			return bLeft ? ExpandedLeftBrush : ExpandedRightBrush;
		}
	}

	if(bHovered)
	{
		return bLeft ? CollapsedHoveredLeftBrush : CollapsedHoveredRightBrush;
	}
	return bLeft ? CollapsedLeftBrush : CollapsedRightBrush;
}

FReply SRigVMGraphNode::OnExpanderArrowClicked(int32 InPinInfoIndex)
{
	if(URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		if(URigVMController* Controller = EdGraphNode->GetController())
		{
			if(!PinInfos.IsValidIndex(InPinInfoIndex))
			{
				return FReply::Unhandled();
			}
			
			const FPinInfo& PinInfo = PinInfos[InPinInfoIndex];
			TArray<int32> PinInfoIndicesToModify;
			PinInfoIndicesToModify.Add(PinInfo.Index);

			// with shift clicked we expand recursively
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				for(const FPinInfo& OtherPinInfo : PinInfos)
				{
					if(PinInfoIndicesToModify.Contains(OtherPinInfo.ParentIndex))
					{
						PinInfoIndicesToModify.Emplace(OtherPinInfo.Index);
						PinInfoIndicesToModify.Add(OtherPinInfo.Index);
					}
				}
				Algo::Reverse(PinInfoIndicesToModify);
			}

			FGuardSkipDirtyBlueprintStatus GuardDirtyBlueprintStatus(Blueprint.GetObject(), true);

			Controller->OpenUndoBracket(PinInfo.bExpanded ? TEXT("Collapsing Pin") : TEXT("Expanding Pin"));
			for(const int32& PinInfoIndexToModify : PinInfoIndicesToModify)
			{
				if(PinInfos[PinInfoIndexToModify].bIsCategoryPin)
				{
					Controller->SetPinCategoryExpansion(EdGraphNode->GetFName(), PinInfos[PinInfoIndexToModify].Identifier, !PinInfo.bExpanded, true, true);
				}
				else
				{
					Controller->SetPinExpansion(PinInfos[PinInfoIndexToModify].Identifier, !PinInfo.bExpanded, true, true);
				}
			}
			Controller->CloseUndoBracket();
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

void SRigVMGraphNode::HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph,
	UObject* InSubject)
{
	switch(InNotifType)
	{
		case ERigVMGraphNotifType::PinExpansionChanged:
		{
			if(!ModelNode.IsValid())
			{
				return;
			}
				
			if(URigVMPin* Pin = Cast<URigVMPin>(InSubject))
			{
				if(Pin->GetNode() == ModelNode.Get())
				{
					const FString PinPath = Pin->GetPinPath();
					for(FPinInfo& PinInfo : PinInfos)
					{
						if(PinInfo.Identifier == PinPath)
						{
							PinInfo.bExpanded = Pin->IsExpanded();
							break;
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::PinCategoryExpansionChanged:
		{
			if(!ModelNode.IsValid())
			{
				return;
			}
			
			if(const URigVMNode* ModelNodeFromNotification = Cast<URigVMNode>(InSubject))
			{
				if(ModelNodeFromNotification == ModelNode.Get())
				{
					const FString PinCategory = ModelNodeFromNotification->GetLastAffectedPinCategory();
					for(FPinInfo& PinInfo : PinInfos)
					{
						if(PinInfo.Identifier == PinCategory)
						{
							PinInfo.bExpanded = ModelNodeFromNotification->IsPinCategoryExpanded(PinCategory);
							break;
						}
					}
				}
			}
			break;
		}
		case ERigVMGraphNotifType::LibraryTemplateChanged:
		{
			if(!ModelNode.IsValid())
			{
				return;
			}
			
			if (URigVMLibraryNode* LibraryNode = Cast<URigVMLibraryNode>(InSubject))
			{
				if(LibraryNode == ModelNode.Get() ||
					LibraryNode->GetEntryNode() == ModelNode.Get() ||
					LibraryNode->GetReturnNode() == ModelNode.Get())
				{
					HandleNodePinsChanged();
				}
			}
			break;
		}
		default:
		{
			break;
		}			
	}
}

void SRigVMGraphNode::UpdatePinTreeView()
{
	static const float PinWidgetSidePadding = 6.f;
	static const float EmptySidePadding = 60.f;
	static const float TopPadding = 2.f; 
	static const float MaxHeight = 30.f;

	check(GraphNode);

	// remove all existing content in the Left
	LeftNodeBox->ClearChildren();

	URigVMEdGraphNode* RigGraphNode = Cast<URigVMEdGraphNode>(GraphNode);
	URigVMNode* Node = RigGraphNode->GetModelNode();
	const FRigVMTemplate* Template = nullptr;
	if (URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(Node))
	{
		Template = TemplateNode->GetTemplate();
	}

	TMap<UEdGraphPin*, int32> EdGraphPinToInputPin;
	for(int32 InputPinIndex = 0; InputPinIndex < InputPins.Num(); InputPinIndex++)
	{
		EdGraphPinToInputPin.Add(InputPins[InputPinIndex]->GetPinObj(), InputPinIndex);
	}
	TMap<UEdGraphPin*, int32> EdGraphPinToOutputPin;
	for(int32 OutputPinIndex = 0; OutputPinIndex < OutputPins.Num(); OutputPinIndex++)
	{
		EdGraphPinToOutputPin.Add(OutputPins[OutputPinIndex]->GetPinObj(), OutputPinIndex);
	}

	TArray<URigVMPin*> RootModelPins = ModelNode->GetPins();
	// orphaned pins are appended to the end of pin list on each side of the node
	RootModelPins.Append(ModelNode->GetOrphanedPins());

	TArray<TTuple<const URigVMPin*, FString>> ModelPinsOrCategories;

	const bool bSupportSubPins = !RigGraphNode->DrawAsCompactNode();

	// sort model pins
	// a) execute IOs, b) IO pins, c) input / visible pins, d) output pins
	struct Local
	{
		static void VisitPinRecursively(const URigVMPin* InPin, TArray<TTuple<const URigVMPin*, FString>>& OutPinsOrCategories, bool bSupportSubPins)
		{
			const TTuple<const URigVMPin*, FString> PinKey(InPin, FString());
			if(OutPinsOrCategories.Contains(PinKey))
			{
				return;
			}
			
			OutPinsOrCategories.AddUnique(PinKey);
			
			if(!bSupportSubPins)
			{
				return;
			}

			if(InPin->ShouldHideSubPins())
			{
				return;
			}

			if (InPin->GetCPPType() == TEXT("FRotator"))
			{
				const TArray<URigVMPin*>& SubPins = InPin->GetSubPins();
				if (SubPins.Num() == 3)
				{
					VisitPinRecursively(SubPins[2], OutPinsOrCategories, bSupportSubPins);
					VisitPinRecursively(SubPins[0], OutPinsOrCategories, bSupportSubPins);
					VisitPinRecursively(SubPins[1], OutPinsOrCategories, bSupportSubPins);
				}	
			}
			else
			{				
				for (URigVMPin* SubPin : InPin->GetSubPins())
				{
					VisitPinRecursively(SubPin, OutPinsOrCategories, bSupportSubPins);
				}
			}
		}
	};

	const FRigVMNodeLayout NodeLayout = ModelNode->GetNodeLayout();
	
	for(int32 SortPhase = 0; SortPhase < 4; SortPhase++)
	{
		for(URigVMPin* RootPin : RootModelPins)
		{
			switch (SortPhase)
			{
				case 0: // execute IO pins
				{
					if(RootPin->IsExecuteContext() && RootPin->GetDirection() == ERigVMPinDirection::IO)
					{
						Local::VisitPinRecursively(RootPin, ModelPinsOrCategories, bSupportSubPins);
					}
					break;
				}
				case 1: // output pins
				{
					if(RootPin->GetDirection() == ERigVMPinDirection::Output)
					{
						Local::VisitPinRecursively(RootPin, ModelPinsOrCategories, bSupportSubPins);
					}
					break;
				}
				case 2: // IO pins
				{
					if(!RootPin->IsExecuteContext() && RootPin->GetDirection() == ERigVMPinDirection::IO)
					{
						Local::VisitPinRecursively(RootPin, ModelPinsOrCategories, bSupportSubPins);
					}
					break;
				}
				case 3: // input / visible pins
				default:
				{
					// for now we only allow node layouts for input pins
					if(!NodeLayout.IsValid())
					{
						if(RootPin->GetDirection() == ERigVMPinDirection::Input || RootPin->GetDirection() == ERigVMPinDirection::Visible)
						{
							Local::VisitPinRecursively(RootPin, ModelPinsOrCategories, bSupportSubPins);
						}
					}
					break;
				}
			}
		}
	}

	// visit the categories / input / visible pins in the right order
	if(NodeLayout.IsValid())
	{
		for(const FRigVMPinCategory& Category : NodeLayout.Categories)
		{
			if(!Category.IsDefaultCategory())
			{
				ModelPinsOrCategories.AddUnique({nullptr, Category.Path});
			}
			for(const FString& PinPath : Category.Elements)
			{
				if(const URigVMPin* Pin = ModelNode->FindPin(PinPath))
				{
					Local::VisitPinRecursively(Pin, ModelPinsOrCategories, bSupportSubPins);
				}
			}
		}

		// when adding pins based on the node layout
		// we may be skipping over pins which have a connection but are not part of the layout
		const TArray<URigVMLink*> Links = ModelNode->GetLinks();
		for(const URigVMLink* Link : Links)
		{
			const URigVMPin* SourcePin = Link->GetSourcePin();
			const URigVMPin* TargetPin = Link->GetTargetPin();
			if(SourcePin == nullptr || TargetPin == nullptr)
			{
				continue;
			}
			const URigVMPin* PinToShow = SourcePin->GetNode() == ModelNode.Get() ? SourcePin : TargetPin;
			const TTuple<const URigVMPin*, FString> Key = {PinToShow, FString()};
			if(ModelPinsOrCategories.Contains(Key))
			{
				continue;
			}
			Local::VisitPinRecursively(PinToShow, ModelPinsOrCategories, bSupportSubPins);
		}
	}

	const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(RigGraphNode->GetSchema());

	FRigVMDispatchContext DispatchContext;
	if(const URigVMDispatchNode* DispatchNode = Cast<URigVMDispatchNode>(ModelNode))
	{
		DispatchContext = DispatchNode->GetDispatchContext();
	}

	TMap<FString, int32> IdentifierToInfoIndex;

	auto AddModelPinOrCategoryPinInfo =
		[this, &IdentifierToInfoIndex, RigSchema, Template, &DispatchContext, RigGraphNode, EdGraphPinToInputPin, EdGraphPinToOutputPin, bSupportSubPins]
	(const URigVMPin* InModelPin, const FString& InCategory)
	{
		const URigVMNode* CurrentModelNode = RigGraphNode->GetModelNode();
		check(CurrentModelNode);
			
		const bool bIsModelPin = InModelPin != nullptr;
		const bool bIsCategoryPin = !InCategory.IsEmpty();
		check(bIsModelPin != bIsCategoryPin);
			
		FPinInfo PinInfo;
		PinInfo.Index = PinInfos.Num();
		PinInfo.ParentIndex = INDEX_NONE;
		PinInfo.bIsCategoryPin = bIsCategoryPin;
		if(bIsModelPin)
		{
			PinInfo.bHasChildren = InModelPin->GetSubPins().Num() > 0;
			PinInfo.bIsContainer = InModelPin->IsArray();
			PinInfo.bIsInputVariable = InModelPin->IsDefinedAsInputVariable();
			PinInfo.bExpanded = InModelPin->IsExpanded();
			PinInfo.Identifier = InModelPin->GetPinPath();
			PinInfo.bShowOnlySubPins = InModelPin->ShouldOnlyShowSubPins();
		}
		else
		{
			PinInfo.bHasChildren = false;
			TArray<FString> CategoriesWithPins = {InCategory};
			CategoriesWithPins.Append(CurrentModelNode->GetSubPinCategories(InCategory, false, true));
			for(const FString& CategoryWithPins : CategoriesWithPins)
			{
				if(!CurrentModelNode->GetPinsForCategory(CategoryWithPins).IsEmpty())
				{
					PinInfo.bHasChildren = true;
					break;
				}
			}
			PinInfo.bIsContainer = false;
			PinInfo.bIsInputVariable = false;
			PinInfo.bExpanded = CurrentModelNode->IsPinCategoryExpanded(InCategory);
			PinInfo.Identifier = InCategory;
			PinInfo.bShowOnlySubPins = false;
			PinInfo.bHideInputWidget = true;
		}
		PinInfo.Depth = 0;
		PinInfo.bAutoHeight = false;

		if(!bSupportSubPins && bIsModelPin)
		{
			PinInfo.bHasChildren = false;
			PinInfo.bIsContainer = false;
		}

		if(bIsModelPin)
		{
			const bool bAskSchemaForEdition = RigSchema && InModelPin->IsStruct() && !InModelPin->IsBoundToVariable();
			PinInfo.bHideInputWidget = (!InModelPin->IsBoundToVariable()) && PinInfo.bIsContainer && !PinInfo.bIsInputVariable;
			if (!PinInfo.bHideInputWidget && !PinInfo.bIsInputVariable)
			{
				if (bAskSchemaForEdition && !PinInfo.bHasChildren)
				{
					const bool bIsStructEditable = RigSchema->IsStructEditable(InModelPin->GetScriptStruct()); 
					PinInfo.bHideInputWidget = !bIsStructEditable;
					PinInfo.bAutoHeight = bIsStructEditable;
				}
				else if(PinInfo.bHasChildren && !InModelPin->IsBoundToVariable())
				{
					PinInfo.bHideInputWidget = true;
				}
			}
		}

		if(bIsModelPin)
		{
			bool bNeedsParentPin = true;
			const FString PinCategory = InModelPin->GetCategory();
			if(!PinCategory.IsEmpty())
			{
				if(PinCategory.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase))
				{
					bNeedsParentPin = false;
				}
				else if(const int32* ParentIndexPtr = IdentifierToInfoIndex.Find(PinCategory))
				{
					PinInfo.ParentIndex = *ParentIndexPtr;
					bNeedsParentPin = false;
				}
			}

			// if the pin is not part of a category, add it under its default parent
			if(bNeedsParentPin)
			{
				if(URigVMPin* ParentPin = InModelPin->GetParentPin())
				{
					const int32* ParentIndexPtr = IdentifierToInfoIndex.Find(ParentPin->GetPinPath());
					if(ParentIndexPtr == nullptr)
					{
						if(InModelPin->GetLinks().IsEmpty())
						{
							// parent pins have to exist for the node to display correctly
							// but allow pins with links to show up
							return;
						}
					}
					else
					{
						PinInfo.ParentIndex = *ParentIndexPtr;
						bNeedsParentPin = false;
					}
				}
			}
		}
		else
		{
			const FString ParentCategory = CurrentModelNode->GetParentPinCategory(InCategory);
			if(!ParentCategory.IsEmpty())
			{
				const int32* ParentIndexPtr = IdentifierToInfoIndex.Find(ParentCategory);
				if(ParentIndexPtr == nullptr)
				{
					return;
				}
				PinInfo.ParentIndex = *ParentIndexPtr;
			}
		}

		if(PinInfo.ParentIndex != INDEX_NONE)
		{
			PinInfo.Depth = PinInfos[PinInfo.ParentIndex].Depth + 1;
			if(PinInfos[PinInfo.ParentIndex].bShowOnlySubPins)
			{
				PinInfo.Depth--;
			}
		}

		TAttribute<EVisibility> PinVisibilityAttribute = TAttribute<EVisibility>::CreateSP(this, &SRigVMGraphNode::GetPinVisibility, PinInfo.Index, false);

		bool bPinWidgetForExpanderLeft = false;
		TSharedPtr<SGraphPin> PinWidgetForExpander;

		bool bPinInfoIsValid = false;
		if(UEdGraphPin* OutputEdGraphPin = RigGraphNode->FindPin(PinInfo.Identifier, EEdGraphPinDirection::EGPD_Output))
		{
			if(const int32* PinIndexPtr = EdGraphPinToOutputPin.Find(OutputEdGraphPin))
			{
				PinInfo.OutputPinWidget = OutputPins[*PinIndexPtr];
				PinInfo.OutputPinWidget->SetVisibility(PinVisibilityAttribute);
				PinWidgetForExpander = PinInfo.OutputPinWidget;
				bPinWidgetForExpanderLeft = false;
				bPinInfoIsValid = true;

				if (bIsModelPin && Template && !InModelPin->IsExecuteContext())
				{
					if (URigVMPin* RootPin = InModelPin->GetRootPin())
					{
						FLinearColor PinColorAndOpacity = PinInfo.OutputPinWidget->GetColorAndOpacity();
						if (Template->FindArgument(RootPin->GetFName()) == nullptr && Template->FindExecuteArgument(RootPin->GetFName(), DispatchContext) == nullptr)
						{
							PinColorAndOpacity.A = 0.2f;
						}
						else
						{
							PinColorAndOpacity.A = 1.0f;
						}
						PinInfo.OutputPinWidget->SetColorAndOpacity(PinColorAndOpacity);
					}
				}
			}
		}
		
		if(UEdGraphPin* InputEdGraphPin = RigGraphNode->FindPin(PinInfo.Identifier, EEdGraphPinDirection::EGPD_Input))
		{
			if(const int32* PinIndexPtr = EdGraphPinToInputPin.Find(InputEdGraphPin))
			{
				PinInfo.InputPinWidget = InputPins[*PinIndexPtr];
				PinInfo.InputPinWidget->SetVisibility(PinVisibilityAttribute);
				PinWidgetForExpander = PinInfo.InputPinWidget;
				bPinWidgetForExpanderLeft = true;
				bPinInfoIsValid = true;
				if (bIsModelPin && Template && !InModelPin->IsExecuteContext())
				{
					if (URigVMPin* RootPin = InModelPin->GetRootPin())
					{
						FLinearColor PinColorAndOpacity = PinInfo.InputPinWidget->GetColorAndOpacity();
						if (Template->FindArgument(RootPin->GetFName()) == nullptr && Template->FindExecuteArgument(RootPin->GetFName(), DispatchContext) == nullptr)
						{
							PinColorAndOpacity.A = 0.2f;
						}
						else
						{
							PinColorAndOpacity.A = 1.0f;
						}
						PinInfo.InputPinWidget->SetColorAndOpacity(PinColorAndOpacity);
					}
				}
			}
		}

		if(!bPinInfoIsValid)
		{
			return;
		}

		ERigVMPinDirection PinDirection = ERigVMPinDirection::Invalid;
		if(bIsModelPin)
		{
			PinDirection = InModelPin->GetDirection();
		}
		else
		{
			TArray<FString> CategoriesToCheck = {InCategory};
			for(int32 CategoryIndex = 0; CategoryIndex < CategoriesToCheck.Num(); CategoryIndex++)
			{
				const FString& CategoryToCheck = CategoriesToCheck[CategoryIndex];
				const TArray<URigVMPin*> PinsForCategory = CurrentModelNode->GetPinsForCategory(CategoryToCheck);
				if(!PinsForCategory.IsEmpty())
				{
					PinDirection = PinsForCategory[0]->GetDirection();
					break;
				}
				CategoriesToCheck.Append(CurrentModelNode->GetSubPinCategories(CategoryToCheck));
			}
		}

		if(PinDirection == ERigVMPinDirection::Invalid)
		{
			return;
		}

		IdentifierToInfoIndex.Add(PinInfo.Identifier, PinInfos.Add(PinInfo));
		
		// check if this pin has sub pins
		TSharedPtr<SHorizontalBox> FullPinHorizontalRowWidget = PinWidgetForExpander->GetFullPinHorizontalRowWidget().Pin();
		if(FullPinHorizontalRowWidget.IsValid())
		{
			// indent the pin by padding
			const float DepthIndentation = 12.f * float(PinInfo.Depth + (PinInfo.bHasChildren ? 0 : 1));
			const float LeftIndentation = bPinWidgetForExpanderLeft ? DepthIndentation : 0.f;
			const float RightIndentation = bPinWidgetForExpanderLeft ? 0.f : DepthIndentation;
			const FMargin LineIndentation = FMargin(RightIndentation, 0, LeftIndentation, 0); 

			static const TSharedRef<FTagMetaData> ExpanderButtonMetadata = MakeShared<FTagMetaData>(TEXT("SRigVMGraphNode.ExpanderButton"));

			// check if this pin widget may already have the expander button 
			int32 ExpanderSlotIndex = INDEX_NONE;
			int32 PaddedSlotIndex = INDEX_NONE;

			for(int32 SlotIndex=0; SlotIndex<FullPinHorizontalRowWidget->NumSlots(); SlotIndex++)
			{
				const TSharedRef<SWidget> Widget = FullPinHorizontalRowWidget->GetSlot(SlotIndex).GetWidget();
				const TSharedPtr<FTagMetaData> Metadata = Widget->GetMetaData<FTagMetaData>();
				if(Metadata.IsValid())
				{
					if(Metadata->Tag == ExpanderButtonMetadata->Tag)
					{
						ExpanderSlotIndex = SlotIndex;
						break;
					}
				}
			}

			// The expander needs to be recreated to adjust for PinInfo.Index changes
			if (FullPinHorizontalRowWidget->IsValidSlotIndex(ExpanderSlotIndex))
			{
				SHorizontalBox::FSlot& Slot = FullPinHorizontalRowWidget->GetSlot(ExpanderSlotIndex);
				const TSharedRef<SWidget> Widget = Slot.GetWidget();
				Widget->RemoveMetaData(ExpanderButtonMetadata);
				FullPinHorizontalRowWidget->RemoveSlot(Widget);
				ExpanderSlotIndex = INDEX_NONE;
			}
	
			if(PinInfo.bHasChildren && ExpanderSlotIndex == INDEX_NONE)
			{
				// only inject the expander arrow for inputs on input / IO
				// or for output pins
				if(
					(
						(
							(PinDirection == ERigVMPinDirection::Input) ||
							(PinDirection == ERigVMPinDirection::IO)
						) &&
						bPinWidgetForExpanderLeft
					) ||
					(
						(PinDirection == ERigVMPinDirection::Output) &&
						(!bPinWidgetForExpanderLeft)
					)
				)
				{
					// Add the expander arrow
					FullPinHorizontalRowWidget->InsertSlot(bPinWidgetForExpanderLeft ? 1 : FullPinHorizontalRowWidget->GetChildren()->Num() - 1)
					.Padding(FMargin(0, 0, 0, 0))
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.ContentPadding(0.f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.ClickMethod( EButtonClickMethod::MouseDown )
						.OnClicked(this, &SRigVMGraphNode::OnExpanderArrowClicked, PinInfo.Index)
						.ToolTipText(LOCTEXT("ExpandSubPin", "Expand Pin"))
						[
							SNew(SImage)
							.Image(this, &SRigVMGraphNode::GetExpanderImage, PinInfo.Index, bPinWidgetForExpanderLeft, false)
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
						.AddMetaData(ExpanderButtonMetadata)
					];
				}
			}

			// adjust the padding
			{
				const int32 SlotToAdjustIndex = bPinWidgetForExpanderLeft ? 0 : FullPinHorizontalRowWidget->NumSlots() - 1;
				SHorizontalBox::FSlot& Slot = FullPinHorizontalRowWidget->GetSlot(SlotToAdjustIndex);
				Slot.SetPadding(LineIndentation);
			}
		}
	};
	
	for(const TTuple<const URigVMPin*,FString>& ModelPinOrCategory : ModelPinsOrCategories)
	{
		AddModelPinOrCategoryPinInfo(ModelPinOrCategory.Get<0>(), ModelPinOrCategory.Get<1>());
	}

	// add spacer widget at the start
	LeftNodeBox->AddSlot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	.AutoHeight()
	[
		SNew(SSpacer)
		.Size(FVector2D(1.f, 2.f))
	];

	// add the tags this node potentially has
	ERigVMTagDisplayMode TagDisplayMode = ERigVMTagDisplayMode::All;
	if(Blueprint.IsValid())
	{
		TagDisplayMode = Blueprint->GetRigGraphDisplaySettings().TagDisplayMode;
	}
	if(ModelNode.IsValid() && TagDisplayMode != ERigVMTagDisplayMode::None)
	{
		if(URigVMFunctionReferenceNode* FunctionRefNode = Cast<URigVMFunctionReferenceNode>(ModelNode.Get()))
		{
			const FRigVMVariant Variant = FunctionRefNode->GetReferencedFunctionHeader().Variant;

			const bool bContainsAnyTags = Variant.Tags.ContainsByPredicate([TagDisplayMode](const FRigVMTag& InTag) -> bool
			{
				if(TagDisplayMode == ERigVMTagDisplayMode::DeprecationOnly)
				{
					return InTag.bMarksSubjectAsInvalid;
				}
				return true;
			});
			
			if(bContainsAnyTags)
			{
				TWeakObjectPtr<URigVMFunctionReferenceNode> WeakFunctionRefNode = FunctionRefNode;
				
				LeftNodeBox->AddSlot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.AutoHeight()
				.Padding(2, 0, 2, 2)
				[
					SNew(SRigVMVariantTagWidget)
					.Visibility_Lambda([WeakFunctionRefNode]() -> EVisibility
					{
						if(WeakFunctionRefNode.IsValid())
						{
							const FRigVMVariant Variant = WeakFunctionRefNode->GetReferencedFunctionHeader().Variant;
							if(!Variant.Tags.IsEmpty())
							{
								return EVisibility::Visible;
							}
						}
						return EVisibility::Collapsed;
					})
					.Orientation(EOrientation::Orient_Horizontal)
					.CanAddTags(false)
					.EnableContextMenu(false)
					.OnGetTags_Lambda([WeakFunctionRefNode, TagDisplayMode]() -> TArray<FRigVMTag>
					{
						if(WeakFunctionRefNode.IsValid())
						{
							const FRigVMVariant Variant = WeakFunctionRefNode->GetReferencedFunctionHeader().Variant;
							if(TagDisplayMode == ERigVMTagDisplayMode::DeprecationOnly)
							{
								return Variant.Tags.FilterByPredicate([TagDisplayMode](const FRigVMTag& InTag) -> bool
								{
									return InTag.bMarksSubjectAsInvalid;
								});
							}
							return Variant.Tags;
						}
						return {};
					})
				];
			}
		}
	}

	auto AddArrayPlusButtonLambda = [this](URigVMPin* InModelPin, TSharedPtr<SHorizontalBox> InSlotLayout, const float InEmptySidePadding)
	{
		// add array plus button
		InSlotLayout->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.Padding(PinWidgetSidePadding, TopPadding, InEmptySidePadding, 0.f)
		[
			SNew(SButton)
			.ContentPadding(0.0f)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.OnClicked(this, &SRigVMGraphNode::HandleAddArrayElement, InModelPin->GetPinPath())
			.IsEnabled(this, &SGraphNode::IsNodeEditable)
			.Cursor(EMouseCursor::Default)
			.Visibility(this, &SRigVMGraphNode::GetArrayPlusButtonVisibility, InModelPin)
			.ToolTipText(LOCTEXT("AddArrayElement", "Add Array Element"))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
				]
			]
		];
	};

	struct FPinRowInfo
	{
		int32 InputPinInfo;
		int32 OutputPinInfo;
		
		FPinRowInfo()
			: InputPinInfo(INDEX_NONE)
			, OutputPinInfo(INDEX_NONE)
		{}
	};

	TArray<FPinRowInfo> Rows;
	Rows.Reserve(PinInfos.Num());

	// build a tree with the pin infos in order as before
	for(const FPinInfo& PinInfo : PinInfos)
	{
		FPinRowInfo Row;
		if(PinInfo.InputPinWidget.IsValid())
		{
			Row.InputPinInfo = PinInfo.Index;
		}
		if(PinInfo.OutputPinWidget.IsValid())
		{
			Row.OutputPinInfo = PinInfo.Index;
		}
		Rows.Add(Row);
	}

	// compact the rows - re-use input rows for outputs
	// if their sub-pins count matches
	if(RigGraphNode->DrawAsCompactNode())
	{
		for(int32 RowIndex = 0; RowIndex < Rows.Num();)
		{
			if(Rows[RowIndex].OutputPinInfo == INDEX_NONE)
			{
				RowIndex++;
				continue;
			}

			bool bFoundMatch = false;
			for(int32 InputRowIndex = 0; InputRowIndex < RowIndex; InputRowIndex++)
			{
				if(Rows[InputRowIndex].OutputPinInfo == INDEX_NONE)
				{
					const FPinInfo& OutputPinInfo = PinInfos[Rows[RowIndex].OutputPinInfo];
					const FPinInfo& InputPinInfo = PinInfos[Rows[InputRowIndex].InputPinInfo];
					if(OutputPinInfo.bHasChildren == false &&
						InputPinInfo.bHasChildren == false)
					{
						Rows[InputRowIndex].OutputPinInfo = Rows[RowIndex].OutputPinInfo;
						bFoundMatch = true;
						break;
					}
				}
			}

			if(bFoundMatch)
			{
				Rows.RemoveAt(RowIndex);
			}
			else
			{
				RowIndex++;
			}
		}
	}

	for(const FPinRowInfo& Row : Rows)
	{
		const FPinInfo& InputPinInfo = PinInfos[Row.InputPinInfo == INDEX_NONE ? Row.OutputPinInfo : Row.InputPinInfo];
		const FPinInfo& OutputPinInfo = PinInfos[Row.OutputPinInfo == INDEX_NONE ? Row.InputPinInfo : Row.OutputPinInfo];

		const bool bIsRootPin =
			InputPinInfo.InputPinWidget.IsValid() ?
			InputPinInfo.ParentIndex == INDEX_NONE :
			OutputPinInfo.ParentIndex == INDEX_NONE;
		const bool bUsesReorderWidget = bIsRootPin && ModelNode->IsA<URigVMFunctionInterfaceNode>();

		TSharedPtr<SHorizontalBox> SlotLayout;
		
		if(InputPinInfo.InputPinWidget.IsValid())
		{
			if(InputPinInfo.bHideInputWidget)
			{
				if(InputPinInfo.InputPinWidget->GetValueWidget() != SNullWidget::NullWidget)
				{
					InputPinInfo.InputPinWidget->GetValueWidget()->SetVisibility(EVisibility::Collapsed);
				}
			}
				
			// input pins
			if(!OutputPinInfo.OutputPinWidget.IsValid())
			{
				SHorizontalBox::FSlot* FirstSlot = nullptr;

				const float MyEmptySidePadding =
					(InputPinInfo.bHideInputWidget && !bUsesReorderWidget)
					? EmptySidePadding : 0.f; 
				
				LeftNodeBox->AddSlot()
                .HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.AutoHeight()
				.MaxHeight(InputPinInfo.bAutoHeight ? TAttribute<float>() : MaxHeight)
                [
                    SAssignNew(SlotLayout, SHorizontalBox)
                    .Visibility(this, &SRigVMGraphNode::GetPinVisibility, InputPinInfo.Index, false)
		            
                    +SHorizontalBox::Slot()
                    .Expose(FirstSlot)
                    .FillWidth(1.f)
                    .HAlign(HAlign_Fill)
                    .Padding(PinWidgetSidePadding, TopPadding, InputPinInfo.bIsContainer ? 0.f : MyEmptySidePadding, 0.f)
                    [
                        InputPinInfo.InputPinWidget.ToSharedRef()
                    ]
                ];

				if(InputPinInfo.bIsContainer)
				{
					URigVMPin* ModelPin = ModelNode->GetGraph()->FindPin(InputPinInfo.Identifier);
					if(ModelPin)
					{
						// make sure to minimize the width of the label
						FirstSlot->SetAutoWidth();
						AddArrayPlusButtonLambda(ModelPin, SlotLayout, MyEmptySidePadding);
					}
				}
			}
			// io pins
			else
			{
				SHorizontalBox::FSlot* FirstSlot = nullptr;

				OutputPinInfo.OutputPinWidget->SetShowLabel(false);
			
				LeftNodeBox->AddSlot()
                .HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
                .AutoHeight()
				.MaxHeight(InputPinInfo.bAutoHeight ? TAttribute<float>() : MaxHeight)
                [
                    SAssignNew(SlotLayout, SHorizontalBox)
                    .Visibility(this, &SRigVMGraphNode::GetPinVisibility, OutputPinInfo.Index, false)

                    +SHorizontalBox::Slot()
					.Expose(FirstSlot)
                    .FillWidth(1.f)
                    .HAlign(HAlign_Left)
                    .VAlign(VAlign_Center)
                    .Padding(PinWidgetSidePadding, TopPadding, 0.f, 0.f)
                    [
                        InputPinInfo.InputPinWidget.ToSharedRef()
                    ]
                ];

				if(InputPinInfo.bIsContainer)
				{
					URigVMPin* ModelPin = ModelNode->GetGraph()->FindPin(InputPinInfo.Identifier);
					if(ModelPin)
					{
						// make sure to minimize the width of the label
						FirstSlot->SetAutoWidth();
						AddArrayPlusButtonLambda(ModelPin, SlotLayout, EmptySidePadding);
					}
				}

				SlotLayout->AddSlot()
				.FillWidth(1.f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(0.f, TopPadding, PinWidgetSidePadding, 0.f)
				[
					OutputPinInfo.OutputPinWidget.ToSharedRef()
				];
			}
		}
		// output pins
		else if(OutputPinInfo.OutputPinWidget.IsValid())
		{
			const float MyEmptySidePadding = bUsesReorderWidget ? 0.f : EmptySidePadding;
			
			LeftNodeBox->AddSlot()
            .HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
            .AutoHeight()
			.MaxHeight(OutputPinInfo.bAutoHeight ? TAttribute<float>() : MaxHeight)
            [
				SAssignNew(SlotLayout, SHorizontalBox)
	            .Visibility(this, &SRigVMGraphNode::GetPinVisibility, OutputPinInfo.Index, false)
	            
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
	            .Padding(MyEmptySidePadding, TopPadding, PinWidgetSidePadding, 0.f)
				[
	                OutputPinInfo.OutputPinWidget.ToSharedRef()
				]
            ];
		}

		if (bUsesReorderWidget)
		{
			const bool bIsInput = ModelNode->IsA<URigVMFunctionReturnNode>();
			IRigVMClientHost* Host = RigGraphNode->GetImplementingOuter<IRigVMClientHost>();
			URigVMPin* ModelPin = ModelNode->GetGraph()->FindPin(bIsInput ? InputPinInfo.Identifier : OutputPinInfo.Identifier);

			SlotLayout->InsertSlot(bIsInput ? SlotLayout->NumSlots() : 0)
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(SRigVMGraphPinReorderHandle)
				.bIsInput(bIsInput)
				.Width(EmptySidePadding)
				.Pin(ModelPin)
				.LineExtension(44.f)
				.CanReorder(this, &SRigVMGraphNode::CanReorderExposedPin)
				.OnReorder(this, &SRigVMGraphNode::OnReorderExposedPin)
				.WeakClientHost(Host)
			];
		}
	}

	CreateAddPinButton();

	CreateWorkflowWidgets();

	// add spacer widget at the end
	LeftNodeBox->AddSlot()
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Center)
	.AutoHeight()
	[
		SNew(SSpacer)
		.Size(FVector2D(1.f, 4.f))
	];
}

ERigVMNodeDefaultValueOverrideState::Type SRigVMGraphNode::GetPinDefaultValueOverrideState() const
{
	if(!CVarRigVMEnablePinOverrides.GetValueOnAnyThread())
	{
		return ERigVMNodeDefaultValueOverrideState::None;
	}

	if (ModelNode.IsValid())
	{
		return ModelNode.Get()->GetPinDefaultValueOverrideState();
	}
	return ERigVMNodeDefaultValueOverrideState::None;
}

bool SRigVMGraphNode::IsCategoryPin(const SGraphPin* InGraphPin) const
{
	if (const URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		if (const UEdGraphPin* Pin = InGraphPin->GetPinObj())
		{
			return RigVMEdGraphNode->CachedCategoryPins.Contains(Pin->GetName());
		}
	}
	return false;
}

EOverrideWidgetStatus::Type SRigVMGraphNode::GetOverrideStatus() const
{
	const ERigVMNodeDefaultValueOverrideState::Type OverrideState = GetPinDefaultValueOverrideState();
	if(OverrideState == ERigVMNodeDefaultValueOverrideState::None)
	{
		return EOverrideWidgetStatus::None;
	}
	if(OverrideState == ERigVMNodeDefaultValueOverrideState::SomePins)
	{
		return EOverrideWidgetStatus::ChangedInside;
	}
	
	return EOverrideWidgetStatus::ChangedHere;
}

FSlateColor SRigVMGraphNode::GetPreviewHereColor() const
{
	if (const URigVMNode* CurrentModelNode = ModelNode.Get())
	{
		if (CurrentModelNode->HasEarlyExitMarker())
		{
			return FSlateColor(EStyleColor::AccentOrange);
		}
		else if (CurrentModelNode->IsExcludedByEarlyExit())
		{
			return FSlateColor(EStyleColor::SelectInactive);
		}
	}
	if (bPreviewHereIndicatorHovered)
	{
		return FSlateColor(EStyleColor::Hover);
	}
	return FSlateColor::UseForeground();
}

int32 SRigVMGraphNode::CanReorderExposedPin(const URigVMPin* InPinToReorder, const URigVMPin* InTargetPin, EItemDropZone InDropZone)
{
	if (!InPinToReorder || !InTargetPin || InPinToReorder == InTargetPin)
	{
		return INDEX_NONE;
	}

	check(InPinToReorder->IsRootPin());
	check(InTargetPin->IsRootPin());
	check(InPinToReorder->GetNode() == InTargetPin->GetNode());

	const URigVMFunctionInterfaceNode* InterfaceNode = Cast<URigVMFunctionInterfaceNode>(InPinToReorder->GetNode());
	if (InterfaceNode == nullptr)
	{
		return INDEX_NONE;
	}

	const URigVMPin* PinToReorder = InterfaceNode->FindReferencedPin(InPinToReorder);
	const URigVMPin* TargetPin = InterfaceNode->FindReferencedPin(InTargetPin);
	if (PinToReorder == nullptr || TargetPin == nullptr)
	{
		return INDEX_NONE;
	}
	
	if (InDropZone == EItemDropZone::OntoItem)
	{
		InDropZone = EItemDropZone::BelowItem;
	}

	int32 NewPinIndex = TargetPin->GetPinIndex();
	if (InDropZone == EItemDropZone::BelowItem)
	{
		if (NewPinIndex < PinToReorder->GetPinIndex())
		{
			NewPinIndex++;
		}
	}
	else if (InDropZone == EItemDropZone::AboveItem)
	{
		if (NewPinIndex > PinToReorder->GetPinIndex())
		{
			NewPinIndex--;
		}
	}

	if (PinToReorder->GetPinIndex() == NewPinIndex)
	{
		return INDEX_NONE;
	}

	return NewPinIndex;
}

void SRigVMGraphNode::OnReorderExposedPin(const URigVMPin* InPinToReorder, const URigVMPin* InTargetPin, EItemDropZone InDropZone)
{
	const int32 NewPinIndex = CanReorderExposedPin(InPinToReorder, InTargetPin, InDropZone);
	if (NewPinIndex == INDEX_NONE)
	{
		return;
	}

	const URigVMNode* InterfaceNode = ModelNode.Get();
	if (!InterfaceNode)
	{
		return;
	}

	URigVMEdGraphNode* RigVMEdGraphNode = Cast<URigVMEdGraphNode>(GraphNode);
	if (RigVMEdGraphNode == nullptr)
	{
		return;
	}

	URigVMController* Controller = RigVMEdGraphNode->GetController();
	if (Controller == nullptr)
	{
		return;
	}

	(void)Controller->SetExposedPinIndex(InPinToReorder->GetFName(), NewPinIndex);
}

#undef LOCTEXT_NAMESPACE
