// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphNodeComment.h"
#include "EdGraphNode_Comment.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyPathHelpers.h"
#include "UObject/PropertyPortFlags.h"
#include "RigVMBlueprintLegacy.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "RigVMModel/RigVMController.h"
#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "SRigVMGraphNodeComment"

SRigVMGraphNodeComment::SRigVMGraphNodeComment()
	: SGraphNodeComment()
	, CachedNodeCommentColor(FLinearColor(-1.f, -1.f, -1.f, -1.f))
	, CachedColorBubble(INDEX_NONE)
{
}

FReply SRigVMGraphNodeComment::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && bUserIsDragging)
	{ 
		// bUserIsDragging is set to false in the base function

		// Resize the node	
		UserSize.X = FMath::RoundToFloat(UserSize.X);
		UserSize.Y = FMath::RoundToFloat(UserSize.Y);

		GetNodeObj()->ResizeNode(UserSize);

		if (GraphNode)
		{
			UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(GraphNode);
			if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(CommentNode->GetOuter()))
			{
				if (FRigVMEditorAssetInterfacePtr Blueprint = Graph->GetOuter())
				{
					FVector2D Position(CommentNode->NodePosX, CommentNode->NodePosY);
					FVector2D Size(CommentNode->NodeWidth, CommentNode->NodeHeight);
					if (URigVMController* Controller = Blueprint->GetController(Graph))
					{
						Controller->OpenUndoBracket(TEXT("Resize Comment Box"));
						Controller->SetNodePositionByName(CommentNode->GetFName(), Position, true, false, true);
						Controller->SetNodeSizeByName(CommentNode->GetFName(), Size, true, false, true);
						Controller->CloseUndoBracket();
					}
				}
			}
		} 
	} 
	
	// Calling the base function at the end such that above actions are included in the undo system transaction scope.
	// When undo is triggered, FBlueprintEditor::HandleUndoTransaction() will be called to make sure the undone changes are reflected in UI.
	return SGraphNodeComment::OnMouseButtonUp(MyGeometry, MouseEvent);
}

void SRigVMGraphNodeComment::EndUserInteraction() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	if (GraphNode)
	{
		if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
		{
			RigSchema->EndGraphNodeInteraction(GraphNode);
		}
	}

	SGraphNodeComment::EndUserInteraction();
}

void SRigVMGraphNodeComment::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// catch a renaming action!
	if (GraphNode)
	{
		UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(GraphNode);

		if (CachedColorBubble == INDEX_NONE)
		{
			CachedColorBubble = (int8) CommentNode->bColorCommentBubble;
		}

		const FString CurrentCommentTitle = GetNodeComment();
		const int32 CurrentCommentFontSize = CommentNode->GetFontSize();
		const bool CurrentCommentBubbleVisible = CommentNode->bCommentBubbleVisible;
		const bool CurrentCommentColorBubble = CommentNode->bColorCommentBubble;
		if (CurrentCommentTitle != CachedCommentTitle ||
			CurrentCommentFontSize != CachedFontSize ||
			CurrentCommentBubbleVisible != bCachedBubbleVisibility ||
			CurrentCommentColorBubble != (bool) CachedColorBubble)
		{
			if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(CommentNode->GetOuter()))
			{
				if (FRigVMEditorAssetInterfacePtr Blueprint = Graph->GetOuter())
				{
					if (URigVMController* Controller = Blueprint->GetController(Graph))
					{
						Controller->SetCommentTextByName(CommentNode->GetFName(), CurrentCommentTitle, CurrentCommentFontSize, CurrentCommentBubbleVisible, CurrentCommentColorBubble, true, true);
						CachedColorBubble = (int8) CurrentCommentColorBubble;
					}
				}
			}
		}

		if (CachedNodeCommentColor.R < -SMALL_NUMBER)
		{
			CachedNodeCommentColor = CommentNode->CommentColor;
		}
		else
		{
			FLinearColor CurrentNodeCommentColor = CommentNode->CommentColor;
			if (!FVector4(CachedNodeCommentColor - CurrentNodeCommentColor).IsNearlyZero3())
			{
				if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(CommentNode->GetOuter()))
				{
					if (FRigVMEditorAssetInterfacePtr Blueprint = Graph->GetOuter())
					{
						if (URigVMController* Controller = Blueprint->GetController(Graph))
						{
							// for now we won't use our undo for this kind of change
							Controller->SetNodeColorByName(GraphNode->GetFName(), CurrentNodeCommentColor, false, true);
							CachedNodeCommentColor = CurrentNodeCommentColor;
						}
					}
				}
			}
		}
	}

	SGraphNodeComment::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SRigVMGraphNodeComment::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (!NodeFilter.Find(SharedThis(this)))
	{
		if (GraphNode && !RequiresSecondPassLayout())
		{
			if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
			{
				const FVector2f CommentOldPosition = UE::Slate::CastToVector2f(RigSchema->GetNodePositionAtStartOfInteraction(GraphNode));
				const FVector2f Delta = NewPosition - CommentOldPosition;

				TArray<UEdGraphNode*> NodesToMove = URigVMEdGraphSchema::GetNodesToMoveForNode(GraphNode);
				for(UEdGraphNode* NodeToMove : NodesToMove)
				{
					// If the internal node is also selected, it will also be moved, so we don't need to take care of it
					if (NodeToMove->IsSelected())
					{
						continue;
					}
					
					const FVector2f OldPosition = UE::Slate::CastToVector2f(RigSchema->GetNodePositionAtStartOfInteraction(NodeToMove));

					// Don't drag note content if either of the shift keys are down.
					FModifierKeysState KeysState = FSlateApplication::Get().GetModifierKeys();

					if((NodeToMove == GraphNode) || (IsNodeUnderComment(CommentOldPosition, OldPosition) && !KeysState.IsShiftDown()))
					{ 
						const FVector2f AccumulatedPosition = OldPosition + Delta; 
						RigSchema->SetNodePosition(NodeToMove, FVector2D(AccumulatedPosition), false);
					}
				}
			}
		}
	}
}

/*
void SRigVMGraphNodeComment::OnCommentTextCommitted(const FText& NewComment, ETextCommit::Type CommitInfo)
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	if (GraphNode)
	{
		UEdGraphNode_Comment* CommentNode = CastChecked<UEdGraphNode_Comment>(GraphNode);
		if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(CommentNode->GetOuter()))
		{
			if (URigVMBlueprint* Blueprint = Cast<URigVMBlueprint>(Graph->GetOuter()))
			{
				//FVector2D Position(CommentNode->NodePosX, CommentNode->NodePosY);
				//Blueprint->ModelController->SetNodePosition(CommentNode->GetFName(), Position, true);
			}
		}
	}
}
*/

bool SRigVMGraphNodeComment::IsNodeUnderComment(UEdGraphNode_Comment* InCommentNode, const TSharedRef<SGraphNode> InNodeWidget) const
{
	return IsNodeUnderComment(InNodeWidget->GetPosition2f());
}


bool SRigVMGraphNodeComment::IsNodeUnderComment(UEdGraphNode* InNode) const
{
	return IsNodeUnderComment(FVector2f(static_cast<float>(InNode->NodePosX), static_cast<float>(InNode->NodePosY)));
}

bool SRigVMGraphNodeComment::IsNodeUnderComment(const FVector2f& InNodePosition) const
{
	return IsNodeUnderComment(GetPosition2f(), InNodePosition);
}

bool SRigVMGraphNodeComment::IsNodeUnderComment(const FVector2f& InCommentPosition, const FVector2f& InNodePosition) const
{
	const FVector2f NodeSize = GetDesiredSize();
	const FSlateRect CommentRect(InCommentPosition.X, InCommentPosition.Y, InCommentPosition.X + NodeSize.X, InCommentPosition.Y + NodeSize.Y);

	return CommentRect.ContainsPoint(InNodePosition);
}

#undef LOCTEXT_NAMESPACE
