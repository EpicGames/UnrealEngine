// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphNodeKnot.h"
#include "Widgets/SToolTip.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "IDocumentation.h"
#include "RigVMModel/RigVMClient.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMSettings.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "SRigVMGraphNodeKnot"

/** Graph pin for RigVM reroute (knot) nodes. Branches behavior on URigVMEditorSettings::bDirectRerouteNodeEditing. */
class SRigVMGraphPinKnot
	: public SGraphPinKnot
{
public:
	virtual FReply OnPinMouseDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent) override
	{
		// Subclass-aware settings read (matches the canvas-side handler).
		const UEdGraphNode* OwningNode = GraphPinObj ? GraphPinObj->GetOwningNode() : nullptr;
		const URigVMEdGraph* RigEdGraph = OwningNode ? Cast<URigVMEdGraph>(OwningNode->GetGraph()) : nullptr;
		const URigVMGraph* RigGraph = RigEdGraph ? RigEdGraph->GetModel() : nullptr;
		const IRigVMClientHost* ClientHost = RigGraph ? RigGraph->GetImplementingOuter<IRigVMClientHost>() : nullptr;
		const UClass* SettingsClass = ClientHost ? ClientHost->GetRigVMEditorSettingsClass() : URigVMEditorSettings::StaticClass();
		const bool bDirectRerouteNodeEditing = GetDefault<URigVMEditorSettings>(SettingsClass)->bDirectRerouteNodeEditing;
		if (!bDirectRerouteNodeEditing)
		{
			return SGraphPinKnot::OnPinMouseDown(SenderGeometry, MouseEvent);
		}

		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton &&
			GraphPinObj &&
			!GraphPinObj->bNotConnectable &&
			IsEditingEnabled())
		{
			if (MouseEvent.IsControlDown())
			{
				// Ctrl+LMB creates a connection by dragging much like common graph pins
				return SGraphPin::OnPinMouseDown(SenderGeometry, MouseEvent);
			}
			if (MouseEvent.GetModifierKeys().AnyModifiersDown())
			{
				// Use defaults when any other modifier is down (Alt-delete etc.)
				return SGraphPinKnot::OnPinMouseDown(SenderGeometry, MouseEvent);
			}

			const bool bFullyConnected = [this]() -> bool
			{
				const UEdGraphNode* OwningNode = GraphPinObj ? GraphPinObj->GetOwningNode() : nullptr;
				if (!OwningNode)
				{
					return false;
				}
				for (const UEdGraphPin* SiblingPin : OwningNode->Pins)
				{
					if (!SiblingPin || SiblingPin->LinkedTo.IsEmpty())
					{
						return false;
					}
				}
				return true;
			}();

			if (!bFullyConnected)
			{
				// Dangling knot: extend connections instead of moving the node.
				return SGraphPin::OnPinMouseDown(SenderGeometry, MouseEvent);
			}

			// Plain LMB falls through to the node for move
			return FReply::Unhandled();
		}

		return SGraphPinKnot::OnPinMouseDown(SenderGeometry, MouseEvent);
	}
};

void SRigVMGraphNodeKnot::Construct(const FArguments& InArgs, UEdGraphNode* InKnot)
{
	SGraphNodeKnot::Construct(SGraphNodeKnot::FArguments(), InKnot);

	if (URigVMEdGraphNode* CRNode = Cast<URigVMEdGraphNode>(InKnot))
	{
		CRNode->OnNodeBeginRemoval().AddSP(this, &SRigVMGraphNodeKnot::HandleNodeBeginRemoval);
	}
}

void SRigVMGraphNodeKnot::EndUserInteraction() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->CancelTransaction(0);
	}
#endif

	if (GraphNode)
	{
		if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
		{
			RigSchema->EndGraphNodeInteraction(GraphNode);
		}
	}

	SGraphNodeKnot::EndUserInteraction();
}

void SRigVMGraphNodeKnot::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	if (!NodeFilter.Find(SharedThis(this)))
	{
		if (GraphNode && !RequiresSecondPassLayout())
		{
			if (const URigVMEdGraphSchema* RigSchema = Cast<URigVMEdGraphSchema>(GraphNode->GetSchema()))
			{
				RigSchema->SetNodePosition(GraphNode, FVector2D(NewPosition), false);
			}
		}
	}
}

void SRigVMGraphNodeKnot::HandleNodeBeginRemoval()
{
	if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GraphNode))
	{
		RigNode->OnNodeBeginRemoval().RemoveAll(this);
	}
	
	for (const TSharedRef<SGraphPin>& GraphPin: InputPins)
	{
		GraphPin->SetPinObj(nullptr);
	}
	for (const TSharedRef<SGraphPin>& GraphPin: OutputPins)
	{
		GraphPin->SetPinObj(nullptr);
	}

	InputPins.Reset();
	OutputPins.Reset();
	
	InvalidateGraphData();
}

TSharedPtr<SGraphPin> SRigVMGraphNodeKnot::CreatePinWidget(UEdGraphPin* Pin) const
{
	return SNew(SRigVMGraphPinKnot, Pin);
}

void SRigVMGraphNodeKnot::UpdateGraphNode()
{
	SGraphNodeKnot::UpdateGraphNode();

	if (!SWidget::GetToolTip().IsValid())
	{
		TSharedRef<SToolTip> DefaultToolTip = IDocumentation::Get()->CreateToolTip( TAttribute< FText >( this, &SGraphNode::GetNodeTooltip ), NULL, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName() );
		SetToolTip(DefaultToolTip);
	}
}

#undef LOCTEXT_NAMESPACE
