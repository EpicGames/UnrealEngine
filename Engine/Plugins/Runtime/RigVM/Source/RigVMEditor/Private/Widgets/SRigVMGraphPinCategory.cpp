// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinCategory.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMModel/RigVMController.h"
#include "Widgets/SNullWidget.h"

void SRigVMGraphPinCategory::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

FReply SRigVMGraphPinCategory::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(const UEdGraphPin* EdGraphPin = GraphPinObj)
	{
		if(const URigVMEdGraphNode* EdGraphNode = Cast<URigVMEdGraphNode>(EdGraphPin->GetOwningNode()))
		{
			if(URigVMController* Controller = EdGraphNode->GetController())
			{
				if(const URigVMGraph* Model = Controller->GetGraph())
				{
					if(const URigVMNode* Node = Model->FindNodeByName(EdGraphNode->GetFName()))
					{
						const bool bIsExpanded = Node->IsPinCategoryExpanded(EdGraphPin->GetName());
						if(Controller->SetPinCategoryExpansion(Node->GetFName(), EdGraphPin->GetName(), !bIsExpanded))
						{
							return FReply::Handled();
						}
					}
				}
			}
		}
	}
	return SGraphPin::OnMouseButtonDown(MyGeometry, MouseEvent);
}

TSharedRef<SWidget>	SRigVMGraphPinCategory::GetDefaultValueWidget()
{
	return SNullWidget::NullWidget;
}

