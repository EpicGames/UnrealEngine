// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinCurveFloat.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMBlueprintLegacy.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "EdGraphSchema_K2.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "PropertyPathHelpers.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"
#include "RigVMModel/RigVMController.h"

void SRigVMGraphPinCurveFloat::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinCurveFloat::GetDefaultValueWidget()
{
	URigVMEdGraph* RigGraph = Cast<URigVMEdGraph>(GraphPinObj->GetOwningNode()->GetGraph());

	// 360 is the minimum width required to display keys' values 
	TSharedRef<SWidget> Widget = SNew(SBox)
		.MinDesiredWidth(360.f)
		.MaxDesiredWidth(400.f)
		.MinDesiredHeight(175.f)
		.MaxDesiredHeight(300.f)
		[
			SAssignNew(CurveEditor, SCurveEditor)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			.ViewMinInput(0.f)
			.ViewMaxInput(1.f)
			.ViewMinOutput(0.f)
			.ViewMaxOutput(1.f)
			.TimelineLength(1.f)
			.DesiredSize(FVector2D(360, 200))
			.HideUI(true)
		];

	CurveEditor->SetCurveOwner(this);

	return Widget;
}

void SRigVMGraphPinCurveFloat::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UpdateAndGetCurve();
	SGraphPin::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TArray<FRichCurveEditInfoConst> SRigVMGraphPinCurveFloat::GetCurves() const
{
	TArray<FRichCurveEditInfoConst> Curves;
	Curves.Add(Curve.GetRichCurveConst());
	return Curves;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SRigVMGraphPinCurveFloat::GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const
{
	Curves.Add(Curve.GetRichCurveConst());
}

TArray<FRichCurveEditInfo> SRigVMGraphPinCurveFloat::GetCurves()
{
	TArray<FRichCurveEditInfo> Curves;
	Curves.Add(UpdateAndGetCurve().GetRichCurve());
	return Curves;
}

FRuntimeFloatCurve& SRigVMGraphPinCurveFloat::UpdateAndGetCurve()
{
	if (UEdGraphPin* Pin = GetPinObj())
	{
		if (URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(Pin->GetOwningNode()))
		{
			if (URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(Pin->GetName()))
			{
				FString TextToImport = ModelPin->GetDefaultValue();
				FRuntimeFloatCurve::StaticStruct()->ImportText(*TextToImport, &Curve, nullptr, EPropertyPortFlags::PPF_None, nullptr, FRuntimeFloatCurve::StaticStruct()->GetName(), true);
			}
		}
	}
	return Curve;
}


void SRigVMGraphPinCurveFloat::ModifyOwner()
{
	if (UEdGraphPin* Pin = GetPinObj())
	{
		Pin->Modify();
	}
}

TArray<const UObject*> SRigVMGraphPinCurveFloat::GetOwners() const
{
	TArray<const UObject*> Owners;
	if (UEdGraphPin* Pin = GetPinObj())
	{
		if (UEdGraphNode* Node = Pin->GetOwningNode())
		{
			Owners.Add(Node);
		}
	}
	return Owners;
}

void SRigVMGraphPinCurveFloat::MakeTransactional()
{
}

bool SRigVMGraphPinCurveFloat::IsValidCurve(FRichCurveEditInfo CurveInfo)
{
	if (UEdGraphPin* Pin = GetPinObj())
	{
		if (URigVMEdGraphNode* Node = Cast<URigVMEdGraphNode>(Pin->GetOwningNode()))
		{
			if (URigVMUnitNode* StructModelNode = Cast<URigVMUnitNode>(Node->GetModelNode()))
			{
				FString NodeName, PropertyName;
				Pin->PinName.ToString().Split(TEXT("."), &NodeName, &PropertyName);

				if (FProperty* CurveProperty = StructModelNode->GetScriptStruct()->FindPropertyByName(*PropertyName))
				{
					return true;
				}
			}
			else
			{
				// for any other node, the curve is valid
				return true;
			}
		}
	}
	return false;
}

void SRigVMGraphPinCurveFloat::OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos)
{
	if (UEdGraphPin* Pin = GetPinObj())
	{
		if (URigVMEdGraphNode* Node = Cast<URigVMEdGraphNode>(Pin->GetOwningNode()))
		{
			FString ExportedText;
			FRuntimeFloatCurve DefaultCurve;
			FRuntimeFloatCurve::StaticStruct()->ExportText(ExportedText, &Curve, &DefaultCurve, nullptr, EPropertyPortFlags::PPF_None, nullptr, true);
			Node->GetController()->SetPinDefaultValue(Pin->GetName(), ExportedText, true, true, true);
		}
	}
	ModifyOwner();
}
