// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMGraphPinEditableNameValueWidget.h"
#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "GraphPinEditableNameValueWidget"

void SRigVMGraphPinEditableNameValueWidget::Construct(const FArguments& InArgs)
{
	this->OnComboBoxOpening = InArgs._OnComboBoxOpening;
	this->OnSelectionChanged = InArgs._OnSelectionChanged;
	this->OnGenerateWidget = InArgs._OnGenerateWidget;

	OptionsSource = InArgs._OptionsSource;
	CustomScrollbar = InArgs._CustomScrollbar;

	TSharedRef<SWidget> ComboBoxMenuContent =
		SNew(SBox)
		.MaxDesiredHeight(InArgs._MaxListHeight)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(this->SearchField, SEditableTextBox)
				.HintText(LOCTEXT("SearchOrRename", "Search / Rename"))
				.OnTextChanged(this, &SRigVMGraphPinEditableNameValueWidget::OnSearchTextChanged)
				.OnTextCommitted(this, &SRigVMGraphPinEditableNameValueWidget::OnSearchTextCommitted)
			]

			+ SVerticalBox::Slot()
			[
				SAssignNew(this->ComboListView, SComboListType)
				.ListItemsSource(OptionsSource)
				.OnGenerateRow(this, &SRigVMGraphPinEditableNameValueWidget::GenerateMenuItemRow)
				.OnSelectionChanged(this, &SRigVMGraphPinEditableNameValueWidget::OnSelectionChanged_Internal)
				.SelectionMode(ESelectionMode::Single)
				.ExternalScrollbar(InArgs._CustomScrollbar)
			]
		];

	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock)
			.Text(NSLOCTEXT("SRigVMGraphPinEditableNameValueWidget", "ContentWarning", "No Content Provided"))
			.ColorAndOpacity(FLinearColor::Red);
	}


	SComboButton::Construct(SComboButton::FArguments()
		.Method(InArgs._Method)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		.MenuContent()
		[
			ComboBoxMenuContent
		]
		.HasDownArrow(InArgs._HasDownArrow)
		.ContentPadding(InArgs._ContentPadding)
		.OnMenuOpenChanged(this, &SRigVMGraphPinEditableNameValueWidget::OnMenuOpenChanged)
		.IsFocusable(true)
		);
	SetMenuContentWidgetToFocus(ComboListView);

	// Need to establish the selected item at point of construction so its available for querying
	// NB: If you need a selection to fire use SetItemSelection rather than setting an IntiallySelectedItem
	SelectedItem = InArgs._InitiallySelectedItem;
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
	{
		ComboListView->Private_SetItemSelection(SelectedItem, true);
	}

}

void SRigVMGraphPinEditableNameValueWidget::ClearSelection()
{
	ComboListView->ClearSelection();
}

void SRigVMGraphPinEditableNameValueWidget::SetSelectedItem(TSharedPtr<FString> InSelectedItem)
{
	if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(InSelectedItem))
	{
		ComboListView->SetSelection(InSelectedItem);
	}
	else
	{
		ComboListView->ClearSelection();
	}
}

TSharedPtr<FString> SRigVMGraphPinEditableNameValueWidget::GetSelectedItem()
{
	return SelectedItem;
}

void SRigVMGraphPinEditableNameValueWidget::RefreshOptions()
{
	if (!ComboListView->IsPendingRefresh())
	{
		ComboListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SRigVMGraphPinEditableNameValueWidget::GenerateMenuItemRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (OnGenerateWidget.IsBound())
	{
		FString SearchToken = SearchField->GetText().ToString().ToLower();
		EVisibility WidgetVisibility = EVisibility::Visible;
		if (!SearchToken.IsEmpty())
		{
			if (InItem->ToLower().Find(SearchToken) < 0)
			{
				WidgetVisibility = EVisibility::Collapsed;
			}
		}
		return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
			.Visibility(WidgetVisibility)
			[
				OnGenerateWidget.Execute(InItem)
			];
	}
	else
	{
		return SNew(SComboRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNew(STextBlock).Text(NSLOCTEXT("SlateCore", "ComboBoxMissingOnGenerateWidgetMethod", "Please provide a .OnGenerateWidget() handler."))
			];

	}
}

void SRigVMGraphPinEditableNameValueWidget::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		if (TListTypeTraits<TSharedPtr<FString>>::IsPtrValid(SelectedItem))
		{
			// Ensure the ListView selection is set back to the last committed selection
			ComboListView->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
			ComboListView->RequestScrollIntoView(SelectedItem, 0);
		}

		// Set focus back to ComboBox for users focusing the ListView that just closed
		FSlateApplication::Get().ForEachUser([&](FSlateUser& User) {
			if (FSlateApplication::Get().HasUserFocusedDescendants(AsShared(), User.GetUserIndex()))
			{
				FSlateApplication::Get().SetUserFocus(User.GetUserIndex(), AsShared(), EFocusCause::SetDirectly);
			}
		});

	}
	else
	{
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SRigVMGraphPinEditableNameValueWidget::SetFocusPostConstruct));
	}
}

EActiveTimerReturnType SRigVMGraphPinEditableNameValueWidget::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (SearchField.IsValid())
	{
		bool bSucceeded = false;
		FSlateApplication::Get().ForEachUser([&](FSlateUser& User) {
			if (FSlateApplication::Get().SetUserFocus(User.GetUserIndex(), SearchField->AsShared(), EFocusCause::SetDirectly))
			{
				bSucceeded = true;
			}
		});

		if (bSucceeded)
		{
			return EActiveTimerReturnType::Stop;
		}
	}

	return EActiveTimerReturnType::Continue;
}

void SRigVMGraphPinEditableNameValueWidget::OnSelectionChanged_Internal(TSharedPtr<FString> ProposedSelection, ESelectInfo::Type SelectInfo)
{
	// Ensure that the proposed selection is different
	if (SelectInfo != ESelectInfo::OnNavigation)
	{
		// Ensure that the proposed selection is different from selected
		if (ProposedSelection != SelectedItem)
		{
			SelectedItem = ProposedSelection;
			OnSelectionChanged.ExecuteIfBound(ProposedSelection, SelectInfo);
		}
		// close combo even if user reselected item
		this->SetIsOpen(false);
	}
}

void SRigVMGraphPinEditableNameValueWidget::OnSearchTextChanged(const FText& ChangedText)
{
	FString SearchToken = ChangedText.ToString().ToLower();
	for (int32 i = 0; i < OptionsSource->Num(); i++)
	{
		TSharedPtr<ITableRow> Row = ComboListView->WidgetFromItem((*OptionsSource)[i]);
		if (Row)
		{
			if (SearchToken.IsEmpty())
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else if ((*OptionsSource)[i]->ToLower().Find(SearchToken) >= 0)
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else
			{
				Row->AsWidget()->SetVisibility(EVisibility::Collapsed);
			}
		}
	}

	ComboListView->RequestListRefresh();

	SelectedItem = TSharedPtr< FString >();
}

void SRigVMGraphPinEditableNameValueWidget::OnSearchTextCommitted(const FText& ChangedText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		TSharedPtr<FString> ProposedSelection = MakeShared<FString>(ChangedText.ToString());
		OnSelectionChanged_Internal(ProposedSelection, ESelectInfo::OnKeyPress);
	}
}

FReply SRigVMGraphPinEditableNameValueWidget::OnButtonClicked()
{
	// if user clicked to close the combo menu
	if (this->IsOpen())
	{
		// Re-select first selected item, just in case it was selected by navigation previously
		TArray<TSharedPtr<FString>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
		}
	}
	else
	{
		SearchField->SetText(FText::GetEmpty());
		OnComboBoxOpening.ExecuteIfBound();
	}

	return SComboButton::OnButtonClicked();
}

#undef LOCTEXT_NAMESPACE
