// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinEnumPicker.h"

#include "DetailLayoutBuilder.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMBlueprintLegacy.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "UObject/UObjectIterator.h"

void SRigVMEnumPicker::Construct(const FArguments& InArgs)
{
	OnEnumChangedDelegate = InArgs._OnEnumChanged;
	bIsEnabled = InArgs._IsEnabled;
	GetCurrentEnumDelegate = InArgs._GetCurrentEnum;

	PopulateEnumOptions();

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(150.f)
		.MaxDesiredWidth(400.f)
		[
			SNew(SSearchableComboBox)
			.OptionsSource(&EnumOptions)
			.OnSelectionChanged(this, &SRigVMEnumPicker::HandleControlEnumChanged)
			.OnGenerateWidget(this, &SRigVMEnumPicker::OnGetEnumNameWidget)
			.Content()
			[
				SNew(STextBlock)
				.Text_Lambda([this]() -> FText
				{
					const UEnum* ControlEnum = GetCurrentEnum();
					if (ControlEnum)
					{
						return FText::FromName(ControlEnum->GetFName());
					}
					return FText();
				})
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];
}

TSharedRef<SWidget> SRigVMEnumPicker::OnGetEnumNameWidget(TSharedPtr<FString> InItem)
{
	if (InItem.IsValid())
	{
		UEnum* Enum = FindObject<UEnum>(nullptr, **InItem.Get(), EFindObjectFlags::None);
		if (Enum)
		{
			return SNew(STextBlock)
			.Text(FText::FromString(Enum->GetName()))
			.ToolTip(FSlateApplication::Get().MakeToolTip(FText::FromString(Enum->GetPathName())))
			.Font(IDetailLayoutBuilder::GetDetailFont());
		}
	}
	return SNew(STextBlock)
	.Text(FText::FromString(TEXT("None")))
	.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SRigVMEnumPicker::PopulateEnumOptions()
{
	EnumOptions.Reset();
	EnumOptions.Add(MakeShareable(new FString(TEXT("None"))));
	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		UEnum* Enum = *EnumIt;

		if (Enum->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed) || !Enum->HasAllFlags(RF_Public))
		{
			continue;
		}

		// Any asset based enum is valid
		if (!Enum->IsAsset())
		{
			// Native enums only allowed if contain RigVMTypeAllowed metadata
			if (!Enum->HasMetaData(TEXT("RigVMTypeAllowed")))
			{
				continue;
			}
		}

		EnumOptions.Add(MakeShareable(new FString(Enum->GetPathName())));
	}
}

void SRigVMGraphPinEnumPicker::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	ModelPin = InArgs._ModelPin;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinEnumPicker::GetDefaultValueWidget()
{
	return SNew(SRigVMEnumPicker)
	.GetCurrentEnum_Lambda([this]()
	{
		UEnum* ControlEnum = nullptr;
		if (ModelPin)
		{
			const FString& Path = ModelPin->GetDefaultValue();
			ControlEnum = FindObject<UEnum>(nullptr, *Path, EFindObjectFlags::None);
		}
		return ControlEnum;
	})
	.OnEnumChanged(this, &SRigVMGraphPinEnumPicker::HandleControlEnumChanged);
}

void SRigVMGraphPinEnumPicker::HandleControlEnumChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo)
{
	if(ModelPin)
	{
		if(FRigVMEditorAssetInterfacePtr Blueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(ModelPin))
		{
			if(URigVMController* Controller = Blueprint->GetOrCreateController(ModelPin->GetGraph()))
			{
				Controller->SetPinDefaultValue(ModelPin->GetPinPath(), *InItem.Get());
			}
		}
	}
}




