// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinNameList.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "EdGraphSchema_K2.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMBlueprintLegacy.h"

namespace SRigVMGraphPinNameListDefs
{
	// Active foreground pin alpha
	static const float ActivePinForegroundAlpha = 1.f;
	// InActive foreground pin alpha
	static const float InactivePinForegroundAlpha = 0.15f;
	// Active background pin alpha
	static const float ActivePinBackgroundAlpha = 0.8f;
	// InActive background pin alpha
	static const float InactivePinBackgroundAlpha = 0.4f;
};

void SRigVMGraphPinNameList::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	this->ModelPin = InArgs._ModelPin;
	this->OnGetNameListContent = InArgs._OnGetNameListContent;
	this->OnGetNameListContentForValidation = InArgs._OnGetNameListContentForValidation;
	this->OnGetNameFromSelection = InArgs._OnGetNameFromSelection;
	this->OnGetSelectedClicked = InArgs._OnGetSelectedClicked;
	this->OnBrowseClicked = InArgs._OnBrowseClicked;
	this->bMarkupInvalidItems = InArgs._MarkupInvalidItems;
	this->EnableNameListCache = InArgs._EnableNameListCache;
	this->SearchHintText = InArgs._SearchHintText;
	this->AllowUserProvidedText = InArgs._AllowUserProvidedText;

	UpdateNameLists();
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinNameList::GetDefaultValueWidget()
{
	TSharedPtr<FRigVMStringWithTag> InitialSelected;
	const TArray<TSharedPtr<FRigVMStringWithTag>>* List = GetNameList();
	for (const TSharedPtr<FRigVMStringWithTag>& Item : (*List))
	{
		if (Item->Equals(GetNameListText().ToString()))
		{
			InitialSelected = Item;
		}
	}

	return SNew(SBox)
		.MinDesiredWidth(150.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SAssignNew(NameListComboBox, SRigVMGraphPinNameListValueWidget)
					.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
					.OptionsSource(CurrentList)
					.OnGenerateWidget(this, &SRigVMGraphPinNameList::MakeNameListItemWidget)
					.OnSelectionChanged(this, &SRigVMGraphPinNameList::OnNameListChanged)
					.OnComboBoxOpening(this, &SRigVMGraphPinNameList::OnNameListComboBox)
					.InitiallySelectedItem(InitialSelected)
					.SearchHintText(SearchHintText)
					.AllowUserProvidedText(AllowUserProvidedText)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SRigVMGraphPinNameList::GetNameListText)
						.ColorAndOpacity(this, &SRigVMGraphPinNameList::GetNameColor)
						.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
					]
			]
		
			// Use button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1,0)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity( this, &SRigVMGraphPinNameList::OnGetWidgetBackground )
				.OnClicked(this, &SRigVMGraphPinNameList::HandleGetSelectedClicked)
				.ContentPadding(1.f)
				.ToolTipText(NSLOCTEXT("SRigVMGraphPinNameList", "ObjectGraphPin_Use_Tooltip", "Use item selected"))
				.Visibility(OnGetNameFromSelection.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
				[
					SNew(SImage)
					.ColorAndOpacity( this, &SRigVMGraphPinNameList::OnGetWidgetForeground )
					.Image(FAppStyle::GetBrush("Icons.CircleArrowLeft"))
				]
			]

			// Browse button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1,0)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "NoBorder" )
				.ButtonColorAndOpacity( this, &SRigVMGraphPinNameList::OnGetWidgetBackground )
				.OnClicked(this, &SRigVMGraphPinNameList::HandleBrowseClicked)
				.ContentPadding(0.f)
				.ToolTipText(NSLOCTEXT("SRigVMGraphPinNameList", "ObjectGraphPin_Browse_Tooltip", "Browse"))
				.Visibility(OnGetNameFromSelection.IsBound() ? EVisibility::Visible : EVisibility::Collapsed)
				[
					SNew(SImage)
					.ColorAndOpacity( this, &SRigVMGraphPinNameList::OnGetWidgetForeground )
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
				
		];
}

const TArray<TSharedPtr<FRigVMStringWithTag>>* SRigVMGraphPinNameList::GetNameList(bool bForContent) const
{
	const TArray<TSharedPtr<FRigVMStringWithTag>>* Result = nullptr;
	
	// if we are looking for the validation name list, try to get it from the
	// dedicated delegate first - but fall back on the content list nevertheless
	if(!bForContent)
	{
		if (OnGetNameListContentForValidation.IsBound())
		{
			Result = OnGetNameListContentForValidation.Execute(ModelPin);
			if(Result)
			{
				return Result;
			}
		}
	}

	if (OnGetNameListContent.IsBound())
	{
		Result = OnGetNameListContent.Execute(ModelPin);
	}

	if(Result == nullptr)
	{
		Result = &EmptyList;
	}

	return Result;
}

FText SRigVMGraphPinNameList::GetNameListText() const
{
	const FString DefaultString = GraphPinObj->GetDefaultAsString();
	for (const TSharedPtr<FRigVMStringWithTag>& Item : (*CurrentList))
	{
		if (Item->Equals(DefaultString))
		{
			return FText::FromString(Item->GetStringWithTag());
		}
	}
	return FText::FromString( DefaultString );
}

void SRigVMGraphPinNameList::SetNameListText(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if(!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeElementNameListPinValue", "Change Element Name Pin Value" ) );
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
	}
}

void SRigVMGraphPinNameList::UpdateNameLists(bool bUpdateCurrent, bool bUpdateValidation)
{
	if(CurrentList == nullptr || bUpdateCurrent)
	{
		CurrentList = GetNameList(true);
	}
	if(ValidationList == nullptr || bUpdateValidation)
	{
		ValidationList = GetNameList(false);
	}
}

FSlateColor SRigVMGraphPinNameList::GetNameColor() const
{
	if(bMarkupInvalidItems)
	{
		const FString DefaultString = GraphPinObj->GetDefaultAsString();

		if(!EnableNameListCache)
		{
			((SRigVMGraphPinNameList*)this)->UpdateNameLists(false, true);
		}

		bool bFound = false;
		for (const TSharedPtr<FRigVMStringWithTag>& Item : (*ValidationList))
		{
			if (Item->Equals(DefaultString))
			{
				bFound = true;
				break;
			}
		}

		if(!bFound || DefaultString.IsEmpty() || DefaultString == FName(NAME_None).ToString())
		{
			return FSlateColor(FLinearColor::Red);
		}
	}
	return FSlateColor::UseForeground();
}

TSharedRef<SWidget> SRigVMGraphPinNameList::MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem)
{
	//TODO: make this prettier
	return 	SNew(STextBlock).Text(FText::FromString(InItem->GetStringWithTag())).Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SRigVMGraphPinNameList::OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = FName(NAME_None).ToString();
		if (NewSelection.IsValid())
		{
			NewValue = NewSelection->GetString();
		}
		SetNameListText(FText::FromString(NewValue), ETextCommit::OnEnter);
	}
}

void SRigVMGraphPinNameList::OnNameListComboBox()
{
	UpdateNameLists();

	TSharedPtr<FRigVMStringWithTag> CurrentlySelected;
	for (const TSharedPtr<FRigVMStringWithTag>& Item : (*CurrentList))
	{
		if (Item->Equals(GetNameListText().ToString()))
		{
			CurrentlySelected = Item;
			break;
		}
	}
	NameListComboBox->SetOptionsSource(CurrentList);
	if(CurrentlySelected.IsValid())
	{
		NameListComboBox->SetSelectedItem(CurrentlySelected);
	}
}

FSlateColor SRigVMGraphPinNameList::OnGetWidgetForeground() const
{
	float Alpha = IsHovered() ? SRigVMGraphPinNameListDefs::ActivePinForegroundAlpha : SRigVMGraphPinNameListDefs::InactivePinForegroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SRigVMGraphPinNameList::OnGetWidgetBackground() const
{
	float Alpha = IsHovered() ? SRigVMGraphPinNameListDefs::ActivePinBackgroundAlpha : SRigVMGraphPinNameListDefs::InactivePinBackgroundAlpha;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FReply SRigVMGraphPinNameList::HandleGetSelectedClicked()
{
	if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetPinObj()->GetOwningNode()->GetGraph()))
	{
		if (OnGetNameFromSelection.IsBound() && OnGetSelectedClicked.IsBound())
		{
			const TArray<TSharedPtr<FRigVMStringWithTag>> Result = OnGetNameFromSelection.Execute();
			if (Result.Num() > 0)
			{
				if (Result[0].IsValid() && Result[0] != nullptr)
				{
					const FString DefaultValue = Result[0]->GetString();
					FReply Reply = OnGetSelectedClicked.Execute(Graph, ModelPin, DefaultValue);
					if(Reply.IsEventHandled())
					{
						UpdateNameLists();
					}
					return Reply;
				}
			}
		}
	}
	return FReply::Handled();
}

FReply SRigVMGraphPinNameList::HandleBrowseClicked()
{
	if (URigVMEdGraph* Graph = Cast<URigVMEdGraph>(GetPinObj()->GetOwningNode()->GetGraph()))
	{
		if(OnBrowseClicked.IsBound())
		{
			TSharedPtr<FRigVMStringWithTag> Selected = NameListComboBox->GetSelectedItem();
			if (Selected.IsValid() && ModelPin)
			{
				return OnBrowseClicked.Execute(Graph, ModelPin, Selected->GetString());
			}
		}
	}

	return FReply::Handled();
}
