// Copyright Epic Games, Inc. All Rights Reservekd.

#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateUser.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "GraphPinNameListValueWidget"

void SRigVMGraphPinNameListValueWidget::Construct(const FArguments& InArgs)
{
	this->OnComboBoxOpening = InArgs._OnComboBoxOpening;
	this->OnSelectionChanged = InArgs._OnSelectionChanged;
	this->OnGenerateWidget = InArgs._OnGenerateWidget;
	this->AllowUserProvidedText = InArgs._AllowUserProvidedText;

	OptionsSource = InArgs._OptionsSource;
	CustomScrollbar = InArgs._CustomScrollbar;

	const bool bShouldCloseAfterSelection = true;
	const TSharedPtr<FUICommandList> NullCommandList = nullptr;
	FMenuBuilder MenuBuilder(bShouldCloseAfterSelection, NullCommandList);
	
	const FName DefaultSection = NAME_None;
	MenuBuilder.BeginSection(DefaultSection);
	{
		const FText& NoLabel = FText::GetEmpty();

		MenuBuilder.AddWidget(
			SAssignNew(this->SearchField, SEditableTextBox)
			.HintText(InArgs._SearchHintText)
			.OnTextChanged(this, &SRigVMGraphPinNameListValueWidget::OnSearchTextChanged)
			.OnTextCommitted(this, &SRigVMGraphPinNameListValueWidget::OnSearchTextCommitted)
			.OnKeyDownHandler(this, &SRigVMGraphPinNameListValueWidget::OnSearchTextKeyDown),
			NoLabel);

		MenuBuilder.AddWidget(
			SAssignNew(this->ComboListView, SComboListType)
			.ListItemsSource(OptionsSource)
			.OnGenerateRow(this, &SRigVMGraphPinNameListValueWidget::GenerateMenuItemRow)
			.OnSelectionChanged(this, &SRigVMGraphPinNameListValueWidget::OnSelectionChanged_Internal, false)
			.SelectionMode(ESelectionMode::Single)
			.ExternalScrollbar(InArgs._CustomScrollbar)
			.OnKeyDownHandler(this, &SRigVMGraphPinNameListValueWidget::OnComboListKeyDown),
			NoLabel);
	}
	MenuBuilder.EndSection();

	const TSharedRef<SWidget> ComboBoxMenuContent =
		SNew(SBox)
		.MinDesiredWidth(150.0f)
		.MaxDesiredHeight(InArgs._MaxListHeight)
		[
			MenuBuilder.MakeWidget()
		];

	// Set up content
	TSharedPtr<SWidget> ButtonContent = InArgs._Content.Widget;
	if (InArgs._Content.Widget == SNullWidget::NullWidget)
	{
		SAssignNew(ButtonContent, STextBlock)
			.Text(NSLOCTEXT("SRigVMGraphPinNameListValueWidget", "ContentWarning", "No Content Provided"))
			.ColorAndOpacity(FLinearColor::Red);
	}


	SComboButton::Construct(SComboButton::FArguments()
		.Method(InArgs._Method)
		.ButtonContent()
		[
			ButtonContent.ToSharedRef()
		]
		.MenuContent()
		[
			ComboBoxMenuContent
		]
		.HasDownArrow(InArgs._HasDownArrow)
		.ContentPadding(InArgs._ContentPadding)
		.OnMenuOpenChanged(this, &SRigVMGraphPinNameListValueWidget::OnMenuOpenChanged)
		.IsFocusable(true)
		);
	SetMenuContentWidgetToFocus(ComboListView);

	// Need to establish the selected item at point of construction so its available for querying
	// NB: If you need a selection to fire use SetItemSelection rather than setting an IntiallySelectedItem
	SelectedItem = InArgs._InitiallySelectedItem;
	if (TListTypeTraits<TSharedPtr<FRigVMStringWithTag>>::IsPtrValid(SelectedItem))
	{
		ComboListView->Private_SetItemSelection(SelectedItem, true);
	}

}

void SRigVMGraphPinNameListValueWidget::ClearSelection()
{
	ComboListView->ClearSelection();
}

void SRigVMGraphPinNameListValueWidget::SetSelectedItem(TSharedPtr<FRigVMStringWithTag> InSelectedItem)
{
	if (TListTypeTraits<TSharedPtr<FRigVMStringWithTag>>::IsPtrValid(InSelectedItem))
	{
		ComboListView->SetSelection(InSelectedItem);
	}
	else
	{
		ComboListView->ClearSelection();
	}
}

void SRigVMGraphPinNameListValueWidget::SetOptionsSource(const TArray<TSharedPtr<FRigVMStringWithTag>>* InOptionsSource)
{
	check(InOptionsSource);
	OptionsSource = InOptionsSource;
	ComboListView->SetItemsSource(OptionsSource);
}

TSharedPtr<FRigVMStringWithTag> SRigVMGraphPinNameListValueWidget::GetSelectedItem()
{
	return SelectedItem;
}

void SRigVMGraphPinNameListValueWidget::RefreshOptions()
{
	if (!ComboListView->IsPendingRefresh())
	{
		ComboListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SRigVMGraphPinNameListValueWidget::GenerateMenuItemRow(TSharedPtr<FRigVMStringWithTag> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (OnGenerateWidget.IsBound())
	{
		FString SearchToken = SearchField->GetText().ToString().ToLower();
		EVisibility WidgetVisibility = EVisibility::Visible;
		if (!SearchToken.IsEmpty())
		{
			FString SearchTokenUnderscores = SearchToken.Replace(TEXT(" "), TEXT("_"));
			if ((InItem->GetString().ToLower().Find(SearchToken) < 0) &&
				(InItem->GetString().ToLower().Find(SearchTokenUnderscores) < 0))
			{
				WidgetVisibility = EVisibility::Collapsed;
			}
		}
		return SNew(SComboRow<TSharedPtr<FRigVMStringWithTag>>, OwnerTable)
			.Visibility(WidgetVisibility)
			[
				OnGenerateWidget.Execute(InItem)
			];
	}
	else
	{
		return SNew(SComboRow<TSharedPtr<FRigVMStringWithTag>>, OwnerTable)
			[
				SNew(STextBlock).Text(NSLOCTEXT("SlateCore", "ComboBoxMissingOnGenerateWidgetMethod", "Please provide a .OnGenerateWidget() handler."))
			];

	}
}

void SRigVMGraphPinNameListValueWidget::OnMenuOpenChanged(bool bOpen)
{
	if (bOpen == false)
	{
		if (TListTypeTraits<TSharedPtr<FRigVMStringWithTag>>::IsPtrValid(SelectedItem))
		{
			// Ensure the ListView selection is set back to the last committed selection
			ComboListView->SetSelection(SelectedItem, ESelectInfo::OnNavigation);
			ComboListView->RequestScrollIntoView(SelectedItem, 0);
		}

		// Set focus back to ComboBox for users focusing the ListView that just closed
		TSharedRef<SWidget> ThisRef = AsShared();
		FSlateApplication::Get().ForEachUser([&ThisRef](FSlateUser& User) {
			if (User.HasFocusedDescendants(ThisRef))
			{
				User.SetFocus(ThisRef);
			}
		});

	}
	else
	{
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SRigVMGraphPinNameListValueWidget::SetFocusPostConstruct));
	}
}

EActiveTimerReturnType SRigVMGraphPinNameListValueWidget::SetFocusPostConstruct(double InCurrentTime, float InDeltaTime)
{
	if (SearchField.IsValid())
	{
		bool bSucceeded = false;
		FSlateApplication::Get().ForEachUser([&](FSlateUser& User) {
			bSucceeded |= User.SetFocus(SearchField.ToSharedRef());
		});

		if (bSucceeded)
		{
			return EActiveTimerReturnType::Stop;
		}
	}

	return EActiveTimerReturnType::Continue;
}

void SRigVMGraphPinNameListValueWidget::OnSelectionChanged_Internal(TSharedPtr<FRigVMStringWithTag> ProposedSelection, ESelectInfo::Type SelectInfo, bool bForce)
{
	// Ensure that the proposed selection is different
	if (SelectInfo != ESelectInfo::OnNavigation || bForce)
	{
		// Ensure that the proposed selection is different from selected
		if ((ProposedSelection.IsValid() && (ProposedSelection != SelectedItem)) || bForce)
		{
			SelectedItem = ProposedSelection;
			OnSelectionChanged.ExecuteIfBound(ProposedSelection, ESelectInfo::OnMouseClick); // SelectInfo);
		}
		// close combo even if user reselected item
		this->SetIsOpen(false);
	}
}

void SRigVMGraphPinNameListValueWidget::OnSearchTextChanged(const FText& ChangedText)
{
	FString SearchToken = ChangedText.ToString().ToLower();
	FString SearchTokenUnderscores = SearchToken.Replace(TEXT(" "), TEXT("_"));

	for (int32 i = 0; i < OptionsSource->Num(); i++)
	{
		const TSharedPtr<FRigVMStringWithTag>& Option = (*OptionsSource)[i];

		TSharedPtr<ITableRow> Row = ComboListView->WidgetFromItem(Option);
		if (Row)
		{
			if (SearchToken.IsEmpty())
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else if (Option->GetString().ToLower() == SearchToken)
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
				ComboListView->Private_ClearSelection();
				ComboListView->Private_SetItemSelection(Option, true);
			}
			else if ((Option->GetString().ToLower().Find(SearchToken) >= 0) ||
				(Option->GetString().ToLower().Find(SearchTokenUnderscores) >= 0))
			{
				Row->AsWidget()->SetVisibility(EVisibility::Visible);
			}
			else
			{
				Row->AsWidget()->SetVisibility(EVisibility::Collapsed);
			}
		}
	}

	ComboListView->RequestListRefresh();

	SelectedItem = TSharedPtr< FRigVMStringWithTag >();
}

void SRigVMGraphPinNameListValueWidget::OnSearchTextCommitted(const FText& ChangedText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		const FString LowerCaseProposedSelection = ChangedText.ToString().ToLower();
		for (int32 i = 0; i < OptionsSource->Num(); i++)
		{
			const TSharedPtr<FRigVMStringWithTag>& Option = (*OptionsSource)[i];
			if (Option->GetString().ToLower() == LowerCaseProposedSelection)
			{
				OnSelectionChanged_Internal(Option, ESelectInfo::OnKeyPress, true);
				return;
			}
		}

		if(AllowUserProvidedText)
		{
			const TSharedPtr<FRigVMStringWithTag> UserDefinedOption = MakeShareable(new FRigVMStringWithTag(ChangedText.ToString()));
			OnSelectionChanged_Internal(UserDefinedOption, ESelectInfo::OnKeyPress, true);
		}
	}
}

FReply SRigVMGraphPinNameListValueWidget::OnSearchTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Down)
	{
		for (int32 i = 0; i < OptionsSource->Num(); i++)
		{
			const TSharedPtr<FRigVMStringWithTag>& Option = (*OptionsSource)[i];
			TSharedPtr<ITableRow> Row = ComboListView->WidgetFromItem(Option);
			if (Row)
			{
				if (Row->AsWidget()->GetVisibility() == EVisibility::Visible)
				{
					ComboListView->SetSelection(Option, ESelectInfo::OnNavigation);
					ComboListView->RequestScrollIntoView(Option, 0);

					SelectedItem = Option;
					OnSelectionChanged.ExecuteIfBound(Option, ESelectInfo::OnNavigation);

					FSlateApplication::Get().SetKeyboardFocus(ComboListView.ToSharedRef(), EFocusCause::SetDirectly);
					return FReply::Handled();
				}
			}
		}
	}

	return FReply::Unhandled();
}

FReply SRigVMGraphPinNameListValueWidget::OnComboListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<TSharedPtr<FRigVMStringWithTag>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::OnKeyPress);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}


FReply SRigVMGraphPinNameListValueWidget::OnButtonClicked()
{
	// if user clicked to close the combo menu
	if (this->IsOpen())
	{
		// Re-select first selected item, just in case it was selected by navigation previously
		TArray<TSharedPtr<FRigVMStringWithTag>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			OnSelectionChanged_Internal(SelectedItems[0], ESelectInfo::Direct);
		}
	}
	else
	{
		SearchField->SetText(FText::GetEmpty());
		OnComboBoxOpening.ExecuteIfBound();
	}

	return SComboButton::OnButtonClicked();
}

#undef LOCTEXT_NAMESPACE
