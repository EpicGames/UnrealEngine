// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinQuat.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMModel/RigVMController.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Widgets/Layout/SBox.h"

void SRigVMGraphPinQuat::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	ModelPin = InArgs._ModelPin;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinQuat::GetDefaultValueWidget()
{
	TSharedRef<SWidget> Widget = SNew(SBox)
		.MinDesiredWidth(150.f)
		.MaxDesiredWidth(400.f)
		[
			SNew(SRotatorInputBox)
			.DisplayToggle(false)
			.bColorAxisLabels(true)
			.Pitch(this, &SRigVMGraphPinQuat::GetRotatorComponent, 0)
			.Yaw(this, &SRigVMGraphPinQuat::GetRotatorComponent, 1)
			.Roll(this, &SRigVMGraphPinQuat::GetRotatorComponent, 2)
			.OnPitchChanged(this, &SRigVMGraphPinQuat::OnRotatorComponentChanged, 0)
			.OnYawChanged(this, &SRigVMGraphPinQuat::OnRotatorComponentChanged, 1)
			.OnRollChanged(this, &SRigVMGraphPinQuat::OnRotatorComponentChanged, 2)
			.OnPitchCommitted(this, &SRigVMGraphPinQuat::OnRotatorComponentCommitted, 0, true)
			.OnYawCommitted(this, &SRigVMGraphPinQuat::OnRotatorComponentCommitted, 1, true)
			.OnRollCommitted(this, &SRigVMGraphPinQuat::OnRotatorComponentCommitted, 2, true)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		];
	return Widget;
}

TOptional<FRotator> SRigVMGraphPinQuat::GetRotator() const
{
	if(IsValid(ModelPin))
	{
		const FString DefaultValue = ModelPin->GetDefaultValueStoredByUserInterface();
		if(!DefaultValue.IsEmpty())
		{
			// try to import a quaternion
			{
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose);
				FQuat Quat = FQuat::Identity;
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity()); 
				TBaseStructure<FQuat>::Get()->ImportText(*DefaultValue, &Quat, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FQuat>::Get()->GetName(), true);
				if(ErrorPipe.NumErrors == 0)
				{
					return Quat.Rotator();
				}
			}

			// also try to import a rotator - in case that's how it got stored
			{
				FRigVMPinDefaultValueImportErrorContext ErrorPipe(ELogVerbosity::Verbose); 
				FRotator Rotator = FRotator::ZeroRotator;
				LOG_SCOPE_VERBOSITY_OVERRIDE(LogExec, ErrorPipe.GetMaxVerbosity()); 
				TBaseStructure<FRotator>::Get()->ImportText(*DefaultValue, &Rotator, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FQuat>::Get()->GetName(), true);
				if(ErrorPipe.NumErrors == 0)
				{
					return Rotator;
				}
			}
		}
	}
	return TOptional<FRotator>();
}

void SRigVMGraphPinQuat::OnRotatorCommitted(FRotator InRotator, ETextCommit::Type InCommitType, bool bUndoRedo)
{
	if(ModelPin)
	{
		FString NewDefaultValue;	
		TBaseStructure<FRotator>::Get()->ExportText(NewDefaultValue, &InRotator, &InRotator, nullptr, PPF_None, nullptr);

		if(FRigVMEditorAssetInterfacePtr Blueprint = IRigVMEditorAssetInterface::GetInterfaceOuter(ModelPin))
		{
			if(URigVMController* Controller = Blueprint->GetOrCreateController(ModelPin->GetGraph()))
			{
				Controller->SetPinDefaultValue(ModelPin->GetPinPath(), NewDefaultValue, true, bUndoRedo, false, bUndoRedo, true);
			}
		}
	}
}

TOptional<float> SRigVMGraphPinQuat::GetRotatorComponent(int32 InComponent) const
{
	TOptional<FRotator> Rotator = GetRotator();
	if(Rotator.IsSet())
	{
		if(InComponent == 0)
		{
			return static_cast<float>(Rotator.GetValue().Pitch);
		}
		if(InComponent == 1)
		{
			return static_cast<float>(Rotator.GetValue().Yaw);
		}
		return static_cast<float>(Rotator.GetValue().Roll);
	}
	return TOptional<float>();
}

void SRigVMGraphPinQuat::OnRotatorComponentChanged(float InValue, int32 InComponent)
{
	return OnRotatorComponentCommitted(InValue, ETextCommit::Default, InComponent, false);
}

void SRigVMGraphPinQuat::OnRotatorComponentCommitted(float InValue, ETextCommit::Type InCommitType, int32 InComponent, bool bUndoRedo)
{
	const TOptional<FRotator> OptionalRotator = GetRotator();
	if(OptionalRotator.IsSet())
	{
		FRotator Rotator = OptionalRotator.GetValue();
		if(InComponent == 0)
		{
			Rotator.Pitch = InValue;
		}
		else if(InComponent == 1)
		{
			Rotator.Yaw = InValue;
		}
		else
		{
			Rotator.Roll = InValue;
		}
		OnRotatorCommitted(Rotator, InCommitType, bUndoRedo);
	}
}
