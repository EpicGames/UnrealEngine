// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinUserDataNameSpace.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMCore/RigVMAssetUserData.h"
#include "Widgets/Layout/SBox.h"
#include "ScopedTransaction.h"

void SRigVMGraphPinUserDataNameSpace::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinUserDataNameSpace::GetDefaultValueWidget()
{
	TSharedPtr<FString> InitialSelected;
	TArray<TSharedPtr<FString>>& LocalNameSpaces = GetNameSpaces();
	for (TSharedPtr<FString> Item : LocalNameSpaces)
	{
		if (Item->Equals(GetNameSpaceText().ToString()))
		{
			InitialSelected = Item;
		}
	}

	return SNew(SBox)
		.MinDesiredWidth(150.f)
		.MaxDesiredWidth(400.f)
		[
			SAssignNew(NameComboBox, SRigVMGraphPinEditableNameValueWidget)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.OptionsSource(&NameSpaces)
				.OnGenerateWidget(this, &SRigVMGraphPinUserDataNameSpace::MakeNameSpaceItemWidget)
				.OnSelectionChanged(this, &SRigVMGraphPinUserDataNameSpace::OnNameSpaceChanged)
				.OnComboBoxOpening(this, &SRigVMGraphPinUserDataNameSpace::OnNameSpaceComboBox)
				.InitiallySelectedItem(InitialSelected)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SRigVMGraphPinUserDataNameSpace::GetNameSpaceText)
				]
		];
}

FText SRigVMGraphPinUserDataNameSpace::GetNameSpaceText() const
{
	return FText::FromString( GraphPinObj->GetDefaultAsString() );
}

void SRigVMGraphPinUserDataNameSpace::SetNameSpaceText(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if(!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeNameSpacePinValue", "Change Bone Name Pin Value" ) );
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
	}
}

TSharedRef<SWidget> SRigVMGraphPinUserDataNameSpace::MakeNameSpaceItemWidget(TSharedPtr<FString> InItem)
{
	return 	SNew(STextBlock).Text(FText::FromString(*InItem));// .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SRigVMGraphPinUserDataNameSpace::OnNameSpaceChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();
		SetNameSpaceText(FText::FromString(NewValue), ETextCommit::OnEnter);
	}
}

void SRigVMGraphPinUserDataNameSpace::OnNameSpaceComboBox()
{
	TSharedPtr<FString> CurrentlySelected;
	TArray<TSharedPtr<FString>>& LocalNameSpaces = GetNameSpaces();
	for (TSharedPtr<FString> Item : LocalNameSpaces)
	{
		if (Item->Equals(GetNameSpaceText().ToString()))
		{
			CurrentlySelected = Item;
		}
	}

	NameComboBox->SetSelectedItem(CurrentlySelected);
}

TArray<TSharedPtr<FString>>& SRigVMGraphPinUserDataNameSpace::GetNameSpaces()
{
	NameSpaces.Reset();

	TArray<FString> NameSpaceLookup;
	if (FRigVMEditorAssetInterfacePtr EditorAsset = IRigVMEditorAssetInterface::GetInterfaceOuter(GraphPinObj->GetOwningNode()))
	{
		if(const UObject* DebuggedObject = EditorAsset->GetObjectBeingDebugged())
		{
			if(DebuggedObject->Implements<UInterface_AssetUserData>())
			{
				const IInterface_AssetUserData* AssetUserDataHost = CastChecked<IInterface_AssetUserData>(DebuggedObject);
				if(const TArray<UAssetUserData*>* UserDataArray = AssetUserDataHost->GetAssetUserDataArray())
				{
					for(const UAssetUserData* UserData : *UserDataArray)
					{
						if(const UNameSpacedUserData* NameSpacedUserData = Cast<UNameSpacedUserData>(UserData))
						{
							const FString& NameSpace = NameSpacedUserData->NameSpace; 
							if(!NameSpaceLookup.Contains(NameSpace))
							{
								NameSpaceLookup.Add(NameSpace);
								NameSpaces.Add(MakeShared<FString>(NameSpace));
							}
						}
					}
				}
			}
		}
	}

	return NameSpaces;
}
