// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinUserDataPath.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EdGraphSchema_K2.h"
#include "DetailLayoutBuilder.h"
#include "RigVMModel/RigVMController.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "EdGraphSchema_K2.h"
#include "RigVMDeveloperTypeUtils.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphPinUserDataPath"

static const FText GraphPinUserDataPathMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");

void SRigVMUserDataPath::Construct(const FArguments& InArgs)
{
	ModelPins = InArgs._ModelPins;
	if(ModelPins.IsEmpty())
	{
		return;
	}

	bAllowUObjects = false;
	for(const URigVMPin* ModelPin : ModelPins)
	{
		if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(ModelPin->GetNode()))
		{
			if(const UScriptStruct* ScriptStruct = UnitNode->GetScriptStruct())
			{
				if(const FProperty* Property = ScriptStruct->FindPropertyByName(ModelPin->GetFName()))
				{
					bAllowUObjects = Property->HasMetaData(TEXT("AllowUObjects"));
				}
			}
		}
	}

	MenuAnchor = SNew(SMenuAnchor)
	.OnGetMenuContent(this, &SRigVMUserDataPath::GetTopLevelMenuContent)
	[
		SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton").ButtonStyle)
		.OnClicked_Lambda([this]() 
		{
			MenuAnchor->SetIsOpen(true); 
			return FReply::Handled(); 
		})
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(1.0f, 0.0f)
			[
				SNew(SImage)
				.Image(this, &SRigVMUserDataPath::GetUserDataIcon)
				.ColorAndOpacity_Lambda([this]() -> FSlateColor
				{
					return GetUserDataColor();
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(this, &SRigVMUserDataPath::GetUserDataPathText)
			]

			+ SHorizontalBox::Slot()
			.Padding(2, 0)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ChevronDown"))
			]
		]
	];

	this->ChildSlot
	[
		MenuAnchor.ToSharedRef()
	];
}

FString SRigVMUserDataPath::GetUserDataPath(URigVMPin* ModelPin) const
{
	if (ModelPin)
	{
		return ModelPin->GetDefaultValue();
	}
	return FString();
}

FString SRigVMUserDataPath::GetUserDataPath() const
{
	if (ModelPins.Num() > 0)
	{
		const FString FirstPath = GetUserDataPath(ModelPins[0]);
		for(int32 Index = 1; Index < ModelPins.Num(); Index++)
		{
			if(!GetUserDataPath(ModelPins[Index]).Equals(FirstPath, ESearchCase::CaseSensitive))
			{
				return FString();
			}
		}
		return FirstPath;
	}
	return FString();
}

const FSlateBrush* SRigVMUserDataPath::GetUserDataIcon() const
{
	if(const UNameSpacedUserData::FUserData* UserData = GetUserData())
	{
		return GetUserDataIcon(UserData);
	}
	return nullptr;
}

const FSlateBrush* SRigVMUserDataPath::GetUserDataIcon(const UNameSpacedUserData::FUserData* InUserData) const
{
	static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
	static const FLazyName ArrayTypeIcon(TEXT("Kismet.VariableList.ArrayTypeIcon"));

	if (InUserData)
	{
		if(InUserData->IsArray())
		{
			return FAppStyle::GetBrush(ArrayTypeIcon);
		}
	}
	return FAppStyle::GetBrush(TypeIcon);
}

FLinearColor SRigVMUserDataPath::GetUserDataColor() const
{
	if(const UNameSpacedUserData::FUserData* UserData = GetUserData())
	{
		return GetUserDataColor(UserData);
	}
	return FLinearColor::White;
}

FLinearColor SRigVMUserDataPath::GetUserDataColor(const UNameSpacedUserData::FUserData* InUserData) const
{
	if (InUserData)
	{
		const FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(FGuid(), InUserData->GetProperty(), nullptr);
		if(ExternalVariable.IsValid(true))
		{
			const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
			const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(ExternalVariable);
			return Schema->GetPinTypeColor(PinType);
		}
		if(bAllowUObjects && InUserData->IsUObject())
		{
			return FLinearColor::Blue;
		}
	}
	return FLinearColor::White;
}

TSharedRef<SWidget> SRigVMUserDataPath::GetTopLevelMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, true);
	FillUserDataPathMenu(MenuBuilder, FString());
	return MenuBuilder.MakeWidget();
}

void SRigVMUserDataPath::FillUserDataPathMenu(FMenuBuilder& InMenuBuilder, FString InParentPath)
{
	const UNameSpacedUserData* UserDataObject = GetUserDataObject();
	if(UserDataObject == nullptr)
	{
		return;
	}

	if(InParentPath.IsEmpty())
	{
		InMenuBuilder.BeginSection("UserData", LOCTEXT("UserData", "User Data"));
	}
	{
		static const FLazyName PropertyIcon(TEXT("Kismet.VariableList.TypeIcon"));
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

		const TArray<const UNameSpacedUserData::FUserData*> UserDataArray = UserDataObject->GetUserDataArray(InParentPath);

		for(const UNameSpacedUserData::FUserData* UserData : UserDataArray)
		{
			const FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(FGuid(), UserData->GetProperty(),nullptr);

			FEdGraphPinType PinType;

			bool bIsCompatible = ExternalVariable.IsValid(true);
			if(bIsCompatible)
			{
				PinType = RigVMTypeUtils::PinTypeFromExternalVariable(ExternalVariable);
			}
			if(!bIsCompatible)
			{
				if(bAllowUObjects && UserData->IsUObject())
				{
					UClass* PropertyClass = CastField<FObjectProperty>(UserData->GetProperty())->PropertyClass;
					PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
					PinType.PinSubCategoryObject = TWeakObjectPtr<UClass>(PropertyClass);
					bIsCompatible = true;
				}
			}

			if(!bIsCompatible)
			{
				continue;
			}

			const FUIAction Action(FExecuteAction::CreateSP(this, &SRigVMUserDataPath::HandleSetUserDataPath, UserData->GetPath()));

			TSharedRef<SWidget> Content = 
				SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(2.0f, 0.0f))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1.0f, 0.0f)
					[
						SNew(SImage)
						.Image(GetUserDataIcon(UserData))
						.ColorAndOpacity(GetUserDataColor(UserData))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(UserData->GetDisplayName()))
					];

			const FString& UserDataPath = UserData->GetPath();
			if(UserDataObject->GetUserDataArray(UserDataPath).Num() > 0)
			{
				InMenuBuilder.AddSubMenu(Action, Content, FNewMenuDelegate::CreateSP(this, &SRigVMUserDataPath::FillUserDataPathMenu, UserDataPath));
			}
			else
			{
				InMenuBuilder.AddMenuEntry(Action, Content);
			}
		}
	}
	if(InParentPath.IsEmpty())
	{
		InMenuBuilder.EndSection();
	}
}

void SRigVMUserDataPath::HandleSetUserDataPath(FString InUserDataPath)
{
	FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface();
	if(ModelPins.IsEmpty() || (Blueprint == nullptr))
	{
		return;
	}

	for(const URigVMPin* ModelPin : ModelPins)
	{
		URigVMGraph* Model = ModelPin->GetGraph();
		if(Model == nullptr)
		{
			continue;
		}

		URigVMController* Controller = Blueprint->GetOrCreateController(Model);
		if(Controller == nullptr)
		{
			continue;
		}

		Controller->OpenUndoBracket(TEXT("Set User Data Path"));
		Controller->SetPinDefaultValue(ModelPin->GetPinPath(), InUserDataPath);

		if(const UNameSpacedUserData::FUserData* UserData = GetUserData())
		{
			const FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(FGuid(), UserData->GetProperty(), nullptr);
			if(ExternalVariable.IsValid(true))
			{
				const TRigVMTypeIndex ExpectedTypeIndex = FRigVMRegistry::Get().GetTypeIndexFromCPPType(ExternalVariable.GetExtendedCPPType().ToString()); 
				static const TArray<FString> PinPathsToCheck = {TEXT("Default"), TEXT("Result")};
				if(const URigVMTemplateNode* TemplateNode = Cast<URigVMTemplateNode>(ModelPin->GetNode()))
				{
					for(const FString& PinPathToCheck : PinPathsToCheck)
					{
						if(const URigVMPin* PinToCheck = TemplateNode->FindPin(PinPathToCheck))
						{
							if(!FRigVMRegistry::Get().CanMatchTypes(PinToCheck->GetTypeIndex(), ExpectedTypeIndex, true))
							{
								Controller->UnresolveTemplateNodes({TemplateNode->GetFName()});
								Controller->ResolveWildCardPin(PinToCheck->GetPinPath(), ExpectedTypeIndex);
							}
						}
					}
				}
			}
		}

		Controller->CloseUndoBracket();
	}
}

FRigVMEditorAssetInterfacePtr SRigVMUserDataPath::GetRigVMAssetInterface() const
{
	if(!ModelPins.IsEmpty())
	{
		return IRigVMEditorAssetInterface::GetInterfaceOuter(ModelPins[0]);
	}
	return nullptr;
}

FString SRigVMUserDataPath::GetUserDataNameSpace(URigVMPin* ModelPin) const
{
	if(const URigVMNode* Node = ModelPin->GetNode())
	{
		static constexpr TCHAR NameSpaceName[] = TEXT("NameSpace");
		if(const URigVMPin* NameSpacePin = Node->FindPin(NameSpaceName))
		{
			return NameSpacePin->GetDefaultValue();
		}
	}
	return FString();
}

FString SRigVMUserDataPath::GetUserDataNameSpace() const
{
	if(ModelPins.Num() > 0)
	{
		const FString FirstNameSpace = GetUserDataNameSpace(ModelPins[0]);
		for(int32 Index = 1; Index < ModelPins.Num(); Index++)
		{
			if(!GetUserDataNameSpace(ModelPins[Index]).Equals(FirstNameSpace, ESearchCase::CaseSensitive))
			{
				return FString();
			}
		}
		return FirstNameSpace;
	}
	return FString();
}

const UNameSpacedUserData* SRigVMUserDataPath::GetUserDataObject() const
{
	const FString NameSpace = GetUserDataNameSpace();
	if(NameSpace.IsEmpty())
	{
		return nullptr;
	}
	
	if(const FRigVMEditorAssetInterfacePtr Blueprint = GetRigVMAssetInterface())
	{
		if(const UObject* DebuggedObject = Blueprint->GetObjectBeingDebugged())
		{
			if(DebuggedObject->Implements<UInterface_AssetUserData>())
			{
				const IInterface_AssetUserData* AssetUserDataHost = CastChecked<IInterface_AssetUserData>(DebuggedObject);
				if(const TArray<UAssetUserData*>* UserDataArray = AssetUserDataHost->GetAssetUserDataArray())
				{
					for(int32 Index = UserDataArray->Num() - 1; UserDataArray->IsValidIndex(Index); Index--)
					{
						const UAssetUserData* UserData = (*UserDataArray)[Index];
						if(const UNameSpacedUserData* NameSpacedUserData = Cast<UNameSpacedUserData>(UserData))
						{
							if(NameSpace.Equals(NameSpacedUserData->NameSpace, ESearchCase::CaseSensitive))
							{
								return NameSpacedUserData;
							}
						}
					}
				}
			}
		}
	}

	return nullptr;
}

const UNameSpacedUserData::FUserData* SRigVMUserDataPath::GetUserData() const
{
	const FString UserDataPath = GetUserDataPath();
	if(UserDataPath.IsEmpty())
	{
		return nullptr;
	}
	if(const UNameSpacedUserData* UserDataObject = GetUserDataObject())
	{
		return UserDataObject->GetUserData(UserDataPath);
	}
	return nullptr;
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SRigVMGraphPinUserDataPath::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	this->ModelPins = InArgs._ModelPins;
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinUserDataPath::GetDefaultValueWidget()
{
	return SNew(SRigVMUserDataPath)
		.ModelPins(ModelPins)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility);
}


#undef LOCTEXT_NAMESPACE
