// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinVariableBinding.h"
#include "Features/IModularFeatures.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "EdGraphSchema_K2.h"
#include "DetailLayoutBuilder.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMBlueprintGeneratedClass.h"
#include "RigVMDeveloperTypeUtils.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Kismet2/BlueprintEditorUtils.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphPinVariableBinding"

static const FText RigVMGraphVariableBindingMultipleValues = LOCTEXT("MultipleValues", "Multiple Values");

void SRigVMGraphVariableBinding::Construct(const FArguments& InArgs)
{
	this->ModelPins = InArgs._ModelPins;
	this->Blueprint = InArgs._Asset;
	this->bCanRemoveBinding = InArgs._CanRemoveBinding;

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	BindingArgs.CurrentBindingText.BindRaw(this, &SRigVMGraphVariableBinding::GetBindingText);
	BindingArgs.CurrentBindingImage.BindRaw(this, &SRigVMGraphVariableBinding::GetBindingImage);
	BindingArgs.CurrentBindingColor.BindRaw(this, &SRigVMGraphVariableBinding::GetBindingColor);

	BindingArgs.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda([this](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
	{
		return OnCanBindProperty(InProperty);
	});

	BindingArgs.OnCanBindToClass.BindSP(this, &SRigVMGraphVariableBinding::OnCanBindToClass);

	BindingArgs.OnAddBinding.BindSP(this, &SRigVMGraphVariableBinding::OnAddBinding);
	BindingArgs.OnCanRemoveBinding.BindSP(this, &SRigVMGraphVariableBinding::OnCanRemoveBinding);
	BindingArgs.OnRemoveBinding.BindSP(this, &SRigVMGraphVariableBinding::OnRemoveBinding);

	BindingArgs.bGeneratePureBindings = true;
	BindingArgs.bAllowNewBindings = true;
	BindingArgs.bAllowArrayElementBindings = false;
	BindingArgs.bAllowStructMemberBindings = false;
	BindingArgs.bAllowUObjectFunctions = false;

	BindingArgs.MenuExtender = MakeShareable(new FExtender);
	BindingArgs.MenuExtender->AddMenuExtension(
		"Properties",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateSP(this, &SRigVMGraphVariableBinding::FillLocalVariableMenu));

	if (Blueprint)
	{
		TArray<FBindingContextStruct> BindingContexts;
		const UStruct* Struct = Blueprint->GetVariablesStruct();
		BindingContexts.Add(const_cast<UStruct*>(Struct));
		this->ChildSlot
		[
			PropertyAccessEditor.MakePropertyBindingWidget(BindingContexts, BindingArgs)
		];
	}
}

FText SRigVMGraphVariableBinding::GetBindingText(URigVMPin* ModelPin) const
{
	if (ModelPin && ModelPin->GetGraph())
	{
		const FString VariablePath = ModelPin->GetBoundVariablePath();
		return FText::FromString(VariablePath);
	}
	return FText();
}

FText SRigVMGraphVariableBinding::GetBindingText() const
{
	if (ModelPins.Num() > 0)
	{
		const FText FirstText = GetBindingText(ModelPins[0]);
		for(int32 Index = 1; Index < ModelPins.Num(); Index++)
		{
			if(!GetBindingText(ModelPins[Index]).EqualTo(FirstText))
			{
				return RigVMGraphVariableBindingMultipleValues;
			}
		}
		return FirstText;
	}
	return FText();
}

const FSlateBrush* SRigVMGraphVariableBinding::GetBindingImage() const
{
	static const FLazyName TypeIcon(TEXT("Kismet.VariableList.TypeIcon"));
	static const FLazyName ArrayTypeIcon(TEXT("Kismet.VariableList.ArrayTypeIcon"));

	if (ModelPins.Num() > 0)
	{
		if(ModelPins[0]->IsArray())
		{
			return FAppStyle::GetBrush(ArrayTypeIcon);
		}
	}
	return FAppStyle::GetBrush(TypeIcon);
}

FLinearColor SRigVMGraphVariableBinding::GetBindingColor() const
{
	if (Blueprint)
	{
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();
		FName BoundVariable(NAME_None);

		if(ModelPins.Num() > 0)
		{
			BoundVariable = *ModelPins[0]->GetBoundVariableName();
		}

		for (const FRigVMGraphVariableDescription& VariableDescription : Blueprint->GetAssetVariables())
		{
			if (VariableDescription.Name == BoundVariable)
			{
				const FEdGraphPinType EdPinType = RigVMTypeUtils::PinTypeFromRigVMVariableDescription(VariableDescription);
				return Schema->GetPinTypeColor(EdPinType);
			}
		}

		if (ModelPins.Num() > 0)
		{
			URigVMGraph* Model = ModelPins[0]->GetGraph();
			if(Model == nullptr)
			{
				return  FLinearColor::Red;
			}
			if (BoundVariable.IsNone() && ModelPins[0]->IsDefinedAsInputVariable())
			{
				return FLinearColor::Red;
			}

			const TArray<FRigVMGraphVariableDescription>& LocalVariables =  Model->GetLocalVariables(true);
			for(const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
			{
				const FRigVMExternalVariable ExternalVariable = LocalVariable.ToExternalVariable();
				if(!ExternalVariable.IsValid(true))
				{
					continue;
				}

				if (ExternalVariable.GetName() == BoundVariable)
				{
					const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(ExternalVariable);
					return Schema->GetPinTypeColor(PinType);
				}
			}

			const TArray<FRigVMGraphVariableDescription>& InputVariables =  Model->GetInputVariables();
			for(const FRigVMGraphVariableDescription& InputVariable : InputVariables)
			{
				const FRigVMExternalVariable ExternalVariable = InputVariable.ToExternalVariable();
				if(!ExternalVariable.IsValid(true))
				{
					continue;
				}

				if (ExternalVariable.GetName() == BoundVariable)
				{
					const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(ExternalVariable);
					return Schema->GetPinTypeColor(PinType);
				}
			}
		}
	}
	return FLinearColor::White;
}

bool SRigVMGraphVariableBinding::OnCanBindProperty(FProperty* InProperty) const
{
	if (InProperty == BindingArgs.Property)
	{
		return true;
	}

	if (InProperty)
	{
		const FRigVMExternalVariable ExternalVariable = FRigVMExternalVariable::Make(FGuid(), InProperty, nullptr);
		if(ModelPins.Num() > 0)
		{
			return ModelPins[0]->CanBeBoundToVariable(ExternalVariable);
		}
	}

	return false;
}

bool SRigVMGraphVariableBinding::OnCanBindToClass(UClass* InClass) const
{
	if (InClass)
	{
		return InClass->ClassGeneratedBy == Blueprint.GetObject();
	}
	return true;
}

void SRigVMGraphVariableBinding::OnAddBinding(FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
{
	if (Blueprint)
	{
		TArray<FString> Parts;
		for (const FBindingChainElement& ChainElement : InBindingChain)
		{
			//ensure(ChainElement.Field);
			if (!ChainElement.Field.GetFName().IsNone())
			{
				Parts.Add(ChainElement.Field.GetName());
			}
		}

		if(ModelPins.Num() > 0)
		{
			for(URigVMPin* ModelPin : ModelPins)
			{
				Blueprint->GetController(ModelPin->GetGraph())->BindPinToVariable(ModelPin->GetPinPath(), FString::Join(Parts, TEXT(".")), true /* undo */, true /* python */);
			}
		}
	}
}

bool SRigVMGraphVariableBinding::OnCanRemoveBinding(FName InPropertyName)
{
	return bCanRemoveBinding;
}

void SRigVMGraphVariableBinding::OnRemoveBinding(FName InPropertyName)
{
	if (Blueprint)
	{
		if(ModelPins.Num() > 0)
		{
			for(URigVMPin* ModelPin : ModelPins)
			{
				Blueprint->GetController(ModelPin->GetGraph())->UnbindPinFromVariable(ModelPin->GetPinPath(), true /* undo */, true /* python */);
			}
		}
	}
}

void SRigVMGraphVariableBinding::FillLocalVariableMenu(FMenuBuilder& MenuBuilder)
{
	if(ModelPins.Num() == 0)
	{
		return;
	}

	URigVMGraph* Model = ModelPins[0]->GetGraph();
	if(Model == nullptr)
	{
		return;
	}

	const bool bAllowInputArguments = !ModelPins[0]->IsDefinedAsInputVariable();
	constexpr bool bAllowInputVariables = true;

	TArray<FRigVMGraphVariableDescription> ValidVariables;
	const TArray<FRigVMGraphVariableDescription>& LocalVariables = Model->GetLocalVariables(bAllowInputArguments);
	for(const FRigVMGraphVariableDescription& LocalVariable : LocalVariables)
	{
		const FRigVMExternalVariable ExternalVariable = LocalVariable.ToExternalVariable();
		if(!ExternalVariable.IsValid(true))
		{
			continue;
		}
		if(!ModelPins[0]->CanBeBoundToVariable(ExternalVariable))
		{
			continue;
		}
		ValidVariables.Add(LocalVariable);
	}
	const TArray<FRigVMGraphVariableDescription>& InputVariables = Model->GetInputVariables();
	for(const FRigVMGraphVariableDescription& InputVariable : InputVariables)
	{
		const FRigVMExternalVariable ExternalVariable = InputVariable.ToExternalVariable();
		if(!ExternalVariable.IsValid(true))
		{
			continue;
		}
		if(!ModelPins[0]->CanBeBoundToVariable(ExternalVariable))
		{
			continue;
		}
		ValidVariables.Add(InputVariable);
	}

	if(ValidVariables.IsEmpty())
	{
		return;
	}
	
	MenuBuilder.BeginSection("Variables", LOCTEXT("Variables", "Variables"));
	{
		static const FLazyName PropertyIcon(TEXT("Kismet.VariableList.TypeIcon"));
		const URigVMEdGraphSchema* Schema = GetDefault<URigVMEdGraphSchema>();

		for(const FRigVMGraphVariableDescription& ValidVariable : ValidVariables)
		{
			const FRigVMExternalVariable ExternalVariable = ValidVariable.ToExternalVariable();
			const FEdGraphPinType PinType = RigVMTypeUtils::PinTypeFromExternalVariable(ExternalVariable);

			MenuBuilder.AddMenuEntry(
				FUIAction(FExecuteAction::CreateSP(this, &SRigVMGraphVariableBinding::HandleBindToLocalVariable, ValidVariable)),
				SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(18.0f, 0.0f))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FBlueprintEditorUtils::GetIconFromPin(PinType, true))
						.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromName(ValidVariable.Name))
					]);
		}
	}
	MenuBuilder.EndSection(); // Local Variables
}

void SRigVMGraphVariableBinding::HandleBindToLocalVariable(FRigVMGraphVariableDescription InLocalVariable)
{
	if(ModelPins.IsEmpty() || (Blueprint == nullptr))
	{
		return;
	}

	for(URigVMPin* ModelPin : ModelPins)
	{
		URigVMGraph* Model = ModelPin->GetGraph();
		if(Model == nullptr)
		{
			continue;
		}

		URigVMController* Controller = Blueprint->GetOrCreateController(Model);
		if(Controller == nullptr)
		{
			continue;
		}

		Controller->BindPinToVariable(ModelPin->GetPinPath(), InLocalVariable.Name.ToString(), true, true);
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////

void SRigVMGraphPinVariableBinding::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	this->ModelPins = InArgs._ModelPins;
	this->Blueprint = InArgs._Asset;

	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinVariableBinding::GetDefaultValueWidget()
{
	return SNew(SRigVMGraphVariableBinding)
		.Asset(Blueprint)
		.ModelPins(ModelPins);
}

#undef LOCTEXT_NAMESPACE