// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMGraphPinVariableName.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "EdGraphSchema_K2.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "RigVMBlueprintLegacy.h"

void SRigVMGraphPinVariableName::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SRigVMGraphPinVariableName::GetDefaultValueWidget()
{
	URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GraphPinObj->GetOwningNode());
	URigVMGraph* Model = RigNode->GetModelNode()->GetGraph();

	TSharedPtr<FString> InitialSelected;

	TArray<TSharedPtr<FString>>& LocalVariableNames = GetVariableNames();
	for (TSharedPtr<FString> Item : LocalVariableNames)
	{
		if (Item->Equals(GetVariableNameText().ToString()))
		{
			InitialSelected = Item;
		}
	}

	return SNew(SBox)
		.MinDesiredWidth(150.f)
		.MaxDesiredWidth(400.f)
		[
			SAssignNew(NameComboBox, SRigVMGraphPinEditableNameValueWidget)
				.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
				.OptionsSource(&VariableNames)
				.OnGenerateWidget(this, &SRigVMGraphPinVariableName::MakeVariableNameItemWidget)
				.OnSelectionChanged(this, &SRigVMGraphPinVariableName::OnVariableNameChanged)
				.OnComboBoxOpening(this, &SRigVMGraphPinVariableName::OnVariableNameComboBox)
				.InitiallySelectedItem(InitialSelected)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SRigVMGraphPinVariableName::GetVariableNameText)
				]
		];
}

FText SRigVMGraphPinVariableName::GetVariableNameText() const
{
	return FText::FromString( GraphPinObj->GetDefaultAsString() );
}

void SRigVMGraphPinVariableName::SetVariableNameText(const FText& NewTypeInValue, ETextCommit::Type /*CommitInfo*/)
{
	if(!GraphPinObj->GetDefaultAsString().Equals(NewTypeInValue.ToString()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("GraphEditor", "ChangeVariableNamePinValue", "Change Bone Name Pin Value" ) );
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, NewTypeInValue.ToString());
	}
}

TSharedRef<SWidget> SRigVMGraphPinVariableName::MakeVariableNameItemWidget(TSharedPtr<FString> InItem)
{
	return 	SNew(STextBlock).Text(FText::FromString(*InItem));// .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

void SRigVMGraphPinVariableName::OnVariableNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();
		SetVariableNameText(FText::FromString(NewValue), ETextCommit::OnEnter);
	}
}

void SRigVMGraphPinVariableName::OnVariableNameComboBox()
{
	TSharedPtr<FString> CurrentlySelected;
	TArray<TSharedPtr<FString>>& LocalVariableNames = GetVariableNames();
	for (TSharedPtr<FString> Item : LocalVariableNames)
	{
		if (Item->Equals(GetVariableNameText().ToString()))
		{
			CurrentlySelected = Item;
		}
	}

	NameComboBox->SetSelectedItem(CurrentlySelected);
}

TArray<TSharedPtr<FString>>& SRigVMGraphPinVariableName::GetVariableNames()
{
	if(URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(GraphPinObj->GetOwningNode()))
	{
		if(URigVMVariableNode* ModelNode = Cast<URigVMVariableNode>(RigNode->GetModelNode()))
		{
			VariableNames.Reset();

			FRigVMGraphVariableDescription MyDescription = ModelNode->GetVariableDescription();

			TArray<FRigVMGraphVariableDescription> VariableDescriptions = ModelNode->GetGraph()->GetVariableDescriptions();
			Algo::SortBy(VariableDescriptions, &FRigVMGraphVariableDescription::Name, FNameLexicalLess());

			for (FRigVMGraphVariableDescription& VariableDescription : VariableDescriptions)
			{
				if (VariableDescription.CPPType == MyDescription.CPPType)
				{
					VariableNames.Add(MakeShared<FString>(VariableDescription.Name.ToString()));
				}
			}

			if (FRigVMEditorAssetInterfacePtr Blueprint = ModelNode->GetGraph()->GetOuter())
			{
				for (FRigVMGraphVariableDescription& BPVariable : Blueprint->GetAssetVariables())
				{
					FRigVMGraphVariableDescription* Found = VariableDescriptions.FindByPredicate([&BPVariable](const FRigVMGraphVariableDescription& Description) {
						if (Description.Name == BPVariable.Name)
						{
							return true;
						}
						return false;
					});

					if (!Found)
					{
						VariableNames.Add(MakeShared<FString>(BPVariable.Name.ToString()));
					}
				}
			}
		}
	}

	return VariableNames;
}
