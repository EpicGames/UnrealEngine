// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMLogWidget.h"
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"
#include "UserInterface/SMessageLogListing.h"
#include "Kismet2/CompilerResultsLog.h"

#define LOCTEXT_NAMESPACE "SRigVMLogWidget"

SRigVMLogWidget::SRigVMLogWidget()
{
}

SRigVMLogWidget::~SRigVMLogWidget()
{
	for(const TWeakPtr<IMessageLogListing>& Listing : BoundListings)
	{
		if(Listing.IsValid())
		{
			Listing.Pin()->OnDataChanged().RemoveAll(this);
		}
	}
}

void SRigVMLogWidget::Construct(
	const FArguments& InArgs)
{
	ListingModel = FMessageLogListingModel::Create(InArgs._LogName);
	ListingView = FMessageLogListingViewModel::Create(ListingModel.ToSharedRef(), InArgs._LogLabel);
	ListingView->SetShowFilters(InArgs._ShowFilters);
	ListingView->SetAllowClear(InArgs._AllowClear);
	ListingView->SetDiscardDuplicates(InArgs._DiscardDuplicates);
	ListingView->SetScrollToBottom(InArgs._ScrollToBottom);

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(InArgs._HeightOverride)
		[
			SNew(SMessageLogListing, ListingView.ToSharedRef())
		]
	];
}

void SRigVMLogWidget::BindLog(const FName& InLogName)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if(MessageLogModule.IsRegisteredLogListing(InLogName))
	{
		const TSharedRef<IMessageLogListing> Listing = MessageLogModule.GetLogListing(InLogName);
		BindLog(Listing);
	}
}

void SRigVMLogWidget::UnbindLog(const FName& InLogName)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if(MessageLogModule.IsRegisteredLogListing(InLogName))
	{
		const TSharedRef<IMessageLogListing> Listing = MessageLogModule.GetLogListing(InLogName);
		UnbindLog(Listing);
	}
}

void SRigVMLogWidget::BindLog(TSharedPtr<IMessageLogListing> InListing)
{
	if(InListing.IsValid())
	{
		const TWeakPtr<IMessageLogListing> WeakListing = InListing.ToWeakPtr();
		if(!BoundListings.Contains(WeakListing))
		{
			BoundListings.Add(WeakListing);
			ListingInfo.FindOrAdd(InListing->GetName()) = FBoundListingInfo(*InListing.Get());
			(void)InListing->OnDataChanged().AddSP(this, &SRigVMLogWidget::OnBoundListingChanged, WeakListing);
		}
	}
}

void SRigVMLogWidget::UnbindLog(TSharedPtr<IMessageLogListing> InListing)
{
	if(InListing.IsValid())
	{
		BoundListings.Remove(InListing.ToWeakPtr());
		ListingInfo.Remove(InListing->GetName());
		(void)InListing->OnDataChanged().RemoveAll(this);
	}
}

void SRigVMLogWidget::BindLog(const FRigVMEditorAssetInterfacePtr InRigVMBlueprint)
{
	TSharedPtr<IMessageLogListing> Listing = nullptr;
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	const FName LogName = InRigVMBlueprint->GetCompileLog().GetLogName();

	// Reuse any existing log, or create a new one (that is not held onto bey the message log system)
	if(MessageLogModule.IsRegisteredLogListing(LogName))
	{
		Listing = MessageLogModule.GetLogListing(LogName);
	}
	else
	{
		FMessageLogInitializationOptions LogInitOptions;
		LogInitOptions.bShowInLogWindow = false;
		Listing = MessageLogModule.CreateLogListing(LogName, LogInitOptions);
	}
	
	if (Listing)
	{
		BindLog(Listing);
	}
}

void SRigVMLogWidget::UnbindLog(const FRigVMEditorAssetInterfacePtr InRigVMBlueprint)
{
	TSharedPtr<IMessageLogListing> Listing = nullptr;
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	const FName LogName = InRigVMBlueprint->GetCompileLog().GetLogName();

	// Reuse any existing log, or create a new one (that is not held onto bey the message log system)
	if(MessageLogModule.IsRegisteredLogListing(LogName))
	{
		Listing = MessageLogModule.GetLogListing(LogName);
	}
	else
	{
		FMessageLogInitializationOptions LogInitOptions;
		LogInitOptions.bShowInLogWindow = false;
		Listing = MessageLogModule.CreateLogListing(LogName, LogInitOptions);
	}
	
	if (Listing)
	{
		UnbindLog(Listing);
	}
}

void SRigVMLogWidget::OnBoundListingChanged(TWeakPtr<IMessageLogListing> InWeakListing)
{
	if(!InWeakListing.IsValid())
	{
		return;
	}

	const IMessageLogListing* Listing = InWeakListing.Pin().Get();

	FBoundListingInfo& Info = ListingInfo.FindChecked(Listing->GetName());

	const TArray< TSharedRef<class FTokenizedMessage> >& Messages = Listing->GetFilteredMessages();
	while(Messages.Num() > Info.NumMessages)
	{
		ListingModel->AddMessage(Messages[Info.NumMessages++]);
	}
}

SRigVMLogWidget::FBoundListingInfo::FBoundListingInfo(const IMessageLogListing& InListing)
	: NumMessages(InListing.GetFilteredMessages().Num())
{
}

#undef LOCTEXT_NAMESPACE
