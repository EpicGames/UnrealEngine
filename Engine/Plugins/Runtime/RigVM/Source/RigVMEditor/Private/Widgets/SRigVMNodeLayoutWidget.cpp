// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMNodeLayoutWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "DetailLayoutBuilder.h"
#include "RigVMStringUtils.h"
#include "SPositiveActionButton.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SRigVMNodeLayoutWidget"

SRigVMNodeLayoutWidget::SRigVMNodeLayoutWidget()
{
}

SRigVMNodeLayoutWidget::~SRigVMNodeLayoutWidget()
{
}

void SRigVMNodeLayoutWidget::Construct(
	const FArguments& InArgs)
{
	OnGetUncategorizedPins = InArgs._OnGetUncategorizedPins;
	OnGetCategories = InArgs._OnGetCategories;
	OnCategoryAdded = InArgs._OnCategoryAdded;
	OnCategoryRemoved = InArgs._OnCategoryRemoved;
	OnCategoryRenamed = InArgs._OnCategoryRenamed;
	OnGetElementLabel = InArgs._OnGetElementLabel;
	OnElementLabelChanged = InArgs._OnElementLabelChanged;
	OnGetElementCategory = InArgs._OnGetElementCategory;
	OnGetElementIndexInCategory = InArgs._OnGetElementIndexInCategory;
	OnGetElementColor = InArgs._OnGetElementColor;
	OnGetElementIcon = InArgs._OnGetElementIcon;
	OnElementIndexInCategoryChanged = InArgs._OnElementIndexInCategoryChanged;
	OnElementCategoryChanged = InArgs._OnElementCategoryChanged;
	OnGetStructuralHash = InArgs._OnGetStructuralHash;
	OnValidateCategoryName = InArgs._OnValidateCategoryName;
	OnValidateElementName = InArgs._OnValidateElementName;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(0, 4, 4, 0)
		[
			SNew(SPositiveActionButton)
			.OnClicked(this, &SRigVMNodeLayoutWidget::HandleAddCategory)
			.Cursor(EMouseCursor::Default)
			.Visibility_Lambda([this]()
			{
				return IsNodeLayoutEditable() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.ToolTipText(LOCTEXT("AddCategory", "Add Category"))
			.Text(LOCTEXT("AddCategory", "Add Category"))
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.HAlign(HAlign_Fill)
		.Padding(4, 4, 4, 4)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SBox)
				.MaxDesiredHeight(InArgs._MaxScrollBoxSize)
				[
					SNew(SScrollBox)
					+SScrollBox::Slot()
					.VAlign(VAlign_Fill)
					[
						SAssignNew(TreeView, STreeView<TSharedPtr<FNodeLayoutRow>>)
						.SelectionMode(ESelectionMode::Single)
						.TreeItemsSource(&NodeLayoutRows)
						.OnGenerateRow(this, &SRigVMNodeLayoutWidget::GenerateRow)
						.OnGetChildren(this, &SRigVMNodeLayoutWidget::GetChildrenForRow)
						.OnExpansionChanged(this, &SRigVMNodeLayoutWidget::OnItemExpansionChanged)
						.OnSelectionChanged(this, &SRigVMNodeLayoutWidget::OnItemSelectionChanged)
					]
				]
			]
		]
	];

	Refresh();

	if(OnGetStructuralHash.IsBound())
	{
		SetCanTick(true);
	}
}

void SRigVMNodeLayoutWidget::Refresh()
{
	struct FItemState : FNodeLayoutRow::FState
	{
		static void RecordItemState(const TSharedPtr<FNodeLayoutRow>& InRow, TMap<FString, FItemState>& OutItemStates)
		{
			FItemState State;
			*static_cast<FNodeLayoutRow::FState*>(&State) = InRow->State;
			OutItemStates.FindOrAdd(InRow->Path, State);
			RecordItemStates(InRow->ChildRows, OutItemStates);
		}

		static void RecordItemStates(const TArray<TSharedPtr<FNodeLayoutRow>>& InRows, TMap<FString, FItemState>& OutItemStates)
		{
			for(const TSharedPtr<FNodeLayoutRow>& Row : InRows)
			{
				RecordItemState(Row, OutItemStates);
			}
		}

		static void ApplyItemState(const TSharedPtr<FNodeLayoutRow>& InOutRow, const TMap<FString, FItemState>& InItemStates)
		{
			if(const FItemState* State = InItemStates.Find(InOutRow->Path))
			{
				InOutRow->State = *State;
			}
			ApplyItemStates(InOutRow->ChildRows, InItemStates);
		}

		static void ApplyItemStates(const TArray<TSharedPtr<FNodeLayoutRow>>& InOutRows, const TMap<FString, FItemState>& InItemStates)
		{
			for(const TSharedPtr<FNodeLayoutRow>& Row : InOutRows)
			{
				ApplyItemState(Row, InItemStates);
			}
		}
	};
	
	TMap<FString, FItemState> ItemStates;
	FItemState::RecordItemStates(NodeLayoutRows, ItemStates);
	NodeLayoutRows.Reset();

	TMap<FString, TSharedPtr<FNodeLayoutRow>> PathToRow;

	auto AddElement = [&PathToRow, this](const FString& InElementPath, bool bAddToList) -> TSharedPtr<FNodeLayoutRow>
	{
		TSharedPtr<FNodeLayoutRow> Row = MakeShareable(new FNodeLayoutRow);
		Row->bIsCategory = false;
		Row->bIsUncategorized = true;
		Row->Path = Row->Label = InElementPath;

		if(bAddToList)
		{
			FString ParentPath;
			if(RigVMStringUtils::SplitPinPathAtEnd(Row->Path, ParentPath, Row->Label))
			{
				if(const TSharedPtr<FNodeLayoutRow>* ParentRow = PathToRow.Find(ParentPath))
				{
					(*ParentRow)->ChildRows.Add(Row);
				}
				else
				{
					NodeLayoutRows.Add(Row);
				}
			}
			else
			{
				NodeLayoutRows.Add(Row);
			}
		}
		PathToRow.Add(Row->Path, Row);

		//  use the user defined label if need be
		if(OnGetElementLabel.IsBound())
		{
			const FString Label = OnGetElementLabel.Execute(Row->Path);
			if(!Label.IsEmpty())
			{
				Row->Label = Label;
			}
		}
		if(OnGetElementColor.IsBound())
		{
			Row->Color = OnGetElementColor.Execute(Row->Path);
		}
		if(OnGetElementIcon.IsBound())
		{
			Row->Icon = OnGetElementIcon.Execute(Row->Path);
		}

		return Row;
	};

	if(OnGetCategories.IsBound())
	{
		const TArray<FRigVMPinCategory> Categories = OnGetCategories.Execute();
		for(const FRigVMPinCategory& Category : Categories)
		{
			TSharedPtr<FNodeLayoutRow> Row = MakeShareable(new FNodeLayoutRow);
			Row->bIsCategory = true;
			Row->bIsUncategorized = true;
			Row->Path = Category.Path;
			Row->Label = Category.GetName();
			// this may get overriden later when the states get reapplied
			Row->State.bExpanded = true;
			
			FString ParentPath, LastBit;;
			if(RigVMStringUtils::SplitNodePathAtEnd(Row->Path, ParentPath, LastBit))
			{
				if(const TSharedPtr<FNodeLayoutRow>* ParentRow = PathToRow.Find(ParentPath))
				{
					(*ParentRow)->ChildRows.Add(Row);
				}
				else
				{
					NodeLayoutRows.Add(Row);
				}
			}
			else
			{
				NodeLayoutRows.Add(Row);
			}
			PathToRow.Add(Row->Path, Row);

			for(const FString& Element : Category.Elements)
			{
				TSharedPtr<FNodeLayoutRow> ElementRow = AddElement(Element, false);
				ElementRow->bIsUncategorized = false;
				Row->ChildRows.Add(ElementRow);
			}

			if(!ItemStates.Contains(Row->Path))
			{
				FItemState State;
				*static_cast<FNodeLayoutRow::FState*>(&State) = Row->State;
				ItemStates.Add(Row->Path, State);
			}
		}
	}

	if(OnGetUncategorizedPins.IsBound())
	{
		const TArray<FString> PinPaths = OnGetUncategorizedPins.Execute();
		for(const FString& PinPath : PinPaths)
		{
			(void)AddElement(PinPath, true);
		}
	}
	
	TreeView->RequestTreeRefresh();
	
	FItemState::ApplyItemStates(NodeLayoutRows, ItemStates);
	TreeView->ClearSelection();

	for(const TPair<FString, FItemState>& Pair : ItemStates)
	{
		if(const TSharedPtr<FNodeLayoutRow>* RowPtr = PathToRow.Find(Pair.Key))
		{
			const TSharedPtr<FNodeLayoutRow>& Row = *RowPtr;
			if(Row)
			{
				TreeView->SetItemExpansion(Row, Pair.Value.bExpanded);
				TreeView->SetItemSelection(Row, Pair.Value.bSelected);
			}
		}
	}
}

void SRigVMNodeLayoutWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if(OnGetStructuralHash.IsBound())
	{
		const uint32 CurrentStructuralHash = OnGetStructuralHash.Execute();
		if(!LastStructuralHash.IsSet() || LastStructuralHash.GetValue() != CurrentStructuralHash)
		{
			LastStructuralHash = CurrentStructuralHash;
			Refresh();
		}
	}
}

FReply SRigVMNodeLayoutWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(InKeyEvent.GetKey() == EKeys::F2)
	{
		const TArray<TSharedPtr<FNodeLayoutRow>> SelectedRows = TreeView->GetSelectedItems();
		if(!SelectedRows.IsEmpty())
		{
			SelectedRows[0]->RequestRename();
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::~SRigVMNowLayoutRow()
{
}

void SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	NodeLayoutRow = InArgs._NodeLayoutRow;
	check(NodeLayoutRow.IsValid());
	OnGetCategories = InArgs._OnGetCategories;
	OnCategoryRenamed = InArgs._OnCategoryRenamed;
	OnElementLabelChanged = InArgs._OnElementLabelChanged;
	OnElementCategoryChanged = InArgs._OnElementCategoryChanged;
	OnCategoryRemoved = InArgs._OnCategoryRemoved;
	OnValidateCategoryName = InArgs._OnValidateCategoryName;
	OnValidateElementName = InArgs._OnValidateElementName;

	const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	LabelEditWidget = ConstructLabel(NodeLayoutRow, HorizontalBox, this);

	if(NodeLayoutRow->IsCategory() || NodeLayoutRow->IsCategorizedPin())
	{
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Fill)
		[
			SNew(SSpacer)
		];

		HorizontalBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
			.ToolTipText_Lambda([this]()
			{
				if(NodeLayoutRow->IsCategory())
				{
					return LOCTEXT("RemoveCategory", "Remove category");
				}

				return LOCTEXT("RemoveFromCategory", "Remove from category");
			})
			.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
			.Visibility_Lambda([this]()
			{
				if(NodeLayoutRow->IsCategory())
				{
					if(OnGetCategories.IsBound())
					{
						if(OnGetCategories.Execute().Num() > 1)
						{
							return EVisibility::Collapsed;
						}
					}
				}
				return EVisibility::Visible;
			})
			.OnClicked_Lambda([this]() -> FReply
			{
				if(NodeLayoutRow->IsCategory())
				{
					if(OnCategoryRemoved.IsBound())
					{
						OnCategoryRemoved.Execute(NodeLayoutRow->Path);
						return FReply::Handled();
					}
				}
				else
				{
					if(OnElementCategoryChanged.IsBound())
					{
						OnElementCategoryChanged.Execute(NodeLayoutRow->Path, FString());
						return FReply::Handled();
					}
				}
				return FReply::Unhandled();
			})
			.ContentPadding(0.0f)
			[ 
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Icons.X") )
				.DesiredSizeOverride(FVector2D(16, 16))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}
	
	STableRow<TSharedPtr<FNodeLayoutRow>>::FArguments SuperArguments;
	SuperArguments.Content()
	[
		HorizontalBox
	];
	SuperArguments.Padding(0.f);
	SuperArguments.OnCanAcceptDrop(InArgs._OnCanAcceptDrop);
	SuperArguments.OnAcceptDrop(InArgs._OnAcceptDrop);
	SuperArguments.OnPaintDropIndicator(InArgs._OnPaintDropIndicator);
	SuperArguments.OnDragDetected(InArgs._OnDragDetected);
	SuperArguments.OnDragEnter(InArgs._OnDragEnter);
	SuperArguments.OnDragLeave(InArgs._OnDragLeave);
	SuperArguments.OnDrop(InArgs._OnDrop);
	STableRow< TSharedPtr<FNodeLayoutRow> >::Construct(SuperArguments, OwnerTableView);
}

TSharedPtr<SInlineEditableTextBlock> SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::ConstructLabel(TSharedPtr<FNodeLayoutRow> InNodeLayoutRow, TSharedRef<SHorizontalBox> OutHorizontalBox, SRigVMNowLayoutRow* InRow)
{
	if(InNodeLayoutRow->Icon)
	{
		OutHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0, 0, 4, 0)
		[
			SNew(SImage)
			.Image(InNodeLayoutRow->Icon)
			.DesiredSizeOverride(FVector2D(16, 16))
			.ColorAndOpacity(InNodeLayoutRow->Color)
		];
	}

	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;

	if(InRow && (
		(InNodeLayoutRow->IsCategory() &&
			!InNodeLayoutRow->Path.Equals(FRigVMPinCategory::GetDefaultCategoryName(), ESearchCase::IgnoreCase)
		) || InNodeLayoutRow->IsCategorizedPin()))
	{
		OutHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
			.Text(FText::FromString(InNodeLayoutRow->Label))
			.OnVerifyTextChanged(InRow, &SRigVMNowLayoutRow::OnVerifyLabelChanged)
			.OnTextCommitted(InRow, &SRigVMNowLayoutRow::OnLabelCommitted)
			.IsSelected(InRow, &SRigVMNowLayoutRow::IsSelected)
			.MultiLine(false)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

		InNodeLayoutRow->OnRequestRename.BindSP(InlineEditableTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}
	else
	{
		OutHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromString(InNodeLayoutRow->Label))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	}
	
	return InlineEditableTextBlock;
}

void SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::OnLabelCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	if(NodeLayoutRow->IsCategory())
	{
		if(OnCategoryRenamed.IsBound())
		{
			FString NewPath = InText.ToString();
			FString Left, Right = NodeLayoutRow->Path;
			if(RigVMStringUtils::SplitNodePathAtEnd(NodeLayoutRow->Path, Left, Right))
			{
				NewPath = RigVMStringUtils::JoinNodePath(Left, NewPath);
			}
			OnCategoryRenamed.Execute(NodeLayoutRow->Path, NewPath);
		}
	}
	else if(OnElementLabelChanged.IsBound())
	{
		OnElementLabelChanged.Execute(NodeLayoutRow->Path, InText.ToString());
	}
}

bool SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::OnVerifyLabelChanged(const FText& InText, FText& OutErrorMessage)
{
	if(NodeLayoutRow->IsCategory())
	{
		if(OnValidateCategoryName.IsBound())
		{
			return OnValidateCategoryName.Execute(NodeLayoutRow->Path, InText.ToString(), OutErrorMessage);
		}
	}
	if(OnValidateElementName.IsBound())
	{
		return OnValidateElementName.Execute(NodeLayoutRow->Path, InText.ToString(), OutErrorMessage);
	}
	return true;
}

bool SRigVMNodeLayoutWidget::SRigVMNowLayoutRow::IsSelected() const
{
	return OwnerTablePtr.Pin()->Private_IsItemSelected(NodeLayoutRow);
}

TSharedRef<ITableRow> SRigVMNodeLayoutWidget::GenerateRow(TSharedPtr<FNodeLayoutRow> InNodeLayoutRow, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRigVMNowLayoutRow, OwnerTable)
		.NodeLayoutRow(InNodeLayoutRow)
		.OnElementLabelChanged(OnElementLabelChanged)
		.OnElementCategoryChanged(OnElementCategoryChanged)
		.OnCategoryRemoved(OnCategoryRemoved)
		.OnCategoryRenamed(OnCategoryRenamed)
		.OnValidateCategoryName(OnValidateCategoryName)
		.OnValidateElementName(OnValidateElementName)
		.OnDragDetected_Lambda([this, InNodeLayoutRow](const FGeometry&, const FPointerEvent&) -> FReply
			{
				return OnDragDetectedForRow(InNodeLayoutRow);
			})
		.OnCanAcceptDrop_Lambda(
			[this](const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FNodeLayoutRow> InTargetRow) -> TOptional<EItemDropZone>
			{
				return OnCanAcceptDrop(InDragDropEvent, InTargetRow, InDropZone);
			})
		.OnAcceptDrop_Lambda(
			[this](const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FNodeLayoutRow> InTargetRow) -> FReply
			{
				return OnAcceptDrop(InDragDropEvent, InTargetRow, InDropZone);
			})
		.ToolTipText_Lambda([InNodeLayoutRow]()
		{
			return FText::FromString(InNodeLayoutRow->Path);
		})
	;
}

void SRigVMNodeLayoutWidget::GetChildrenForRow(TSharedPtr<FNodeLayoutRow> InNodeLayoutRow, TArray<TSharedPtr<FNodeLayoutRow>>& OutChildren)
{
	OutChildren = InNodeLayoutRow->ChildRows;
}

void SRigVMNodeLayoutWidget::OnItemExpansionChanged(TSharedPtr<FNodeLayoutRow> InRow, bool bExpanded)
{
	if(InRow)
	{
		InRow->State.bExpanded = bExpanded;
	}
}

void SRigVMNodeLayoutWidget::OnItemSelectionChanged(TSharedPtr<FNodeLayoutRow> InRow, ESelectInfo::Type InSelectInfo)
{
	struct Local
	{
		static void SyncSelectionState(TSharedPtr<STreeView<TSharedPtr<FNodeLayoutRow>>>& InTreeView, const TSharedPtr<FNodeLayoutRow>& InRow)
		{
			InRow->State.bSelected = InTreeView->IsItemSelected(InRow);
			SyncSelectionStates(InTreeView, InRow->ChildRows);
		}

		static void SyncSelectionStates(TSharedPtr<STreeView<TSharedPtr<FNodeLayoutRow>>>& InTreeView, const TArray<TSharedPtr<FNodeLayoutRow>>& InRows)
		{
			for(const TSharedPtr<FNodeLayoutRow>& Row : InRows)
			{
				SyncSelectionState(InTreeView, Row);
			}
		}
	};
	Local::SyncSelectionStates(TreeView, NodeLayoutRows);
}

bool SRigVMNodeLayoutWidget::IsNodeLayoutEditable() const
{
	return IsEnabled();
}

FReply SRigVMNodeLayoutWidget::HandleAddCategory()
{
	if(OnCategoryAdded.IsBound())
	{
		FString NewCategoryName = TEXT("Category");
		if(OnGetCategories.IsBound())
		{
			if(OnGetCategories.Execute().IsEmpty())
			{
				NewCategoryName = FRigVMPinCategory::GetDefaultCategoryName();
			}
		}
		OnCategoryAdded.Execute(NewCategoryName);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedRef<SRigVMNodeLayoutWidget::FRigVMNodeLayoutDragDropOp> SRigVMNodeLayoutWidget::FRigVMNodeLayoutDragDropOp::New(const TArray<TSharedPtr<FNodeLayoutRow>>& InNodeLayoutRows)
{
	TSharedRef<FRigVMNodeLayoutDragDropOp> Op = MakeShared<FRigVMNodeLayoutDragDropOp>();
	Op->NodeLayoutRows = InNodeLayoutRows;
	return Op;
}

TSharedPtr<SWidget> SRigVMNodeLayoutWidget::FRigVMNodeLayoutDragDropOp::GetDefaultDecorator() const
{
	TSharedPtr<SHorizontalBox> HorizontalBox;
	TSharedPtr<SWidget> Result = SNew(SBorder)
	.Padding(2)
	.Visibility(EVisibility::Visible)
	.BorderImage(FAppStyle::GetBrush("Menu.Background"))
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
	];
	SRigVMNowLayoutRow::ConstructLabel(NodeLayoutRows[0], HorizontalBox.ToSharedRef(), nullptr);
	return Result;
}

FVector2D SRigVMNodeLayoutWidget::FRigVMNodeLayoutDragDropOp::GetDecoratorPosition() const
{
	return FSlateApplication::Get().GetCursorPos();
}

FReply SRigVMNodeLayoutWidget::OnDragDetectedForRow(TSharedPtr<FNodeLayoutRow> InSourceRow)
{
	if(IsNodeLayoutEditable())
	{
		if(InSourceRow.IsValid())
		{
			const TSharedRef<FRigVMNodeLayoutDragDropOp> DragDropOp = FRigVMNodeLayoutDragDropOp::New({InSourceRow});
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SRigVMNodeLayoutWidget::OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, TSharedPtr<FNodeLayoutRow> InTargetRow, EItemDropZone InDropZone)
{
	static const TOptional<EItemDropZone> InvalidDropZone;

	const TSharedPtr<FRigVMNodeLayoutDragDropOp> NodeLayoutDragDropOp = InDragDropEvent.GetOperationAs<FRigVMNodeLayoutDragDropOp>();
	if(!NodeLayoutDragDropOp.IsValid())
	{
		return InvalidDropZone;
	}
	
	if(NodeLayoutDragDropOp->GetNodeLayoutRows().IsEmpty())
	{
		return InvalidDropZone;
	}
	
	const TSharedPtr<FNodeLayoutRow> SourceRow = NodeLayoutDragDropOp->GetNodeLayoutRows()[0];
	if(SourceRow.Get() == InTargetRow.Get())
	{
		return InvalidDropZone;
	}

	if(SourceRow->IsCategory() && InTargetRow->IsPin())
	{
		return InvalidDropZone;
	}

	if(InTargetRow->IsCategory())
	{
		if(SourceRow->IsPin())
		{
			return EItemDropZone::OntoItem;
		}
		if(SourceRow->IsCategory())
		{
			// for now we don't allow nesting of categories
			// even though the API in the controller allows for it.
			if(InDropZone == EItemDropZone::AboveItem ||
				InDropZone == EItemDropZone::BelowItem)
			{
				return InDropZone;
			}
		}
	}
	else if(InTargetRow->IsPin())
	{
		if(SourceRow->IsPin())
		{
			if(InDropZone == EItemDropZone::AboveItem ||
				InDropZone == EItemDropZone::BelowItem)
			{
				return InDropZone;
			}
		}
	}
			
	return InvalidDropZone;
}

FReply SRigVMNodeLayoutWidget::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, TSharedPtr<FNodeLayoutRow> InTargetRow, EItemDropZone InDropZone)
{
	if(!OnCanAcceptDrop(InDragDropEvent, InTargetRow, InDropZone).IsSet())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FRigVMNodeLayoutDragDropOp> NodeLayoutDragDropOp = InDragDropEvent.GetOperationAs<FRigVMNodeLayoutDragDropOp>();
	if(!NodeLayoutDragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}
	
	if(NodeLayoutDragDropOp->GetNodeLayoutRows().IsEmpty())
	{
		return FReply::Unhandled();
	}
	
	const TSharedPtr<FNodeLayoutRow> SourceRow = NodeLayoutDragDropOp->GetNodeLayoutRows()[0];

	if(InTargetRow->IsCategory())
	{
		if(SourceRow->IsPin())
		{
			if(OnElementCategoryChanged.IsBound())
			{
				int32 IndexInCategory = INDEX_NONE;
				const TArray<FRigVMPinCategory> Categories = OnGetCategories.Execute();
				for(const FRigVMPinCategory& Category : Categories)
				{
					if(Category.Path == InTargetRow->Path)
					{
						IndexInCategory = Category.Elements.Num();
						break;
					}
				}

				OnElementCategoryChanged.Execute(SourceRow->Path, InTargetRow->Path);
				
				if(OnElementIndexInCategoryChanged.IsBound())
				{
					OnElementIndexInCategoryChanged.Execute(SourceRow->Path, IndexInCategory);
				}
				return FReply::Handled();
			}
		}
		if(SourceRow->IsCategory())
		{
			if(InDropZone == EItemDropZone::AboveItem ||
				InDropZone == EItemDropZone::BelowItem)
			{
				// reorder categories
				// todo
				return FReply::Handled();
			}
		}
	}
	else if(InTargetRow->IsPin())
	{
		if(SourceRow->IsPin())
		{
			if(InDropZone == EItemDropZone::AboveItem ||
				InDropZone == EItemDropZone::BelowItem)
			{
				if(OnElementCategoryChanged.IsBound())
				{
					if(InTargetRow->IsUncategorizedPin())
					{
						// remove the pin category / set the pin to uncategorized
						OnElementCategoryChanged.Execute(SourceRow->Path, FString());
						return FReply::Handled();
					}
					if(OnGetElementCategory.IsBound())
					{
						// copy the category from the target onto me
						const FString TargetCategory = OnGetElementCategory.Execute(InTargetRow->Path);
						if(!TargetCategory.IsEmpty())
						{
							OnElementCategoryChanged.Execute(SourceRow->Path, TargetCategory);

							if(OnGetElementIndexInCategory.IsBound() && OnElementIndexInCategoryChanged.IsBound())
							{
								const int32 TargetIndex = OnGetElementIndexInCategory.Execute(InTargetRow->Path);
								if(TargetIndex != INDEX_NONE)
								{
									OnElementIndexInCategoryChanged.Execute(
										SourceRow->Path,
										InDropZone == EItemDropZone::AboveItem ? TargetIndex : TargetIndex + 1);
								}
							}
							return FReply::Handled();
						}
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

FReply SRigVMNodeLayoutWidget::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	// don't allow to drop anything onto the widget itself
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE // SRigVMNodeLayoutWidget
