// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMNodePreviewWidget.h"

#define LOCTEXT_NAMESPACE "SRigVMNodePreviewWidget"

void SRigVMNodePreviewWidget::Construct(const FArguments& InArgs)
{
	Environment = InArgs._Environment;
	check(Environment);
	Environment->OnChanged().BindSP(this, &SRigVMNodePreviewWidget::UpdateNodeWidget);
	NodeEnabledAttribute = InArgs._NodeEnabled;
	PinDragAndDropEnabledAttribute = InArgs._PinDragAndDropEnabled;
	
	ChildSlot
	.Padding(InArgs._Padding);
	
	UpdateNodeWidget();
}

void SRigVMNodePreviewWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	Environment->Tick_GameThead(InDeltaTime);
}

void SRigVMNodePreviewWidget::UpdateNodeWidget()
{
	ChildSlot.DetachWidget();

	if(URigVMEdGraphNode* EdGraphNode = Environment->GetEdGraphNode())
	{
		ChildSlot.AttachWidget(SNew(SRigVMGraphNode)
			.GraphNodeObj(EdGraphNode)
			.Enabled(NodeEnabledAttribute)
			.PinDragDropEnabled(PinDragAndDropEnabledAttribute)
		);
	}
}

#undef LOCTEXT_NAMESPACE
