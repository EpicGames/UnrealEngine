// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMSwapAssetReferencesWidget.h"
#include "Widgets/SRigVMBulkEditDialog.h"

#include "RigVMBlueprintLegacy.h"
#include "Dialogs/Dialogs.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/AppStyle.h"
#include "Algo/Count.h"
#include "RigVMEditorBlueprintLibrary.h"
#include "Editor/RigVMNewEditor.h"
#include "Misc/UObjectToken.h"
#include "RigVMModel/RigVMControllerActions.h"

#define LOCTEXT_NAMESPACE "SRigVMSwapAssetReferencesWidget"

uint32 FRigVMSwapAssetReferencesContext::GetVisibleChildrenHash() const
{
	return HashCombine(FRigVMTreeContext::GetVisibleChildrenHash(), GetTypeHash(SourceAsset));
}

TArray<TSharedRef<FRigVMTreeNode>> FRigVMTreeAssetRefAssetNode::GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	if (AssetRefNodes.IsEmpty())
	{
		if(const TSharedPtr<FRigVMSwapAssetReferencesContext> SwapContext = Cast<FRigVMSwapAssetReferencesContext>(InContext))
		{
			for (const FSoftObjectPath& Reference : SwapContext->GetReferences())
			{
				if (SoftObjectPath == Reference.GetWithoutSubPath())
				{
					TSharedRef<FRigVMTreeReferenceNode> RefNode = FRigVMTreeReferenceNode::Create(Reference);
					if(GetCheckState() == ECheckBoxState::Checked)
					{
						RefNode->SetCheckState(ECheckBoxState::Checked);
					}
					AssetRefNodes.Add(RefNode);
				}
			}
		}
	}
	return AssetRefNodes;
}

bool FRigVMTreeTargetAssetFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreePackageNode> AssetNode = Cast<FRigVMTreePackageNode>(InNode))
	{
		if(const TSharedPtr<FRigVMSwapAssetReferencesContext> AssetSwapContext = Cast<FRigVMSwapAssetReferencesContext>(InContext))
		{
			return AssetNode->GetAssetData() == AssetSwapContext->GetSourceAsset();
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

FText FRigVMTreeAssetVariantFilter::GetLabel() const
{
	return LOCTEXT("OnlyShowVariants", "Only Show Variants");
}

bool FRigVMTreeAssetVariantFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreeCategoryNode> TargetAssetNode = Cast<FRigVMTreeCategoryNode>(InNode))
	{
		if(const TSharedPtr<FRigVMSwapAssetReferencesContext> SwapContext = Cast<FRigVMSwapAssetReferencesContext>(InContext))
		{
			if (!SourceVariants.Contains(SwapContext->GetSourceAsset().GetFullName()))
			{
				static const FName AssetVariantPropertyName = TEXT("AssetVariant");
				const FProperty* AssetVariantProperty = CastField<FProperty>(SwapContext->GetSourceAsset().GetClass()->FindPropertyByName(AssetVariantPropertyName));
				const FString VariantStr = SwapContext->GetSourceAsset().GetTagValueRef<FString>(AssetVariantPropertyName);
				FRigVMVariant AssetVariant;
				if (!VariantStr.IsEmpty())
				{
					AssetVariantProperty->ImportText_Direct(*VariantStr, &AssetVariant, nullptr, EPropertyPortFlags::PPF_None);
				}

				if (!AssetVariant.Guid.IsValid())
				{
					AssetVariant.Guid = FRigVMVariant::GenerateGUID(SwapContext->GetSourceAsset().PackageName.ToString());
				}

				SourceVariants.FindOrAdd(SwapContext->GetSourceAsset().GetFullName()) = URigVMBuildData::Get()->FindAssetVariantRefs(AssetVariant.Guid);
			}

			TArray<FRigVMVariantRef>& VariantRefs = SourceVariants.FindChecked(SwapContext->GetSourceAsset().GetFullName());
			if(!VariantRefs.IsEmpty())
			{
				return !VariantRefs.ContainsByPredicate([InNode](const FRigVMVariantRef& VariantRef)
				{
					return VariantRef.ObjectPath.GetWithoutSubPath() == InNode->GetAssetData().GetSoftObjectPath();
				});
			}
		}
	}
	return true;
}

bool FRigVMSwapAssetReferenceTask::Execute(const TSharedRef<FRigVMTreePhase>& InPhase)
{
	if (SwapFunction.IsBound())
	{
		const TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
		Message->AddText(LOCTEXT("Swap", "Swapping"));
		Message->AddToken(FAssetNameToken::Create(ReferencePath.GetWithoutSubPath().ToString()));
		Message->AddText(FText::FromString(ReferencePath.GetSubPathString()));
		InPhase->GetContext()->LogMessage(Message);

		return SwapFunction.Execute(ReferencePath, NewAsset);
	}
	return false;
}

void SRigVMSwapAssetReferencesWidget::Construct(const FArguments& InArgs)
{
	static const TArray<TSharedRef<FRigVMTreeFilter>> DefaultFilters = {
		FRigVMTreeEngineContentFilter::Create(),
		FRigVMTreeDeveloperContentFilter::Create()
	};
	static const TSharedRef<FRigVMTreeFilter> DefaultPathFilter = FRigVMTreePathFilter::Create();

	// enable show engine content by default (this filter has to be inverted)
	DefaultFilters[0]->SetEnabled(false);

	OnGetReferences = InArgs._OnGetReferences;
	OnSwapReference = InArgs._OnSwapReference;
	SourceAssetFilters = InArgs._SourceAssetFilters;
	TargetAssetFilters = InArgs._TargetAssetFilters;
	PickTargetContext = FRigVMSwapAssetReferencesContext::Create();
	PickAssetRefsContext = FRigVMSwapAssetReferencesContext::Create();
	bSkipPickingRefs = InArgs._SkipPickingRefs;

	TOptional<int32> PhaseToActivate;

	SetSourceAsset(InArgs._Source);
	SetTargetAsset(InArgs._Target);
	PickAssetRefsContext->SetReferences(InArgs._ReferencePaths);

	TArray<TSharedRef<FRigVMTreePhase>> Phases;
	if(!InArgs._Source.IsValid())
	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKSOURCE, TEXT("Pick Source Asset"), FRigVMTreeContext::Create());
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(DefaultPathFilter);
		Phase->SetNodes(GetAssetNodes(InArgs, PHASE_PICKSOURCE));
		Phase->PrimaryButtonText().Set(LOCTEXT("Next", "Next"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			return Selection.ContainsByPredicate([](const TSharedRef<FRigVMTreeNode>& Node)
			{
				return Node->IsA<FRigVMTreePackageNode>();
			});
		});
		Phase->OnPrimaryAction().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			check(Selection.Num() == 1);
			if(Cast<FRigVMTreePackageNode>(Selection[0]))
			{
				GetBulkEditWidget()->ActivatePhase(PHASE_PICKTARGET);
				return FReply::Handled();
			}
			return FReply::Unhandled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}
	
	if(!InArgs._Target.IsValid())
	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKTARGET, TEXT("Pick Target Asset"), PickTargetContext.ToSharedRef());
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(DefaultPathFilter);
		Phase->GetContext()->Filters.Add(FRigVMTreeTargetAssetFilter::Create());
		Phase->GetContext()->Filters.Add(FRigVMTreeAssetVariantFilter::Create());
		Phase->SetNodes(GetAssetNodes(InArgs, PHASE_PICKTARGET));
		Phase->PrimaryButtonText().Set(LOCTEXT("Next", "Next"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			return Selection.ContainsByPredicate([](const TSharedRef<FRigVMTreeNode>& Node)
			{
				return Node->IsA<FRigVMTreePackageNode>();
			});
		});
		Phase->OnPrimaryAction().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			check(Selection.Num() == 1);
			if(Cast<FRigVMTreePackageNode>(Selection[0]))
			{
				GetBulkEditWidget()->ActivatePhase(PHASE_PICKASSETREFS);
				return FReply::Handled();
			}
			return FReply::Unhandled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}
	
	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKASSETREFS, TEXT("Pick Asset References"), PickAssetRefsContext.ToSharedRef());
		Phase->SetAllowsMultiSelection(true);
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(FRigVMTreePathFilter::Create());
		
		Phase->PrimaryButtonText().Set(LOCTEXT("SwapAssetRefs", "Swap Asset References"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			return true;
		});
		Phase->PrimaryButtonText().BindLambda([this]()
		{
			if(GetBulkEditWidget()->HasAnyVisibleCheckedNode())
			{
				return LOCTEXT("SwapAssetRefs", "Swap Asset References");
			}
			return LOCTEXT("Done", "Done");
		});
		Phase->OnPrimaryAction().BindLambda([this, Phase]()
		{
			GetBulkEditWidget()->GetTreeView()->GetTreeView()->ClearSelection();
			
			TArray<TSharedRef<FRigVMTreeNode>> AllCheckedNodes = GetBulkEditWidget()->GetCheckedNodes();
			if(AllCheckedNodes.IsEmpty())
			{
				GetBulkEditWidget()->CloseDialog();
				return FReply::Handled();
			}

			const TSharedPtr<FRigVMSwapAssetReferencesContext> Context = Cast<FRigVMSwapAssetReferencesContext>(Phase->GetContext());
			check(Context);

			for(int32 Index = 0; Index < AllCheckedNodes.Num(); Index++)
			{
				AllCheckedNodes.Append(AllCheckedNodes[Index]->GetChildren(Context.ToSharedRef()));
			}
			
			const TArray<TSharedRef<FRigVMTreeNode>> ModuleRefs = AllCheckedNodes.FilterByPredicate([](const TSharedRef<FRigVMTreeNode>& Node) {
				return Node->IsA<FRigVMTreeReferenceNode>();
			});
			
			if(ModuleRefs.IsEmpty())
			{
				return FReply::Handled();
			}
		
			TSet<FSoftObjectPath> VisitedPackages;
			TArray<TSharedRef<FRigVMTreeTask>> Tasks;
			
			for(const TSharedRef<FRigVMTreeNode>& Node : ModuleRefs)
			{
				const TSharedPtr<FRigVMTreeReferenceNode> ModuleRefNode = Cast<FRigVMTreeReferenceNode>(Node);
				check(ModuleRefNode);
				
				TSharedPtr<FRigVMTreeAssetRefAssetNode> AssetNode = Cast<FRigVMTreeAssetRefAssetNode>(ModuleRefNode->GetRoot());
			
				if(!VisitedPackages.Contains(AssetNode->GetPackagePath()))
				{
					Tasks.Add(FRigVMTreeLoadPackageForNodeTask::Create(AssetNode.ToSharedRef()));
					VisitedPackages.Add(AssetNode->GetPackagePath());
				}
				Tasks.Add(FRigVMSwapAssetReferenceTask::Create(ModuleRefNode->GetReferencePath(), Context->GetTargetAsset(), OnSwapReference));
			}
	
			if(Tasks.IsEmpty())
			{
				return FReply::Unhandled();
			}
			GetBulkEditWidget()->QueueTasks(Tasks);
			return FReply::Handled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0)
			[
				SAssignNew(BulkEditWidget, SRigVMBulkEditWidget)
				.Phases(Phases)
				.PhaseToActivate(PhaseToActivate.GetValue())
				.OnPhaseActivated(this, &SRigVMSwapAssetReferencesWidget::OnPhaseActivated)
				.OnNodeSelected(this, &SRigVMSwapAssetReferencesWidget::OnNodeSelected)
				.OnNodeDoubleClicked(this, &SRigVMSwapAssetReferencesWidget::OnNodeDoubleClicked)
				//.RightWidget(FunctionPreviewBox)
				.BulkEditTitle(LOCTEXT("SwapAssetReference", "Swap Asset Reference"))
				.BulkEditConfirmMessage(LOCTEXT("SwapAssetReferencesConfirmMessage", "This edit is going to swap asset references without support for undo. Are you sure?"))
				.BulkEditConfirmIniField(TEXT("RigVMSwapAssetReferences_Warning"))
				.EnableUndo(InArgs._EnableUndo)
				.CloseOnSuccess(InArgs._CloseOnSuccess)
			]
		]
	];
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMSwapAssetReferencesWidget::GetAssetNodes(const FArguments& InArgs, const int32& InPhase)
{
	TArray<TSharedRef<FRigVMTreeNode>> Nodes;

	auto AssetFilter = [this, InPhase] (const FAssetData& AssetData)
		{
			TArray<FRigVMAssetDataFilter>& Filters = (InPhase == PHASE_PICKSOURCE) ? SourceAssetFilters : TargetAssetFilters;
			for (const FRigVMAssetDataFilter& Filter : Filters)
			{
				if (!Filter.Execute(AssetData))
				{
					return false;
				}
			}
			return true;
		};
	
	TArray<FAssetData> Assets = URigVMEditorBlueprintLibrary::GetAssetsWithFilter(nullptr, FRigVMAssetDataFilter::CreateLambda(AssetFilter));
	Assets.Append(URigVMEditorBlueprintLibrary::GetAssetsWithFilter(URigVMBlueprintGeneratedClass::StaticClass(), FRigVMAssetDataFilter::CreateLambda(AssetFilter)));
	for(const FAssetData& Asset : Assets)
	{
		if(Asset.IsValid())
		{
			Nodes.Add(FRigVMTreePackageNode::Create(Asset)->ToSharedRef());
		}
	}

	return Nodes;
}

void SRigVMSwapAssetReferencesWidget::OnPhaseActivated(TSharedRef<FRigVMTreePhase> Phase)
{
	switch(Phase->GetID())
	{
		case PHASE_PICKSOURCE:
		{
			if(PickTargetContext->GetSourceAsset().IsValid())
			{
				if(const TSharedPtr<FRigVMTreeNode> Node = Phase->FindVisibleNode(PickTargetContext->GetSourceAsset().PackageName.ToString()))
				{
					const TSharedPtr<SRigVMChangesTreeView> TreeView = GetBulkEditWidget()->GetTreeView();
					TreeView->SetSelection(Node.ToSharedRef(), true);
				}
			}
			break;
		}
		case PHASE_PICKTARGET:
		{
			if(PickTargetContext->GetSourceAsset().IsValid())
			{
				if(const TSharedPtr<FRigVMTreeNode> Node = Phase->FindVisibleNode(PickTargetContext->GetSourceAsset().PackageName.ToString()))
				{
					const TSharedPtr<SRigVMChangesTreeView> TreeView = GetBulkEditWidget()->GetTreeView();
					TreeView->SetSelection(Node.ToSharedRef(), true);
				}
			}
			break;
		}
		case PHASE_PICKASSETREFS:
		{
			TArray<FSoftObjectPath> References = PickAssetRefsContext->GetReferences();
			if (OnGetReferences.IsBound())
			{
				References = OnGetReferences.Execute(PickAssetRefsContext->GetSourceAsset());
				PickAssetRefsContext->SetReferences(References);
			}

			TSet<FAssetData> Assets;
			TArray<TSharedRef<FRigVMTreeNode>> Nodes;
			for (const FSoftObjectPath& Reference : References)
			{
				const FAssetData ReferenceAssetData = IAssetRegistry::Get()->GetAssetByObjectPath(Reference.GetWithoutSubPath());
				if (!Assets.Contains(ReferenceAssetData))
				{
					TSharedRef<FRigVMTreeAssetRefAssetNode> NewAssetNode = FRigVMTreeAssetRefAssetNode::Create(ReferenceAssetData);
					if (bSkipPickingRefs)
					{
						NewAssetNode->SetCheckState(ECheckBoxState::Checked);
					}
					Nodes.Add(NewAssetNode);
					Assets.Add(ReferenceAssetData);
				}
			}
			
			Phase->SetNodes(Nodes);

			if(bSkipPickingRefs)
			{
				// process to next phase
				GetBulkEditWidget()->OnPrimaryButtonClicked();
			}
			
			break;
		}
		default:
		{
			return;
		}
	}
}

FReply SRigVMSwapAssetReferencesWidget::OnNodeSelected(TSharedRef<FRigVMTreeNode> Node)
{
	if(const TSharedPtr<FRigVMTreePackageNode> AssetNode = Cast<FRigVMTreePackageNode>(Node))
	{
		if(BulkEditWidget->GetActivePhase()->GetID() == PHASE_PICKSOURCE)
		{
			SetSourceAsset(AssetNode->GetAssetData());
			return FReply::Handled();
		}
		if(BulkEditWidget->GetActivePhase()->GetID() == PHASE_PICKTARGET)
		{
			SetTargetAsset(AssetNode->GetAssetData());
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

FReply SRigVMSwapAssetReferencesWidget::OnNodeDoubleClicked(TSharedRef<FRigVMTreeNode> Node)
{
	if(Node->IsA<FRigVMTreePackageNode>())
	{
		const FAssetData AssetData = Node->GetAssetData();
		if(AssetData.IsValid())
		{
			// force load
			if(UObject* TopLevelObject = AssetData.GetSoftObjectPath().GetWithoutSubPath().TryLoad())
			{
				const bool bValidAsset = TopLevelObject->GetClass()->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()) ||
										TopLevelObject->GetClass()->ImplementsInterface(URigVMRuntimeAssetInterface::StaticClass());
				if(bValidAsset)
				{
					if(const UObject* ObjectReference = StaticLoadObject(UObject::StaticClass(), nullptr, *Node->GetPath(), nullptr))
					{
						UObject* EditorObject = TopLevelObject;
						if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = TopLevelObject)
						{
							EditorObject = RuntimeAsset->GetEditorOnlyData();
						}
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(TopLevelObject);
						if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(EditorObject, true))
						{
							if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
							{
								RigVMEditor->HandleJumpToHyperlink(ObjectReference);
								return FReply::Handled();
							}
						}
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

void SRigVMSwapAssetReferencesWidget::SetSourceAsset(const FAssetData& InAsset)
{
	PickTargetContext->SetSourceAsset(InAsset);
	PickAssetRefsContext->SetSourceAsset(InAsset);
	PickAssetRefsContext->ClearReferences(); // If the source asset has changed, we should clear any existing references
}

void SRigVMSwapAssetReferencesWidget::SetTargetAsset(const FAssetData& InAsset)
{
	PickAssetRefsContext->SetTargetAsset(InAsset);
}

#undef LOCTEXT_NAMESPACE
