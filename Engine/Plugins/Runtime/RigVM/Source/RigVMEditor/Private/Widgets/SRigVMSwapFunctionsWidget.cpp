// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMSwapFunctionsWidget.h"

#include "IAssetTools.h"
#include "Widgets/SRigVMBulkEditDialog.h"

#include "RigVMBlueprintLegacy.h"
#include "Dialogs/Dialogs.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Styling/AppStyle.h"
#include "Algo/Count.h"
#include "Editor/RigVMNewEditor.h"
#include "Misc/UObjectToken.h"
#include "RigVMModel/RigVMControllerActions.h"

#define LOCTEXT_NAMESPACE "SRigVMSwapFunctionsWidget"

uint32 FRigVMSwapFunctionContext::GetVisibleChildrenHash() const
{
	return HashCombine(FRigVMTreeContext::GetVisibleChildrenHash(), GetTypeHash(SourceIdentifier));
}

const FSlateBrush* FRigVMTreeFunctionRefNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon.GetIcon();
}

FRigVMTreeFunctionRefGraphNode::FRigVMTreeFunctionRefGraphNode(const URigVMGraph* InFunctionGraph)
	: FRigVMTreeNode(InFunctionGraph->GetPathName())
	, WeakGraph(InFunctionGraph)
{
	if(const URigVMGraph* ParentGraph = InFunctionGraph->GetParentGraph())
	{
		if(ParentGraph->IsA<URigVMFunctionLibrary>())
		{
			static const FString FunctionPrefix = TEXT("Function ");
			OptionalLabel = FText::FromString(FunctionPrefix + InFunctionGraph->GetTypedOuter<URigVMNode>()->GetName());
		}
	}

	if(InFunctionGraph->IsRootGraph())
	{
		// let's see if there is only one event
		FString EventName;
		if(Algo::CountIf(InFunctionGraph->GetNodes(), [&EventName](const URigVMNode* NodeToCount) -> bool
		{
			if(NodeToCount->IsEvent() && NodeToCount->CanOnlyExistOnce())
			{
				if(EventName.IsEmpty())
				{
					if(const URigVMUnitNode* UnitNode = Cast<URigVMUnitNode>(NodeToCount))
					{
						if(UnitNode->GetScriptStruct())
						{
							EventName = UnitNode->GetScriptStruct()->GetDisplayNameText().ToString();
						}
					}
				}
				return true;
			}
			return false;
		}) == 1)
		{
			static constexpr TCHAR EventGraphNameFormat[] = TEXT("%s Graph");
			const FString DesiredGraphName = FString::Printf(EventGraphNameFormat, *EventName);
			OptionalLabel = FText::FromString(DesiredGraphName);
		}
		else
		{
			if(InFunctionGraph->GetName() == FRigVMClient::RigVMModelPrefix)
			{
				OptionalLabel = LOCTEXT("MainGraph", "Main Graph");
			}
		}
	}
}

TArray<TSharedRef<FRigVMTreeNode>> FRigVMTreeFunctionRefGraphNode::GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	if(FunctionRefNodes.IsEmpty())
	{
		if(const URigVMGraph* Graph = WeakGraph.Get())
		{
			for(const URigVMNode* Node : Graph->GetNodes())
			{
				if(const URigVMFunctionReferenceNode* FunctionReferenceNode = Cast<URigVMFunctionReferenceNode>(Node))
				{
					if(const TSharedPtr<FRigVMSwapFunctionContext> SwapContext = Cast<FRigVMSwapFunctionContext>(InContext))
					{
						if(FunctionReferenceNode->GetFunctionIdentifier() != SwapContext->GetSourceIdentifier())
						{
							continue;
						}
					}
					TSharedRef<FRigVMTreeFunctionRefNode> FunctionRefNode = FRigVMTreeFunctionRefNode::Create(FunctionReferenceNode);
					FunctionRefNodes.Add(FunctionRefNode);
				}
			}
		}
	}
	return FunctionRefNodes;
}

FText FRigVMTreeFunctionRefGraphNode::GetLabel() const
{
	if(OptionalLabel.IsSet())
	{
		return OptionalLabel.GetValue();
	}
	
	static const FString ContainedGraphSuffix = TEXT(".ContainedGraph");

	const FText Label = FRigVMTreeNode::GetLabel();
	FString LabelString = Label.ToString();
	if(LabelString.EndsWith(ContainedGraphSuffix))
	{
		LabelString = LabelString.LeftChop(ContainedGraphSuffix.Len());
	}

	return FText::FromString(LabelString);
}

const FSlateBrush* FRigVMTreeFunctionRefGraphNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");
	return Icon.GetIcon();
}

void FRigVMTreeFunctionRefGraphNode::DirtyChildren()
{
	FRigVMTreeNode::DirtyChildren();
	FunctionRefNodes.Reset();
}

bool FRigVMTreeFunctionRefGraphNode::ContainsFunctionReference(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	const TArray<TSharedRef<FRigVMTreeNode>> NewChildren = GetVisibleChildren(InContext);
	for(const TSharedRef<FRigVMTreeNode>& NewChild : NewChildren)
	{
		if(NewChild->IsA<FRigVMTreeFunctionRefNode>())
		{
			return true;
		}
		if(NewChild->IsA<FRigVMTreeFunctionRefGraphNode>())
		{
			if(Cast<FRigVMTreeFunctionRefGraphNode>(NewChild)->ContainsFunctionReference(InContext))
			{
				return true;
			}
		}
	}
	return false;
}

FRigVMTreeFunctionRefAssetNode::FRigVMTreeFunctionRefAssetNode(const FAssetData& InAssetData)
	: FRigVMTreePackageNode(InAssetData)
{
}

void FRigVMTreeFunctionRefAssetNode::DirtyChildren()
{
	FRigVMTreePackageNode::DirtyChildren();
	LoadedGraphNodes.Reset();
	MetaDataBasedNodes.Reset();
	ReferenceNodeDatas.Reset();
}

TArray<TSharedRef<FRigVMTreeNode>> FRigVMTreeFunctionRefAssetNode::GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const
{
	if(LoadedGraphNodes.IsEmpty())
	{
		const TSoftObjectPtr<UObject> SoftObject(SoftObjectPath);
		if(const UObject* AssetObject = SoftObject.Get())
		{
			const IRigVMClientHost* ClientHost = Cast<IRigVMClientHost>(AssetObject);
			if (!ClientHost)
			{
				if (TScriptInterface<const IRigVMRuntimeAssetInterface> RuntimeAsset = AssetObject)
				{
					ClientHost = Cast<IRigVMClientHost>(RuntimeAsset->GetEditorOnlyData());
				}
			}
			if(ClientHost)
			{
				if(const FRigVMClient* Client = ClientHost->GetRigVMClient())
				{
					TArray<URigVMGraph*> Models = Client->GetAllModels(true, true);
					for(const URigVMGraph* Graph : Models)
					{
						TSharedRef<FRigVMTreeFunctionRefGraphNode> GraphNode = FRigVMTreeFunctionRefGraphNode::Create(Graph);
						if(GraphNode->ContainsFunctionReference(InContext))
						{
							LoadedGraphNodes.Add(GraphNode);
						}
					}
				}
			}
		}

		// if we can't find anything - let's fall back on asset metadata
		if(LoadedGraphNodes.IsEmpty() && !IsLoaded())
		{
			if(MetaDataBasedNodes.IsEmpty())
			{
				if(ReferenceNodeDatas.IsEmpty())
				{
					const FAssetData AssetData = GetAssetData();
					if(AssetData.IsValid())
					{
						if(const UClass* Class = AssetData.GetClass())
						{
							static const FLazyName FunctionReferenceNodeDataName = TEXT("FunctionReferenceNodeData");

							const FArrayProperty* ReferenceNodeDataProperty =
							  CastField<FArrayProperty>(Class->FindPropertyByName(FunctionReferenceNodeDataName));
							if(ReferenceNodeDataProperty)
							{
								const FAssetDataTagMapSharedView::FFindTagResult FoundValue = AssetData.TagsAndValues.FindTag(FunctionReferenceNodeDataName);
								if (FoundValue.IsSet())
								{
									const FString ReferenceNodeDataString = FoundValue.AsString();
									if(!ReferenceNodeDataString.IsEmpty())
									{
										ReferenceNodeDataProperty->ImportText_Direct(*ReferenceNodeDataString, &ReferenceNodeDatas, nullptr, EPropertyPortFlags::PPF_None);
									}
								}
							}
						}
					}
				}
				
				for(const FRigVMReferenceNodeData& ReferenceNodeData : ReferenceNodeDatas)
				{
					if(const TSharedPtr<FRigVMSwapFunctionContext> SwapContext = Cast<FRigVMSwapFunctionContext>(InContext))
					{
						if(ReferenceNodeData.ReferencedFunctionIdentifier != SwapContext->GetSourceIdentifier())
						{
							continue;
						}
					}
					TSharedRef<FRigVMTreeFunctionRefNode> RefNode = FRigVMTreeFunctionRefNode::Create(ReferenceNodeData);
					MetaDataBasedNodes.Add(RefNode);
				}
			}
			return MetaDataBasedNodes;
		}
	}
	return LoadedGraphNodes;
}

FText FRigVMTreeFunctionIdentifierNode::GetLabel() const
{
	static const FString RigVMFunctionLibraryPrefix = TEXT("RigVMFunctionLibrary.");
	FText Label = FRigVMTreeNode::GetLabel();
	const FString LabelString = Label.ToString();
	if(LabelString.StartsWith(RigVMFunctionLibraryPrefix))
	{
		Label = FText::FromString(LabelString.Mid(RigVMFunctionLibraryPrefix.Len()));
	}
	return Label;
}

const FSlateBrush* FRigVMTreeFunctionIdentifierNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon.GetIcon();
}

const TArray<FRigVMTag>& FRigVMTreeFunctionIdentifierNode::GetTags() const
{
	const FRigVMGraphFunctionHeader Header = URigVMBuildData::Get()->FindFunctionHeader(Identifier);
	if(Header.IsValid())
	{
		Tags = Header.Variant.Tags;
	}
	return FRigVMTreeNode::GetTags();
}

void FRigVMTreeFunctionIdentifierAssetNode::AddChildNode(const TSharedRef<FRigVMTreeNode>& InNode)
{
	AddChildImpl(InNode);
}

void FRigVMTreeFunctionIdentifierAssetNode::DirtyChildren()
{
	// no need to dirty things here
	// since the content of this is set up using a push model
}

bool FRigVMTreeEmptyFunctionRefGraphFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(InNode->IsA<FRigVMTreeFunctionRefGraphNode>())
	{
		if(InNode->GetVisibleChildren(InContext).IsEmpty())
		{
			return true;
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

bool FRigVMTreeEmptyFunctionRefAssetFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(InNode->IsA<FRigVMTreeFunctionRefAssetNode>())
	{
		if(InNode->GetVisibleChildren(InContext).IsEmpty())
		{
			return true;
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

FText FRigVMTreeFunctionWithNoRefsFilter::GetLabel() const
{
	return LOCTEXT("ShowFunctionsWithoutReferences", "Show Unused Functions");
}

bool FRigVMTreeFunctionWithNoRefsFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreeFunctionIdentifierNode> FunctionIdentifierNode = Cast<FRigVMTreeFunctionIdentifierNode>(InNode))
	{
		const FRigVMFunctionReferenceArray* References = URigVMBuildData::Get()->FindFunctionReferences(FunctionIdentifierNode->GetIdentifier());
		if((References == nullptr) || (References->Num() == 0))
		{
			return true;
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

bool FRigVMTreeSourceFunctionFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreeFunctionRefNode> FunctionRefNode = Cast<FRigVMTreeFunctionRefNode>(InNode))
	{
		if(const TSharedPtr<FRigVMSwapFunctionContext> FunctionSwapContext = Cast<FRigVMSwapFunctionContext>(InContext))
		{
			if(FunctionRefNode->GetIdentifier() != FunctionSwapContext->GetSourceIdentifier())
			{
				return true;
			}
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

bool FRigVMTreeTargetFunctionFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreeFunctionIdentifierNode> FunctionIdentifierNode = Cast<FRigVMTreeFunctionIdentifierNode>(InNode))
	{
		if(const TSharedPtr<FRigVMSwapFunctionContext> FunctionSwapContext = Cast<FRigVMSwapFunctionContext>(InContext))
		{
			if(FunctionIdentifierNode->GetIdentifier() == FunctionSwapContext->GetSourceIdentifier())
			{
				return true;
			}
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

FText FRigVMTreeFunctionVariantFilter::GetLabel() const
{
	return LOCTEXT("OnlyShowVariants", "Only Show Variants");
}

bool FRigVMTreeFunctionVariantFilter::Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext)
{
	if(const TSharedPtr<FRigVMTreeFunctionIdentifierNode> FunctionIdentifierNode = Cast<FRigVMTreeFunctionIdentifierNode>(InNode))
	{
		if(const TSharedPtr<FRigVMSwapFunctionContext> FunctionSwapContext = Cast<FRigVMSwapFunctionContext>(InContext))
		{
			if(IsFunctionVariant(FunctionSwapContext->GetSourceIdentifier()))
			{
				if (!IsFunctionVariantOf(
					FunctionIdentifierNode->GetIdentifier(),
					FunctionSwapContext->GetSourceIdentifier()))
				{
					return true;
				}
			}
		}
	}
	return FRigVMTreeFilter::Filters(InNode, InContext);
}

bool FRigVMTreeFunctionVariantFilter::IsFunctionVariant(const FRigVMGraphFunctionIdentifier& InIdentifier) const
{
	if(const bool* bResult = LibraryNodePathToIsVariant.Find(InIdentifier.GetLibraryNodePath()))
	{
		return *bResult;
	}
	const bool bResult = InIdentifier.IsVariant();
	LibraryNodePathToIsVariant.Add(InIdentifier.GetLibraryNodePath(), bResult);
	return bResult;
}

bool FRigVMTreeFunctionVariantFilter::IsFunctionVariantOf(const FRigVMGraphFunctionIdentifier& InIdentifier, const FRigVMGraphFunctionIdentifier& InSourceIdentifier) const
{
	if(const TArray<FRigVMGraphFunctionIdentifier>* Identifiers = LibraryNodePathToVariants.Find(InSourceIdentifier.GetLibraryNodePath()))
	{
		return Identifiers->Contains(InIdentifier);
	}
	const TArray<FRigVMGraphFunctionIdentifier> Identifiers = InSourceIdentifier.GetVariantIdentifiers();
	LibraryNodePathToVariants.Add(InSourceIdentifier.GetLibraryNodePath(), Identifiers);
	return Identifiers.Contains(InIdentifier);
}

bool FRigVMSwapFunctionTask::Execute(const TSharedRef<FRigVMTreePhase>& InPhase)
{
	URigVMFunctionReferenceNode* ReferenceNode = GetReferenceNode(InPhase);
	if(ReferenceNode == nullptr)
	{
		return false;
	}

	URigVMGraph* Graph = ReferenceNode->GetGraph();
	check(Graph);
	IRigVMClientHost* RigVMClientHost = Graph->GetImplementingOuter<IRigVMClientHost>();
	check(RigVMClientHost);

	URigVMController* Controller = RigVMClientHost->GetOrCreateController(Graph);
	check(Controller);

	const TSharedRef<FUObjectToken> ReferenceNodeToken = FUObjectToken::Create(ReferenceNode);

	TWeakObjectPtr<URigVMFunctionReferenceNode> WeakRefNode(ReferenceNode);
	ReferenceNodeToken->OnMessageTokenActivated(FOnMessageTokenActivated::CreateLambda([WeakRefNode](const TSharedRef<IMessageToken>&)
	{
		if(WeakRefNode.IsValid())
		{
			if (FRigVMEditorAssetInterfacePtr EditorAsset = IRigVMEditorAssetInterface::GetInterfaceOuter(WeakRefNode.Get()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorAsset->GetAssetObjectForEditor());
				
				if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(EditorAsset.GetObject(), true))
				{
					if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
					{
						RigVMEditor->HandleJumpToHyperlink(WeakRefNode.Get());
					}
				}
			}
		}
	}));

	const TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info);
	Message->AddText(LOCTEXT("Swapping", "Swapping"));
	Message->AddToken(ReferenceNodeToken);
	InPhase->GetContext()->LogMessage(Message);

	// always assume success
	(void)Controller->SwapFunctionReference(ReferenceNode, Identifier, true, IsUndoEnabled(), true);
	(void)ReferenceNode->MarkPackageDirty();
	return true;
}

URigVMFunctionReferenceNode* FRigVMSwapFunctionTask::GetReferenceNode(const TSharedRef<FRigVMTreePhase>& InPhase) const
{
	// we expect this to be loaded by now
	UObject* NodeObject = StaticFindObject(UObject::StaticClass(), NULL, *ObjectPath, EFindObjectFlags::None);
	if(NodeObject == nullptr)
	{
		InPhase->GetContext()->LogError(FString::Printf(TEXT("Cannot find reference node '%s'."), *ObjectPath));
		return nullptr;
	}
	
	URigVMFunctionReferenceNode* ReferenceNode = Cast<URigVMFunctionReferenceNode>(NodeObject);
	if(ReferenceNode == nullptr)
	{
		InPhase->GetContext()->LogError(FString::Printf(TEXT("ObjectPath '%s' doesn't refer to a reference node."), *ObjectPath));
		return nullptr;
	}
	return ReferenceNode;
}

FRigVMEditorAssetInterfacePtr FRigVMSwapFunctionTask::GetRigVMAsset(const TSharedRef<FRigVMTreePhase>& InPhase) const
{
	URigVMFunctionReferenceNode* ReferenceNode = GetReferenceNode(InPhase);
	if(ReferenceNode == nullptr)
	{
		return nullptr;
	}
	return IRigVMEditorAssetInterface::GetInterfaceOuter(ReferenceNode);
}

void SRigVMSwapFunctionsWidget::Construct(const FArguments& InArgs)
{
	static const TArray<TSharedRef<FRigVMTreeFilter>> DefaultFilters = {
		FRigVMTreeEngineContentFilter::Create(),
		FRigVMTreeDeveloperContentFilter::Create()
	};
	static const TSharedRef<FRigVMTreeFilter> DefaultPathFilter = FRigVMTreePathFilter::Create();

	// enable show engine content by default (this filter has to be inverted)
	DefaultFilters[0]->SetEnabled(false);

	PickTargetContext = FRigVMSwapFunctionContext::Create();
	PickFunctionRefsContext = FRigVMSwapFunctionContext::Create();
	SourcePreviewEnvironment = MakeShareable(new FRigVMMinimalEnvironment());
	TargetPreviewEnvironment = MakeShareable(new FRigVMMinimalEnvironment());
	bSkipPickingFunctionRefs = InArgs._SkipPickingFunctionRefs;

	TOptional<int32> PhaseToActivate;

	SetSourceFunction(InArgs._Source);
	SetTargetFunction(InArgs._Target);

	TArray<TSharedRef<FRigVMTreePhase>> Phases;
	if(!InArgs._Source.IsValid())
	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKSOURCE, TEXT("Pick Source Function"), FRigVMTreeContext::Create());
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(DefaultPathFilter);
		Phase->GetContext()->Filters.Add(FRigVMTreeFunctionWithNoRefsFilter::Create());
		Phase->SetNodes(GetFunctionIdentifierNodes(InArgs));
		Phase->PrimaryButtonText().Set(LOCTEXT("Next", "Next"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			return Selection.ContainsByPredicate([](const TSharedRef<FRigVMTreeNode>& Node)
			{
				return Node->IsA<FRigVMTreeFunctionIdentifierNode>();
			});
		});
		Phase->OnPrimaryAction().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			check(Selection.Num() == 1);
			if(Cast<FRigVMTreeFunctionIdentifierNode>(Selection[0]))
			{
				GetBulkEditWidget()->ActivatePhase(PHASE_PICKTARGET);
				return FReply::Handled();
			}
			return FReply::Unhandled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}
	
	if(!InArgs._Target.IsValid())
	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKTARGET, TEXT("Pick Target Function"), PickTargetContext.ToSharedRef());
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(DefaultPathFilter);
		Phase->GetContext()->Filters.Add(FRigVMTreeTargetFunctionFilter::Create());
		Phase->GetContext()->Filters.Add(FRigVMTreeFunctionVariantFilter::Create());
		Phase->SetNodes(GetFunctionIdentifierNodes(InArgs));
		Phase->PrimaryButtonText().Set(LOCTEXT("Next", "Next"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			return Selection.ContainsByPredicate([](const TSharedRef<FRigVMTreeNode>& Node)
			{
				return Node->IsA<FRigVMTreeFunctionIdentifierNode>();
			});
		});
		Phase->OnPrimaryAction().BindLambda([this]()
		{
			const TArray<TSharedRef<FRigVMTreeNode>> Selection = GetBulkEditWidget()->GetSelectedNodes();
			check(Selection.Num() == 1);
			if(TSharedPtr<FRigVMTreeFunctionIdentifierNode> FunctionIdentifierNode = Cast<FRigVMTreeFunctionIdentifierNode>(Selection[0]))
			{
				GetBulkEditWidget()->ActivatePhase(PHASE_PICKFUNCTIONREFS);
				return FReply::Handled();
			}
			return FReply::Unhandled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}

	{
		TSharedRef<FRigVMTreePhase> Phase = FRigVMTreePhase::Create(PHASE_PICKFUNCTIONREFS, TEXT("Pick Function References"), PickFunctionRefsContext.ToSharedRef());
		Phase->SetAllowsMultiSelection(true);
		Phase->GetContext()->Filters = DefaultFilters;
		Phase->GetContext()->Filters.Add(FRigVMTreePathFilter::Create());
		Phase->GetContext()->Filters.Add(FRigVMTreeEmptyFunctionRefGraphFilter::Create());
		Phase->GetContext()->Filters.Add(FRigVMTreeEmptyFunctionRefAssetFilter::Create());
		Phase->GetContext()->Filters.Add(FRigVMTreeSourceFunctionFilter::Create());

		Phase->SetNodes(GetFunctionRefNodes(InArgs));
		Phase->PrimaryButtonText().Set(LOCTEXT("SwapFunctions", "Swap Functions"));
		Phase->IsPrimaryButtonVisible().BindLambda([this]()
		{
			return true;
		});
		Phase->PrimaryButtonText().BindLambda([this]()
		{
			if(GetBulkEditWidget()->HasAnyVisibleCheckedNode())
			{
				return LOCTEXT("SwapFunction", "Swap Function");
			}
			return LOCTEXT("Done", "Done");
		});
		Phase->OnPrimaryAction().BindLambda([this, Phase]()
		{
			GetBulkEditWidget()->GetTreeView()->GetTreeView()->ClearSelection();
			
			const TArray<TSharedRef<FRigVMTreeNode>> AllCheckedNodes = GetBulkEditWidget()->GetCheckedNodes();
			if(AllCheckedNodes.IsEmpty())
			{
				GetBulkEditWidget()->CloseDialog();
				return FReply::Handled();
			}
			
			const TArray<TSharedRef<FRigVMTreeNode>> FunctionRefs = AllCheckedNodes.FilterByPredicate([](const TSharedRef<FRigVMTreeNode>& Node) {
				return Node->IsA<FRigVMTreeFunctionRefNode>();
			});

			if(FunctionRefs.IsEmpty())
			{
				return FReply::Handled();
			}

			const TSharedPtr<FRigVMSwapFunctionContext> Context = Cast<FRigVMSwapFunctionContext>(Phase->GetContext());
			check(Context);

			TSet<FString> VisitedPackages;
			TArray<TSharedRef<FRigVMTreeTask>> Tasks;

			for(const TSharedRef<FRigVMTreeNode>& Node : FunctionRefs)
			{
				const TSharedPtr<FRigVMTreeFunctionRefNode> FunctionRefNode = Cast<FRigVMTreeFunctionRefNode>(Node);
				check(FunctionRefNode);
				if(FunctionRefNode->GetIdentifier() != Context->GetSourceIdentifier())
				{
					continue;
				}

				const FAssetData AssetData = FunctionRefNode->GetAssetData();
				const FString PackagePath = AssetData.ToSoftObjectPath().GetLongPackageName();

				if(!VisitedPackages.Contains(PackagePath))
				{
					Tasks.Add(FRigVMTreeLoadPackageForNodeTask::Create(FunctionRefNode->GetRoot()));
					VisitedPackages.Add(PackagePath);
				}
				Tasks.Add(FRigVMSwapFunctionTask::Create(FunctionRefNode->GetPath(), Context->GetTargetIdentifier()));
			}

			if(Tasks.IsEmpty())
			{
				return FReply::Unhandled();
			}
			GetBulkEditWidget()->QueueTasks(Tasks);
			return FReply::Handled();
		});
		PhaseToActivate = PhaseToActivate.Get(Phase->GetID());
		Phases.Add(Phase);
	}

	const TSharedRef<SWidget> FunctionPreviewBox = SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		SNew(SRigVMNodePreviewWidget)
		.Environment(SourcePreviewEnvironment)
		.Visibility_Lambda([this]()
		{
			return PickTargetContext->GetSourceIdentifier().IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
		})
	]
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Top)
	.AutoWidth()
	[
		SNew(SRigVMNodePreviewWidget)
		.Environment(TargetPreviewEnvironment)
		.Visibility_Lambda([this]()
		{
			return PickFunctionRefsContext->GetTargetIdentifier().IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
		})
	];
		
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0)
			[
				SAssignNew(BulkEditWidget, SRigVMBulkEditWidget)
				.Phases(Phases)
				.PhaseToActivate(PhaseToActivate.GetValue())
				.OnPhaseActivated(this, &SRigVMSwapFunctionsWidget::OnPhaseActivated)
				.OnNodeSelected(this, &SRigVMSwapFunctionsWidget::OnNodeSelected)
				.OnNodeDoubleClicked(this, &SRigVMSwapFunctionsWidget::OnNodeDoubleClicked)
				.RightWidget(FunctionPreviewBox)
				.BulkEditTitle(LOCTEXT("SwapFunctions", "Swap Functions"))
				.BulkEditConfirmMessage(LOCTEXT("SwapFunctionsConfirmMessage", "This edit is going to swap functions without support for undo. Are you sure?"))
				.BulkEditConfirmIniField(TEXT("RigVMSwapFunctions_Warning"))
				.EnableUndo(InArgs._EnableUndo)
				.CloseOnSuccess(InArgs._CloseOnSuccess)
			]
		]
	];
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMSwapFunctionsWidget::GetFunctionIdentifierNodes(const FArguments& InArgs)
{
	TArray<TSharedRef<FRigVMTreeNode>> Nodes;
	TMap<FString, TSharedRef<FRigVMTreeFunctionIdentifierAssetNode>> VisitedAssets;
	
	URigVMBuildData::Get()->InitializeIfNeeded();
	TArray<FRigVMGraphFunctionIdentifier> Identifiers = URigVMBuildData::Get()->GetAllFunctionIdentifiers();
	
	// Add private functions from the given assets
	for (const FAssetData& AssetData : InArgs._Assets)
	{
		if (AssetData.IsAssetLoaded())
		{
			UObject* AssetObj = AssetData.FastGetAsset();
			TScriptInterface<IRigVMGraphFunctionHost> FunctionHost = AssetObj;
			if (!FunctionHost)
			{
				if (TScriptInterface<IRigVMClientHost> ClientHost = AssetObj)
				{
					FunctionHost = ClientHost->GetRigVMGraphFunctionHost();
				}
			}
			
			if (FunctionHost)
			{
				for (const FRigVMGraphFunctionData& FuncData : FunctionHost->GetRigVMGraphFunctionStore()->PrivateFunctions)
				{
					Identifiers.Add(FuncData.Header.LibraryPointer);
				}
			}
		}
	}
	
	for(const FRigVMGraphFunctionIdentifier& Identifier : Identifiers)
	{
		FString PackagePath = Identifier.HostObject.GetLongPackageName();
		TSharedPtr<FRigVMTreeFunctionIdentifierAssetNode> PackageCategory;
		if(!VisitedAssets.Contains(PackagePath))
		{
			FAssetData AssetData = FRigVMTreeContext::FindAssetFromAnyPath(PackagePath, false);
			if (!AssetData.IsValid())
			{
				AssetData = FRigVMTreeContext::FindAssetFromAnyPath( Identifier.HostObject.GetAssetPath().ToString(), false);
			}
			
			if(AssetData.IsValid())
			{
				PackageCategory = FRigVMTreeFunctionIdentifierAssetNode::Create(AssetData);
				VisitedAssets.Add(PackagePath, PackageCategory.ToSharedRef());
				Nodes.Add(PackageCategory.ToSharedRef());
			}
			else
			{
				continue;
			}
		}
		else
		{
			PackageCategory = VisitedAssets.FindChecked(PackagePath);
		}
		PackageCategory->AddChildNode(FRigVMTreeFunctionIdentifierNode::Create(Identifier));
	}

	return Nodes;
}

TArray<TSharedRef<FRigVMTreeNode>> SRigVMSwapFunctionsWidget::GetFunctionRefNodes(const FArguments& InArgs)
{
	TArray<TSharedRef<FRigVMTreeNode>> Nodes;

	for(const URigVMFunctionReferenceNode* FunctionReferenceNode : InArgs._FunctionReferenceNodes)
	{
		Nodes.Add(FRigVMTreeFunctionRefNode::Create(FunctionReferenceNode));
		if(InArgs._SkipPickingFunctionRefs)
		{
			Nodes.Last()->SetCheckState(ECheckBoxState::Checked);
		}
	}

	if(!InArgs._SkipPickingFunctionRefs)
	{
		for(const URigVMGraph* Graph : InArgs._Graphs)
		{
			Nodes.Add(FRigVMTreeFunctionRefGraphNode::Create(Graph));
		}
		for(const FAssetData& AssetData : InArgs._Assets)
		{
			Nodes.Add(FRigVMTreeFunctionRefAssetNode::Create(AssetData));
		}
	}
	return Nodes;
}

void SRigVMSwapFunctionsWidget::OnPhaseActivated(TSharedRef<FRigVMTreePhase> Phase)
{
	switch(Phase->GetID())
	{
		case PHASE_PICKSOURCE:
		{
			if(PickTargetContext->GetSourceIdentifier().IsValid())
			{
				if(const TSharedPtr<FRigVMTreeNode> Node = Phase->FindVisibleNode(PickTargetContext->GetSourceIdentifier().GetLibraryNodePath()))
				{
					const TSharedPtr<SRigVMChangesTreeView> TreeView = GetBulkEditWidget()->GetTreeView();
					TreeView->SetSelection(Node.ToSharedRef(), true);
				}
			}
			break;
		}
		case PHASE_PICKTARGET:
		{
			if(PickFunctionRefsContext->GetTargetIdentifier().IsValid())
			{
				if(const TSharedPtr<FRigVMTreeNode> Node = Phase->FindVisibleNode(PickFunctionRefsContext->GetTargetIdentifier().GetLibraryNodePath()))
				{
					const TSharedPtr<SRigVMChangesTreeView> TreeView = GetBulkEditWidget()->GetTreeView();
					TreeView->SetSelection(Node.ToSharedRef(), true);
				}
			}
			break;
		}
		case PHASE_PICKFUNCTIONREFS:
		{
			if(bSkipPickingFunctionRefs)
			{
				// process to next phase
				GetBulkEditWidget()->OnPrimaryButtonClicked();
			}
			break;
		}
		default:
		{
			return;
		}
	}
}

FReply SRigVMSwapFunctionsWidget::OnNodeSelected(TSharedRef<FRigVMTreeNode> Node)
{
	if(const TSharedPtr<FRigVMTreeFunctionIdentifierNode> FunctionIdentifierNode = Cast<FRigVMTreeFunctionIdentifierNode>(Node))
	{
		if(BulkEditWidget->GetActivePhase()->GetID() == PHASE_PICKSOURCE)
		{
			SetSourceFunction(FunctionIdentifierNode->GetIdentifier());
			return FReply::Handled();
		}
		if(BulkEditWidget->GetActivePhase()->GetID() == PHASE_PICKTARGET)
		{
			SetTargetFunction(FunctionIdentifierNode->GetIdentifier());
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

FReply SRigVMSwapFunctionsWidget::OnNodeDoubleClicked(TSharedRef<FRigVMTreeNode> Node)
{
	if(Node->IsA<FRigVMTreeFunctionIdentifierNode>() || Node->IsA<FRigVMTreeFunctionRefNode>())
	{
		const FAssetData AssetData = Node->GetRoot()->GetAssetData();
		if(AssetData.IsValid())
		{
			// force load
			if(UObject* TopLevelObject = AssetData.GetSoftObjectPath().GetWithoutSubPath().TryLoad())
			{
				if(TopLevelObject->GetClass()->ImplementsInterface(URigVMEditorAssetInterface::StaticClass()) || 
					TopLevelObject->GetClass()->ImplementsInterface(URigVMRuntimeAssetInterface::StaticClass()))
				{
					if(const UObject* ObjectReference = StaticLoadObject(UObject::StaticClass(), nullptr, *Node->GetPath(), nullptr))
					{
						UObject* EditorObject = TopLevelObject;
						if (TScriptInterface<IRigVMRuntimeAssetInterface> RuntimeAsset = TopLevelObject)
						{
							EditorObject = RuntimeAsset->GetEditorOnlyData();
						}
						GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(EditorObject);
						if(IAssetEditorInstance* Editor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(TopLevelObject, true))
						{
							if (FRigVMEditorBase* RigVMEditor = FRigVMEditorBase::GetFromAssetEditorInstance(Editor))
							{
								RigVMEditor->HandleJumpToHyperlink(ObjectReference);
								return FReply::Handled();
							}
						}
					}
				}
			}
		}
	}
	return FReply::Unhandled();
}

void SRigVMSwapFunctionsWidget::SetSourceFunction(const FRigVMGraphFunctionIdentifier& InIdentifier)
{
	PickTargetContext->SetSourceIdentifier(InIdentifier);
	PickFunctionRefsContext->SetSourceIdentifier(InIdentifier);
	if(SourcePreviewEnvironment && InIdentifier.IsValid())
	{
		SourcePreviewEnvironment->SetFunctionNode(InIdentifier);
	}
}

void SRigVMSwapFunctionsWidget::SetTargetFunction(const FRigVMGraphFunctionIdentifier& InIdentifier)
{
	PickFunctionRefsContext->SetTargetIdentifier(InIdentifier);
	if(TargetPreviewEnvironment && InIdentifier.IsValid())
	{
		TargetPreviewEnvironment->SetFunctionNode(InIdentifier);
	}
}

#undef LOCTEXT_NAMESPACE
