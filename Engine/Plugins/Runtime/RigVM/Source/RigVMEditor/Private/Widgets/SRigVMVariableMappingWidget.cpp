// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/SRigVMVariableMappingWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRigVMVariableMappingWidget"

static const FLazyName ColumnId_VarLabel("Variable");
static const FLazyName ColumnID_MappingLabel("Mapping");

void SRigVMVariableMappingTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	OnVariableMappingChanged = InArgs._OnVariableMappingChanged;
	OnGetVariableMapping = InArgs._OnGetVariableMapping;
	OnGetAvailableMapping = InArgs._OnGetAvailableMapping;
	OnGetFilteredText = InArgs._OnGetFilteredText;
	OnVariableOptionAvailable = InArgs._OnVariableOptionAvailable;
	OnPinCheckStateChanged = InArgs._OnPinCheckStateChanged;
	OnPinGetCheckState = InArgs._OnPinGetCheckState;
	OnPinIsEnabledCheckState = InArgs._OnPinIsEnabledCheckState;

	check(Item.IsValid());

	VariableOptionList.Reset();
	if (OnGetAvailableMapping.IsBound())
	{
		TArray<FName> ListOfMappingVars;

		OnGetAvailableMapping.Execute(Item->GetPathName(), ListOfMappingVars);
		for (auto& MappingVar : ListOfMappingVars)
		{
			VariableOptionList.Add(MakeShareable(new FString(MappingVar.ToString())));
		}
	}
	//SMultiColumnTableRow< FRigVMVariableMappingInfoPtr >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
	STableRow<TSharedPtr<FRigVMVariableMappingInfoPtr>>::Construct(
		STableRow<TSharedPtr<FRigVMVariableMappingInfoPtr>>::FArguments()
		.Padding(FMargin(3.0f, 2.0f))
		.Content()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(0.5f)
			.Padding(0.0f, 4.0f)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FText::FromString(Item->GetDisplayName()))
				//.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("BoldText"))
				.HighlightText(this, &SRigVMVariableMappingTreeRow::GetFilterText)
				//.IsReadOnly(true)
				//.IsSelected(this, &SMultiColumnTableRow< FRigVMVariableMappingInfoPtr >::IsSelectedExclusively)
			]
// 			+ SHorizontalBox::Slot()
// 			.Padding(2.0f, 4.0f)
// 			.HAlign(HAlign_Right)
// 			[
// 				SNew(SHorizontalBox)
// // 				+ SHorizontalBox::Slot()
// // 				.FillWidth(0.5f)
// // 				[
// // 					SNew(STextBlock)
// // 					.Text(LOCTEXT("EmptyString", ""))
// // 				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 4.0f)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ExposeAsPinPropertyValue", "Use Pin "))
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 4.0f)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SRigVMVariableMappingTreeRow::IsPinChecked)
					.OnCheckStateChanged(this, &SRigVMVariableMappingTreeRow::OnPinCheckStatusChanged)
					.IsEnabled(this, &SRigVMVariableMappingTreeRow::IsPinEnabled)
				]
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 4.0f)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Optional", " OR Use Curve "))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.5f)
				[
					SAssignNew(VarOptionComboBox, SSearchableComboBox)
					.OptionsSource(&VariableOptionList)
					.OnGenerateWidget(this, &SRigVMVariableMappingTreeRow::MakeVarOptionComboWidget)
					.OnSelectionChanged(this, &SRigVMVariableMappingTreeRow::OnVarOptionSourceChanged)
					.OnComboBoxOpening(this, &SRigVMVariableMappingTreeRow::OnVarOptionComboOpening)
					.IsEnabled(this, &SRigVMVariableMappingTreeRow::IsVarOptionEnabled)
					.ContentPadding(2.f)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SRigVMVariableMappingTreeRow::GetVarOptionComboBoxContent)
						//.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.ToolTipText(this, &SRigVMVariableMappingTreeRow::GetVarOptionComboBoxToolTip)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(FOnClicked::CreateSP(this, &SRigVMVariableMappingTreeRow::OnClearButtonClicked))
					.Text(FText::FromString(TEXT("x")))
				]

			//]
		],
		InOwnerTableView);
}

bool SRigVMVariableMappingTreeRow::IsVarOptionEnabled() const
{
	// if pin is not checked
	if (IsPinChecked()==ECheckBoxState::Unchecked && 
		OnVariableOptionAvailable.IsBound())
	{
		return OnVariableOptionAvailable.Execute(Item->GetPathName());
	}

	return false;
}

ECheckBoxState SRigVMVariableMappingTreeRow::IsPinChecked() const
{
	if (OnPinGetCheckState.IsBound())
	{
		return OnPinGetCheckState.Execute(Item->GetPathName());
	}

	return ECheckBoxState::Unchecked;
}

void SRigVMVariableMappingTreeRow::OnPinCheckStatusChanged(ECheckBoxState NewState)
{
	if (OnPinCheckStateChanged.IsBound())
	{
		OnPinCheckStateChanged.Execute(NewState, Item->GetPathName());
	}
}

bool SRigVMVariableMappingTreeRow::IsPinEnabled() const
{
	if (OnPinIsEnabledCheckState.IsBound())
	{
		return OnPinIsEnabledCheckState.Execute(Item->GetPathName());
	}

	return false;
}


FReply SRigVMVariableMappingTreeRow::OnClearButtonClicked()
{
	if (OnVariableMappingChanged.IsBound())
	{
		OnVariableMappingChanged.Execute(Item->GetPathName(), NAME_None);
	}

	return FReply::Handled();
}

FText SRigVMVariableMappingTreeRow::GetFilterText() const
{
	if (OnGetFilteredText.IsBound())
	{
		return OnGetFilteredText.Execute();
	}

	return FText::GetEmpty();
}

void SRigVMVariableMappingTreeRow::OnVarOptionComboOpening()
{
	VariableOptionList.Reset();
	if (OnGetAvailableMapping.IsBound())
	{
		TArray<FName> ListOfMappingVars;

		OnGetAvailableMapping.Execute(Item->GetPathName(), ListOfMappingVars);
		for (auto& MappingVar : ListOfMappingVars)
		{
			VariableOptionList.Add(MakeShareable(new FString(MappingVar.ToString())));
		}
	}

	if (OnGetVariableMapping.IsBound())
	{
		FName MappedVar = OnGetVariableMapping.Execute(Item->GetPathName());
		if (MappedVar.IsNone())
		{
			VarOptionComboBox->ClearSelection();
		}
	}
}

void SRigVMVariableMappingTreeRow::OnVarOptionSourceChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		FString NewValue = *NewSelection.Get();
		FName NewName = FName(*NewValue);

		OnVariableMappingChanged.ExecuteIfBound(Item->GetPathName(), NewName);
	}
}

FText SRigVMVariableMappingTreeRow::GetVarOptionComboBoxContent() const
{
	if (OnGetVariableMapping.IsBound())
	{
		return FText::FromName(OnGetVariableMapping.Execute(Item->GetPathName()));
	}

	return FText::FromString(TEXT("Invalid"));
}

FText SRigVMVariableMappingTreeRow::GetVarOptionComboBoxToolTip() const
{
	return LOCTEXT("VarOptionComboToolTip", "Map input/output variable to available options.");
}

TSharedRef<SWidget> SRigVMVariableMappingTreeRow::MakeVarOptionComboWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));// .Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}
//////////////////////////////////////////////////////////////////////////
// SRigVMVariableMappingWidget

void SRigVMVariableMappingWidget::Construct(const FArguments& InArgs/*, FSimpleMulticastDelegate& InOnPostUndo*/)
{
	OnGetAvailableMappingDelegate = InArgs._OnGetAvailableMapping;
	OnGetVariableMappingDelegate = InArgs._OnGetVariableMapping;
	OnVariableMappingChangedDelegate = InArgs._OnVariableMappingChanged;
	OnCreateVariableMappingDelegate = InArgs._OnCreateVariableMapping;
	OnVariableOptionAvailableDelegate = InArgs._OnVariableOptionAvailable;
	OnPinCheckStateChangedDelegate = InArgs._OnPinCheckStateChanged;
	OnPinGetCheckStateDelegate = InArgs._OnPinGetCheckState;
	OnPinIsEnabledCheckStateDelegate = InArgs._OnPinIsEnabledCheckState;

	//InOnPostUndo.Add(FSimpleDelegate::CreateSP(this, &SRigVMVariableMappingWidget::PostUndo));

	ChildSlot
		[
			SNew(SVerticalBox)

			// now show bone mapping
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2)
			[
				SNew(SHorizontalBox)
				// Filter entry
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SAssignNew(NameFilterBox, SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged(this, &SRigVMVariableMappingWidget::OnFilterTextChanged)
					.OnTextCommitted(this, &SRigVMVariableMappingWidget::OnFilterTextCommitted)
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)		// This is required to make the scrollbar work, as content overflows Slate containers by default
			[
				SAssignNew(VariableMappingTreeView, SRigVMVariableMappingTreeView)
				.TreeItemsSource(&VariableMappingList)
				.OnGenerateRow(this, &SRigVMVariableMappingWidget::MakeTreeRowWidget)
				.OnGetChildren(this, &SRigVMVariableMappingWidget::GetChildrenForInfo)
			]
		];

	RefreshVariableMappingList();
}

TSharedRef<ITableRow> SRigVMVariableMappingWidget::MakeTreeRowWidget(TSharedPtr<FRigVMVariableMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRigVMVariableMappingTreeRow, OwnerTable)
		.Item(InInfo)
		.OnVariableMappingChanged(OnVariableMappingChangedDelegate)
		.OnGetVariableMapping(OnGetVariableMappingDelegate)
		.OnGetAvailableMapping(OnGetAvailableMappingDelegate)
		.OnGetFilteredText(this, &SRigVMVariableMappingWidget::GetFilterText)
		.OnVariableOptionAvailable(OnVariableOptionAvailableDelegate)
		.OnPinCheckStateChanged(OnPinCheckStateChangedDelegate)
		.OnPinGetCheckState(OnPinGetCheckStateDelegate)
		.OnPinIsEnabledCheckState(OnPinIsEnabledCheckStateDelegate);
}

void SRigVMVariableMappingWidget::GetChildrenForInfo(TSharedPtr<FRigVMVariableMappingInfo> InInfo, TArray< TSharedPtr<FRigVMVariableMappingInfo> >& OutChildren)
{
	OutChildren = InInfo->Children;
}

void SRigVMVariableMappingWidget::OnFilterTextChanged(const FText& SearchText)
{
	// need to make sure not to have the same text go
	// otherwise, the widget gets recreated multiple times causing 
	// other issue
	if (FilterText.CompareToCaseIgnored(SearchText) != 0)
	{
		FilterText = SearchText;
		RefreshVariableMappingList();
	}
}

void SRigVMVariableMappingWidget::OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo)
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged(SearchText);
}

TSharedRef<ITableRow> SRigVMVariableMappingWidget::GenerateVariableMappingRow(TSharedPtr<FRigVMVariableMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return
		SNew(SRigVMVariableMappingTreeRow, OwnerTable)
		.Item(InInfo)
		.OnVariableMappingChanged(OnVariableMappingChangedDelegate)
		.OnGetVariableMapping(OnGetVariableMappingDelegate)
		.OnGetAvailableMapping(OnGetAvailableMappingDelegate)
		.OnGetFilteredText(this, &SRigVMVariableMappingWidget::GetFilterText);
}

void SRigVMVariableMappingWidget::RefreshVariableMappingList()
{
	OnCreateVariableMappingDelegate.ExecuteIfBound(FilterText.ToString(), VariableMappingList);

	VariableMappingTreeView->RequestListRefresh();
}

// void SRigVMVariableMappingWidget::PostUndo()
// {
// 	RefreshVariableMappingList();
// }

#undef LOCTEXT_NAMESPACE

