// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMVariantTagWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "RigVMSettings.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SRigVMVariantTagWidget"

SRigVMVariantCapsule::SRigVMVariantCapsule()
{
}

SRigVMVariantCapsule::~SRigVMVariantCapsule()
{
}

void SRigVMVariantCapsule::Construct(
	const FArguments& InArgs)
{
	NameAttribute = InArgs._Name;
	ColorAttribute = InArgs._Color;
	OnRemoveTag = InArgs._OnRemoveTag;
	EnableContextMenu = InArgs._EnableContextMenu;

	TAttribute<float> MinDesiredWithAttribute;
	if(InArgs._MinDesiredLabelWidth > SMALL_NUMBER)
	{
		MinDesiredWithAttribute = InArgs._MinDesiredLabelWidth;
	}
	
	SButton::FArguments ButtonArgs;
	ButtonArgs
	.ButtonStyle( &FRigVMEditorStyle::Get().GetWidgetStyle< FButtonStyle >( "TagButton" ) )
	.ContentPadding(0.f)
	.HAlign(HAlign_Fill)
	.OnClicked(InArgs._OnClicked)
	.Content()
	[
		SNew(SBorder)
		.Padding(0.0f)
		.HAlign(HAlign_Fill)
		.BorderImage(InArgs._CapsuleTagBorder)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(0)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
				.ToolTipText(InArgs._ToolTipText)
				.ColorAndOpacity(this, &SRigVMVariantCapsule::GetColor)
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4,1,4,1))
			.AutoWidth()
			[
				SNew(STextBlock)
				.MinDesiredWidth(MinDesiredWithAttribute)
				.Text(InArgs._Label)
				.ToolTipText(InArgs._ToolTipText)
				.ColorAndOpacity(this, &SRigVMVariantCapsule::GetLabelColor)
			]
		]
	];
	
	SButton::Construct(ButtonArgs);

	SetPadding(InArgs._Padding);
}

FSlateColor SRigVMVariantCapsule::GetColor() const
{
	if(IsEnabled())
	{
		return ColorAttribute.Get();
	}
	return FLinearColor::Black;
}

FSlateColor SRigVMVariantCapsule::GetLabelColor() const
{
	if(IsEnabled())
	{
		return FSlateColor::UseForeground();
	}
	return FSlateColor::UseSubduedForeground();
}

FReply SRigVMVariantCapsule::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(EnableContextMenu.Get() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		FMenuBuilder MenuBuilder( true, NULL );
		if(OnRemoveTag.IsBound())
		{
			FUIAction RemoveTagAction( FExecuteAction::CreateSP( this, &SRigVMVariantCapsule::HandleRemoveTag) );
			MenuBuilder.AddMenuEntry( LOCTEXT("RemoveTag", "RemoveTag"), FText::GetEmpty(), FSlateIcon(), RemoveTagAction, NAME_None, EUserInterfaceActionType::Button);
		}
		const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
		return FReply::Handled();
	}
	return SButton::OnMouseButtonDown(MyGeometry, MouseEvent);
}

void SRigVMVariantCapsule::HandleRemoveTag()
{
	if(OnRemoveTag.IsBound() && (NameAttribute.IsBound() || NameAttribute.IsSet()))
	{
		(void)OnRemoveTag.Execute(NameAttribute.Get());
	}
}

SRigVMVariantTagWidget::SRigVMVariantTagWidget()
	: LastTagHash(0)
{
}

SRigVMVariantTagWidget::~SRigVMVariantTagWidget()
{
}

void SRigVMVariantTagWidget::Construct(
	const FArguments& InArgs)
{
	OnGetTags = InArgs._OnGetTags;
	OnAddTag = InArgs._OnAddTag;
	OnRemoveTag = InArgs._OnRemoveTag;
	CanAddTags = InArgs._CanAddTags;
	EnableContextMenu = InArgs._EnableContextMenu;
	LastTagHash = 0;
	MinDesiredLabelWidth = InArgs._MinDesiredLabelWidth;
	CapsuleTagBorder = InArgs._CapsuleTagBorder;

	const TAttribute<EVisibility> BoxVisibilityAttribute = TAttribute<EVisibility>::CreateLambda([this]() -> EVisibility
	{
		if(!OnGetTags.IsBound())
		{
			return EVisibility::Collapsed;
		}
		return OnGetTags.Execute().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	});

	if(InArgs._Orientation == EOrientation::Orient_Vertical)
	{
		SAssignNew(VerticalCapsuleBox, SVerticalBox)
		.Visibility(BoxVisibilityAttribute);
	}
	else
	{
		SAssignNew(HorizontalCapsuleBox, SHorizontalBox)
		.Visibility(BoxVisibilityAttribute);
	}

	const TSharedRef<SWidget> CapsuleBox = 
		HorizontalCapsuleBox.IsValid() ? 
		StaticCastSharedRef<SWidget>(HorizontalCapsuleBox.ToSharedRef()) :
		StaticCastSharedRef<SWidget>(VerticalCapsuleBox.ToSharedRef());

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.AutoHeight()
		.Padding( 0,0,0,0 )
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				CapsuleBox
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0)
			[
				SNew(STextBlock)
				.MinDesiredWidth(50)
				.Text(LOCTEXT("NoTags", "No tags"))
				.ToolTipText(LOCTEXT("NoTagsToolTip", "There are no tags currently applied. Press the plus button to add a tag."))
				.Visibility_Lambda([this]() -> EVisibility
				{
					if(!OnGetTags.IsBound())
					{
						return EVisibility::Collapsed;
					}
					return OnGetTags.Execute().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
				})
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Top)
			.Padding( 7,5,0,0 )
			[
				SNew(SComboButton)
				.Visibility_Lambda([this]() -> EVisibility
				{
					if(!CanAddTags.IsBound() && !CanAddTags.IsSet())
					{
						return EVisibility::Collapsed;
					}
					return CanAddTags.Get() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Method(EPopupMethod::UseCurrentWindow)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Icons.PlusCircle")))
				]
				.OnGetMenuContent(this, &SRigVMVariantTagWidget::OnBuildAddTagMenuContent)
			]
		]
	];

	UpdateCapsules();
	SetCanTick(InArgs._EnableTick);
}

void SRigVMVariantTagWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SBox::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	UpdateCapsules();
}

void SRigVMVariantTagWidget::UpdateCapsules()
{
	if(!OnGetTags.IsBound())
	{
		return;
	}

	TArray<FRigVMTag> AssignedTags = OnGetTags.Execute();
	AssignedTags.Sort([](const FRigVMTag& A, const FRigVMTag& B) -> bool
	{
		return FCString::Strcmp(*A.GetLabel(), *B.GetLabel()) < 0;
	});
	
	uint32 Hash = 0;
	for(const FRigVMTag& AssignedTag : AssignedTags)
	{
		Hash = HashCombine(Hash, GetTypeHash(AssignedTag));
	}

	if(Hash == LastTagHash)
	{
		return;
	}
	LastTagHash = Hash;

	if(HorizontalCapsuleBox)
	{
		HorizontalCapsuleBox->ClearChildren();
	}
	else
	{
		VerticalCapsuleBox->ClearChildren();
	}

	for(const FRigVMTag& AssignedTag : AssignedTags)
	{
		if(!AssignedTag.bShowInUserInterface)
		{
			continue;
		}

		TSharedRef<SRigVMVariantCapsule> Capsule = SNew(SRigVMVariantCapsule)
		.Name(AssignedTag.Name)
		.Color(AssignedTag.Color)
		.Label(FText::FromString(AssignedTag.GetLabel()))
		.ToolTipText(AssignedTag.ToolTip)
		.EnableContextMenu(EnableContextMenu)
		.MinDesiredLabelWidth(MinDesiredLabelWidth)
		.OnRemoveTag(OnRemoveTag)
		.CapsuleTagBorder(CapsuleTagBorder);

		if(HorizontalCapsuleBox)
		{
			HorizontalCapsuleBox->AddSlot()
			.Padding(0, 0, 3, 0)
			.AutoWidth()
			[
				Capsule
			];
		}
		else
		{
			VerticalCapsuleBox->AddSlot()
			.Padding(0, 3, 0, 0)
			[
				Capsule
			];
		}
	}
}

FReply SRigVMVariantTagWidget::OnAddTagClicked(const FName& InTagName) const
{
	if(OnAddTag.IsBound())
	{
		(void)OnAddTag.Execute(InTagName);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

TSharedRef<SWidget> SRigVMVariantTagWidget::OnBuildAddTagMenuContent() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(FName("AddTag"), LOCTEXT("AddTag", "Add Tag"));

	const URigVMProjectSettings* Settings = GetDefault<URigVMProjectSettings>(URigVMProjectSettings::StaticClass());
	TArray<FRigVMTag> AvailableTags = Settings->VariantTags;
	AvailableTags.Sort([](const FRigVMTag& A, const FRigVMTag& B) -> bool
	{
		return FCString::Strcmp(*A.GetLabel(), *B.GetLabel()) < 0;
	});

	TArray<FRigVMTag> AssignedTags;
	if(OnGetTags.IsBound())
	{
		AssignedTags = OnGetTags.Execute();
	}

	for(const FRigVMTag& AvailableTag
		: AvailableTags)
	{
		if(!AvailableTag.IsValid())
		{
			continue;
		}
		if(!AvailableTag.bShowInUserInterface)
		{
			continue;
		}

		const bool bAlreadyHasTag = AssignedTags.ContainsByPredicate([AvailableTag](const FRigVMTag& ExistingTag) -> bool
		{
			return AvailableTag.Name == ExistingTag.Name;
		});
		
		const FName TagName = AvailableTag.Name;
		MenuBuilder.AddWidget(
			SNew(SRigVMVariantCapsule)
			.IsEnabled(!bAlreadyHasTag)
			.Name(AvailableTag.Name)
			.Color(AvailableTag.Color)
			.Label(FText::FromString(AvailableTag.GetLabel()))
			.ToolTipText(AvailableTag.ToolTip)
			.EnableContextMenu(false)
			.MinDesiredLabelWidth(MinDesiredLabelWidth)
			.OnClicked_Lambda([this, TagName]()
			{
				const FReply Reply = OnAddTagClicked(TagName);
				if(WeakAddTagMenuWidget.IsValid())
				{
					FSlateApplication::Get().DismissMenuByWidget(WeakAddTagMenuWidget.Pin().ToSharedRef());
				}
				return Reply;
			})
			, FText(), false, true, AvailableTag.ToolTip);
	}

	MenuBuilder.EndSection();
	
	TSharedRef<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	WeakAddTagMenuWidget = MenuWidget.ToWeakPtr();
	
	return MenuWidget;
}

#undef LOCTEXT_NAMESPACE // SRigVMVariantTagWidget
