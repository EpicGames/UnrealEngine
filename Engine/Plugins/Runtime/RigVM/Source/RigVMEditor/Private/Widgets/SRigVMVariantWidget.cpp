// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRigVMVariantWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "DetailLayoutBuilder.h"
#include "Editor/RigVMEditorTools.h"
#include "AssetThumbnail.h"
#include "RigVMModel/RigVMBuildData.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SRigVMVariantWidget"

void SRigVMVariantToolTipWithTags::Construct(const FArguments& InArgs)
{
	GetTagsDelegate = InArgs._OnGetTags;

	SuperClassArgs._Text = InArgs._ToolTipText;
	SToolTip::Construct(
	SuperClassArgs
		.TextMargin(11.0f)
		.BorderImage( FCoreStyle::Get().GetBrush("ToolTip.BrightBackground") )
	);
}

bool SRigVMVariantToolTipWithTags::IsEmpty() const
{
	if(!GetTextTooltip().IsEmpty())
	{
		return false;
	}
	if(GetTagsDelegate.IsBound())
	{
		return GetTagsDelegate.Execute().IsEmpty();
	}
	return true;
}

void SRigVMVariantToolTipWithTags::OnOpening()
{
	const TSharedPtr<SVerticalBox> ContentsWidget = SNew(SVerticalBox);

	ContentsWidget->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.Padding(0.f, 0.f, 0.f, 0.f)
	[
		SNew( STextBlock )
		.Text( SuperClassArgs._Text )
		.Font( SuperClassArgs._Font )
		.ColorAndOpacity( FLinearColor::Black )
		.WrapTextAt_Static( &SToolTip::GetToolTipWrapWidth )
	];

	if(GetTagsDelegate.IsBound())
	{
		ContentsWidget->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SRigVMVariantTagWidget)
			.Visibility_Lambda([this]() -> EVisibility
			{
				return GetTagsDelegate.Execute().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
			})
			.OnGetTags(GetTagsDelegate)
			.Orientation(EOrientation::Orient_Horizontal)
			.CanAddTags(false)
			.EnableContextMenu(false)
		];
	}
	SetContentWidget(ContentsWidget->AsShared());
}
	
void SRigVMVariantToolTipWithTags::OnClosed()
{
	SToolTip::OnClosed();
	ResetContentWidget();
}

void SRigVMVariantGuidWidget::Construct(const FArguments& InArgs)
{
	OnContextMenu = InArgs._OnContextMenu;
	
	SBox::FArguments SuperArguments;
	SuperArguments.Padding(0);
	SuperArguments.HAlign(HAlign_Left);
	SuperArguments.VAlign(VAlign_Center);

	TAttribute<FGuid> Guid = InArgs._Guid;
	SuperArguments.Content()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text_Lambda([this, Guid]()
		{
			return FText::FromString(Guid.Get().ToString(EGuidFormats::DigitsWithHyphensLower));
		})
	];
	
	SBox::Construct(SuperArguments);
}

FReply SRigVMVariantGuidWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(OnContextMenu.IsBound() && MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		if(const TSharedPtr<SWidget> ContextMenuWidget = OnContextMenu.Execute())
		{
			FSlateApplication::Get().PushMenu(
				SharedThis(this),
				FWidgetPath(),
				ContextMenuWidget.ToSharedRef(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
			return FReply::Handled();
		}
	}
	return SBox::OnMouseButtonDown(MyGeometry, MouseEvent);
}

SRigVMVariantWidget::SRigVMVariantWidget()
	: VariantRefHash(UINT32_MAX)
{
}

SRigVMVariantWidget::~SRigVMVariantWidget()
{
}

void SRigVMVariantWidget::Construct(
	const FArguments& InArgs)
{
	VariantAttribute = InArgs._Variant;
	SubjectVariantRefAttribute = InArgs._SubjectVariantRef;
	
	OnVariantChanged = InArgs._OnVariantChanged;

	VariantRefsAttribute = InArgs._VariantRefs;
	OnCreateVariantRefRow = InArgs._OnCreateVariantRefRow;
	OnBrowseVariantRef = InArgs._OnBrowseVariantRef;
	OnVariantRefContextMenu = InArgs._OnVariantRefContextMenu;

	if(!OnCreateVariantRefRow.IsBound())
	{
		OnCreateVariantRefRow.BindSP(this, &SRigVMVariantWidget::CreateDefaultVariantRefRow);
	}

	if(!OnVariantRefContextMenu.IsBound())
	{
		OnVariantRefContextMenu.BindSP(this, &SRigVMVariantWidget::CreateDefaultVariantRefContextMenu);
	}

	ContextAttribute = InArgs._Context;
	if(!ContextAttribute.IsSet() && !ContextAttribute.IsBound())
	{
		ContextAttribute = FRigVMVariantWidgetContext();
	}

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 8, 0, 0)
		.HAlign(HAlign_Left)
		[
			SNew(SRigVMVariantGuidWidget)
			.Guid_Lambda([this]()
			{
				return VariantAttribute.Get().Guid;
			})
			.OnContextMenu_Lambda([this]() -> TSharedPtr<SWidget>
			{
				const FRigVMVariantRef SubjectVariantRef = SubjectVariantRefAttribute.Get();
				if(SubjectVariantRef.IsValid())
				{
					return OnVariantRefContextMenu.Execute(SubjectVariantRef);
				}
				return SNullWidget::NullWidget;
			})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0, 8, 0, 0)
		[
			SAssignNew(TagWidget, SRigVMVariantTagWidget)
			.OnGetTags(InArgs._OnGetTags)
			.OnAddTag(InArgs._OnAddTag)
			.OnRemoveTag(InArgs._OnRemoveTag)
			.CanAddTags(InArgs._CanAddTags)
			.EnableContextMenu(InArgs._EnableTagContextMenu)
			.MinDesiredLabelWidth(50.f)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0, 8, 0, 0)
		[
			SNew(SScrollBox)
			.Visibility(this, &SRigVMVariantWidget::GetVariantRefListVisibility)
			+ SScrollBox::Slot()
			.MaxSize(InArgs._MaxVariantRefListHeight)
			[
				SAssignNew(VariantRefListBox, SVerticalBox)
			]
		]
	];

	VariantRefListBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(0, 4, 0, 0)
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text_Lambda([this]()
		{
			return VariantTreeRowInfos.IsEmpty() ?
				LOCTEXT("NoOtherVariants", "No other variants found.") :
				LOCTEXT("MatchingVariants", "Matching Variants:");
		})
	];

	VariantRefListBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Left)
	.Padding(0, 0, 0, 0)
	[
		SAssignNew(VariantRefTreeView, STreeView<TSharedPtr<FVariantTreeRowInfo>>)
		.SelectionMode(ESelectionMode::None)
		.OnMouseButtonDoubleClick_Lambda([this](TSharedPtr<FVariantTreeRowInfo> InRowInfo)
		{
			(void)OnBrowseVariantRef.ExecuteIfBound(InRowInfo->VariantRef);
		})
		.Visibility_Lambda([this]()
		{
			return VariantTreeRowInfos.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
		})
		.TreeItemsSource(&VariantTreeRowInfos)
		.OnGenerateRow(this, &SRigVMVariantWidget::GenerateVariantTreeRow)
		.OnGetChildren(this, &SRigVMVariantWidget::GetChildrenForVariantInfo)
		.OnContextMenuOpening(this, &SRigVMVariantWidget::OnVariantRefTreeContextMenu)
	];

	SetCanTick(true);
}

void SRigVMVariantWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SBox::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	uint32 NewHash = 0;
	const TArray<FRigVMVariantRef> NewVariantRefs = VariantRefsAttribute.Get();
	for(const FRigVMVariantRef& NewVariantRef : NewVariantRefs)
	{
		NewHash = HashCombine(NewHash, GetTypeHash(NewVariantRef));
	}

	if(NewHash != VariantRefHash)
	{
		VariantRefHash = NewHash;
		VariantRefs = NewVariantRefs;

		// sort the variants by path length - but make sure that
		// variant refs within our own context come first
		const FString ParentPath = GetVariantContext().ParentPath; 
		VariantRefs.Sort([ParentPath](const FRigVMVariantRef& A, const FRigVMVariantRef& B)
		{
			FString PathA = A.ObjectPath.ToString(); 
			FString PathB = B.ObjectPath.ToString();
			if(PathA.StartsWith(ParentPath, ESearchCase::CaseSensitive))
			{
				PathA = PathA.Mid(ParentPath.Len());
			}
			if(PathB.StartsWith(ParentPath, ESearchCase::CaseSensitive))
			{
				PathB = PathB.Mid(ParentPath.Len());
			}
			return PathA.Compare(PathB) < 0;
		});
		
		RebuildVariantRefList();
	}
}

SRigVMVariantWidget::SRigVMVariantRefTreeRow::~SRigVMVariantRefTreeRow()
{
}

void SRigVMVariantWidget::SRigVMVariantRefTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
{
	STableRow<TSharedPtr<FVariantTreeRowInfo>>::FArguments SuperArguments;
	SuperArguments.Content()
	[
		InArgs._Content.ToSharedRef()
	];
	SuperArguments.Padding(0.f);
	
	STableRow< TSharedPtr<FVariantTreeRowInfo> >::Construct(SuperArguments, OwnerTableView);
}

const FRigVMVariantWidgetContext& SRigVMVariantWidget::GetVariantContext() const
{
	return ContextAttribute.Get();
}

EVisibility SRigVMVariantWidget::GetVariantRefListVisibility() const
{
	return VariantRefs.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedRef<ITableRow> SRigVMVariantWidget::GenerateVariantTreeRow(TSharedPtr<FVariantTreeRowInfo> InRowInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> RowWidget = SNew(SRigVMVariantRefTreeRow, OwnerTable)
		.Content(OnCreateVariantRefRow.Execute(InRowInfo->VariantRef));

	if(InRowInfo)
	{
		InRowInfo->RowWidget = RowWidget.ToWeakPtr();
	}
	return RowWidget;
}

void SRigVMVariantWidget::GetChildrenForVariantInfo(TSharedPtr<FVariantTreeRowInfo> InInfo, TArray<TSharedPtr<FVariantTreeRowInfo>>& OutChildren)
{
	OutChildren = InInfo->NestedInfos;
}

TSharedPtr<SWidget> SRigVMVariantWidget::CreateDefaultVariantRefRow(const FRigVMVariantRef& InVariantRef) const
{
	const FRigVMVariantRef LocalVariantRef = InVariantRef;
	
	TSharedPtr<SToolTip> TooltipWithTags = SNew(SRigVMVariantToolTipWithTags)
		.ToolTipText(FText::FromString(InVariantRef.ObjectPath.ToString()))
		.OnGetTags_Lambda([LocalVariantRef]()
		{
			return LocalVariantRef.Variant.Tags;
		});
	
	if(!InVariantRef.ObjectPath.IsSubobject())
	{
		const FAssetData AssetData = UE::RigVM::Editor::Tools::FindAssetFromAnyPath(InVariantRef.ObjectPath.ToString(), true);
		const TSharedPtr<FAssetThumbnail> AssetThumbnail = MakeShareable(new FAssetThumbnail(AssetData, 32, 32, TSharedPtr<FAssetThumbnailPool>() ));
		const FAssetThumbnailConfig ThumbnailConfig;

		TSharedRef<SBorder> ThumbnailBorder = SNew(SBorder);
		ThumbnailBorder->SetVisibility(EVisibility::SelfHitTestInvisible);
		ThumbnailBorder->SetPadding(FMargin(0.0f, 0.0f, 4.0f, 4.0f));
		ThumbnailBorder->SetBorderImage(FAppStyle::Get().GetBrush("PropertyEditor.AssetTileItem.DropShadow"));
		ThumbnailBorder->SetContent(
			SNew(SOverlay)
			+SOverlay::Slot()
			.Padding(1.0f)
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FStyleDefaults::GetNoBrush())
				.OnMouseDoubleClick_Lambda(
					[this, LocalVariantRef](
						const FGeometry&,
						const FPointerEvent&) -> FReply
					{
						(void)OnBrowseVariantRef.ExecuteIfBound(LocalVariantRef);
						return FReply::Handled();
					})
				[
					SNew(SBox)
					.ToolTip(TooltipWithTags)
					.WidthOverride(32.f)
					.HeightOverride(32.f)
					[
						AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig)
					]
				]
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(this, &SRigVMVariantWidget::GetThumbnailBorder, ThumbnailBorder)
				.Visibility(EVisibility::SelfHitTestInvisible)
			]
		);

		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.0f,3.0f,5.0f,0.0f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			ThumbnailBorder
		]

		+ SHorizontalBox::Slot()
		.Padding(0.0f)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoHeight()
			[
				SNew(SComboButton)
				.ToolTip(TooltipWithTags)
				.IsEnabled(false)
				.ButtonContent()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.FillWidth(1)
					.VAlign(VAlign_Center)
					[
						// Show the name of the asset or actor
						SNew(STextBlock)
						.Font( FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont") ) )
						.Text(FText::FromString(InVariantRef.ObjectPath.GetAssetName()))
					]
				]
			]
		];
	}
	

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	const FSlateBrush* Icon = nullptr;
	
	static const FString RigVMFunctionLibraryToken = TEXT("RigVMFunctionLibrary");
	if(InVariantRef.ObjectPath.ToString().Contains(RigVMFunctionLibraryToken, ESearchCase::CaseSensitive))
	{
		static FSlateIcon FunctionIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
		Icon = FunctionIcon.GetIcon(); 
	}

	if(Icon)
	{
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0, 0, 3, 0)
		[
			SNew(SImage)
			.Image(Icon)
			.DesiredSizeOverride(FVector2D(16, 16))
		];
	}

	FString DisplayLabel;
	if(InVariantRef.ObjectPath.IsSubobject())
	{
		DisplayLabel = InVariantRef.ObjectPath.GetSubPathString();
		(void)DisplayLabel.Split(TEXT("."), nullptr, &DisplayLabel, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
	}
	else
	{
		DisplayLabel = InVariantRef.ObjectPath.GetAssetName();
	}

	HorizontalBox->AddSlot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.AutoWidth()
	.Padding(0, 0, 0, 0)
	[
		SNew(STextBlock)
		.Text(FText::FromString(DisplayLabel))
	];

	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.OnClicked_Lambda([this, LocalVariantRef]() -> FReply
			{
				(void)OnBrowseVariantRef.ExecuteIfBound(LocalVariantRef);
				return FReply::Handled();
			})
		.ContentPadding(FMargin(1, 0))
		.ToolTip(TooltipWithTags)
		[
			HorizontalBox
		];
}

void SRigVMVariantWidget::RebuildVariantRefList()
{
	VariantTreeRowInfos.Reset();
	TMap<FString, TSharedPtr<FVariantTreeRowInfo>> PathToRowInfo;

	const TArray<FRigVMVariantRef> AllAssetVariantRefs = URigVMBuildData::Get()->GatherAllAssetVariantRefs();
	TMap<FString, FRigVMVariantRef> AssetPathToVariantRef;
	for(const FRigVMVariantRef& AssetVariantRef : AllAssetVariantRefs)
	{
		AssetPathToVariantRef.FindOrAdd(AssetVariantRef.ObjectPath.ToString()) = AssetVariantRef;
	}

	const FSoftObjectPath ContextAssetObjectPath = FSoftObjectPath(GetVariantContext().ParentPath).GetWithoutSubPath();
	const FString ContextAssetPath = ContextAssetObjectPath.ToString();
	 
	for(FRigVMVariantRef VariantRef : VariantRefs)
	{
		TSharedPtr<FVariantTreeRowInfo> ParentRowInfo;
		if(VariantRef.ObjectPath.IsSubobject())
		{
			const FString AssetPath = VariantRef.ObjectPath.GetWithoutSubPath().ToString();
			if(AssetPath != ContextAssetPath)
			{
				if(const FRigVMVariantRef* AssetVariantRef = AssetPathToVariantRef.Find(AssetPath))
				{
					if(!PathToRowInfo.Contains(AssetPath))
					{
						ParentRowInfo = MakeShareable(new FVariantTreeRowInfo);
						ParentRowInfo->VariantRef = *AssetVariantRef;
						PathToRowInfo.Add(AssetPath, ParentRowInfo);
						VariantTreeRowInfos.Add(ParentRowInfo);
					}
					else
					{
						ParentRowInfo = PathToRowInfo.FindChecked(AssetPath);
					}
				}
			}
		}

		TSharedPtr<FVariantTreeRowInfo> RowInfo = MakeShareable(new FVariantTreeRowInfo);
		RowInfo->VariantRef = VariantRef;
		if(ParentRowInfo)
		{
			ParentRowInfo->NestedInfos.Add(RowInfo);
		}
		else
		{
			VariantTreeRowInfos.Add(RowInfo);
		}
	}
	
	VariantRefTreeView->RequestTreeRefresh();
}

const FSlateBrush* SRigVMVariantWidget::GetThumbnailBorder(TSharedRef<SBorder> InThumbnailBorder) const
{
	static const FName HoveredBorderName("PropertyEditor.AssetThumbnailBorderHovered");
	static const FName RegularBorderName("PropertyEditor.AssetThumbnailBorder");
	return InThumbnailBorder->IsHovered() ? FAppStyle::Get().GetBrush(HoveredBorderName) : FAppStyle::Get().GetBrush(RegularBorderName);
}

TSharedPtr<SWidget> SRigVMVariantWidget::OnVariantRefTreeContextMenu()
{
	const FVector2f MousePosition = FSlateApplication::Get().GetCursorPos();

	for(const TSharedPtr<FVariantTreeRowInfo>& VariantTreeRowInfo : VariantTreeRowInfos)
	{
		if(VariantTreeRowInfo->RowWidget.IsValid())
		{
			if(const TSharedPtr<SWidget> Content = VariantTreeRowInfo->RowWidget.Pin()->GetContent())
			{
				const FGeometry TickSpaceGeometry = Content->GetTickSpaceGeometry();
				if(TickSpaceGeometry.IsUnderLocation(MousePosition))
				{
					return OnVariantRefContextMenu.Execute(VariantTreeRowInfo->VariantRef);
				}
			}
		}
	}
	return SNullWidget::NullWidget;
}

TSharedPtr<SWidget> SRigVMVariantWidget::CreateDefaultVariantRefContextMenu(const FRigVMVariantRef& InVariantRef)
{
	FMenuBuilder MenuBuilder(true, nullptr);
	{
		if (InVariantRef.IsValid())
		{
			URigVMBuildData* BuildData = URigVMBuildData::Get();
			TArray<FRigVMVariantRef> MatchingVariants = BuildData->FindAssetVariantRefs(InVariantRef.Variant.Guid);
			if(MatchingVariants.IsEmpty())
			{
				MatchingVariants = BuildData->FindFunctionVariantRefs(InVariantRef.Variant.Guid);
			}

			MenuBuilder.AddMenuEntry(
			 LOCTEXT("CopyGuidToClipboardLabel", "Copy Variant Guid"),
			 LOCTEXT("CopyGuidToClipboardTooltip", "Copies the variant guid to the clipboard"),
			 FSlateIcon{},
			 FUIAction(FExecuteAction::CreateLambda([this, InVariantRef]()
				{
				 	FPlatformApplicationMisc::ClipboardCopy(*InVariantRef.Variant.Guid.ToString(EGuidFormats::DigitsWithHyphensLower));
				}),
				FCanExecuteAction())
			);

			MenuBuilder.AddMenuEntry(
				 LOCTEXT("SplitVariantRefLabel", "Split Variant from Set"),
				 LOCTEXT("SplitVariantRefTooltip", "Removes this variant from the set and applies a unique GUID to it"),
				 FSlateIcon{},
				 FUIAction(FExecuteAction::CreateLambda([this, InVariantRef, BuildData]()
					{
				 		BuildData->SplitVariantFromSet(InVariantRef);
				 		RebuildVariantRefList();
					}),
					FCanExecuteAction::CreateLambda([MatchingVariants]() -> bool
					{
						return MatchingVariants.Num() > 1;
					}))
				);

			// for the main subject also offer to join another set
			const FRigVMVariantRef SubjectVariantRef = SubjectVariantRefAttribute.Get();
			if(SubjectVariantRef == InVariantRef)
			{
				MenuBuilder.AddMenuEntry(
					 LOCTEXT("JoinVariantRefLabel", "Join Variant Set (from Clipboard)"),
					 LOCTEXT("JoinVariantRefTooltip", "Joins the Variant Set given the GUID on the clipboard"),
					 FSlateIcon{},
					 FUIAction(FExecuteAction::CreateLambda([this, InVariantRef, BuildData]()
						{
					 		FString TextToImport;
							FPlatformApplicationMisc::ClipboardPaste(TextToImport);
						 	const FGuid GuidToCheck(TextToImport);
					 		BuildData->JoinVariantSet(InVariantRef, GuidToCheck);
							RebuildVariantRefList();
						}),
						FCanExecuteAction::CreateLambda([InVariantRef]() -> bool
						{
							FString TextToImport;
							FPlatformApplicationMisc::ClipboardPaste(TextToImport);
							if(!TextToImport.IsEmpty())
							{
								const FGuid GuidToCheck(TextToImport);
								return GuidToCheck.IsValid() && GuidToCheck != InVariantRef.Variant.Guid;
							}
							return false;
						}))
					);
			}
		}
	}
	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE // SRigVMVariantWidget
