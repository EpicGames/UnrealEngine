// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphEnumNodeSpawner : public URigVMEdGraphNodeSpawner
{
public:

	virtual ~URigVMEdGraphEnumNodeSpawner() {}

	/**
	 * Creates a new URigVMEdGraphEnumNodeSpawner
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_API URigVMEdGraphEnumNodeSpawner(UEnum* InEnum, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const override;
	// End URigVMEdGraphNodeSpawner interface

private:

	UEnum* Enum;
	friend class UEngineTestControlRig;
};

#undef UE_API
