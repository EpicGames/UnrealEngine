// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "RigVMBlueprintLegacy.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphFunctionRefNodeSpawner : public URigVMEdGraphNodeSpawner
{

public:

	virtual ~URigVMEdGraphFunctionRefNodeSpawner() {}
	
	/**
	 * Creates a new URigVMEdGraphFunctionRefNodeSpawner, charged with spawning a function reference
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_DEPRECATED(5.7, "Plase use URigVMEdGraphFunctionRefNodeSpawner(URigVMLibraryNode* InFunction)")
	static URigVMEdGraphFunctionRefNodeSpawner* CreateFromFunction(URigVMLibraryNode* InFunction) { return nullptr; }
	UE_API URigVMEdGraphFunctionRefNodeSpawner(URigVMLibraryNode* InFunction);

	/**
	 * Creates a new URigVMEdGraphFunctionRefNodeSpawner, charged with spawning a function reference
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMGraphFunctionHeader& InPublicFunction)")
	static URigVMEdGraphFunctionRefNodeSpawner* CreateFromAssetData(const FAssetData& InAssetData, const FRigVMGraphFunctionHeader& InPublicFunction) { return nullptr; }
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMOldPublicFunctionData& InPublicFunction)")
	static URigVMEdGraphFunctionRefNodeSpawner* CreateFromAssetData(const FAssetData& InAssetData, const FRigVMOldPublicFunctionData& InPublicFunction) { return nullptr; }
	UE_API URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMGraphFunctionHeader& InPublicFunction);
	UE_API URigVMEdGraphFunctionRefNodeSpawner(const FAssetData& InAssetData, const FRigVMOldPublicFunctionData& InPublicFunction);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const override;
	UE_API virtual bool IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const override;
	// End URigVMEdGraphNodeSpawner interface

private:
	/** Returns a resolved asset path. This is directly from the asset data if CreateFromAssetData was used to create the spawner, or if
	 *  CreateFromFunction, the resolved function path.
	 */
	FSoftObjectPath GetResolvedAssetPath() const;
	
	/** The public function definition we will spawn from [optional] */
	mutable FRigVMGraphFunctionHeader ReferencedPublicFunctionHeader;

	/** Marked as true for local function definitions */
	bool bIsLocalFunction;

	mutable FSoftObjectPath AssetPath;
	mutable FString FunctionPath;

	static UE_API URigVMEdGraphNode* SpawnNode(UEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr Blueprint, FRigVMGraphFunctionHeader& InFunction, FVector2D const Location);

	friend class UEngineTestControlRig;
	friend class FRigVMEditorBase;
};

#undef UE_API
