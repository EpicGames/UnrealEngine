// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "RigVMModel/RigVMGraph.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphInvokeEntryNodeSpawner : public URigVMEdGraphNodeSpawner
{
public:
	virtual ~URigVMEdGraphInvokeEntryNodeSpawner() {}

	UE_DEPRECATED(5.7, "Plase use URigVMEdGraphInvokeEntryNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FName& InEntryName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)")
	static URigVMEdGraphInvokeEntryNodeSpawner* CreateForEntry(URigVMBlueprint* InBlueprint, const FName& InEntryName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip) { return nullptr; }
	UE_API URigVMEdGraphInvokeEntryNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FName& InEntryName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const override;
	UE_API virtual bool IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const override;
	// End URigVMEdGraphNodeSpawner interface

private:

	/** The pin type we will spawn */
	TWeakInterfacePtr<IRigVMEditorAssetInterface> Blueprint;
	TWeakObjectPtr<URigVMGraph> GraphOwner;
	FName EntryName;

	friend class UEngineTestControlRig;
};

#undef UE_API
