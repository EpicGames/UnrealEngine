// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintLegacy.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintActionFilter.h"
#include "RigVMEdGraphNodeSpawner.generated.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphNodeSpawner : public TSharedFromThis<URigVMEdGraphNodeSpawner>
{
public:
	virtual ~URigVMEdGraphNodeSpawner() {}

	UE_DEPRECATED(5.7, "Plase use SetRelatedBlueprintClass(UClass* InClass)")
	void SetRelatedBlueprintClass(TSubclassOf<URigVMBlueprint> InClass){}
	UE_API void SetRelatedBlueprintClass(UClass* InClass);

	struct FPinInfo
	{
		FName Name;
		ERigVMPinDirection Direction;
		FName CPPType;
		UObject* CPPTypeObject;

		FPinInfo()
			: Name(NAME_None)
			, Direction(ERigVMPinDirection::Invalid)
			, CPPType(NAME_None)
			, CPPTypeObject(nullptr)
		{}

		FPinInfo(const FName& InName, ERigVMPinDirection InDirection, const FName& InCPPType, UObject* InCPPTypeObject)
		: Name(InName)
		, Direction(InDirection)
		, CPPType(InCPPType)
		, CPPTypeObject(InCPPTypeObject)
		{}
	};

	UE_API virtual bool IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const;
	virtual FString GetSpawnerSignature() const { return FString(); }
	virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const { return nullptr; }
	static UE_API URigVMEdGraphNode* SpawnTemplateNode(URigVMEdGraph* InParentGraph, const TArray<FPinInfo>& InPins, const FName& InNodeName = NAME_None);

	FText GetMenuName() const { return MenuName;}
	FText GetMenuTooltip() const { return MenuTooltip;}
	FText GetMenuCategory() const { return MenuCategory;}
	FText GetMenuKeywords() const { return MenuKeywords;}
	FSlateIcon GetMenuIcon() const { return MenuIcon;}
	FLinearColor GetMenuColor() const { return MenuIconTint;}
	
protected:

	UClass* RelatedBlueprintClass;
	UClass* NodeClass;

	// Menu signature
	FText MenuName;
	FText MenuTooltip;
	FText MenuCategory;
	FText MenuKeywords;
	FSlateIcon MenuIcon;
	FLinearColor MenuIconTint = FLinearColor::White;

	friend class URigVMEdGraphNodeBlueprintSpawner;
};

UCLASS(MinimalAPI, Transient)
class URigVMEdGraphNodeBlueprintSpawner : public UBlueprintNodeSpawner
{
	GENERATED_BODY()

public:

	static URigVMEdGraphNodeBlueprintSpawner* CreateFromRigVMSpawner(TSharedPtr<URigVMEdGraphNodeSpawner> InNodeSpawner)
	{
		URigVMEdGraphNodeBlueprintSpawner* NewSpawner = NewObject<URigVMEdGraphNodeBlueprintSpawner>(GetTransientPackage());
		NewSpawner->RigVMSpawner = InNodeSpawner;
		NewSpawner->NodeClass = InNodeSpawner->NodeClass;
		NewSpawner->DefaultMenuSignature.MenuName = InNodeSpawner->MenuName;
		NewSpawner->DefaultMenuSignature.Tooltip = InNodeSpawner->MenuTooltip;
		NewSpawner->DefaultMenuSignature.Category = InNodeSpawner->MenuCategory;
		NewSpawner->DefaultMenuSignature.Keywords = InNodeSpawner->MenuKeywords;
		NewSpawner->DefaultMenuSignature.Icon = InNodeSpawner->MenuIcon;
		NewSpawner->DefaultMenuSignature.IconTint = InNodeSpawner->MenuIconTint;
		return NewSpawner;
	}

	// UBlueprintNodeSpawner interface
	UE_API virtual bool IsTemplateNodeFilteredOut(FBlueprintActionFilter const& Filter) const override;
	UE_API virtual void Prime() override;
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual FBlueprintActionUiSpec GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

private:

	TSharedPtr<URigVMEdGraphNodeSpawner> RigVMSpawner;
};



#undef UE_API
