// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMModel/Nodes/RigVMTemplateNode.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphTemplateNodeSpawner : public URigVMEdGraphNodeSpawner
{
public:
	
	virtual ~URigVMEdGraphTemplateNodeSpawner() {}
	/**
	 * Creates a new URigVMEdGraphVariableNodeSpawner, charged with spawning 
	 * a new member-variable node
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphTemplateNodeSpawner(const FName& InNotation, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)")
	static URigVMEdGraphTemplateNodeSpawner* CreateFromNotation(const FName& InNotation, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip) { return nullptr; }
	UE_API URigVMEdGraphTemplateNodeSpawner(const FName& InNotation, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API bool IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const override;
	// End URigVMEdGraphNodeSpawner interface

private:
	/** The unit type we will spawn */
	UPROPERTY(Transient)
	FName TemplateNotation;

	const FRigVMTemplate* Template = nullptr;

	static UE_API URigVMEdGraphNode* SpawnNode(URigVMEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr Blueprint, const FRigVMTemplate* Template, FVector2D const Location);

	friend class UEngineTestControlRig;
	friend class FRigVMEditorBase;
};

#undef UE_API
