// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "RigVMBlueprintLegacy.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "RigVMModel/Nodes/RigVMUnitNode.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphUnitNodeSpawner : public URigVMEdGraphNodeSpawner
{
public:
	
	virtual ~URigVMEdGraphUnitNodeSpawner() {}
	
	/**
	 * Creates a new URigVMEdGraphVariableNodeSpawner, charged with spawning 
	 * a new unit node
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphUnitNodeSpawner(UScriptStruct* InStruct, const FName& InMethodName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)")
	static URigVMEdGraphUnitNodeSpawner* CreateFromStruct(UScriptStruct* InStruct, const FName& InMethodName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip) { return nullptr; }
	UE_API URigVMEdGraphUnitNodeSpawner(UScriptStruct* InStruct, const FName& InMethodName, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D const Location) const override;
	// URigVMEdGraphNodeSpawner interface
	
	static UE_API void HookupMutableNode(URigVMNode* InModelNode, FRigVMEditorAssetInterfacePtr InRigBlueprint);

private:
	/** The unit type we will spawn */
	TObjectPtr<UScriptStruct> StructTemplate;

	FName MethodName;

	static UE_API URigVMEdGraphNode* SpawnNode(URigVMEdGraph* ParentGraph, FRigVMEditorAssetInterfacePtr Blueprint, UScriptStruct* StructTemplate, const
	                                           FName& InMethodName, FVector2D Location);

	friend class UEngineTestControlRig;
	friend class FRigVMEditorBase;
};

#undef UE_API
