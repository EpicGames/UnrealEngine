// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "UObject/UnrealType.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "RigVMCore/RigVMExternalVariable.h"
#include "RigVMCore/RigVMVariableDescription.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class URigVMEdGraphVariableNodeSpawner : public URigVMEdGraphNodeSpawner
{

public:
	
	virtual ~URigVMEdGraphVariableNodeSpawner() {}

	/**
	 * Creates a new URigVMEdGraphVariableNodeSpawner, charged with spawning 
	 * a new member-variable node
	 * 
	 * @return A newly allocated instance of this class.
	 */
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphVariableNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FRigVMExternalVariable& InExternalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)")
	static URigVMEdGraphVariableNodeSpawner* CreateFromExternalVariable(URigVMBlueprint* InBlueprint, const FRigVMExternalVariable& InExternalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip) { return nullptr; }
	UE_DEPRECATED(5.7, "Please use URigVMEdGraphVariableNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, URigVMGraph* InGraphOwner, const FRigVMGraphVariableDescription& InLocalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip)")
	static URigVMEdGraphVariableNodeSpawner* CreateFromLocalVariable(URigVMBlueprint* InBlueprint, URigVMGraph* InGraphOwner, const FRigVMGraphVariableDescription& InLocalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip) { return nullptr; }

	UE_API URigVMEdGraphVariableNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, const FRigVMExternalVariable& InExternalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);
	UE_API URigVMEdGraphVariableNodeSpawner(FRigVMEditorAssetInterfacePtr InBlueprint, URigVMGraph* InGraphOwner, const FRigVMGraphVariableDescription& InLocalVariable, bool bInIsGetter, const FText& InMenuDesc, const FText& InCategory, const FText& InTooltip);

	// URigVMEdGraphNodeSpawner interface
	UE_API virtual FString GetSpawnerSignature() const override;
	UE_API virtual URigVMEdGraphNode* Invoke(URigVMEdGraph* ParentGraph, FVector2D Location) const override;
	UE_API virtual bool IsTemplateNodeFilteredOut(TArray<UObject*>& InAssets, TArray<UEdGraph*> InGraphs, TArray<UEdGraphPin*> InPins) const override;
	// End URigVMEdGraphNodeSpawner interface

private:

	/** The pin type we will spawn */
	TWeakInterfacePtr<IRigVMEditorAssetInterface> Blueprint;
	TWeakObjectPtr<URigVMGraph> GraphOwner;
	FRigVMExternalVariable ExternalVariable;
	bool bIsGetter;
	bool bIsLocalVariable;	

	friend class UEngineTestControlRig;
};

#undef UE_API
