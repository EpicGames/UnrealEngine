// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintConnectionDrawingPolicy.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;

class FRigVMEdGraphConnectionDrawingPolicy : public FKismetConnectionDrawingPolicy
{
public:
	FRigVMEdGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
		: FKismetConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements, InGraphObj)
	{
		CastImage = FAppStyle::GetBrush( TEXT("GraphEditor.Cast_16x") );
		ArrowImage = nullptr;
		ArrowRadius = CastImage->ImageSize * InZoomFactor * 0.5f;
		MidpointImage = nullptr;
		MidpointRadius = CastImage->ImageSize * InZoomFactor * 0.5f;
	}

	UE_API virtual void SetIncompatiblePinDrawState(const TSharedPtr<SGraphPin>& StartPin, const TSet< TSharedRef<SWidget> >& VisiblePins) override;
	UE_API virtual void ResetIncompatiblePinDrawState(const TSet< TSharedRef<SWidget> >& VisiblePins) override;
	UE_API virtual void BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries) override;
	UE_API virtual void DrawPinGeometries(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes) override;
	UE_API virtual void DetermineLinkGeometry(
	FArrangedChildren& ArrangedNodes, 
	TSharedRef<SWidget>& OutputPinWidget,
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	/*out*/ FArrangedWidget*& StartWidgetGeometry,
	/*out*/ FArrangedWidget*& EndWidgetGeometry) override;
	UE_API virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	virtual bool UseDrawStateCaching() const override { return true; }

private:
	// Each time a reroute node is encountered, input geometry is compared to output geometry to see if the pins on the reroute node need to be reversed
	TMap<URigVMEdGraphNode*, bool> RerouteNodeToReversedDirectionMap;

	bool UseLowDetailConnections() const { return ZoomFactor <= 0.175f; /* zoom level -9 */ }

	UE_API bool ShouldChangeTangentForReouteControlPoint(URigVMEdGraphNode* Node);
	// Average of the positions of all pins connected to InPin
	UE_API bool GetAverageConnectedPositionForPin(UEdGraphPin* InPin, FVector2f& OutPos) const;

	const FSlateBrush* CastImage;
};

#undef UE_API
