// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

#define UE_API RIGVMEDITOR_API

class FRigVMEdGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
public:
	UE_API virtual FName GetFactoryName() const;
private:
	UE_API virtual TSharedPtr<class SGraphNode> CreateNode(UEdGraphNode* Node) const override;
};

#undef UE_API
