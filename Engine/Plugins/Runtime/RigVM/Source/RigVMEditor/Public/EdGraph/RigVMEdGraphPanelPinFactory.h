// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"

#define UE_API RIGVMEDITOR_API

class FRigVMEdGraphPanelPinFactory : public FGraphPanelPinFactory
{
public:
	UE_API virtual FName GetFactoryName() const;
	
	// FGraphPanelPinFactory interface
	UE_API virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;


	UE_API virtual TSharedPtr<class SGraphPin> CreatePin_Internal(class UEdGraphPin* InPin) const;
};

#undef UE_API
