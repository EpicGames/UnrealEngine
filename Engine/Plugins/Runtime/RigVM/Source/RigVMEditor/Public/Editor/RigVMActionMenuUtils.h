// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

class UClass;
class UK2Node;
struct FBlueprintActionContext;
struct FRigVMActionMenuBuilder;
struct FEdGraphSchemaAction;


struct FRigVMActionMenuUtils
{
	
	/**
	 * A centralized utility function for constructing blueprint context menus.
	 * Rolls the supplied Context and SelectedProperties into a series of 
	 * filters that're used to construct the menu.
	 *
	 * @param  Context				Contains the blueprint/graph/pin that the menu is for.
	 * @param  bIsContextSensitive	
	 * @param  ClassTargetMask		
	 * @param  MenuOut				The structure that will be populated with context menu items.
	 */
	RIGVMEDITOR_API static void MakeContextMenu(FBlueprintActionContext const& Context, bool bIsContextSensitive, uint32 ClassTargetMask, FRigVMActionMenuBuilder& MenuOut);

	/**
	 * A number of different palette actions hold onto node-templates in different 
	 * ways. This handles most of those cases and looks to extract said node- 
	 * template from the specified action.
	 * 
	 * @param  PaletteAction	The action you want a node-template for.
	 * @return A pointer to the extracted node (NULL if the action doesn't have one, or we don't support the specific action type yet)
	 */
	RIGVMEDITOR_API static const UK2Node* ExtractNodeTemplateFromAction(const FEdGraphSchemaAction& PaletteAction);
	RIGVMEDITOR_API static const UK2Node* ExtractNodeTemplateFromAction(const TSharedPtr<FEdGraphSchemaAction>& PaletteAction);
};

