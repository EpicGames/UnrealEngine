// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "RigVMEditorAsset.h"

#define UE_API RIGVMEDITOR_API

class IDetailLayoutBuilder;
class IRigVMEditor;

class FRigVMClassSettingsDetailCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<IRigVMEditor> InEditor);
	UE_API FRigVMClassSettingsDetailCustomization(TSharedPtr<IRigVMEditor> InEditor, FRigVMEditorAssetInterfacePtr Asset);


	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	void CustomizeSettings(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IRigVMEditor> Editor, FRigVMEditorAssetInterfacePtr Asset);
	void CustomizeDefaults(IDetailLayoutBuilder& DetailLayout, TSharedPtr<IRigVMEditor> Editor, FRigVMEditorAssetInterfacePtr Asset);

private:
	UE_API void OnFinishedChangingSettings(const FPropertyChangedEvent& PropertyChangedEvent);

private:
	/** The Blueprint editor we are embedded in */
	TWeakPtr<IRigVMEditor> EditorPtr;

	/** The blueprint we are editing */
	TWeakInterfacePtr<IRigVMEditorAssetInterface> AssetPtr;

	
};

#undef UE_API
