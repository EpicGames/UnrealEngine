// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "RigVMHost.h"
#include "RigVMEditorAsset.h"
#include "Styling/SlateTypes.h"
#include "IPropertyUtilities.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditor;
class IPropertyHandle;

class FRigVMCommentNodeDetailCustomization : public IDetailCustomization
{
	FRigVMCommentNodeDetailCustomization()
	: BlueprintBeingCustomized(nullptr)
	{}

	
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMCommentNodeDetailCustomization);
	}

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized;
	mutable TArray<TWeakObjectPtr<URigVMCommentNode>> ObjectsBeingCustomized;
	
	UE_API void CleanupInvalidObjects() const;

	UE_API void GetValuesFromNode(TWeakObjectPtr<URigVMCommentNode> Node);
	UE_API void SetValues(TWeakObjectPtr<URigVMCommentNode> Node);
	
	UE_API FText GetText() const;
	UE_API void SetText(const FText& InNewText, ETextCommit::Type InCommitType);

	UE_API FLinearColor GetColor() const;
	UE_API FReply OnChooseColor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API void OnColorPicked(FLinearColor LinearColor);

	UE_API ECheckBoxState IsShowingBubbleEnabled() const;
	UE_API void OnShowingBubbleStateChanged(ECheckBoxState InValue);

	UE_API ECheckBoxState IsColorBubbleEnabled() const;
	UE_API void OnColorBubbleStateChanged(ECheckBoxState InValue);

	UE_API TOptional<int> GetFontSize() const;
	UE_API void OnFontSizeChanged(int32 InValue, ETextCommit::Type Arg);
	
	FString CommentText;
	bool bShowingBubble;
	bool bBubbleColorEnabled;
	int32 FontSize;
};



#undef UE_API
