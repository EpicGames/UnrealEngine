// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API RIGVMEDITOR_API

class FRigVMNewEditor;

struct FRigVMDetailsInspectorTabSummoner : public FWorkflowTabFactory
{
public:

	static const FName TabID() { static FName ID = TEXT("RigVM Details"); return ID; }

public:
	UE_API FRigVMDetailsInspectorTabSummoner(const TSharedRef<FRigVMNewEditor>& InRigVMEditor);

	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	UE_API virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FRigVMNewEditor> RigVMEditor;
};

#undef UE_API
