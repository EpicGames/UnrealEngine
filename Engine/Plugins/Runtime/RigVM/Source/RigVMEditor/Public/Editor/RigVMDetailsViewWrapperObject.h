// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMNode.h"
#include "RigVMDetailsViewWrapperObject.generated.h"

#define UE_API RIGVMEDITOR_API

class URigVMDetailsViewWrapperObject;

DECLARE_EVENT_ThreeParams(URigVMDetailsViewWrapperObject, FWrappedPropertyChangedChainEvent, URigVMDetailsViewWrapperObject*, const FString&, FPropertyChangedChainEvent&);

UCLASS(MinimalAPI)
class URigVMDetailsViewWrapperObject : public UObject
{
public:
	GENERATED_BODY()

	UE_API URigVMDetailsViewWrapperObject();

	// Creating wrappers from a given struct
	UE_API virtual UClass* GetClassForStruct(UScriptStruct* InStruct, bool bCreateIfNeeded = true) const;
	static UE_API URigVMDetailsViewWrapperObject* MakeInstance(UClass* InWrapperObjectClass, UObject* InOuter, UScriptStruct* InStruct, uint8* InStructMemory, UObject* InSubject = nullptr);
	UE_API UScriptStruct* GetWrappedStruct() const;

	// Creating wrappers from a RigVMNode
	UE_API virtual UClass* GetClassForNodes(TArray<URigVMNode*> InNodes, bool bCreateIfNeeded = true) const;
	static UE_API URigVMDetailsViewWrapperObject* MakeInstance(UClass* InWrapperObjectClass, UObject* InOuter, TArray<URigVMNode*> InNodes, URigVMNode* InSubject);
	
	static UE_API void MarkOutdatedClass(UClass* InClass);
	static UE_API bool IsValidClass(UClass* InClass);
	UE_API bool IsActive() const;
	
	static UE_API URigVMDetailsViewWrapperObject* GetOrCreateNewInstance(const UClass* InWrapperObjectClass, UObject* InOuter);
	
	// Return an instance to the pool for reuse
	static UE_API void ReturnToPool(URigVMDetailsViewWrapperObject* InInstance);

	// Clear pooled instance of a certain class
	static UE_API void ClearClassPool(UClass* InClass);
	
	// Clear all pooled instances (call on module shutdown)
	static UE_API void ClearAllClassPools();
	
	UE_API FString GetWrappedNodeNotation() const;
	
	bool IsChildOf(const UStruct* InStruct) const
	{
		const UScriptStruct* WrappedStruct = GetWrappedStruct();
		return WrappedStruct && WrappedStruct->IsChildOf(InStruct);
	}

	template<typename T>
	bool IsChildOf() const
	{
		return IsChildOf(T::StaticStruct());
	}

	UE_API virtual void SetContent(const uint8* InStructMemory, const UStruct* InStruct);
	UE_API virtual void GetContent(uint8* OutStructMemory, const UStruct* InStruct) const;
	UE_API void SetContent(URigVMNode* InNode);

	template<typename T>
	T GetContent() const
	{
		check(IsChildOf<T>());
		
		T Result;
		GetContent((uint8*)&Result, T::StaticStruct());
		return Result;
	}

	template<typename T>
	void SetContent(const T& InValue)
	{
		check(IsChildOf<T>());

		SetContent((const uint8*)&InValue, T::StaticStruct());
	}

	UE_API UObject* GetSubject() const;
	UE_API void SetSubject(UObject* InSubject);

	FWrappedPropertyChangedChainEvent& GetWrappedPropertyChangedChainEvent() { return WrappedPropertyChangedChainEvent; }
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

private:
	
	// Reset instance state for reuse
	void ResetForReuse();

	static UE_API void CopyPropertiesForUnrelatedStructs(uint8* InTargetMemory, const UStruct* InTargetStruct, const uint8* InSourceMemory, const UStruct* InSourceStruct);
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);
	UE_API void SetContentForPin(URigVMPin* InPin);

	struct FPerClassInfo
	{
		FString Notation;
		UScriptStruct* ScriptStruct;
		
		FPerClassInfo()
			: Notation()
			, ScriptStruct(nullptr)
		{}

		FPerClassInfo(UScriptStruct* InScriptStruct)
		: Notation()
		, ScriptStruct(InScriptStruct)
		{}

		FPerClassInfo(const FString& InNotation)
		: Notation(InNotation)
		, ScriptStruct(nullptr)
		{}

		friend uint32 GetTypeHash(const FPerClassInfo& Info)
		{
			return HashCombine(GetTypeHash(Info.Notation), GetTypeHash(Info.ScriptStruct));
		}

		bool operator ==(const FPerClassInfo& Other) const
		{
			return Notation == Other.Notation && ScriptStruct == Other.ScriptStruct;
		}

		bool operator !=(const FPerClassInfo& Other) const
		{
			return Notation != Other.Notation || ScriptStruct != Other.ScriptStruct;
		}
	};
	
	static UE_API TMap<FPerClassInfo, UClass*> InfoToClass;
	static UE_API TMap<UClass*, FPerClassInfo> ClassToInfo;
	static UE_API TSet<UClass*> OutdatedClassToRecreate;
	
	// Pool of available wrapper objects per class
	static TMap<UClass*, TArray<URigVMDetailsViewWrapperObject*>> InstancePool;

	// Maximum pool size per class (prevents unbounded growth)
	static constexpr int32 MaxPoolSizePerClass = 32;
	
	bool bIsSettingValue;

	UPROPERTY()
	TWeakObjectPtr<UObject> SubjectPtr;
	
	FWrappedPropertyChangedChainEvent WrappedPropertyChangedChainEvent;
};

#undef UE_API
