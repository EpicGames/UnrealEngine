// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMEditorGraphExplorerTabSummoner.h"
#include "RigVMNewEditorMode.h"
#include "RigVMCore/RigVMExecuteContext.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMModel/RigVMController.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "RigVMModel/RigVMNotifications.h"
#include "Widgets/SRigVMEditorGraphExplorer.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "GraphEditor.h"
#include "Misc/NotifyHook.h"
#include "TickableEditorObject.h"
#include "EditorUndoClient.h"
#include "RigVMEditorModule.h"
#include "SNodePanel.h"
#include "Editor/RigVMDetailsViewWrapperObject.h"

#define UE_API RIGVMEDITOR_API

#define LOCTEXT_NAMESPACE "RigVMEditor"

class IRigVMEditorAssetInterface;
class FDocumentTracker;
class SRigVMEditorGraphExplorer;
class URigVM;
class FTabInfo;
class FRigVMEditorBase;
class URigVMController;
class URigVMHost;
class FTransaction;
class SGraphEditor;
class FPreviewScene;

DECLARE_MULTICAST_DELEGATE_TwoParams(FRigVMEditorClosed, const IRigVMEditor*, FRigVMEditorAssetInterfacePtr);
DECLARE_DELEGATE_RetVal_TwoParams(FReply, FRigVMEditorKeyDownDelegate, const FGeometry&, const FKeyEvent&);


/* Enums to use when grouping the blueprint members in the list panel. The order here will determine the order in the list */
namespace RigVMNodeSectionID
{
	// Keep the values as they are defined in NodeSectionID, which is defined in BlueprintEditor.h
	// TODO: Once there is no need for FRigVMLegacyEditor, improve the definition of this enum, including a uint8 definition
	enum Type
	{
		NONE = 0,
		GRAPH = 1,					// Graph
		FUNCTION = 4,				// Functions
		VARIABLE = 8,				// Variables
		LOCAL_VARIABLE = 12			// Local variables
	};
};

struct FRigVMEditorModes
{
	// Mode constants
	static inline const FLazyName RigVMEditorMode = FLazyName(TEXT("RigVM"));
	static FText GetLocalizedMode(const FName InMode)
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add(RigVMEditorMode, NSLOCTEXT("RigVMEditorModes", "RigVMEditorMode", "RigVM"));
		}

		check(InMode != NAME_None);
		const FText* OutDesc = LocModes.Find(InMode);
		check(OutDesc);
		return *OutDesc;
	}
private:
	FRigVMEditorModes() {}
};

struct FRigVMCustomDebugObject
{
public:
	// Custom object to include, regardless of the current debugging World
	UObject* Object;

	// Override for the object name (if not empty)
	FString NameOverride;

public:
	FRigVMCustomDebugObject()
		: Object(nullptr)
	{
	}

	FRigVMCustomDebugObject(UObject* InObject, const FString& InLabel)
		: Object(InObject)
		, NameOverride(InLabel)
	{
	}
};

class IRigVMEditor
{
public:

	virtual TSharedPtr<FAssetEditorToolkit> GetHostingApp() = 0;
	UE_DEPRECATED(5.7, "Plase use GetRigVMAsset")
	virtual URigVMBlueprint* GetRigVMBlueprint() const { return Cast<URigVMBlueprint>(GetRigVMAssetInterface().GetObject()); }
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const = 0;
	virtual const TArray< UObject* >* GetObjectsCurrentlyBeingEdited() const = 0;
	virtual URigVMHost* GetRigVMHost() const = 0;

	virtual TSharedPtr<FTabManager> GetTabManager() = 0;
	virtual FName GetGraphExplorerWidgetID() = 0;
	virtual TSharedPtr<class SRigVMDetailsInspector> GetRigVMInspector() const = 0;
	virtual TSharedPtr<SRigVMEditorGraphExplorer> GetGraphExplorerWidget() = 0;
#if WITH_RIGVMLEGACYEDITOR
	virtual TSharedPtr<class SKismetInspector> GetKismetInspector() const = 0;
#endif
	
	virtual bool GetIsContextSensitive() = 0;
	virtual void SetIsContextSensitive(const bool bIsContextSensitive) = 0;

	virtual void SetGraphExplorerWidget(TSharedPtr<SRigVMEditorGraphExplorer> InWidget) = 0;
	
	virtual void GetPinTypeSelectorFilters(TArray<TSharedPtr<class IPinTypeSelectorFilter>>& OutFilters) const = 0;

	DECLARE_EVENT(IRigVMEditor, FOnRefreshEvent);
	virtual FOnRefreshEvent OnRefresh() = 0;
	virtual void ForceEditorRefresh(ERefreshRigVMEditorReason::Type Reason = ERefreshRigVMEditorReason::UnknownReason) = 0;

	DECLARE_EVENT_OneParam(IRigVMEditor, FPreviewHostUpdated, IRigVMEditor*);
	virtual FPreviewHostUpdated& OnPreviewHostUpdated()  = 0;

	virtual FRigVMEditorClosed& OnEditorClosed() = 0;
	virtual FRigVMEditorKeyDownDelegate& GetKeyDownDelegate() = 0;
	virtual UEdGraph* GetFocusedGraph() const = 0;
	virtual URigVMGraph* GetFocusedModel() const = 0;
	virtual FNotifyHook* GetNotifyHook() = 0;
	virtual TWeakPtr<class SGraphEditor> GetFocusedGraphEditor() = 0;

	virtual bool InEditingMode() const = 0;
	virtual bool IsEditable(UEdGraph* InGraph) const = 0;
	
	virtual void OnGraphNodeDropToPerform(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) = 0;
	virtual TSharedRef<FUICommandList> GetToolkitCommands() = 0;

	virtual bool SelectLocalVariable(const UEdGraph* Graph, const FName& VariableName) = 0;
	virtual bool SelectVariable(const FName& VariableName) = 0;
	virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename = false) = 0;
	virtual void OnAddNewLocalVariable() = 0;
	virtual bool CanAddNewLocalVariable() const = 0;
	virtual void OnAddNewVariable() = 0;
	virtual void OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription) = 0;
	virtual void AddNewFunctionVariant(const UEdGraph* InOriginalFunction) = 0;
	virtual TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause) = 0;
	
	// Type of new document/graph being created by a menu item
	enum ECreatedDocumentType
	{
		CGT_NewVariable,
		CGT_NewFunctionGraph,
		CGT_NewMacroGraph,
		CGT_NewAnimationLayer,
		CGT_NewEventGraph,
		CGT_NewLocalVariable
	};
	virtual void OnNewDocumentClicked(ECreatedDocumentType GraphType) = 0;
	virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false) = 0;
	
	virtual void GetDebugObjects(TArray<FRigVMCustomDebugObject>& DebugList) const = 0;
	virtual bool OnlyShowCustomDebugObjects() const = 0;
	virtual FString GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const = 0;

	virtual TSharedPtr<SGraphEditor> OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus = true) = 0;
	virtual void ZoomToSelection_Clicked() = 0;

	virtual void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) = 0;
	virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) = 0;
	virtual FPreviewScene* GetPreviewScene() = 0;

	virtual bool IsRunGraphOnceOn() const = 0;
	virtual bool CanRunGraphOnce() const = 0;

	virtual bool IsDetailsPanelEditingGlobalOptions() const = 0;
	virtual bool IsDetailsPanelEditingClassDefaults() const = 0;

protected:
	
	virtual TSharedRef<IRigVMEditor> SharedRef() = 0;
	virtual TSharedRef<const IRigVMEditor> SharedRef() const = 0;

	
	virtual const TSharedPtr<FAssetEditorToolkit> GetHostingApp() const = 0;
	virtual void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable = false, const bool bInUseSmallToolbarIcons = false, const TOptional<EAssetOpenMethod>& InOpenMethod = TOptional<EAssetOpenMethod>()) = 0;
	virtual void CreateEditorToolbar() = 0;
	virtual void CommonInitialization(const TArray<FRigVMEditorAssetInterfacePtr>& InitBlueprints, bool bShouldOpenInDefaultsMode) = 0;
	virtual TSharedPtr<FDocumentTracker> GetDocumentManager() const = 0;
	virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode) = 0;
	virtual void RegenerateMenusAndToolbars() = 0;
	virtual void SetCurrentMode(FName NewMode) = 0;
	virtual FEditorModeTools& GetToolkitEditorModeManager() const = 0;
	virtual void PostLayoutBlueprintEditorInitialization() = 0;
	virtual bool FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results) = 0;
	virtual void CloseDocumentTab(const UObject* DocumentID) = 0;
	virtual TSharedPtr<FApplicationMode> CreateEditorMode() = 0;
	virtual const FName GetEditorAppName() const = 0;
	virtual const TArray< UObject* >& GetEditingBlueprints() const = 0;
	virtual const TSharedRef<IToolkitHost> GetToolkitHost() const = 0;
	virtual bool IsHosted() const = 0;
	virtual void BringToolkitToFrontImpl() = 0;
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) = 0;
	virtual TWeakPtr<FDocumentTabFactory> GetGraphEditorTabFactory() const = 0;
	virtual bool TransactionObjectAffectsBlueprint(UObject* InTransactedObject) = 0;
	virtual FEdGraphPinType GetLastPinTypeUsed() = 0;
	virtual void LogSimpleMessage(const FText& MessageText) = 0;
	virtual void RenameNewlyAddedAction(FName InActionName) = 0;
	virtual FGraphPanelSelectionSet GetSelectedNodes() const = 0;
	virtual void SetUISelectionState(FName SelectionOwner) = 0;
	virtual void AnalyticsTrackNodeEvent(IRigVMEditorAssetInterface* Blueprint, UEdGraphNode *GraphNode, bool bNodeDelete = false) const = 0;
	virtual void PostUndo(bool bSuccess) = 0;
	virtual void PostRedo(bool bSuccess) = 0;
	virtual UEdGraphPin* GetCurrentlySelectedPin() const = 0;
	virtual void CreateDefaultCommands() = 0;
	virtual TSharedRef<SGraphEditor> CreateGraphEditorWidget(TSharedRef<class FTabInfo> InTabInfo, class UEdGraph* InGraph) = 0;
	virtual void CompileImpl() = 0;
	virtual void SaveAsset_Execute_Impl() = 0;
	virtual void SaveAssetAs_Execute_Impl() = 0;
	virtual bool IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const = 0;
	virtual bool IsEditableImpl(UEdGraph* InGraph) const = 0;
	virtual void JumpToNode(const class UEdGraphNode* Node, bool bRequestRename = false) = 0;
	virtual void JumpToPin(const class UEdGraphPin* Pin) = 0;
	virtual void AddToolbarExtender(TSharedPtr<FExtender> Extender) = 0;
	virtual void RemoveToolbarExtender(TSharedPtr<FExtender> Extender) = 0;
	virtual void AddMenuExtender(TSharedPtr<FExtender> Extender) = 0;
	virtual void RemoveMenuExtender(TSharedPtr<FExtender> Extender) = 0;
	virtual TSharedPtr<class IMessageLogListing> GetCompilerResultsListing() = 0;
	virtual void OnBlueprintChangedInnerImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled = false) = 0;
	virtual void RefreshEditorsImpl(ERefreshRigVMEditorReason::Type Reason = ERefreshRigVMEditorReason::UnknownReason) = 0;
	virtual void SetupGraphEditorEventsImpl(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents) = 0;
	virtual FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed) = 0;
	virtual void AddReferencedObjectsImpl(FReferenceCollector& Collector) = 0;
	virtual FGraphAppearanceInfo GetGraphAppearanceImpl(class UEdGraph* InGraph) const = 0;
	virtual void NotifyPreChangeImpl( FProperty* PropertyAboutToChange ) = 0;
	virtual void NotifyPostChangeImpl( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) = 0;
	virtual FName GetSelectedVariableName() = 0;
	virtual bool IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename) = 0;
	virtual void EditClassDefaults_Clicked() = 0;
	virtual void EditGlobalOptions_Clicked() = 0;
	virtual void TryInvokingDetailsTab(bool bFlash = true) = 0;
	virtual void RefreshInspector() = 0;
	virtual void RefreshStandAloneDefaultsEditor() = 0;
	virtual void RestoreEditedObjectState() = 0;
	virtual void SetupViewForBlueprintEditingMode() = 0;
	virtual void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager) = 0;
	virtual void AddCompileWidget(FToolBarBuilder& ToolbarBuilder) = 0;
	virtual void AddSettingsAndDefaultWidget(FToolBarBuilder& ToolbarBuilder) = 0;
	virtual void AddFindWidget(FToolBarBuilder& ToolbarBuilder) = 0;
	virtual void AddSelectedDebugObjectWidget(FToolBarBuilder& ToolbarBuilder) = 0;
	virtual void AddAutoCompileWidget(FToolBarBuilder& ToolbarBuilder) = 0;
	virtual void AddRunOnceWidget(FToolBarBuilder& ToolbarBuilder) = 0;
};

class FRigVMEditorBase : public IRigVMEditor
{
public:
	
	/**
	 * Edits the specified asset(s)
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InRigVMBlueprint	The blueprint object to start editing.
	 */
	UE_API virtual void InitRigVMEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, FRigVMEditorAssetInterfacePtr InRigVMBlueprint);

	static UE_API FRigVMEditorBase* GetFromAssetEditorInstance(IAssetEditorInstance* Instance);
	// returns the blueprint being edited
	UE_DEPRECATED(5.7, "Please use GetRigVMAsset")
	UE_API virtual URigVMBlueprint* GetRigVMBlueprint() const override;
	UE_API virtual FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const override;
	
	UE_API void HandleJumpToHyperlink(const UObject* InSubject);

	UE_API void Compile();
	UE_API bool IsCompilingEnabled() const;

	UE_API void DeleteSelectedNodes();
	UE_API bool CanDeleteNodes() const;
	UE_API void CopySelectedNodes();
	UE_API bool CanCopyNodes() const;
	UE_API void PasteNodes();
	UE_API bool CanPasteNodes() const;
	UE_API void CutSelectedNodes();
	UE_API bool CanCutNodes() const;
	UE_API void DuplicateNodes();
	UE_API bool CanDuplicateNodes() const;
	
	UE_API void OnStartWatchingPin();
	UE_API bool CanStartWatchingPin() const;
	UE_API void OnStopWatchingPin();
	UE_API bool CanStopWatchingPin() const;

	virtual FOnRefreshEvent OnRefresh() override { return OnRefreshEvent; }

	UE_API FText GetGraphDecorationString(UEdGraph* InGraph) const;
	UE_API virtual bool IsEditable(UEdGraph* InGraph) const override;

	UE_API virtual void OnGraphNodeDropToPerform(TSharedPtr<FDragDropOperation> InDragDropOp, UEdGraph* InGraph, const FVector2f& InNodePosition, const FVector2f& InScreenPosition) override;

	/**
	 * Util for finding a glyph and color for a variable.
	 *
	 * @param Property       The variable's property
	 * @param IconColorOut      The resulting color for the glyph
	 * @param SecondaryBrushOut The resulting secondary glyph brush (used for Map types)
	 * @param SecondaryColorOut The resulting secondary color for the glyph (used for Map types)
	 * @return					The resulting glyph brush
	 */
	static UE_API FSlateBrush const* GetVarIconAndColorFromProperty(const FProperty* Property, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut);

	/**
	* Util for finding a glyph and color for a variable.
	*
	* @param PinType       The variable's pin type
	* @param IconColorOut      The resulting color for the glyph
	* @param SecondaryBrushOut The resulting secondary glyph brush (used for Map types)
	* @param SecondaryColorOut The resulting secondary color for the glyph (used for Map types)
	* @return					The resulting glyph brush
	*/
	static UE_API FSlateBrush const* GetVarIconAndColorFromPinType(const FEdGraphPinType& PinType, FSlateColor& IconColorOut, FSlateBrush const*& SecondaryBrushOut, FSlateColor& SecondaryColorOut);
	
protected:
	
	UE_API FRigVMEditorBase();
	virtual ~FRigVMEditorBase()= default;

	UE_API void UnbindEditor();
	

	UE_API void HandleAssetRequestedOpen(UObject* InObject);
	UE_API void HandleAssetRequestClose(UObject* InObject, EAssetEditorCloseReason InReason);
	bool bRequestedReopen = false;

	UE_API virtual const FName GetEditorModeName() const;

	// FBlueprintEditor interface
	UE_API virtual bool InEditingMode() const override;
	UE_API TSubclassOf<UEdGraphSchema> GetDefaultSchemaClass() const;
	UE_API void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor);

	//  FTickableEditorObject Interface
	UE_API void Tick(float DeltaTime);

	// IToolkit Interface
	UE_API void BringToolkitToFront();
	UE_API FName GetToolkitFName() const;
	UE_API FName GetToolkitContextFName() const;
	UE_API FText GetBaseToolkitName() const;
	UE_API FText GetToolkitToolTipText() const;
	UE_API FString GetWorldCentricTabPrefix() const;
	UE_API FLinearColor GetWorldCentricTabColorScale() const;	
	UE_API void InitToolMenuContextImpl(FToolMenuContext& MenuContext);

	// BlueprintEditor interface
	UE_API bool TransactionObjectAffectsBlueprintImpl(UObject* InTransactedObject);
	UE_API virtual bool CanAddNewLocalVariable() const override;
	UE_API virtual void OnAddNewLocalVariable() override;
	UE_API virtual void OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription) override;


	bool IsNativeParentClassCodeLinkEnabled() const { return false; }
	bool ReparentBlueprint_IsVisible() const { return false; }
	UE_API FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph);
	bool ShouldLoadBPLibrariesFromAssetRegistry() { return false; }
	UE_API bool JumpToHyperlinkImpl(const UObject* ObjectReference, bool bRequestRename = false);
	virtual bool ShouldOpenGraphByDefault() const { return true; }
	UE_API virtual void AddNewFunctionVariant(const UEdGraph* InOriginalFunction) override;

	// FEditorUndoClient Interface
	UE_API void PostUndoImpl(bool bSuccess);
	UE_API void PostRedoImpl(bool bSuccess);

	UE_API virtual void PostTransaction(bool bSuccess, const FTransaction* Transaction, bool bIsRedo);

	//  FTickableEditorObject Interface
	UE_API TStatId GetStatId() const;

	// returns the currently debugged / viewed host
	UE_API virtual URigVMHost* GetRigVMHost() const override;

	UE_API virtual UObject* GetOuterForHost() const;

	// returns the class to use for detail wrapper objects (UI shim layer)
	UE_API virtual UClass* GetDetailWrapperClass() const;

	// allows the editor to fill an empty graph
	virtual void CreateEmptyGraphContent(URigVMController* InController) {}

	virtual FPreviewHostUpdated& OnPreviewHostUpdated() override { return PreviewHostUpdated;  }

	virtual FRigVMEditorClosed& OnEditorClosed() override { return RigVMEditorClosedDelegate; }
	virtual FRigVMEditorKeyDownDelegate& GetKeyDownDelegate() override { return OnKeyDownDelegate; }

	UE_API virtual bool SelectLocalVariable(const UEdGraph* Graph, const FName& VariableName) override;
	UE_API virtual bool SelectVariable(const FName& VariableName) override;

	// FBlueprintEditor Interface
	UE_API void CreateDefaultCommandsImpl();
	UE_API void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList);

	UE_API void SaveAsset_Execute();
	UE_API void SaveAssetAs_Execute();
	bool IsInAScriptingMode() const { return true; }
	UE_API virtual void OnNewDocumentClicked(ECreatedDocumentType GraphType) override;
	UE_API bool IsSectionVisibleImpl(RigVMNodeSectionID::Type InSectionID) const;
	UE_API bool NewDocument_IsVisibleForTypeImpl(ECreatedDocumentType GraphType) const;
	UE_API FGraphAppearanceInfo GetGraphAppearance(class UEdGraph* InGraph) const;
	UE_API void OnActiveTabChanged( TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated );
	UE_API void OnSelectedNodesChangedImpl(const TSet<class UObject*>& NewSelection);
	UE_API void OnBlueprintChangedImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled);
	UE_API virtual void ForceEditorRefresh(ERefreshRigVMEditorReason::Type Reason = ERefreshRigVMEditorReason::UnknownReason) override;
	UE_API void SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents);
	UE_API void FocusInspectorOnGraphSelection(const TSet<class UObject*>& NewSelection, bool bForceRefresh = false);
#if WITH_RIGVMLEGACYEDITOR
	virtual TSharedPtr<SKismetInspector> GetKismetInspector() const override { return nullptr; }
#endif

	UE_API virtual void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject) override;
	UE_API virtual void HandleVMCompiledEvent(UObject* InCompiledObject, URigVM* InVM, FRigVMExtendedExecuteContext& InContext);
	UE_API virtual void HandleVMExecutedEvent(URigVMHost* InHost, const FName& InEventName);
	
	// FNotifyHook Interface
	UE_API void NotifyPreChange(FProperty* PropertyAboutToChange);
	UE_API void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged);
	/** delegate for changing property */
	UE_API virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API void OnPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent);
	UE_API virtual void OnWrappedPropertyChangedChainEvent(URigVMDetailsViewWrapperObject* InWrapperObject, const FString& InPropertyPath, FPropertyChangedChainEvent& InPropertyChangedChainEvent);
	UE_API void OnRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier& InFunction, URigVMController* InTargetController, IRigVMGraphFunctionHost* InTargetFunctionHost, bool bForce);
	UE_API FRigVMController_BulkEditResult OnRequestBulkEditDialog(FRigVMEditorAssetInterfacePtr InBlueprint, URigVMController* InController, URigVMLibraryNode* InFunction, ERigVMControllerBulkEditType InEditType);
	UE_API bool OnRequestBreakLinksDialog(TArray<URigVMLink*> InLinks);
	UE_API TRigVMTypeIndex OnRequestPinTypeSelectionDialog(const TArray<TRigVMTypeIndex>& InTypes);

	UE_API bool UpdateDefaultValueForVariable(FRigVMGraphVariableDescription& InVariable, bool bUseDebuggedObject);

	URigVMController* ActiveController;

	/** Push a newly compiled/opened host to the editor */
	UE_API virtual void UpdateRigVMHost();
	virtual void UpdateRigVMHost_PreClearOldHost(URigVMHost* InPreviousHost) {};

	/** Update the name lists for use in name combo boxes */
	UE_API virtual void CacheNameLists();

	// FGCObject Interface
	UE_API void AddReferencedObjects( FReferenceCollector& Collector );

	UE_API virtual void BindCommands();
	UE_API virtual void UnbindCommands();

	UE_API void ToggleAutoCompileGraph();
	UE_API bool IsAutoCompileGraphOn() const;
	bool CanAutoCompileGraph() const { return true; }
	UE_API void ToggleLockGraph();
	UE_API bool IsLockGraphOn() const;
	bool CanLockGraph() const;
	UE_API void ToggleEventQueue();
	UE_API void ToggleRunGraphOnce();
	UE_API void SetRunGraphOnce(bool InRunGraphOnceEnabled);
	UE_API virtual bool IsRunGraphOnceOn() const override;
	virtual bool CanRunGraphOnce() const override { return false; }
	virtual void OnRunGraphOnceToggled(bool bRunGraphOnceEnabled) {}
	UE_API TSharedRef<SWidget> GenerateEventQueueMenuContent();
	UE_API virtual FMenuBuilder GenerateBulkEditMenu();
	UE_API TSharedRef<SWidget> GenerateBulkEditMenuContent();
	UE_API virtual void GenerateEventQueueMenuContent(FMenuBuilder& MenuBuilder);

	/** Wraps the normal blueprint editor's action menu creation callback */
	UE_API FActionMenuContent HandleCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);
	UE_API void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);

	/** Undo Action**/
	UE_API void UndoAction();

	/** Redo Action **/
	UE_API void RedoAction();
	
	UE_API void OnCreateComment();

	bool IsDetailsPanelRefreshSuspended() const { return bSuspendDetailsPanelRefresh; }
	bool& GetSuspendDetailsPanelRefreshFlag() { return bSuspendDetailsPanelRefresh; }
	UE_API TArray<TWeakObjectPtr<UObject>> GetSelectedObjects() const;
	UE_API virtual void SetDetailObjects(const TArray<UObject*>& InObjects);
	UE_API virtual void SetDetailObjects(const TArray<UObject*>& InObjects, bool bChangeUISelectionState);
	UE_API virtual void SetDetailObjectFilter(TSharedPtr<FDetailsViewObjectFilter> InObjectFilter);
	UE_API void SetMemoryStorageDetails(const TArray<FRigVMMemoryStorageStruct*>& InStructs);
	UE_API void SetDetailViewForGraph(URigVMGraph* InGraph);
	UE_API void SetDetailViewForFocusedGraph();
	UE_API void SetDetailViewForLocalVariable();
	UE_API void SetDetailViewForVariable();
	UE_API virtual void RefreshDetailView();
	UE_API bool DetailViewShowsAnyRigUnit() const;
	UE_API bool DetailViewShowsLocalVariable() const;
	UE_API bool DetailViewShowsVariable() const;
	UE_API bool DetailViewShowsStruct(UScriptStruct* InStruct) const;
	UE_API void ClearDetailObject(bool bChangeUISelectionState = true);
	UE_API void ClearDetailsViewWrapperObjects();
	const TArray<TStrongObjectPtr<URigVMDetailsViewWrapperObject>>& GetWrapperObjects() const { return WrapperObjects; }

	UE_API void SetHost(URigVMHost* InHost);

	UE_API virtual URigVMGraph* GetFocusedModel() const override;
	UE_API URigVMController* GetFocusedController() const;
	UE_API TSharedPtr<SGraphEditor> GetGraphEditor(UEdGraph* InEdGraph) const;

	/** Extend menu */
	UE_API void ExtendMenu();

	/** Extend toolbar */
	UE_API void ExtendToolbar();
	
	/** Fill the toolbar with content */
	UE_API virtual void FillToolbar(FToolBarBuilder& ToolbarBuilder, bool bEndSection = true);
	UE_API virtual void AddLockEditingWidget(FToolBarBuilder& ToolbarBuilder);
	
	UE_API virtual TArray<FName> GetDefaultEventQueue() const;
	UE_API TArray<FName> GetEventQueue() const;
	UE_API void SetEventQueue(TArray<FName> InEventQueue);
	UE_API virtual void SetEventQueue(TArray<FName> InEventQueue, bool bCompile);
	virtual int32 GetEventQueueComboValue() const { return INDEX_NONE; }
	virtual FText GetEventQueueLabel() const { return FText(); }
	UE_API virtual FSlateIcon GetEventQueueIcon(const TArray<FName>& InEventQueue) const;
	UE_API FSlateIcon GetEventQueueIcon() const;

	UE_API virtual void GetDebugObjects(TArray<FRigVMCustomDebugObject>& DebugList) const override;
	virtual bool OnlyShowCustomDebugObjects() const override { return true; }
	UE_API void HandleSetObjectBeingDebugged(UObject* InObject);
	UE_API virtual FString GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const override;

	/** Update stale watch pins */
	UE_API void UpdateStaleWatchedPins();
	
	UE_API virtual void HandleRefreshEditorFromBlueprint(FRigVMEditorAssetInterfacePtr InBlueprint);
	UE_API void HandleVariableDroppedFromBlueprint(UObject* InSubject, FProperty* InVariableToDrop, const FVector2D& InDropPosition, const FVector2D& InScreenPosition);
	UE_API void OnGraphNodeClicked(URigVMEdGraphNode* InNode, const FGeometry& InNodeGeometry, const FPointerEvent& InMouseEvent);
	UE_API void OnNodeDoubleClicked(FRigVMEditorAssetInterfacePtr InBlueprint, URigVMNode* InNode);
	UE_API void OnGraphImported(UEdGraph* InEdGraph);
	UE_API bool OnActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const;
	UE_API void FrameSelection();
	UE_API void SwapFunctionWithinAsset();
	UE_API void SwapFunctionAcrossProject();
	UE_API void SwapFunctionForAssets(const TArray<FAssetData>& InAssets, bool bSetupUndo);
	UE_API void SwapAssetReferences();

	/** Enables or disables heat map profiling for the graph */
	UE_API void ToggleProfiling();

	UE_API void OnOpenSelectedNodesInNewTab();
	UE_API bool CanOpenSelectedNodesInNewTab() const;

	/** Once the log is collected update the graph */
	UE_API void UpdateGraphCompilerErrors();

	/** Returns true if PIE is currently running */
	static UE_API bool IsPIERunning();

	UE_API void OnPIEStopped(bool bSimulation);

	UE_API virtual void ToggleFadeOutUnrelateNodes();
	UE_API virtual bool IsToggleFadeOutUnrelatedNodesChecked() const;

	UE_API virtual void AddRunOnceWidget(FToolBarBuilder& ToolbarBuilder) override;

public:
	UE_API void OnGraphEditorTick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime,
		TSharedRef<SGraphEditor> InGraphEditor, URigVMEdGraph* InGraph);

protected:
	/** Our currently running rig vm instance */
	//TObjectPtr<URigVMHost> Host;

	FPreviewHostUpdated PreviewHostUpdated;

	TSharedPtr<SRigVMEditorGraphExplorer> GraphExplorerWidget;

	FRigVMEditorClosed RigVMEditorClosedDelegate;
	FRigVMEditorKeyDownDelegate OnKeyDownDelegate;

	UE_API virtual void SetGraphExplorerWidget(TSharedPtr<SRigVMEditorGraphExplorer> InWidget) override;
	virtual TSharedPtr<SRigVMEditorGraphExplorer> GetGraphExplorerWidget() override { return GraphExplorerWidget; }

	UE_API bool IsEditingSingleBlueprint() const;
	
protected:
	bool bAnyErrorsLeft;
	TMap<FString, FString> KnownInstructionLimitWarnings;
	FString LastDebuggedHost;

	bool bSuspendDetailsPanelRefresh;
	bool bDetailsPanelRequiresClear;
	bool bAllowBulkEdits;
	bool bIsSettingObjectBeingDebugged;

	bool bRigVMEditorInitialized;

	/** Are we currently compiling through the user interface */
	bool bIsCompilingThroughUI;

	TArray<TStrongObjectPtr<URigVMDetailsViewWrapperObject>> WrapperObjects;

	/** The log to use for errors resulting from the init phase of the units */
	FRigVMLog RigVMLog;
	
	TArray<FName> LastEventQueue;

	/** The extender to pass to the level editor to extend it's window menu */
	TSharedPtr<FExtender> MenuExtender;

	/** Toolbar extender */
	TSharedPtr<FExtender> ToolbarExtender;

	FDelegateHandle PropertyChangedHandle;

	FOnRefreshEvent OnRefreshEvent;

	bool bRunGraphOnce;

	friend class SRigVMExecutionStackView;
	friend class SRigVMEditorGraphExplorer;

private:
	/** Called when the editor module requests finding nodes */
	void OnRequestFindNodeReferences(const UE::RigVMEditor::FRigVMEditorFindNodeReferencesParams Params);
};

struct FRigVMEditorZoomLevelsContainer : public FZoomLevelsContainer
{
	struct FRigVMEditorZoomLevelEntry
	{
	public:
		FRigVMEditorZoomLevelEntry(float InZoomAmount, const FText& InDisplayText, EGraphRenderingLOD::Type InLOD)
			: DisplayText(FText::Format(NSLOCTEXT("GraphEditor", "Zoom", "Zoom {0}"), InDisplayText))
		, ZoomAmount(InZoomAmount)
		, LOD(InLOD)
		{
		}

	public:
		FText DisplayText;
		float ZoomAmount;
		EGraphRenderingLOD::Type LOD;
	};
	
	FRigVMEditorZoomLevelsContainer()
	{
		ZoomLevels.Reserve(22);
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.025f, FText::FromString(TEXT("-14")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.070f, FText::FromString(TEXT("-13")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.100f, FText::FromString(TEXT("-12")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.125f, FText::FromString(TEXT("-11")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.150f, FText::FromString(TEXT("-10")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.175f, FText::FromString(TEXT("-9")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.200f, FText::FromString(TEXT("-8")), EGraphRenderingLOD::LowestDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.225f, FText::FromString(TEXT("-7")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.250f, FText::FromString(TEXT("-6")), EGraphRenderingLOD::LowDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.375f, FText::FromString(TEXT("-5")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.500f, FText::FromString(TEXT("-4")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.675f, FText::FromString(TEXT("-3")), EGraphRenderingLOD::MediumDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.750f, FText::FromString(TEXT("-2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(0.875f, FText::FromString(TEXT("-1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.000f, FText::FromString(TEXT("1:1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.250f, FText::FromString(TEXT("+1")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.375f, FText::FromString(TEXT("+2")), EGraphRenderingLOD::DefaultDetail));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.500f, FText::FromString(TEXT("+3")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.675f, FText::FromString(TEXT("+4")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.750f, FText::FromString(TEXT("+5")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(1.875f, FText::FromString(TEXT("+6")), EGraphRenderingLOD::FullyZoomedIn));
		ZoomLevels.Add(FRigVMEditorZoomLevelEntry(2.000f, FText::FromString(TEXT("+7")), EGraphRenderingLOD::FullyZoomedIn));
	}

	float GetZoomAmount(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].ZoomAmount;
	}

	int32 GetNearestZoomLevel(float InZoomAmount) const override
	{
		for (int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
		{
			if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}
	
	FText GetZoomText(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].DisplayText;
	}
	
	int32 GetNumZoomLevels() const override
	{
		return ZoomLevels.Num();
	}
	
	int32 GetDefaultZoomLevel() const override
	{
		return 14;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		checkSlow(ZoomLevels.IsValidIndex(InZoomLevel));
		return ZoomLevels[InZoomLevel].LOD;
	}

	TArray<FRigVMEditorZoomLevelEntry> ZoomLevels;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
