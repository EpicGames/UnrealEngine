// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditor;

struct FRigVMEditorGraphExplorerTabSummoner : public FWorkflowTabFactory
{
public:

	static const FName TabID() { static FName ID = TEXT("RigVM Graph Explorer"); return ID; }
	
public:
	UE_API FRigVMEditorGraphExplorerTabSummoner(const TSharedRef<IRigVMEditor>& InRigVMEditor);
	
	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<IRigVMEditor> RigVMEditor;
};

#undef UE_API
