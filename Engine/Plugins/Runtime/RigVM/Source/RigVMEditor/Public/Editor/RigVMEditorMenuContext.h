// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMEditorAsset.h"
#include "EdGraph/EdGraph.h"

#include "RigVMEditorMenuContext.generated.h"

#define UE_API RIGVMEDITOR_API

class URigVMHost;
class IRigVMEditor;
class URigVMGraph;
class URigVMNode;
class URigVMPin;

USTRUCT(BlueprintType)
struct FRigVMEditorGraphMenuContext
{
	GENERATED_BODY()

	FRigVMEditorGraphMenuContext()
		: Graph(nullptr)
		, Node(nullptr)
		, Pin(nullptr)
	{
	}
	
	FRigVMEditorGraphMenuContext(TObjectPtr<const URigVMGraph> InGraph, TObjectPtr<const URigVMNode> InNode, TObjectPtr<const URigVMPin> InPin)
		: Graph(InGraph)
		, Node(InNode)
		, Pin(InPin)
	{
	}
	
	/** The graph associated with this context. */
	UPROPERTY(BlueprintReadOnly, Category = RigVMEditor)
	TObjectPtr<const URigVMGraph> Graph;

	/** The node associated with this context. */
	UPROPERTY(BlueprintReadOnly, Category = RigVMEditor)
	TObjectPtr<const URigVMNode> Node;

	/** The pin associated with this context; may be NULL when over a node. */
	UPROPERTY(BlueprintReadOnly, Category = RigVMEditor)
	TObjectPtr<const URigVMPin> Pin;
};

UCLASS(MinimalAPI, BlueprintType)
class URigVMEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/**
	 *	Initialize the Context
	 * @param InRigVMEditor 	    The RigVM Editor hosting the menus
	 * @param InGraphMenuContext 	Additional context for specific menus
	*/
	UE_API void Init(TWeakPtr<IRigVMEditor> InRigVMEditor, const FRigVMEditorGraphMenuContext& InGraphMenuContext = FRigVMEditorGraphMenuContext());

	/** Get the rigvm blueprint that we are editing */
	UFUNCTION(BlueprintCallable, Category = RigVMEditor, meta=(DeprecatedFunction, DeprecationMessage="GetRigVMBlueprint is deprecated. Please use GetRigVMAssetInterface instead." ))
    UE_API URigVMBlueprint* GetRigVMBlueprint() const;

	UFUNCTION(BlueprintCallable, Category = RigVMEditor)
	UE_API TScriptInterface<IRigVMEditorAssetInterface> GetRigVMAssetInterface() const;
	
	/** Get the active rigvm host instance in the viewport */
	UFUNCTION(BlueprintCallable, Category = RigVMEditor)
    UE_API URigVMHost* GetRigVMHost() const;

	/** Returns true if either alt key is down */
	UFUNCTION(BlueprintCallable, Category = RigVMEditor)
	UE_API bool IsAltDown() const;
	
	/** Returns context for graph node context menu */
	UFUNCTION(BlueprintCallable, Category = RigVMEditor)
	UE_API FRigVMEditorGraphMenuContext GetGraphMenuContext();
	
	UE_API IRigVMEditor* GetRigVMEditor() const;

private:
	TWeakPtr<IRigVMEditor> WeakRigVMEditor;

	FRigVMEditorGraphMenuContext GraphMenuContext;
};

#undef UE_API
