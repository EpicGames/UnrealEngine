// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Brushes/SlateRoundedBoxBrush.h"

class FRigVMEditorStyle final
	: public FSlateStyleSet
{
	class FContentRootBracket
	{
	public:
		FContentRootBracket(FRigVMEditorStyle* InStyle, const FString& NewContentRoot)
			: Style(InStyle)
			, PreviousContentRoot(InStyle->GetContentRootDir())
		{
			Style->SetContentRoot(NewContentRoot);
		}

		~FContentRootBracket()
		{
			Style->SetContentRoot(PreviousContentRoot);
		}
	private:
		FRigVMEditorStyle* Style;
		FString PreviousContentRoot;
	};
	
	FRigVMEditorStyle()
		: FSlateStyleSet("RigVMEditorStyle")
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon14x14(14.0f, 14.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FString RigVMPluginContentDir = FPaths::EnginePluginsDir() / TEXT("Runtime/RigVM/Content");
		const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
		SetContentRoot(RigVMPluginContentDir);
		SetCoreContentRoot(EngineEditorSlateDir);

		const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

		// Edit mode styles
		Set("RigVMEditMode", new IMAGE_BRUSH_SVG("Slate/RigVM_Release", Icon40x40));
		Set("RigVMEditMode.Small", new IMAGE_BRUSH_SVG("Slate/RigVM_Release", Icon20x20));

		// RigVM Editor styles
		{
			// tab icons
			Set("RigVM.TabIcon", new IMAGE_BRUSH("Slate/RigVM_Tab_16x", Icon16x16));
			Set("ExecutionStack.TabIcon", new IMAGE_BRUSH_SVG("Slate/RigVM_ExecutionStack", Icon16x16));
			
			// icons for control units
			Set("RigVM.Unit", new IMAGE_BRUSH("Slate/RigVM_Unit_16x", Icon16x16));
			Set("RigVM.Template", new IMAGE_BRUSH("Slate/RigVM_Template_16x", Icon16x16));

			Set("RigVM.AutoCompileGraph", new IMAGE_BRUSH("Slate/RigVM_AutoCompile", Icon20x20));
			Set("RigVM.AutoCompileGraph.Small", new IMAGE_BRUSH("Slate/RigVM_AutoCompile", Icon20x20));

			Set("RigVM.Bug.Dot", new IMAGE_BRUSH("Slate/RigVM_BugDot_32x", Icon16x16));
			Set("RigVM.Bug.Normal", new IMAGE_BRUSH("Slate/RigVM_Bug_28x", Icon14x14));
			Set("RigVM.Bug.Open", new IMAGE_BRUSH("Slate/RigVM_BugOpen_28x", Icon14x14));
			Set("RigVM.Bug.Solid", new IMAGE_BRUSH("Slate/RigVM_BugSolid_28x", Icon14x14));

			Set("RigVM.ResumeExecution", new IMAGE_BRUSH_SVG("Slate/RigVM_Resume", Icon40x40));
			Set("RigVM.ReleaseMode", new IMAGE_BRUSH_SVG("Slate/RigVM_Release", Icon40x40));
			Set("RigVM.DebugMode", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Debug", Icon40x40));
		}

		// Graph styles
		{
			Set("RigVM.Node.PinTree.Arrow_Collapsed_Left", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Collapsed_Left", Icon10x10, DefaultForeground));
			Set("RigVM.Node.PinTree.Arrow_Collapsed_Hovered_Left", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Collapsed_Hovered_Left", Icon10x10, DefaultForeground));

			Set("RigVM.Node.PinTree.Arrow_Expanded_Left", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Expanded_Left", Icon10x10, DefaultForeground));
			Set("RigVM.Node.PinTree.Arrow_Expanded_Hovered_Left", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Expanded_Hovered_Left", Icon10x10, DefaultForeground));

			Set("RigVM.Node.PinTree.Arrow_Collapsed_Right", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Collapsed_Right", Icon10x10, DefaultForeground));
			Set("RigVM.Node.PinTree.Arrow_Collapsed_Hovered_Right", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Collapsed_Hovered_Right", Icon10x10, DefaultForeground));

			Set("RigVM.Node.PinTree.Arrow_Expanded_Right", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Expanded_Right", Icon10x10, DefaultForeground));
			Set("RigVM.Node.PinTree.Arrow_Expanded_Hovered_Right", new IMAGE_BRUSH("Slate/RigVM_TreeArrow_Expanded_Hovered_Right", Icon10x10, DefaultForeground));
		}

		// Button styles
		{
			FButtonStyle TagButton = FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

			// disable padding on the tag button
			TagButton
			.SetNormalPadding(0)
			.SetPressedPadding(0);
			
			Set( "TagButton", TagButton );
		}

		// Tag Capsule styles
		{
			Set("RigVM.TagCapsule", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.0f));
			Set("RigVM.TagCapsuleDark", new FSlateRoundedBoxBrush(FStyleColors::Header, 3.0f));
		}
	}
public:

	static FRigVMEditorStyle& Get()
	{
		static FRigVMEditorStyle Inst;
		return Inst;
	}

	static void Register()
	{
		FSlateStyleRegistry::RegisterSlateStyle(Get());
	}

	static void Unregister()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(Get());
	}
};
