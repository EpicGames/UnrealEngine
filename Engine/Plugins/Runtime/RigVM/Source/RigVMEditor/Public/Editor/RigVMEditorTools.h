// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "FrontendFilterBase.h"
#include "RigVMCore/RigVMVariant.h"

#define UE_API RIGVMEDITOR_API

class URigVMController;
class URigVMGraph;
class URigVMFunctionLibrary;
class IRigVMGraphFunctionHost;
struct FRigVMGraphFunctionIdentifier;

namespace UE::RigVM::Editor::Tools
{

RIGVMEDITOR_API bool PasteNodes(const FVector2D& PasteLocation
	, const FString& TextToImport
	, URigVMController* InFocusedController
	, URigVMGraph* InFocusedModel
	, URigVMFunctionLibrary* InLocalFunctionLibrary
	, IRigVMGraphFunctionHost* InGraphFunctionHost
	, bool bSetupUndoRedo = true
	, bool bPrintPythonCommands = false);

RIGVMEDITOR_API TArray<FName> ImportNodesFromText(const FVector2D& PasteLocation
	, const FString& TextToImport
	, URigVMController* InFocusedController
	, URigVMGraph* InFocusedModel
	, URigVMFunctionLibrary* InLocalFunctionLibrary
	, IRigVMGraphFunctionHost* InGraphFunctionHost
	, bool bSetupUndoRedo = true
	, bool bPrintPythonCommands = false);

RIGVMEDITOR_API void OnRequestLocalizeFunctionDialog(FRigVMGraphFunctionIdentifier& InFunction
	, URigVMController* InTargetController
	, IRigVMGraphFunctionHost* InTargetFunctionHost
	, bool bForce);
	
RIGVMEDITOR_API FAssetData FindAssetFromAnyPath(const FString& InPartialOrFullPath, bool bConvertToRootPath);

class FFilterByAssetTag : public FFrontendFilter
{
public:
	UE_API FFilterByAssetTag(TSharedPtr<FFrontendFilterCategory> InCategory, const FRigVMTag& InTag);

	FString GetName() const override { return Tag.bMarksSubjectAsInvalid ? TEXT("Exclude ") + Tag.Name.ToString() : Tag.Name.ToString(); }
	FText GetDisplayName() const override { return FText::FromString(Tag.bMarksSubjectAsInvalid ? TEXT("Exclude ") + Tag.GetLabel() : Tag.GetLabel()); }
	FText GetToolTipText() const override { return Tag.ToolTip; }
	FLinearColor GetColor() const override { return Tag.Color; };

	UE_API bool PassesFilter(const FContentBrowserItem& InItem) const override;
	bool ShouldBeMarkedAsInvalid() const { return Tag.bMarksSubjectAsInvalid; }

private:
	FRigVMTag Tag;
};

} // end namespace UE::RigVM::Editor::Tools

#undef UE_API
