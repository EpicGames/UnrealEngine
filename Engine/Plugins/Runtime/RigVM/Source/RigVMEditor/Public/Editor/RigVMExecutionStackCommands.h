// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class RIGVMEDITOR_API FRigVMExecutionStackCommands : public TCommands<FRigVMExecutionStackCommands>
{
public:
	FRigVMExecutionStackCommands() : TCommands<FRigVMExecutionStackCommands>
	(
		"RigVMExecutionStack",
		NSLOCTEXT("Contexts", "RigStack", "Execution Stack"),
		NAME_None, // "MainFrame" // @todo Fix this crash
		"RigVMEditorStyle" // Icon Style Set
	)
	{}
	
	/** Focuses on a selected operator's node */
	TSharedPtr< FUICommandInfo > FocusOnSelection;

	/** Looks for a specific instruction by index and brings it into focus */
	TSharedPtr< FUICommandInfo > GoToInstruction;

	/** Selects the target instruction(s) of an instruction */
	TSharedPtr< FUICommandInfo > SelectTargetInstructions;

	/** Sets or unsets the preview here early exit instruction */
	TSharedPtr< FUICommandInfo > ToggleEarlyExitInstruction;

	/** Steps to the next early exist instruction */
	TSharedPtr< FUICommandInfo > StepEarlyExitInstruction;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
