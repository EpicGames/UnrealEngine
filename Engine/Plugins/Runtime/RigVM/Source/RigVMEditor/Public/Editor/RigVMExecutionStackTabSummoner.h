// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditor;

struct FRigVMExecutionStackTabSummoner : public FWorkflowTabFactory
{
public:

	static inline const FLazyName TabID = FLazyName(TEXT("Execution Stack"));
	
public:
	UE_API FRigVMExecutionStackTabSummoner(const TSharedRef<IRigVMEditor>& InRigVMEditor);
	
	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
protected:
	TWeakPtr<IRigVMEditor> RigVMEditor;
};

#undef UE_API
