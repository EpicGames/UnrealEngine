// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMEditorAsset.h"
#include "Input/Reply.h"
#include "Widgets/Views/STreeView.h"
#include "EdGraph/EdGraphPin.h"
#include "Textures/SlateIcon.h"
#include "EdGraph/EdGraphSchema.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditorAssetInterface;
class URigVMBlueprint;
class SWidget;
class SDockTab;
class UBlueprint;
class FUICommandList;


class FRigVMFindResult : public TSharedFromThis< FRigVMFindResult >
{
public:
	FRigVMFindResult() = default;
	virtual ~FRigVMFindResult() = default;

	/* Create a root */
	UE_DEPRECATED(5.7, "Please use FRigVMFindResult(FRigVMEditorAssetInterfacePtr InBlueprint)")
	UE_API explicit FRigVMFindResult(TWeakObjectPtr<URigVMBlueprint> InBlueprint);
	UE_DEPRECATED(5.7, "Please use FRigVMFindResult(FRigVMEditorAssetInterfacePtr InBlueprint, const FText& InDisplayText)")
	UE_API explicit FRigVMFindResult(TWeakObjectPtr<URigVMBlueprint> InBlueprint, const FText& InDisplayText);
	UE_API explicit FRigVMFindResult(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint);
	UE_API explicit FRigVMFindResult(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint, const FText& InDisplayText);

	/* Called when user clicks on the search item */
	UE_API virtual FReply OnClick();

	/* Get Category for this search result */
	UE_API virtual FText GetCategory() const;

	/* Create an icon to represent the result */
	UE_API virtual TSharedRef<SWidget>	CreateIcon() const;

	/** Finalizes any content for the search data that was unsafe to do on a separate thread */
	virtual void FinalizeSearchData() {};

	/** gets the blueprint housing all these search results */
	UE_DEPRECATED(5.7, "Please use FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface()")
	UE_API URigVMBlueprint* GetBlueprint() const;
	UE_API FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface();
	UE_API const FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const;

	/**
	* Parses search info for specific data important for displaying the search result in an easy to understand format
	*
	* @param	InKey			This is the tag for the data, describing what it is so special handling can occur if needed
	* @param	InValue			Compared against search query to see if it passes the filter, sometimes data is rejected because it is deemed unsearchable
	*/
	virtual void ParseSearchInfo(FText InKey, FText InValue) {};

	/** Returns the Object represented by this search information give the Blueprint it can be found in */
	UE_API virtual UObject* GetObject(FRigVMEditorAssetInterfacePtr InBlueprint) const;

	/** Returns the display string for the row */
	UE_API FText GetDisplayString() const;

public:
	/*Any children listed under this category */
	TArray< TSharedPtr<FRigVMFindResult> > Children;

	/*If it exists it is the blueprint*/
	TWeakPtr<FRigVMFindResult> Parent;

	/*If it exists it is the blueprint*/
	TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakBlueprint;

	/*The display text for this item */
	FText DisplayText;
};

typedef TSharedPtr<FRigVMFindResult> FRigVMFindResultPtr;
typedef STreeView<FRigVMFindResultPtr>  SRigVMFindResultTreeViewType;

/** Some utility functions to help with Find-in-Blueprint functionality */
namespace RigVMFindReferencesHelpers
{
	/**
	 * Retrieves the pin type as a string value
	 *
	 * @param InPinType		The pin type to look at
	 *
	 * @return				The pin type as a string in format [category]'[sub-category object]'
	 */
	FString GetPinTypeAsString(const FEdGraphPinType& InPinType);

	/**
	 * Parses a pin type from passed in key names and values
	 *
	 * @param InKey					The key name for what the data should be translated as
	 * @param InValue				Value to be be translated
	 * @param InOutPinType			Modifies the PinType based on the passed parameters, building it up over multiple calls
	 * @return						TRUE when the parsing is successful
	 */
	bool ParsePinType(FText InKey, FText InValue, FEdGraphPinType& InOutPinType);

	/**
	* Iterates through all the given tree node's children and tells the tree view to expand them
	*/
	void ExpandAllChildren(FRigVMFindResultPtr InTreeNode, TSharedPtr<STreeView<FRigVMFindResultPtr>> InTreeView);
}

/** Graph nodes use this class to store their data */
class FRigVMFindReferencesGraphNode : public FRigVMFindResult
{
public:
	UE_DEPRECATED(5.7, "Please use FRigVMFindReferencesGraphNode(FRigVMEditorAssetInterfacePtr InBlueprint)")
	FRigVMFindReferencesGraphNode(TWeakObjectPtr<URigVMBlueprint> InBlueprint);
	FRigVMFindReferencesGraphNode(FRigVMEditorAssetInterfacePtr InBlueprint);
	virtual ~FRigVMFindReferencesGraphNode() = default;

	/** FRigVMFindResult Interface */
	virtual FReply OnClick() override;
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual void FinalizeSearchData() override;
	virtual UObject* GetObject(FRigVMEditorAssetInterfacePtr InBlueprint) const override;
	virtual FText GetCategory() const override;
	/** End FRigVMFindResult Interface */

private:
	/** The Node Guid to find when jumping to the node */
	FGuid NodeGuid;

	/** The glyph brush for this node */
	FSlateIcon Glyph;

	/** The glyph color for this node */
	FLinearColor GlyphColor;

	/*The class this item refers to */
	UClass* Class;

	/*The class name this item refers to */
	FString ClassName;
};

/** Pins use this class to store their data */
class FRigVMFindReferencesPin : public FRigVMFindResult
{
public:
	UE_DEPRECATED(5.7, "Plase use FRigVMFindReferencesPin(FRigVMEditorAssetInterfacePtr InBlueprint, FString InSchemaName)")
	FRigVMFindReferencesPin(TWeakObjectPtr<URigVMBlueprint> InBlueprint, FString InSchemaName);
	FRigVMFindReferencesPin(FRigVMEditorAssetInterfacePtr InBlueprint, FString InSchemaName);
	virtual ~FRigVMFindReferencesPin() = default;

	/** FRigVMFindResult Interface */
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	virtual void FinalizeSearchData() override;
	/** End FRigVMFindResult Interface */

private:
	/** The name of the schema this pin exists under */
	FString SchemaName;

	/** The pin that this search result refers to */
	FEdGraphPinType PinType;

	/** Pin's icon color */
	FSlateColor IconColor;
};

/** Property data is stored here */
class FRigVMFindReferencesVariable : public FRigVMFindResult
{
public:
	UE_DEPRECATED(5.7, "Plase use FRigVMFindReferencesVariable(FRigVMEditorAssetInterfacePtr InBlueprint)")
	FRigVMFindReferencesVariable(TWeakObjectPtr<URigVMBlueprint> InBlueprint);
	FRigVMFindReferencesVariable(FRigVMEditorAssetInterfacePtr InBlueprint);
	virtual ~FRigVMFindReferencesVariable() = default;

	/** FRigVMFindResult Interface */
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	virtual void FinalizeSearchData() override;
	/** End FRigVMFindResult Interface */

private:
	/** The pin that this search result refers to */
	FEdGraphPinType PinType;

	/** The default value of a property as a string */
	FString DefaultValue;
};

/** Graphs, such as functions and macros, are stored here */
class FRigVMFindReferencesGraph : public FRigVMFindResult
{
public:
	UE_DEPRECATED(5.7, "Please use FRigVMFindReferencesGraph(FRigVMEditorAssetInterfacePtr InBlueprint, EGraphType InGraphType)")
	FRigVMFindReferencesGraph(TWeakObjectPtr<URigVMBlueprint> InBlueprint, EGraphType InGraphType);
	FRigVMFindReferencesGraph(FRigVMEditorAssetInterfacePtr InBlueprint, EGraphType InGraphType);
	virtual ~FRigVMFindReferencesGraph() = default;

	/** FRigVMFindResult Interface */
	virtual FReply OnClick() override;
	virtual TSharedRef<SWidget>	CreateIcon() const override;
	virtual void ParseSearchInfo(FText InKey, FText InValue) override;
	virtual FText GetCategory() const override;
	/** End FRigVMFindResult Interface */

private:
	/** The type of graph this represents */
	EGraphType GraphType;
};

/*Widget for searching for (functions/events) across all blueprints or just a single blueprint */
class SRigVMFindReferences: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRigVMFindReferences )
	{}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<class FRigVMEditorBase> InBlueprintEditor = nullptr);

	/** Focuses this widget's search box, and changes the mode as well, and optionally the search terms */
	UE_API void FocusForUse(bool bSetFindWithinBlueprint, FString NewSearchTerms = FString(), bool bSelectFirstResult = false);

	/** SWidget overrides */
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	
	/** The main function that will find references and build the tree */
	UE_API void FindReferences(const FString& SearchTerms);

	/** Register any Find-in-Blueprint commands */
	UE_API void RegisterCommands();

	/*Called when user changes the text they are searching for */
	UE_API void OnSearchTextChanged(const FText& Text);

	/*Called when user changes commits text to the search box */
	UE_API void OnSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	/* Get the children of a row */
	UE_API void OnGetChildren( FRigVMFindResultPtr InItem, TArray< FRigVMFindResultPtr >& OutChildren );

	/* Called when user double clicks on a new result */
	UE_API void OnTreeSelectionDoubleClicked( FRigVMFindResultPtr Item );

	/* Called when a new row is being generated */
	UE_API TSharedRef<ITableRow> OnGenerateRow(FRigVMFindResultPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Callback to build the context menu when right clicking in the tree */
	UE_API TSharedPtr<SWidget> OnContextMenuOpening();

	/** Helper function to select all items */
	UE_API void SelectAllItemsHelper(FRigVMFindResultPtr InItemToSelect);

	/** Callback when user attempts to select all items in the search results */
	UE_API void OnSelectAllAction();

	/** Callback when user attempts to copy their selection in the Find-in-Blueprints */
	UE_API void OnCopyAction();

private:
	/** Pointer back to the blueprint editor that owns us */
	TWeakPtr<class FRigVMEditorBase> EditorPtr;
	
	/* The tree view displays the results */
	TSharedPtr<SRigVMFindResultTreeViewType> TreeView;

	/** The search text box */
	TSharedPtr<class SSearchBox> SearchTextField;
	
	/* This buffer stores the currently displayed results */
	TArray<FRigVMFindResultPtr> ItemsFound;

	/* Map relationship between element hash and its result  */ 
	TMap<uint32, FRigVMFindResultPtr> ElementHashToResult;

	/* The string to highlight in the results */
	FText HighlightText;

	/* The string to search for */
	FString	SearchValue;

	/** Commands handled by this widget */
	TSharedPtr< FUICommandList > CommandList;
};

#undef UE_API
