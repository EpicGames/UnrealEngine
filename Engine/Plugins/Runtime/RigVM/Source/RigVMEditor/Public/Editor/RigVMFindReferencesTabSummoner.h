// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define UE_API RIGVMEDITOR_API

#define LOCTEXT_NAMESPACE "RigVMEditor"

//////////////////////////////////////////////////////////////////////////
// FCompilerResultsSummoner

struct FRigVMFindReferencesTabSummoner : public FWorkflowTabFactory
{

public:

	static const FName TabID() { static FName ID = TEXT("RigVM Find References"); return ID; }
	
	UE_API FRigVMFindReferencesTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp);

	UE_API virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("FindResultsTooltip", "The find results tab shows results of searching this Blueprint (or all Blueprints).");
	}
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
