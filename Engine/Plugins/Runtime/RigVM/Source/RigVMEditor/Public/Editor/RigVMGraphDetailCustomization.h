// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IDetailCustomNodeBuilder.h"
#include "UObject/WeakObjectPtr.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMBlueprintLegacy.h"
#include "Editor/RigVMNewEditor.h"
#include "Editor/RigVMLegacyEditor.h"
#include "SGraphPin.h"
#include "Widgets/SRigVMGraphNode.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Editor/RigVMDetailsViewWrapperObject.h"
#include "IDetailPropertyExtensionHandler.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "SAdvancedTransformInputBox.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Widgets/SRigVMLogWidget.h"
#include "Widgets/SRigVMNodePreviewWidget.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailDragDropHandler.h"

#define UE_API RIGVMEDITOR_API

class IDetailLayoutBuilder;
class FRigVMGraphDetailCustomizationImpl;

class FRigVMFunctionArgumentReorderDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRigVMFunctionArgumentReorderDragDropOp, FDecoratedDragDropOp)

	FRigVMFunctionArgumentReorderDragDropOp(const TWeakObjectPtr<URigVMPin>& InPinPtr);

	/** Inits the tooltip, needs to be called after constructing */
	void Init();

	/** Update the drag tool tip indicating whether the current drop target is valid */
	void SetValidTarget(bool IsValidTarget);

	const URigVMPin* GetPin() const;

private:
	TWeakObjectPtr<URigVMPin> PinPtr;
};

class FRigVMFunctionArgumentReorderDragDropHandler : public IDetailDragDropHandler
{
public:
	FRigVMFunctionArgumentReorderDragDropHandler(
		const TWeakObjectPtr<URigVMPin>& InPinPtr,
		const TWeakObjectPtr<URigVMGraph>& InGraphPtr,
		const TWeakInterfacePtr<IRigVMClientHost>& InWeakRigVMClientHost
	)
		: PinPtr(InPinPtr)
		, GraphPtr(InGraphPtr)
		, WeakRigVMClientHost(InWeakRigVMClientHost)
	{
	}

	virtual ~FRigVMFunctionArgumentReorderDragDropHandler() override = default;

	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;
	virtual bool AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const override;

	const URigVMPin* GetPin() const;

private:
	TWeakObjectPtr<URigVMPin> PinPtr;
	TWeakObjectPtr<URigVMGraph> GraphPtr;
	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;
};

class FRigVMFunctionArgumentGroupLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FRigVMFunctionArgumentGroupLayout>
{
public:
	UE_API FRigVMFunctionArgumentGroupLayout(
		const TWeakObjectPtr<URigVMGraph>& InGraph,
		const TWeakInterfacePtr<IRigVMClientHost>& InRigVMClientHost,
		const TWeakPtr<IRigVMEditor>& InEditor,
		bool bInputs);
	UE_API virtual ~FRigVMFunctionArgumentGroupLayout();

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override { OnRebuildChildren = InOnRegenerateChildren; }
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:

	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

	TWeakObjectPtr<URigVMGraph> GraphPtr;
	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;
	TWeakPtr<IRigVMEditor> RigVMEditorPtr;
	bool bIsInputGroup;
	FSimpleDelegate OnRebuildChildren;
};

class FRigVMFunctionArgumentLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FRigVMFunctionArgumentLayout>
{
public:

	FRigVMFunctionArgumentLayout(
		const TWeakObjectPtr<URigVMPin>& InPin,
		const TWeakObjectPtr<URigVMGraph>& InGraph, 
		const TWeakInterfacePtr<IRigVMClientHost>& InRigVMClientHost,
		const TWeakPtr<IRigVMEditor>& InEditor)
		: PinPtr(InPin)
		, GraphPtr(InGraph)
		, WeakRigVMClientHost(InRigVMClientHost)
		, RigVMEditorPtr(InEditor)
		, NameValidator((const FRigVMEditorAssetInterfacePtr)nullptr, InGraph.Get(), InPin->GetFName()) // TODO check if missing Blueprint affects name validation
	{}

private:

	/** IDetailCustomNodeBuilder Interface*/
	UE_API virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return PinPtr.Get()->GetFName(); }
	virtual bool InitiallyCollapsed() const override { return true; }

private:

	typedef TTuple<EPinContainerType, bool, FText, FText, const FSlateBrush*> TContainerType;

	/** Determines if this pin should not be editable */
	UE_API bool ShouldPinBeReadOnly(bool bIsEditingPinType = false) const;

	/** Determines if editing the pins on the node should be read only */
	UE_API bool IsPinEditingReadOnly(bool bIsEditingPinType = false) const;

	/** Callbacks for all the functionality for modifying arguments */
	UE_API void OnRemoveClicked();

	UE_API int32 OnGetArgumentIndex() const;
	UE_API FText OnGetArgNameText() const;
	UE_API FText OnGetArgToolTipText() const;
	UE_API void OnArgNameTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit);

	UE_API FEdGraphPinType OnGetPinInfo() const;
	UE_API void PinInfoChanged(const FEdGraphPinType& PinType);
	UE_API void OnPrePinInfoChange(const FEdGraphPinType& PinType);

	UE_API const TArray<TSharedPtr<TContainerType>> GetContainerTypes() const;
	UE_API bool CanChangeContainerType() const;
	UE_API TSharedRef<SWidget> GetContainerTypeMenuContent();
	UE_API const FSlateBrush* GetContainerTypeIconImage() const;
	UE_API FSlateColor GetContainerTypeIconColor() const;
	UE_API void OnContainerTypeSelectionChanged(TSharedPtr<TContainerType> InContainerType);

private:

	/** The argument pin that this layout reflects */
	TWeakObjectPtr<URigVMPin> PinPtr;
	
	/** The target graph that this argument is on */
	TWeakObjectPtr<URigVMGraph> GraphPtr;

	/** The ed graph schema that this is using */
	mutable TWeakObjectPtr<URigVMEdGraphSchema> EdGraphSchemaPtr;

	/** The asset host we are editing */
	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;

	/** The editor we are editing */
	TWeakPtr<IRigVMEditor> RigVMEditorPtr;

	/** Holds a weak pointer to the argument name widget, used for error notifications */
	TWeakPtr<SEditableTextBox> ArgumentNameWidget;

	/** The validator to check if a name for an argument is valid */
	FRigVMLocalVariableNameValidator NameValidator;

	const FSlateBrush* CachedImg_Pin_Disconnected = nullptr;
	const FSlateBrush* CachedImg_RefPin_Disconnected = nullptr;
};

class FRigVMFunctionArgumentDefaultNode : public IDetailCustomNodeBuilder, public TSharedFromThis<FRigVMFunctionArgumentDefaultNode>
{
public:
	UE_API FRigVMFunctionArgumentDefaultNode(
		const TWeakObjectPtr<URigVMGraph>& InGraph,
		const TWeakInterfacePtr<IRigVMClientHost>& InClientHost
	);
	UE_API virtual ~FRigVMFunctionArgumentDefaultNode();

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override { OnRebuildChildren = InOnRegenerateChildren; }
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}
	UE_API virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:

	UE_API void OnGraphChanged(const FEdGraphEditAction& InAction);
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject, bool bFromTargetToSource);

	TWeakObjectPtr<URigVMGraph> GraphPtr;
	TWeakObjectPtr<URigVMEdGraph> EdGraphOuterPtr;
	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;
	FSimpleDelegate OnRebuildChildren;
	TSharedPtr<SRigVMNodePreviewWidget> PreviewWidget;
	FDelegateHandle GraphChangedDelegateHandle;
	TSharedPtr<FRigVMMinimalEnvironment> MinimalEnvironment;
	bool bIsHandlingModifiedEvent;
};

/** Customization for editing rig vm graphs */
class FRigVMGraphDetailCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<IRigVMEditor> InBlueprintEditor, const UClass* InExpectedBlueprintClass);
	UE_API FRigVMGraphDetailCustomization(TSharedPtr<IRigVMEditor> RigVMigEditor, FRigVMEditorAssetInterfacePtr RigVMBlueprint);
#if WITH_RIGVMLEGACYEDITOR
	static UE_API TSharedPtr<IDetailCustomization> MakeLegacyInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor, const UClass* InExpectedBlueprintClass);
	UE_API FRigVMGraphDetailCustomization(TSharedPtr<IBlueprintEditor> RigVMigEditor, FRigVMEditorAssetInterfacePtr RigVMBlueprint);
#endif


	// IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	/** The editor we are embedded in */
	TWeakPtr<IRigVMEditor> RigVMEditorPtr;

	/** The graph we are editing */
	TWeakObjectPtr<URigVMEdGraph> GraphPtr;

	/** The blueprint we are editing */
	TWeakInterfacePtr<IRigVMEditorAssetInterface> RigVMBlueprintPtr;

	TSharedPtr<FRigVMGraphDetailCustomizationImpl> RigVMGraphDetailCustomizationImpl;
};

class FRigVMGraphDetailCustomizationImpl : public TSharedFromThis<FRigVMGraphDetailCustomizationImpl>
{
public:
	UE_API void CustomizeDetails(IDetailLayoutBuilder& DetailLayout,
		URigVMGraph* Model,
		URigVMController* Controller,
		IRigVMClientHost* InRigVMClientHost,
		TWeakPtr<IRigVMEditor> InEditor);

private:

	UE_API bool IsAddNewInputOutputEnabled() const;
	UE_API EVisibility GetAddNewInputOutputVisibility() const;
	UE_API FReply OnAddNewInputClicked();
	UE_API FReply OnAddNewOutputClicked();
	UE_API FText GetNodeCategory() const;
	UE_API void SetNodeCategory(const FText& InNewText, ETextCommit::Type InCommitType);
	UE_API FText GetNodeKeywords() const;
	UE_API void SetNodeKeywords(const FText& InNewText, ETextCommit::Type InCommitType);
	UE_API FText GetNodeDescription() const;
	UE_API void SetNodeDescription(const FText& InNewText, ETextCommit::Type InCommitType);
	UE_API FLinearColor GetNodeColor() const;
	UE_API void SetNodeColor(FLinearColor InColor, bool bSetupUndoRedo);
	UE_API void OnNodeColorBegin();
	UE_API void OnNodeColorEnd();
	UE_API void OnNodeColorCancelled(FLinearColor OriginalColor);
	UE_API FReply OnNodeColorClicked();
	UE_API FText GetCurrentAccessSpecifierName() const;
	UE_API void OnAccessSpecifierSelected( TSharedPtr<FRigVMStringWithTag> SpecifierName, ESelectInfo::Type SelectInfo );
	UE_API TSharedRef<ITableRow> HandleGenerateRowAccessSpecifier( TSharedPtr<FRigVMStringWithTag> SpecifierName, const TSharedRef<STableViewBase>& OwnerTable );
	UE_API ECheckBoxState GetFunctionPureCheckState() const;
	UE_API void OnFunctionPureCheckStateChanged(ECheckBoxState NewState);
	UE_API bool IsValidFunction() const;
	UE_API FRigVMVariant GetVariant() const;
	UE_API FRigVMVariantRef GetSubjectVariantRef() const;
	UE_API TArray<FRigVMVariantRef> GetVariantRefs() const;

private:

	UE_API void OnVariantChanged(const FRigVMVariant& InVariant);
	UE_API void OnBrowseVariantRef(const FRigVMVariantRef& InVariantRef);
	UE_API TArray<FRigVMTag> OnGetAssignedTags() const;
	UE_API void OnAddAssignedTag(const FName& InTagName);
	UE_API void OnRemoveAssignedTag(const FName& InTagName);

	UE_API URigVMLibraryNode* GetLibraryNode() const;
	UE_API URigVMNode* GetNodeForLayout() const;
	UE_API const FRigVMNodeLayout* GetNodeLayout() const;
	UE_API TArray<FString> GetUncategorizedPins() const;
	UE_API TArray<FRigVMPinCategory> GetPinCategories() const;
	UE_API FString GetPinCategory(FString InPinPath) const;
	UE_API int32 GetPinIndexInCategory(FString InPinPath) const;
	UE_API FString GetPinLabel(FString InPinPath) const;
	UE_API FLinearColor GetPinColor(FString InPinPath) const;
	UE_API const FSlateBrush* GetPinIcon(FString InPinPath) const;
	UE_API void HandleCategoryAdded(FString InCategory);
	UE_API void HandleCategoryRemoved(FString InCategory);
	UE_API void HandleCategoryRenamed(FString InOldCategory, FString InNewCategory);
	UE_API void HandlePinCategoryChanged(FString InPinPath, FString InCategory);
	UE_API void HandlePinLabelChanged(FString InPinPath, FString InNewLabel);
	UE_API void HandlePinIndexInCategoryChanged(FString InPinPath, int32 InIndexInCategory);
	static UE_API bool ValidateName(FString InNewName, FText& OutErrorMessage);
	UE_API bool HandleValidateCategoryName(FString InCategoryPath, FString InNewName, FText& OutErrorMessage);
	UE_API bool HandleValidatePinDisplayName(FString InPinPath, FString InNewName, FText& OutErrorMessage);

	UE_API uint32 GetNodeLayoutHash() const;

	/** The graph we are editing */
	TWeakObjectPtr<URigVMGraph> WeakModel;

	/** The graph controller we are editing */
	TWeakObjectPtr<URigVMController> WeakController;

	/** The editor we are embedded in */
	TWeakPtr<IRigVMEditor> RigVMEditorPtr;

	/** The asset host we are editing */
	TWeakInterfacePtr<IRigVMClientHost> RigVMClientHost;

	/** The color block widget */
	TSharedPtr<SColorBlock> ColorBlock;

	/** Set to true if the UI is currently picking a color */
	bool bIsPickingColor;

	static UE_API TArray<TSharedPtr<FRigVMStringWithTag>> AccessSpecifierStrings;
	mutable TOptional<FRigVMNodeLayout> CachedNodeLayout;
};

/** Customization for editing a rig vm node */
class FRigVMWrappedNodeDetailCustomization : public IDetailCustomization
{
public:
	
	UE_API FRigVMWrappedNodeDetailCustomization();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	UE_API TSharedRef<SWidget> MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem);
	UE_API FText GetNameListText(const FNameProperty* InProperty) const;
	UE_API TSharedPtr<FRigVMStringWithTag> GetCurrentlySelectedItem(const FNameProperty* InProperty, const TArray<TSharedPtr<FRigVMStringWithTag>>* InNameList) const;
	UE_API void SetNameListText(const FText& NewTypeInValue, ETextCommit::Type, const FNameProperty* InProperty, TSharedRef<IPropertyUtilities> PropertyUtilities);
	UE_API void OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo, const FNameProperty* InProperty, TSharedRef<IPropertyUtilities> PropertyUtilities);
	UE_API void OnNameListComboBox(const FNameProperty* InProperty, const TArray<TSharedPtr<FRigVMStringWithTag>>* InNameList);
	UE_API void CustomizeLiveValues(IDetailLayoutBuilder& DetailLayout);

	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized;
	TArray<TWeakObjectPtr<URigVMDetailsViewWrapperObject>> ObjectsBeingCustomized;
	TArray<TWeakObjectPtr<URigVMNode>> NodesBeingCustomized;
	TMap<FName, TSharedPtr<SRigVMGraphPinNameListValueWidget>> NameListWidgets;
};

/** Customization for editing a rig vm integer control enum class */
class FRigVMGraphEnumDetailCustomization : public IPropertyTypeCustomization
{
public:

	UE_API FRigVMGraphEnumDetailCustomization();
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMGraphEnumDetailCustomization);
	}

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	TArray<uint8*> GetMemoryBeingCustomized()
	{
		TArray<uint8*> MemoryPtr;
		MemoryPtr.Reserve(ObjectsBeingCustomized.Num() + StructsBeingCustomized.Num());

		for(const TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
		{
			if(Object.IsValid())
			{
				MemoryPtr.Add((uint8*)Object.Get());
			}
		}

		for(const TSharedPtr<FStructOnScope>& StructPtr: StructsBeingCustomized)
		{
			if(StructPtr.IsValid())
			{
				MemoryPtr.Add(StructPtr->GetStructMemory());
			}
		}

		return MemoryPtr;
	}
	
	bool GetPropertyChain(TSharedRef<class IPropertyHandle> InPropertyHandle, FEditPropertyChain& OutPropertyChain, TArray<int32> &OutPropertyArrayIndices, bool& bOutEnabled)
	{
		if (!InPropertyHandle->IsValidHandle())
		{
			return false;
		}
		
		OutPropertyChain.Empty();
		OutPropertyArrayIndices.Reset();
		bOutEnabled = false;

		const bool bHasObject = !ObjectsBeingCustomized.IsEmpty() && ObjectsBeingCustomized[0].Get();
		const bool bHasStruct = !StructsBeingCustomized.IsEmpty() && StructsBeingCustomized[0].Get();
		
		if (bHasStruct || bHasObject)
		{
			TSharedPtr<class IPropertyHandle> ChainHandle = InPropertyHandle;
			while (ChainHandle.IsValid() && ChainHandle->GetProperty() != nullptr)
			{
				OutPropertyChain.AddHead(ChainHandle->GetProperty());
				OutPropertyArrayIndices.Insert(ChainHandle->GetIndexInArray(), 0);
				ChainHandle = ChainHandle->GetParentHandle();
			}

			if (OutPropertyChain.GetHead() != nullptr)
			{
				OutPropertyChain.SetActiveMemberPropertyNode(OutPropertyChain.GetTail()->GetValue());
				bOutEnabled = !OutPropertyChain.GetHead()->GetValue()->HasAnyPropertyFlags(CPF_EditConst);
				return true;
			}
		}
		return false;
	}

	// extracts the value for a nested property from an outer owner
	static UEnum** ContainerMemoryBlockToEnumPtr(uint8* InMemoryBlock, FEditPropertyChain& InPropertyChain, TArray<int32> &InPropertyArrayIndices)
	{
		if (InPropertyChain.GetHead() == nullptr)
		{
			return nullptr;
		}
		
		FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = InPropertyChain.GetHead();
		uint8* MemoryPtr = InMemoryBlock;
		int32 ChainIndex = 0;
		do
		{
			const FProperty* Property = PropertyNode->GetValue();
			MemoryPtr = Property->ContainerPtrToValuePtr<uint8>(MemoryPtr);

			PropertyNode = PropertyNode->GetNextNode();
			ChainIndex++;
			
			if(InPropertyArrayIndices.IsValidIndex(ChainIndex))
			{
				const int32 ArrayIndex = InPropertyArrayIndices[ChainIndex];
				if(ArrayIndex != INDEX_NONE)
				{
					const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property->GetOwnerProperty());
					check(ArrayProperty);
					
					FScriptArrayHelper ArrayHelper(ArrayProperty, MemoryPtr);
					if(!ArrayHelper.IsValidIndex(ArrayIndex))
					{
						return nullptr;
					}
					MemoryPtr = ArrayHelper.GetRawPtr(ArrayIndex);

					// skip to the next property node already
					PropertyNode = PropertyNode->GetNextNode();
					ChainIndex++;
				}
			}
		}
		while (PropertyNode);

		return (UEnum**)MemoryPtr;
	}

	UE_API void HandleControlEnumChanged(TSharedPtr<FString> InEnumPath, ESelectInfo::Type InSelectType, TSharedRef<IPropertyHandle> InPropertyHandle);

	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized;
	URigVMGraph* GraphBeingCustomized;
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	TArray<TSharedPtr<FStructOnScope>> StructsBeingCustomized;
};

/** Customization for editing a rig vm node */
class FRigVMGraphMathTypeDetailCustomization : public IPropertyTypeCustomization
{
public:

	UE_API FRigVMGraphMathTypeDetailCustomization();
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMGraphMathTypeDetailCustomization);
	}

	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:

	bool GetPropertyChain(TSharedRef<class IPropertyHandle> InPropertyHandle, FEditPropertyChain& OutPropertyChain, TArray<int32> &OutPropertyArrayIndices, bool& bOutEnabled)
	{
		if (!InPropertyHandle->IsValidHandle())
		{
			return false;
		}
		
		OutPropertyChain.Empty();
		OutPropertyArrayIndices.Reset();
		bOutEnabled = false;

		if (InPropertyHandle->GetNumPerObjectValues() > 0)
		{
			TSharedPtr<class IPropertyHandle> ChainHandle = InPropertyHandle;
			while (ChainHandle.IsValid() && ChainHandle->GetProperty() != nullptr)
			{
				OutPropertyChain.AddHead(ChainHandle->GetProperty());
				OutPropertyArrayIndices.Insert(ChainHandle->GetIndexInArray(), 0);
				ChainHandle = ChainHandle->GetParentHandle();					
			}

			if (OutPropertyChain.GetHead() != nullptr)
			{
				OutPropertyChain.SetActiveMemberPropertyNode(OutPropertyChain.GetTail()->GetValue());
				bOutEnabled = !OutPropertyChain.GetHead()->GetValue()->HasAnyPropertyFlags(CPF_EditConst);
				return true;
			}
		}
		return false;
	}

	// extracts the value for a nested property (for Example Settings.WorldTransform) from an outer owner
	template<typename ValueType>
	static ValueType& ContainerMemoryBlockToValueRef(uint8* InMemoryBlock, ValueType& InDefault, FEditPropertyChain& InPropertyChain, TArray<int32> &InPropertyArrayIndices)
	{
		if (InPropertyChain.GetHead() == nullptr)
		{
			return InDefault;
		}
		
		FEditPropertyChain::TDoubleLinkedListNode* PropertyNode = InPropertyChain.GetHead();
		uint8* MemoryPtr = InMemoryBlock;
		int32 ChainIndex = 0;
		do
		{
			const FProperty* Property = PropertyNode->GetValue();
			MemoryPtr = Property->ContainerPtrToValuePtr<uint8>(MemoryPtr);

			PropertyNode = PropertyNode->GetNextNode();
			ChainIndex++;
			
			if(InPropertyArrayIndices.IsValidIndex(ChainIndex))
			{
				const int32 ArrayIndex = InPropertyArrayIndices[ChainIndex];
				if(ArrayIndex != INDEX_NONE)
				{
					const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property->GetOwnerProperty());
					check(ArrayProperty);
					
					FScriptArrayHelper ArrayHelper(ArrayProperty, MemoryPtr);
					if(!ArrayHelper.IsValidIndex(ArrayIndex))
					{
						return InDefault;
					}
					MemoryPtr = ArrayHelper.GetRawPtr(ArrayIndex);

					// skip to the next property node already
					PropertyNode = PropertyNode->GetNextNode();
					ChainIndex++;
				}
			}
		}
		while (PropertyNode);

		return *(ValueType*)MemoryPtr;
	}

	// specializations for FEulerTransform and FRotator at the end of this file
	template<typename ValueType>
	static bool IsQuaternionBasedRotation() { return true; }

	// returns the numeric value of a vector component (or empty optional for varying values)
	template<typename VectorType, typename NumericType>
	TOptional<NumericType> GetVectorComponent(TSharedRef<class IPropertyHandle> InPropertyHandle, int32 InComponent) 
	{
		TOptional<NumericType> Result;
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return Result;
		}

		if (TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(InComponent))
		{
			if (ChildHandle->IsValidHandle())
			{
				NumericType Value;
				if (ChildHandle->GetValue(Value) == FPropertyAccess::Success) // note that this will fail if multiple values
				{
					Result = Value;
				}
			}
		}

		return Result;
	};

	// called when a numeric value of a vector component is changed
	template<typename VectorType, typename NumericType>
	void OnVectorComponentChanged(TSharedRef<class IPropertyHandle> InPropertyHandle, int32 InComponent, NumericType InValue, bool bIsCommit, ETextCommit::Type InCommitType = ETextCommit::Default)
	{
		URigVMController* Controller = nullptr;
		if (TStrongObjectPtr<UObject> Blueprint = BlueprintBeingCustomized.GetWeakObjectPtr().Pin(); GraphBeingCustomized.IsValid())
		{
			Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized.Get());
			if (bIsCommit)
			{
				Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InPropertyHandle->GetProperty()->GetName()));
			}
		}

		if (TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(InComponent))
		{
			if (ChildHandle->IsValidHandle())
			{
				ChildHandle->SetValue(InValue);
			}
		}

		if (Controller && bIsCommit)
		{
			Controller->CloseUndoBracket();
		}
	};

	// specializations for FVector and FVector4 at the end of this file
	template<typename VectorType>
	void ExtendVectorArgs(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr) {}

	template<typename VectorType, int32 NumberOfComponents>
	void MakeVectorHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils);

	template<typename RotationType>
	void MakeRotationHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils);

	template<typename TransformType>
	void ConfigureTransformWidgetArgs(TSharedRef<class IPropertyHandle> InPropertyHandle, typename SAdvancedTransformInputBox<TransformType>::FArguments& WidgetArgs, TConstArrayView<FName> ComponentNames);

	template<typename TransformType>
	void MakeTransformHeaderRow(TSharedRef<class IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TConstArrayView<FName> ComponentNames);

	template<typename TransformType>
	void MakeTransformChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils, TConstArrayView<FName> ComponentNames);

	// returns the rotation for rotator or quaternions (or empty optional for varying values)
	template<typename RotationType>
	TOptional<RotationType> GetRotation(TSharedRef<class IPropertyHandle> InPropertyHandle)
	{
		TOptional<RotationType> Result;
		FEditPropertyChain PropertyChain;
		TArray<int32> PropertyArrayIndices;
		bool bEnabled;
		if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
		{
			return Result;
		}

		if (InPropertyHandle->IsValidHandle())
		{
			RotationType Value;
			if (InPropertyHandle->GetValue(Value) == FPropertyAccess::Success) // note that this will fail if multiple values
			{
				Result = Value;
			}
		}

		return Result;
	};

	// called when a rotation value is changed / committed
	template<typename RotationType>
	void OnRotationChanged(TSharedRef<class IPropertyHandle> InPropertyHandle, RotationType InValue, bool bIsCommit, ETextCommit::Type InCommitType = ETextCommit::Default)
	{
		
		FEditPropertyChain PropertyChain;
        TArray<int32> PropertyArrayIndices;
        bool bEnabled;
        if (!GetPropertyChain(InPropertyHandle, PropertyChain, PropertyArrayIndices, bEnabled))
        {
        	return;
        }
	
		URigVMController* Controller = nullptr;
		if (TStrongObjectPtr<UObject> Blueprint = BlueprintBeingCustomized.GetWeakObjectPtr().Pin(); GraphBeingCustomized.IsValid())
		{
			Controller = BlueprintBeingCustomized->GetController(GraphBeingCustomized.Get());
			if(bIsCommit)
			{
				Controller->OpenUndoBracket(FString::Printf(TEXT("Set %s"), *InPropertyHandle->GetProperty()->GetName()));
			}
		}

		if (InPropertyHandle->IsValidHandle())
		{
			InPropertyHandle->SetValue(InValue);
		}

		if(Controller && bIsCommit)
		{
			Controller->CloseUndoBracket();
		}
	};

	// specializations for FRotator and FQuat at the end of this file
	template<typename RotationType>
	void ExtendRotationArgs(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr) {}

	template<typename TransformType>
	static FName GetTranslationPropertyName()
	{
		return TEXT("Translation");
	}

	template<typename TransformType>
	static FName GetRotationPropertyName()
	{
		return TEXT("Rotation");
	}

	template<typename TransformType>
	static FName GetScalePropertyName()
	{
		return TEXT("Scale3D");
	}

	TWeakInterfacePtr<IRigVMEditorAssetInterface> BlueprintBeingCustomized;
	TWeakObjectPtr<URigVMGraph> GraphBeingCustomized;
};

template<>
inline bool FRigVMGraphMathTypeDetailCustomization::IsQuaternionBasedRotation<FEulerTransform>() { return false; }

template<>
inline bool FRigVMGraphMathTypeDetailCustomization::IsQuaternionBasedRotation<FRotator>() { return false; }

template<>
inline void FRigVMGraphMathTypeDetailCustomization::ExtendVectorArgs<FVector>(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr)
{
	using VectorType = FVector;
	typedef typename VectorType::FReal NumericType;
	typedef SNumericVectorInputBox<NumericType, VectorType, 3> SLocalVectorInputBox;

	typename SLocalVectorInputBox::FArguments& Args = *(typename SLocalVectorInputBox::FArguments*)ArgumentsPtr; 
	Args
	.Z_Lambda([this, InPropertyHandle]()
	{
		return GetVectorComponent<VectorType, NumericType>(InPropertyHandle, 2);
	})
	.OnZChanged_Lambda([this, InPropertyHandle](NumericType Value)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 2, Value, false);
	})
	.OnZCommitted_Lambda([this, InPropertyHandle](NumericType Value, ETextCommit::Type CommitType)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 2, Value, true, CommitType);
	});
}

template<>
inline void FRigVMGraphMathTypeDetailCustomization::ExtendVectorArgs<FVector4>(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr)
{
	using VectorType = FVector4;
	typedef typename VectorType::FReal NumericType;
	typedef SNumericVectorInputBox<NumericType, VectorType, 4> SLocalVectorInputBox;

	typename SLocalVectorInputBox::FArguments& Args = *(typename SLocalVectorInputBox::FArguments*)ArgumentsPtr; 
	Args
	.Z_Lambda([this, InPropertyHandle]()
	{
		return GetVectorComponent<VectorType, NumericType>(InPropertyHandle, 2);
	})
	.OnZChanged_Lambda([this, InPropertyHandle](NumericType Value)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 2, Value, false);
	})
	.OnZCommitted_Lambda([this, InPropertyHandle](NumericType Value, ETextCommit::Type CommitType)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 2, Value, true, CommitType);
	})
	.W_Lambda([this, InPropertyHandle]()
	{
		return GetVectorComponent<VectorType, NumericType>(InPropertyHandle, 3);
	})
	.OnWChanged_Lambda([this, InPropertyHandle](NumericType Value)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 3, Value, false);
	})
	.OnWCommitted_Lambda([this, InPropertyHandle](NumericType Value, ETextCommit::Type CommitType)
	{
		OnVectorComponentChanged<VectorType, NumericType>(InPropertyHandle, 3, Value, true, CommitType);
	});
}

template<>
inline void FRigVMGraphMathTypeDetailCustomization::ExtendRotationArgs<FQuat>(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr)
{
	using RotationType = FQuat;
	typedef typename RotationType::FReal NumericType;
	typedef SAdvancedRotationInputBox<NumericType> SLocalRotationInputBox;
	typename SLocalRotationInputBox::FArguments& Args = *(typename SLocalRotationInputBox::FArguments*)ArgumentsPtr; 

	Args.Quaternion_Lambda([this, InPropertyHandle]() -> TOptional<RotationType>
	{
		return GetRotation<RotationType>(InPropertyHandle);
	});

	Args.OnQuaternionChanged_Lambda([this, InPropertyHandle](RotationType InValue)
	{
		OnRotationChanged<RotationType>(InPropertyHandle, InValue, false);
	});

	Args.OnQuaternionCommitted_Lambda([this, InPropertyHandle](RotationType InValue, ETextCommit::Type InCommitType)
	{
		OnRotationChanged<RotationType>(InPropertyHandle, InValue, true, InCommitType);
	});
}

template<>
inline void FRigVMGraphMathTypeDetailCustomization::ExtendRotationArgs<FRotator>(TSharedRef<class IPropertyHandle> InPropertyHandle, void* ArgumentsPtr)
{
	using RotationType = FRotator;
	typedef typename RotationType::FReal NumericType;
	typedef SAdvancedRotationInputBox<NumericType> SLocalRotationInputBox;
	typename SLocalRotationInputBox::FArguments& Args = *(typename SLocalRotationInputBox::FArguments*)ArgumentsPtr; 

	Args.Rotator_Lambda([this, InPropertyHandle]() -> TOptional<RotationType>
	{
		return GetRotation<RotationType>(InPropertyHandle);
	});

	Args.OnRotatorChanged_Lambda([this, InPropertyHandle](RotationType InValue)
	{
		OnRotationChanged<RotationType>(InPropertyHandle, InValue, false);
	});

	Args.OnRotatorCommitted_Lambda([this, InPropertyHandle](RotationType InValue, ETextCommit::Type InCommitType)
	{
		OnRotationChanged<RotationType>(InPropertyHandle, InValue, true, InCommitType);
	});
}

template<>
inline FName FRigVMGraphMathTypeDetailCustomization::GetTranslationPropertyName<FEulerTransform>()
{
	return TEXT("Location");
}

template<>
inline FName FRigVMGraphMathTypeDetailCustomization::GetScalePropertyName<FEulerTransform>()
{
	return TEXT("Scale");
}

#undef UE_API
