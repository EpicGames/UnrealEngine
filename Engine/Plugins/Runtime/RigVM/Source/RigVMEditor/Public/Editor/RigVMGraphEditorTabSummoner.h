// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "EdGraph/EdGraph.h"

#define UE_API RIGVMEDITOR_API

#define LOCTEXT_NAMESPACE "RigVMGraphEditorTabSummoner"

class FRigVMNewEditor;
class UEdGraphSchema;
struct FGraphDisplayInfo;
class SGraphEditor;

struct FRigVMLocalKismetCallbacks
{
	static FText GetGraphDisplayName(const UEdGraph* Graph);
};

struct FRigVMGraphEditorTabSummoner : public FDocumentTabFactoryForObjects<UEdGraph>
{
	static const FName TabID() { static FName ID = TEXT("RigVM Graph Editor"); return ID; }
	
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SGraphEditor>, FOnCreateGraphEditorWidget, TSharedRef<FTabInfo>, UEdGraph*);

	UE_API FRigVMGraphEditorTabSummoner(const TSharedRef<FRigVMNewEditor>& InRigVMEditor, FOnCreateGraphEditorWidget CreateGraphEditorWidgetCallback);

	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;
	
	virtual void OnTabBackgrounded(TSharedPtr<SDockTab> Tab) const override;

	virtual void SaveState(TSharedPtr<SDockTab> Tab, TSharedPtr<FTabPayload> Payload) const override;

	protected:
	virtual TAttribute<FText> ConstructTabNameForObject(UEdGraph* DocumentID) const override
	{
		return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FRigVMLocalKismetCallbacks::GetGraphDisplayName, (const UEdGraph*)DocumentID));
	}

	virtual TSharedRef<SWidget> CreateTabBodyForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	virtual const FSlateBrush* GetTabIconForObject(const FWorkflowTabSpawnInfo& Info, UEdGraph* DocumentID) const override;

	protected:
	TWeakPtr<class FRigVMNewEditor> BlueprintEditorPtr;
	FOnCreateGraphEditorWidget OnCreateGraphEditorWidget;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API