// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_RIGVMLEGACYEDITOR

#include "CoreMinimal.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorTabs.h"
#include "RigVMHost.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMLegacyEditorMode.h"
#include "Editor/RigVMDetailsViewWrapperObject.h"
#include "AssetRegistry/AssetData.h"
#include "Editor/RigVMNewEditor.h"

class FRigVMGraphExplorerDragDropOp;
class SRigVMEditorGraphExplorer;
class FRigVMLegacyEditor;

class FRigVMLegacyEditor : public FBlueprintEditor, public FRigVMEditorBase
{
public:

	RIGVMEDITOR_API FRigVMLegacyEditor();

	virtual TSharedRef<IRigVMEditor> SharedRef() override { return StaticCastSharedRef<IRigVMEditor>(SharedThis(this)); }
	virtual TSharedRef<const IRigVMEditor> SharedRef() const override { return StaticCastSharedRef<const IRigVMEditor>(SharedThis(this)); }

protected:

	// IRigVMEditor overrides
	virtual TSharedPtr<FAssetEditorToolkit> GetHostingApp() override { return AsShared(); }
	virtual const TSharedPtr<FAssetEditorToolkit> GetHostingApp() const override { return ConstCastSharedRef<FAssetEditorToolkit>(AsShared()); }
	RIGVMEDITOR_API virtual void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons, const TOptional<EAssetOpenMethod>& InOpenMethod) override;
	RIGVMEDITOR_API virtual void CreateEditorToolbar() override;
	RIGVMEDITOR_API virtual void CommonInitialization(const TArray<FRigVMEditorAssetInterfacePtr>& InitBlueprints, bool bShouldOpenInDefaultsMode) override;
	virtual TSharedPtr<FDocumentTracker> GetDocumentManager() const override { return DocumentManager; }
	virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode) override { FBlueprintEditor::AddApplicationMode(ModeName, Mode); }
	virtual void RegenerateMenusAndToolbars() override { FBlueprintEditor::RegenerateMenusAndToolbars(); }
	virtual void SetCurrentMode(FName NewMode) override { FBlueprintEditor::SetCurrentMode(NewMode); }
	virtual FEditorModeTools& GetToolkitEditorModeManager() const override { return FBlueprintEditor::GetEditorModeManager(); }
	virtual void PostLayoutBlueprintEditorInitialization() override { FBlueprintEditor::PostLayoutBlueprintEditorInitialization(); }
	virtual TSharedPtr<SGraphEditor> OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus) override { return FBlueprintEditor::OpenGraphAndBringToFront(Graph, bSetFocus); }
	virtual bool FindOpenTabsContainingDocument(const UObject* DocumentID, TArray<TSharedPtr<SDockTab>>& Results) override { return FBlueprintEditor::FindOpenTabsContainingDocument(DocumentID, Results); }
	virtual TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause) override { return FBlueprintEditor::OpenDocument(DocumentID, Cause); }
	virtual void CloseDocumentTab(const UObject* DocumentID) override { FBlueprintEditor::CloseDocumentTab(DocumentID); }
	virtual TSharedPtr<FTabManager> GetTabManager() override { return FBlueprintEditor::GetTabManager(); }
#if WITH_RIGVMLEGACYEDITOR
	virtual TSharedPtr<SKismetInspector> GetKismetInspector() const override { return Inspector.IsValid() ? Inspector.ToSharedRef().ToSharedPtr() : nullptr; }
#endif
	virtual TSharedPtr<SRigVMDetailsInspector> GetRigVMInspector() const override { return nullptr; }
	RIGVMEDITOR_API virtual TSharedPtr<FApplicationMode> CreateEditorMode() override;
	RIGVMEDITOR_API virtual const FName GetEditorAppName() const override;
	virtual const TArray< UObject* >& GetEditingBlueprints() const override { return FBlueprintEditor::GetEditingObjects(); }
	RIGVMEDITOR_API virtual void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor) override;
	virtual const TSharedRef<IToolkitHost> GetToolkitHost() const override { return FBlueprintEditor::GetToolkitHost(); }
	virtual bool IsHosted() const override { return FBlueprintEditor::IsHosted(); }
	virtual void BringToolkitToFrontImpl() override { FBlueprintEditor::BringToolkitToFront(); }
	RIGVMEDITOR_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual TWeakPtr<SGraphEditor> GetFocusedGraphEditor() override { return FocusedGraphEdPtr; }
	virtual TWeakPtr<FDocumentTabFactory> GetGraphEditorTabFactory() const override { return GraphEditorTabFactoryPtr; }
	RIGVMEDITOR_API virtual bool TransactionObjectAffectsBlueprint(UObject* InTransactedObject) override;
	RIGVMEDITOR_API virtual FEdGraphPinType GetLastPinTypeUsed() override;
	virtual void LogSimpleMessage(const FText& MessageText) override { FBlueprintEditor::LogSimpleMessage(MessageText); }
	virtual void RenameNewlyAddedAction(FName InActionName) override { FBlueprintEditor::RenameNewlyAddedAction(InActionName); }
	virtual FGraphPanelSelectionSet GetSelectedNodes() const override { return FBlueprintEditor::GetSelectedNodes(); }
	virtual void SetUISelectionState(FName SelectionOwner) override { FBlueprintEditor::SetUISelectionState(SelectionOwner); }
	virtual void AnalyticsTrackNodeEvent(UBlueprint* Blueprint, UEdGraphNode* GraphNode, bool bNodeDelete) const override { FBlueprintEditor::AnalyticsTrackNodeEvent(Blueprint, GraphNode, bNodeDelete); }
	virtual void AnalyticsTrackNodeEvent(IRigVMEditorAssetInterface* Blueprint, UEdGraphNode* GraphNode, bool bNodeDelete) const override { FBlueprintEditor::AnalyticsTrackNodeEvent(Cast<UBlueprint>(Blueprint->GetObject()), GraphNode, bNodeDelete); }
	RIGVMEDITOR_API virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename) override;
	RIGVMEDITOR_API virtual void PostUndo(bool bSuccess) override;
	RIGVMEDITOR_API virtual void PostRedo(bool bSuccess) override;
	virtual UEdGraphPin* GetCurrentlySelectedPin() const override { return FBlueprintEditor::GetCurrentlySelectedPin();}
	RIGVMEDITOR_API virtual void CreateDefaultCommands() override;
	RIGVMEDITOR_API virtual TSharedRef<SGraphEditor> CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph) override;
	virtual void CompileImpl() override { FBlueprintEditor::Compile(); }
	virtual void SaveAsset_Execute_Impl() override { FBlueprintEditor::SaveAsset_Execute(); }
	virtual void SaveAssetAs_Execute_Impl() override { FBlueprintEditor::SaveAssetAs_Execute(); }
	virtual bool IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const override { return FBlueprintEditor::IsGraphInCurrentBlueprint(InGraph); }
	virtual bool IsEditableImpl(UEdGraph* InGraph) const override { return FBlueprintEditor::IsEditable(InGraph); }
	virtual UEdGraph* GetFocusedGraph() const override { return FBlueprintEditor::GetFocusedGraph(); };
	virtual void JumpToNode(const UEdGraphNode* Node, bool bRequestRename) override { FBlueprintEditor::JumpToNode(Node, bRequestRename); }
	virtual void JumpToPin(const UEdGraphPin* Pin) override { FBlueprintEditor::JumpToPin(Pin); }
	virtual void AddToolbarExtender(TSharedPtr<FExtender> Extender) override { FBlueprintEditor::AddToolbarExtender(Extender); }
	virtual void RemoveToolbarExtender(TSharedPtr<FExtender> Extender) override { FBlueprintEditor::RemoveToolbarExtender(Extender); }
	virtual void AddMenuExtender(TSharedPtr<FExtender> Extender) override { FBlueprintEditor::AddMenuExtender(Extender); }
	virtual void RemoveMenuExtender(TSharedPtr<FExtender> Extender) override { FBlueprintEditor::RemoveMenuExtender(Extender); }
	virtual TSharedPtr<IMessageLogListing> GetCompilerResultsListing() override { return CompilerResultsListing; }
	virtual void OnBlueprintChangedInnerImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled) override { FBlueprintEditor::OnBlueprintChangedImpl(Cast<UBlueprint>(InBlueprint->GetObject()), bIsJustBeingCompiled); }
	RIGVMEDITOR_API virtual void RefreshEditorsImpl(ERefreshRigVMEditorReason::Type Reason) override;
	virtual void SetupGraphEditorEvents(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents) override { FRigVMEditorBase::SetupGraphEditorEvents(InGraph, InEvents); }
	virtual void SetupGraphEditorEventsImpl(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents) override { FBlueprintEditor::SetupGraphEditorEvents(InGraph, InEvents); }
	virtual FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed) override { return FBlueprintEditor::OnCreateGraphActionMenu(InGraph, InNodePosition, InDraggedPins, bAutoExpand, InOnMenuClosed); }
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { FRigVMEditorBase::AddReferencedObjects(Collector); }
	virtual void AddReferencedObjectsImpl(FReferenceCollector& Collector) override { FBlueprintEditor::AddReferencedObjects(Collector); }
	RIGVMEDITOR_API virtual bool NewDocument_IsVisibleForType(FBlueprintEditor::ECreatedDocumentType GraphType) const override;
	RIGVMEDITOR_API virtual bool IsSectionVisible(NodeSectionID::Type InSectionID) const override;
	virtual FGraphAppearanceInfo GetGraphAppearanceImpl(UEdGraph* InGraph) const override { return FBlueprintEditor::GetGraphAppearance(InGraph); }
	virtual void NotifyPreChangeImpl(FProperty* PropertyAboutToChange) override { FBlueprintEditor::NotifyPreChange(PropertyAboutToChange); }
	virtual void NotifyPostChangeImpl(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override { FBlueprintEditor::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged); }
	RIGVMEDITOR_API virtual FName GetSelectedVariableName() override;
	virtual bool IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename) override { return FBlueprintEditor::IsNodeTitleVisible(Node, bRequestRename); }
	virtual void EditClassDefaults_Clicked() override { FBlueprintEditor::EditClassDefaults_Clicked(); }
	virtual void EditGlobalOptions_Clicked() override { FBlueprintEditor::EditGlobalOptions_Clicked(); }
	virtual bool IsDetailsPanelEditingClassDefaults() const override { return FBlueprintEditor::IsDetailsPanelEditingClassDefaults(); }
	virtual bool IsDetailsPanelEditingGlobalOptions() const override { return FBlueprintEditor::IsDetailsPanelEditingGlobalOptions(); }
	virtual void TryInvokingDetailsTab(bool bFlash) override { FBlueprintEditor::TryInvokingDetailsTab(bFlash); }
	virtual FName GetGraphExplorerWidgetID() override { return FBlueprintEditorTabs::MyBlueprintID; }
	virtual void RefreshInspector() override { FBlueprintEditor::RefreshInspector(); }
	virtual void RefreshStandAloneDefaultsEditor() override { FBlueprintEditor::RefreshStandAloneDefaultsEditor(); }
	virtual void GetPinTypeSelectorFilters(TArray<TSharedPtr<IPinTypeSelectorFilter>>& OutFilters) const override { FBlueprintEditor::GetPinTypeSelectorFilters(OutFilters); }
	virtual void OnAddNewVariable() override { FBlueprintEditor::OnAddNewVariable(); }
	virtual void ZoomToSelection_Clicked() override { FBlueprintEditor::ZoomToSelection_Clicked(); }
	virtual void RestoreEditedObjectState() override { FBlueprintEditor::RestoreEditedObjectState(); }
	virtual void SetupViewForBlueprintEditingMode() override { FBlueprintEditor::SetupViewForBlueprintEditingMode(); }
	virtual bool GetIsContextSensitive() override { return FBlueprintEditor::GetIsContextSensitive(); }
	virtual void SetIsContextSensitive(const bool bIsContextSensitive) override { FBlueprintEditor::GetIsContextSensitive() = bIsContextSensitive; }
	virtual void RegisterToolbarTab(const TSharedRef<FTabManager>& InTabManager) override { FBlueprintEditor::RegisterToolbarTab(InTabManager); }
	virtual const TArray<UObject*>* GetObjectsCurrentlyBeingEdited() const override { return FBlueprintEditor::GetObjectsCurrentlyBeingEdited(); }
	RIGVMEDITOR_API virtual void AddCompileWidget(FToolBarBuilder& ToolbarBuilder) override;
	virtual void AddSettingsAndDefaultWidget(FToolBarBuilder& ToolbarBuilder) override {}
	virtual void AddFindWidget(FToolBarBuilder& ToolbarBuilder) override {}
	RIGVMEDITOR_API virtual void AddSelectedDebugObjectWidget(FToolBarBuilder& ToolbarBuilder) override;
	RIGVMEDITOR_API virtual void AddAutoCompileWidget(FToolBarBuilder& ToolbarBuilder) override;
	virtual FString GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const override { return FRigVMEditorBase::GetCustomDebugObjectLabel(ObjectBeingDebugged); }
public:
	virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult) override { FBlueprintEditor::SummonSearchUI(bSetFindWithinBlueprint, NewSearchTerms, bSelectFirstResult); }
	virtual TSharedRef<FUICommandList> GetToolkitCommands() override { return ToolkitCommands; }
	RIGVMEDITOR_API virtual void OnClose() override;
	virtual FPreviewScene* GetPreviewScene() override { return &PreviewScene; }

	virtual void ToggleHideUnrelatedNodes() override
	{
		ToggleFadeOutUnrelateNodes();
	}

	virtual bool IsToggleHideUnrelatedNodesChecked() const override
	{
		return IsToggleFadeOutUnrelatedNodesChecked();
	}
	
protected:
	RIGVMEDITOR_API virtual void Tick(float DeltaTime) override;
	// IRigVMEditor overrides

	///////////////////////////////////////////////////////////////////////////////////////////////
	// FBlueprintEditor overrides
	virtual UBlueprint* GetBlueprintObj() const override { return Cast<UBlueprint>(FRigVMEditorBase::GetRigVMAssetInterface()->GetObject()); }
	virtual TSubclassOf<UEdGraphSchema> GetDefaultSchemaClass() const override { return FRigVMEditorBase::GetDefaultSchemaClass(); }
	virtual bool InEditingMode() const override { return FRigVMEditorBase::InEditingMode(); }
	virtual bool CanAddNewLocalVariable() const override { return FRigVMEditorBase::CanAddNewLocalVariable(); }
	virtual void OnAddNewLocalVariable() override { FRigVMEditorBase::OnAddNewLocalVariable(); }
	virtual void OnPasteNewLocalVariable(const FBPVariableDescription& VariableDescription) override { FRigVMEditorBase::OnPasteNewLocalVariable(VariableDescription); }
	virtual void DeleteSelectedNodes() override { FRigVMEditorBase::DeleteSelectedNodes(); }
	virtual bool CanDeleteNodes() const override { return FRigVMEditorBase::CanDeleteNodes(); }
	virtual void CopySelectedNodes() override { FRigVMEditorBase::CopySelectedNodes(); }
	virtual bool CanCopyNodes() const override { return FRigVMEditorBase::CanCopyNodes(); }
	virtual void PasteNodes() override { FRigVMEditorBase::PasteNodes(); }
	virtual bool CanPasteNodes() const override { return FRigVMEditorBase::CanPasteNodes(); }
	virtual bool IsNativeParentClassCodeLinkEnabled() const override { return FRigVMEditorBase::IsNativeParentClassCodeLinkEnabled(); }
	virtual bool ReparentBlueprint_IsVisible() const override { return FRigVMEditorBase::ReparentBlueprint_IsVisible(); }
	virtual FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph) override { return FRigVMEditorBase::OnSpawnGraphNodeByShortcut(InChord, InPosition, InGraph); }
	virtual bool ShouldLoadBPLibrariesFromAssetRegistry() override { return FRigVMEditorBase::ShouldLoadBPLibrariesFromAssetRegistry(); }
	virtual bool ShouldOpenGraphByDefault() const { return FRigVMEditorBase::ShouldOpenGraphByDefault(); }
	virtual void AddNewFunctionVariant(const UEdGraph* InOriginalFunction) override { return FRigVMEditorBase::AddNewFunctionVariant(InOriginalFunction); }
	virtual bool SelectLocalVariable(const UEdGraph* Graph, const FName& VariableName) override { return FRigVMEditorBase::SelectLocalVariable(Graph, VariableName); }
	virtual void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList) { return FRigVMEditorBase::OnCreateGraphEditorCommands(GraphEditorCommandsList); }
	virtual void Compile() override { return FRigVMEditorBase::Compile(); }
	virtual void SaveAsset_Execute() override { return FRigVMEditorBase::SaveAsset_Execute(); }
	virtual void SaveAssetAs_Execute() override { return FRigVMEditorBase::SaveAssetAs_Execute(); }
	virtual bool IsInAScriptingMode() const override { return FRigVMEditorBase::IsInAScriptingMode(); }
	virtual void NewDocument_OnClicked(FBlueprintEditor::ECreatedDocumentType GraphType) override { return FRigVMEditorBase::OnNewDocumentClicked((FRigVMEditorBase::ECreatedDocumentType)GraphType); }
	virtual bool AreEventGraphsAllowed() const override { return true; }
	virtual bool AreMacrosAllowed() const override { return false; }
	virtual bool AreDelegatesAllowed() const override { return false; }
	virtual FGraphAppearanceInfo GetGraphAppearance(class UEdGraph* InGraph) const override { return FRigVMEditorBase::GetGraphAppearance(InGraph); }
	virtual bool IsEditable(UEdGraph* InGraph) const override { return FRigVMEditorBase::IsEditable(InGraph); }
	virtual bool IsCompilingEnabled() const override { return FRigVMEditorBase::IsCompilingEnabled(); }
	virtual FText GetGraphDecorationString(UEdGraph* InGraph) const override { return FRigVMEditorBase::GetGraphDecorationString(InGraph); }
	virtual void OnActiveTabChanged( TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated ) override { return FRigVMEditorBase::OnActiveTabChanged(PreviouslyActive, NewlyActivated); }
	virtual void OnSelectedNodesChangedImpl(const TSet<class UObject*>& NewSelection) override { FRigVMEditorBase::OnSelectedNodesChangedImpl(NewSelection); }
	virtual void OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled) override { return FRigVMEditorBase::OnBlueprintChangedImpl(Cast<URigVMBlueprint>(InBlueprint), bIsJustBeingCompiled); }
	virtual void RefreshEditors(ERefreshBlueprintEditorReason::Type Reason = ERefreshBlueprintEditorReason::UnknownReason) override { return ForceEditorRefresh((ERefreshRigVMEditorReason::Type)Reason); }
	virtual void FocusInspectorOnGraphSelection(const TSet<class UObject*>& NewSelection, bool bForceRefresh = false) override { return FRigVMEditorBase::FocusInspectorOnGraphSelection(NewSelection, bForceRefresh); }
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override { return FRigVMEditorBase::NotifyPreChange(PropertyAboutToChange); }
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override { return FRigVMEditorBase::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged); }
	virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) override { return FRigVMEditorBase::OnFinishedChangingProperties(PropertyChangedEvent); }
	virtual TStatId GetStatId() const override { return FRigVMEditorBase::GetStatId(); }
	virtual FName GetToolkitFName() const override { return FRigVMEditorBase::GetToolkitFName(); }
	virtual FName GetToolkitContextFName() const override { return FRigVMEditorBase::GetToolkitContextFName(); }
	virtual FText GetBaseToolkitName() const override { return FRigVMEditorBase::GetBaseToolkitName(); }
	virtual FText GetToolkitToolTipText() const override { return FRigVMEditorBase::GetToolkitToolTipText(); }
	virtual FString GetWorldCentricTabPrefix() const override { return FRigVMEditorBase::GetWorldCentricTabPrefix(); }
	virtual FLinearColor GetWorldCentricTabColorScale() const override { return FRigVMEditorBase::GetWorldCentricTabColorScale(); }
	virtual void OnStartWatchingPin() override { FRigVMEditorBase::OnStartWatchingPin(); }
	virtual bool CanStartWatchingPin() const override { return FRigVMEditorBase::CanStartWatchingPin(); }
	virtual void OnStopWatchingPin() override { FRigVMEditorBase::OnStopWatchingPin(); }
	virtual bool CanStopWatchingPin() const override { return FRigVMEditorBase::CanStopWatchingPin(); }
	virtual void OnCreateComment() override { FRigVMEditorBase::OnCreateComment(); }
	RIGVMEDITOR_API virtual void GetCustomDebugObjects(TArray<FCustomDebugObject>& DebugList) const override;
	virtual bool OnlyShowCustomDebugObjects() const override { return FRigVMEditorBase::OnlyShowCustomDebugObjects(); }
	virtual void HandleSetObjectBeingDebugged(UObject* InObject) override { FRigVMEditorBase::HandleSetObjectBeingDebugged(InObject); }
	virtual bool OnActionMatchesName(FEdGraphSchemaAction* InAction, const FName& InName) const override { return FRigVMEditorBase::OnActionMatchesName(InAction, InName); }

	// IToolkit Interface
	virtual void BringToolkitToFront() override { FRigVMEditorBase::BringToolkitToFront(); }

	virtual FNotifyHook* GetNotifyHook() override { return this; }

};

#endif
