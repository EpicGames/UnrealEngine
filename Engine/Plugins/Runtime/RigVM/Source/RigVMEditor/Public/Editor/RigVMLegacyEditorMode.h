// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_RIGVMLEGACYEDITOR

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "RigVMBlueprintLegacy.h"
#include "BlueprintEditorModes.h"

class URigVMBlueprint;
class FRigVMLegacyEditor;

class FRigVMLegacyEditorMode : public FBlueprintEditorApplicationMode
{
public:
	FRigVMLegacyEditorMode(const TSharedRef<FRigVMLegacyEditor>& InRigVMEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	// Set of spawnable tabs
	FWorkflowAllowedTabSet TabFactories;

private:
	TWeakObjectPtr<URigVMBlueprint> RigVMBlueprintPtr;
};

#endif