// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "RigVMHost.h"
#include "RigVMBlueprintLegacy.h"
#include "Editor/RigVMDetailsViewWrapperObject.h"
#include "EdGraph/RigVMEdGraph.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"
#include "Styling/SlateTypes.h"
#include "IPropertyUtilities.h"
#include "EdGraph/RigVMEdGraphSchema.h"

#define UE_API RIGVMEDITOR_API

class IPropertyHandle;
struct FRigVMMemoryStorageStruct;

class FRigVMLocalVariableDetailCustomization : public IDetailCustomization
{
	FRigVMLocalVariableDetailCustomization()
	: GraphBeingCustomized(nullptr)
	, BlueprintBeingCustomized(nullptr)
	, NameValidator((const FRigVMEditorAssetInterfacePtr)nullptr, nullptr, NAME_None)
	{}

	
public:

	// Makes a new instance of this detail layout class for a specific detail view requesting it
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMLocalVariableDetailCustomization);
	}

	/** IDetailCustomization interface */
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	URigVMGraph* GraphBeingCustomized;
	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized;
	FRigVMGraphVariableDescription VariableDescription;
	TArray<TWeakObjectPtr<URigVMDetailsViewWrapperObject>> ObjectsBeingCustomized;

	TSharedPtr<IPropertyHandle> NameHandle;
	TSharedPtr<IPropertyHandle> TypeHandle;
	TSharedPtr<IPropertyHandle> TypeObjectHandle;
	TSharedPtr<IPropertyHandle> DefaultValueHandle;
	TSharedPtr<FRigVMMemoryStorageStruct> MemoryStorage;

	FRigVMLocalVariableNameValidator NameValidator;
	TArray<TSharedPtr<FString>> EnumOptions;

	UE_API FText GetName() const;
	UE_API void SetName(const FText& InNewText, ETextCommit::Type InCommitType);
	UE_API bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);
	
	UE_API FEdGraphPinType OnGetPinInfo() const;
	UE_API void HandlePinInfoChanged(const FEdGraphPinType& PinType);

	UE_API ECheckBoxState HandleBoolDefaultValueIsChecked( ) const;
	UE_API void OnBoolDefaultValueChanged(ECheckBoxState InCheckBoxState);
};



#undef UE_API
