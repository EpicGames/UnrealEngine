// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/RigVMController.h"
#include "RigVMModel/RigVMClient.h"
#include "EdGraph/RigVMEdGraph.h"
#include "EdGraph/RigVMEdGraphNode.h"
#include "UObject/StrongObjectPtr.h"

#define UE_API RIGVMEDITOR_API

class FRigVMMinimalEnvironment : public TSharedFromThis<FRigVMMinimalEnvironment>
{
public:
	UE_API FRigVMMinimalEnvironment(const IRigVMAssetInterface* InAssetInterface = nullptr);
	UE_API FRigVMMinimalEnvironment(IRigVMClientHost* InRigVMClientHost, UObject* InAssetObject);

	UE_API URigVMGraph* GetModel() const;
	UE_API URigVMController* GetController() const;
	UE_API URigVMNode* GetNode() const;
	UE_API URigVMEdGraph* GetEdGraph() const;
	UE_API URigVMEdGraphNode* GetEdGraphNode() const;

	UE_API void SetSchemata(const IRigVMAssetInterface* InAssetInterface);
	UE_API void SetSchemata(const IRigVMClientHost* InRigVMClientHost, const UObject* InAssetObject);
	UE_API void SetNode(URigVMNode* InModelNode);
	UE_API URigVMNode* SetFunctionReference(URigVMLibraryNode* InFunctionDefinition, const FString& InNodeName);
	UE_API URigVMNode* SetEmptyCollapseNode(URigVMCollapseNode* InSourceCollapseNode, const FString& InNodeName);
	UE_API void SetFunctionNode(const FRigVMGraphFunctionIdentifier& InIdentifier);
	
	UE_API FSimpleDelegate& OnChanged();
	UE_API void Tick_GameThead(float InDeltaTime);

	UE_API void SynchronizeCollapseNode(URigVMCollapseNode* InSource);
	
private:
	UE_API void HandleModified(ERigVMGraphNotifType InNotification, URigVMGraph* InGraph, UObject* InSubject);
	void InitGraphData(UObject* InOuter);

	TWeakInterfacePtr<IRigVMClientHost> WeakRigVMClientHost;
	TStrongObjectPtr<URigVMGraph> ModelGraph;
	TStrongObjectPtr<URigVMController> ModelController;
	UClass* EdGraphClass;
	UClass* EdGraphNodeClass;
	TStrongObjectPtr<URigVMEdGraph> EdGraph;
	TWeakObjectPtr<URigVMNode> ModelNode;
	TWeakObjectPtr<URigVMEdGraphNode> EdGraphNode;
	std::atomic<int32> NumModifications;
	FSimpleDelegate ChangedDelegate;
	FDelegateHandle ModelHandle;
};

#undef UE_API
