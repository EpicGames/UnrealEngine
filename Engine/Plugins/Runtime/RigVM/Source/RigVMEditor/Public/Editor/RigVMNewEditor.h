// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorSettings.h"
#include "RigVMEditor.h"
#include "RigVMSettings.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

#define UE_API RIGVMEDITOR_API

class SRigVMFindReferences;
class SRigVMDetailsInspector;

struct FRigVMNewEditorTabs
{
	// Tab identifiers
	static UE_API const FName CompilerResultsID();
};

class FRigVMNewEditor : public FWorkflowCentricApplication, public FRigVMEditorBase, public FGCObject, public FNotifyHook, public FTickableEditorObject, public FEditorUndoClient, public FNoncopyable
{
public:
	UE_API FRigVMNewEditor();
	UE_API virtual void OnClose() override;

	virtual TSharedRef<IRigVMEditor> SharedRef() override { return StaticCastSharedRef<IRigVMEditor>(SharedThis(this)); }
	virtual TSharedRef<const IRigVMEditor> SharedRef() const override { return StaticCastSharedRef<const IRigVMEditor>(SharedThis(this)); }

	virtual TSharedPtr<FAssetEditorToolkit> GetHostingApp() override { return AsShared(); }
	virtual const TSharedPtr<FAssetEditorToolkit> GetHostingApp() const override { return ConstCastSharedRef<FAssetEditorToolkit>(AsShared()); }
protected:
	
	UE_API virtual void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, const FName AppIdentifier, const TSharedRef<FTabManager::FLayout>& StandaloneDefaultLayout, const bool bCreateDefaultStandaloneMenu, const bool bCreateDefaultToolbar, const TArray<UObject*>& ObjectsToEdit, const bool bInIsToolbarFocusable, const bool bInUseSmallToolbarIcons, const TOptional<EAssetOpenMethod>& InOpenMethod) override;
	virtual void CreateEditorToolbar() override {}
	UE_API virtual void CommonInitialization(const TArray<FRigVMEditorAssetInterfacePtr>& InitBlueprints, bool bShouldOpenInDefaultsMode) override;
	UE_API void OnBlueprintChanged(UObject* InBlueprint);
	virtual TSharedPtr<FDocumentTracker> GetDocumentManager() const override { return DocumentManager; }
	virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode) override { FWorkflowCentricApplication::AddApplicationMode(ModeName, Mode); }
	virtual void RegenerateMenusAndToolbars() override { FWorkflowCentricApplication::RegenerateMenusAndToolbars(); }
	UE_API virtual void SetCurrentMode(FName NewMode) override;
	virtual FEditorModeTools& GetToolkitEditorModeManager() const override { return FWorkflowCentricApplication::GetEditorModeManager(); }
	UE_API virtual void PostLayoutBlueprintEditorInitialization() override;
	UE_API virtual TSharedPtr<SGraphEditor> OpenGraphAndBringToFront(UEdGraph* Graph, bool bSetFocus = true) override;
	UE_API virtual bool FindOpenTabsContainingDocument(const UObject* DocumentID, /*inout*/ TArray< TSharedPtr<SDockTab> >& Results) override;
	UE_API virtual TSharedPtr<SDockTab> OpenDocument(const UObject* DocumentID, FDocumentTracker::EOpenDocumentCause Cause) override;
	UE_API virtual void CloseDocumentTab(const UObject* DocumentID) override;
	virtual TSharedPtr<FTabManager> GetTabManager() override { return FWorkflowCentricApplication::GetTabManager(); }
public:
	virtual TSharedPtr<SRigVMDetailsInspector> GetRigVMInspector() const override { return Inspector.IsValid() ? Inspector.ToSharedRef().ToSharedPtr() : nullptr; }
	virtual void SetInspector(TSharedPtr<SRigVMDetailsInspector> InWidget) { Inspector = InWidget; };
	
	TSharedRef<SWidget> GetCompilerResults() const;
	TSharedRef<SRigVMFindReferences> GetFindResults() const { return FindResults.ToSharedRef(); }
	UE_API virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename) override;
protected:
	virtual TSharedPtr<IMessageLogListing> GetCompilerResultsListing() override { return CompilerResultsListing; }
	UE_API virtual TSharedPtr<FApplicationMode> CreateEditorMode() override;
	UE_API virtual const FName GetEditorAppName() const override;
	UE_API virtual FText GetToolkitName() const override;
	UE_API virtual FText GetToolkitToolTipText() const override;
	virtual const TArray< UObject* >& GetEditingBlueprints() const override { return FWorkflowCentricApplication::GetEditingObjects(); }
	virtual const TSharedRef<IToolkitHost> GetToolkitHost() const override { return FWorkflowCentricApplication::GetToolkitHost(); }
	virtual bool IsHosted() const override { return FWorkflowCentricApplication::IsHosted(); }
	virtual void BringToolkitToFrontImpl() override { FWorkflowCentricApplication::BringToolkitToFront(); }
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	virtual TSharedRef<FUICommandList> GetToolkitCommands() override { return ToolkitCommands; }
	virtual TWeakPtr<SGraphEditor> GetFocusedGraphEditor() override { return FocusedGraphEdPtr; }
	virtual TWeakPtr<FDocumentTabFactory> GetGraphEditorTabFactory() const override { return GraphEditorTabFactoryPtr; }
	UE_API virtual bool TransactionObjectAffectsBlueprint(UObject* InTransactedObject) override;
	UE_API virtual FEdGraphPinType GetLastPinTypeUsed() override;
	UE_API virtual void LogSimpleMessage(const FText& MessageText) override;
	UE_API virtual void RenameNewlyAddedAction(FName InActionName) override;
	UE_API virtual FGraphPanelSelectionSet GetSelectedNodes() const override;
	UE_API virtual void SetUISelectionState(FName SelectionOwner) override;
	UE_API virtual void AnalyticsTrackNodeEvent(IRigVMEditorAssetInterface* Blueprint, UEdGraphNode* GraphNode, bool bNodeDelete) const override;
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	UE_API virtual UEdGraphPin* GetCurrentlySelectedPin() const override;
	UE_API virtual void CreateDefaultCommands() override;
	UE_API virtual TSharedRef<SGraphEditor> CreateGraphEditorWidget(TSharedRef<FTabInfo> InTabInfo, UEdGraph* InGraph) override;
	UE_API virtual void CompileImpl() override;
	virtual void SaveAsset_Execute_Impl() override { FWorkflowCentricApplication::SaveAsset_Execute(); }
	virtual void SaveAssetAs_Execute_Impl() override { FWorkflowCentricApplication::SaveAssetAs_Execute(); }
	UE_API virtual bool IsGraphInCurrentBlueprint(const UEdGraph* InGraph) const override;
	UE_API virtual bool IsEditableImpl(UEdGraph* InGraph) const override;
	UE_API virtual UEdGraph* GetFocusedGraph() const override;
	UE_API virtual void JumpToNode(const UEdGraphNode* Node, bool bRequestRename) override;
	UE_API virtual void JumpToPin(const UEdGraphPin* Pin) override;
	virtual void AddToolbarExtender(TSharedPtr<FExtender> Extender) override { FWorkflowCentricApplication::AddToolbarExtender(Extender); }
	virtual void RemoveToolbarExtender(TSharedPtr<FExtender> Extender) override { FWorkflowCentricApplication::RemoveToolbarExtender(Extender); };
	virtual void AddMenuExtender(TSharedPtr<FExtender> Extender) override { FWorkflowCentricApplication::AddMenuExtender(Extender); }
	virtual void RemoveMenuExtender(TSharedPtr<FExtender> Extender) override { FWorkflowCentricApplication::RemoveMenuExtender(Extender); }
	UE_API virtual void OnBlueprintChangedInnerImpl(IRigVMEditorAssetInterface* InBlueprint, bool bIsJustBeingCompiled) override;
	UE_API virtual void RefreshEditorsImpl(ERefreshRigVMEditorReason::Type Reason) override;
	UE_API virtual void SetupGraphEditorEventsImpl(UEdGraph* InGraph, SGraphEditor::FGraphEditorEvents& InEvents) override;

	/** Called when the graph canvas was double clicked */
	UE_API void HandleGraphCanvasDoubleClicked(UEdGraph* InGraph);
	UE_API void OnClosedActionMenu(UEdGraph* InEdGraph) const;
	UE_API virtual FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed) override;
	UE_API virtual void AddReferencedObjectsImpl(FReferenceCollector& Collector) override;
	UE_API virtual bool IsSectionVisible(RigVMNodeSectionID::Type InSectionID) const; 
	UE_API virtual bool NewDocument_IsVisibleForType(ECreatedDocumentType GraphType) const;
	UE_API virtual FGraphAppearanceInfo GetGraphAppearanceImpl(UEdGraph* InGraph) const override;
	UE_API virtual void NotifyPreChangeImpl(FProperty* PropertyAboutToChange) override;
	UE_API virtual void NotifyPostChangeImpl(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	UE_API virtual FName GetSelectedVariableName() override;
	UE_API virtual bool IsNodeTitleVisible(const UEdGraphNode* Node, bool bRequestRename) override;
	UE_API virtual void EditClassDefaults_Clicked() override;
	UE_API virtual void EditGlobalOptions_Clicked() override;
	UE_API virtual bool IsDetailsPanelEditingGlobalOptions() const override;
	UE_API virtual bool IsDetailsPanelEditingClassDefaults() const override;
	UE_API virtual void TryInvokingDetailsTab(bool bFlash = true) override;
	virtual FName GetGraphExplorerWidgetID() override { return FRigVMEditorGraphExplorerTabSummoner::TabID(); }
	UE_API virtual void RefreshInspector() override;
	UE_API virtual void RefreshStandAloneDefaultsEditor() override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual void GetPinTypeSelectorFilters(TArray<TSharedPtr<IPinTypeSelectorFilter>>& OutFilters) const override;
	UE_API virtual void OnAddNewVariable() override;
	UE_API virtual void ZoomToSelection_Clicked() override;
	UE_API void FindInBlueprint_Clicked();
public:
	UE_API virtual void SummonSearchUI(bool bSetFindWithinBlueprint, FString NewSearchTerms, bool bSelectFirstResult) override;
	UE_API void SaveEditedObjectState();
	UE_API virtual void RestoreEditedObjectState() override;
	UE_API virtual void SetupViewForBlueprintEditingMode() override;
	UE_API virtual void OnGraphEditorFocused(const TSharedRef<SGraphEditor>& InGraphEditor);
	UE_API virtual void OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor);
	virtual bool GetIsContextSensitive() override { return bIsActionMenuContextSensitive; }
	virtual void SetIsContextSensitive(const bool bIsContextSensitive) override { bIsActionMenuContextSensitive = bIsContextSensitive; }
	virtual void RegisterToolbarTab(const TSharedRef<FTabManager>& InTabManager) override { FAssetEditorToolkit::RegisterTabSpawners(InTabManager); }
	virtual const TArray<UObject*>* GetObjectsCurrentlyBeingEdited() const override { return FAssetEditorToolkit::GetObjectsCurrentlyBeingEdited(); }
	UE_API virtual void AddCompileWidget(FToolBarBuilder& ToolbarBuilder) override;
	UE_API virtual void AddSettingsAndDefaultWidget(FToolBarBuilder& ToolbarBuilder) override;
	UE_API virtual void AddFindWidget(FToolBarBuilder& ToolbarBuilder) override;
	UE_API virtual void AddSelectedDebugObjectWidget(FToolBarBuilder& ToolbarBuilder) override;
	virtual void AddAutoCompileWidget(FToolBarBuilder& ToolbarBuilder) override {}
	virtual void Compile() { FRigVMEditorBase::Compile(); }
	virtual void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList) { FRigVMEditorBase::OnCreateGraphEditorCommands(GraphEditorCommandsList); }
	virtual bool ShouldOpenGraphByDefault() const { return FRigVMEditorBase::ShouldOpenGraphByDefault(); }
	virtual void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent) { FRigVMEditorBase::OnFinishedChangingProperties(PropertyChangedEvent); }
	virtual void HandleSetObjectBeingDebugged(UObject* InObject) { FRigVMEditorBase::HandleSetObjectBeingDebugged(InObject); }
	virtual FReply OnSpawnGraphNodeByShortcut(FInputChord InChord, const FVector2f& InPosition, UEdGraph* InGraph) { return FRigVMEditorBase::OnSpawnGraphNodeByShortcut(InChord, InPosition, InGraph); }
	virtual FPreviewScene* GetPreviewScene() override { return nullptr; }

	//~ Begin IToolkit Interface
	UE_API virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	//~ End IToolkit Interface


	virtual FLinearColor GetWorldCentricTabColorScale() const override { return FRigVMEditorBase::GetWorldCentricTabColorScale(); }
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override { return FRigVMEditorBase::AddReferencedObjects(Collector); }
	virtual FString GetReferencerName() const override { return TEXT("FRigVMNewEditor"); }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FRigVMNewEditor, STATGROUP_Tickables); }
	UE_API virtual void StartEditingDefaults(bool bAutoFocus = true, bool bForceRefresh = false);

	UE_API float GetInstructionTextOpacity(UEdGraph* InGraph) const;
	UE_API virtual void ClearSelectionStateFor(FName SelectionOwner);
protected:
	UE_API void OnLogTokenClicked(const TSharedRef<IMessageToken>& MessageToken);

	/** Dumps messages to the compiler log, with an option to force it to display/come to front */
	UE_API void DumpMessagesToCompilerLog(const TArray<TSharedRef<class FTokenizedMessage>>& Messages, bool bForceMessageDisplay);
public:
	UE_DEPRECATED(5.7, "Please use void CreateDefaultTabContents(const TArray<FRigVMEditorAssetInterfacePtr> InBlueprints)")
	void CreateDefaultTabContents(const TArray<UBlueprint*> InBlueprints){}
	UE_API void CreateDefaultTabContents(const TArray<FRigVMEditorAssetInterfacePtr> InBlueprints);

	UE_API TSharedRef<SWidget> GenerateCompileOptionsMenu();
	UE_API void MakeSaveOnCompileSubMenu(FMenuBuilder& InMenu);
	UE_API void SetSaveOnCompileSetting(ERigVMSaveOnCompile NewSetting);
	UE_API bool IsSaveOnCompileOptionSet(ERigVMSaveOnCompile Option);
	UE_API void ToggleJumpToErrorNodeSetting();
	UE_API bool IsJumpToErrorNodeOptionSet();
	UE_API UEdGraphNode* FindNodeWithError(UBlueprint* Blueprint, EMessageSeverity::Type Severity);
	UE_API UEdGraphNode* FindNodeWithError(FCompilerResultsLog const& ErrorLog, EMessageSeverity::Type Severity);
	UE_API FText GetCompileStatusTooltip() const;
	UE_API FSlateIcon GetCompileStatusImage() const;

public:
	static UE_API const FSlateBrush* GetGlyphForGraph(const UEdGraph* Graph, bool bInLargeIcon);

	static UE_API const FName SelectionState_GraphExplorer();
	static UE_API const FName SelectionState_Graph();
	static UE_API const FName SelectionState_ClassSettings();
	static UE_API const FName SelectionState_ClassDefaults();

	virtual FNotifyHook* GetNotifyHook() override { return this; }

	UE_API void OnSelectedNodesChanged(const FGraphPanelSelectionSet& NewSelection);

	UE_API void OnAlignTop();
	UE_API void OnAlignMiddle();
	UE_API void OnAlignBottom();
	UE_API void OnAlignLeft();
	UE_API void OnAlignCenter();
	UE_API void OnAlignRight();
	UE_API void OnStraightenConnections();
	UE_API void OnDistributeNodesH();
	UE_API void OnDistributeNodesV();
	UE_API void SelectAllNodes();
	UE_API bool CanSelectAllNodes() const;

protected:
	
	TSharedPtr<FDocumentTracker> DocumentManager;

	/** Node inspector widget */
	TSharedPtr<class SRigVMDetailsInspector> Inspector;

	/** Currently focused graph editor */
	TWeakPtr<class SGraphEditor> FocusedGraphEdPtr;

	/** Factory that spawns graph editors; used to look up all tabs spawned by it. */
	TWeakPtr<FDocumentTabFactory> GraphEditorTabFactoryPtr;

	/** The current UI selection state of this editor */
	FName CurrentUISelection;

	/** Command list for the graph editor */
	TSharedPtr<FUICommandList> GraphEditorCommands;

	/** Compiler results log, with the log listing that it reflects */
	TSharedPtr<class SWidget> CompilerResults;
	TSharedPtr<class IMessageLogListing> CompilerResultsListing;

	/** Find results log as well as the search filter */
	TSharedPtr<class SRigVMFindReferences> FindResults;

	/** When set, flags which graph has a action menu currently open (if null, no graphs do). */
	UEdGraph* HasOpenActionMenu;
	
	/** Used to nicely fade instruction text, when the context menu is opened. */
	float InstructionsFadeCountdown;

	/** defaults inspector widget */
	TSharedPtr<class SRigVMDetailsInspector> DefaultEditor;
	
	/** True if the editor was opened in defaults mode */
	bool bWasOpenedInDefaultsMode;

	/** Did we update the blueprint when it opened */
	bool bBlueprintModifiedOnOpen;

	/** Whether the graph action menu should be sensitive to the pins dragged off of */
	bool bIsActionMenuContextSensitive;

	FName ElementInDetailsPanel = NAME_None;

	/** Last frame in which the graph was double clicked. Useful to filter excess double clicks in a single frame */
	uint64 LastDoubleClickFrame = TNumericLimits<uint64>::Max();

	UE_API void HandlePreVariablesChanged(FName InName);
	UE_API void HandlePostVariablesChanged(FName InName);
};


#undef UE_API
