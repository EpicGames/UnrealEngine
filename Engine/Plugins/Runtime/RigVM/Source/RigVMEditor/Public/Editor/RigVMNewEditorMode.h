// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#define UE_API RIGVMEDITOR_API

class FRigVMNewEditor;

struct FRigVMNewEditorApplicationModes
{
	// Mode identifiers
	static UE_API const FName StandardRigVMEditorMode();
	static UE_API const FName RigVMDefaultsMode();
	
private:
	FRigVMNewEditorApplicationModes() {}
};

class FRigVMNewEditorMode : public FApplicationMode
{
public:
	UE_API FRigVMNewEditorMode(const TSharedRef<FRigVMNewEditor>& InRigVMEditor); 

	// FApplicationMode interface
	UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

	UE_API void RegisterToolbarTab(const TSharedRef<class FTabManager>& TabManager);

	UE_API virtual void PostActivateMode() override;
	UE_API virtual void PreDeactivateMode() override;

protected:
	// Set of spawnable tabs
	FWorkflowAllowedTabSet TabFactories;

	TWeakPtr<FRigVMNewEditor> Editor;
};

#undef UE_API
