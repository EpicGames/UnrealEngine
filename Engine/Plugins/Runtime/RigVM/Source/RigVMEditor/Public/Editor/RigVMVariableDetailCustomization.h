// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "IDetailCustomization.h"
#include "RigVMEditorAsset.h"
#include "Styling/SlateTypes.h"
#include "UObject/WeakFieldPtr.h"
#include "Layout/Visibility.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SListView.h"

#define UE_API RIGVMEDITOR_API

class SRigVMEditorGraphExplorer;
class IDetailLayoutBuilder;
class IBlueprintEditor;
class UBlueprint;
class IRigVMEditor;
class FRigVMVariableDetailCustomization;
class UK2Node_EditablePinBase;
class FStructOnScope;
class SEditableTextBox;
class SComboButton;

class FRigVMVariableDetailCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static UE_API TSharedRef<IDetailCustomization> MakeInstance(TSharedPtr<IRigVMEditor> InEditor);
	UE_API FRigVMVariableDetailCustomization(TSharedPtr<IRigVMEditor> InEditor, FRigVMEditorAssetInterfacePtr Blueprint);

#if WITH_RIGVMLEGACYEDITOR
	static UE_API TSharedPtr<IDetailCustomization> MakeLegacyInstance(TSharedPtr<IBlueprintEditor> InBlueprintEditor);
	UE_API FRigVMVariableDetailCustomization(TSharedPtr<IBlueprintEditor> RigVMigEditor, UBlueprint* Blueprint);
#endif

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	UE_API void PopulateCategories();
	UE_API void OnFinishedChangingVariable(const FPropertyChangedEvent& PropertyChangedEvent);

private:
	/** Accessors passed to parent */
	UE_API FName GetVariableName() const;
	UE_API FText OnGetVariableName() const;
	UE_API void OnVarNameCommitted(const FText& InNewName, ETextCommit::Type InTextCommit);

	// Callbacks for uproperty details customization
	UE_API FEdGraphPinType OnGetVarType() const;
	UE_API void OnVarTypeChanged(const FEdGraphPinType& NewPinType);

	UE_API void OnBrowseToVarType() const;
	UE_API bool CanBrowseToVarType() const;

	
	UE_API FText OnGetTooltipText() const;
	UE_API void OnTooltipTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	UE_API EVisibility IsToolTipVisible() const;
	
	UE_API FText OnGetCategoryText() const;
	UE_API void OnCategoryTextCommitted(const FText& NewText, ETextCommit::Type InTextCommit, FName VarName);
	UE_API TSharedRef< ITableRow > MakeCategoryViewWidget( TSharedPtr<FText> Item, const TSharedRef< STableViewBase >& OwnerTable );
	UE_API void OnCategorySelectionChanged( TSharedPtr<FText> ProposedSelection, ESelectInfo::Type /*SelectInfo*/ );
	

	UE_API ECheckBoxState OnGetExposedToSpawnCheckboxState() const;
	UE_API void OnExposedToSpawnChanged(ECheckBoxState InNewState);

	UE_API ECheckBoxState OnGetPublicCheckboxState() const;
	UE_API void OnPublicChanged(ECheckBoxState InNewState);
	
	UE_API ECheckBoxState OnGetExposedToCinematicsCheckboxState() const;
	UE_API void OnExposedToCinematicsChanged(ECheckBoxState InNewState);


	UE_API FText OnGetMetaKeyValue(FName Key) const;
	UE_API void OnMetaKeyValueChanged(const FText& NewMinValue, ETextCommit::Type CommitInfo, FName Key);
	UE_API EVisibility RangeVisibility() const;


	
	/** Refreshes cached data that changes after a Blueprint recompile */
	UE_API void OnPostEditorRefresh();

private:
	/** The Blueprint editor we are embedded in */
	TWeakPtr<IRigVMEditor> EditorPtr;

	/** The blueprint we are editing */
	TWeakInterfacePtr<IRigVMEditorAssetInterface> BlueprintPtr;

	/** The widget used when in variable name editing mode */ 
	TSharedPtr<SEditableTextBox> VarNameEditableTextBox;
	
	/** A list of all category names to choose from */
	TArray<TSharedPtr<FText>> CategorySource;
	/** Widgets for the categories */
	TWeakPtr<SComboButton> CategoryComboButton;
	TWeakPtr<SListView<TSharedPtr<FText>>> CategoryListView;

	/** Cached property for the variable we are affecting */
	TWeakFieldPtr<const FProperty> CachedVariableProperty;

	/** Cached name for the variable we are affecting */
	mutable FName CachedVariableName;
};

#undef UE_API
