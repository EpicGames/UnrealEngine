// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMCore/RigVMVariant.h"
#include "Widgets/SRigVMLogWidget.h"

class IPropertyHandle;

class FRigVMVariantDetailCustomization : public IPropertyTypeCustomization
{
public:

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FRigVMVariantDetailCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	FRigVMVariant GetVariant() const;
	FRigVMVariantRef GetSubjectVariantRef() const;
	TArray<FRigVMVariantRef> GetVariantRefs() const;
	
protected:

	void OnVariantChanged(const FRigVMVariant& InNewVariant);
	void OnBrowseVariantRef(const FRigVMVariantRef& InVariantRef);
	TArray<FRigVMTag> OnGetTags() const;
	void OnAddTag(const FName& InTagName);
	void OnRemoveTag(const FName& InTagName);

	FRigVMEditorAssetInterfacePtr BlueprintBeingCustomized = nullptr;

	/** The log widget used for function variants */
	TSharedPtr<SRigVMLogWidget> VariantLog;
};
