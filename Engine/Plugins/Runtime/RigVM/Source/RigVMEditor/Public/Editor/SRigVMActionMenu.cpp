// Copyright Epic Games, Inc. All Rights Reserved.


#include "SRigVMActionMenu.h"

#include "Editor/RigVMEditor.h"
#include "IDocumentation.h"
#include "RigVMNewEditor.h"
#include "SPinTypeSelector.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Editor/RigVMActionMenuUtils.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintNodeTemplateCache.h"
#include "K2Node_Variable.h"
#include "RigVMActionMenuItem.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMBlueprintUtils.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SToolTip.h"
#include "RigVMDeveloperTypeUtils.h"

#define LOCTEXT_NAMESPACE "SRigVMGraphContextMenu"

/** Action to promote a pin to a variable */
USTRUCT()
struct FRigVMAction_PromoteVariable : public FEdGraphSchemaAction
{
	FRigVMAction_PromoteVariable(bool bInToMemberVariable)
		: FEdGraphSchemaAction(	FText(),
								bInToMemberVariable? LOCTEXT("PromoteToVariable", "Promote to variable") : LOCTEXT("PromoteToLocalVariable", "Promote to local variable"),
								bInToMemberVariable ? LOCTEXT("PromoteToVariable", "Promote to variable") : LOCTEXT("PromoteToLocalVariable", "Promote to local variable"),
								1)
		, bToMemberVariable(bInToMemberVariable)
	{
	}

	// FEdGraphSchemaAction interface
	virtual UEdGraphNode* PerformAction( class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override
	{
		if( ( ParentGraph != NULL ) && ( FromPin != NULL ) )
		{
			if (FRigVMEditorAssetInterfacePtr RigVMBlueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph))
			{
				if (URigVMController* Controller = RigVMBlueprint->GetController(ParentGraph))
				{
					if(const URigVMEdGraphNode* RigNode = Cast<URigVMEdGraphNode>(FromPin->GetOwningNode()))
					{
						if(const URigVMPin* ModelPin = RigNode->GetModelPinFromPinPath(FromPin->GetName()))
						{
							Controller->PromotePinToVariable(ModelPin->GetPinPath(), true, FDeprecateSlateVector2D(Location), true, true);
						}
					}
				}
			}
		}
		return NULL;
	}
	// End of FEdGraphSchemaAction interface

	/* Pointer to the blueprint editor containing the blueprint in which we will promote the variable. */
	TWeakPtr<class IRigVMEditor> MyBlueprintEditor;

	/* TRUE if promoting to member variable, FALSE if promoting to local variable */
	bool bToMemberVariable;
};


/*******************************************************************************
* SRigVMActionMenu
*******************************************************************************/

SRigVMActionMenu::~SRigVMActionMenu()
{
	OnClosedCallback.ExecuteIfBound();
	
}

UEdGraphNode* FRigVMGraphSchemaAction_NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode)
{
	if (URigVMEdGraph* RigVMEdGraph = Cast<URigVMEdGraph>(ParentGraph))
	{
		if (URigVMEdGraphNode* NewNode = NodeSpawner->Invoke(RigVMEdGraph, FVector2D(Location)))
		{
			if (FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph))
			{
				return NewNode;
			}
			
			FRigVMEditorAssetInterfacePtr Blueprint = FRigVMBlueprintUtils::FindAssetForGraph(ParentGraph);
			URigVMController* Controller = Blueprint->GetController(ParentGraph);
			if (FromPin)
			{
				RigVMEdGraph->GetRigVMEdGraphSchema()->AutowireNewNode(NewNode, FromPin);
			}

			if (bSelectNewNode)
			{
				Controller->SelectNode(NewNode->GetModelNode(), true);
			}
		}
	}

	return nullptr;
}

void SRigVMActionMenu::Construct( const FArguments& InArgs, TSharedPtr<IRigVMEditor> InEditor )
{
	bActionExecuted = false;

	this->GraphObj = InArgs._GraphObj;
	this->DraggedFromPins = InArgs._DraggedFromPins;
	this->NewNodePosition = InArgs._NewNodePosition;
	this->OnClosedCallback = InArgs._OnClosedCallback;
	this->EditorPtr = InEditor;

	// Generate the context display; showing the user what they're picking something for
	//@TODO: Should probably be somewhere more schema-sensitive than the graph panel!
	FSlateColor TypeColor;
	FString TypeOfDisplay;
	const FSlateBrush* ContextIcon = nullptr;

	if (DraggedFromPins.Num() == 1)
	{
		UEdGraphPin* OnePin = DraggedFromPins[0];

		const UEdGraphSchema* Schema = OnePin->GetSchema();
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (!Schema->IsA(UEdGraphSchema_K2::StaticClass()) || !K2Schema->IsExecPin(*OnePin))
		{
			// Get the type color and icon
			TypeColor = Schema->GetPinTypeColor(OnePin->PinType);
			ContextIcon = FAppStyle::GetBrush( OnePin->PinType.IsArray() ? TEXT("Graph.ArrayPin.Connected") : TEXT("Graph.Pin.Connected") );
		}
	}

	// Build the widget layout
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		.Padding(5.0f)
		[
			// Achieving fixed width by nesting items within a fixed width box.
			SNew(SBox)
			.WidthOverride(400.0f)
			.HeightOverride(400.0f)
			[
				SNew(SVerticalBox)

				// TYPE OF SEARCH INDICATOR
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 2, 2, 5)
				[
					SNew(SHorizontalBox)

					// Search context description
					+SHorizontalBox::Slot()
					.FillWidth(1.f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &SRigVMActionMenu::GetSearchContextDesc)
						.Font(FAppStyle::GetFontStyle(FName("RigVMEditor.ActionMenu.ContextDescriptionFont")))
						.ToolTip(IDocumentation::Get()->CreateToolTip(
							LOCTEXT("RigVMActionMenuContextTextTooltip", "Describes the current context of the action list"),
							NULL,
							TEXT("Shared/Editors/RigVMEditor"),
							TEXT("RigVMActionMenuContextText")))
						.AutoWrapText(true)
					]

					// Context Toggle
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(this, &SRigVMActionMenu::OnContextToggleChanged)
						.IsChecked(this, &SRigVMActionMenu::ContextToggleIsChecked)
						.ToolTip(IDocumentation::Get()->CreateToolTip(
							LOCTEXT("RigVMActionMenuContextToggleTooltip", "Should the list be filtered to only actions that make sense in the current context?"),
							NULL,
							TEXT("Shared/Editors/RigVMEditor"),
							TEXT("RigVMActionMenuContextToggle")))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("RigVMActionMenuContextToggle", "Context Sensitive"))
						]
					]
				]

				// ACTION LIST
				+SVerticalBox::Slot()
				[
					SAssignNew(GraphActionMenu, SGraphActionMenu)
						.OnActionSelected(this, &SRigVMActionMenu::OnActionSelected)
						.OnActionDoubleClicked(this, &SRigVMActionMenu::OnActionSelected, ESelectInfo::OnMouseClick)
						.OnCreateWidgetForAction(SGraphActionMenu::FOnCreateWidgetForAction::CreateSP(this, &SRigVMActionMenu::OnCreateWidgetForAction))
						.OnGetActionList(this, &SRigVMActionMenu::OnGetActionList)
						.DraggedFromPins(DraggedFromPins)
						.GraphObj(GraphObj)
						
				]

				// PROGRESS BAR
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.HeightOverride(2.f)
					.Visibility_Lambda([this]()
					{
						return ContextMenuBuilder.IsValid() && ContextMenuBuilder->GetNumPendingActions() > 0 ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
					})
					[
						SNew(SProgressBar)
						.BorderPadding(FVector2D(0, 0))
						.Percent_Lambda([this]()
						{
							return ContextMenuBuilder.IsValid() ? ContextMenuBuilder->GetPendingActionsProgress() : 0.0f;
						})
					]
				]
			]
		]
	);
}

FText SRigVMActionMenu::GetSearchContextDesc() const
{
	bool bIsContextSensitive = EditorPtr.Pin()->GetIsContextSensitive();
	bool bHasPins = DraggedFromPins.Num() > 0;
	if (!bIsContextSensitive)
	{
		return LOCTEXT("MenuPrompt_AllPins", "All Possible Actions");
	}
	else if (!bHasPins)
	{
		return LOCTEXT("MenuPrompt_BlueprintActions", "All Actions for this Blueprint");
	}
	else if (DraggedFromPins.Num() == 1)
	{
		UEdGraphPin* OnePin = DraggedFromPins[0];

		const UEdGraphSchema* Schema = OnePin->GetSchema();
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (Schema->IsA(UEdGraphSchema_K2::StaticClass()) && K2Schema->IsExecPin(*OnePin))
		{
			return LOCTEXT("MenuPrompt_ExecPin", "Executable actions");
		}
		else
		{
			// Get the type string
			const FString TypeStringRaw = UEdGraphSchema_K2::TypeToText(OnePin->PinType).ToString();

			//@TODO: Add a parameter to TypeToText indicating the kind of formating requested
			const FString TypeString = (TypeStringRaw.Replace(TEXT("'"), TEXT(" "))).TrimEnd();

			if (OnePin->Direction == EGPD_Input)
			{
				return FText::Format(LOCTEXT("MenuPrompt_InputPin", "Actions providing a(n) {0}"), FText::FromString(TypeString));
			}
			else
			{
				return FText::Format(LOCTEXT("MenuPrompt_OutputPin", "Actions taking a(n) {0}"), FText::FromString(TypeString));
			}
		}
	}
	else
	{
		return FText::Format(LOCTEXT("MenuPrompt_ManyPins", "Actions for {0} pins"), FText::AsNumber(DraggedFromPins.Num()));
	}
}

void SRigVMActionMenu::OnContextToggleChanged(ECheckBoxState CheckState)
{
	EditorPtr.Pin()->SetIsContextSensitive(CheckState == ECheckBoxState::Checked);
	GraphActionMenu->RefreshAllActions(true, false);
}

ECheckBoxState SRigVMActionMenu::ContextToggleIsChecked() const
{
	return EditorPtr.Pin()->GetIsContextSensitive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<FGraphActionListBuilderBase> SRigVMActionMenu::OnGetActionList()
{
	check(EditorPtr.IsValid());
	TSharedPtr<IRigVMEditor> BlueprintEditor = EditorPtr.Pin();
	bool const bIsContextSensitive = BlueprintEditor->GetIsContextSensitive();

	uint32 ContextTargetMask = 0;
	FBlueprintActionContext FilterContext;
	ConstructActionContext(FilterContext);

	FRigVMActionMenuBuilder::EConfigFlags ConfigFlags = FRigVMActionMenuBuilder::DefaultConfig;
	if (GetDefault<URigVMEditorSettings>()->bEnableContextMenuTimeSlicing)
	{
		ConfigFlags |= FRigVMActionMenuBuilder::UseTimeSlicing;
	}

	ContextMenuBuilder = MakeShared<FRigVMActionMenuBuilder>(ConfigFlags);
	FRigVMEditorAssetInterfacePtr Asset = BlueprintEditor->GetRigVMAssetInterface();
	if (IRigVMEditorModule* RigVMEditorModule = Asset->GetEditorModule())
	{
		TArray<TSharedRef<URigVMEdGraphNodeSpawner>> NodeSpawners;
		RigVMEditorModule->GetTypeActions(Asset, NodeSpawners);
		RigVMEditorModule->GetInstanceActions(Asset, NodeSpawners);
		
		for (TSharedRef<URigVMEdGraphNodeSpawner>& NodeSpawner : NodeSpawners)
		{
			TSharedPtr<FRigVMGraphSchemaAction_NewNode> NewNodeAction(new FRigVMGraphSchemaAction_NewNode(
					NodeSpawner->GetMenuCategory(),
					NodeSpawner->GetMenuName(),
					NodeSpawner->GetMenuTooltip(),
					0
				));
			NewNodeAction->NodeSpawner = NodeSpawner;

			ContextMenuBuilder->AddAction(NewNodeAction);
		}
	}

	// NOTE: cannot call GetGraphContextActions() during serialization and GC due to its use of FindObject()
	if (!UE::IsSavingPackage() && !IsGarbageCollecting() && FilterContext.Blueprints.Num() > 0)
	{
		FRigVMActionMenuUtils::MakeContextMenu(FilterContext, bIsContextSensitive, ContextTargetMask, *ContextMenuBuilder);
	}

	// also try adding promote to variable if we can do so.
	//TryInsertPromoteToVariable(FilterContext, *ContextMenuBuilder);

	// give the schema the opportunity to add another action
	if (const URigVMEdGraphSchema* Schema = Cast<const URigVMEdGraphSchema>(GraphObj->GetSchema()))
	{
		Schema->InsertAdditionalActions({Asset}, FilterContext.Graphs, FilterContext.Pins, *ContextMenuBuilder);
	}

	return ContextMenuBuilder.ToSharedRef();
}

void SRigVMActionMenu::ConstructActionContext(FBlueprintActionContext& ContextDescOut)
{
	check(EditorPtr.IsValid());
	TSharedPtr<IRigVMEditor> BlueprintEditor = EditorPtr.Pin();
	bool const bIsContextSensitive = BlueprintEditor->GetIsContextSensitive();

	// we still want context from the graph (even if the user has unchecked
	// "Context Sensitive"), otherwise the user would be presented with nodes
	// that can't be placed in the graph... if the user isn't being presented
	// with a valid node, then fix it up in filtering
	ContextDescOut.Graphs.Add(GraphObj);

	if (UBlueprint* Blueprint = Cast<UBlueprint>(BlueprintEditor->GetRigVMAssetInterface().GetObject()))
	{
		const bool bBlueprintIsValid = IsValid(Blueprint) && Blueprint->GeneratedClass && (Blueprint->GeneratedClass->ClassGeneratedBy == Blueprint);
		if (!ensure(bBlueprintIsValid))  // to track UE-11597 and UE-11595
		{
			return;
		}

		//ContextDescOut.EditorPtr = EditorPtr;
		ContextDescOut.Blueprints.Add(Blueprint);
	}

	if (bIsContextSensitive)
	{
		ContextDescOut.Pins = DraggedFromPins;
	}
}

TSharedRef<SEditableTextBox> SRigVMActionMenu::GetFilterTextBox()
{
	return GraphActionMenu->GetFilterTextBox();
}


TSharedRef<SWidget> SRigVMActionMenu::OnCreateWidgetForAction(FCreateWidgetForActionData* const InCreateData)
{
	//InCreateData->bHandleMouseButtonDown = true;

	TSharedPtr<FEdGraphSchemaAction> GraphAction = InCreateData->Action;
	TWeakPtr<FEdGraphSchemaAction> WeakGraphAction = GraphAction;
	
	// construct the icon widget
	FSlateBrush const* IconBrush   = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateBrush const* SecondaryBrush = FAppStyle::GetBrush(TEXT("NoBrush"));
	FSlateColor        IconColor   = FSlateColor::UseForeground();
	FSlateColor        SecondaryIconColor   = FSlateColor::UseForeground();
	FText			   IconToolTip = GraphAction->GetTooltipDescription();
	FString			   IconDocLink, IconDocExcerpt;
	
	// Get Palette Item Icon
	{
		// Default to tooltip based on action supplied
		IconToolTip = GraphAction->GetTooltipDescription().IsEmpty() ? GraphAction->GetMenuDescription() : GraphAction->GetTooltipDescription();

		if (GraphAction->GetTypeId() == FRigVMActionMenuItem::StaticGetTypeId())
		{
			FRigVMActionMenuItem* NodeSpawnerAction = (FRigVMActionMenuItem*)GraphAction.Get();
			IconBrush = NodeSpawnerAction->GetMenuIcon(IconColor);

			TSubclassOf<UEdGraphNode> VarNodeClass = NodeSpawnerAction->GetRawAction()->NodeClass;

			// if the node is a variable getter or setter, use the variable icon instead, because maps need two brushes
			if (*VarNodeClass && VarNodeClass->IsChildOf(UK2Node_Variable::StaticClass()))
			{
				const UK2Node_Variable* TemplateNode = Cast<UK2Node_Variable>(NodeSpawnerAction->GetRawAction()->GetTemplateNode());
				FProperty* Property = TemplateNode->GetPropertyForVariable();
				IconBrush = FRigVMEditorBase::GetVarIconAndColorFromProperty(Property, IconColor, SecondaryBrush, SecondaryIconColor);
			}
		}
		else if (GraphAction->GetTypeId() == FRigVMGraphSchemaAction_NewNode::StaticGetTypeId())
		{
			FRigVMGraphSchemaAction_NewNode* NodeSpawnerAction = (FRigVMGraphSchemaAction_NewNode*)GraphAction.Get();
			IconBrush = NodeSpawnerAction->NodeSpawner->GetMenuIcon().GetOptionalIcon();
		}
	}
	TSharedRef<SWidget> IconWidget = SPinTypeSelector::ConstructPinTypeImage(
		IconBrush, 
		IconColor, 
		SecondaryBrush, 
		SecondaryIconColor, 
		IDocumentation::Get()->CreateToolTip(IconToolTip, NULL, IconDocLink, IconDocExcerpt));
	//IconWidget->SetEnabled(bIsEditingEnabled);

	auto ConstructToolTipWidget = [&]()
	{
		TSharedPtr<FEdGraphSchemaAction> PaletteAction = GraphAction;
		UEdGraphNode const* const NodeTemplate = FRigVMActionMenuUtils::ExtractNodeTemplateFromAction(PaletteAction);

		//FRigVMActionMenuItem::FDocExcerptRef DocExcerptRef;
		FString DocExcerptLink;
		FString DocExcerptName;
		FText NodeToolTipText = PaletteAction->GetTooltipDescription().IsEmpty() ? PaletteAction->GetMenuDescription() : PaletteAction->GetTooltipDescription();

		if (PaletteAction.IsValid())
		{
			if (NodeTemplate != nullptr)
			{
				// Take rich tooltip from node
				DocExcerptLink = NodeTemplate->GetDocumentationLink();
				DocExcerptName = NodeTemplate->GetDocumentationExcerptName();
				NodeToolTipText = NodeTemplate->GetTooltipText();
			}
		}

		// If the node wants to create tooltip text, use that instead, because its probably more detailed
		return IDocumentation::Get()->CreateToolTip(NodeToolTipText, nullptr, DocExcerptLink, DocExcerptName);
	};
	TSharedRef<SToolTip> ToolTipWidget = ConstructToolTipWidget();
	
	auto CreateTextSlotWidget = [&]() -> TSharedRef<SWidget>
	{

		TSharedPtr<SOverlay> DisplayWidget;
		TSharedPtr<SInlineEditableTextBlock> EditableTextElement;
		FText MenuDesc = GraphAction->GetMenuDescription();
		SAssignNew(DisplayWidget, SOverlay)
			+SOverlay::Slot()
			[
				SAssignNew(EditableTextElement, SInlineEditableTextBlock)
					.Text_Lambda([MenuDesc](){return MenuDesc;})
					.HighlightText(InCreateData->HighlightText)
					.ToolTip(ToolTipWidget)
					//.IsSelected(InCreateData->IsRowSelectedDelegate)
					.IsReadOnly(true)
			];
		return DisplayWidget.ToSharedRef();
	};

	// construct the text widget
	TSharedRef<SWidget> NameSlotWidget = CreateTextSlotWidget( );
	
	// Create the widget with an icon
	TSharedRef<SHorizontalBox> ActionBox = SNew(SHorizontalBox);
	{
		ActionBox.Get().AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				IconWidget
			];

		ActionBox.Get().AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(/* horizontal */ 3.0f, /* vertical */ 3.0f)
			[
				NameSlotWidget
			];
	}

	// Now, create the actual widget
	return ActionBox;
}

void SRigVMActionMenu::OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType )
{
	if (InSelectionType == ESelectInfo::OnMouseClick  || InSelectionType == ESelectInfo::OnKeyPress || SelectedAction.Num() == 0)
	{
		for ( int32 ActionIndex = 0; ActionIndex < SelectedAction.Num(); ActionIndex++ )
		{
			if ( SelectedAction[ActionIndex].IsValid() && GraphObj != nullptr )
			{
				// Don't dismiss when clicking on dummy action
				if ( !bActionExecuted && (SelectedAction[ActionIndex]->GetTypeId() != FEdGraphSchemaAction_Dummy::StaticGetTypeId()))
				{
					FSlateApplication::Get().DismissAllMenus();
					bActionExecuted = true;
				}

				UEdGraphNode* ResultNode = SelectedAction[ActionIndex]->PerformAction(GraphObj, DraggedFromPins, NewNodePosition);

				if ( ResultNode != nullptr )
				{
					NewNodePosition.Y += UEdGraphSchema_K2::EstimateNodeHeight( ResultNode );
				}
			}
		}
	}
}

void SRigVMActionMenu::TryInsertPromoteToVariable(FBlueprintActionContext const& MenuContext, FGraphActionListBuilderBase& OutAllActions)
{
	// If we can promote this to a variable add a menu entry to do so.
	const URigVMEdGraphSchema* Schema = Cast<const URigVMEdGraphSchema>(GraphObj->GetSchema());
	if ((Schema != nullptr) && (MenuContext.Pins.Num() > 0))
	{
		if (Schema->CanPromotePinToVariable(*MenuContext.Pins[0], true))
		{
			TSharedPtr<FRigVMAction_PromoteVariable> PromoteAction = TSharedPtr<FRigVMAction_PromoteVariable>(new FRigVMAction_PromoteVariable(true));
			PromoteAction->MyBlueprintEditor = EditorPtr;
			OutAllActions.AddAction(PromoteAction);
		}

		if (MenuContext.Graphs.Num() == 1 && FBlueprintEditorUtils::DoesSupportLocalVariables(MenuContext.Graphs[0]) && Schema->CanPromotePinToVariable(*MenuContext.Pins[0], false))
		{
			TSharedPtr<FRigVMAction_PromoteVariable> LocalPromoteAction = TSharedPtr<FRigVMAction_PromoteVariable>(new FRigVMAction_PromoteVariable(false));
			LocalPromoteAction->MyBlueprintEditor = EditorPtr;
			OutAllActions.AddAction( LocalPromoteAction );
		}
	}
}

void SRigVMActionMenu::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SBorder::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	int32 NewIdxStart = ContextMenuBuilder->GetNumActions();
	if (ContextMenuBuilder.IsValid() && ContextMenuBuilder->ProcessPendingActions())
	{
		GraphActionMenu->UpdateForNewActions(NewIdxStart);
	}
}

#undef LOCTEXT_NAMESPACE
