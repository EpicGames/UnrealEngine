// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "RigVMActionMenuItem.h"
#include "Editor/RigVMActionMenuBuilder.h"
#include "SGraphActionMenu.h"
#include "EdGraph/NodeSpawners/RigVMEdGraphNodeSpawner.h"
#include "GraphEditor.h"

#include "SRigVMActionMenu.generated.h"

class SGraphActionMenu;
class IRigVMEditor;

/// Action to add a new RigVM node to the graph
USTRUCT()
struct FRigVMGraphSchemaAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	// Inherit the base class's constructors
	using FEdGraphSchemaAction::FEdGraphSchemaAction;

	TSharedPtr<URigVMEdGraphNodeSpawner> NodeSpawner;

	static FName StaticGetTypeId() { static FName Type("FRigVMGraphSchemaAction_NewNode"); return Type; }
	FName GetTypeId() const override { return StaticGetTypeId(); }

	// FEdGraphSchemaAction overrides
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
};

/*******************************************************************************
* SRigVMActionMenu
*******************************************************************************/

class SRigVMActionMenu : public SBorder
{
public:

	SLATE_BEGIN_ARGS( SRigVMActionMenu )
		: _GraphObj( nullptr )
		, _NewNodePosition( FVector2f::ZeroVector )
	{}

		SLATE_ARGUMENT( UEdGraph*, GraphObj )
		SLATE_ARGUMENT( FDeprecateSlateVector2D, NewNodePosition )
		SLATE_ARGUMENT( TArray<UEdGraphPin*>, DraggedFromPins )
		SLATE_ARGUMENT( SGraphEditor::FActionMenuClosed, OnClosedCallback )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, TSharedPtr<IRigVMEditor> InEditor );

	~SRigVMActionMenu();

	TSharedRef<SEditableTextBox> GetFilterTextBox();

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

protected:
	/** UI Callback functions */
	FText GetSearchContextDesc() const;
	void OnContextToggleChanged(ECheckBoxState CheckState);
	ECheckBoxState ContextToggleIsChecked() const;

	void OnActionSelected( const TArray< TSharedPtr<FEdGraphSchemaAction> >& SelectedAction, ESelectInfo::Type InSelectionType );

	TSharedRef<SWidget> OnCreateWidgetForAction(struct FCreateWidgetForActionData* const InCreateData);

	/** Callback used to populate all actions list in SGraphActionMenu */
	TSharedRef<FGraphActionListBuilderBase> OnGetActionList();

	/**  */
	void ConstructActionContext(FBlueprintActionContext& ContextDescOut);

	/** Function to try to insert a promote to variable entry if it is possible to do so. */
	void TryInsertPromoteToVariable(FBlueprintActionContext const& Context, FGraphActionListBuilderBase& OutAllActions);


private:
	TObjectPtr<UEdGraph> GraphObj = nullptr;
	TArray<UEdGraphPin*> DraggedFromPins;
	FDeprecateSlateVector2D NewNodePosition;
	SGraphEditor::FActionMenuClosed OnClosedCallback;

	TSharedPtr<SGraphActionMenu> GraphActionMenu;
	TWeakPtr<IRigVMEditor> EditorPtr;
	TSharedPtr<FRigVMActionMenuBuilder> ContextMenuBuilder;

	bool bActionExecuted = false;
};
