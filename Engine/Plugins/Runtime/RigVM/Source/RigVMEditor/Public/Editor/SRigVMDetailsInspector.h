// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "PropertyEditorDelegates.h"
#include "RigVMEditorAsset.h"
#include "SlateFwd.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditor;
class FProperty;
class FStructOnScope;
class IDetailsView;
class SBorder;
class SDockTab;
class SRigVMEditorGraphExplorer;
class SWidget;
class SScrollBar;
class UObject;
struct FGeometry;
struct FPropertyChangedEvent;

typedef TSet<class UObject*> FInspectorSelectionSet;

//////////////////////////////////////////////////////////////////////////
// SRigVMDetailsInspector

/** Widget that shows properties and tools related to the selected node(s) */
class SRigVMDetailsInspector : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS( SRigVMDetailsInspector )
		: _ShowPublicViewControl(false)
		, _HideNameArea(true)
		, _SetNotifyHook(true)
		, _ShowTitleArea(false)
		, _ShowLocalVariables(false)
		, _ScrollbarAlignment(HAlign_Right)
		, _ShowSectionSelector(false)
		{}

		SLATE_ARGUMENT(TWeakPtr<IRigVMEditor>, Editor)
		SLATE_ARGUMENT(TWeakPtr<SRigVMEditorGraphExplorer>, GraphExplorerWidget)
		SLATE_ATTRIBUTE( bool, ShowPublicViewControl )
		SLATE_ARGUMENT( bool, HideNameArea )
		SLATE_ARGUMENT( FIsPropertyEditingEnabled, IsPropertyEditingEnabledDelegate )
		SLATE_ARGUMENT( FOnFinishedChangingProperties::FDelegate, OnFinishedChangingProperties )
		SLATE_ARGUMENT( FName, ViewIdentifier)
		SLATE_ARGUMENT( bool, SetNotifyHook)
		SLATE_ARGUMENT( bool, ShowTitleArea)
		SLATE_ARGUMENT( bool, ShowLocalVariables)
		SLATE_ARGUMENT( TSharedPtr<SScrollBar>, ExternalScrollbar)
		SLATE_ARGUMENT( EHorizontalAlignment, ScrollbarAlignment)
		SLATE_ARGUMENT( bool, ShowSectionSelector)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

	/** Options for ShowDetails */
	struct FShowDetailsOptions
	{
		FText ForcedTitle;
		bool bForceRefresh;
		bool bShowComponents;
		bool bHideFilterArea;

		FShowDetailsOptions()
			:ForcedTitle()
			,bForceRefresh(false)
			,bShowComponents(true)
			,bHideFilterArea(false)
		{}

		FShowDetailsOptions(const FText& InForcedTitle, bool bInForceRefresh = false)
			:ForcedTitle(InForcedTitle)
			,bForceRefresh(bInForceRefresh)
			,bShowComponents(true)
			,bHideFilterArea(false)
		{}
	};

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** Update the inspector window to show information on the supplied object */
	UE_API void ShowDetailsForSingleObject(UObject* Object, const FShowDetailsOptions& Options = FShowDetailsOptions());

	/** Update the inspector window to show information on the supplied objects */
	UE_API void ShowDetailsForObjects(const TArray<UObject*>& PropertyObjects, const FShowDetailsOptions& Options = FShowDetailsOptions());

	/** Update the inspector window to show single struct. This invalidates ShowDetailsForObjects */
	UE_API void ShowSingleStruct(TSharedPtr<FStructOnScope> InStructToDisplay);

	/** Used to control visibility of a property in the property window */
	UE_API bool IsPropertyVisible( const struct FPropertyAndParent& PropertyAndParent ) const;


	TSharedPtr<class IDetailsView> GetPropertyView() const { return PropertyView; }

	UE_API TSharedPtr<SDockTab> GetOwnerTab() const;
	UE_API void SetOwnerTab(TSharedRef<SDockTab> Tab);


	/** returns the list of selected objects */
	UE_API const TArray< TWeakObjectPtr<UObject> >& GetSelectedObjects() const;

	UE_API void OnEditorClose(const IRigVMEditor* RigVMEditorBase, FRigVMEditorAssetInterfacePtr RigVMBlueprint);

	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;

protected:
	/** Update the inspector window to show information on the supplied objects */
	UE_API void UpdateFromObjects(const TArray<UObject*>& PropertyObjects, TArray<UObject*>& SelectionInfo, const FShowDetailsOptions& Options);

	/** Add this property and all its child properties to SelectedObjectProperties */
	UE_API void AddPropertiesRecursive(FProperty* Property);

	/** Pointer back to the kismet 2 tool that owns us */
	TWeakPtr<IRigVMEditor> WeakEditor;

	/** The tab that owns this details view. */
	TWeakPtr<SDockTab> OwnerTab;

	/** String used as the title above the property window */
	FText PropertyViewTitle;

	/** Should we currently show the property view */
	bool bShowInspectorPropertyView;

	/** State of CheckBox representing whether to show only the public variables*/
	ECheckBoxState	PublicViewState;

	/** Property viewing widget */
	TSharedPtr<class IDetailsView> PropertyView;

	/** Selected objects for this detail view */
	TArray< TWeakObjectPtr<UObject> > SelectedObjects;

	/** Border widget that wraps a dynamic context-sensitive widget for editing objects that the property window is displaying */
	TSharedPtr<SBorder> ContextualEditingBorderWidget;

	/** If true show the public view control */
	TAttribute<bool> bShowPublicView;

	/** If true show the kismet inspector title widget */
	bool bShowTitleArea;

	/** Set of object properties that should be visible */
	TSet< TWeakFieldPtr<FProperty> > SelectedObjectProperties;
	
	/** User defined delegate for IsPropertyEditingEnabled: */
	FIsPropertyEditingEnabled IsPropertyEditingEnabledDelegate;

	/** User defined delegate for OnFinishedChangingProperties */
	FOnFinishedChangingProperties::FDelegate UserOnFinishedChangingProperties;

	/** When TRUE, the Kismet inspector needs to refresh the details view on Tick */
	bool bRefreshOnTick;

	/** Holds the property objects that need to be displayed by the inspector starting on the next tick */
	TArray<TObjectPtr<UObject>> RefreshPropertyObjects;

	/** Details options that are used by the inspector on the next refresh. */
	FShowDetailsOptions RefreshOptions;

	/** Struct to preview */
	TSharedPtr<FStructOnScope> StructToDisplay;

	/* Sturct Detail View */
	TSharedPtr<class IStructureDetailsView> StructureDetailsView;

	/** Update the inspector window to show information on the single struct */
	UE_API void UpdateFromSingleStruct(const TSharedPtr<FStructOnScope>& InStructToDisplay);

	/** Is struct view property read only */
	UE_API bool IsStructViewPropertyReadOnly(const struct FPropertyAndParent& PropertyAndParent) const;

protected:

	/** Returns whether the property view be visible */
	UE_API EVisibility GetPropertyViewVisibility() const;

	/** Returns whether the properties in the view should be editable */
	UE_API bool IsPropertyEditingEnabled() const;

	/**
	 * Generates a widget that is used to edit the specified object array contextually.  This widget
	 * will be displayed along with a property view in the level editor
	 */
	UE_API TSharedRef<SWidget> MakeContextualEditingWidget(TArray<UObject*>& SelectionInfo, const FShowDetailsOptions& Options);

	/**
	 * Generates the text for the title in the contextual editing widget
	 */
	UE_API FText GetContextualEditingWidgetTitle() const;

	UE_API ECheckBoxState GetPublicViewCheckboxState() const;
	UE_API void SetPublicViewCheckboxState(ECheckBoxState InIsChecked);

	UE_API bool IsAnyParentOrContainerSelected(const FPropertyAndParent& PropertyAndParent) const;

	/** Callback invoked after a value change on the selected object(s) */
	UE_API void OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent);

};

#undef UE_API
