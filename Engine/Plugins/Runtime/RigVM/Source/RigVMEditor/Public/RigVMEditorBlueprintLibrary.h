// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintLegacy.h"
#include "AssetRegistry/AssetData.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RigVMCore/RigVMGraphFunctionDefinition.h"
#include "RigVMEditorBlueprintLibrary.generated.h"

#define UE_API RIGVMEDITOR_API

class URigVMNode;
class URigVMEditorAsset;

UENUM(BlueprintType)
enum ERigVMBlueprintLoadLogSeverity : uint8
{
	Display,
	Warning,
	Error
};

USTRUCT(BlueprintType)
struct FRigVMBlueprintLoadLogEntry
{
	GENERATED_BODY()

	FRigVMBlueprintLoadLogEntry()
		: Severity(ERigVMBlueprintLoadLogSeverity::Display)
		, Subject(nullptr)
	{}

	FRigVMBlueprintLoadLogEntry(ERigVMBlueprintLoadLogSeverity InSeverity, UObject* InSubject, const FString& InMessage)
		: Severity(InSeverity)
		, Subject(InSubject)
		, Message(InMessage)
	{}
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Log)
	TEnumAsByte<ERigVMBlueprintLoadLogSeverity> Severity;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Log)
	TObjectPtr<UObject> Subject;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category=Log)
	FString Message;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMAssetDataFilter, const FAssetData&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMBlueprintFilter, const URigVMBlueprint*, const TArray<FRigVMBlueprintLoadLogEntry>&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMNodeFilter, const URigVMBlueprint*, const URigVMNode*);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(bool, FRigVMAssetDataFilterDynamic, FAssetData, AssetData);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FRigVMBlueprintFilterDynamic, const URigVMBlueprint*, Blueprint, TArray<FRigVMBlueprintLoadLogEntry>, LogDuringLoad);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FRigVMNodeFilterDynamic, const URigVMBlueprint*, Blueprint, const URigVMNode*, Node);

UCLASS(MinimalAPI, BlueprintType, meta=(ScriptName="RigVMBlueprintLibrary"))
class URigVMEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API void RecompileVM(URigVMBlueprint* InBlueprint);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API void RecompileVMIfRequired(URigVMBlueprint* InBlueprint);
	
	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API void RequestAutoVMRecompilation(URigVMBlueprint* InBlueprint);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "RigVM Blueprint")
	static UE_API URigVMGraph* GetModel(URigVMBlueprint* InBlueprint);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "RigVM Blueprint")
	static UE_API URigVMController* GetController(URigVMBlueprint* InBlueprint);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API TArray<URigVMBlueprint*> LoadAssets();

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API TArray<URigVMBlueprint*> LoadAssetsByClass(TSubclassOf<URigVMBlueprint> InClass);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "LoadAssetsWithBlueprintFilter", ScriptName = "LoadAssetsWithBlueprintFilter"))
		static UE_API TArray<URigVMBlueprint*> LoadAssetsWithBlueprintFilter_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMBlueprintFilterDynamic InBlueprintFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "LoadAssetsWithAssetDataFilter", ScriptName = "LoadAssetsWithAssetDataFilter"))
		static UE_API TArray<URigVMBlueprint*> LoadAssetsWithAssetDataFilter_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilterDynamic InAssetDataFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "LoadAssetsWithNodeFilter", ScriptName = "LoadAssetsWithNodeFilter"))
	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithNodeFilter_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMNodeFilterDynamic InNodeFilter);

	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithNodeFilter(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMNodeFilter InNodeFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "LoadAssetsWithAssetDataAndBlueprintFilters", ScriptName = "LoadAssetsWithAssetDataAndBlueprintFilters"))
	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithAssetDataAndBlueprintFilters_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilterDynamic InAssetDataFilter,
		FRigVMBlueprintFilterDynamic InBlueprintFilter);

	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithAssetDataAndBlueprintFilters(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilter InAssetDataFilter,
		FRigVMBlueprintFilter InBlueprintFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "LoadAssetsWithAssetDataAndNodeFilters", ScriptName = "LoadAssetsWithAssetDataAndNodeFilters"))
	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithAssetDataAndNodeFilters_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilterDynamic InAssetDataFilter,
		FRigVMNodeFilterDynamic InNodeFilter);

	static UE_API TArray<URigVMBlueprint*> LoadAssetsWithAssetDataAndNodeFilters(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilter InAssetDataFilter,
		FRigVMNodeFilter InNodeFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint", meta = (DisplayName = "GetAssetsWithFilter", ScriptName = "GetAssetsWithFilter"))
	static UE_API TArray<FAssetData> GetAssetsWithFilter_ForBlueprint(
		TSubclassOf<URigVMBlueprint> InClass,
		FRigVMAssetDataFilterDynamic InAssetDataFilter);

	static UE_API TArray<FAssetData> GetAssetsWithFilter(
		UClass* InClass,
		FRigVMAssetDataFilter InAssetDataFilter);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API bool RenderRigVMSubjectToPNG(UObject* InUnitOrDispatchStructOrLibraryNode, const FString& OutFileName);

	UFUNCTION(BlueprintCallable, Category = "RigVM Blueprint")
	static UE_API bool RenderRigVMTemplateToPNG(FName InTemplateNotation, const FString& OutFileName);

	static UE_API bool RenderModelNodeToPNG(URigVMEditorAsset* EditorAsset, URigVMNode* InModelNode, const FString& OutFileName);
	static UE_API UObject* MakeRigVMAssetForSubject(UObject* InUnitOrDispatchStructOrLibraryNode, const URigVMSchema *& OutSchema);
};

#undef UE_API
