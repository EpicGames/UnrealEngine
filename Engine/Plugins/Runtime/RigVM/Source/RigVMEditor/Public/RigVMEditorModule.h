// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RigVMEditor.h: Module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "IAssetTypeActions.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMEditorAsset.h"
#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "EdGraph/RigVMEdGraphPanelNodeFactory.h"
#include "EdGraph/RigVMEdGraphPanelPinFactory.h"
#include "EdGraph/RigVMEdGraphSchema.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Modules/ModuleManager.h"
#if RIGVM_TRACE_ENABLED
#include "RigVMTrace.h"
#include "Trace/RigVMTraceModule.h"
#endif

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNodeSpawner;
class URigVMEditorAssetInterface;
class IRigVMEditorAssetInterface;
class IRigVMEditorModule;
class URigVMUserWorkflowOptions;
class URigVMPin;
class URigVMNode;
class URigVMEdGraphNode;
class IRigVMClientHost;
class URigVMEdGraphSchema;
class FRigVMEditorBase;
class FBlueprintActionDatabaseRegistrar;

DECLARE_LOG_CATEGORY_EXTERN(LogRigVMEditor, Log, All);

extern RIGVMEDITOR_API TAutoConsoleVariable<bool> CVarRigVMUseNewEditor;
extern RIGVMEDITOR_API TAutoConsoleVariable<bool> CVarRigVMBlueprintIndependentAssets;

/** Describes the reason for Refreshing the editor */
namespace ERefreshRigVMEditorReason
{
	enum Type
	{
		BlueprintCompiled,
		PostUndo,
		UnknownReason
	};
}

namespace UE::RigVMEditor
{
	/** Params when finding references for a node */
	struct FRigVMEditorFindNodeReferencesParams
	{
		FRigVMEditorFindNodeReferencesParams(
			const TWeakInterfacePtr<IRigVMEditorAssetInterface> InRigVMAsset,
			const TWeakObjectPtr<const URigVMEdGraphNode> InEdGraphNode,
			const bool bInSearchInAllBlueprints)
			: WeakRigVMAsset(InRigVMAsset)
			, WeakEdGraphNode(InEdGraphNode)
			, bSearchInAllBlueprints(bInSearchInAllBlueprints)
		{}

		/** The asset that contains the node */
		const TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakRigVMAsset;

		/** The ed graph node */
		const TWeakObjectPtr<const URigVMEdGraphNode> WeakEdGraphNode;

		/** If true, searches in all blueprints */
		const bool bSearchInAllBlueprints;
	};

	DECLARE_EVENT_OneParam(IRigVMEditorModule, FRigVMEditorOnRequestFindNodeReferencesDelegate, FRigVMEditorFindNodeReferencesParams /** FindNodeReferencesParams */);
}

// shallow interface declaration for use within RigVMDeveloper 
class IRigVMEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility, public FStructureEditorUtils::INotifyOnStructChanged
{
public:

	static IRigVMEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IRigVMEditorModule >(TEXT("RigVMEditor"));
	}

	virtual void GetContextMenuActions(const URigVMEdGraphSchema* Schema, class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const = 0;
	virtual void GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) = 0;
	virtual void GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners) = 0;
	virtual void GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) = 0;
	virtual void GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners) = 0;
	virtual void GetNodeContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, class UToolMenu* Menu) const = 0;
	virtual void GetPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, class UToolMenu* Menu) const = 0;
	virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const = 0;
	virtual bool AssetsPublicFunctionsAllowed(const FAssetData& InAssetData) const = 0;
};

class FRigVMEditorModule : public IRigVMEditorModule
{
public:

	static UE_API FRigVMEditorModule& Get();

	/** IModuleInterface implementation */
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	UE_API void StartupModuleCommon();
	UE_API void ShutdownModuleCommon();

	/** RigVMEditorModule interface */
	UE_DEPRECATED(5.7, "Please update your code")
	virtual UClass* GetRigVMBlueprintClass() const { return nullptr; }
	UE_DEPRECATED(5.7, "Please update your code")
	const URigVMBlueprint* GetRigVMBlueprintCDO() const { return nullptr; }
	UE_API virtual void GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) override;
	UE_API virtual void GetTypeActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners) override;
	UE_API virtual void GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) override;
	UE_API virtual void GetInstanceActions(FRigVMEditorAssetInterfacePtr RigVMBlueprint, TArray<TSharedRef<URigVMEdGraphNodeSpawner>>& OutGraphNodeSpawners) override;
	UE_API virtual void GetNodeContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, class UToolMenu* Menu) const override;
	UE_API virtual void GetPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, class UToolMenu* Menu) const override;
	virtual bool AssetsPublicFunctionsAllowed(const FAssetData& InAssetData) const override { return true; }

	/** IHasMenuExtensibility interface */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }

	/** IHasToolBarExtensibility interface */
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	/* FStructureEditorUtils::INotifyOnStructChanged Interface, used to respond to changes to user defined structs */
	UE_API virtual void PreChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType) override;
	UE_API virtual void PostChange(const UUserDefinedStruct* Changed, FStructureEditorUtils::EStructureEditorChangeInfo ChangedType) override;

	/** Get all toolbar extenders */
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FRigVMEditorToolbarExtender, const TSharedRef<FUICommandList> /*InCommandList*/, TSharedRef<FRigVMEditorBase> /*InRigVMEditor*/);
	UE_API virtual TArray<FRigVMEditorToolbarExtender>& GetAllRigVMEditorToolbarExtenders();

	UE_API virtual void GetContextMenuActions(const URigVMEdGraphSchema* Schema, class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;

	/** Make sure to create the root graph for a given blueprint */
	UE_API void CreateRootGraphIfRequired(FRigVMEditorAssetInterfacePtr InBlueprint) const;

	UE_API virtual FConnectionDrawingPolicy* CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj) const override;

	/** Returns an event broadcast when finding references for a node is requested */
	static UE::RigVMEditor::FRigVMEditorOnRequestFindNodeReferencesDelegate& GetOnRequestFindNodeReferences() { return OnRequestFindNodeReferences; }

protected:

	UE_API bool IsRigVMEditorModuleBase() const;
	virtual UClass* GetRigVMLegacyBlueprintClass() const { return URigVMBlueprint::StaticClass(); }
	virtual UClass* GetRigVMAssetClass() const { return URigVMEditorAssetInterface::StaticClass(); }
	virtual UClass* GetRigVMEdGraphSchemaClass() const { return URigVMEdGraphSchema::StaticClass(); }
	virtual UScriptStruct* GetRigVMExecuteContextStruct() const { return FRigVMExecuteContext::StaticStruct(); }

	/** Specific section callbacks for the context menu */
	UE_API virtual void GetNodeActionsContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeWorkflowContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeEventsContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeDefaultValueContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeConversionContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeDebugContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeVariablesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeTemplatesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeOrganizationContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeVariantContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeTestContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetNodeDisplayContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const URigVMEdGraphNode* EdGraphNode, URigVMNode* ModelNode, UToolMenu* Menu) const;
	UE_API virtual void GetExposedPinContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinWorkflowContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinDebugContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinArrayContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinSequenceContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinAggregateContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinTemplateContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinConversionContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinVariableContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinResetDefaultContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;
	UE_API virtual void GetPinInjectedNodesContextMenuActions(TScriptInterface<IRigVMClientHost> RigVMClientHost, const UEdGraphPin* EdGraphPin, URigVMPin* ModelPin, UToolMenu* Menu) const;

	/** Extensibility managers */
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TArray<FRigVMEditorToolbarExtender> RigVMEditorToolbarExtenders;
	
	/** Node factory for the rigvm graph */
	TSharedPtr<FRigVMEdGraphPanelNodeFactory> EdGraphPanelNodeFactory;

	/** Pin factory for the rigvm graph */
	TSharedPtr<FRigVMEdGraphPanelPinFactory> EdGraphPanelPinFactory;

	/** Delegate handles for blueprint utils */
	FDelegateHandle RefreshAllNodesDelegateHandle;
	FDelegateHandle ReconstructAllNodesDelegateHandle;
	FDelegateHandle BlueprintVariableCustomizationHandle;

#if RIGVM_TRACE_ENABLED
	FRewindDebuggerForRigVM RewindDebuggerForRigVM;
	FRigVMTraceModule RigVMTraceModule;
#endif
	
private:

	/** StaticStruct is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> PropertiesToUnregisterOnShutdown;

	/** Registered asset type actions */
	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;

	UE_API void HandleNewBlueprintCreated(UBlueprint* InBlueprint);

	UE_API bool ShowWorkflowOptionsDialog(URigVMUserWorkflowOptions* InOptions) const;

	/** Event broadcast when finding references for a node is requested */
	static UE::RigVMEditor::FRigVMEditorOnRequestFindNodeReferencesDelegate OnRequestFindNodeReferences;
	
	FDelegateHandle AssetsPreDeleteDelegate;
	static void HandleAssetsPreDelete(const TArray<UObject*>& InAssetsToDelete);

	friend class URigVMEdGraphNode;
};

#undef UE_API
