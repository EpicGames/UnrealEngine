// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebugger.h"
#include "IRewindDebuggerExtension.h"
#include "RewindDebuggerTrack.h"
#include "IRewindDebuggerTrackCreator.h"
#include "TraceServices/ModuleService.h"
#include "SEventTimelineView.h"
#include "RigVMTrace.h"

class URigVMHost;

#if RIGVM_TRACE_ENABLED

#define UE_API RIGVMEDITOR_API

/**
 * The trace module implements the rewind debugger extension to record traces
 */
class FRigVMTraceModule : public TraceServices::IModule
{
public:
	virtual ~FRigVMTraceModule() = default;
	
	// TraceServices::IModule interface
	UE_API virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	UE_API virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	UE_API virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	UE_API virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("rigvm"); }

private:
	static FName ModuleName;
};

class FRewindDebuggerForRigVM : public IRewindDebuggerExtension
{
public:
	FRewindDebuggerForRigVM() {}
	virtual ~FRewindDebuggerForRigVM() = default;

	UE_API virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
	virtual FString GetName() override { return TEXT("RigVMDebugger"); }

private:

	double LastTraceTime = 0.0;
	double LastLogTime = 0.0;
	TMap<uint64,TWeakObjectPtr<URigVMHost>> AffectedHosts; 
};

#undef UE_API

#endif
