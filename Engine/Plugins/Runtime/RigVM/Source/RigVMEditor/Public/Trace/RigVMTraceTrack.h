// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebugger.h"
#include "RewindDebuggerTrack.h"
#include "IRewindDebuggerTrackCreator.h"
#include "TraceServices/ModuleService.h"
#include "Widgets/SBoxPanel.h"
#include "SEventTimelineView.h"
#include "RigVMTrace.h"

#if RIGVM_TRACE_ENABLED

#define UE_API RIGVMEDITOR_API

/**
 * The execute track advertises the relevant data per frame for a given RigVM host.
 */
class FRewindDebuggerRigVMExecuteTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FRewindDebuggerRigVMExecuteTrack(uint64 InHostId);

private:
	
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;

	virtual FSlateIcon GetIconInternal() override
	{
		return Icon;
	}

	virtual uint64 GetObjectIdInternal() const override
	{
		return HostId;
	}

	virtual FName GetNameInternal() const override { return "RigVMExecute"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "RigVMExecuteTrackName", "Execute"); }

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;
	
	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int EventUpdateRequested = 0;
	
	uint64 HostId;
	FSlateIcon Icon;
	FText TrackName;

	double PreviousScrubTime = 0.0f;
	double AbsoluteTime = 0.0;
	double DeltaTime = 0.0;

	TSharedPtr<SVerticalBox> DetailsVerticalBox;
};


/**
 * The RigVM rewind debugger track is the main track / parent track for all
 * other RigVM related tracks
 */
class FRewindDebuggerRigVMTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FRewindDebuggerRigVMTrack(uint64 InObjectId);
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override { return Icon; }
	virtual FName GetNameInternal() const override { return "RigVM"; }
	virtual FText GetDisplayNameInternal() const override { return NSLOCTEXT("RewindDebugger", "RigVMTrackName", "RigVM"); }
	virtual uint64 GetObjectIdInternal() const override { return HostId; }
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	FSlateIcon Icon;
	uint64 HostId;

	TArray<TSharedPtr<FRewindDebuggerTrack>> Children;
};

#if WITH_ENGINE
/**
 * Factory class to create RigVM rewind debugger tracks
 */
class FRewindDebuggerRigVMTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

#undef UE_API

#endif
