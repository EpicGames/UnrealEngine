// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMTreeToolkitDefines.h"
#include "RigVMTreeToolkitFilter.h"
#include "Input/Reply.h"
#include "Logging/TokenizedMessage.h"

class FRigVMTreeRootNode;
class FRigVMTreeTask;

/**
 * The context is the top level object passed to anything
 * that has to interact with the tree, like visible node traversal,
 * task execution etc.
 */
class FRigVMTreeContext : public FRigVMTreeElement
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeContext, FRigVMTreeElement)

	virtual uint32 GetVisibleChildrenHash() const;
	bool FiltersNode(TSharedRef<FRigVMTreeNode> InNode) const;

	TArray<TSharedRef<FRigVMTreeFilter>> Filters;

	static FAssetData FindAssetFromAnyPath(const FString& InPartialOrFullPath, bool bUseRootPath);

	DECLARE_EVENT_OneParam(FRigVMTreeContext, FLogTokenizedMessage, const TSharedRef<FTokenizedMessage>&)
	void LogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const;
	void LogMessage(const FText& InText) const;
	void LogMessage(const FString& InString) const;
	void LogWarning(const FText& InText) const;
	void LogWarning(const FString& InString) const;
	void LogError(const FText& InText) const;
	void LogError(const FString& InString) const;

private:
	uint32 HashOffset = 0;
	FLogTokenizedMessage OnLogTokenizedMessage;

	friend class FRigVMTreePhase;
	friend class SRigVMBulkEditWidget;
};

/**
 * The Phase describes a phase of performing a UI wizard,
 * like picking a set of inputs, or performing tasks.
 * This can be also seen as the steps a wizard goes through
 * when performing a UI process.
 */
class FRigVMTreePhase : public FRigVMTreeElement
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreePhase, FRigVMTreeElement)

	DECLARE_DELEGATE_RetVal(FReply, FReplyDelegate)
	DECLARE_DELEGATE_OneParam(FQueueTasksDelegate, const TArray<TSharedRef<FRigVMTreeTask>>&)

	FRigVMTreePhase(int32 InID, const FString& InName, const TSharedRef<FRigVMTreeContext>& InContext);

	int32 GetID() const
	{
		return ID;
	}

	const FString& GetName() const
	{
		return Name;
	}

	bool IsActive() const 
	{
		return bIsActive;
	}

	virtual bool AllowsMultiSelection() const
	{
		return bAllowsMultiSelection;
	}

	void SetAllowsMultiSelection(bool InAllowsMultiSelection)
	{
		bAllowsMultiSelection = InAllowsMultiSelection;
	}

	TAttribute<bool>& IsCancelButtonVisible()
	{
		return IsCancelButtonVisibleAttribute;
	}

	TAttribute<bool>& IsCancelButtonEnabled()
	{
		return IsCancelButtonEnabledAttribute;
	}

	FReplyDelegate& OnCancel()
	{
		return OnCancelDelegate;
	}

	FReply Cancel() const
	{
		if(OnCancelDelegate.IsBound())
		{
			return OnCancelDelegate.Execute();
		}
		return FReply::Unhandled();
	}

	TAttribute<bool>& IsPrimaryButtonVisible()
	{
		return IsPrimaryButtonVisibleAttribute;
	}

	TAttribute<bool>& IsPrimaryButtonEnabled()
	{
		return IsPrimaryButtonEnabledAttribute;
	}

	TAttribute<FText>& PrimaryButtonText()
	{
		return PrimaryButtonTextAttribute;
	}

	FReplyDelegate& OnPrimaryAction()
	{
		return OnPrimaryActionDelegate;
	}

	FReply PrimaryAction() const
	{
		if(OnPrimaryActionDelegate.IsBound())
		{
			return OnPrimaryActionDelegate.Execute();
		}
		return FReply::Unhandled();
	}

	TSharedRef<FRigVMTreeContext> GetContext() const
	{
		return Context;
	}

	void IncrementContextHash();

	TArray<TSharedRef<FRigVMTreeNode>> GetAllNodes() const;

	const TArray<TSharedRef<FRigVMTreeNode>>& GetVisibleNodes() const;

	TSharedPtr<FRigVMTreeNode> FindVisibleNode(const FString& InPath) const;

	void AddNode(const TSharedRef<FRigVMTreeNode>& InNode);
	void RemoveNode(const TSharedRef<FRigVMTreeNode>& InNode);
	void SetNodes(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes);

	FQueueTasksDelegate& OnQueueTasks()
	{
		return QueueTasksDelegate;
	}
	
	void QueueTasks(const TArray<TSharedRef<FRigVMTreeTask>>& InTasks) const
	{
		(void)QueueTasksDelegate.ExecuteIfBound(InTasks);
	}

private:

	int32 ID;
	FString Name;
	bool bIsActive;
	bool bAllowsMultiSelection;
	TSharedRef<FRigVMTreeContext> Context;
	TSharedRef<FRigVMTreeRootNode> RootNode;
	TAttribute<bool> IsCancelButtonVisibleAttribute;
	TAttribute<bool> IsCancelButtonEnabledAttribute;
	FReplyDelegate OnCancelDelegate;
	TAttribute<bool> IsPrimaryButtonVisibleAttribute;
	TAttribute<bool> IsPrimaryButtonEnabledAttribute;
	TAttribute<FText> PrimaryButtonTextAttribute;
	FReplyDelegate OnPrimaryActionDelegate;
	FQueueTasksDelegate QueueTasksDelegate;

	friend class SRigVMBulkEditWidget;
};
