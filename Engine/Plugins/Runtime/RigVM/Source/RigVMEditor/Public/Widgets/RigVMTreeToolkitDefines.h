// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define DEFINE_RIGVMTREETOOLKIT_QUOTE(str) #str
#define DEFINE_RIGVMTREETOOLKIT_ELEMENT(TypeName, SuperTypeName) \
typedef SuperTypeName Super; \
static inline const FLazyName Type = FLazyName(TEXT(DEFINE_RIGVMTREETOOLKIT_QUOTE(TypeName))); \
virtual FName GetType() const override \
{ \
	return Type; \
} \
virtual bool IsOfType(const FName& InType) const override \
{ \
	if(InType == Type) \
	{ \
		return true; \
	} \
	return Super::IsOfType(InType); \
} \
template <typename... Types> \
static TSharedRef<TypeName> Create(Types... Args) \
{ \
	TSharedRef<TypeName> Ptr = MakeShared<TypeName>(Args...); \
	Ptr->Initialize(); \
	return Ptr; \
} \
TSharedPtr<TypeName> ToSharedPtr() const \
{ \
	return ToSharedRef().ToSharedPtr(); \
} \
TSharedRef<TypeName> ToSharedRef() const \
{ \
	TypeName* MutableThis = const_cast<TypeName*>(this); \
	return StaticCastSharedRef<TypeName>(SharedThis(MutableThis)); \
} \
template<typename T> \
friend TSharedPtr<T> Cast(const TSharedRef<TypeName>& InElement) \
{ \
	if(InElement->IsA<T>()) \
	{ \
		return StaticCastSharedPtr<T>(InElement->ToSharedPtr()); \
	} \
	return nullptr; \
} \
template<typename T> \
friend const TSharedRef<T>& CastChecked(const TSharedRef<TypeName>& InElement) \
{ \
	check(InElement->IsA<T>()) \
	return StaticCastSharedRef<T>(InElement); \
}
	
/**
 * The Tree Element is the base class for anything within a
 * tree and manages the lifetime as well as type checking
 */
class FRigVMTreeElement : public TSharedFromThis<FRigVMTreeElement>
{
public:
	static inline const FLazyName Type = FLazyName(TEXT("FRigVMTreeElement"));
	virtual FName GetType() const
	{
		return Type;
	}

	virtual bool IsOfType(const FName& InType) const
	{
		return InType == Type;
	}

	template<typename T>
	bool IsA() const
	{
		return IsOfType(T::Type);
	}

	virtual ~FRigVMTreeElement() = default;
	
	virtual void Initialize()
	{
	}

protected:
	FRigVMTreeElement()
	{
	}
};
