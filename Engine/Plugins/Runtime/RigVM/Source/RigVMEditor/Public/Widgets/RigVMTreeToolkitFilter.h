// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMTreeToolkitDefines.h"

class FRigVMTreeNode;
class FRigVMTreeContext;

/**
 * A Filter is used to hide elements from a tree.
 * Filters can be used to filter by folder, filter
 * by tags or anything else.
 */
class FRigVMTreeFilter : public FRigVMTreeElement
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFilter, FRigVMTreeElement)

	virtual bool CanBeToggledInUI() const
	{
		return false;
	}

	virtual bool IsInvertedInUI() const
	{
		return false;
	}

	virtual FText GetLabel() const
	{
		return FText();
	}

	virtual FText GetToolTip() const
	{
		return FText();
	}
	
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext);

	virtual uint32 GetVisibleChildrenHash() const;
	
	bool IsEnabled() const
	{
		return bEnabled;
	}

	void SetEnabled(bool InEnabled)
	{
		bEnabled = InEnabled;
	}
	
private:
	bool bEnabled = true;
};

/**
 * The Path filter is used to include elements only if they
 * match a user provided path.
 */
class FRigVMTreePathFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreePathFilter, FRigVMTreeFilter)

	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;

	virtual uint32 GetVisibleChildrenHash() const override;
	
	const FString& GetFilterText() const
	{ 
		return FilterText;
	}
	void SetFilterText(const FString& InFilterText)
	{ 
		FilterText = InFilterText;
	}

private:
	FString FilterText;
};

/**
 * The Engine Content Filter can be used to hide elements
 * which reside on the engine content path.
 */
class FRigVMTreeEngineContentFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeEngineContentFilter, FRigVMTreeFilter)

	virtual bool CanBeToggledInUI() const override
	{
		return true;
	}

	virtual bool IsInvertedInUI() const override
	{
		return true;
	}

	virtual FText GetLabel() const override;

	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

/**
 * The Developer Content Filter can be used to hide items which 
 * reside on a developer content path.
 */
class FRigVMTreeDeveloperContentFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeDeveloperContentFilter, FRigVMTreeFilter)

	virtual bool CanBeToggledInUI() const override
	{
		return true;
	}

	virtual bool IsInvertedInUI() const override
	{
		return true;
	}

	virtual FText GetLabel() const override;
	
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};
