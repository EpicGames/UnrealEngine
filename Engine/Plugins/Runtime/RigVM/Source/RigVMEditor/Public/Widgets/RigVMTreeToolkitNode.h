// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMBlueprintLegacy.h"
#include "RigVMTreeToolkitDefines.h"
#include "RigVMCore/RigVMVariant.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"

class FRigVMTreeContext;

/**
 * A Node is the base element for anything that is shown
 * within the tree.
 */
class FRigVMTreeNode : public FRigVMTreeElement
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeNode, FRigVMTreeElement)
	
	FRigVMTreeNode(const FString& InPath)
		: Path(InPath)
	{
	}

	virtual const FString& GetPath() const
	{
		return Path;
	}

	virtual FAssetData GetAssetData() const;
	UE_DEPRECATED(5.7, "Please use IRigVMEditorAssetInterface* GetAsset() const")
	virtual URigVMBlueprint* GetBlueprint() const { return Cast<URigVMBlueprint>(GetAsset()->GetObject()); }
	virtual IRigVMEditorAssetInterface* GetAsset() const;

	virtual FText GetLabel() const;

	virtual void GetContextMenu(FMenuBuilder& InMenuBuilder)
	{
	}

	virtual bool IsCheckable() const
	{
		return false;
	}

	ECheckBoxState GetCheckState() const;

	void SetCheckState(ECheckBoxState InNewState);
	void ResetCheckState();

	virtual bool ShouldExpandByDefault() const
	{
		return false;
	}
	
	virtual const TArray<FRigVMTag>& GetTags() const
	{
		return Tags;
	}
	
	int32 GetDepth() const;

	TSharedPtr<FRigVMTreeNode> GetParent() const
	{
		if(Parent)
		{
			return Parent->ToSharedPtr();
		}
		return nullptr;
	}

	TSharedRef<FRigVMTreeNode> GetRoot() const;

	const TArray<TSharedRef<FRigVMTreeNode>>& GetVisibleChildren(const TSharedRef<FRigVMTreeContext>& InContext) const
	{
		UpdateVisibleChildren(InContext);
		return VisibleChildren;
	}

	TArray<TSharedRef<FRigVMTreeNode>> GetChildren(const TSharedRef<FRigVMTreeContext>& InContext) const
	{
		UpdateChildren(InContext);
		return Children.Get(TArray<TSharedRef<FRigVMTreeNode>>());
	}

	virtual bool HasVisibleChildren() const
	{
		return !VisibleChildren.IsEmpty();
	}

	TSharedPtr<FRigVMTreeNode> FindVisibleChild(const FString& InFullPath, const TSharedRef<FRigVMTreeContext>& InContext) const;

	virtual void DirtyChildren();
	virtual void DirtyVisibleChildren();
	void RequestRefresh(bool bForce = false);

	virtual const FSlateBrush* GetIconAndTint(FLinearColor& OutColor) const
	{
		return nullptr;
	}
	virtual const FSlateBrush* GetBackgroundImage(bool bIsHovered, bool bIsSelected) const;
	virtual FSlateColor GetBackgroundColor(bool bIsHovered, bool bIsSelected) const;

	virtual bool IsLoaded() const;

protected:
	virtual TArray<TSharedRef<FRigVMTreeNode>> GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const
	{
		return Children.Get(TArray<TSharedRef<FRigVMTreeNode>>());
	}
	void AddChildImpl(const TSharedRef<FRigVMTreeNode>& InChild) const;
	void UpdateChildren(const TSharedRef<FRigVMTreeContext>& InContext) const;
	void UpdateVisibleChildren(const TSharedRef<FRigVMTreeContext>& InContext) const;
	void DirtyVisibleParent() const;
	bool ContainsAnyVisibleCheckedNode() const;

	static TSharedPtr<FRigVMTreeNode> FindVisibleNodeInSet(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes, const FString& InPath, const TSharedRef<FRigVMTreeContext>& InContext);

	FString Path;
	const FRigVMTreeNode* Parent = nullptr;
	mutable TOptional<TArray<TSharedRef<FRigVMTreeNode>>> Children;
	mutable uint32 VisibleChildrenHash = UINT32_MAX;
	mutable TArray<TSharedRef<FRigVMTreeNode>> VisibleChildren;
	mutable bool bIsDirtyingParent = false;
	TOptional<ECheckBoxState> CheckState;
	mutable TArray<FRigVMTag> Tags;

	DECLARE_DELEGATE_OneParam(FRefreshDelegate, bool)
	FRefreshDelegate RefreshDelegate;

	friend class SRigVMChangesTreeRow;
	friend class SRigVMChangesTreeView;
	friend class FRigVMTreePhase;
	friend class FRemoveNodeTask;
};

/**
 * A Root Node is a node which can be placed only at
 * the root of the tree.
 */
class FRigVMTreeRootNode : public FRigVMTreeNode
{
public:

	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeRootNode, FRigVMTreeNode)
	
	FRigVMTreeRootNode();

	void AddChild(const TSharedRef<FRigVMTreeNode>& InNode);
	void RemoveChild(const TSharedRef<FRigVMTreeNode>& InNode);
	void SetChildren(const TArray<TSharedRef<FRigVMTreeNode>>& InNodes);
};

/** 
 * A Category Node is a node which presents itself as a category,
 * providing a collapsed UI header.
 */
class FRigVMTreeCategoryNode : public FRigVMTreeNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeCategoryNode, FRigVMTreeNode)
	
	FRigVMTreeCategoryNode(const FString& InPath)
		: FRigVMTreeNode(InPath)
	{
	}

	virtual const FSlateBrush* GetBackgroundImage(bool bIsHovered, bool bIsSelected) const override;
	virtual FSlateColor GetBackgroundColor(bool bIsHovered, bool bIsSelected) const override;
};

/**
 * A Package Node is a special Category Node which reacts to 
 * the package being loaded and offers to update its content.
 */
class FRigVMTreePackageNode : public FRigVMTreeCategoryNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreePackageNode, FRigVMTreeCategoryNode)
	
	FRigVMTreePackageNode(const FAssetData& InAssetData);

	virtual const FSlateBrush* GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void Initialize() override;

	virtual FText GetLabel() const override;
	virtual bool IsLoaded() const override;
	virtual const TArray<FRigVMTag>& GetTags() const override;

	const FSoftObjectPath& GetPackagePath() const { return SoftObjectPath; }

protected:
	void HandleAssetLoaded(UObject* InAsset);

	mutable TOptional<bool> bIsLoaded;
	FSoftObjectPath SoftObjectPath;
	mutable FSlateIcon Icon;
	mutable TOptional<const FSlateBrush*> IconBrush;
	mutable bool bRetrievedTags;
};
