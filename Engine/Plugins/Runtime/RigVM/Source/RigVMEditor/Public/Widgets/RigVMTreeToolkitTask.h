// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigVMTreeToolkitDefines.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SRigVMLogWidget.h"

class FRigVMTreeNode;
class FRigVMTreePhase;
class FRigVMTreeContext;

/**
 * A Task is a small piece of work that needs to be performed.
 */
class FRigVMTreeTask : public FRigVMTreeElement
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeTask, FRigVMTreeElement)

	virtual bool Execute(const TSharedRef<FRigVMTreePhase>& InPhase)
	{
		return false;
	};

	UE_DEPRECATED(5.7, "Please use FRigVMEditorAssetInterfacePtr GetRigVMAsset(const TSharedRef<FRigVMTreePhase>& InPhase) const")
	virtual URigVMBlueprint* GetBlueprint(const TSharedRef<FRigVMTreePhase>& InPhase) const
	{
		return nullptr;
	}

	virtual FRigVMEditorAssetInterfacePtr GetRigVMAsset(const TSharedRef<FRigVMTreePhase>& InPhase) const
	{
		return nullptr;
	}

	virtual bool RequiresRefresh() const
	{
		return false;
	}

	virtual bool RequiresUndo() const
	{
		return false;
	}

	virtual TArray<FString> GetAffectedNodes() const
	{
		return {};
	}
	
	void SetEnableUndo(bool bEnabled)
	{
		bEnableUndo = bEnabled;
	}

	bool IsUndoEnabled() const
	{
		return bEnableUndo;
	}

private:
	bool bEnableUndo = false;
};

/**
 * The Load Package For Node Task will load a package related to a node
 * given the node's path.
 */
class FRigVMTreeLoadPackageForNodeTask : public FRigVMTreeTask
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeLoadPackageForNodeTask, FRigVMTreeTask)

	FRigVMTreeLoadPackageForNodeTask(const TSharedRef<FRigVMTreeNode>& InNode);

	virtual bool Execute(const TSharedRef<FRigVMTreePhase>& InPhase) override;

	virtual bool RequiresRefresh() const override
	{
		return true;
	}
private:

	FAssetData AssetData;
};

/**
 * The Compile Blueprint Task is used to compile a blueprint related
 * to a node in the tree (based on the node's path).
 */
class FRigVMCompileBlueprintTask : public FRigVMTreeTask
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMCompileBlueprintTask, FRigVMTreeTask)

	FRigVMCompileBlueprintTask(const TSharedRef<FRigVMTreeNode>& InNode);

	virtual bool Execute(const TSharedRef<FRigVMTreePhase>& InPhase) override;

	virtual bool RequiresRefresh() const override
	{
		return true;
	}

private:

	FSoftObjectPath ObjectPath;
};
