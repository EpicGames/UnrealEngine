// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRigVMBulkEditWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/SWindow.h"

template<typename InWidgetType = SRigVMBulkEditWidget>
class SRigVMBulkEditDialog : public SWindow
{
public:

	SLATE_BEGIN_ARGS(SRigVMBulkEditDialog)
		: _WindowSize(1200.0f, 1000.0f)
	{}
	SLATE_ARGUMENT(FVector2D, WindowSize)
	SLATE_ARGUMENT(typename InWidgetType::FArguments, WidgetArgs);
	SLATE_END_ARGS()

	TSharedRef<SRigVMBulkEditWidget> GetBulkEditWidget() const
	{
		return Widget->GetBulkEditWidget();
	}

	void Construct(const FArguments& InArgs)
	{
		Widget = SArgumentNew(InArgs._WidgetArgs, InWidgetType);
		SWindow::Construct(SWindow::FArguments()
			.Title(this, &SRigVMBulkEditDialog::GetDialogTitle)
			.SizingRule( ESizingRule::UserSized )
			.ClientSize(InArgs._WindowSize)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.FillHeight(1)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding(2)
					[
						GetBulkEditWidget()
					]
				]
			]
		);
	}

	FText GetDialogTitle() const
	{
		return GetBulkEditWidget()->GetDialogTitle();
	}

	void ShowNormal()
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelRegularWindow();

		if( !ParentWindow.IsValid() )
		{
			// Parent to the main frame window
			if(FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				const IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}
		}

		if(ParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(SharedThis(this), ParentWindow.ToSharedRef(), true);
		}
		else
		{
			FSlateApplication::Get().AddWindow(SharedThis(this), true);
		}
	}

private:

	TSharedPtr<InWidgetType> Widget;
};
