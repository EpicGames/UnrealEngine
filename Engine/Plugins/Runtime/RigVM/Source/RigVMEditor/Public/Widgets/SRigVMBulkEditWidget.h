// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SRigVMChangesTreeView.h"
#include "Widgets/SRigVMLogWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Interfaces/IMainFrameModule.h"
#include "ScopedTransaction.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_OneParam(FOnPhaseActivated, TSharedRef<FRigVMTreePhase>);

class SRigVMBulkEditWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMBulkEditWidget)
		: _EnableUndo(false)
		, _CloseOnSuccess(false)
		, _PhaseToActivate(INDEX_NONE)
		, _BulkEditConfirmIniField(TEXT("BulkEdit_Warning"))
	{}
	SLATE_ARGUMENT(bool, EnableUndo)
	SLATE_ARGUMENT(bool, CloseOnSuccess)
	SLATE_ARGUMENT(TArray<TSharedRef<FRigVMTreePhase>>, Phases)
	SLATE_ARGUMENT(int32, PhaseToActivate)
	SLATE_EVENT(FOnRigVMTreeNodeSelected, OnNodeSelected)
	SLATE_EVENT(FOnRigVMTreeNodeDoubleClicked, OnNodeDoubleClicked)
	SLATE_EVENT(FOnPhaseActivated, OnPhaseActivated)
	SLATE_ARGUMENT(TSharedPtr<SWidget>, HeaderWidget)
	SLATE_ARGUMENT(TSharedPtr<SWidget>, FooterWidget)
	SLATE_ARGUMENT(TSharedPtr<SWidget>, LeftWidget)
	SLATE_ARGUMENT(TSharedPtr<SWidget>, RightWidget)
	SLATE_ATTRIBUTE(FText, BulkEditTitle)
	SLATE_ATTRIBUTE(FText, BulkEditConfirmMessage)
	SLATE_ATTRIBUTE(FString, BulkEditConfirmIniField)
	SLATE_END_ARGS()

	UE_API virtual ~SRigVMBulkEditWidget() override;

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API FText GetDialogTitle() const;

	UE_API void QueueTasks(const TArray<TSharedRef<FRigVMTreeTask>>& InTasks);
	UE_API void CancelTasks();

	TSharedRef<SRigVMBulkEditWidget> GetBulkEditWidget()
	{
		return SharedThis(this);
	}

	TSharedPtr<SRigVMChangesTreeView> GetTreeView()
	{
		return TreeView;
	}

	TSharedPtr<SRigVMLogWidget> GetLogWidget()
	{
		return BulkEditLogWidget;
	}

	UE_API TSharedPtr<FRigVMTreePhase> GetActivePhasePtr() const;
	UE_API TSharedRef<FRigVMTreePhase> GetActivePhase() const;
	UE_API TSharedPtr<FRigVMTreePhase> FindPhase(int32 InID) const;
	UE_API bool ActivatePhase(int32 InID);
	UE_API TSharedRef<FRigVMTreeContext> GetContext() const;

	UE_API TArray<TSharedRef<FRigVMTreeNode>> GetSelectedNodes() const;
	UE_API bool HasAnyVisibleCheckedNode() const;
	UE_API TArray<TSharedRef<FRigVMTreeNode>> GetCheckedNodes() const;
	
	UE_API FReply OnBackButtonClicked();
	UE_API FReply OnCancelButtonClicked();
	UE_API FReply OnPrimaryButtonClicked();

	UE_API void CloseDialog();

private:

	TArray<TSharedRef<FRigVMTreePhase>> Phases;
	TArray<int32> ActivatedPhaseIDs;
	
	TSharedPtr<SRigVMChangesTreeView> TreeView;
	TSharedPtr<SRigVMLogWidget> BulkEditLogWidget;

	FOnPhaseActivated OnPhaseActivated;

	TAttribute<FText> BulkEditTitle;
	TAttribute<FText> BulkEditConfirmMessage;
	TAttribute<FString> BulkEditConfirmIniField;

	UE_API bool AreTasksInProgress() const;
	UE_API EVisibility GetTasksProgressVisibility() const;
	UE_API TOptional<float> GetTasksProgressPercentage() const;

	UE_API void OnLogMessage(const TSharedRef<FTokenizedMessage>& InMessage) const;
	UE_API void OnScriptException(ELogVerbosity::Type InVerbosity, const TCHAR* InMessage, const TCHAR* InStackMessage);

	mutable FCriticalSection TasksCriticalSection;
	TArray<TSharedRef<FRigVMTreeTask>> RemainingTasks;
	TArray<TSharedRef<FRigVMTreeTask>> CompletedTasks;
	bool bTasksSucceeded = false;
	bool bShowLog = false;
	bool bEnableUndo = false;
	bool bCloseOnSuccess = false;
	TSharedPtr<FScopedTransaction> Transaction;

	UE_API EVisibility GetBackButtonVisibility() const;
	UE_API bool IsBackButtonEnabled() const;
	UE_API EVisibility GetCancelButtonVisibility() const;
	UE_API bool IsCancelButtonEnabled() const;
	UE_API EVisibility GetPrimaryButtonVisibility() const;
	UE_API bool IsPrimaryButtonEnabled() const;
	UE_API FText GetPrimaryButtonText() const;
	UE_API bool IsReadyToClose() const;
};

#undef UE_API
