// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/RigVMTreeToolkitNode.h"
#include "Widgets/RigVMTreeToolkitFilter.h"
#include "Widgets/RigVMTreeToolkitContext.h"
#include "Widgets/RigVMTreeToolkitTask.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"

#define UE_API RIGVMEDITOR_API

class SRigVMChangesTreeView;

class SRigVMChangesTreeRow
	: public STableRow<TSharedRef<FRigVMTreeNode>>
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMChangesTreeRow)
	{}
	SLATE_ARGUMENT(TSharedPtr<FRigVMTreeNode>, Node)
	SLATE_ARGUMENT(TSharedPtr<SRigVMChangesTreeView>, OwningWidget)
	SLATE_END_ARGS()

	UE_API virtual ~SRigVMChangesTreeRow() override;
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

private:

	UE_API EVisibility GetCheckBoxVisibility() const;
	UE_API ECheckBoxState GetCheckBoxState() const;
	UE_API void OnCheckBoxStateChanged(ECheckBoxState InNewState);
	UE_API const FSlateBrush* GetBackgroundImage() const;
	UE_API FSlateColor GetBackgroundColor() const;
	UE_API FOptionalSize GetIndentWidth() const;
	UE_API FSlateColor GetTextColor() const;
	UE_API const FSlateBrush* GetIcon() const;
	UE_API FSlateColor GetIconColor() const;
	UE_API const FSlateBrush* GetExpanderImage() const;
	UE_API FReply OnExpanderMouseButtonDown(const FGeometry& SenderGeometry, const FPointerEvent& MouseEvent);
	UE_API EVisibility GetExpanderVisibility() const;
	UE_API TArray<FRigVMTag> GetVariantTags() const;
	UE_API void RequestRefresh(bool bForce);

	TSharedPtr<FRigVMTreeNode> Node;
	SRigVMChangesTreeView* OwningWidget = nullptr;
	mutable TOptional<TArray<FRigVMTag>> Tags;
};

DECLARE_DELEGATE_RetVal_OneParam(FReply,FOnRigVMTreeNodeSelected,TSharedRef<FRigVMTreeNode>);
DECLARE_DELEGATE_RetVal_OneParam(FReply,FOnRigVMTreeNodeDoubleClicked,TSharedRef<FRigVMTreeNode>);

class SRigVMChangesTreeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMChangesTreeView)
	{}
	SLATE_ATTRIBUTE(TSharedPtr<FRigVMTreePhase>, Phase)
	SLATE_EVENT(FOnRigVMTreeNodeSelected, OnNodeSelected)
	SLATE_EVENT(FOnRigVMTreeNodeDoubleClicked, OnNodeDoubleClicked)
	SLATE_END_ARGS()

	UE_API virtual ~SRigVMChangesTreeView() override;

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	UE_API EVisibility GetPathFilterVisibility() const;
	UE_API void OnPathFilterTextChanged(const FText& SearchText);
	UE_API void OnPathFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	UE_API void RequestRefresh_AnyThread(bool bForce = false);
	UE_API void RefreshFilteredNodes(bool bForce = false);
	UE_API void RefreshFilteredNodesIfRequired();

	UE_API void OnPhaseChanged();
	UE_API void SetSelection(const TSharedPtr<FRigVMTreeNode>& InNode, bool bRequestScrollIntoView = false);

	TSharedPtr<STreeView<TSharedRef<FRigVMTreeNode>>> GetTreeView()
	{
		return TreeView;
	}

	UE_API TSharedRef<FRigVMTreePhase> GetPhase() const;
	UE_API TSharedRef<FRigVMTreeContext> GetContext() const;
	UE_API TSharedPtr<FRigVMTreePathFilter> GetPathFilter() const;

	UE_API TArray<TSharedRef<FRigVMTreeNode>> GetSelectedNodes() const;
	UE_API bool HasAnyVisibleCheckedNode() const;
	UE_API TArray<TSharedRef<FRigVMTreeNode>> GetCheckedNodes() const;
	
private:

	TAttribute<TSharedPtr<FRigVMTreePhase>> PhaseAttribute;
	TArray<TSharedRef<FRigVMTreeNode>> FilteredNodes;
	
	UE_API FText GetPathFilterText();
	TSharedPtr<SSearchBox> PathFilterBox;

	TSharedPtr<STreeView<TSharedRef<FRigVMTreeNode>>> TreeView;

	FOnRigVMTreeNodeSelected OnNodeSelected;
	FOnRigVMTreeNodeDoubleClicked OnNodeDoubleClicked;

	TAttribute<FText> BulkEditTitle;
	TAttribute<FText> BulkEditConfirmMessage;
	TAttribute<FString> BulkEditConfirmIniField;

	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(TSharedRef<FRigVMTreeNode> InNode, const TSharedRef<STableViewBase>& OwnerTable);

	UE_API void GetChildrenForNode(TSharedRef<FRigVMTreeNode> InNode, TArray<TSharedRef<FRigVMTreeNode>>& OutChildren);
	UE_API void OnSelectionChanged(TSharedPtr<FRigVMTreeNode> Selection, ESelectInfo::Type SelectInfo);
	UE_API void OnTreeElementDoubleClicked(TSharedRef<FRigVMTreeNode> InNode);
	UE_API ESelectionMode::Type GetSelectionMode() const;

	UE_API TSharedPtr<SWidget> OnGetNodeContextMenuContent();

	UE_API EVisibility GetSettingsButtonVisibility() const;
	UE_API FReply OnSettingsButtonClicked();

	std::atomic<int32> RequestRefreshCount;
	std::atomic<int32> RequestRefreshForceCount;
};

#undef UE_API
