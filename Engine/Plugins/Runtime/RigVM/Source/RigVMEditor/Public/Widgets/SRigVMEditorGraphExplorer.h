// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GraphEditorDragDropAction.h"
#include "RigVMEditorAsset.h"
#include "RigVMBlueprintLegacy.h"
#include "DragAndDrop/GraphNodeDragDropOp.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"
#include "RigVMModel/RigVMGraph.h"
#include "SRigVMEditorGraphExplorerTreeView.h"

class IRigVMEditor;
class SRigVMEditorGraphExplorerTreeView;
namespace UE::RigVMEditor { class FRigVMEdGraphNodeRegistry; }

//
class URigVMGraph;
class URigVMEdGraph;
class IRigVMEditorAssetInterface;
class SGraphActionMenu;
struct FCreateWidgetForActionData;
class FRigVMEditorGraphExplorerTreeElement;

class FMenuBuilder;
class SSearchBox;
class FUICommandList;
//

class FRigVMGraphExplorerDragDropOp : public FGraphEditorDragDropAction 
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRigVMExplorerDragDropOp, FGraphEditorDragDropAction)

	UE_DEPRECATED(5.7, "Please use New(const FRigVMExplorerElementKey& InElement, FRigVMEditorAssetInterfacePtr InBlueprint);")
	static TSharedRef<FRigVMGraphExplorerDragDropOp> New(const FRigVMExplorerElementKey& InElement, TObjectPtr<URigVMBlueprint> InBlueprint);
	static TSharedRef<FRigVMGraphExplorerDragDropOp> New(const FRigVMExplorerElementKey& InElement, FRigVMEditorAssetInterfacePtr InBlueprint);

	const FRigVMExplorerElementKey& GetElement() const
	{
		return Element;
	}

	UE_DEPRECATED(5.7, "Please use FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const")
	TObjectPtr<URigVMBlueprint> GetBlueprint() const
	{
		return Cast<URigVMBlueprint>(GetRigVMAssetInterface().GetObject());
	}

	FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const
	{
		return SourceBlueprint;
	}

	virtual void Construct() override;

	virtual FReply DroppedOnPanel(const TSharedRef<SWidget>& Panel, const FVector2f& ScreenPosition, const FVector2f& GraphPosition, UEdGraph& Graph) override;
	virtual FReply DroppedOnPin(const FVector2f& ScreenPosition, const FVector2f& GraphPosition) override;

private:

	FRigVMExplorerElementKey Element;
	FRigVMEditorAssetInterfacePtr SourceBlueprint;
};

class FRigVMEditorGraphExplorerCommands : public TCommands<FRigVMEditorGraphExplorerCommands>
{
public:
	FRigVMEditorGraphExplorerCommands();

	TSharedPtr<FUICommandInfo> OpenGraph;
	TSharedPtr<FUICommandInfo> OpenGraphInNewTab;
	TSharedPtr<FUICommandInfo> CreateGraph;
	TSharedPtr<FUICommandInfo> CreateFunction;
	TSharedPtr<FUICommandInfo> CreateVariable;
	TSharedPtr<FUICommandInfo> CreateLocalVariable;
	TSharedPtr<FUICommandInfo> AddFunctionVariant;
	TSharedPtr<FUICommandInfo> PasteFunction;
	TSharedPtr<FUICommandInfo> PasteVariable;
	TSharedPtr<FUICommandInfo> PasteLocalVariable;
	TSharedPtr<FUICommandInfo> RemoveUnusedFunctions;
	TSharedPtr<FUICommandInfo> RemoveUnusedVariables;
	TSharedPtr<FUICommandInfo> FindReferences;

	void RegisterCommands() override;
};

class SRigVMEditorGraphExplorer :
	public SCompoundWidget
{
	using FRigVMEdGraphNodeRegistry = UE::RigVMEditor::FRigVMEdGraphNodeRegistry;

public:
	SLATE_BEGIN_ARGS(SRigVMEditorGraphExplorer) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<IRigVMEditor> InRigVMEditor);

	// Refresh the graph action menu.
	void Refresh();

	// SWidget overrides
	void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	void SetLastPinTypeUsed(const FEdGraphPinType& InType) { LastPinType = InType; }
	FEdGraphPinType GetLastPinTypeUsed() const { return LastPinType; }

	FName GetSelectedVariableName() const;
	ERigVMExplorerElementType::Type GetSelectedType() const;

	void ClearSelection();

	/** Returns a registry for Ed Graph Nodes that hold RigVMFunctionReferenceNodes */
	const TSharedPtr<FRigVMEdGraphNodeRegistry>& GetEdGraphNodeFunctionRegistry() const { return EdGraphNodeFunctionRegistry; }

	/** Returns a registry for Ed Graph Nodes that hold RigVMVariableNodes */
	const TSharedPtr<FRigVMEdGraphNodeRegistry>& GetEdGraphNodeVariableRegistry() const { return EdGraphNodeVariableRegistry; }

	void HandleGraphModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);

private:
	bool bNeedsRefresh = true;

	FEdGraphPinType LastPinType;

	/** Command list for widget-level keyboard shortcuts */
	TSharedPtr<FUICommandList> UICommandList;
	
	void RegisterCommands();
	void CreateWidgets();

	// Add new menu
	TSharedRef<SWidget> CreateAddNewMenuWidget();
	void BuildAddNewMenu(FMenuBuilder& MenuBuilder);

	TArray<const URigVMGraph*> GetRootGraphs() const;
	TArray<const URigVMGraph*> GetChildrenGraphs(const FString& InParentGraphPath) const;
	TArray<URigVMNode*> GetEventNodesInGraph(const FString& InParentGraphPath) const;
	TArray<URigVMLibraryNode*> GetFunctions() const;
	TArray<FRigVMGraphVariableDescription> GetVariables() const;
	TArray<FRigVMGraphVariableDescription> GetLocalVariables() const;
	FText GetGraphDisplayName(const FString& InGraphPath) const;
	FText GetEventDisplayName(const FString& InNodePath) const;
	const FSlateBrush* GetGraphIcon(const FString& InGraphPath) const;
	FText GetGraphTooltip(const FString& InGraphPath) const;
	void OnGraphClicked(const FString& InGraphPath);
	void OnEventClicked(const FString& InEventPath);
	void OnFunctionClicked(const FString& InFunctionPath);
	void OnVariableClicked(const FRigVMExplorerElementKey& InVariable);
	void OnGraphDoubleClicked(const FString& InGraphPath);
	void OnEventDoubleClicked(const FString& InEventPath);
	void OnFunctionDoubleClicked(const FString& InFunctionPath);
	bool OnSetFunctionCategory(const FString& InFunctionPath, const FString& InCategory);
	FString OnGetFunctionCategory(const FString& InPath) const;
	FText OnGetFunctionTooltip(const FString& InFunctionPath) const;
	FText OnGetVariableTooltip(const FString& InVariable) const;
	bool OnSetVariableCategory(const FString& InVariable, const FString& InCategory);
	FString OnGetVariableCategory(const FString& InVariable) const;
	FEdGraphPinType OnGetVariablePinType(const FRigVMExplorerElementKey& InVariable);
	bool OnSetVariablePinType(const FRigVMExplorerElementKey& InVariable, const FEdGraphPinType& InType);
	bool OnIsVariablePublic(const FString& InVariableName) const;
	bool OnToggleVariablePublic(const FString& InVariableName) const;
	bool IsFunctionFocused() const;
	TArray<TSharedPtr<IPinTypeSelectorFilter>> GetCustomPinFilters() const;
	
	TSharedPtr<SWidget> OnContextMenuOpening();
	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	
	void OnCopy();
	bool CanCopy() const;
	void OnCut();
	bool CanCut() const;
	void OnDuplicate();
	bool CanDuplicate() const;
	void OnPasteGeneric();
	bool CanPasteGeneric() const;
	void OnPasteFunction();
	bool CanPasteFunction() const;
	void OnPasteVariable();
	bool CanPasteVariable() const;
	void OnPasteLocalVariable();
	bool CanPasteLocalVariable() const;



	/** Support functions for view options for Show Empty Sections */
	void OnToggleShowEmptySections();
	bool IsShowingEmptySections() const;

	// Command functions
	void OnOpenGraph(bool bOnNewTab = false);
	bool CanOpenGraph() const;

	void OnCreateGraph();
	bool CanCreateGraph() const;

	void OnCreateFunction();
	bool CanCreateFunction() const;

	void OnCreateVariable();
	bool CanCreateVariable() const;

	void OnCreateLocalVariable();
	bool CanCreateLocalVariable() const;

	void OnAddFunctionVariant();
	bool CanAddFunctionVariant() const;
	
	void OnDeleteEntry();
	bool CanDeleteEntry() const;

	void OnRenameEntry();
	bool CanRenameEntry() const;
	bool OnRenameGraph(const FString& InOldPath, const FString& InNewPath);
	bool OnCanRenameGraph(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const;
	bool OnRenameFunction(const FString& InOldPath, const FString& InNewPath);
	bool OnCanRenameFunction(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const;
	bool OnRenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName);
	bool OnCanRenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName, FText& OutErrorMessage) const;
	bool OnReorderVariable(const FRigVMExplorerElementKey& InVariable, int32 NewIndex);
	bool OnReorderLocalVariable(const FRigVMExplorerElementKey& InVariable, int32 NewIndex);

	void OnFindReferences(const bool bSearchAllBlueprints);
	bool CanFindReferences() const;

	void OnRemoveUnusedFunctions();
	bool CanRemoveUnusedFunctions() const;
	void OnRemoveUnusedVariables();
	bool CanRemoveUnusedVariables() const;

	TWeakPtr<IRigVMEditor> RigVMEditor;

	TSharedPtr<SRigVMEditorGraphExplorerTreeView> TreeView;

	TSharedPtr<SSearchBox> FilterBox;

	/** A registry for Ed Graph Nodes that hold URigVMFunctionReferenceNode */
	TSharedPtr<FRigVMEdGraphNodeRegistry> EdGraphNodeFunctionRegistry;

	/** A registry for Ed Graph Nodes that hold RigVMVariableNodes */
	TSharedPtr<FRigVMEdGraphNodeRegistry> EdGraphNodeVariableRegistry;

	bool bShowEmptySections = true;
};

