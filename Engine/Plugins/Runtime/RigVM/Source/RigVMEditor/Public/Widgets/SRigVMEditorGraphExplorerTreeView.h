// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPalette.h"
#include "RigVMModel/RigVMGraph.h"
#include "Widgets/Views/STreeView.h"

class FRigVMEditorBase;
class SSearchBox;
class SRigVMEditorGraphExplorer;
class SRigVMEditorGraphExplorerTreeView;
class SRigVMEditorGraphExplorerItem;
class FRigVMEditorGraphExplorerTreeElement;
class URigVMGraph;
class IPinTypeSelectorFilter;
namespace UE::RigVMEditor { class FRigVMEdGraphNodeRegistry; }

namespace ERigVMExplorerElementType
{
	enum Type
	{
		Invalid,
		Section,
		FunctionCategory,
		VariableCategory,
		Graph,
		Event,
		Function,
		Variable,
		LocalVariable
	};
}

/** The type of section in the graph explorer */
enum class ERigVMGraphExplorerSectionType : uint8
{
	None,
	Graphs,
	Functions,
	Variables, 
	LocalVariables
};

struct FRigVMExplorerElementKey
{

	FRigVMExplorerElementKey()
		: Type(ERigVMExplorerElementType::Invalid), Name(FString()){}

	FRigVMExplorerElementKey(const ERigVMExplorerElementType::Type& InType, const FString& InName)
		: Type(InType), Name(InName){}
	
	ERigVMExplorerElementType::Type Type;
	FString Name;

	friend bool operator==(const FRigVMExplorerElementKey& A, const FRigVMExplorerElementKey& B)
	{
		return A.Type == B.Type && A.Name == B.Name;
	}

	friend uint32 GetTypeHash(const FRigVMExplorerElementKey& Key)
	{
		return HashCombine(GetTypeHash(Key.Type), GetTypeHash(Key.Name));
	}
};


DECLARE_DELEGATE_RetVal(TArray<const URigVMGraph*>, FRigVMGraphExplorer_OnGetRootGraphs);
DECLARE_DELEGATE( FRigVMGraphExplorer_OnCreateGraph);
DECLARE_DELEGATE( FRigVMGraphExplorer_OnCreateFunction);
DECLARE_DELEGATE( FRigVMGraphExplorer_OnCreateVariable);
DECLARE_DELEGATE_RetVal_OneParam(TArray<const URigVMGraph*>, FRigVMGraphExplorer_OnGetChildrenGraphs, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(TArray<URigVMNode*>, FRigVMGraphExplorer_OnGetEventNodesInGraph, const FString&);
DECLARE_DELEGATE_RetVal(TArray<URigVMLibraryNode*>, FRigVMGraphExplorer_OnGetFunctions);
DECLARE_DELEGATE_RetVal(TArray<FRigVMGraphVariableDescription>, FRigVMGraphExplorer_OnGetVariables);
DECLARE_DELEGATE_RetVal(bool, FRigVMGraphExplorer_OnIsFunctionFocused);
DECLARE_DELEGATE_RetVal_OneParam(FText, FRigVMGraphExplorer_OnGetGraphDisplayName, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FText, FRigVMGraphExplorer_OnGetEventDisplayName, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FText, FRigVMGraphExplorer_OnGetGraphTooltip, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(const FSlateBrush*, FRigVMGraphExplorer_OnGetGraphIcon, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnGraphClicked, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnEventClicked, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnFunctionClicked, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnVariableClicked, const FRigVMExplorerElementKey&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnGraphDoubleClicked, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnEventDoubleClicked, const FString&);
DECLARE_DELEGATE_OneParam(FRigVMGraphExplorer_OnFunctionDoubleClicked, const FString&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnRenameGraph, const FString&, const FString&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnRenameFunction, const FString&, const FString&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnRenameVariable, const FRigVMExplorerElementKey&, const FString&);
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FRigVMGraphExplorer_OnCanRenameGraph, const FString&, const FString&, FText&);
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FRigVMGraphExplorer_OnCanRenameFunction, const FString&, const FString&, FText&);
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FRigVMGraphExplorer_OnCanRenameVariable, const FRigVMExplorerElementKey&, const FString&, FText&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnSetFunctionCategory, const FString&, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FString, FRigVMGraphExplorer_OnGetFunctionCategory, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FText, FRigVMGraphExplorer_OnGetFunctionTooltip, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FText, FRigVMGraphExplorer_OnGetVariableTooltip, const FString&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnSetVariableCategory, const FString&, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(FString, FRigVMGraphExplorer_OnGetVariableCategory, const FString&);
DECLARE_DELEGATE_RetVal(TSharedPtr<SWidget>, FRigVMGraphExplorer_OnRequestContextMenu);
DECLARE_DELEGATE_RetVal_OneParam(FEdGraphPinType, FRigVMGraphExplorer_OnGetVariablePinType, const FRigVMExplorerElementKey&);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnSetVariablePinType, const FRigVMExplorerElementKey&, const FEdGraphPinType&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMGraphExplorer_OnIsVariablePublic, const FString&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FRigVMGraphExplorer_OnToggleVariablePublic, const FString&);
DECLARE_DELEGATE_RetVal(TArray<TSharedPtr<IPinTypeSelectorFilter>>, FRigVMGraphExplorer_OnGetCustomPinFilters);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FRigVMGraphExplorer_OnReorderVariable, const FRigVMExplorerElementKey&, int32 /* new index */);

typedef STreeView<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>::FOnSelectionChanged FRigVMGraphExplorer_OnSelectionChanged;

struct FRigVMEditorGraphExplorerTreeDelegates
{
public:
	FRigVMGraphExplorer_OnGetRootGraphs OnGetRootGraphs;
	FRigVMGraphExplorer_OnGetChildrenGraphs OnGetChildrenGraphs;
	FRigVMGraphExplorer_OnGetEventNodesInGraph OnGetEventNodesInGraph;
	FRigVMGraphExplorer_OnGetFunctions OnGetFunctions;
	FRigVMGraphExplorer_OnGetVariables OnGetVariables;
	FRigVMGraphExplorer_OnGetVariables OnGetLocalVariables;
	FRigVMGraphExplorer_OnIsFunctionFocused OnIsFunctionFocused;
	FRigVMGraphExplorer_OnGetGraphDisplayName OnGetGraphDisplayName;
	FRigVMGraphExplorer_OnGetGraphDisplayName OnGetEventDisplayName;
	FRigVMGraphExplorer_OnGetGraphTooltip OnGetGraphTooltip;
	FRigVMGraphExplorer_OnGetGraphIcon OnGetGraphIcon;
	FRigVMGraphExplorer_OnGraphClicked OnGraphClicked;
	FRigVMGraphExplorer_OnEventClicked OnEventClicked;
	FRigVMGraphExplorer_OnFunctionClicked OnFunctionClicked;
	FRigVMGraphExplorer_OnVariableClicked OnVariableClicked;
	FRigVMGraphExplorer_OnGraphDoubleClicked OnGraphDoubleClicked;
	FRigVMGraphExplorer_OnEventDoubleClicked OnEventDoubleClicked;
	FRigVMGraphExplorer_OnFunctionDoubleClicked OnFunctionDoubleClicked;
	FRigVMGraphExplorer_OnCreateGraph OnCreateGraph;
	FRigVMGraphExplorer_OnCreateFunction OnCreateFunction;
	FRigVMGraphExplorer_OnCreateVariable OnCreateVariable;
	FRigVMGraphExplorer_OnCreateVariable OnCreateLocalVariable;
	FRigVMGraphExplorer_OnRenameGraph OnRenameGraph;
	FRigVMGraphExplorer_OnCanRenameGraph OnCanRenameGraph;
	FRigVMGraphExplorer_OnRenameFunction OnRenameFunction;
	FRigVMGraphExplorer_OnCanRenameFunction OnCanRenameFunction;
	FRigVMGraphExplorer_OnRenameVariable OnRenameVariable;
	FRigVMGraphExplorer_OnCanRenameVariable OnCanRenameVariable;
	FRigVMGraphExplorer_OnSetFunctionCategory OnSetFunctionCategory;
	FRigVMGraphExplorer_OnGetFunctionCategory OnGetFunctionCategory;
	FRigVMGraphExplorer_OnGetFunctionTooltip OnGetFunctionTooltip;
	FRigVMGraphExplorer_OnGetVariableTooltip OnGetVariableTooltip;
	FRigVMGraphExplorer_OnSetVariableCategory OnSetVariableCategory;
	FRigVMGraphExplorer_OnGetVariableCategory OnGetVariableCategory;
	FRigVMGraphExplorer_OnGetVariablePinType OnGetVariablePinType;
	FRigVMGraphExplorer_OnSetVariablePinType OnSetVariablePinType;
	FRigVMGraphExplorer_OnIsVariablePublic OnIsVariablePublic;
	FRigVMGraphExplorer_OnToggleVariablePublic OnToggleVariablePublic;
	FRigVMGraphExplorer_OnGetCustomPinFilters OnGetCustomPinFilters;
	FRigVMGraphExplorer_OnReorderVariable OnReorderVariable;
	FRigVMGraphExplorer_OnReorderVariable OnReorderLocalVariable;

	FRigVMGraphExplorer_OnSelectionChanged OnSelectionChanged;
	FRigVMGraphExplorer_OnRequestContextMenu OnRequestContextMenu;
	FOnDragDetected OnDragDetected;

	FRigVMEditorGraphExplorerTreeDelegates()
	{
	}

	TArray<const URigVMGraph*> GetRootGraphs() const
	{
		if (OnGetRootGraphs.IsBound())
		{
			return OnGetRootGraphs.Execute();
		}
		return TArray<const URigVMGraph*>();
	}

	TArray<const URigVMGraph*> GetChildrenGraphs(const FString& InParentPath) const
	{
		if (OnGetChildrenGraphs.IsBound())
		{
			return OnGetChildrenGraphs.Execute(InParentPath);
		}
		return TArray<const URigVMGraph*>();
	}

	TArray<URigVMNode*> GetEventNodesInGraph(const FString& InParentPath) const
	{
		if (OnGetEventNodesInGraph.IsBound())
		{
			return OnGetEventNodesInGraph.Execute(InParentPath);
		}
		return TArray<URigVMNode*>();
	}

	TArray<URigVMLibraryNode*> GetFunctions() const
	{
		if (OnGetFunctions.IsBound())
		{
			return OnGetFunctions.Execute();
		}
		return TArray<URigVMLibraryNode*>();
	}

	TArray<FRigVMGraphVariableDescription> GetVariables() const
	{
		if (OnGetVariables.IsBound())
		{
			return OnGetVariables.Execute();
		}
		return TArray<FRigVMGraphVariableDescription>();
	}

	TArray<FRigVMGraphVariableDescription> GetLocalVariables() const
	{
		if (OnGetLocalVariables.IsBound())
		{
			return OnGetLocalVariables.Execute();
		}
		return TArray<FRigVMGraphVariableDescription>();
	}

	bool IsFunctionFocused() const
	{
		if (OnIsFunctionFocused.IsBound())
		{
			return OnIsFunctionFocused.Execute();
		}
		return false;
	}

	FText GetGraphDisplayName(const FString& InPath) const
	{
		if (OnGetGraphDisplayName.IsBound())
		{
			return OnGetGraphDisplayName.Execute(InPath);
		}
		return FText();
	}

	FText GetEventDisplayName(const FString& InPath) const
	{
		if (OnGetEventDisplayName.IsBound())
		{
			return OnGetEventDisplayName.Execute(InPath);
		}
		return FText();
	}
	
	FText GetGraphTooltip(const FString& InPath) const
	{
		if (OnGetGraphTooltip.IsBound())
		{
			return OnGetGraphTooltip.Execute(InPath);
		}
		return FText();
	}

	const FSlateBrush* GetGraphIcon(const FString& InPath) const
	{
		if (OnGetGraphIcon.IsBound())
		{
			return OnGetGraphIcon.Execute(InPath);
		}
		return nullptr;
	}

	void GraphClicked(const FString& InPath) const
	{
		if (OnGraphClicked.IsBound())
		{
			OnGraphClicked.Execute(InPath);
		}
	}

	void EventClicked(const FString& InPath) const
	{
		if (OnEventClicked.IsBound())
		{
			OnEventClicked.Execute(InPath);
		}
	}

	void FunctionClicked(const FString& InPath) const
	{
		if (OnFunctionClicked.IsBound())
		{
			OnFunctionClicked.Execute(InPath);
		}
	}

	void VariableClicked(const FRigVMExplorerElementKey& InVariable) const
	{
		if (OnVariableClicked.IsBound())
		{
			OnVariableClicked.Execute(InVariable);
		}
	}

	void GraphDoubleClicked(const FString& InPath) const
	{
		if (OnGraphDoubleClicked.IsBound())
		{
			OnGraphDoubleClicked.Execute(InPath);
		}
	}

	void EventDoubleClicked(const FString& InPath) const
	{
		if (OnEventDoubleClicked.IsBound())
		{
			OnEventDoubleClicked.Execute(InPath);
		}
	}

	void FunctionDoubleClicked(const FString& InPath) const
	{
		if (OnFunctionDoubleClicked.IsBound())
		{
			OnFunctionDoubleClicked.Execute(InPath);
		}
	}

	void CreateGraph() const
	{
		if (OnCreateGraph.IsBound())
		{
			OnCreateGraph.Execute();
		}
	}

	void CreateFunction() const
	{
		if (OnCreateFunction.IsBound())
		{
			OnCreateFunction.Execute();
		}
	}

	void CreateVariable() const
	{
		if (OnCreateVariable.IsBound())
		{
			OnCreateVariable.Execute();
		}
	}

	void CreateLocalVariable() const
	{
		if (OnCreateLocalVariable.IsBound())
		{
			OnCreateLocalVariable.Execute();
		}
	}

	void RenameGraph(const FString& InOldPath, const FString& InNewPath) const
	{
		if (OnRenameGraph.IsBound())
		{
			OnRenameGraph.Execute(InOldPath, InNewPath);
		}
	}

	bool CanRenameGraph(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const
	{
		if (OnCanRenameGraph.IsBound())
		{
			return OnCanRenameGraph.Execute(InOldPath, InNewPath, OutErrorMessage);
		}
		return false;
	}

	void RenameFunction(const FString& InOldPath, const FString& InNewPath) const
	{
		if (OnRenameFunction.IsBound())
		{
			OnRenameFunction.Execute(InOldPath, InNewPath);
		}
	}

	bool CanRenameFunction(const FString& InOldPath, const FString& InNewPath, FText& OutErrorMessage) const
	{
		if (OnCanRenameFunction.IsBound())
		{
			return OnCanRenameFunction.Execute(InOldPath, InNewPath, OutErrorMessage);
		}
		return false;
	}

	void RenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName) const
	{
		if (OnRenameVariable.IsBound())
		{
			OnRenameVariable.Execute(InOldKey, InNewName);
		}
	}

	bool CanRenameVariable(const FRigVMExplorerElementKey& InOldKey, const FString& InNewName, FText& OutErrorMessage) const
	{
		if (OnCanRenameVariable.IsBound())
		{
			return OnCanRenameVariable.Execute(InOldKey, InNewName, OutErrorMessage);
		}
		return false;
	}

	void SetFunctionCategory(const FString& InPath, const FString& InCategory) const
	{
		if (OnSetFunctionCategory.IsBound())
		{
			OnSetFunctionCategory.Execute(InPath, InCategory);
		}
	}

	FString GetFunctionCategory(const FString& InPath) const
	{
		if (OnGetFunctionCategory.IsBound())
		{
			return OnGetFunctionCategory.Execute(InPath);
		}
		return FString(); 
	}

	FText GetFunctionTooltip(const FString& InPath) const
	{
		if (OnGetFunctionTooltip.IsBound())
		{
			return OnGetFunctionTooltip.Execute(InPath);
		}
		return FText();
	}

	FText GetVariableTooltip(const FString& InPath) const
	{
		if (OnGetVariableTooltip.IsBound())
		{
			return OnGetVariableTooltip.Execute(InPath);
		}
		return FText();
	}

	void SetVariableCategory(const FString& InPath, const FString& InCategory) const
	{
		if (OnSetVariableCategory.IsBound())
		{
			OnSetVariableCategory.Execute(InPath, InCategory);
		}
	}

	FString GetVariableCategory(const FString& InPath) const
	{
		if (OnGetVariableCategory.IsBound())
		{
			return OnGetVariableCategory.Execute(InPath);
		}
		return FString(); 
	}

	void HandleSelectionChanged(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> Selection, ESelectInfo::Type SelectInfo)
	{
		if(bSuspendSelectionDelegate)
		{
			return;
		}
		TGuardValue<bool> Guard(bSuspendSelectionDelegate, true);
		(void)OnSelectionChanged.ExecuteIfBound(Selection, SelectInfo);
	}

	TSharedPtr<SWidget> RequestContextMenu()
	{
		if (OnRequestContextMenu.IsBound())
		{
			return OnRequestContextMenu.Execute();
		}
		return nullptr;
	}

	FEdGraphPinType GetVariablePinType(const FRigVMExplorerElementKey& InVariableKey)
	{
		if (OnGetVariablePinType.IsBound())
		{
			return OnGetVariablePinType.Execute(InVariableKey);
		}
		return FEdGraphPinType();
	}

	bool SetVariablePinType(const FRigVMExplorerElementKey& InVariableKey, const FEdGraphPinType& InType)
	{
		if (OnSetVariablePinType.IsBound())
		{
			return OnSetVariablePinType.Execute(InVariableKey, InType);
		}
		return false;
	}

	bool IsVariablePublic(const FString& InVariable)
	{
		if (OnIsVariablePublic.IsBound())
		{
			return OnIsVariablePublic.Execute(InVariable);
		}
		return false;
	}

	bool ToggleVariablePublic(const FString& InVariable)
	{
		if (OnToggleVariablePublic.IsBound())
		{
			return OnToggleVariablePublic.Execute(InVariable);
		}
		return false;
	}

	TArray<TSharedPtr<IPinTypeSelectorFilter>> GetCustomPinFilters()
	{
		if (OnGetCustomPinFilters.IsBound())
		{
			return OnGetCustomPinFilters.Execute();
		}
		return TArray<TSharedPtr<IPinTypeSelectorFilter>>();
	}

private:

	bool bSuspendSelectionDelegate = false;

	friend class SRigVMEditorGraphExplorerTreeView;
};

class SRigVMEditorGraphExplorerItem  : public STableRow<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>
{
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnAddClickedOnSection, const FRigVMExplorerElementKey&);
	
	using FRigVMEdGraphNodeRegistry = UE::RigVMEditor::FRigVMEdGraphNodeRegistry;

public:
	SLATE_BEGIN_ARGS(SRigVMEditorGraphExplorerItem)
	{}
		SLATE_EVENT( FOnAddClickedOnSection, OnAddClickedOnSection )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs,
		const TSharedRef<STableViewBase>& InOwnerTable,
		TSharedRef<FRigVMEditorGraphExplorerTreeElement> InElement,
		TSharedPtr<SRigVMEditorGraphExplorerTreeView> InTreeView);

	TSharedRef<SWidget> CreateIconWidget(const FRigVMExplorerElementKey& Key);
	TSharedRef<SWidget> CreateTextSlotWidget(const FRigVMExplorerElementKey& Key, const FText& InHighlightText);
	FText GetDisplayText() const;
	FText GetItemTooltip() const;
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	FReply OnAddButtonClickedOnSection(TSharedRef<FRigVMEditorGraphExplorerTreeElement> InElement);

	// Drag and drop support for reordering
	virtual TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigVMEditorGraphExplorerTreeElement> TargetItem);
	virtual FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FRigVMEditorGraphExplorerTreeElement> TargetItem);

private:
	/** Updates OptionalIsUsedInGraph */
	void UpdateIsUsedInGraph();

	/** Returns a registry for ed graph nodes this item uses, or nullptr if no registry exists */
	TSharedPtr<FRigVMEdGraphNodeRegistry> GetEdGraphNodeRegistry() const;

	TWeakPtr<FRigVMEditorGraphExplorerTreeElement> WeakExplorerElement;
	TWeakPtr<SRigVMEditorGraphExplorerTreeView> WeakTreeView;
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;
	FRigVMEditorGraphExplorerTreeDelegates Delegates;

	/** If set and true, there are occurences in the graph. Only set for supported types (functions and variables as of 5.7) */
	TOptional<bool> OptionalIsUsedInGraph;

	FOnAddClickedOnSection OnAddClickedOnSection;
};


/** An item in the tree */
class FRigVMEditorGraphExplorerTreeElement : public TSharedFromThis<FRigVMEditorGraphExplorerTreeElement>
{
public:
	FRigVMEditorGraphExplorerTreeElement(const FRigVMExplorerElementKey& InKey, TWeakPtr<SRigVMEditorGraphExplorerTreeView> InTreeView)
		: Key(InKey) {}

	FRigVMExplorerElementKey Key;
	TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> Children;

	TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigVMEditorGraphExplorerTreeElement> InRigTreeElement, TSharedPtr<SRigVMEditorGraphExplorerTreeView> InTreeView);

	void RequestRename();

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

};

class SRigVMEditorGraphExplorerTreeView : public STreeView<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>
{
	using FRigVMEdGraphNodeRegistry = UE::RigVMEditor::FRigVMEdGraphNodeRegistry;

public:

	SLATE_BEGIN_ARGS(SRigVMEditorGraphExplorerTreeView)
	{}
		SLATE_ARGUMENT(FRigVMEditorGraphExplorerTreeDelegates, RigTreeDelegates)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SRigVMEditorGraphExplorer>& InGraphExplorer);
	virtual ~SRigVMEditorGraphExplorerTreeView() = default;

	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	void SaveAndClearSparseItemInfos()
	{
		// Only save the info if there is something to save (do not overwrite info with an empty map)
		if (!SparseItemInfos.IsEmpty())
		{
			OldSparseItemInfos = SparseItemInfos;
		}
		ClearExpandedItems();
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	void RestoreSparseItemInfos(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> ItemPtr)
	{
		for (const auto& Pair : OldSparseItemInfos)
		{
			if (Pair.Key->Key == ItemPtr->Key)
			{
				// the SparseItemInfos now reference the new element, but keep the same expansion state
				SparseItemInfos.Add(ItemPtr, Pair.Value);
				break;
			}
		}
	}

	void RefreshTreeView(bool bRebuildContent = true);
	TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedPtr< SWidget > CreateContextMenu();
	void SetExpansionRecursive(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
	void HandleGetChildrenForTree(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InItem, TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& OutChildren);
	TArray<FRigVMExplorerElementKey> GetKeys() const;
	TArray<FRigVMExplorerElementKey> GetSelectedKeys() const;
	void SetSelection(TArray<FRigVMExplorerElementKey>& InSelectedKeys);
	TSharedPtr<FRigVMEditorGraphExplorerTreeElement> FindElement(const FRigVMExplorerElementKey& Key);
	
	const TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>>& GetRootElements() const { return RootElements; }
	FRigVMEditorGraphExplorerTreeDelegates& GetRigTreeDelegates() { return Delegates; }
	void OnItemClicked(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement);
	void OnItemDoubleClicked(TSharedPtr<FRigVMEditorGraphExplorerTreeElement> InElement);
	FReply OnAddButtonClickedOnSection(const FRigVMExplorerElementKey& InSectionKey);

	/** Returns the type of section the key references, or ERigVMGraphExplorerSectionType::None if the key doesn't reference a section.  */
	ERigVMGraphExplorerSectionType GetSectionType(const FRigVMExplorerElementKey& Key) const;

	/** Returns the outer graph explorer or nullptr if there is no valid outer graph explorer */
	TSharedPtr<SRigVMEditorGraphExplorer> GetGraphExplorer() const;

private:

	/** Returns true if the eleemnt refers to a variable used in the graph. The element's key type has to correspond to a variable (ensured) */
	bool IsVariableElementUsedInGraph(const TSharedRef<FRigVMEditorGraphExplorerTreeElement>& Element) const;

	/** A temporary snapshot of the SparseItemInfos in STreeView, used during RefreshTreeView() */
	TSparseItemMap OldSparseItemInfos;

	/** Backing array for tree view */
	TArray<TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> RootElements;

	/** A map for looking up items based on their key */
	TMap<FRigVMExplorerElementKey, TSharedPtr<FRigVMEditorGraphExplorerTreeElement>> ElementMap;

	/** A map for looking up a parent based on their key */
	TMap<FRigVMExplorerElementKey, FRigVMExplorerElementKey> ParentMap;
	
	FRigVMEditorGraphExplorerTreeDelegates Delegates;

	FText FilterText;

	/** The outer graph explorer widget */
	TWeakPtr<SRigVMEditorGraphExplorer> WeakGraphExplorer;
	
	bool bWasAddNewItemClicked = false;
	
	friend class SRigVMEditorGraphExplorer;
	friend class SRigVMEditorGraphExplorerItem;
};
