// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/RigVMEditor.h"
#include "Widgets/Input/SComboBox.h"

#define UE_API RIGVMEDITOR_API

struct FRigVMDebugObjectInstance
{
	/** Actual object to debug, can be null */
	TWeakObjectPtr<UObject> ObjectPtr;

	/** Friendly label for object to debug */
	FString ObjectLabel;

	/** Raw object path of spawned PIE object, this is not a SoftObjectPath because we don't want it to get fixed up */
	FString ObjectPath;

	/** Object path to object in the editor, will only be set for static objects */
	FString EditorObjectPath;

	FRigVMDebugObjectInstance(TWeakObjectPtr<UObject> InPtr, const FString& InLabel)
		: ObjectPtr(InPtr)
		, ObjectLabel(InLabel)
	{
	}

	/** Returns true if this is the special entry for no specific object */
	bool IsEmptyObject() const
	{
		return ObjectPath.IsEmpty();
	}

	/** If this has no editor path, it was spawned */
	bool IsSpawnedObject() const
	{
		return !ObjectPath.IsEmpty() && EditorObjectPath.IsEmpty();
	}

	/** If editor and object path are the same length because there's no prefix, this is the editor object */
	bool IsEditorObject() const
	{
		return !ObjectPath.IsEmpty() && ObjectPath.Len() == EditorObjectPath.Len();
	}
};

class SRigVMEditorSelectedDebugObjectWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMEditorSelectedDebugObjectWidget){}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedPtr<IRigVMEditor> InEditor);

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	/** Adds an object to the list of debug choices */
	UE_API void AddDebugObject(UObject* TestObject, const FString& TestObjectName = FString());

private:
	UE_API FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const;

	/** Creates a list of all debug objects **/
	UE_API void GenerateDebugObjectInstances(bool bRestoreSelection);

	/** Generate list of active PIE worlds to debug **/
	UE_API void GenerateDebugWorldNames(bool bRestoreSelection);

	/** Refresh the widget. **/
	UE_API void OnRefresh();

	/** Returns the entry for the current debug actor */
	UE_API TSharedPtr<FRigVMDebugObjectInstance> GetDebugObjectInstance() const;

	/** Returns the name of the current debug actor */
	UE_API TSharedPtr<FString> GetDebugWorldName() const;

	/** Handles the selection changed event for the debug actor combo box */
	UE_API void DebugObjectSelectionChanged(TSharedPtr<FRigVMDebugObjectInstance> NewSelection, ESelectInfo::Type SelectInfo);

	UE_API void DebugWorldSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);

	/** Called when user clicks button to select the current object being debugged */
	UE_API void SelectedDebugObject_OnClicked();

	/** Returns true if a debug actor is currently selected */
	UE_API bool IsDebugObjectSelected() const;

	UE_API EVisibility IsDebugWorldComboVisible() const;

	/* Returns the string to indicate no debug object is selected */
	UE_API const FString& GetNoDebugString() const;

	UE_API const FString& GetDebugAllWorldsString() const;

	/** Helper method to construct a debug object label string */
	UE_API FString MakeDebugObjectLabel(UObject* TestObject, bool bAddContextIfSelectedInEditor, bool bAddSpawnedContext) const;

	/** Fills in data for a specific instance */
	UE_API void FillDebugObjectInstance(TSharedPtr<FRigVMDebugObjectInstance> Instance);

	/** Called to create a widget for each debug object item */
	UE_API TSharedRef<SWidget> CreateDebugObjectItemWidget(TSharedPtr<FRigVMDebugObjectInstance> InItem);

	/** Returns the combo button label to use for the currently-selected debug object item */
	UE_API FText GetSelectedDebugObjectTextLabel() const;

private:
	/** Pointer back to the rigvm editor tool that owns us */
	TWeakPtr<IRigVMEditor> Editor;

	/** Lists of actors of a given blueprint type and their names */
	TArray<TSharedPtr<FRigVMDebugObjectInstance>> DebugObjects;

	/** PIE worlds that we can debug */
	TArray< TWeakObjectPtr<UWorld> > DebugWorlds;
	TArray< TSharedPtr<FString> > DebugWorldNames;

	/** Widget containing the names of all possible debug actors. This is a "generic" SComboBox rather than an STextComboBox so that we can customize the label on the combo button widget. */
	TSharedPtr<SComboBox<TSharedPtr<FRigVMDebugObjectInstance>>> DebugObjectsComboBox;

	TSharedPtr<STextComboBox> DebugWorldsComboBox;

	TWeakObjectPtr<UObject> LastObjectObserved;
};

#undef UE_API
