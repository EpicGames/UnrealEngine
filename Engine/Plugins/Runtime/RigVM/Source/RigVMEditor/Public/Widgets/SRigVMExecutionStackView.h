// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/RigVMEdGraph.h"
#include "RigVMHost.h"
#include "RigVMBlueprintLegacy.h"
#include "Widgets/Views/STreeView.h"
#include "RigVMCore/RigVM.h"
#include "RigVMBlueprintLegacy.h"

#define UE_API RIGVMEDITOR_API

class IRigVMEditor;
class SRigVMExecutionStackView;
class FUICommandList;
class SSearchBox;

namespace ERigStackEntry
{
	enum Type
	{
		Error,
		Warning,
		Info,
		Callable,
		Instruction,
		Block,
	};
}

/** An item in the stack */
class FRigStackEntry : public TSharedFromThis<FRigStackEntry>
{
public:
	UE_API FRigStackEntry(int32 InEntryIndex, ERigStackEntry::Type InEntryType, int32 InSubjectIndex, ERigVMOpCode InOpCode, const FString& InLabel, const FRigVMASTProxy& InProxy);

	UE_DEPRECATED(5.7, "Use TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigStackEntry> InEntry, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SRigVMExecutionStackView> InStackView, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint); instead")
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigStackEntry> InEntry, TSharedRef<FUICommandList> InCommandList, TSharedPtr<SRigVMExecutionStackView> InStackView, TWeakObjectPtr<UObject> InBlueprint);
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FRigStackEntry> InEntry, TSharedRef<FUICommandList>
	                                               InCommandList, TSharedPtr<SRigVMExecutionStackView> InStackView, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint);

	friend uint32 GetTypeHash(const FRigStackEntry& InEntry)
	{
		return HashCombine(
			GetTypeHash(InEntry.EntryType),
			GetTypeHash(InEntry.SubjectIndex),
			GetTypeHash(InEntry.OpCode),
			GetTypeHash(InEntry.Label)
		);
	}

	void FilterChildren(const FString& InSearchText);
	bool MatchesFilter(const FString& InSearchText);

	int32 EntryIndex;
	ERigStackEntry::Type EntryType;
	int32 SubjectIndex;
	FString CallPath;
	FRigVMCallstack Callstack;
	ERigVMOpCode OpCode;
	FString Label;
	TArray<TSharedPtr<FRigStackEntry>> Children;
	TOptional<TArray<TSharedPtr<FRigStackEntry>>> FilteredChildren;
	mutable TOptional<bool> ShowAsFadedOut;
	mutable TOptional<FText> VisitedCountText;
	mutable TOptional<FText> DurationText;
	mutable double MicroSeconds;
	mutable TArray<double> MicroSecondsFrames;
};

class SRigStackItem : public STableRow<TSharedPtr<FRigStackEntry>>
{
	SLATE_BEGIN_ARGS(SRigStackItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FRigStackEntry> InStackEntry, TSharedRef<FUICommandList> InCommandList, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint);

private:
	TWeakPtr<FRigStackEntry> WeakStackEntry;
	TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakBlueprint;
	TWeakPtr<FUICommandList> WeakCommandList;

	FText GetIndexText() const;
	FText GetLabelText() const;
	FSlateColor GetLabelColorAndOpacity() const;
	FText GetTooltip() const;
	FText GetVisitedCountText() const;
	FText GetDurationText() const;
};

class SRigVMExecutionStackTreeView : public STreeView<TSharedPtr<FRigStackEntry>>
{
	typedef STreeView<TSharedPtr<FRigStackEntry>> BaseView;
	
public:
	
	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	void SaveAndClearSparseItemInfos()
	{
		// Only save the info if there is something to save (do not overwrite info with an empty map)
		if (!BaseView::SparseItemInfos.IsEmpty())
		{
			OldSparseItemInfos = BaseView::SparseItemInfos;
			// Pre-build a hash-keyed map so RestoreSparseItemInfos can do O(1) lookup
			// instead of an O(N) linear scan per entry (avoid O(N^2) on large rigs).
			OldSparseItemInfosByHash.Reset();
			OldSparseItemInfosByHash.Reserve(OldSparseItemInfos.Num());
			for (const auto& Pair : OldSparseItemInfos)
			{
				OldSparseItemInfosByHash.Add(GetTypeHash(*Pair.Key.Get()), Pair.Value);
			}
		}
		ClearExpandedItems();
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	void RestoreSparseItemInfos(TSharedPtr<FRigStackEntry> EntryPtr)
	{
		const uint32 NewHash = GetTypeHash(*EntryPtr.Get());
		if (const FSparseItemInfo* Found = OldSparseItemInfosByHash.Find(NewHash))
		{
			// the SparseItemInfos now reference the new element, but keep the same expansion state
			BaseView::SparseItemInfos.Add(EntryPtr, *Found);
		}
		else
		{
			// set default state as expanded if not found
			BaseView::SparseItemInfos.Add(EntryPtr, FSparseItemInfo(true, false));
		}
	}

private:
	TSparseItemMap OldSparseItemInfos;
	TMap<uint32, FSparseItemInfo> OldSparseItemInfosByHash;
};

class SRigVMExecutionStackView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMExecutionStackView) {}
	SLATE_END_ARGS()

	UE_API ~SRigVMExecutionStackView();

	UE_API void Construct(const FArguments& InArgs, TSharedRef<IRigVMEditor> InRigVMEditor);

	/** Set Selection Changed */
	UE_API void OnSelectionChanged(TSharedPtr<FRigStackEntry> Selection, ESelectInfo::Type SelectInfo);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API TSharedPtr< SWidget > CreateContextMenu();

protected:

	/** Rebuild the tree view */
	UE_API void RefreshTreeView(URigVM* InVM, FRigVMExtendedExecuteContext* InVMContext, bool bRebuildEntries);

private:

	/** Bind commands that this widget handles */
	UE_API void BindCommands();

	/** Make a row widget for the table */
	UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FRigStackEntry> InItem, const TSharedRef<STableViewBase>& OwnerTable, TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint);

	/** Get children for the tree */
	UE_API void HandleGetChildrenForTree(TSharedPtr<FRigStackEntry> InItem, TArray<TSharedPtr<FRigStackEntry>>& OutChildren);

	/** Focus on the selected operator in the graph*/
	UE_API void HandleFocusOnSelectedGraphNode();

	/** Offers a dialog to move to a specific instruction */
	UE_API void HandleGoToInstruction();

	/** Selects the target instructions for the current selection */
	UE_API void HandleSelectTargetInstructions();

	/** Sets or unsets the preview here early exit instruction */
	UE_API void HandleToggleEarlyExitInstruction();
	
	/** Steps to the next early exist instruction */
	UE_API void HandleStepEarlyExitInstruction();

	UE_API TArray<TSharedPtr<FRigStackEntry>> GetTargetItems(const TArray<TSharedPtr<FRigStackEntry>>& InItems) const;

	UE_API void UpdateTargetItemHighlighting();

	UE_API void OnVMCompiled(UObject* InCompiledObject, URigVM* InCompiledVM, FRigVMExtendedExecuteContext& InVMContext);
	UE_API void OnObjectBeingDebuggedChanged(UObject* InObjectBeingDebugged);

	/** Search box widget */
	TSharedPtr<SSearchBox> FilterBox;
	FText FilterText;
	UE_API void OnFilterTextChanged(const FText& SearchText);


	bool bSuspendModelNotifications;
	bool bSuspendControllerSelection;
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);
	UE_API void HandleHostInitializedEvent(URigVMHost* InHost, const FName& InEventName);
	UE_API void HandleHostExecutedEvent(URigVMHost* InHost, const FName& InEventName);
	UE_API void HandlePreviewHostUpdated(IRigVMEditor* InEditor);
	UE_API void HandleItemMouseDoubleClick(TSharedPtr<FRigStackEntry> InItem);

	/** Populate the execution stack with descriptive names for each instruction */
	UE_API void PopulateStackView(URigVM* InVM, FRigVMExtendedExecuteContext* InVMContext);

	void BindHostDelegates();
	void BindHostDelegates(UObject* InObjectBeingDebugged);
	void UnbindHostDelegates();
	void UnbindAssetDelegates();

	TSharedPtr<FRigStackEntry> MakeEntry(ERigStackEntry::Type InEntryType, int32 InSubjectIndex, ERigVMOpCode InOpCode, const FString& InLabel, const FRigVMASTProxy& InProxy);

	TSharedPtr<SRigVMExecutionStackTreeView> TreeView;

	/** Command list we bind to */
	TSharedPtr<FUICommandList> CommandList;

	TWeakPtr<IRigVMEditor> WeakEditor;
	TWeakInterfacePtr<IRigVMEditorAssetInterface> WeakRigVMBlueprint;

	TArray<TSharedPtr<FRigStackEntry>> RootEntries;
	TArray<TSharedPtr<FRigStackEntry>> FilteredRootEntries;
	TArray<TSharedPtr<FRigStackEntry>> AllEntries;
	TMap<int32, TSharedPtr<FRigStackEntry>> InstructionToEntry;
	TMap<int32, TSharedPtr<FRigStackEntry>> CallableToEntry;
	TArray<TTuple<int32,int32>> Blocks;
	TMap<TTuple<int32,int32>, TSharedPtr<FRigStackEntry>> BlockToEntry;

	FDelegateHandle OnModelModified;
	FDelegateHandle OnHostInitializedHandle;
	FDelegateHandle OnHostExecutedHandle;
	FDelegateHandle OnVMCompiledHandle;
	FDelegateHandle OnPreviewHostUpdatedHandle;
	FDelegateHandle OnObjectBeingDebuggedChangedHandle;
	TWeakObjectPtr<URigVMHost> PreviouslyDebuggedHost;
};

#undef UE_API
