// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintLegacy.h"
#include "Dialogs/Dialogs.h"
#include "IAssetTypeActions.h"

#define UE_API RIGVMEDITOR_API

class ITableRow;
class STableViewBase;

DECLARE_DELEGATE_OneParam(FRigVMOnFocusOnLinkRequestedDelegate, URigVMLink*);

class SRigVMGraphBreakLinksWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphBreakLinksWidget) {}
	SLATE_ARGUMENT(FRigVMOnFocusOnLinkRequestedDelegate, OnFocusOnLink)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TArray<URigVMLink*> InLinks);

private:

	/** Handler for when the user double clicks, presses enter, or presses space on an asset */
	UE_API void OnLinkDoubleClicked(const URigVMLink* InLink, EAssetTypeActivationMethod::Type ActivationMethod);

	UE_API TSharedRef<ITableRow> GenerateItemRow(URigVMLink* Item, const TSharedRef<STableViewBase>& OwnerTable);
 
	TArray<URigVMLink*> Links;
	FRigVMOnFocusOnLinkRequestedDelegate OnFocusOnLink;

	UE_API void HandleItemMouseDoubleClick(URigVMLink* InItem);

	friend class SRigVMGraphBreakLinksDialog;
};

class SRigVMGraphBreakLinksDialog : public SWindow
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMGraphBreakLinksDialog)
	{
	}

	SLATE_ARGUMENT(TArray<URigVMLink*>, Links)
	SLATE_ARGUMENT(FRigVMOnFocusOnLinkRequestedDelegate, OnFocusOnLink)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API EAppReturnType::Type ShowModal();

protected:

	TSharedPtr<SRigVMGraphBreakLinksWidget> BreakLinksWidget;
	
	UE_API FReply OnButtonClick(EAppReturnType::Type ButtonID);
	UE_API bool IsOkButtonEnabled() const;
	EAppReturnType::Type UserResponse;
};

#undef UE_API
