// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "RigVMModel/RigVMPin.h"
#include "IPropertyAccessEditor.h"
#include "RigVMBlueprintLegacy.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_OneParam(FOnTypeSelected, TRigVMTypeIndex);

class SRigVMGraphChangePinType : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigVMGraphChangePinType)
	: _Types()
	, _OnTypeSelected(nullptr)
	{}

		SLATE_ARGUMENT(TArray<TRigVMTypeIndex>, Types)
		SLATE_EVENT(FOnTypeSelected, OnTypeSelected)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

protected:

	static UE_API FText GetBindingText(const FRigVMTemplateArgumentType& InType);
	UE_API FText GetBindingText(URigVMPin* ModelPin) const;
	UE_API FText GetBindingText() const;
	UE_API const FSlateBrush* GetBindingImage() const;
	UE_API FLinearColor GetBindingColor() const;
	UE_API bool OnCanBindProperty(FProperty* InProperty) const;
	UE_API bool OnCanBindToClass(UClass* InClass) const;
	UE_API void OnAddBinding(FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain);
	UE_API void FillPinTypeMenu( FMenuBuilder& MenuBuilder );
	UE_API void HandlePinTypeChanged(FRigVMTemplateArgumentType InType);

	TArray<TRigVMTypeIndex> Types;
	FOnTypeSelected OnTypeSelected;
	FPropertyBindingWidgetArgs BindingArgs;
};

#undef UE_API
