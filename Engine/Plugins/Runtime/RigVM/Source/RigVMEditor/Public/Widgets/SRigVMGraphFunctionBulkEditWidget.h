// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "Dialogs/Dialogs.h"
#include "IAssetTypeActions.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphFunctionBulkEditWidget;

class SRigVMGraphFunctionBulkEditWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphFunctionBulkEditWidget) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, FRigVMEditorAssetInterfacePtr InBlueprint, URigVMController* InController, URigVMLibraryNode* InFunction, ERigVMControllerBulkEditType InEditType);

private:

	/** Handler for when an asset context menu has been requested. */
	UE_API TSharedPtr<SWidget> OnGetAssetContextMenu( const TArray<FAssetData>& SelectedAssets );

	/** Handler for when the user double clicks, presses enter, or presses space on an asset */
	UE_API void OnAssetsActivated(const TArray<FAssetData>& ActivatedAssets, EAssetTypeActivationMethod::Type ActivationMethod);

	/** Show a progress bar and load all of the assets */
	UE_API void LoadAffectedAssets();

	FRigVMEditorAssetInterfacePtr Blueprint;
	URigVMController* Controller;
	URigVMLibraryNode* Function;
	ERigVMControllerBulkEditType EditType;

	UE_API TSharedRef<SWidget> MakeAssetViewForReferencedAssets();

	friend class SRigVMGraphFunctionBulkEditDialog;
};

class SRigVMGraphFunctionBulkEditDialog : public SWindow
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMGraphFunctionBulkEditDialog)
	{
	}

	SLATE_ARGUMENT(FRigVMEditorAssetInterfacePtr, Blueprint)
	SLATE_ARGUMENT(URigVMController*, Controller)
	SLATE_ARGUMENT(URigVMLibraryNode*, Function)
	SLATE_ARGUMENT(ERigVMControllerBulkEditType, EditType)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API EAppReturnType::Type ShowModal();

protected:

	TSharedPtr<SRigVMGraphFunctionBulkEditWidget> BulkEditWidget;
	
	UE_API FReply OnButtonClick(EAppReturnType::Type ButtonID);
	UE_API bool IsOkButtonEnabled() const;
	EAppReturnType::Type UserResponse;
};

#undef UE_API
