// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMModel/Nodes/RigVMLibraryNode.h"
#include "Widgets/Views/SListView.h"
#include "Dialogs/Dialogs.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphFunctionLocalizationWidget;

class SRigVMGraphFunctionLocalizationItem : public TSharedFromThis<SRigVMGraphFunctionLocalizationItem>
{
public:
	UE_API SRigVMGraphFunctionLocalizationItem(const FRigVMGraphFunctionIdentifier& InFunction);

	FText DisplayText;
	FText ToolTipText;
	const FRigVMGraphFunctionIdentifier Function;
};

class SRigVMGraphFunctionLocalizationTableRow : public STableRow<TSharedPtr<SRigVMGraphFunctionLocalizationItem>>
{
	SLATE_BEGIN_ARGS(SRigVMGraphFunctionLocalizationTableRow) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, SRigVMGraphFunctionLocalizationWidget* InLocalizationWidget, TSharedRef<SRigVMGraphFunctionLocalizationItem> InFunctionItem);
};

class SRigVMGraphFunctionLocalizationWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphFunctionLocalizationWidget) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FRigVMGraphFunctionIdentifier& InFunctionToLocalize, IRigVMGraphFunctionHost* InTargetFunctionHost);

	UE_API TSharedRef<ITableRow> GenerateFunctionListRow(TSharedPtr<SRigVMGraphFunctionLocalizationItem> InItem, const TSharedRef<STableViewBase>& InOwningTable);
	UE_API ECheckBoxState IsFunctionEnabled(const FRigVMGraphFunctionIdentifier InFunction) const;
	UE_API void SetFunctionEnabled(ECheckBoxState NewState, const FRigVMGraphFunctionIdentifier InFunction);
	UE_API bool IsFunctionPublic(const FRigVMGraphFunctionIdentifier InFunction) const;

private:

	TArray<FRigVMGraphFunctionIdentifier> FunctionsToLocalize;
	TArray<TSharedPtr<SRigVMGraphFunctionLocalizationItem>> FunctionItems;
	TMap<FRigVMGraphFunctionIdentifier, TSharedRef<SRigVMGraphFunctionLocalizationTableRow>> TableRows;

	friend class SRigVMGraphFunctionLocalizationTableRow;
	friend class SRigVMGraphFunctionLocalizationDialog;
};

class SRigVMGraphFunctionLocalizationDialog : public SWindow
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMGraphFunctionLocalizationDialog)
	{
	}

	SLATE_ARGUMENT(FRigVMGraphFunctionIdentifier, Function)
	SLATE_ARGUMENT(IRigVMGraphFunctionHost*, GraphFunctionHost)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API EAppReturnType::Type ShowModal();
	UE_API TArray<FRigVMGraphFunctionIdentifier>& GetFunctionsToLocalize();

protected:

	TSharedPtr<SRigVMGraphFunctionLocalizationWidget> FunctionsWidget;
	
	UE_API FReply OnButtonClick(EAppReturnType::Type ButtonID);
	UE_API bool IsOkButtonEnabled() const;
	EAppReturnType::Type UserResponse;
};

#undef UE_API
