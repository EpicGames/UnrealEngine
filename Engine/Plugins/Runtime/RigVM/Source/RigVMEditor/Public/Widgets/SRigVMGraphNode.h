// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphNode.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMBlueprintLegacy.h"
#include "Overrides/SOverrideStatusWidget.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraphNode;
class STableViewBase;
class SOverlay;
class SGraphPin;
class UEdGraphPin;

class SRigVMGraphPinReorderHandle : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_RetVal_ThreeParams(int32, FCanReorderDelegate, const URigVMPin* /* Source */, const URigVMPin* /* Target */, EItemDropZone);
	DECLARE_DELEGATE_ThreeParams(FReorderDelegate, const URigVMPin* /* Source */, const URigVMPin* /* Target */, EItemDropZone);
	
	SLATE_BEGIN_ARGS(SRigVMGraphPinReorderHandle)
		: _bIsInput(true)
		, _Width(50.f)
		, _Height(24.f)
		, _LineExtension(0.f)
	{}
		SLATE_ARGUMENT(bool, bIsInput)
		SLATE_ARGUMENT(float, Width)
		SLATE_ARGUMENT(float, Height)
		SLATE_ARGUMENT(TWeakObjectPtr<URigVMPin>, Pin)
		SLATE_ATTRIBUTE(float, LineExtension)
		SLATE_EVENT(FCanReorderDelegate, CanReorder)
		SLATE_EVENT(FReorderDelegate, OnReorder)
		SLATE_ARGUMENT(TWeakInterfacePtr<IRigVMClientHost>, WeakClientHost)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:

	bool bIsInput = true;
	TWeakObjectPtr<URigVMPin> Pin;
	TAttribute<float> LineExtension;
	FCanReorderDelegate CanReorderDelegate;
	FReorderDelegate ReorderDelegate;
	TOptional<EItemDropZone> DropZone;
	TWeakInterfacePtr<IRigVMClientHost> WeakClientHost;

	friend class SRigVMGraphNode;
};

class SRigVMGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphNode)
		: _GraphNodeObj(nullptr)
		, _Enabled(true)
		, _PinDragDropEnabled(true)
		{}

	SLATE_ARGUMENT(URigVMEdGraphNode*, GraphNodeObj)
	SLATE_ATTRIBUTE(bool, Enabled)
	SLATE_ATTRIBUTE(bool, PinDragDropEnabled)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	// SGraphNode interface
	UE_API virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle) override;

	UE_API virtual void EndUserInteraction() const override;
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	UE_API virtual void AddPin( const TSharedRef<SGraphPin>& PinToAdd ) override;
	UE_API virtual void CreateStandardPinWidget(UEdGraphPin* CurPin) override;
	virtual void SetDefaultTitleAreaWidget(TSharedRef<SOverlay> DefaultTitleAreaWidget) override
	{
		TitleAreaWidget = DefaultTitleAreaWidget;
	}
	UE_API virtual const FSlateBrush * GetNodeBodyBrush() const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	UE_API virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	UE_API virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	UE_API virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	UE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;

	UE_API virtual void RefreshErrorInfo() override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual bool IsHidingPinWidgets() const override { return UseLowDetailNodeContent(); }
	UE_API virtual bool UseLowDetailPinNames() const override;
	UE_API virtual void UpdateGraphNode() override;
	UE_API void UpdateStandardNode();
	UE_API void UpdateCompactNode();
	UE_API virtual TOptional<FSlateColor> GetPinTextColor(const SGraphPin* InGraphPin) const override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	UE_API void CreateAddPinButton();

	UE_API void CreateWorkflowWidgets();

	/** Callback function executed when Add pin button is clicked */
	UE_API virtual FReply OnAddPin() override;

protected:

	UE_API bool UseLowDetailNodeContent() const;

	UE_API FVector2D GetLowDetailDesiredSize() const;

	UE_API EVisibility GetTitleVisibility() const;
	UE_API EVisibility GetArrayPlusButtonVisibility(URigVMPin* InModelPin) const;

	UE_API FText GetPinLabel(TWeakPtr<SGraphPin> GraphPin) const;

	UE_API virtual TOptional<FSlateColor> GetHighlightColor(const SGraphPin* InGraphPin) const override;
	UE_API FSlateColor GetVariableLabelTextColor(TWeakObjectPtr<URigVMFunctionReferenceNode> FunctionReferenceNode, FName InVariableName) const;
	UE_API FText GetVariableLabelTooltipText(TWeakInterfacePtr<IRigVMEditorAssetInterface> InBlueprint, FName InVariableName) const;

	UE_API FReply HandleAddArrayElement(FString InModelPinPath);

	UE_API void HandleNodeTitleDirtied();
	UE_API void HandleNodePinsChanged();
	UE_API void HandleNodeBeginRemoval();

	UE_API FText GetInstructionCountText() const;
	UE_API FText GetInstructionDurationText() const;

	UE_API TSharedRef<SWidget> OnOverrideWidgetMenu() const;

protected:

	UE_API int32 GetNodeTopologyVersion() const;
	UE_API EVisibility GetPinVisibility(int32 InPinInfoIndex, bool bAskingForSubPin) const;
	UE_API const FSlateBrush * GetExpanderImage(int32 InPinInfoIndex, bool bLeft, bool bHovered) const;
	UE_API FReply OnExpanderArrowClicked(int32 InPinInfoIndex);
	UE_API void HandleModifiedEvent(ERigVMGraphNotifType InNotifType, URigVMGraph* InGraph, UObject* InSubject);
	UE_API virtual void UpdatePinTreeView();
	UE_API ERigVMNodeDefaultValueOverrideState::Type GetPinDefaultValueOverrideState() const;
	UE_API bool IsCategoryPin(const SGraphPin* InGraphPin) const;
	UE_API EOverrideWidgetStatus::Type GetOverrideStatus() const;
	UE_API FSlateColor GetPreviewHereColor() const;

	int32 CanReorderExposedPin(const URigVMPin* InPinToReorder, const URigVMPin* InTargetPin, EItemDropZone InDropZone);
	void OnReorderExposedPin(const URigVMPin* InPinToReorder, const URigVMPin* InTargetPin, EItemDropZone InDropZone);

	/** Cached widget title area */
	TSharedPtr<SOverlay> TitleAreaWidget;

	int32 NodeErrorType;

	TSharedPtr<SButton> PreviewHereIndicatorWidget;
	bool bPreviewHereIndicatorHovered;
	TSharedPtr<SImage> VisualDebugIndicatorWidget;
	TSharedPtr<STextBlock> InstructionCountTextBlockWidget;
	TSharedPtr<STextBlock> InstructionDurationTextBlockWidget;
	TSharedPtr<SOverrideStatusWidget> OverrideStatusWidget; 

	static UE_API const FSlateBrush* CachedImg_CR_Pin_Connected;
	static UE_API const FSlateBrush* CachedImg_CR_Pin_Disconnected;

	/** Cache the node title so we can invalidate it */
	TSharedPtr<SNodeTitle> NodeTitle;

	TWeakInterfacePtr<IRigVMEditorAssetInterface> Blueprint;

	FVector2D LastHighDetailSize;

	struct FPinInfo
	{
		int32 Index;
		int32 ParentIndex;
		bool bIsCategoryPin;
		bool bHasChildren;
		bool bHideInputWidget;
		bool bIsContainer;
		bool bIsInputVariable;
		int32 Depth;
		FString Identifier;
		TSharedPtr<SGraphPin> InputPinWidget;
		TSharedPtr<SGraphPin> OutputPinWidget;
		bool bExpanded;
		bool bAutoHeight;
		bool bShowOnlySubPins;
	};

	
	/**
	 * Simple tagging metadata
	 */
	class FPinInfoMetaData : public ISlateMetaData
	{
	public:
		SLATE_METADATA_TYPE(FPinInfoMetaData, ISlateMetaData)

		FPinInfoMetaData(const FString& InCPPType, const FString& InBoundVariableName, bool InIsInputVariable)
		: CPPType(InCPPType)
		, BoundVariableName(InBoundVariableName)
		, bIsInputVariable(InIsInputVariable)
		{}

		FString CPPType;
		FString BoundVariableName;
		bool bIsInputVariable;
	};
	
	TArray<FPinInfo> PinInfos;
	TWeakObjectPtr<URigVMNode> ModelNode;

	// Pins to keep after calling HandleNodePinsChanged. We recycle these pins in
	// CreateStandardPinWidget.
	TMap<const UEdGraphPin *, TSharedRef<SGraphPin>> PinsToKeep;

	// Delayed pin deletion. To deal with the fact that pin deletion cannot occur until we
	// have re-generated the pin list. SRigVMGraphNode has already relinquished them
	// but we still have a pointer to them in our pin widget.
	TSet<UEdGraphPin *> PinsToDelete;

	TAttribute<bool> PinDragDropEnabledAttribute;
};

#undef UE_API
