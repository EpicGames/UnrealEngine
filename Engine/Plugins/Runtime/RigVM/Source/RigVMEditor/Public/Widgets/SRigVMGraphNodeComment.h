// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphNodeComment.h"

#define UE_API RIGVMEDITOR_API

class UEdGraphNode_Comment;

class SRigVMGraphNodeComment : public SGraphNodeComment
{
public:

	UE_API SRigVMGraphNodeComment();

	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void EndUserInteraction() const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;

protected:

	UE_API virtual bool IsNodeUnderComment(UEdGraphNode_Comment* InCommentNode, const TSharedRef<SGraphNode> InNodeWidget) const override;
	UE_API bool IsNodeUnderComment(UEdGraphNode* InNode) const;
	UE_API bool IsNodeUnderComment(const FVector2f& InNodePosition) const;
	UE_API bool IsNodeUnderComment(const FVector2f& InCommentPosition, const FVector2f& InNodePosition) const;

private:

	FLinearColor CachedNodeCommentColor;

	int8 CachedColorBubble;
};

#undef UE_API
