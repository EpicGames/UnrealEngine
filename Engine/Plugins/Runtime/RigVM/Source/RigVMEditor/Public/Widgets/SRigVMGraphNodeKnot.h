// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SGraphNodeKnot.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphNodeKnot : public SGraphNodeKnot
{
	SLATE_BEGIN_ARGS(SRigVMGraphNodeKnot) {} 
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphNode* InKnot);

	// SGraphNode interface
	UE_API virtual void EndUserInteraction() const override;
	UE_API virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	UE_API virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;

	UE_API void HandleNodeBeginRemoval();
	UE_API void UpdateGraphNode() override;
};

#undef UE_API
