// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphPinCategory : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinCategory)
	{}
	SLATE_END_ARGS()


	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	virtual const FSlateBrush* GetPinIcon() const override { return nullptr; }
	//~ End SGraphPin Interface
};

#undef UE_API
