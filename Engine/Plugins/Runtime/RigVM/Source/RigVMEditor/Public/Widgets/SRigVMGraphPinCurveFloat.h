// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SCurveEditor.h"
#include "Curves/CurveFloat.h"
#include "SGraphPin.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphPinCurveFloat : public SGraphPin, public FCurveOwnerInterface
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinCurveFloat) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	/** FCurveOwnerInterface interface */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_API virtual TArray<FRichCurveEditInfoConst> GetCurves() const override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	UE_API virtual void GetCurves(TAdderReserverRef<FRichCurveEditInfoConst> Curves) const override;
	UE_API virtual TArray<FRichCurveEditInfo> GetCurves() override;
	UE_API virtual void ModifyOwner() override;
	UE_API virtual TArray<const UObject*> GetOwners() const override;
	UE_API virtual void MakeTransactional() override;
	UE_API virtual void OnCurveChanged(const TArray<FRichCurveEditInfo>& ChangedCurveEditInfos) override;
	UE_API virtual bool IsValidCurve(FRichCurveEditInfo CurveInfo) override;

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SGraphPin Interface

	UE_API FRuntimeFloatCurve& UpdateAndGetCurve();

	TSharedPtr<SCurveEditor> CurveEditor;
	FRuntimeFloatCurve Curve;
};

#undef UE_API
