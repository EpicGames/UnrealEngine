// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "RigVMModel/RigVMPin.h"

#define UE_API RIGVMEDITOR_API

class SRigVMEnumPicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal(TObjectPtr<UEnum>, FGetCurrentEnum);
	
	SLATE_BEGIN_ARGS(SRigVMEnumPicker){}
		SLATE_EVENT(SSearchableComboBox::FOnSelectionChanged, OnEnumChanged)
		SLATE_EVENT(FGetCurrentEnum, GetCurrentEnum)
		SLATE_ARGUMENT(bool, IsEnabled)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

protected:

	void HandleControlEnumChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo)
	{
		if (OnEnumChangedDelegate.IsBound())
		{
			return OnEnumChangedDelegate.Execute(InItem, InSelectionInfo);
		}
	}

	UEnum* GetCurrentEnum()
	{
		if (GetCurrentEnumDelegate.IsBound())
		{
			return GetCurrentEnumDelegate.Execute();
		}
		return nullptr;
	}

	UE_API TSharedRef<SWidget> OnGetEnumNameWidget(TSharedPtr<FString> InItem);
	UE_API void PopulateEnumOptions();

	TArray<TSharedPtr<FString>> EnumOptions;
	SSearchableComboBox::FOnSelectionChanged OnEnumChangedDelegate;
	FGetCurrentEnum GetCurrentEnumDelegate;
	bool bIsEnabled;
};

class SRigVMGraphPinEnumPicker : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinEnumPicker)
	: _ModelPin(nullptr) {}
		SLATE_ARGUMENT(URigVMPin*, ModelPin)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API void HandleControlEnumChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionInfo);
	
	URigVMPin* ModelPin;

};

#undef UE_API
