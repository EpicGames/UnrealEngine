// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "RigVMModel/RigVMPin.h"
#include "Widgets/SRigVMGraphPinNameListValueWidget.h"

#define UE_API RIGVMEDITOR_API

class URigVMEdGraph;

class SRigVMGraphPinNameList : public SGraphPin
{
public:

	DECLARE_DELEGATE_RetVal_OneParam( const TArray<TSharedPtr<FRigVMStringWithTag>>*, FOnGetNameListContent, URigVMPin*);
	DECLARE_DELEGATE_RetVal( const TArray<TSharedPtr<FRigVMStringWithTag>>, FOnGetNameFromSelection);
	DECLARE_DELEGATE_RetVal_ThreeParams( FReply, FOnGetSelectedClicked, URigVMEdGraph*, URigVMPin*, FString);
	DECLARE_DELEGATE_RetVal_ThreeParams( FReply, FOnBrowseClicked, URigVMEdGraph*, URigVMPin*, FString);

	SLATE_BEGIN_ARGS(SRigVMGraphPinNameList)
		: _MarkupInvalidItems(true)
		, _EnableNameListCache(true)
		, _SearchHintText(NSLOCTEXT("SRigVMGraphPinNameList", "Search", "Search"))
		, _AllowUserProvidedText(false)
	{}

		SLATE_ARGUMENT(URigVMPin*, ModelPin)
		SLATE_EVENT(FOnGetNameListContent, OnGetNameListContent)
		SLATE_EVENT(FOnGetNameListContent, OnGetNameListContentForValidation)
		SLATE_EVENT(FOnGetNameFromSelection, OnGetNameFromSelection)
		SLATE_EVENT(FOnGetSelectedClicked, OnGetSelectedClicked)
		SLATE_EVENT(FOnBrowseClicked, OnBrowseClicked)
		SLATE_ARGUMENT(bool, MarkupInvalidItems)
		SLATE_ARGUMENT(bool, EnableNameListCache)
		SLATE_ARGUMENT(FText, SearchHintText)
		SLATE_ARGUMENT(bool, AllowUserProvidedText)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API const TArray<TSharedPtr<FRigVMStringWithTag>>* GetNameList(bool bForContent = true) const;
	UE_API FText GetNameListText() const;
	UE_API FSlateColor GetNameColor() const;
	UE_API virtual void SetNameListText(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);
	UE_API void UpdateNameLists(bool bUpdateCurrent = true, bool bUpdateValidation = true);

	UE_API TSharedRef<SWidget> MakeNameListItemWidget(TSharedPtr<FRigVMStringWithTag> InItem);
	UE_API void OnNameListChanged(TSharedPtr<FRigVMStringWithTag> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API void OnNameListComboBox();

	FOnGetNameListContent OnGetNameListContent;
	FOnGetNameListContent OnGetNameListContentForValidation;
	URigVMPin* ModelPin;
	TSharedPtr<SRigVMGraphPinNameListValueWidget> NameListComboBox;
	TArray<TSharedPtr<FRigVMStringWithTag>> EmptyList;
	const TArray<TSharedPtr<FRigVMStringWithTag>>* CurrentList;
	const TArray<TSharedPtr<FRigVMStringWithTag>>* ValidationList;
	bool bMarkupInvalidItems;
	bool EnableNameListCache;
	FText SearchHintText;
	bool AllowUserProvidedText;

	/** Helper buttons. */
	UE_API FSlateColor OnGetWidgetForeground() const;
	UE_API FSlateColor OnGetWidgetBackground() const;
	UE_API FReply HandleGetSelectedClicked();
	UE_API FReply HandleBrowseClicked();
	FOnGetNameFromSelection OnGetNameFromSelection;
	FOnGetSelectedClicked OnGetSelectedClicked;
	FOnBrowseClicked OnBrowseClicked;
};

#undef UE_API
