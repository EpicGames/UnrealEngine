// Copyright Epic Games, Inc. All Rights Reservekd.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "EdGraph/RigVMEdGraph.h"

#define UE_API RIGVMEDITOR_API

/**
 * A searchable text combo box
 */
class SRigVMGraphPinNameListValueWidget : public SComboButton
{
public:
	/** Type of list used for showing menu options. */
	typedef SListView< TSharedPtr<FRigVMStringWithTag> > SComboListType;
	/** Delegate type used to generate widgets that represent Options */
	typedef typename TSlateDelegates< TSharedPtr<FRigVMStringWithTag> >::FOnGenerateWidget FOnGenerateWidget;
	typedef typename TSlateDelegates< TSharedPtr<FRigVMStringWithTag> >::FOnSelectionChanged FOnSelectionChanged;

	SLATE_BEGIN_ARGS(SRigVMGraphPinNameListValueWidget)
		: _Content()
		, _ContentPadding(FMargin(3.0, 3.0))
		, _OptionsSource()
		, _OnSelectionChanged()
		, _OnGenerateWidget()
		, _InitiallySelectedItem(nullptr)
		, _Method()
		, _MaxListHeight(450.0f)
		, _HasDownArrow(true)
		, _SearchHintText(NSLOCTEXT("SRigVMGraphPinNameListValueWidget", "Search", "Search"))
		, _AllowUserProvidedText(false)
	{}

	/** Slot for this button's content (optional) */
	SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ATTRIBUTE(FMargin, ContentPadding)

		SLATE_ARGUMENT(const TArray< TSharedPtr<FRigVMStringWithTag> >*, OptionsSource)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

		/** Called when combo box is opened, before list is actually created */
		SLATE_EVENT(FOnComboBoxOpening, OnComboBoxOpening)

		/** The custom scrollbar to use in the ListView */
		SLATE_ARGUMENT(TSharedPtr<SScrollBar>, CustomScrollbar)

		/** The option that should be selected when the combo box is first created */
		SLATE_ARGUMENT(TSharedPtr<FRigVMStringWithTag>, InitiallySelectedItem)

		SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)

		/** The max height of the combo box menu */
		SLATE_ARGUMENT(float, MaxListHeight)

		/**
		 * When false, the down arrow is not generated and it is up to the API consumer
		 * to make their own visual hint that this is a drop down.
		 */
		SLATE_ARGUMENT(bool, HasDownArrow)

		/*
		 * The visible text in the search / editable text field
		 */
		SLATE_ARGUMENT(FText, SearchHintText)

		/*
		 * If set to true the user is allowed to enter custom text here
		 */
		SLATE_ARGUMENT(bool, AllowUserProvidedText)

	SLATE_END_ARGS()

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs   Declaration from which to construct the combo box
	 */
	UE_API void Construct(const FArguments& InArgs);

	UE_API void ClearSelection();

	UE_API void SetSelectedItem(TSharedPtr<FRigVMStringWithTag> InSelectedItem);
	UE_API void SetOptionsSource(const TArray< TSharedPtr<FRigVMStringWithTag> >* InOptionsSource);

	/** @return the item currently selected by the combo box. */
	UE_API TSharedPtr<FRigVMStringWithTag> GetSelectedItem();

	/**
	 * Requests a list refresh after updating options
	 * Call SetSelectedItem to update the selected item if required
	 * @see SetSelectedItem
	 */
	UE_API void RefreshOptions();

private:

	/** Generate a row for the InItem in the combo box's list (passed in as OwnerTable). Do this by calling the user-specified OnGenerateWidget */
	UE_API TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FRigVMStringWithTag> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	//** Called if the menu is closed
	UE_API void OnMenuOpenChanged(bool bOpen);

	/** Invoked when the selection in the list changes */
	UE_API void OnSelectionChanged_Internal(TSharedPtr<FRigVMStringWithTag> ProposedSelection, ESelectInfo::Type SelectInfo, bool bForce = false);

	/** Invoked when the search text changes */
	UE_API void OnSearchTextChanged(const FText& ChangedText);

	/** Invoked when the search is committed*/
	UE_API void OnSearchTextCommitted(const FText& ChangedText, ETextCommit::Type CommitType);

	/** Special case handling for search box key commands */
	UE_API FReply OnSearchTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Special case handling for combo list key commands */
	UE_API FReply OnComboListKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** Handle clicking on the content menu */
	UE_API virtual FReply OnButtonClicked() override;

	/** The item style to use. */
	const FTableRowStyle* ItemStyle;

private:
	/** Delegate that is invoked when the selected item in the combo box changes */
	FOnSelectionChanged OnSelectionChanged;
	/** The item currently selected in the combo box */
	TSharedPtr<FRigVMStringWithTag> SelectedItem;
	/** The search field used for the combox box's contents */
	TSharedPtr< SEditableTextBox > SearchField;
	/** The ListView that we pop up; visualized the available options. */
	TSharedPtr< SComboListType > ComboListView;
	/** The Scrollbar used in the ListView. */
	TSharedPtr< SScrollBar > CustomScrollbar;
	/** Delegate to invoke before the combo box is opening. */
	FOnComboBoxOpening OnComboBoxOpening;
	/** Delegate to invoke when we need to visualize an option as a widget. */
	FOnGenerateWidget OnGenerateWidget;

	const TArray< TSharedPtr<FRigVMStringWithTag> >* OptionsSource = nullptr;
	bool AllowUserProvidedText;

	/** Used to focus the name box immediately following construction */
	UE_API EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);
};


#undef UE_API
