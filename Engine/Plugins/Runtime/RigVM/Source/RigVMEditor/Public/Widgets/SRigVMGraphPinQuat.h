// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "RigVMModel/RigVMPin.h"
#include "SGraphPin.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphPinQuat : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinQuat)
		: _ModelPin(nullptr)
	{}
		SLATE_ARGUMENT(URigVMPin*, ModelPin)
	SLATE_END_ARGS()


	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API TOptional<FRotator> GetRotator() const;
	UE_API void OnRotatorCommitted(FRotator InRotator, ETextCommit::Type InCommitType, bool bUndoRedo);
	UE_API TOptional<float> GetRotatorComponent(int32 InComponent) const;
	UE_API void OnRotatorComponentChanged(float InValue, int32 InComponent);
	UE_API void OnRotatorComponentCommitted(float InValue, ETextCommit::Type InCommitType, int32 InComponent, bool bUndoRedo);

	URigVMPin* ModelPin;
};

#undef UE_API
