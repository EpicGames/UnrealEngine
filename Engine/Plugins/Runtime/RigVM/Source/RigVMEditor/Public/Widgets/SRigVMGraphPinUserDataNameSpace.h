// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "Widgets/SRigVMGraphPinEditableNameValueWidget.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphPinUserDataNameSpace : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinUserDataNameSpace) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API FText GetNameSpaceText() const;
	UE_API virtual void SetNameSpaceText(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);

	UE_API TSharedRef<SWidget> MakeNameSpaceItemWidget(TSharedPtr<FString> InItem);
	UE_API void OnNameSpaceChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API void OnNameSpaceComboBox();
	UE_API TArray<TSharedPtr<FString>>& GetNameSpaces();

	TArray<TSharedPtr<FString>> NameSpaces;
	TSharedPtr<SRigVMGraphPinEditableNameValueWidget> NameComboBox;

};

#undef UE_API
