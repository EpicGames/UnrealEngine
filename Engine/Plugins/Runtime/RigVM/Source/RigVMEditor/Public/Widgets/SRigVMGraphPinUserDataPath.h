// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SButton.h"
#include "SGraphPin.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMBlueprintLegacy.h"
#include "RigVMCore/RigVMAssetUserData.h"
#include "IPropertyAccessEditor.h"

#define UE_API RIGVMEDITOR_API

class SRigVMUserDataPath : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigVMUserDataPath)
	: _ModelPins()
	{}

	SLATE_ARGUMENT(TArray<URigVMPin*>, ModelPins)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

protected:

	UE_API FString GetUserDataPath(URigVMPin* ModelPin) const;
	UE_API FString GetUserDataPath() const;
	FText GetUserDataPathText() const { return FText::FromString(GetUserDataPath()); }
	UE_API const FSlateBrush* GetUserDataIcon() const;
	UE_API FLinearColor GetUserDataColor() const;
	UE_API const FSlateBrush* GetUserDataIcon(const UNameSpacedUserData::FUserData* InUserData) const;
	UE_API FLinearColor GetUserDataColor(const UNameSpacedUserData::FUserData* InUserData) const;
	UE_API TSharedRef<SWidget> GetTopLevelMenuContent();
	UE_API void FillUserDataPathMenu( FMenuBuilder& InMenuBuilder, FString InParentPath );
	UE_API void HandleSetUserDataPath(FString InUserDataPath);

	UE_API FRigVMEditorAssetInterfacePtr GetRigVMAssetInterface() const;
	UE_API FString GetUserDataNameSpace(URigVMPin* ModelPin) const;
	UE_API FString GetUserDataNameSpace() const;
	UE_API const UNameSpacedUserData* GetUserDataObject() const;
	UE_API const UNameSpacedUserData::FUserData* GetUserData() const;

	TArray<URigVMPin*> ModelPins;
	TSharedPtr<SMenuAnchor> MenuAnchor;
	bool bAllowUObjects = false;
};

class  SRigVMGraphPinUserDataPath : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SRigVMGraphPinUserDataPath){}

		SLATE_ARGUMENT(TArray<URigVMPin*>, ModelPins)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	TArray<URigVMPin*> ModelPins;
};

#undef UE_API
