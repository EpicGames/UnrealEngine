// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "RigVMModel/RigVMPin.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "IPropertyAccessEditor.h"
#include "RigVMBlueprintLegacy.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphVariableBinding : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigVMGraphVariableBinding)
	: _ModelPins()
    , _Asset(nullptr)
	, _Blueprint(nullptr)
	, _CanRemoveBinding(true)
	{}

		SLATE_ARGUMENT(TArray<URigVMPin*>, ModelPins)
		SLATE_ARGUMENT(FName, InnerVariableName)
		SLATE_ARGUMENT(FRigVMEditorAssetInterfacePtr, Asset)
		SLATE_ARGUMENT_DEPRECATED(URigVMBlueprint*, Blueprint, 5.7, "Please use Asset instead")
		SLATE_ARGUMENT(bool, CanRemoveBinding)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

protected:

	UE_API FText GetBindingText(URigVMPin* ModelPin) const;
	UE_API FText GetBindingText() const;
	UE_API const FSlateBrush* GetBindingImage() const;
	UE_API FLinearColor GetBindingColor() const;
	UE_API bool OnCanBindProperty(FProperty* InProperty) const;
	UE_API bool OnCanBindToClass(UClass* InClass) const;
	UE_API void OnAddBinding(FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain);
	UE_API bool OnCanRemoveBinding(FName InPropertyName);
	UE_API void OnRemoveBinding(FName InPropertyName);
	UE_API void FillLocalVariableMenu( FMenuBuilder& MenuBuilder );
	UE_API void HandleBindToLocalVariable(FRigVMGraphVariableDescription InLocalVariable);

	TArray<URigVMPin*> ModelPins;
	FRigVMEditorAssetInterfacePtr Blueprint;
	FPropertyBindingWidgetArgs BindingArgs;
	bool bCanRemoveBinding;
};

class SRigVMGraphPinVariableBinding : public SGraphPin
{
public:

	SLATE_BEGIN_ARGS(SRigVMGraphPinVariableBinding){}

		SLATE_ARGUMENT(TArray<URigVMPin*>, ModelPins)
		SLATE_ARGUMENT_DEPRECATED(URigVMBlueprint*, Blueprint, 5.7, "Please use Asset instead")
		SLATE_ARGUMENT(FRigVMEditorAssetInterfacePtr, Asset)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:

	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	TArray<URigVMPin*> ModelPins;
	FRigVMEditorAssetInterfacePtr Blueprint;
};

#undef UE_API
