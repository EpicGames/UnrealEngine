// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "SGraphPin.h"
#include "Widgets/SRigVMGraphPinEditableNameValueWidget.h"

#define UE_API RIGVMEDITOR_API

class SRigVMGraphPinVariableName : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SRigVMGraphPinVariableName) {}
	SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

protected:
	//~ Begin SGraphPin Interface
	UE_API virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

	UE_API FText GetVariableNameText() const;
	UE_API virtual void SetVariableNameText(const FText& NewTypeInValue, ETextCommit::Type CommitInfo);

	UE_API TSharedRef<SWidget> MakeVariableNameItemWidget(TSharedPtr<FString> InItem);
	UE_API void OnVariableNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API void OnVariableNameComboBox();
	UE_API TArray<TSharedPtr<FString>>& GetVariableNames();

	TArray<TSharedPtr<FString>> VariableNames;
	TSharedPtr<SRigVMGraphPinEditableNameValueWidget> NameComboBox;

};

#undef UE_API
