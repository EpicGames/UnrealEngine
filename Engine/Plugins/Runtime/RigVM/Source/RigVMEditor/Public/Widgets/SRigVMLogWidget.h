// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"
#include "Model/MessageLogListingModel.h"
#include "Presentation/MessageLogListingViewModel.h"
#include "RigVMBlueprintLegacy.h"

class IMessageLogListing;

class SRigVMLogWidget : public SBox
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMLogWidget)
		: _LogName(TEXT("RigVMLog"))
		, _LogLabel(NSLOCTEXT("SRigVMLogWidget", "RigVMLog", "RigVM Log"))
		, _ShowFilters(true)
		, _AllowClear(true)
		, _DiscardDuplicates(false)
		, _ScrollToBottom(true)
	{
	}
	SLATE_ATTRIBUTE(FOptionalSize, HeightOverride)
	SLATE_ARGUMENT(FName, LogName)
	SLATE_ARGUMENT(FText, LogLabel)
	SLATE_ARGUMENT(bool, ShowFilters);
	SLATE_ARGUMENT(bool, AllowClear);
	SLATE_ARGUMENT(bool, DiscardDuplicates);
	SLATE_ARGUMENT(bool, ScrollToBottom);
	SLATE_END_ARGS()

	SRigVMLogWidget();
	virtual ~SRigVMLogWidget() override;
	
	void Construct(const FArguments& InArgs);

	void BindLog(const FName& InLogName);
	void UnbindLog(const FName& InLogName);
	void BindLog(TSharedPtr<IMessageLogListing> InListing);
	void UnbindLog(TSharedPtr<IMessageLogListing> InListing);
	void BindLog(const FRigVMEditorAssetInterfacePtr InRigVMBlueprint);
	void UnbindLog(const FRigVMEditorAssetInterfacePtr InRigVMBlueprint);

	TSharedRef<FMessageLogListingModel> GetListing() const
	{
		return ListingModel.ToSharedRef();
	}

	TSharedRef<FMessageLogListingModel> GetListing()
	{
		return ListingModel.ToSharedRef();
	}

private:

	void OnBoundListingChanged(TWeakPtr<IMessageLogListing> InWeakListing);
	
	TSharedPtr<FMessageLogListingModel> ListingModel;
	TSharedPtr<FMessageLogListingViewModel> ListingView;
	TArray<TWeakPtr<IMessageLogListing>> BoundListings;

	struct FBoundListingInfo
	{
		FBoundListingInfo()
			: NumMessages(0)
		{
		}

		FBoundListingInfo(const IMessageLogListing& InListing);
		
		int32 NumMessages;
	};

	TMap<FName, FBoundListingInfo> ListingInfo;
};