// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "RigVMModel/RigVMNode.h"
#include "Input/DragAndDrop.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_RetVal(TArray<FString> /* pin paths  */, FRigVMNodeLayoutWidget_OnGetUncategorizedPins);
DECLARE_DELEGATE_RetVal(TArray<FRigVMPinCategory> /* categories */, FRigVMNodeLayoutWidget_OnGetCategories);
DECLARE_DELEGATE_OneParam(FRigVMNodeLayoutWidget_OnCategoryAdded, FString /* category */);
DECLARE_DELEGATE_OneParam(FRigVMNodeLayoutWidget_OnCategoryRemoved, FString /* category */);
DECLARE_DELEGATE_TwoParams(FRigVMNodeLayoutWidget_OnCategoryRenamed, FString /* old */, FString /* new */);
DECLARE_DELEGATE_RetVal_OneParam(FString, FRigVMNodeLayoutWidget_OnGetElementLabel, FString /* element path */);
DECLARE_DELEGATE_TwoParams(FRigVMNodeLayoutWidget_OnElementLabelChanged, FString /* element path */, FString /* new label */);
DECLARE_DELEGATE_RetVal_OneParam(FString, FRigVMNodeLayoutWidget_OnGetElementCategory, FString /* element path */);
DECLARE_DELEGATE_RetVal_OneParam(int32, FRigVMNodeLayoutWidget_OnGetElementIndexInCategory, FString /* element path */);
DECLARE_DELEGATE_RetVal_OneParam(FLinearColor, FRigVMNodeLayoutWidget_OnGetElementColor, FString /* element path */);
DECLARE_DELEGATE_RetVal_OneParam(const FSlateBrush*, FRigVMNodeLayoutWidget_OnGetElementIcon, FString /* element path */);
DECLARE_DELEGATE_TwoParams(FRigVMNodeLayoutWidget_OnElementIndexInCategoryChanged, FString /* element path */, int32 /* new element index */);
DECLARE_DELEGATE_TwoParams(FRigVMNodeLayoutWidget_OnElementCategoryChanged, FString /* element */, FString /* new category */);
DECLARE_DELEGATE_RetVal(uint32, FRigVMNodeLayoutWidget_OnGetStructuralHash);
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FRigVMNodeLayoutWidget_ValidateName, FString /* in path */, FString /* new name */, FText& /* out error */);

class SRigVMNodeLayoutWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMNodeLayoutWidget)
		: _MaxScrollBoxSize(300.f)
	{
	}
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetUncategorizedPins, OnGetUncategorizedPins)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetCategories, OnGetCategories)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnCategoryAdded, OnCategoryAdded)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnCategoryRemoved, OnCategoryRemoved)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnCategoryRenamed, OnCategoryRenamed)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetElementLabel, OnGetElementLabel)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnElementLabelChanged, OnElementLabelChanged)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetElementCategory, OnGetElementCategory)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetElementIndexInCategory, OnGetElementIndexInCategory)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetElementColor, OnGetElementColor)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetElementIcon, OnGetElementIcon)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnElementIndexInCategoryChanged, OnElementIndexInCategoryChanged)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnElementCategoryChanged, OnElementCategoryChanged)
	SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetStructuralHash, OnGetStructuralHash)
	SLATE_EVENT(FRigVMNodeLayoutWidget_ValidateName, OnValidateCategoryName)
	SLATE_EVENT(FRigVMNodeLayoutWidget_ValidateName, OnValidateElementName)
	SLATE_ATTRIBUTE(FOptionalSize, MaxScrollBoxSize)
	SLATE_END_ARGS()

	UE_API SRigVMNodeLayoutWidget();
	UE_API virtual ~SRigVMNodeLayoutWidget() override;
	
	UE_API void Construct(const FArguments& InArgs);
	UE_API void Refresh();
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	FRigVMNodeLayoutWidget_OnGetUncategorizedPins OnGetUncategorizedPins;
	FRigVMNodeLayoutWidget_OnGetCategories OnGetCategories;
	FRigVMNodeLayoutWidget_OnCategoryAdded OnCategoryAdded;
	FRigVMNodeLayoutWidget_OnCategoryRemoved OnCategoryRemoved;
	FRigVMNodeLayoutWidget_OnCategoryRenamed OnCategoryRenamed;
	FRigVMNodeLayoutWidget_OnGetElementLabel OnGetElementLabel;
	FRigVMNodeLayoutWidget_OnElementLabelChanged OnElementLabelChanged;
	FRigVMNodeLayoutWidget_OnGetElementCategory OnGetElementCategory;
	FRigVMNodeLayoutWidget_OnGetElementIndexInCategory OnGetElementIndexInCategory;
	FRigVMNodeLayoutWidget_OnGetElementColor OnGetElementColor;
	FRigVMNodeLayoutWidget_OnGetElementIcon OnGetElementIcon;
	FRigVMNodeLayoutWidget_OnElementIndexInCategoryChanged OnElementIndexInCategoryChanged;
	FRigVMNodeLayoutWidget_OnElementCategoryChanged OnElementCategoryChanged;
	FRigVMNodeLayoutWidget_OnGetStructuralHash OnGetStructuralHash;
	FRigVMNodeLayoutWidget_ValidateName OnValidateCategoryName;
	FRigVMNodeLayoutWidget_ValidateName OnValidateElementName;

	struct FNodeLayoutRow
	{
		FNodeLayoutRow()
			: bIsCategory(true)
			, bIsUncategorized(false)
			, Color(FLinearColor::White)
			, Icon(nullptr)
		{
		}

		struct FState
		{
			FState()
				: bExpanded(false)
				, bSelected(false)
			{
			}

			bool bExpanded;
			bool bSelected;
		};
		
		bool bIsCategory;
		bool bIsUncategorized;
		FString Path;
		FString Label;
		FLinearColor Color;
		const FSlateBrush* Icon;
		FState State;
		TArray<TSharedPtr<FNodeLayoutRow>> ChildRows;
		FSimpleDelegate OnRequestRename;

		bool IsCategory() const
		{
			return bIsCategory;
		}

		bool IsPin() const
		{
			return !bIsCategory;
		}

		bool IsUncategorizedPin() const
		{
			return IsPin() && bIsUncategorized;
		}

		bool IsCategorizedPin() const
		{
			return IsPin() && !bIsUncategorized;
		}

		void RequestRename()
		{
			(void)OnRequestRename.ExecuteIfBound();
		}
	};

	DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<EItemDropZone>, FRigVMNodeLayoutWidget_OnElementCanDrop, TSharedPtr<FNodeLayoutRow>, TSharedPtr<FNodeLayoutRow>, EItemDropZone);
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FRigVMNodeLayoutWidget_OnElementAcceptDrop, TSharedPtr<FNodeLayoutRow>, TSharedPtr<FNodeLayoutRow>, EItemDropZone);

	class SRigVMNowLayoutRow
	: public STableRow<TSharedPtr<FNodeLayoutRow>>
	{
	public:
	
		SLATE_BEGIN_ARGS(SRigVMNowLayoutRow)
		{}
		SLATE_ARGUMENT(TSharedPtr<FNodeLayoutRow>, NodeLayoutRow)
		SLATE_EVENT(FRigVMNodeLayoutWidget_OnGetCategories, OnGetCategories)
		SLATE_EVENT(FRigVMNodeLayoutWidget_OnCategoryRenamed, OnCategoryRenamed)
		SLATE_EVENT(FRigVMNodeLayoutWidget_OnElementLabelChanged, OnElementLabelChanged)
		SLATE_EVENT(FRigVMNodeLayoutWidget_OnElementCategoryChanged, OnElementCategoryChanged)
		SLATE_EVENT(FRigVMNodeLayoutWidget_OnCategoryRemoved, OnCategoryRemoved)
		SLATE_EVENT(FRigVMNodeLayoutWidget_ValidateName, OnValidateCategoryName)
		SLATE_EVENT(FRigVMNodeLayoutWidget_ValidateName, OnValidateElementName)

		SLATE_EVENT( FOnCanAcceptDrop, OnCanAcceptDrop )
		SLATE_EVENT( FOnAcceptDrop,    OnAcceptDrop )
		SLATE_EVENT( FOnPaintDropIndicator, OnPaintDropIndicator )
		SLATE_EVENT( FOnDragDetected,      OnDragDetected )
		SLATE_EVENT( FOnTableRowDragEnter, OnDragEnter )
		SLATE_EVENT( FOnTableRowDragLeave, OnDragLeave )
		SLATE_EVENT( FOnTableRowDrop,      OnDrop )

		SLATE_END_ARGS()

		UE_API virtual ~SRigVMNowLayoutRow() override;
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);
		static UE_API TSharedPtr<SInlineEditableTextBlock> ConstructLabel(TSharedPtr<FNodeLayoutRow> InNodeLayoutRow, TSharedRef<SHorizontalBox> OutHorizontalBox, SRigVMNowLayoutRow* InRow);

		TSharedPtr<FNodeLayoutRow> GetNodeLayoutRow() const { return NodeLayoutRow; }
		UE_API void OnLabelCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
		UE_API bool OnVerifyLabelChanged(const FText& InText, FText& OutErrorMessage);
		UE_API bool IsSelected() const;

	private:
		TSharedPtr<FNodeLayoutRow> NodeLayoutRow;
		FRigVMNodeLayoutWidget_OnGetCategories OnGetCategories;
		FRigVMNodeLayoutWidget_OnCategoryRenamed OnCategoryRenamed;
		FRigVMNodeLayoutWidget_OnElementLabelChanged OnElementLabelChanged;
		FRigVMNodeLayoutWidget_OnElementCategoryChanged OnElementCategoryChanged;
		FRigVMNodeLayoutWidget_OnCategoryRemoved OnCategoryRemoved;
		FRigVMNodeLayoutWidget_ValidateName OnValidateCategoryName;
		FRigVMNodeLayoutWidget_ValidateName OnValidateElementName;
		
		TSharedPtr<SInlineEditableTextBlock> LabelEditWidget;
	};

	UE_API TSharedRef<ITableRow> GenerateRow(TSharedPtr<FNodeLayoutRow> InRow, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API void GetChildrenForRow(TSharedPtr<FNodeLayoutRow> InRow, TArray<TSharedPtr<FNodeLayoutRow>>& OutChildren);
	UE_API void OnItemExpansionChanged(TSharedPtr<FNodeLayoutRow> InRow, bool bExpanded);
	UE_API void OnItemSelectionChanged(TSharedPtr<FNodeLayoutRow> InRow, ESelectInfo::Type InSelectInfo);
	UE_API bool IsNodeLayoutEditable() const;
	UE_API FReply HandleAddCategory();

	class FRigVMNodeLayoutDragDropOp : public FDragDropOperation
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FRigVMNodeLayoutDragDropOp, FDragDropOperation)

		static TSharedRef<FRigVMNodeLayoutDragDropOp> New(const TArray<TSharedPtr<FNodeLayoutRow>>& InNodeLayoutRows);

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
		virtual FVector2D GetDecoratorPosition() const override;
		virtual bool IsWindowlessOperation() const override { return true; }

		const TArray<TSharedPtr<FNodeLayoutRow>>& GetNodeLayoutRows() const
		{
			return NodeLayoutRows;
		}
		
	private:

		/** Data for the property paths this item represents */
		TArray<TSharedPtr<FNodeLayoutRow>> NodeLayoutRows;
	};

	UE_API FReply OnDragDetectedForRow(TSharedPtr<FNodeLayoutRow> InSourceRow);
	UE_API TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, TSharedPtr<FNodeLayoutRow> InTargetRow, EItemDropZone InDropZone);
	UE_API FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, TSharedPtr<FNodeLayoutRow> InTargetRow, EItemDropZone InDropZone);
	UE_API virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);
	
	TArray<TSharedPtr<FNodeLayoutRow>> NodeLayoutRows;
	TSharedPtr<STreeView<TSharedPtr<FNodeLayoutRow>>> TreeView;
	TOptional<uint32> LastStructuralHash;
};

#undef UE_API
