// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SRigVMGraphNode.h"
#include "Editor/RigVMMinimalEnvironment.h"

#define UE_API RIGVMEDITOR_API

class SRigVMNodePreviewWidget : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMNodePreviewWidget)
		: _Padding(16.f)
		, _NodeEnabled(false)
		, _PinDragAndDropEnabled(false)
	{}
	SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_ARGUMENT(TSharedPtr<FRigVMMinimalEnvironment>, Environment)
	SLATE_ATTRIBUTE(bool, NodeEnabled)
	SLATE_ATTRIBUTE(bool, PinDragAndDropEnabled)
	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	UE_API void UpdateNodeWidget();
	
	TSharedPtr<FRigVMMinimalEnvironment> Environment;
	TAttribute<bool> NodeEnabledAttribute;
	TAttribute<bool> PinDragAndDropEnabledAttribute;
};

#undef UE_API
