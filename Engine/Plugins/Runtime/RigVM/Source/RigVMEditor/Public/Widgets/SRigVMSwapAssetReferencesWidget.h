// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SRigVMBulkEditWidget.h"
#include "RigVMEditorBlueprintLibrary.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_RetVal_OneParam(TArray<FSoftObjectPath>, FOnGetReferences, FAssetData);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnSwapReference, FSoftObjectPath, FAssetData);

class FRigVMSwapAssetReferencesContext : public FRigVMTreeContext
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMSwapAssetReferencesContext, FRigVMTreeContext)

	const FAssetData& GetSourceAsset()
	{
		return SourceAsset;
	}

	void SetSourceAsset(const FAssetData& InAsset)
	{
		SourceAsset = InAsset;
	}

	const FAssetData& GetTargetAsset()
	{
		return TargetAsset;
	}

	void SetTargetAsset(const FAssetData& InAsset)
	{
		TargetAsset = InAsset;
	}

	const TArray<FSoftObjectPath>& GetReferences()
	{
		return References;
	}

	void SetReferences(const TArray<FSoftObjectPath>& InReferences)
	{
		References = InReferences;
	}

	void ClearReferences()
	{
		References.Reset();
	}

	virtual uint32 GetVisibleChildrenHash() const override;

private:
	FAssetData SourceAsset;
	FAssetData TargetAsset;
	TArray<FSoftObjectPath> References;
};

// A single reference node
class FRigVMTreeReferenceNode : public FRigVMTreeNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeReferenceNode, FRigVMTreeNode)

	FRigVMTreeReferenceNode(const FSoftObjectPath& InModulePath)
		: FRigVMTreeNode(InModulePath.GetSubPathString())
		, ModulePath(InModulePath)
	{
		
	}

	virtual bool IsCheckable() const override
	{
		return true;
	}
	
	const FSoftObjectPath& GetReferencePath() const
	{
		return ModulePath;
	}

private:
	FSoftObjectPath ModulePath;
};

// The asset node which contains multiple reference nodes
class FRigVMTreeAssetRefAssetNode : public FRigVMTreePackageNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeAssetRefAssetNode, FRigVMTreePackageNode)
	
	FRigVMTreeAssetRefAssetNode(const FAssetData& InAssetData)
		: FRigVMTreePackageNode(InAssetData)
	{
		
	}

	virtual bool IsCheckable() const override
	{
		return true;
	}

protected:
	virtual TArray<TSharedRef<FRigVMTreeNode>> GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const override;

	mutable TArray<TSharedRef<FRigVMTreeNode>> AssetRefNodes;
};

// Filters the source asset from the target asset list
class FRigVMTreeTargetAssetFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeTargetAssetFilter, FRigVMTreeFilter)
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

// Filters target assets to show only variants of the source asset
class FRigVMTreeAssetVariantFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeAssetVariantFilter, FRigVMTreeFilter);
	virtual bool CanBeToggledInUI() const override
	{
		return true;
	}
	virtual bool IsInvertedInUI() const override
	{
		return false;
	}
	virtual FText GetLabel() const override;

	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;

private:
	mutable TMap<FString, TArray<FRigVMVariantRef>> SourceVariants;
};

class FRigVMSwapAssetReferenceTask : public FRigVMTreeTask
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMSwapAssetReferenceTask, FRigVMTreeTask)
	
	FRigVMSwapAssetReferenceTask(const FSoftObjectPath InReferencePath, const FAssetData& InNewAsset, const FOnSwapReference& InSwapFunction)
		: ReferencePath(InReferencePath)
		, NewAsset(InNewAsset)
		, SwapFunction(InSwapFunction)
	{
	}

	virtual bool Execute(const TSharedRef<FRigVMTreePhase>& InPhase) override;

	virtual bool RequiresRefresh() const override
	{
		return true;
	}

	virtual bool RequiresUndo() const override
	{
		return true;
	}
	
private:
	
	FSoftObjectPath ReferencePath;
	FAssetData NewAsset;
	FOnSwapReference SwapFunction;
};

class SRigVMSwapAssetReferencesWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigVMSwapAssetReferencesWidget)
		: _SkipPickingRefs(false)
		, _EnableUndo(false)
		, _CloseOnSuccess(false)
	{}
	SLATE_ARGUMENT(FAssetData, Source)
	SLATE_ARGUMENT(FAssetData, Target)
	SLATE_ARGUMENT(TArray<FSoftObjectPath>, ReferencePaths)
	SLATE_ARGUMENT(bool, SkipPickingRefs)
	SLATE_ARGUMENT(bool, EnableUndo)
	SLATE_ARGUMENT(bool, CloseOnSuccess)
	SLATE_ARGUMENT(TArray<FRigVMAssetDataFilter>, SourceAssetFilters)
	SLATE_ARGUMENT(TArray<FRigVMAssetDataFilter>, TargetAssetFilters)
	SLATE_EVENT(FOnGetReferences, OnGetReferences)
	SLATE_EVENT(FOnSwapReference, OnSwapReference)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	TSharedRef<SRigVMBulkEditWidget> GetBulkEditWidget()
	{
		return BulkEditWidget.ToSharedRef();
	}

private:

	TSharedPtr<SRigVMBulkEditWidget> BulkEditWidget;
	TSharedPtr<FRigVMSwapAssetReferencesContext> PickTargetContext;
	TSharedPtr<FRigVMSwapAssetReferencesContext> PickAssetRefsContext;
	bool bSkipPickingRefs = false;

	TArray<FRigVMAssetDataFilter> SourceAssetFilters;
	TArray<FRigVMAssetDataFilter> TargetAssetFilters;
	
	FOnGetReferences OnGetReferences;
	FOnSwapReference OnSwapReference;

	UE_API TArray<TSharedRef<FRigVMTreeNode>> GetAssetNodes(const FArguments& InArgs, const int32& InPhase);
	UE_API void OnPhaseActivated(TSharedRef<FRigVMTreePhase> Phase);
	UE_API FReply OnNodeSelected(TSharedRef<FRigVMTreeNode> Node);
	UE_API FReply OnNodeDoubleClicked(TSharedRef<FRigVMTreeNode> Node);
	UE_API void SetSourceAsset(const FAssetData& InAsset);
	UE_API void SetTargetAsset(const FAssetData& InAsset);

	static constexpr int32 PHASE_PICKSOURCE = 0;
	static constexpr int32 PHASE_PICKTARGET = 1;
	static constexpr int32 PHASE_PICKASSETREFS = 2;
};

#undef UE_API
