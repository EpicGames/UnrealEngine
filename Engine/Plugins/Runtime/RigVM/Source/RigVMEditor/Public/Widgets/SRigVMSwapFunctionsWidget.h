// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SGraphPin.h"
#include "SRigVMGraphNode.h"
#include "RigVMModel/RigVMGraph.h"
#include "RigVMModel/Nodes/RigVMFunctionReferenceNode.h"
#include "RigVMModel/RigVMClient.h"
#include "Widgets/SRigVMBulkEditWidget.h"
#include "Widgets/SRigVMNodePreviewWidget.h"

#define UE_API RIGVMEDITOR_API

class FRigVMSwapFunctionContext : public FRigVMTreeContext
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMSwapFunctionContext, FRigVMTreeContext)

	const FRigVMGraphFunctionIdentifier& GetSourceIdentifier()
	{
		return SourceIdentifier;
	}

	void SetSourceIdentifier(const FRigVMGraphFunctionIdentifier& InIdentifier)
	{
		SourceIdentifier = InIdentifier;
	}

	const FRigVMGraphFunctionIdentifier& GetTargetIdentifier()
	{
		return TargetIdentifier;
	}

	void SetTargetIdentifier(const FRigVMGraphFunctionIdentifier& InIdentifier)
	{
		TargetIdentifier = InIdentifier;
	}

	virtual uint32 GetVisibleChildrenHash() const override;

private:
	FRigVMGraphFunctionIdentifier SourceIdentifier;
	FRigVMGraphFunctionIdentifier TargetIdentifier;
};

class FRigVMTreeFunctionRefNode : public FRigVMTreeNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionRefNode, FRigVMTreeNode)

	FRigVMTreeFunctionRefNode(const URigVMFunctionReferenceNode* InFunctionReferenceNode)
		: FRigVMTreeNode(InFunctionReferenceNode->GetPathName())
		, Identifier(InFunctionReferenceNode->GetFunctionIdentifier())
	{
	}

	FRigVMTreeFunctionRefNode(const FRigVMReferenceNodeData& InRefNodeData)
		: FRigVMTreeNode(InRefNodeData.ReferenceNodePath)
		, Identifier(InRefNodeData.ReferencedFunctionIdentifier)
	{
	}

	virtual bool IsCheckable() const override
	{
		return true;
	}
	
	const FRigVMGraphFunctionIdentifier& GetIdentifier() const
	{
		return Identifier;
	}
	
	virtual const FSlateBrush* GetIconAndTint(FLinearColor& OutColor) const override;

private:
	FRigVMGraphFunctionIdentifier Identifier;
};

class FRigVMTreeFunctionRefGraphNode : public FRigVMTreeNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionRefGraphNode, FRigVMTreeNode)

	FRigVMTreeFunctionRefGraphNode(const URigVMGraph* InFunctionGraph);

	virtual FText GetLabel() const override;

	virtual const FSlateBrush* GetIconAndTint(FLinearColor& OutColor) const override;

	virtual bool IsCheckable() const override
	{
		return true;
	}

	virtual void DirtyChildren() override;
	bool ContainsFunctionReference(const TSharedRef<FRigVMTreeContext>& InContext) const;

protected:
	virtual TArray<TSharedRef<FRigVMTreeNode>> GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const override;

	TWeakObjectPtr<const URigVMGraph> WeakGraph;
	TOptional<FText> OptionalLabel;
	mutable TArray<TSharedRef<FRigVMTreeNode>> FunctionRefNodes;
};

class FRigVMTreeFunctionRefAssetNode : public FRigVMTreePackageNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionRefAssetNode, FRigVMTreePackageNode)
	
	FRigVMTreeFunctionRefAssetNode(const FAssetData& InAssetData);

	virtual bool IsCheckable() const override
	{
		return true;
	}

	virtual void DirtyChildren() override;

protected:
	virtual TArray<TSharedRef<FRigVMTreeNode>> GetChildrenImpl(const TSharedRef<FRigVMTreeContext>& InContext) const override;

	mutable TArray<TSharedRef<FRigVMTreeNode>> LoadedGraphNodes;
	mutable TArray<TSharedRef<FRigVMTreeNode>> MetaDataBasedNodes;
	mutable TArray<FRigVMReferenceNodeData> ReferenceNodeDatas;

	friend class SRigVMSwapFunctionsWidget;
	friend class FRigVMTreeEmptyFunctionRefAssetFilter;
};

class FRigVMTreeFunctionIdentifierNode : public FRigVMTreeNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionIdentifierNode, FRigVMTreeNode)

	FRigVMTreeFunctionIdentifierNode(const FRigVMGraphFunctionIdentifier& InIdentifier)
		: FRigVMTreeNode(InIdentifier.GetLibraryNodePath())
		, Identifier(InIdentifier)
	{
	}

	virtual FText GetLabel() const override;
	virtual bool IsCheckable() const override { return false; }
	virtual const FSlateBrush* GetIconAndTint(FLinearColor& OutColor) const override;
	virtual const TArray<FRigVMTag>& GetTags() const override;

	const FRigVMGraphFunctionIdentifier& GetIdentifier() const
	{
		return Identifier;
	}

private:
	FRigVMGraphFunctionIdentifier Identifier;
};

class FRigVMTreeFunctionIdentifierAssetNode : public FRigVMTreePackageNode
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionIdentifierAssetNode, FRigVMTreePackageNode)

	FRigVMTreeFunctionIdentifierAssetNode(const FAssetData& InAssetData)
		: FRigVMTreePackageNode(InAssetData)
	{
	}

	virtual bool ShouldExpandByDefault() const override
	{
		return true;
	}

	virtual bool IsCheckable() const override { return false; }
	void AddChildNode(const TSharedRef<FRigVMTreeNode>& InNode);
	virtual void DirtyChildren() override;
};

class FRigVMTreeEmptyFunctionRefGraphFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeEmptyFunctionRefGraphFilter, FRigVMTreeFilter)
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

class FRigVMTreeEmptyFunctionRefAssetFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeEmptyFunctionRefAssetFilter, FRigVMTreeFilter)
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

class FRigVMTreeFunctionWithNoRefsFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionWithNoRefsFilter, FRigVMTreeFilter);
	virtual bool CanBeToggledInUI() const override
	{
		return true;
	}
	virtual bool IsInvertedInUI() const override
	{
		return true;
	}
	virtual FText GetLabel() const override;
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

class FRigVMTreeSourceFunctionFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeSourceFunctionFilter, FRigVMTreeFilter)
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

class FRigVMTreeTargetFunctionFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeTargetFunctionFilter, FRigVMTreeFilter)
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;
};

class FRigVMTreeFunctionVariantFilter : public FRigVMTreeFilter
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMTreeFunctionVariantFilter, FRigVMTreeFilter);
	virtual bool CanBeToggledInUI() const override
	{
		return true;
	}
	virtual bool IsInvertedInUI() const override
	{
		return false;
	}
	virtual FText GetLabel() const override;
	virtual bool Filters(TSharedRef<FRigVMTreeNode>& InNode, const TSharedRef<FRigVMTreeContext>& InContext) override;

private:
	bool IsFunctionVariant(const FRigVMGraphFunctionIdentifier& InIdentifier) const;
	bool IsFunctionVariantOf(const FRigVMGraphFunctionIdentifier& InIdentifier, const FRigVMGraphFunctionIdentifier& InSourceIdentifier) const;
	mutable TMap<FString,bool> LibraryNodePathToIsVariant;
	mutable TMap<FString,TArray<FRigVMGraphFunctionIdentifier>> LibraryNodePathToVariants;
};

class FRigVMSwapFunctionTask : public FRigVMTreeTask
{
public:
	DEFINE_RIGVMTREETOOLKIT_ELEMENT(FRigVMSwapFunctionTask, FRigVMTreeTask)
	
	FRigVMSwapFunctionTask(const FString InObjectPath, const FRigVMGraphFunctionIdentifier& InIdentifier)
		: ObjectPath(InObjectPath)
		, Identifier(InIdentifier)
	{
	}

	virtual bool Execute(const TSharedRef<FRigVMTreePhase>& InPhase) override;

	virtual bool RequiresRefresh() const override
	{
		return true;
	}

	virtual bool RequiresUndo() const override
	{
		return true;
	}

	virtual TArray<FString> GetAffectedNodes() const override
	{
		return { ObjectPath };
	}
	
private:

	URigVMFunctionReferenceNode* GetReferenceNode(const TSharedRef<FRigVMTreePhase>& InPhase) const;
	virtual FRigVMEditorAssetInterfacePtr GetRigVMAsset(const TSharedRef<FRigVMTreePhase>& InPhase) const override;
	
	FString ObjectPath;
	FRigVMGraphFunctionIdentifier Identifier;
};

class SRigVMSwapFunctionsWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRigVMSwapFunctionsWidget)
		: _SkipPickingFunctionRefs(false)
		, _EnableUndo(false)
		, _CloseOnSuccess(false)
	{}
	SLATE_ARGUMENT(FRigVMGraphFunctionIdentifier, Source)
	SLATE_ARGUMENT(FRigVMGraphFunctionIdentifier, Target)
	SLATE_ARGUMENT(TArray<URigVMFunctionReferenceNode*>, FunctionReferenceNodes)
	SLATE_ARGUMENT(TArray<URigVMGraph*>, Graphs)
	SLATE_ARGUMENT(TArray<FAssetData>, Assets)
	SLATE_ARGUMENT(bool, SkipPickingFunctionRefs)
	SLATE_ARGUMENT(bool, EnableUndo)
	SLATE_ARGUMENT(bool, CloseOnSuccess)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	TSharedRef<SRigVMBulkEditWidget> GetBulkEditWidget()
	{
		return BulkEditWidget.ToSharedRef();
	}

private:

	TSharedPtr<SRigVMBulkEditWidget> BulkEditWidget;
	TSharedPtr<FRigVMSwapFunctionContext> PickTargetContext;
	TSharedPtr<FRigVMSwapFunctionContext> PickFunctionRefsContext;
	TSharedPtr<FRigVMMinimalEnvironment> SourcePreviewEnvironment;
	TSharedPtr<FRigVMMinimalEnvironment> TargetPreviewEnvironment;
	bool bSkipPickingFunctionRefs = false;;

	static UE_API TArray<TSharedRef<FRigVMTreeNode>> GetFunctionIdentifierNodes(const FArguments& InArgs);
	static UE_API TArray<TSharedRef<FRigVMTreeNode>> GetFunctionRefNodes(const FArguments& InArgs);
	UE_API void OnPhaseActivated(TSharedRef<FRigVMTreePhase> Phase);
	UE_API FReply OnNodeSelected(TSharedRef<FRigVMTreeNode> Node);
	UE_API FReply OnNodeDoubleClicked(TSharedRef<FRigVMTreeNode> Node);
	UE_API void SetSourceFunction(const FRigVMGraphFunctionIdentifier& InIdentifier);
	UE_API void SetTargetFunction(const FRigVMGraphFunctionIdentifier& InIdentifier);

	static constexpr int32 PHASE_PICKSOURCE = 0;
	static constexpr int32 PHASE_PICKTARGET = 1;
	static constexpr int32 PHASE_PICKFUNCTIONREFS = 2;
};

#undef UE_API
