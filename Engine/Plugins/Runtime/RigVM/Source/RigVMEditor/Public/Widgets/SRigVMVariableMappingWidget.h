// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSearchBox.h"

#define UE_API RIGVMEDITOR_API

//=====================================================================================
// UI for displaying property info
//=====================================================================================

struct FRigVMVariableMappingInfo
{
public:
	// This is the property that is the most shallow type
	// It will be Transform.Translation.X
	FName PropertyName; 
	// Display Name
	FString DisplayName;
	// List of Children
	// in theory, this actually shouldn't be active if you have children
	// but it represent each raw nonetheless
	// this is to map curve which is always float
	TArray<TSharedPtr<FRigVMVariableMappingInfo>> Children;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FRigVMVariableMappingInfo> Make(const FName InPathName)
	{
		return MakeShareable(new FRigVMVariableMappingInfo(InPathName));
	}

	FName GetPathName() const
	{
		return PropertyName;
	}

	FString GetDisplayName() const
	{
		return DisplayName;
	}

protected:
	/** Hidden constructor, always use Make above */
	FRigVMVariableMappingInfo(const FName InPathName)
		: PropertyName(InPathName)
	{
		FString PathString = InPathName.ToString();
		int32 Found = PathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		DisplayName = (Found != INDEX_NONE) ? PathString.RightChop(Found + 1) : PathString;
	}

	/** Hidden constructor, always use Make above */
	FRigVMVariableMappingInfo() {}
};

//////////////////////////////////////////////////////////////////////////
// SRigVMVariableMappingListRow

typedef TSharedPtr< FRigVMVariableMappingInfo > FRigVMVariableMappingInfoPtr;

DECLARE_DELEGATE_TwoParams(FOnRigVMVariableMappingChanged, const FName& /** PathName */, const FName&/** Curve Name**/)
DECLARE_DELEGATE_RetVal_OneParam(FName, FOnRigVMGetVariableMapping, const FName& /** PathName **/)
// type verified, and make sure it's not double booked
DECLARE_DELEGATE_TwoParams(FOnRigVMGetAvailableMapping, const FName& /** PathName **/, TArray<FName>& /** List of available mappings**/)
// to highlight item
DECLARE_DELEGATE_RetVal(FText&, FOnRigVMGetFilteredText)
// caller to create list
DECLARE_DELEGATE_TwoParams(FOnRigVMCreateVariableMapping, const FString&, TArray< TSharedPtr<FRigVMVariableMappingInfo> >&)
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnRigVMVarOptionAvailable, const FName& /** Property Name */);

// pin exposure option
DECLARE_DELEGATE_TwoParams(FOnRigVMPinCheckStateChanged, ECheckBoxState, FName /** Property Name */);
DECLARE_DELEGATE_RetVal_OneParam(ECheckBoxState, FOnRigVMPinGetCheckState, FName /** Property Name */);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnRigVMPinIsCheckEnabled, FName /** Property Name */);

class SRigVMVariableMappingTreeRow
	: public STableRow< TSharedPtr<FRigVMVariableMappingInfoPtr> >
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMVariableMappingTreeRow) {}

	/** The item for this row **/
	SLATE_ARGUMENT(FRigVMVariableMappingInfoPtr, Item)
	SLATE_EVENT(FOnRigVMVariableMappingChanged, OnVariableMappingChanged)
	SLATE_EVENT(FOnRigVMGetVariableMapping, OnGetVariableMapping)
	SLATE_EVENT(FOnRigVMGetAvailableMapping, OnGetAvailableMapping)
	SLATE_EVENT(FOnRigVMGetFilteredText, OnGetFilteredText)
	SLATE_EVENT(FOnRigVMVarOptionAvailable, OnVariableOptionAvailable);
	SLATE_EVENT(FOnRigVMPinCheckStateChanged, OnPinCheckStateChanged)
	SLATE_EVENT(FOnRigVMPinGetCheckState, OnPinGetCheckState)
	SLATE_EVENT(FOnRigVMPinIsCheckEnabled, OnPinIsEnabledCheckState)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

private:

	/** Widget used to display the list of variable option list*/
	TSharedPtr<SSearchableComboBox>	VarOptionComboBox;
	TArray< TSharedPtr< FString > >	VariableOptionList;

	/** The name and weight of the variable option*/
	FRigVMVariableMappingInfoPtr	Item;

	// Curve combo box options
	UE_API FReply OnClearButtonClicked();
	UE_API FText GetFilterText() const;

	UE_API TSharedRef<SWidget> MakeVarOptionComboWidget(TSharedPtr<FString> InItem);
	UE_API void OnVarOptionSourceChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	UE_API FText GetVarOptionComboBoxContent() const;
	UE_API FText GetVarOptionComboBoxToolTip() const;
	UE_API void OnVarOptionComboOpening();
	UE_API bool IsVarOptionEnabled() const;

	UE_API ECheckBoxState IsPinChecked() const;
	UE_API void OnPinCheckStatusChanged(ECheckBoxState NewState);
	UE_API bool IsPinEnabled() const;

	FOnRigVMVariableMappingChanged	OnVariableMappingChanged;
	FOnRigVMGetVariableMapping		OnGetVariableMapping;
	FOnRigVMGetAvailableMapping		OnGetAvailableMapping;
	FOnRigVMGetFilteredText			OnGetFilteredText;
	FOnRigVMVarOptionAvailable		OnVariableOptionAvailable;
	FOnRigVMPinCheckStateChanged		OnPinCheckStateChanged;
	FOnRigVMPinGetCheckState			OnPinGetCheckState;
	FOnRigVMPinIsCheckEnabled		OnPinIsEnabledCheckState;
};

typedef STreeView< TSharedPtr<FRigVMVariableMappingInfo> > SRigVMVariableMappingTreeView;

class SRigVMVariableMappingWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRigVMVariableMappingWidget)
	{}

	SLATE_EVENT(FOnRigVMVariableMappingChanged, OnVariableMappingChanged)
	SLATE_EVENT(FOnRigVMGetVariableMapping, OnGetVariableMapping)
	SLATE_EVENT(FOnRigVMGetAvailableMapping, OnGetAvailableMapping)
	SLATE_EVENT(FOnRigVMCreateVariableMapping, OnCreateVariableMapping)
	SLATE_EVENT(FOnRigVMGetFilteredText, OnGetFilteredText)
	SLATE_EVENT(FOnRigVMVarOptionAvailable, OnVariableOptionAvailable);
	SLATE_EVENT(FOnRigVMPinCheckStateChanged, OnPinCheckStateChanged)
	SLATE_EVENT(FOnRigVMPinGetCheckState, OnPinGetCheckState)
	SLATE_EVENT(FOnRigVMPinIsCheckEnabled, OnPinIsEnabledCheckState)
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	UE_API void Construct(const FArguments& InArgs/*, FSimpleMulticastDelegate& InOnPostUndo*/);

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	UE_API void OnFilterTextChanged(const FText& SearchText);

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	UE_API void OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	UE_API TSharedRef<ITableRow> GenerateVariableMappingRow(TSharedPtr<FRigVMVariableMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Handler for the delete of retarget source
	*/
	UE_API void RefreshVariableMappingList();

private:

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of retarget sources */
	TSharedPtr<SRigVMVariableMappingTreeView> VariableMappingTreeView;

	/** A list of variable mapping list. Used by the VariableMappingListView. */
	TArray< TSharedPtr<FRigVMVariableMappingInfo> > VariableMappingList;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Delegate for undo/redo transaction **/
	UE_API void PostUndo();

	// Make a single tree row widget
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(TSharedPtr<FRigVMVariableMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children for the provided bone info
	UE_API void GetChildrenForInfo(TSharedPtr<FRigVMVariableMappingInfo> InInfo, TArray< TSharedPtr<FRigVMVariableMappingInfo> >& OutChildren);

	FOnRigVMGetAvailableMapping		OnGetAvailableMappingDelegate;
	FOnRigVMVariableMappingChanged	OnVariableMappingChangedDelegate;
	FOnRigVMGetVariableMapping		OnGetVariableMappingDelegate;
	FOnRigVMCreateVariableMapping	OnCreateVariableMappingDelegate;
	FOnRigVMVarOptionAvailable		OnVariableOptionAvailableDelegate;
	FOnRigVMPinCheckStateChanged		OnPinCheckStateChangedDelegate;
	FOnRigVMPinGetCheckState			OnPinGetCheckStateDelegate;
	FOnRigVMPinIsCheckEnabled		OnPinIsEnabledCheckStateDelegate;

};

#undef UE_API
