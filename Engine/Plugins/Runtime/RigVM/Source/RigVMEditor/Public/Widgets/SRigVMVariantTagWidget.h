// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Editor/RigVMEditorStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "RigVMCore/RigVMVariant.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_RetVal(TArray<FRigVMTag>, FRigVMVariant_OnGetTags);
DECLARE_DELEGATE_OneParam(FRigVMVariant_OnAddTag, const FName&);
DECLARE_DELEGATE_OneParam(FRigVMVariant_OnRemoveTag, const FName&);

class SRigVMVariantCapsule : public SButton
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMVariantCapsule)
		: _Padding(0.f)
		, _EnableContextMenu(false)
		, _MinDesiredLabelWidth(0.f)
		, _CapsuleTagBorder(FRigVMEditorStyle::Get().GetBrush("RigVM.TagCapsule"))
	{
	}
	SLATE_ATTRIBUTE(FName, Name)
	SLATE_ATTRIBUTE(FText, Label)
	SLATE_ATTRIBUTE(FText, ToolTipText)
	SLATE_ATTRIBUTE(FSlateColor, Color)
	SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_ATTRIBUTE(FMargin, Padding)
	SLATE_EVENT(FRigVMVariant_OnRemoveTag, OnRemoveTag)
	SLATE_ATTRIBUTE(bool, EnableContextMenu)
	SLATE_ARGUMENT(float, MinDesiredLabelWidth)
	SLATE_ATTRIBUTE(const FSlateBrush*, CapsuleTagBorder)
	SLATE_END_ARGS()

	SRigVMVariantCapsule();
	virtual ~SRigVMVariantCapsule() override;

	void Construct(const FArguments& InArgs);

	FSlateColor GetColor() const;
	FSlateColor GetLabelColor() const;

	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:

	void HandleRemoveTag();

	TAttribute<FName> NameAttribute;
	TAttribute<FSlateColor> ColorAttribute;
	FRigVMVariant_OnRemoveTag OnRemoveTag;
	TAttribute<bool> EnableContextMenu;
};

class SRigVMVariantTagWidget : public SBox
{
public:

	SLATE_BEGIN_ARGS(SRigVMVariantTagWidget)
		: _Orientation(EOrientation::Orient_Vertical)
		, _CanAddTags(false)
		, _EnableContextMenu(false)
		, _MinDesiredLabelWidth(0.f)
		, _EnableTick(true)
		, _CapsuleTagBorder(FRigVMEditorStyle::Get().GetBrush("RigVM.TagCapsule"))
	{
	}
	SLATE_ARGUMENT(EOrientation, Orientation)
	SLATE_ATTRIBUTE(bool, CanAddTags)
	SLATE_ATTRIBUTE(bool, EnableContextMenu)
	SLATE_EVENT(FRigVMVariant_OnGetTags, OnGetTags)
	SLATE_EVENT(FRigVMVariant_OnAddTag, OnAddTag)
	SLATE_EVENT(FRigVMVariant_OnRemoveTag, OnRemoveTag)
	SLATE_ARGUMENT(float, MinDesiredLabelWidth)
	SLATE_ARGUMENT(bool, EnableTick)
	SLATE_ATTRIBUTE(const FSlateBrush*, CapsuleTagBorder)
	SLATE_END_ARGS()

	UE_API SRigVMVariantTagWidget();
	UE_API virtual ~SRigVMVariantTagWidget() override;
	
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	UE_API void UpdateCapsules();
	UE_API FReply OnAddTagClicked(const FName& InTagName) const;
	UE_API TSharedRef<SWidget> OnBuildAddTagMenuContent() const;

	TSharedPtr<SVerticalBox> VerticalCapsuleBox;
	TSharedPtr<SHorizontalBox> HorizontalCapsuleBox;
	mutable TWeakPtr<SWidget> WeakAddTagMenuWidget;

	FRigVMVariant_OnGetTags OnGetTags;
	FRigVMVariant_OnAddTag OnAddTag;
	FRigVMVariant_OnRemoveTag OnRemoveTag;
	TAttribute<bool> CanAddTags;
	TAttribute<bool> EnableContextMenu;
	TAttribute<const FSlateBrush*> CapsuleTagBorder;
	uint32 LastTagHash;
	float MinDesiredLabelWidth;
};

#undef UE_API
