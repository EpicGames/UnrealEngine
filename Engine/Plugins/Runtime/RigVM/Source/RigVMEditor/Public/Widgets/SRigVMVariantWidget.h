// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SToolTip.h"
#include "RigVMCore/RigVMVariant.h"
#include "Widgets/SRigVMVariantTagWidget.h"

#define UE_API RIGVMEDITOR_API

DECLARE_DELEGATE_OneParam(FRigVMVariantWidget_OnVariantChanged, const FRigVMVariant&);
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FRigVMVariantWidget_OnCreateVariantRefRow, const FRigVMVariantRef&);
DECLARE_DELEGATE_OneParam(FRigVMVariantWidget_OnBrowseVariantRef, const FRigVMVariantRef&);
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FRigVMVariantWidget_OnVariantRefContextMenu, const FRigVMVariantRef&);
DECLARE_DELEGATE_RetVal(TSharedPtr<SWidget>, FRigVMVariantWidget_OnContextMenu);

struct FRigVMVariantWidgetContext
{
	FRigVMVariantWidgetContext()
		: ParentPath()
	{
	}
	
	// the path the current context is in
	FString ParentPath;
};

class SRigVMVariantToolTipWithTags : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SRigVMVariantToolTipWithTags) {}
		SLATE_ATTRIBUTE(FText, ToolTipText)
		SLATE_EVENT(FRigVMVariant_OnGetTags, OnGetTags)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// IToolTip interface
	virtual bool IsEmpty() const override;
	virtual void OnOpening() override;
	virtual void OnClosed() override;

private:
	FRigVMVariant_OnGetTags GetTagsDelegate;
	SToolTip::FArguments SuperClassArgs;
};

class SRigVMVariantGuidWidget : public SBox
{
public:
	SLATE_BEGIN_ARGS(SRigVMVariantGuidWidget) {}
		SLATE_ATTRIBUTE(FGuid, Guid)
		SLATE_EVENT(FRigVMVariantWidget_OnContextMenu, OnContextMenu)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

private:
	
	FRigVMVariantWidget_OnContextMenu OnContextMenu;
};

class SRigVMVariantWidget : public SBox
{
public:
	
	SLATE_BEGIN_ARGS(SRigVMVariantWidget)
		: _Context(FRigVMVariantWidgetContext())
		, _MaxVariantRefListHeight(200.f)
		, _CanAddTags(false)
		, _EnableTagContextMenu(false)
	{
	}
	SLATE_ATTRIBUTE(FRigVMVariant, Variant)
	SLATE_ATTRIBUTE(FRigVMVariantRef, SubjectVariantRef)
	SLATE_ATTRIBUTE(TArray<FRigVMVariantRef>, VariantRefs)
	SLATE_ATTRIBUTE(FRigVMVariantWidgetContext, Context)
	SLATE_EVENT(FRigVMVariantWidget_OnVariantChanged, OnVariantChanged)
	SLATE_EVENT(FRigVMVariantWidget_OnCreateVariantRefRow, OnCreateVariantRefRow);
	SLATE_EVENT(FRigVMVariantWidget_OnBrowseVariantRef, OnBrowseVariantRef)
	SLATE_EVENT(FRigVMVariantWidget_OnVariantRefContextMenu, OnVariantRefContextMenu)
	SLATE_ATTRIBUTE(float, MaxVariantRefListHeight)
	SLATE_EVENT(FRigVMVariant_OnGetTags, OnGetTags)
	SLATE_EVENT(FRigVMVariant_OnAddTag, OnAddTag)
	SLATE_EVENT(FRigVMVariant_OnRemoveTag, OnRemoveTag)
	SLATE_ATTRIBUTE(bool, CanAddTags)
	SLATE_ATTRIBUTE(bool, EnableTagContextMenu)
	SLATE_END_ARGS()

	UE_API SRigVMVariantWidget();
	UE_API virtual ~SRigVMVariantWidget() override;
	
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	struct FVariantTreeRowInfo
	{
		FRigVMVariantRef VariantRef;
		TArray<TSharedPtr<FVariantTreeRowInfo>> NestedInfos;
		TWeakPtr<ITableRow> RowWidget;
	};

	class SRigVMVariantRefTreeRow
	: public STableRow<TSharedPtr<FVariantTreeRowInfo>>
	{
	public:
	
		SLATE_BEGIN_ARGS(SRigVMVariantRefTreeRow)
		{}
		SLATE_ARGUMENT(TSharedPtr<SWidget>, Content)
		SLATE_EVENT(FRigVMVariantWidget_OnVariantRefContextMenu, OnVariantRefContextMenu)
		SLATE_END_ARGS()

		UE_API virtual ~SRigVMVariantRefTreeRow() override;
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);
	};

	UE_API const FRigVMVariantWidgetContext& GetVariantContext() const;
	UE_API EVisibility GetVariantRefListVisibility() const;
	UE_API TSharedRef<ITableRow> GenerateVariantTreeRow(TSharedPtr<FVariantTreeRowInfo> InRowInfo, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API void GetChildrenForVariantInfo(TSharedPtr<FVariantTreeRowInfo> InInfo, TArray<TSharedPtr<FVariantTreeRowInfo>>& OutChildren);
	UE_API TSharedPtr<SWidget> CreateDefaultVariantRefRow(const FRigVMVariantRef& InVariantRef) const;
	UE_API void RebuildVariantRefList();
	UE_API const FSlateBrush* GetThumbnailBorder(TSharedRef<SBorder> InThumbnailBorder) const;
	UE_API TSharedPtr<SWidget> OnVariantRefTreeContextMenu();
	UE_API TSharedPtr<SWidget> CreateDefaultVariantRefContextMenu(const FRigVMVariantRef& InVariantRef);

	TAttribute<FRigVMVariant> VariantAttribute;
	TAttribute<FRigVMVariantRef> SubjectVariantRefAttribute;
	FRigVMVariantWidget_OnVariantChanged OnVariantChanged;

	TSharedPtr<SRigVMVariantTagWidget> TagWidget;

	TAttribute<TArray<FRigVMVariantRef>> VariantRefsAttribute;
	FRigVMVariantWidget_OnCreateVariantRefRow OnCreateVariantRefRow;
	FRigVMVariantWidget_OnBrowseVariantRef OnBrowseVariantRef;
	FRigVMVariantWidget_OnVariantRefContextMenu OnVariantRefContextMenu;
	TArray<FRigVMVariantRef> VariantRefs;
	TArray<TSharedPtr<FVariantTreeRowInfo>> VariantTreeRowInfos;
	uint32 VariantRefHash;
	TSharedPtr<SVerticalBox> VariantRefListBox;
	TSharedPtr<STreeView<TSharedPtr<FVariantTreeRowInfo>>> VariantRefTreeView;
	TAttribute<FRigVMVariantWidgetContext> ContextAttribute;
};

#undef UE_API
