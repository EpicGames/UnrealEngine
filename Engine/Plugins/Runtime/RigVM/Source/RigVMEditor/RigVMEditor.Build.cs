// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RigVMEditor : ModuleRules
{
    public RigVMEditor(ReadOnlyTargetRules Target) : base(Target)
    {
	    CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
                "RigVM",
                "RigVMDeveloper",
                "SlateCore",
                "BlueprintGraph",
                "GraphEditor",
                "UnrealEd",
                "Slate",
                "SlateCore",
                "RenderCore",
                "RHI",
                "UMG",
                "InputCore",
                "ToolWidgets",
                "KismetCompiler",
                "KismetWidgets",
                "AnimationWidgets",
                "ApplicationCore",
                "AppFramework",
                "AnimationCore",
                "PropertyEditor",
                "ToolMenus",
                "MessageLog",
                "StructUtilsEditor",
                "ContentBrowser",
                "ContentBrowserData",
                "EditorFramework",
                "AnimationEditorWidgets",
                "WidgetRegistration",
                "TraceAnalysis",
                "TraceLog",
                "TraceServices",
                "RewindDebuggerInterface",
                "GameplayInsights",
				"Scribble",
				"ScribbleEditor",
				"PythonScriptPlugin"
            }
		);
        
        bool bAddBlueprintEditorDependency = true;
        PublicDefinitions.Add("WITH_RIGVMLEGACYEDITOR=" + (bAddBlueprintEditorDependency ? '1' : '0'));
        if (bAddBlueprintEditorDependency)
        {
			PublicDependencyModuleNames.Add("Kismet");
		}
        else
        {
	        PrivateDependencyModuleNames.AddRange(new string[]
	        {
		        "ApplicationCore",
		        "FieldNotification",
		        "InputCore",
		        "Json",
		        "JsonUtilities",
		        "ToolWidgets",
		        "TraceLog",
		        "WorkspaceMenuStructure"
	        });
        }
	}
}
